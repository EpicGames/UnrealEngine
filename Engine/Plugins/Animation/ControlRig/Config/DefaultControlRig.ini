[CoreRedirects]
+ClassRedirects=(OldName="/Script/ControlRigEditor.ControlRigBlueprint", NewName="/Script/ControlRigDeveloper.ControlRigBlueprint")
+ClassRedirects=(OldName="/Script/ControlRigEditor.ControlRigGraph", NewName="/Script/ControlRigDeveloper.ControlRigGraph")
+ClassRedirects=(OldName="/Script/ControlRigEditor.ControlRigGraphNode", NewName="/Script/ControlRigDeveloper.ControlRigGraphNode")
+ClassRedirects=(OldName="/Script/ControlRigEditor.ControlRigGraphSchema", NewName="/Script/ControlRigDeveloper.ControlRigGraphSchema")
+ClassRedirects=(OldName="/Script/ControlRigEditor.AnimGraphNode_ControlRig", NewName="/Script/ControlRigDeveloper.AnimGraphNode_ControlRig")

+StructRedirects=(OldName="RigJoint",NewName="/Script/ControlRig.RigBone")
+PropertyRedirects=(OldName="RigBoneHierarchy.Joints",NewName="RigBoneHierarchy.Bones")
+PropertyRedirects=(OldName="RigHierarchyContainer.BaseHierarchy",NewName="RigHierarchyContainer.BoneHierarchy")
+PropertyRedirects=(OldName="RigUnit_SetTransform.Transform", NewName="RigUnit_SetTransform.Value")
+PropertyRedirects=(OldName="RigUnit_SetTranslation.Translation", NewName="RigUnit_SetTranslation.Value")
+PropertyRedirects=(OldName="RigUnit_SetRotation.Rotation", NewName="RigUnit_SetRotation.Value")
+PropertyRedirects=(OldName="RigUnit_ToWorldSpace_Transform.Transform", NewName="RigUnit_ToWorldSpace_Transform.Value")
+PropertyRedirects=(OldName="RigUnit_ToWorldSpace_Transform.Transform", NewName="RigUnit_ToWorldSpace_Transform.Value")
+PropertyRedirects=(OldName="RigUnit_ToWorldSpace_Location.Location", NewName="RigUnit_ToWorldSpace_Location.Value")
+PropertyRedirects=(OldName="RigUnit_ToWorldSpace_Rotation.Rotation", NewName="RigUnit_ToWorldSpace_Rotation.Value")
+PropertyRedirects=(OldName="RigUnit_ToRigSpace_Transform.Transform", NewName="RigUnit_ToRigSpace_Transform.Value")
+PropertyRedirects=(OldName="RigUnit_ToRigSpace_Location.Location", NewName="RigUnit_ToRigSpace_Location.Value")
+PropertyRedirects=(OldName="RigUnit_ToRigSpace_Rotation.Rotation", NewName="RigUnit_ToRigSpace_Rotation.Value")
+PropertyRedirects=(OldName="RigVMFunction_MathQuaternionRotateVector.Quaternion", NewName="RigVMFunction_MathQuaternionRotateVector.Transform")
+PropertyRedirects=(OldName="RigVMFunction_MathTransformRotateVector.Direction", NewName="RigVMFunction_MathTransformRotateVector.Vector")
+PropertyRedirects=(OldName="RigUnit_SetRelativeTransformForItem.RelativeTransform", NewName="RigUnit_SetRelativeTransformForItem.Value")

+ClassRedirects=(OldName="RigVMGraph",NewName="/Script/RigVMDeveloper.RigVMGraph",ValueChanges=(("Model","RigVMModel")))

+EnumRedirects=(OldName="ERigElementType",ValueChanges=(("Space","Null")))
+EnumRedirects=(OldName="/Script/ControlRig.EControlRigRotationOrder",NewName="/Script/AnimationCore.EEulerRotationOrder")
+EnumRedirects=(OldName="EControlRigRotationOrder",NewName="/Script/AnimationCore.EEulerRotationOrder")


; renaming gizmo to shape
+ClassRedirects=(OldName="/Script/ControlRig.ControlRigGizmoLibrary", NewName="/Script/ControlRig.ControlRigShapeLibrary")
+ClassRedirects=(OldName="/Script/ControlRig.ControlRigGizmoActor", NewName="/Script/ControlRig.ControlRigShapeActor")
+StructRedirects=(OldName="ControlRigGizmoDefinition", NewName="/Script/ControlRig.ControlRigShapeDefinition")
+StructRedirects=(OldName="GizmoActorCreationParam", NewName="/Script/ControlRig.ControlShapeActorCreationParam")
+PropertyRedirects=(OldName="/Script/ControlRig.ControlRigShapeLibrary.DefaultGizmo",NewName="DefaultShape")
+PropertyRedirects=(OldName="/Script/ControlRig.ControlRigShapeLibrary.Gizmos",NewName="Shapes")
+PropertyRedirects=(OldName="/Script/ControlRig.ControlRigShapeDefinition.GizmoName",NewName="ShapeName")
+PropertyRedirects=(OldName="/Script/ControlRig.RigControlSettings.bGizmoEnabled",NewName="bShapeEnabled")
+PropertyRedirects=(OldName="/Script/ControlRig.RigControlSettings.bGizmoVisible",NewName="bShapeVisible")
+PropertyRedirects=(OldName="/Script/ControlRig.RigControlSettings.GizmoName",NewName="ShapeName")
+PropertyRedirects=(OldName="/Script/ControlRig.RigControlSettings.GizmoColor",NewName="ShapeColor")
+PropertyRedirects=(OldName="/Script/ControlRig.RigControlElement.Gizmo",NewName="Shape")
+PropertyRedirects=(OldName="/Script/ControlRigEditor.ControlRigEditModeSettings.GizmoScale",NewName="ShapeScale")

; renaming setup event to construction event
+PropertyRedirects=(OldName="/Script/ControlRig.ControlRigComponent.OnPreSetupDelegate",NewName="OnPreConstructionDelegate")
+PropertyRedirects=(OldName="/Script/ControlRig.ControlRigComponent.OnPostSetupDelegate",NewName="OnPostConstructionDelegate")
+PropertyRedirects=(OldName="/Script/ControlRig.UControlRigSettings.SetupEventBorderColor",NewName="ConstructionEventBorderColor")

; moving core units to RigVM runtime
;+StructRedirects=(OldName="/Script/ControlRig.RigUnit_...",NewName="/Script/RigVM.RigVMFunction_", MatchWildcard=true)
;+StructRedirects=(OldName="/Script/ControlRig.RigDispatch_...",NewName="/Script/RigVM.RigVMDispatch_", MatchWildcard=true)
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NameBase",NewName="/Script/RigVM.RigVMFunction_NameBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NameConcat",NewName="/Script/RigVM.RigVMFunction_NameConcat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NameTruncate",NewName="/Script/RigVM.RigVMFunction_NameTruncate")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NameReplace",NewName="/Script/RigVM.RigVMFunction_NameReplace")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_EndsWith",NewName="/Script/RigVM.RigVMFunction_EndsWith")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StartsWith",NewName="/Script/RigVM.RigVMFunction_StartsWith")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_Contains",NewName="/Script/RigVM.RigVMFunction_Contains")
+StructRedirects=(OldName="/Script/ControlRig.RigDispatch_CoreBase",NewName="/Script/RigVM.RigVMDispatch_CoreBase")
+StructRedirects=(OldName="/Script/ControlRig.RigDispatch_CoreEquals",NewName="/Script/RigVM.RigVMDispatch_CoreEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigDispatch_CoreNotEquals",NewName="/Script/RigVM.RigVMDispatch_CoreNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigDispatch_Print",NewName="/Script/RigVM.RigVMDispatch_Print")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringBase",NewName="/Script/RigVM.RigVMFunction_StringBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringConcat",NewName="/Script/RigVM.RigVMFunction_StringConcat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringTruncate",NewName="/Script/RigVM.RigVMFunction_StringTruncate")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringReplace",NewName="/Script/RigVM.RigVMFunction_StringReplace")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringEndsWith",NewName="/Script/RigVM.RigVMFunction_StringEndsWith")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringStartsWith",NewName="/Script/RigVM.RigVMFunction_StringStartsWith")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringContains",NewName="/Script/RigVM.RigVMFunction_StringContains")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringLength",NewName="/Script/RigVM.RigVMFunction_StringLength")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringTrimWhitespace",NewName="/Script/RigVM.RigVMFunction_StringTrimWhitespace")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringToUppercase",NewName="/Script/RigVM.RigVMFunction_StringToUppercase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringToLowercase",NewName="/Script/RigVM.RigVMFunction_StringToLowercase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringReverse",NewName="/Script/RigVM.RigVMFunction_StringReverse")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringLeft",NewName="/Script/RigVM.RigVMFunction_StringLeft")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringRight",NewName="/Script/RigVM.RigVMFunction_StringRight")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringMiddle",NewName="/Script/RigVM.RigVMFunction_StringMiddle")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringFind",NewName="/Script/RigVM.RigVMFunction_StringFind")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringSplit",NewName="/Script/RigVM.RigVMFunction_StringSplit")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringJoin",NewName="/Script/RigVM.RigVMFunction_StringJoin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_StringPadInteger",NewName="/Script/RigVM.RigVMFunction_StringPadInteger")
+StructRedirects=(OldName="/Script/ControlRig.ControlRigDrawInstruction",NewName="/Script/RigVM.RigVMDrawInstruction")
+StructRedirects=(OldName="/Script/ControlRig.ControlRigDrawContainer",NewName="/Script/RigVM.RigVMDrawContainer")
+StructRedirects=(OldName="/Script/ControlRig.ControlRigDrawInterface",NewName="/Script/RigVM.RigVMDrawInterface")

; moving math units to RigVM runtime
+EnumRedirects=(OldName="EBoneGetterSetterMode",NewName="/Script/RigVM.ERigVMTransformSpace")
+EnumRedirects=(OldName="EControlRigAnimEasingType",NewName="/Script/RigVM.ERigVMAnimEasingType")
+EnumRedirects=(OldName="EControlRigClampSpatialMode",NewName="/Script/RigVM.ERigVMClampSpatialMode")
+EnumRedirects=(OldName="/Script/ControlRig.ERigUnitDebugTransformMode",NewName="/Script/RigVM.ERigUnitDebugTransformMode")
+StructRedirects=(OldName="/Script/ControlRig.CRFourPointBezier",NewName="/Script/RigVM.RigVMFourPointBezier")
+StructRedirects=(OldName="/Script/ControlRig.RigMirrorSettings",NewName="/Script/RigVM.RigVMMirrorSettings")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBase",NewName="/Script/RigVM.RigVMFunction_MathBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMutableBase",NewName="/Script/RigVM.RigVMFunction_MathMutableBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolBase",NewName="/Script/RigVM.RigVMFunction_MathBoolBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolConstant",NewName="/Script/RigVM.RigVMFunction_MathBoolConstant")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathBoolUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathBoolBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathBoolBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolMake",NewName="/Script/RigVM.RigVMFunction_MathBoolMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolConstTrue",NewName="/Script/RigVM.RigVMFunction_MathBoolConstTrue")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolConstFalse",NewName="/Script/RigVM.RigVMFunction_MathBoolConstFalse")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolNot",NewName="/Script/RigVM.RigVMFunction_MathBoolNot")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolAnd",NewName="/Script/RigVM.RigVMFunction_MathBoolAnd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolNand",NewName="/Script/RigVM.RigVMFunction_MathBoolNand")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolNand2",NewName="/Script/RigVM.RigVMFunction_MathBoolNand2")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolOr",NewName="/Script/RigVM.RigVMFunction_MathBoolOr")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolEquals",NewName="/Script/RigVM.RigVMFunction_MathBoolEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolNotEquals",NewName="/Script/RigVM.RigVMFunction_MathBoolNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolToggled",NewName="/Script/RigVM.RigVMFunction_MathBoolToggled")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolFlipFlop",NewName="/Script/RigVM.RigVMFunction_MathBoolFlipFlop")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolOnce",NewName="/Script/RigVM.RigVMFunction_MathBoolOnce")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolToFloat",NewName="/Script/RigVM.RigVMFunction_MathBoolToFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathBoolToInteger",NewName="/Script/RigVM.RigVMFunction_MathBoolToInteger")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorBase",NewName="/Script/RigVM.RigVMFunction_MathColorBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathColorBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathColorBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorMake",NewName="/Script/RigVM.RigVMFunction_MathColorMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorFromFloat",NewName="/Script/RigVM.RigVMFunction_MathColorFromFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorFromDouble",NewName="/Script/RigVM.RigVMFunction_MathColorFromDouble")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorAdd",NewName="/Script/RigVM.RigVMFunction_MathColorAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorSub",NewName="/Script/RigVM.RigVMFunction_MathColorSub")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorMul",NewName="/Script/RigVM.RigVMFunction_MathColorMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathColorLerp",NewName="/Script/RigVM.RigVMFunction_MathColorLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleBase",NewName="/Script/RigVM.RigVMFunction_MathDoubleBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleConstant",NewName="/Script/RigVM.RigVMFunction_MathDoubleConstant")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathDoubleUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathDoubleBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathDoubleBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleMake",NewName="/Script/RigVM.RigVMFunction_MathDoubleMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleConstPi",NewName="/Script/RigVM.RigVMFunction_MathDoubleConstPi")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleConstHalfPi",NewName="/Script/RigVM.RigVMFunction_MathDoubleConstHalfPi")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleConstTwoPi",NewName="/Script/RigVM.RigVMFunction_MathDoubleConstTwoPi")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleConstE",NewName="/Script/RigVM.RigVMFunction_MathDoubleConstE")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleAdd",NewName="/Script/RigVM.RigVMFunction_MathDoubleAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleSub",NewName="/Script/RigVM.RigVMFunction_MathDoubleSub")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleMul",NewName="/Script/RigVM.RigVMFunction_MathDoubleMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleDiv",NewName="/Script/RigVM.RigVMFunction_MathDoubleDiv")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleMod",NewName="/Script/RigVM.RigVMFunction_MathDoubleMod")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleMin",NewName="/Script/RigVM.RigVMFunction_MathDoubleMin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleMax",NewName="/Script/RigVM.RigVMFunction_MathDoubleMax")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoublePow",NewName="/Script/RigVM.RigVMFunction_MathDoublePow")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleSqrt",NewName="/Script/RigVM.RigVMFunction_MathDoubleSqrt")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleNegate",NewName="/Script/RigVM.RigVMFunction_MathDoubleNegate")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleAbs",NewName="/Script/RigVM.RigVMFunction_MathDoubleAbs")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleFloor",NewName="/Script/RigVM.RigVMFunction_MathDoubleFloor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleCeil",NewName="/Script/RigVM.RigVMFunction_MathDoubleCeil")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleRound",NewName="/Script/RigVM.RigVMFunction_MathDoubleRound")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleToInt",NewName="/Script/RigVM.RigVMFunction_MathDoubleToInt")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleSign",NewName="/Script/RigVM.RigVMFunction_MathDoubleSign")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleClamp",NewName="/Script/RigVM.RigVMFunction_MathDoubleClamp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleLerp",NewName="/Script/RigVM.RigVMFunction_MathDoubleLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleRemap",NewName="/Script/RigVM.RigVMFunction_MathDoubleRemap")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleEquals",NewName="/Script/RigVM.RigVMFunction_MathDoubleEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleNotEquals",NewName="/Script/RigVM.RigVMFunction_MathDoubleNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleGreater",NewName="/Script/RigVM.RigVMFunction_MathDoubleGreater")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleLess",NewName="/Script/RigVM.RigVMFunction_MathDoubleLess")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleGreaterEqual",NewName="/Script/RigVM.RigVMFunction_MathDoubleGreaterEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleLessEqual",NewName="/Script/RigVM.RigVMFunction_MathDoubleLessEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleIsNearlyZero",NewName="/Script/RigVM.RigVMFunction_MathDoubleIsNearlyZero")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleIsNearlyEqual",NewName="/Script/RigVM.RigVMFunction_MathDoubleIsNearlyEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleDeg",NewName="/Script/RigVM.RigVMFunction_MathDoubleDeg")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleRad",NewName="/Script/RigVM.RigVMFunction_MathDoubleRad")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleSin",NewName="/Script/RigVM.RigVMFunction_MathDoubleSin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleCos",NewName="/Script/RigVM.RigVMFunction_MathDoubleCos")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleTan",NewName="/Script/RigVM.RigVMFunction_MathDoubleTan")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleAsin",NewName="/Script/RigVM.RigVMFunction_MathDoubleAsin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleAcos",NewName="/Script/RigVM.RigVMFunction_MathDoubleAcos")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleAtan",NewName="/Script/RigVM.RigVMFunction_MathDoubleAtan")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleLawOfCosine",NewName="/Script/RigVM.RigVMFunction_MathDoubleLawOfCosine")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDoubleExponential",NewName="/Script/RigVM.RigVMFunction_MathDoubleExponential")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatBase",NewName="/Script/RigVM.RigVMFunction_MathFloatBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatConstant",NewName="/Script/RigVM.RigVMFunction_MathFloatConstant")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathFloatUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathFloatBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathFloatBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatMake",NewName="/Script/RigVM.RigVMFunction_MathFloatMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatConstPi",NewName="/Script/RigVM.RigVMFunction_MathFloatConstPi")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatConstHalfPi",NewName="/Script/RigVM.RigVMFunction_MathFloatConstHalfPi")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatConstTwoPi",NewName="/Script/RigVM.RigVMFunction_MathFloatConstTwoPi")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatConstE",NewName="/Script/RigVM.RigVMFunction_MathFloatConstE")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatAdd",NewName="/Script/RigVM.RigVMFunction_MathFloatAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatSub",NewName="/Script/RigVM.RigVMFunction_MathFloatSub")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatMul",NewName="/Script/RigVM.RigVMFunction_MathFloatMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatDiv",NewName="/Script/RigVM.RigVMFunction_MathFloatDiv")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatMod",NewName="/Script/RigVM.RigVMFunction_MathFloatMod")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatMin",NewName="/Script/RigVM.RigVMFunction_MathFloatMin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatMax",NewName="/Script/RigVM.RigVMFunction_MathFloatMax")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatPow",NewName="/Script/RigVM.RigVMFunction_MathFloatPow")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatSqrt",NewName="/Script/RigVM.RigVMFunction_MathFloatSqrt")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatNegate",NewName="/Script/RigVM.RigVMFunction_MathFloatNegate")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatAbs",NewName="/Script/RigVM.RigVMFunction_MathFloatAbs")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatFloor",NewName="/Script/RigVM.RigVMFunction_MathFloatFloor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatCeil",NewName="/Script/RigVM.RigVMFunction_MathFloatCeil")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatRound",NewName="/Script/RigVM.RigVMFunction_MathFloatRound")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatToInt",NewName="/Script/RigVM.RigVMFunction_MathFloatToInt")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatSign",NewName="/Script/RigVM.RigVMFunction_MathFloatSign")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatClamp",NewName="/Script/RigVM.RigVMFunction_MathFloatClamp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatLerp",NewName="/Script/RigVM.RigVMFunction_MathFloatLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatRemap",NewName="/Script/RigVM.RigVMFunction_MathFloatRemap")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatEquals",NewName="/Script/RigVM.RigVMFunction_MathFloatEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatNotEquals",NewName="/Script/RigVM.RigVMFunction_MathFloatNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatGreater",NewName="/Script/RigVM.RigVMFunction_MathFloatGreater")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatLess",NewName="/Script/RigVM.RigVMFunction_MathFloatLess")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatGreaterEqual",NewName="/Script/RigVM.RigVMFunction_MathFloatGreaterEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatLessEqual",NewName="/Script/RigVM.RigVMFunction_MathFloatLessEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatIsNearlyZero",NewName="/Script/RigVM.RigVMFunction_MathFloatIsNearlyZero")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatIsNearlyEqual",NewName="/Script/RigVM.RigVMFunction_MathFloatIsNearlyEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatSelectBool",NewName="/Script/RigVM.RigVMFunction_MathFloatSelectBool")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatDeg",NewName="/Script/RigVM.RigVMFunction_MathFloatDeg")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatRad",NewName="/Script/RigVM.RigVMFunction_MathFloatRad")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatSin",NewName="/Script/RigVM.RigVMFunction_MathFloatSin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatCos",NewName="/Script/RigVM.RigVMFunction_MathFloatCos")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatTan",NewName="/Script/RigVM.RigVMFunction_MathFloatTan")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatAsin",NewName="/Script/RigVM.RigVMFunction_MathFloatAsin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatAcos",NewName="/Script/RigVM.RigVMFunction_MathFloatAcos")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatAtan",NewName="/Script/RigVM.RigVMFunction_MathFloatAtan")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatLawOfCosine",NewName="/Script/RigVM.RigVMFunction_MathFloatLawOfCosine")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathFloatExponential",NewName="/Script/RigVM.RigVMFunction_MathFloatExponential")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntBase",NewName="/Script/RigVM.RigVMFunction_MathIntBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathIntUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathIntBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathIntBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntMake",NewName="/Script/RigVM.RigVMFunction_MathIntMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntAdd",NewName="/Script/RigVM.RigVMFunction_MathIntAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntSub",NewName="/Script/RigVM.RigVMFunction_MathIntSub")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntMul",NewName="/Script/RigVM.RigVMFunction_MathIntMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntDiv",NewName="/Script/RigVM.RigVMFunction_MathIntDiv")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntMod",NewName="/Script/RigVM.RigVMFunction_MathIntMod")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntMin",NewName="/Script/RigVM.RigVMFunction_MathIntMin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntMax",NewName="/Script/RigVM.RigVMFunction_MathIntMax")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntPow",NewName="/Script/RigVM.RigVMFunction_MathIntPow")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntNegate",NewName="/Script/RigVM.RigVMFunction_MathIntNegate")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntAbs",NewName="/Script/RigVM.RigVMFunction_MathIntAbs")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntToFloat",NewName="/Script/RigVM.RigVMFunction_MathIntToFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntToDouble",NewName="/Script/RigVM.RigVMFunction_MathIntToDouble")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntSign",NewName="/Script/RigVM.RigVMFunction_MathIntSign")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntClamp",NewName="/Script/RigVM.RigVMFunction_MathIntClamp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntEquals",NewName="/Script/RigVM.RigVMFunction_MathIntEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntNotEquals",NewName="/Script/RigVM.RigVMFunction_MathIntNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntGreater",NewName="/Script/RigVM.RigVMFunction_MathIntGreater")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntLess",NewName="/Script/RigVM.RigVMFunction_MathIntLess")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntGreaterEqual",NewName="/Script/RigVM.RigVMFunction_MathIntGreaterEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntLessEqual",NewName="/Script/RigVM.RigVMFunction_MathIntLessEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixBase",NewName="/Script/RigVM.RigVMFunction_MathMatrixBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathMatrixUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathMatrixBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathMatrixBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixToTransform",NewName="/Script/RigVM.RigVMFunction_MathMatrixToTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixFromTransform",NewName="/Script/RigVM.RigVMFunction_MathMatrixFromTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixFromTransformV2",NewName="/Script/RigVM.RigVMFunction_MathMatrixFromTransformV2")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixToVectors",NewName="/Script/RigVM.RigVMFunction_MathMatrixToVectors")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixFromVectors",NewName="/Script/RigVM.RigVMFunction_MathMatrixFromVectors")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixMul",NewName="/Script/RigVM.RigVMFunction_MathMatrixMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathMatrixInverse",NewName="/Script/RigVM.RigVMFunction_MathMatrixInverse")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionBase",NewName="/Script/RigVM.RigVMFunction_MathQuaternionBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathQuaternionUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathQuaternionBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathQuaternionBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionMake",NewName="/Script/RigVM.RigVMFunction_MathQuaternionMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionFromAxisAndAngle",NewName="/Script/RigVM.RigVMFunction_MathQuaternionFromAxisAndAngle")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionFromEuler",NewName="/Script/RigVM.RigVMFunction_MathQuaternionFromEuler")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionFromRotator",NewName="/Script/RigVM.RigVMFunction_MathQuaternionFromRotator")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionFromRotatorV2",NewName="/Script/RigVM.RigVMFunction_MathQuaternionFromRotatorV2")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionFromTwoVectors",NewName="/Script/RigVM.RigVMFunction_MathQuaternionFromTwoVectors")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionToAxisAndAngle",NewName="/Script/RigVM.RigVMFunction_MathQuaternionToAxisAndAngle")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionScale",NewName="/Script/RigVM.RigVMFunction_MathQuaternionScale")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionScaleV2",NewName="/Script/RigVM.RigVMFunction_MathQuaternionScaleV2")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionToEuler",NewName="/Script/RigVM.RigVMFunction_MathQuaternionToEuler")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionToRotator",NewName="/Script/RigVM.RigVMFunction_MathQuaternionToRotator")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionMul",NewName="/Script/RigVM.RigVMFunction_MathQuaternionMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionInverse",NewName="/Script/RigVM.RigVMFunction_MathQuaternionInverse")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionSlerp",NewName="/Script/RigVM.RigVMFunction_MathQuaternionSlerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionEquals",NewName="/Script/RigVM.RigVMFunction_MathQuaternionEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionNotEquals",NewName="/Script/RigVM.RigVMFunction_MathQuaternionNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionSelectBool",NewName="/Script/RigVM.RigVMFunction_MathQuaternionSelectBool")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionDot",NewName="/Script/RigVM.RigVMFunction_MathQuaternionDot")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionUnit",NewName="/Script/RigVM.RigVMFunction_MathQuaternionUnit")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionRotateVector",NewName="/Script/RigVM.RigVMFunction_MathQuaternionRotateVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionGetAxis",NewName="/Script/RigVM.RigVMFunction_MathQuaternionGetAxis")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionSwingTwist",NewName="/Script/RigVM.RigVMFunction_MathQuaternionSwingTwist")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionRotationOrder",NewName="/Script/RigVM.RigVMFunction_MathQuaternionRotationOrder")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionMakeRelative",NewName="/Script/RigVM.RigVMFunction_MathQuaternionMakeRelative")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionMakeAbsolute",NewName="/Script/RigVM.RigVMFunction_MathQuaternionMakeAbsolute")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathQuaternionMirrorTransform",NewName="/Script/RigVM.RigVMFunction_MathQuaternionMirrorTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFQuatWeightFunctor",NewName="/Script/RigVM.RigVMFunction_MathRBFQuatWeightFunctor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFVectorWeightFunctor",NewName="/Script/RigVM.RigVMFunction_MathRBFVectorWeightFunctor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatWorkData",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatWorkData")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorWorkData",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorWorkData")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateBase",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatBase",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorBase",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatFloat",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatVector",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatColor",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatColor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatQuat",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatQuat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateQuatXform",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateQuatXform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorFloat",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorVector",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorColor",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorColor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorQuat",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorQuat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathRBFInterpolateVectorXform",NewName="/Script/RigVM.RigVMFunction_MathRBFInterpolateVectorXform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformBase",NewName="/Script/RigVM.RigVMFunction_MathTransformBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformMutableBase",NewName="/Script/RigVM.RigVMFunction_MathTransformMutableBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathTransformUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathTransformBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathTransformBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformMake",NewName="/Script/RigVM.RigVMFunction_MathTransformMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformFromEulerTransform",NewName="/Script/RigVM.RigVMFunction_MathTransformFromEulerTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformFromEulerTransformV2",NewName="/Script/RigVM.RigVMFunction_MathTransformFromEulerTransformV2")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformToEulerTransform",NewName="/Script/RigVM.RigVMFunction_MathTransformToEulerTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformMul",NewName="/Script/RigVM.RigVMFunction_MathTransformMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformMakeRelative",NewName="/Script/RigVM.RigVMFunction_MathTransformMakeRelative")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformMakeAbsolute",NewName="/Script/RigVM.RigVMFunction_MathTransformMakeAbsolute")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformAccumulateArray",NewName="/Script/RigVM.RigVMFunction_MathTransformAccumulateArray")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformInverse",NewName="/Script/RigVM.RigVMFunction_MathTransformInverse")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformLerp",NewName="/Script/RigVM.RigVMFunction_MathTransformLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformSelectBool",NewName="/Script/RigVM.RigVMFunction_MathTransformSelectBool")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformRotateVector",NewName="/Script/RigVM.RigVMFunction_MathTransformRotateVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformTransformVector",NewName="/Script/RigVM.RigVMFunction_MathTransformTransformVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformFromSRT",NewName="/Script/RigVM.RigVMFunction_MathTransformFromSRT")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformArrayToSRT",NewName="/Script/RigVM.RigVMFunction_MathTransformArrayToSRT")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformClampSpatially",NewName="/Script/RigVM.RigVMFunction_MathTransformClampSpatially")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathTransformMirrorTransform",NewName="/Script/RigVM.RigVMFunction_MathTransformMirrorTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorBase",NewName="/Script/RigVM.RigVMFunction_MathVectorBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorUnaryOp",NewName="/Script/RigVM.RigVMFunction_MathVectorUnaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorBinaryOp",NewName="/Script/RigVM.RigVMFunction_MathVectorBinaryOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorBinaryAggregateOp",NewName="/Script/RigVM.RigVMFunction_MathVectorBinaryAggregateOp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMake",NewName="/Script/RigVM.RigVMFunction_MathVectorMake")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorFromFloat",NewName="/Script/RigVM.RigVMFunction_MathVectorFromFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorFromDouble",NewName="/Script/RigVM.RigVMFunction_MathVectorFromDouble")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorAdd",NewName="/Script/RigVM.RigVMFunction_MathVectorAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorSub",NewName="/Script/RigVM.RigVMFunction_MathVectorSub")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMul",NewName="/Script/RigVM.RigVMFunction_MathVectorMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorScale",NewName="/Script/RigVM.RigVMFunction_MathVectorScale")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorDiv",NewName="/Script/RigVM.RigVMFunction_MathVectorDiv")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMod",NewName="/Script/RigVM.RigVMFunction_MathVectorMod")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMin",NewName="/Script/RigVM.RigVMFunction_MathVectorMin")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMax",NewName="/Script/RigVM.RigVMFunction_MathVectorMax")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorNegate",NewName="/Script/RigVM.RigVMFunction_MathVectorNegate")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorAbs",NewName="/Script/RigVM.RigVMFunction_MathVectorAbs")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorFloor",NewName="/Script/RigVM.RigVMFunction_MathVectorFloor")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorCeil",NewName="/Script/RigVM.RigVMFunction_MathVectorCeil")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorRound",NewName="/Script/RigVM.RigVMFunction_MathVectorRound")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorSign",NewName="/Script/RigVM.RigVMFunction_MathVectorSign")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorClamp",NewName="/Script/RigVM.RigVMFunction_MathVectorClamp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorLerp",NewName="/Script/RigVM.RigVMFunction_MathVectorLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorRemap",NewName="/Script/RigVM.RigVMFunction_MathVectorRemap")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorEquals",NewName="/Script/RigVM.RigVMFunction_MathVectorEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorNotEquals",NewName="/Script/RigVM.RigVMFunction_MathVectorNotEquals")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorIsNearlyZero",NewName="/Script/RigVM.RigVMFunction_MathVectorIsNearlyZero")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorIsNearlyEqual",NewName="/Script/RigVM.RigVMFunction_MathVectorIsNearlyEqual")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorSelectBool",NewName="/Script/RigVM.RigVMFunction_MathVectorSelectBool")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorDeg",NewName="/Script/RigVM.RigVMFunction_MathVectorDeg")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorRad",NewName="/Script/RigVM.RigVMFunction_MathVectorRad")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorLengthSquared",NewName="/Script/RigVM.RigVMFunction_MathVectorLengthSquared")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorLength",NewName="/Script/RigVM.RigVMFunction_MathVectorLength")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorDistance",NewName="/Script/RigVM.RigVMFunction_MathVectorDistance")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorCross",NewName="/Script/RigVM.RigVMFunction_MathVectorCross")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorDot",NewName="/Script/RigVM.RigVMFunction_MathVectorDot")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorUnit",NewName="/Script/RigVM.RigVMFunction_MathVectorUnit")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorSetLength",NewName="/Script/RigVM.RigVMFunction_MathVectorSetLength")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorClampLength",NewName="/Script/RigVM.RigVMFunction_MathVectorClampLength")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMirror",NewName="/Script/RigVM.RigVMFunction_MathVectorMirror")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorAngle",NewName="/Script/RigVM.RigVMFunction_MathVectorAngle")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorParallel",NewName="/Script/RigVM.RigVMFunction_MathVectorParallel")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorOrthogonal",NewName="/Script/RigVM.RigVMFunction_MathVectorOrthogonal")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorBezierFourPoint",NewName="/Script/RigVM.RigVMFunction_MathVectorBezierFourPoint")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMakeBezierFourPoint",NewName="/Script/RigVM.RigVMFunction_MathVectorMakeBezierFourPoint")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorClampSpatially",NewName="/Script/RigVM.RigVMFunction_MathVectorClampSpatially")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathIntersectPlane",NewName="/Script/RigVM.RigVMFunction_MathIntersectPlane")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathDistanceToPlane",NewName="/Script/RigVM.RigVMFunction_MathDistanceToPlane")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMakeRelative",NewName="/Script/RigVM.RigVMFunction_MathVectorMakeRelative")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMakeAbsolute",NewName="/Script/RigVM.RigVMFunction_MathVectorMakeAbsolute")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_MathVectorMirrorTransform",NewName="/Script/RigVM.RigVMFunction_MathVectorMirrorTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NoiseFloat",NewName="/Script/RigVM.RigVMFunction_NoiseFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NoiseDouble",NewName="/Script/RigVM.RigVMFunction_NoiseDouble")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NoiseVector",NewName="/Script/RigVM.RigVMFunction_NoiseVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_NoiseVector2",NewName="/Script/RigVM.RigVMFunction_NoiseVector2")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_RandomFloat",NewName="/Script/RigVM.RigVMFunction_RandomFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_RandomVector",NewName="/Script/RigVM.RigVMFunction_RandomVector")

; moving remaining units to RigVM runtime
+EnumRedirects=(OldName="ECRSimPointIntegrateType",NewName="/Script/RigVM.ERigVMSimPointIntegrateType")
+StructRedirects=(OldName="/Script/ControlRig.CRSimPoint",NewName="/Script/RigVM.RigVMSimPoint")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_ForLoopCount",NewName="/Script/RigVM.RigVMFunction_ForLoopCount")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_SequenceAggregate",NewName="/Script/RigVM.RigVMFunction_Sequence")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_UserDefinedEvent",NewName="/Script/RigVM.RigVMFunction_UserDefinedEvent")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateBase",NewName="/Script/RigVM.RigVMFunction_AccumulateBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateFloatAdd",NewName="/Script/RigVM.RigVMFunction_AccumulateFloatAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateVectorAdd",NewName="/Script/RigVM.RigVMFunction_AccumulateVectorAdd")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateFloatMul",NewName="/Script/RigVM.RigVMFunction_AccumulateFloatMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateVectorMul",NewName="/Script/RigVM.RigVMFunction_AccumulateVectorMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateQuatMul",NewName="/Script/RigVM.RigVMFunction_AccumulateQuatMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateTransformMul",NewName="/Script/RigVM.RigVMFunction_AccumulateTransformMul")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateFloatLerp",NewName="/Script/RigVM.RigVMFunction_AccumulateFloatLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateVectorLerp",NewName="/Script/RigVM.RigVMFunction_AccumulateVectorLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateQuatLerp",NewName="/Script/RigVM.RigVMFunction_AccumulateQuatLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateTransformLerp",NewName="/Script/RigVM.RigVMFunction_AccumulateTransformLerp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateFloatRange",NewName="/Script/RigVM.RigVMFunction_AccumulateFloatRange")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AccumulateVectorRange",NewName="/Script/RigVM.RigVMFunction_AccumulateVectorRange")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AlphaInterp",NewName="/Script/RigVM.RigVMFunction_AlphaInterp")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AlphaInterpVector",NewName="/Script/RigVM.RigVMFunction_AlphaInterpVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AlphaInterpQuat",NewName="/Script/RigVM.RigVMFunction_AlphaInterpQuat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_DeltaFromPreviousFloat",NewName="/Script/RigVM.RigVMFunction_DeltaFromPreviousFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_DeltaFromPreviousVector",NewName="/Script/RigVM.RigVMFunction_DeltaFromPreviousVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_DeltaFromPreviousQuat",NewName="/Script/RigVM.RigVMFunction_DeltaFromPreviousQuat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_DeltaFromPreviousTransform",NewName="/Script/RigVM.RigVMFunction_DeltaFromPreviousTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_KalmanFloat",NewName="/Script/RigVM.RigVMFunction_KalmanFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_KalmanVector",NewName="/Script/RigVM.RigVMFunction_KalmanVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_KalmanTransform",NewName="/Script/RigVM.RigVMFunction_KalmanTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_SimBase",NewName="/Script/RigVM.RigVMFunction_SimBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_SimBaseMutable",NewName="/Script/RigVM.RigVMFunction_SimBaseMutable")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_Timeline",NewName="/Script/RigVM.RigVMFunction_Timeline")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_TimeLoop",NewName="/Script/RigVM.RigVMFunction_TimeLoop")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_TimeOffsetFloat",NewName="/Script/RigVM.RigVMFunction_TimeOffsetFloat")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_TimeOffsetVector",NewName="/Script/RigVM.RigVMFunction_TimeOffsetVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_TimeOffsetTransform",NewName="/Script/RigVM.RigVMFunction_TimeOffsetTransform")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_VerletIntegrateVector",NewName="/Script/RigVM.RigVMFunction_VerletIntegrateVector")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_DebugPoint",NewName="/Script/RigVM.RigVMFunction_DebugPoint")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_DebugPointMutable",NewName="/Script/RigVM.RigVMFunction_DebugPointMutable")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AnimBase",NewName="/Script/RigVM.RigVMFunction_AnimBase")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AnimEasingType",NewName="/Script/RigVM.RigVMFunction_AnimEasingType")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AnimEasing",NewName="/Script/RigVM.RigVMFunction_AnimEasing")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AnimEvalRichCurve",NewName="/Script/RigVM.RigVMFunction_AnimEvalRichCurve")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_AnimRichCurve",NewName="/Script/RigVM.RigVMFunction_AnimRichCurve")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_GetDeltaTime",NewName="/Script/RigVM.RigVMFunction_GetDeltaTime")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_GetWorldTime",NewName="/Script/RigVM.RigVMFunction_GetWorldTime")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_FramesToSeconds",NewName="/Script/RigVM.RigVMFunction_FramesToSeconds")
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_SecondsToFrames",NewName="/Script/RigVM.RigVMFunction_SecondsToFrames")

; moving editor side types to RigVMEditor module
+StructRedirects=(OldName="/Script/ControlRig.RigGraphDisplaySettings",NewName="/Script/RigVMDeveloper.RigVMEdGraphDisplaySettings")
+StructRedirects=(OldName="/Script/ControlRig.ControlRigPythonSettings",NewName="/Script/RigVMDeveloper.RigVMPythonSettings")
+StructRedirects=(OldName="/Script/ControlRig.ControlRigPublicFunctionArg",NewName="/Script/RigVMDeveloper.RigVMOldPublicFunctionArg")
+StructRedirects=(OldName="/Script/ControlRig.ControlRigPublicFunctionData",NewName="/Script/RigVMDeveloper.RigVMOldPublicFunctionData")
+ClassRedirects=(OldName="/Script/ControlRig.NameSpacedUserData", NewName="/Script/RigVM.NameSpacedUserData")
+ClassRedirects=(OldName="/Script/ControlRig.DataAssetLink", NewName="/Script/RigVM.DataAssetLink")

; moving control rig test data to control rig replay
+StructRedirects=(OldName="/Script/ControlRig.ControlRigTestDataVariable",NewName="/Script/ControlRig.ControlRigReplayVariable")
+EnumRedirects=(OldName="EControlRigTestDataPlaybackMode",NewName="/Script/ControlRig.EControlRigReplayPlaybackMode")

[/Script/Engine.Engine]
+K2ParamRedirects=(NodeName="/Script/BlueprintGraph.K2Node_CallFunction", OldParamName="ControlRigSequencerEditorLibrary.FindOrCreateControlRigTrack.bIsAdditiveControlRig", NewParamName="bIsLayeredControlRig")
