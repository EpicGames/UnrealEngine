{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Control Rig",
	"Description" : "Framework for animation driven by user controls.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "RigVM",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "PropertyAccessEditor",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		},
		{
			"Name": "TweeningUtils",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "ControlRig",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "ControlRigDeveloper",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "ControlRigEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}