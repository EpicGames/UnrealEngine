// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ControlRig : ModuleRules
    {
        public ControlRig(ReadOnlyTargetRules Target) : base(Target)
        {
			NumIncludedBytesPerUnityCPPOverride = 688128; // best unity size found from using UBT ProfileUnitySizes mode

			bRequiresPlatformSDK = true;

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "AnimGraphRuntime",
                    "MovieScene",
                    "MovieSceneTracks",
                    "PropertyPath",
					"TimeManagement",
					"DeveloperSettings",
					"SlateCore"
				}
            );

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "AnimationCore",
                    "LevelSequence",
                    "RigVM",
                    "RHI",
                    "Constraints"
                }
            );

            if (Target.bBuildEditor == true)
            {
                PublicDependencyModuleNames.AddRange(
				    new string[]
					{
						"Slate",
						"SlateCore",
						"RigVMDeveloper",
                        "AnimGraph",
                        "Json",
                        "Serialization",
                        "JsonUtilities",
                        "AnimationBlueprintLibrary",
                    }
                );

                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
						"EditorFramework",
                        "UnrealEd",
                        "BlueprintGraph",
                        "PropertyEditor",
                    }
                );

                PrivateIncludePathModuleNames.Add("ControlRigEditor");
                DynamicallyLoadedModuleNames.Add("ControlRigEditor");
            }
        }
    }
}
