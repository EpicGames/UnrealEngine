// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_ControlRig.h"
#include "Animation/AnimTrace.h"
#include "ControlRig.h"
#include "ControlRigComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimInstanceProxy.h"
#include "GameFramework/Actor.h"
#include "Animation/NodeMappingContainer.h"
#include "AnimationRuntime.h"
#include "ControlRigObjectBinding.h"
#include "ControlRigRuntimeAsset.h"
#include "Algo/ForEach.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimStats.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ControlRig)

#if WITH_EDITOR
#include "Editor.h"
#endif

FAnimNode_ControlRig::FAnimNode_ControlRig()
	: FAnimNode_ControlRigBase()
	, ControlRig(nullptr)
	, Alpha(1.f)
	, AlphaInputType(EAnimAlphaInputType::Float)
	, bAlphaBoolEnabled(true)
	, bSetRefPoseFromSkeleton(false)
	, AlphaCurveName(NAME_None)
	, LODThreshold(INDEX_NONE)
{
}

FAnimNode_ControlRig::~FAnimNode_ControlRig()
{
	if(ControlRig && ControlRig.IsResolved())
	{
		ControlRig->OnInitialized_AnyThread().RemoveAll(this);
	}
}

void FAnimNode_ControlRig::HandleOnInitialized_AnyThread(URigVMHost*, const FName&)
{
	ControlRigHierarchyMappings.ResetRefPoseSetterHash();
}

void FAnimNode_ControlRig::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ControlRigPerClass.Reset();
	if(DefaultControlRigAssetReference.IsValid())
	{
		ControlRigAssetReference = FControlRigAssetStrongReference();
	}
	
	if(UpdateControlRigIfNeeded(InAnimInstance, InAnimInstance->GetRequiredBones()))
	{
		ControlRigHierarchyMappings.UpdateControlRigRefPoseIfNeeded(ControlRig
			, InProxy->GetAnimInstanceObject()
			, InProxy->GetSkelMeshComponent()
			, InProxy->GetRequiredBones()
			, bSetRefPoseFromSkeleton
			, /*bIncludePoseInHash*/ false);
	}

	FAnimNode_ControlRigBase::OnInitializeAnimInstance(InProxy, InAnimInstance);

	InitializeProperties(InAnimInstance, GetTargetAssetReference());
}

void FAnimNode_ControlRig::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FString DebugLine = DebugData.GetNodeName(this);
	DebugLine += FString::Printf(TEXT("(%s)"), *GetNameSafe(ControlRigAssetReference.Get()));
	DebugData.AddDebugItem(DebugLine);
	Source.GatherDebugData(DebugData.BranchFlow(1.f));
}

void FAnimNode_ControlRig::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_ControlRig_Update_AnyThread);

	if (IsLODEnabled(Context.AnimInstanceProxy))
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		// alpha handlers
		InternalBlendAlpha = 0.f;
		switch (AlphaInputType)
		{
		case EAnimAlphaInputType::Float:
			InternalBlendAlpha = AlphaScaleBias.ApplyTo(AlphaScaleBiasClamp.ApplyTo(Alpha, Context.GetDeltaTime()));
			break;
		case EAnimAlphaInputType::Bool:
			InternalBlendAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
			break;
		case EAnimAlphaInputType::Curve:
			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
			{
				InternalBlendAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
			}
			break;
		};

		// Make sure Alpha is clamped between 0 and 1.
		InternalBlendAlpha = FMath::Clamp<float>(InternalBlendAlpha, 0.f, 1.f);

		PropagateInputProperties(Context.AnimInstanceProxy->GetAnimInstanceObject());
	}
	else
	{
		InternalBlendAlpha = 0.f;
	}

	if(const UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetAnimInstanceObject()))
	{
		(void)UpdateControlRigIfNeeded(AnimInstance, Context.AnimInstanceProxy->GetRequiredBones());
	}

	if (ControlRig)
	{
		ControlRigHierarchyMappings.UpdateControlRigRefPoseIfNeeded(ControlRig
		   , Context.AnimInstanceProxy->GetAnimInstanceObject()
		   , Context.AnimInstanceProxy->GetSkelMeshComponent()
		   , Context.AnimInstanceProxy->GetRequiredBones()
		   , bSetRefPoseFromSkeleton
		   , /*bIncludePoseInHash*/ false);
	}

	FAnimNode_ControlRigBase::Update_AnyThread(Context);

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Class"), *GetNameSafe(ControlRigAssetReference.Get()));
}

void FAnimNode_ControlRig::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_ControlRigBase::Initialize_AnyThread(Context);

	if (ControlRig)
	{
		//Don't Inititialize the Control Rig here it may have the wrong VM on the CDO
		SetTargetInstance(ControlRig);
		ControlRig->RequestInit();
		bControlRigRequiresInitialization = true;
		LastBonesSerialNumberForCacheBones = 0;
	}
	else
	{
		SetTargetInstance(nullptr);
	}

	AlphaBoolBlend.Reinitialize();
	AlphaScaleBiasClamp.Reinitialize();
}

void FAnimNode_ControlRig::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// make sure the inputs on the node are evaluated before propagating the inputs
	GetEvaluateGraphExposedInputs().Execute(Context);

	// we also need access to the properties when running construction event
	PropagateInputProperties(Context.AnimInstanceProxy->GetAnimInstanceObject());

	// update the control rig instance just in case the dynamic control rig class has changed
	const UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.GetAnimInstanceObject());
	if (AnimInstance != nullptr)
	{
		(void)UpdateControlRigIfNeeded(AnimInstance, Context.AnimInstanceProxy->GetRequiredBones());
	}

	FAnimNode_ControlRigBase::CacheBones_AnyThread(Context);

	// The call to base cache bones might execute construction event, which will recreate the user generated controls
	// In case one of these controls is exposed as public variable, we have to re-initialize variable mappings
	if (AnimInstance != nullptr && ControlRigVariableMappings.RequiresInitAfterConstruction())
	{
		ControlRigVariableMappings.InitializeProperties(AnimInstance->GetClass(), TargetInstance, GetTargetAssetReference(), SourcePropertyNames, DestPropertyNames);
	}

	ControlRigVariableMappings.ResetCurvesInputToControlCache();

	FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	if(RequiredBones.IsValid())
	{
		ControlRigHierarchyMappings.ResetRefPoseSetterHash();

		URigHierarchy* Hierarchy = nullptr;
		if(UControlRig* CurrentControlRig = GetControlRig())
		{
			Hierarchy = CurrentControlRig->GetHierarchy();
		}

		ControlRigVariableMappings.CacheCurveMappings(InputMapping, OutputMapping, Hierarchy);
	}
}

void FAnimNode_ControlRig::Evaluate_AnyThread(FPoseContext & Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ControlRig, !IsInGameThread());

	// evaluate 
	FAnimNode_ControlRigBase::Evaluate_AnyThread(Output);
}

void FAnimNode_ControlRig::PostSerialize(const FArchive& Ar)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	
	if (ControlRigClass_DEPRECATED)
	{
		ControlRigAssetReference.Set(ControlRigClass_DEPRECATED);
		ControlRigClass_DEPRECATED = nullptr;
	}
	
	if (DefaultControlRigClass_DEPRECATED)
	{
		DefaultControlRigAssetReference.Set(DefaultControlRigClass_DEPRECATED);
		DefaultControlRigClass_DEPRECATED = nullptr;
	}
}

UClass* FAnimNode_ControlRig::GetTargetClass() const
{
	FControlRigAssetStrongReference RigSource = GetTargetAssetReference();
	return RigSource.GetRigClass();
}

FControlRigAssetStrongReference FAnimNode_ControlRig::GetTargetAssetReference() const
{
	if(ControlRigAssetReference.IsValid())
	{
		return ControlRigAssetReference;
	}
	return DefaultControlRigAssetReference;
}

void FAnimNode_ControlRig::UpdateInput(UControlRig* InControlRig, FPoseContext& InOutput)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_ControlRigBase::UpdateInput(InControlRig, InOutput);

	ControlRigVariableMappings.UpdateCurveInputs(InControlRig, InputMapping, InOutput.Curve);
}

void FAnimNode_ControlRig::UpdateOutput(UControlRig* InControlRig, FPoseContext& InOutput)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_ControlRigBase::UpdateOutput(InControlRig, InOutput);

	ControlRigVariableMappings.UpdateCurveOutputs(InControlRig, OutputMapping, InOutput.Curve);
}

void FAnimNode_ControlRig::SetControlRigClass(TSubclassOf<UControlRig> InControlRigClass)
{
	if(DefaultControlRigAssetReference == nullptr)
	{
		DefaultControlRigAssetReference = ControlRigAssetReference;
	}

	// this may be setting an invalid runtime rig class,
	// which will be validated during UpdateControlRigIfNeeded
	ControlRigAssetReference = InControlRigClass;
}

bool FAnimNode_ControlRig::UpdateControlRigIfNeeded(const UAnimInstance* InAnimInstance, const FBoneContainer& InRequiredBones)
{
	FControlRigAssetStrongReference RigSource = GetTargetAssetReference();
	if (RigSource.IsValid() && RigSource.Get() != nullptr)
	{
		if(ControlRig != nullptr)
		{
			if(!RigSource.IsSourceOf(ControlRig))
			{
				UControlRig* NewControlRig = nullptr;

				auto ReportErrorAndSwitchToDefaultRig = [this, InAnimInstance, InRequiredBones, RigSource](const FString& InMessage) -> bool
				{
					static constexpr TCHAR Format[] =  TEXT("[%s] Cannot switch to runtime rig class '%s' - reverting to default. %s");
					UE_LOG(LogControlRig, Warning, Format, *InAnimInstance->GetPathName(), *RigSource.GetPathName(), *InMessage);

					// mark the class to be known - and nullptr - indicating that it is not supported.
					ControlRigPerClass.FindOrAdd(RigSource, nullptr);

					// fall back to the default control rig and switch to that
					ControlRigAssetReference = nullptr;
					
					return UpdateControlRigIfNeeded(InAnimInstance, InRequiredBones);
				};

				// if we are reacting to a programmatic change
				// we need to perform validation between the two control rigs (old and new)
				if((ControlRigAssetReference == RigSource) &&
					DefaultControlRigAssetReference.IsValid() &&
					(RigSource != DefaultControlRigAssetReference))
				{
					// check if we already created this before
					if(const TObjectPtr<UControlRig>* ExistingControlRig = ControlRigPerClass.Find(ControlRigAssetReference))
					{
						NewControlRig = *ExistingControlRig;

						// the existing control rig is nullptr indicates that the class is not supported.
						// the warning will have been logged before - so it's not required to log it again.
						if(NewControlRig == nullptr)
						{
							// fall back to the default control rig and switch to that
							ControlRigAssetReference = nullptr;
							return UpdateControlRigIfNeeded(InAnimInstance, InRequiredBones);
						}
					}
					else
					{
						if(RigSource.IsNative())
						{
							static constexpr TCHAR Format[] = TEXT("Class '%s' is not supported (is it native).");
							return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *RigSource.GetPathName()));
						}
						
						// compare the two classes and make sure that the expected class is a super set in terms of
						// user defined properties.
						ControlRigAssetReference.ForEachVariableProperty([RigSource, ReportErrorAndSwitchToDefaultRig](const FProperty* OldProperty) -> bool
							{
								if(OldProperty->IsNative())
								{
									return true;
								}

								const FProperty* NewProperty = RigSource.FindVariablePropertyByName(OldProperty->GetFName());
								if(NewProperty == nullptr)
								{
									static constexpr TCHAR Format[] = TEXT("Property / Variable '%s' is missing.");
									return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *OldProperty->GetName()));
								}

								if(!NewProperty->SameType(OldProperty))
								{
									const FString OldCPPType = RigVMTypeUtils::GetCPPTypeFromProperty(OldProperty);
									const FString NewCPPType = RigVMTypeUtils::GetCPPTypeFromProperty(NewProperty);
									static constexpr TCHAR Format[] = TEXT("Property / Variable '%s' has incorrect type (is '%s', expected '%s').");
									return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *NewProperty->GetName(), *NewCPPType, *OldCPPType));
								}
								return true;
							});
						
						// create a new control rig using the new class
						{
							// Let's make sure the GC isn't running when we try to create a new Control Rig.
							FGCScopeGuard GCGuard;
							
							NewControlRig = RigSource.CreateInstance(InAnimInstance->GetOwningComponent());
							
							// If the object was created on a non-game thread, clear the async flag immediately, so that it can be
							// garbage collected in the future. 
							(void)NewControlRig->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
														
							NewControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
							NewControlRig->GetObjectBinding()->BindToObject(InAnimInstance->GetOwningComponent());
							NewControlRig->Initialize(true);
							NewControlRig->RequestInit();
						}

						// temporarily set the new control rig to be the target instance
						TGuardValue<TObjectPtr<UObject>> TargetInstanceGuard(TargetInstance, NewControlRig);

						// propagate all variable inputs
						PropagateInputProperties(InAnimInstance);

						// run construction on the rig
						NewControlRig->Execute(FRigUnit_PrepareForExecution::EventName);

						const URigHierarchy* OldHierarchy = ControlRig->GetHierarchy();
						const URigHierarchy* NewHierarchy = NewControlRig->GetHierarchy();

						// now compare the two rigs - we need to check bone hierarchy compatibility.
						const TArray<FRigElementKey> OldBoneKeys = OldHierarchy->GetBoneKeys(false);
						const TArray<FRigElementKey> NewBoneKeys = NewHierarchy->GetBoneKeys(false);
						for(const FRigElementKey& BoneKey : OldBoneKeys)
						{
							if(!NewBoneKeys.Contains(BoneKey))
							{
								static constexpr TCHAR Format[] = TEXT("Bone '%s' is missing from the rig.");
								return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *BoneKey.Name.ToString()));
							}
						}

						// we also need to check curve hierarchy compatibility.
						const TArray<FRigElementKey> OldCurveKeys = OldHierarchy->GetCurveKeys();
						const TArray<FRigElementKey> NewCurveKeys = NewHierarchy->GetCurveKeys();
						for(const FRigElementKey& CurveKey : OldCurveKeys)
						{
							if(!NewCurveKeys.Contains(CurveKey))
							{
								static constexpr TCHAR Format[] = TEXT("Curve '%s' is missing from the rig.");
								return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *CurveKey.Name.ToString()));
							}
						}

						// we also need to check that potentially exposed controls match
						for (int32 PropIdx = 0; PropIdx < DestPropertyNames.Num(); ++PropIdx)
						{
							if(const FRigControlElement* OldControlElement = ControlRig->FindControl(DestPropertyNames[PropIdx]))
							{
								const FRigControlElement* NewControlElement = NewControlRig->FindControl(DestPropertyNames[PropIdx]);
								if(NewControlElement == nullptr)
								{
									static constexpr TCHAR Format[] = TEXT("Control '%s' is missing from the rig.");
									return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *DestPropertyNames[PropIdx].ToString()));
								}

								if(NewControlElement->Settings.ControlType != OldControlElement->Settings.ControlType)
								{
									static const UEnum* ControlTypeEnum = StaticEnum<ERigControlType>();
									const FString OldType = ControlTypeEnum->GetDisplayNameTextByValue((int64)OldControlElement->Settings.ControlType).ToString();
									const FString NewType = ControlTypeEnum->GetDisplayNameTextByValue((int64)NewControlElement->Settings.ControlType).ToString();
									static constexpr TCHAR Format[] = TEXT("Control '%s' has the incorrect type (is '%s', expected '%s').");
									return ReportErrorAndSwitchToDefaultRig(FString::Printf(Format, *DestPropertyNames[PropIdx].ToString(), *NewType, *OldType));
								}
							}
						}

						// fall through: we have a compatible new control rig, let's just use that.
						FControlRigAssetStrongReference NewSource = NewControlRig->GetAssetReference();
						ControlRigPerClass.FindOrAdd(NewSource, NewControlRig);
					}
				}

				// stop listening to the rig, store it for reuse
				ControlRig->OnInitialized_AnyThread().RemoveAll(this);
				FControlRigAssetStrongReference NewSource = ControlRig->GetAssetReference();
				ControlRigPerClass.FindOrAdd(NewSource, ControlRig);
				ControlRig = nullptr;

				if(NewControlRig)
				{
					Swap(NewControlRig, ControlRig);
					SetTargetInstance(ControlRig);
				}
			}
			else
			{
				// we have a control rig of the right class
				return false;
			}
		}

		if(ControlRig == nullptr)
		{
			// Let's make sure the GC isn't running when we try to create a new Control Rig.
			FGCScopeGuard GCGuard;
			
			ControlRig = RigSource.CreateInstance(InAnimInstance->GetOwningComponent());
			if (!ControlRig)
			{
				return false;
			}
			(void)ControlRig->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
			
			ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
			ControlRig->GetObjectBinding()->BindToObject(InAnimInstance->GetOwningComponent());
			ControlRig->Initialize(true);
			ControlRig->RequestInit();
			SetTargetInstance(ControlRig);
		}

		ControlRigHierarchyMappings.ResetRefPoseSetterHash();

		ControlRig->OnInitialized_AnyThread().AddRaw(this, &FAnimNode_ControlRig::HandleOnInitialized_AnyThread);

		ControlRigHierarchyMappings.UpdateInputOutputMappingIfRequired(ControlRig
			, ControlRig->GetHierarchy()
			, InRequiredBones
			, InputBonesToTransfer
			, OutputBonesToTransfer
			, NodeMappingContainer
			, bTransferPoseInGlobalSpace
			, bResetInputPoseToInitial);

		return true;
	}
	return false;
}

void FAnimNode_ControlRig::InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass)
{
	FControlRigAssetStrongReference Reference(InTargetClass);
	InitializeProperties(InSourceInstance, Reference);
}

void FAnimNode_ControlRig::InitializeProperties(const UObject* InSourceInstance, FControlRigAssetStrongReference InTargetClass)
{
	// Full base override, no Super calling

	// Call helper to setup data
	const UClass* SourceClass = InSourceInstance->GetClass();
	ControlRigVariableMappings.InitializeProperties(SourceClass, TargetInstance, InTargetClass, SourcePropertyNames, DestPropertyNames);
}

void FAnimNode_ControlRig::PropagateInputProperties(const UObject* InSourceInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_ControlRig_PropagateInputProperties);


	if (!InSourceInstance)
	{
		return;
	}

	UControlRig* TargetControlRig = Cast<UControlRig>(TargetInstance);
	if (!TargetControlRig)
	{
		return;
	}

	ControlRigVariableMappings.PropagateInputProperties(InSourceInstance, TargetControlRig, DestPropertyNames);
}

#if WITH_EDITOR

void FAnimNode_ControlRig::HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	Super::HandleObjectsReinstanced_Impl(InSourceObject, InTargetObject, OldToNewInstanceMap);
	
	if(ControlRig)
	{
		ControlRig->OnInitialized_AnyThread().RemoveAll(this);
		ControlRig->OnInitialized_AnyThread().AddRaw(this, &FAnimNode_ControlRig::HandleOnInitialized_AnyThread);
	}
}

#endif