// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_ControlRigBase.h"
#include "ControlRig.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/NodeMappingContainer.h"
#include "AnimationRuntime.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SkeletalMesh.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Algo/Transform.h"
#include "Animation/AnimCurveUtils.h"
#include "Tools/ControlRigPoseAdapter.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ControlRigBase)


DECLARE_CYCLE_STAT(TEXT("ControlRig_UpdateInput"), STAT_ControlRig_UpdateInput, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("ControlRig_Evaluate"), STAT_ControlRig_Evaluate, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("ControlRig_Evaluate_Construction"), STAT_ControlRig_Evaluate_Construction, STATGROUP_Anim);
DECLARE_CYCLE_STAT(TEXT("ControlRig_UpdateOutput"), STAT_ControlRig_UpdateOutput, STATGROUP_Anim);

#if ENABLE_ANIM_DEBUG
TAutoConsoleVariable<int32> CVarAnimNodeControlRigDebug(TEXT("a.AnimNode.ControlRig.Debug"), 0, TEXT("Set to 1 to turn on debug drawing for AnimNode_ControlRigBase"));
#endif

// CVar to disable control rig execution within an anim node
static TAutoConsoleVariable<int32> CVarControlRigDisableExecutionAnimNode(TEXT("ControlRig.DisableExecutionInAnimNode"), 0, TEXT("if nonzero we disable the execution of Control Rigs inside an anim node."));

FAnimNode_ControlRigBase::FAnimNode_ControlRigBase()
	: FAnimNode_CustomProperty()
	, bResetInputPoseToInitial(true) 
	, bTransferInputPose(true)
	, bTransferInputCurves(true)
	, bTransferPoseInGlobalSpace(CVarControlRigEnableAnimNodePerformanceOptimizations->GetInt() == 0) // default to local in optimized mode
	, InputSettings(FControlRigIOSettings())
	, OutputSettings(FControlRigIOSettings())
	, bExecute(true)
	, InternalBlendAlpha (1.f)
	, bControlRigRequiresInitialization(true)
	, LastBonesSerialNumberForCacheBones(0)
{
}

void FAnimNode_ControlRigBase::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_CustomProperty::OnInitializeAnimInstance(InProxy, InAnimInstance);
	
	ControlRigHierarchyMappings.InitializeInstance();

	WeakAnimInstanceObject = TWeakObjectPtr<const UAnimInstance>(InAnimInstance);

	USkeletalMeshComponent* Component = InAnimInstance->GetOwningComponent();
	UControlRig* ControlRig = GetControlRig();
	if (Component && Component->GetSkeletalMeshAsset() && ControlRig)
	{
#if WITH_EDITORONLY_DATA
		UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(ControlRig->GetClass());
		if (BlueprintClass)
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(BlueprintClass->ClassGeneratedBy);
			// node mapping container will be saved on the initialization part
			NodeMappingContainer = Component->GetSkeletalMeshAsset()->GetNodeMappingContainer(Blueprint);
		}
#endif

		// register skeletalmesh component for now
		ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, InAnimInstance->GetOwningComponent());
		UpdateGetAssetUserDataDelegate(ControlRig);
	}
}

void FAnimNode_ControlRigBase::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_CustomProperty::Initialize_AnyThread(Context);
	Source.Initialize(Context);
}

void FAnimNode_ControlRigBase::GatherDebugData(FNodeDebugData& DebugData)
{
	Source.GatherDebugData(DebugData.BranchFlow(1.f));
}

void FAnimNode_ControlRigBase::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_CustomProperty::Update_AnyThread(Context);
	Source.Update(Context);

	if (bExecute)
	{
		if (UControlRig* ControlRig = GetControlRig())
		{
			// @TODO: fix this to be thread-safe
			// Pre-update doesn't work for custom anim instances
			// FAnimNode_ControlRigExternalSource needs this to be called to reset to ref pose
			ControlRig->SetDeltaTime(Context.GetDeltaTime());
		}
	}
}

bool FAnimNode_ControlRigBase::CanExecute()
{
	if(CVarControlRigDisableExecutionAnimNode->GetInt() != 0)
	{
		return false;
	}

	if(!ControlRigHierarchyMappings.CanExecute())
	{
		return false;
	}

	if (UControlRig* ControlRig = GetControlRig())
	{
		return ControlRig->CanExecute(); 
	}

	return false;
}

void FAnimNode_ControlRigBase::UpdateInput(UControlRig* ControlRig, FPoseContext& InOutput)
{
	SCOPE_CYCLE_COUNTER(STAT_ControlRig_UpdateInput);

	if(!CanExecute())
	{
		return;
	}

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ControlRigHierarchyMappings.UpdateInput(ControlRig
		, InOutput
		, InputSettings
		, OutputSettings
		, NodeMappingContainer
		, bExecute
		, bTransferInputPose
		, bResetInputPoseToInitial
		, bTransferPoseInGlobalSpace
		, bTransferInputCurves);
}

void FAnimNode_ControlRigBase::UpdateOutput(UControlRig* ControlRig, FPoseContext& InOutput)
{
	SCOPE_CYCLE_COUNTER(STAT_ControlRig_UpdateOutput);

	if(!CanExecute())
	{
		return;
	}

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ControlRigHierarchyMappings.UpdateOutput(ControlRig
		, InOutput
		, OutputSettings
		, NodeMappingContainer
		, bExecute
		, bTransferPoseInGlobalSpace);
}

void FAnimNode_ControlRigBase::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FPoseContext SourcePose(Output);

	if (Source.GetLinkNode())
	{
		Source.Evaluate(SourcePose);
	}
	else
	{
		// apply refpose
		SourcePose.ResetToRefPose();
	}

	ExecuteConstructionIfNeeded();

	if(!ControlRigHierarchyMappings.CheckPoseAdapter())
	{
		Output = SourcePose;
		ensureMsgf(false, TEXT("At this point the pose adapter needs to be set!"));
		return;
	}

	if (CanExecute() && FAnimWeight::IsRelevant(InternalBlendAlpha) && GetControlRig())
	{
		if (FAnimWeight::IsFullWeight(InternalBlendAlpha))
		{
			ExecuteControlRig(SourcePose);
			Output = SourcePose;
		}
		else 
		{
			// this blends additively - by weight
			FPoseContext ControlRigPose(SourcePose);
			ControlRigPose = SourcePose;
			ExecuteControlRig(ControlRigPose);

			FPoseContext AdditivePose(ControlRigPose);
			AdditivePose = ControlRigPose;
			FAnimationRuntime::ConvertPoseToAdditive(AdditivePose.Pose, SourcePose.Pose);
			AdditivePose.Curve.ConvertToAdditive(SourcePose.Curve);
			Output = SourcePose;

			UE::Anim::Attributes::ConvertToAdditive(SourcePose.CustomAttributes, AdditivePose.CustomAttributes);

			FAnimationPoseData BaseAnimationPoseData(Output);
			const FAnimationPoseData AdditiveAnimationPoseData(AdditivePose);
			FAnimationRuntime::AccumulateAdditivePose(BaseAnimationPoseData, AdditiveAnimationPoseData, InternalBlendAlpha, AAT_LocalSpaceBase);
		}
	}
	else // if not relevant, skip to run control rig
		// this may cause issue if we have simulation node in the control rig that accumulates time
	{
		Output = SourcePose;
	}
}

void FAnimNode_ControlRigBase::ExecuteControlRig(FPoseContext& InOutput)
{
	SCOPE_CYCLE_COUNTER(STAT_ControlRig_Evaluate);

	if (UControlRig* ControlRig = GetControlRig())
	{
		// Before we start modifying the RigHierarchy, we need to lock the rig to avoid
		// corrupted state
		UE::TScopeLock LockRig(ControlRig->GetEvaluateMutex());

		UE::Anim::FMeshAttributeContainer MeshAttributeContainer;
		MeshAttributeContainer.CopyFrom(InOutput.CustomAttributes, InOutput.Pose.GetBoneContainer());
		// temporarily give control rig access to the heap allocated attribute container
		// control rig may have rig units that can add/get attributes to/from this container
		UControlRig::FAnimAttributeContainerPtrScope AttributeScope(ControlRig, MeshAttributeContainer);

		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		if(Hierarchy == nullptr)
		{
			return;
		}

		if(ControlRigHierarchyMappings.IsPoseAdapterEnabled())
		{
			if(!ControlRigHierarchyMappings.IsUpdateToDate(Hierarchy))
			{
				ControlRigHierarchyMappings.PerformUpdateToDate(ControlRig
					, Hierarchy
					, InOutput.Pose.GetBoneContainer()
					, NodeMappingContainer
					, bTransferPoseInGlobalSpace
					, bResetInputPoseToInitial);
			}
		}

		// first update input to the system
		UpdateInput(ControlRig, InOutput);
		
		if (bExecute)
		{
			TGuardValue<bool> ResetCurrentTransfromsAfterConstructionGuard(ControlRig->bResetCurrentTransformsAfterConstruction, true);
			
#if WITH_EDITOR
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::BeforeEvaluate"));
			}
#endif

			// pick the event to run
			if(EventQueue.IsEmpty())
			{
				if(bClearEventQueueRequired)
				{
					ControlRig->SetEventQueue({FRigUnit_BeginExecution::EventName});
					bClearEventQueueRequired = false;
				}
			}
			else
			{
				TArray<FName> EventNames;
				Algo::Transform(EventQueue, EventNames, [](const FControlRigAnimNodeEventName& InEventName) 
				{
					return InEventName.EventName;
				});
				ControlRig->SetEventQueue(EventNames);
				bClearEventQueueRequired = true;
			}

			if (ControlRig->IsAdditive())
			{
				ControlRig->ClearPoseBeforeBackwardsSolve();
			}

			// evaluate control rig
			UpdateGetAssetUserDataDelegate(ControlRig);
			ControlRig->Evaluate_AnyThread();

#if ENABLE_ANIM_DEBUG 
			// When Control Rig is at editing time (in CR editor), draw instructions are consumed by ControlRigEditMode, so we need to skip drawing here.
			bool bShowDebug = (CVarAnimNodeControlRigDebug.GetValueOnAnyThread() == 1 && ControlRig->ExecutionType != ERigExecutionType::Editing);

			if (bShowDebug)
			{ 
				QueueControlRigDrawInstructions(ControlRig, InOutput.AnimInstanceProxy);
			}
#endif

#if WITH_EDITOR
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::AfterEvaluate"));
			}
#endif
		}

		// now update output
		UpdateOutput(ControlRig, InOutput);
		InOutput.CustomAttributes.CopyFrom(MeshAttributeContainer, InOutput.Pose.GetBoneContainer());
	}
}

void FAnimNode_ControlRigBase::ExecuteConstructionIfNeeded()
{
	SCOPE_CYCLE_COUNTER(STAT_ControlRig_Evaluate_Construction);

	if (UControlRig* ControlRig = GetControlRig())
	{
		if (!ControlRig->IsConstructionRequired())
		{
			return;
		}
		
		// Before we start modifying the RigHierarchy, we need to lock the rig to avoid
		// corrupted state
		UE::TScopeLock LockRig(ControlRig->GetEvaluateMutex());

		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		if(Hierarchy == nullptr)
		{
			return;
		}

		if (bExecute)
		{
			TGuardValue<bool> ResetCurrentTransfromsAfterConstructionGuard(ControlRig->bResetCurrentTransformsAfterConstruction, true);
			
#if WITH_EDITOR
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::BeforeEvaluate"));
			}
#endif

			// empty queue (only leave construction and post construction)
			const TArray<FName> CurrentEventQueue = ControlRig->GetEventQueue();
			ControlRig->SetEventQueue({});
			TGuardValue<bool> OnlyRunConstruction(ControlRig->bOnlyRunConstruction, true);

			// evaluate control rig
			UpdateGetAssetUserDataDelegate(ControlRig);
			ControlRig->Evaluate_AnyThread();

#if WITH_EDITOR
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::AfterEvaluate"));
			}
#endif

			ControlRig->SetEventQueue(CurrentEventQueue);
		}
	}
}

struct FControlRigControlScope
{
	FControlRigControlScope(UControlRig* InControlRig)
		: ControlRig(InControlRig)
	{
		if (ControlRig.IsValid())
		{
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			Hierarchy->ForEach<FRigControlElement>([this, Hierarchy](FRigControlElement* ControlElement) -> bool
			{
				ControlValues.Add(ControlElement->GetKey(), Hierarchy->GetControlValueByIndex(ControlElement->GetIndex()));
				return true; // continue
			});
		}
	}

	~FControlRigControlScope()
	{
		if (ControlRig.IsValid())
		{
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			for (const TPair<FRigElementKey, FRigControlValue>& Pair: ControlValues)
			{
				Hierarchy->SetControlValue(Pair.Key, Pair.Value);
			}
		}
	}

	TMap<FRigElementKey, FRigControlValue> ControlValues;
	TWeakObjectPtr<UControlRig> ControlRig;
};

void FAnimNode_ControlRigBase::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FAnimNode_CustomProperty::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);

	if (UControlRig* ControlRig = GetControlRig())
	{
		// fill up node names
		const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();

		const uint16 BonesSerialNumber = RequiredBones.GetSerialNumber();

		{
			// We can get a call to CacheBones in the main thread while a worker thread is also executing Control Rig (i.e. a FControlRigLayerInstanceProxy)
			// We have to avoid the concurrent execution, because we will destroy the hierarchy here and create a new one
			UE::TScopeLock EvaluateLock(ControlRig->GetEvaluateMutex());

			// the construction event may create a set of bones that we can map to. let's run construction now.
			if (bExecute)
			{
				const bool bIsLODChange = !bControlRigRequiresInitialization && (BonesSerialNumber != LastBonesSerialNumberForCacheBones);
		
				if(ControlRig->IsConstructionModeEnabled() ||
					(ControlRig->IsConstructionRequired() && (bControlRigRequiresInitialization || bIsLODChange)))
				{
					UpdateGetAssetUserDataDelegate(ControlRig);
					ControlRig->Execute(FRigUnit_PrepareForExecution::EventName);
					bControlRigRequiresInitialization = false;
				}
			}

			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

			ControlRigHierarchyMappings.UpdateInputOutputMappingIfRequired(ControlRig
				, Hierarchy
				, RequiredBones
				, InputBonesToTransfer
				, OutputBonesToTransfer
				, NodeMappingContainer
				, bTransferPoseInGlobalSpace
				, bResetInputPoseToInitial);

			if(bControlRigRequiresInitialization)
			{
				if(bExecute)
				{
					// re-init only if this is the first run
					// and restore control values
					ControlRig->RequestInit();
					bControlRigRequiresInitialization = false;
				}
			}
		
			LastBonesSerialNumberForCacheBones = BonesSerialNumber;

			ControlRigHierarchyMappings.LinkToHierarchy(Hierarchy);
		}
	}
}

UClass* FAnimNode_ControlRigBase::GetTargetClass() const
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		return ControlRig->GetClass();
	}

	return nullptr;
}

FControlRigAssetStrongReference FAnimNode_ControlRigBase::GetTargetAssetReference() const
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		return ControlRig->GetAssetReference();
	}

	return nullptr;
}

void FAnimNode_ControlRigBase::QueueControlRigDrawInstructions(UControlRig* ControlRig, FAnimInstanceProxy* Proxy) const
{
	ensure(ControlRig);
	ensure(Proxy);

	if (ControlRig && Proxy)
	{
		for (const FRigVMDrawInstruction& Instruction : ControlRig->GetDrawInterface())
		{
			if (!Instruction.IsValid())
			{
				continue;
			}

			FTransform InstructionTransform = Instruction.Transform * Proxy->GetComponentTransform();
			switch (Instruction.PrimitiveType)
			{
				case ERigVMDrawSettings::Points:
				{
					for (const FVector& Point : Instruction.Positions)
					{
						Proxy->AnimDrawDebugPoint(InstructionTransform.TransformPosition(Point), Instruction.Thickness, Instruction.Color.ToFColor(true), false, Instruction.Lifetime, Instruction.DepthPriority);
					}
					break;
				}
				case ERigVMDrawSettings::Lines:
				{
					const TArray<FVector>& Points = Instruction.Positions;

					for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex += 2)
					{
						Proxy->AnimDrawDebugLine(InstructionTransform.TransformPosition(Points[PointIndex]), InstructionTransform.TransformPosition(Points[PointIndex + 1]), Instruction.Color.ToFColor(true), false, Instruction.Lifetime, Instruction.Thickness, Instruction.DepthPriority);
					}
					break;
				}
				case ERigVMDrawSettings::LineStrip:
				{
					const TArray<FVector>& Points = Instruction.Positions;

					for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
					{
						Proxy->AnimDrawDebugLine(InstructionTransform.TransformPosition(Points[PointIndex]), InstructionTransform.TransformPosition(Points[PointIndex + 1]), Instruction.Color.ToFColor(true), false, Instruction.Lifetime, Instruction.Thickness, Instruction.DepthPriority);
					}
					break;
				}

				case ERigVMDrawSettings::DynamicMesh:
				{
					// TODO: Add support for this if anyone is actually using it. Currently it is only defined and referenced in an unused API, DrawCone in Control Rig.
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

void FAnimNode_ControlRigBase::UpdateGetAssetUserDataDelegate(UControlRig* InControlRig) const
{
	if (!IsInGameThread())
	{
		return;
	}
	
	if(GetAssetUserData().IsEmpty() || !WeakAnimInstanceObject.IsValid())
	{
		InControlRig->GetExternalAssetUserDataDelegate.Unbind();
		return;
	}
	
	// due to the re-instancing of the anim nodes we have to set this up for every run
	// since the delegate may go stale quickly. to guard against destroyed anim nodes
	// we'll rely on the anim instance to provide an indication if the memory is still valid. 
	TWeakObjectPtr<const UAnimInstance> LocalWeakAnimInstance = WeakAnimInstanceObject;
	InControlRig->GetExternalAssetUserDataDelegate = UControlRig::FGetExternalAssetUserData::CreateLambda([InControlRig, LocalWeakAnimInstance, this]
	{
		if(LocalWeakAnimInstance.IsValid())
		{
			return this->GetAssetUserData();
		}
		if(IsValid(InControlRig))
		{
			InControlRig->GetExternalAssetUserDataDelegate.Unbind();
		}
		return TArray<TObjectPtr<UAssetUserData>>();
	});
}


