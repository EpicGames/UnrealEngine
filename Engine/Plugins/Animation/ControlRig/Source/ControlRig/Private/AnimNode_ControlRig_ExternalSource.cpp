// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_ControlRig_ExternalSource.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ControlRig_ExternalSource)

FAnimNode_ControlRig_ExternalSource::FAnimNode_ControlRig_ExternalSource()
{
	bControlRigRequiresInitialization = false;
}

void FAnimNode_ControlRig_ExternalSource::SetControlRig(UControlRig* InControlRig)
{
	ControlRig = InControlRig;
	// requires initializing animation system
}

UControlRig* FAnimNode_ControlRig_ExternalSource::GetControlRig() const
{
	return (ControlRig.IsValid()? ControlRig.Get() : nullptr);
}

TSubclassOf<UControlRig> FAnimNode_ControlRig_ExternalSource::GetControlRigClass() const
{
	if(UControlRig* CR = GetControlRig())
	{
		return CR->GetClass();
	}
	return nullptr;
}

FControlRigAssetStrongReference FAnimNode_ControlRig_ExternalSource::GetControlRigAssetReference() const
{
	if (UControlRig* CR = GetControlRig())
	{
		return CR->GetAssetReference();
	}
	return FControlRigAssetStrongReference();
}

void FAnimNode_ControlRig_ExternalSource::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_ControlRigBase::Initialize_AnyThread(Context);

	if (ControlRig.IsValid())
	{
		//Don't Initialize the Control Rig here, the owner of the rig is in charge of initializing
		SetTargetInstance(ControlRig.Get());
	}
	else
	{
		SetTargetInstance(nullptr);
	}
}


