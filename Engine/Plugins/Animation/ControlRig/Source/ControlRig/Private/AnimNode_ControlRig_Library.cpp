// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_ControlRig_Library.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ControlRig_Library)

FControlRigReference UAnimNodeControlRigLibrary::ConvertToControlRig(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result)
{
	return FAnimNodeReference::ConvertToType<FControlRigReference>(Node, Result);
}

FControlRigReference UAnimNodeControlRigLibrary::SetControlRigClass(const FControlRigReference& Node, TSubclassOf<UControlRig> ControlRigClass)
{
	Node.CallAnimNodeFunction<FAnimNode_ControlRig>(
	TEXT("SetSequence"),
	[ControlRigClass](FAnimNode_ControlRig& InControlRigNode)
	{
		InControlRigNode.SetControlRigClass(ControlRigClass);
	});

	return Node;
}