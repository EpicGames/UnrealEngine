// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigCommandChange.h"

#include "CoreGlobals.h"
#include "Misc/ITransaction.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

namespace UE::ControlRig
{
void FControlRigCommandChange::StoreUndo(UObject* Object, TUniquePtr<FControlRigCommandChange> Change, const FText& Description)
{
#if WITH_EDITOR
	if (GUndo && Object && Change)
	{
		const FScopedTransaction Transaction(Description);
		GUndo->StoreUndo(Object, MoveTemp(Change));
	}
#endif
}
}
