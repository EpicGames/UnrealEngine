// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/Change.h"

class UObject;

namespace UE::ControlRig
{
/** Base class for command changes in control rig. */
class FControlRigCommandChange : public FCommandChange
{
public:
	
	virtual FString ToString() const override { return TEXT("ControlRigCommand"); }

	/** Adds the command to the undo buffer. */
	static void StoreUndo(UObject* Object, TUniquePtr<FControlRigCommandChange> Change, const FText& Description = FText::GetEmpty());
};
}
