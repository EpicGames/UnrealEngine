// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElementSelectionCommand.h"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"

#include <type_traits>

namespace UE::ControlRig
{
namespace ElementSelectionDetail
{
template<typename TConvertSelectionState> requires std::is_invocable_r_v<bool, TConvertSelectionState, bool /*bIsSelectedWhenRedoing*/>
static void CombineData(
	const FElementSelectionArray& InChanges, URigHierarchyController& InController,
	TConvertSelectionState&& InConvertSelectionState
	)
{
	for (const FElementSelectionData& ChangeData : InChanges)
	{
		const bool bIsSelected = InConvertSelectionState(ChangeData.bSelect);
		constexpr bool bClearSelection = false,  bSetupUndo = false;
		InController.SelectHierarchyKey(ChangeData.Key, bIsSelected, bClearSelection, bSetupUndo);
	}
}

static URigHierarchyController* GetController(UObject* InUndoObject)
{
	URigHierarchy* Hierarchy = Cast<URigHierarchy>(InUndoObject);
	URigHierarchyController* Controller = Hierarchy ? Hierarchy->GetController() : nullptr;
	return Controller;
}
}
	
void FElementSelectionCommand::Apply(UObject* Object)
{
	if (URigHierarchyController* Controller = ElementSelectionDetail::GetController(Object))
	{
		ElementSelectionDetail::CombineData(Changes, *Controller,
			[](bool bSelectionStateWhenApplying) { return bSelectionStateWhenApplying; }
			);
	}
}

void FElementSelectionCommand::Revert(UObject* Object)
{
	if (URigHierarchyController* Controller = ElementSelectionDetail::GetController(Object))
	{
		ElementSelectionDetail::CombineData(Changes, *Controller,
			[](bool bSelectionStateWhenApplying) { return !bSelectionStateWhenApplying; }
			);
	}
}
}
