// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigCommandChange.h"
#include "Rigs/RigHierarchyDefines.h"
#include "UObject/WeakObjectPtrTemplates.h"

class URigHierarchy;

namespace UE::ControlRig
{
struct FElementSelectionData
{
	/** The key whose selection changed. */
	FRigHierarchyKey Key;
	/** When applied (redo), whether the key was selected. */
	bool bSelect;

	FElementSelectionData(const FRigHierarchyKey& Key, bool bSelect) : Key(Key) , bSelect(bSelect) {}
};

// Avoid heap allocation for the most common case of 1 element
using FElementSelectionArray = TArray<FElementSelectionData, TInlineAllocator<1>>;
	
/** Changes the selection of a URigHierarchy. */
class FElementSelectionCommand : public FControlRigCommandChange
{
public:
	
	explicit FElementSelectionCommand(FElementSelectionArray InChanges) : Changes(MoveTemp(InChanges)) {}
	FElementSelectionCommand(const FElementSelectionData& InSingleData)
		: FElementSelectionCommand(FElementSelectionArray( { InSingleData }))
	{}

	//~ Begin FChange Interface
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
	//~ End FChange Interface

private:

	/** The elements and their changed state */
	FElementSelectionArray Changes;
};
}


