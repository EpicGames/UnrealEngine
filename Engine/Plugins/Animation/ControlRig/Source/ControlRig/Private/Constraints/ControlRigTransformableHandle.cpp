// Copyright Epic Games, Inc. All Rights Reserved.


#include "Constraints/ControlRigTransformableHandle.h"

#include "Components/SkeletalMeshComponent.h"
#include "ControlRig.h"
#include "ControlRigComponent.h"
#include "IControlRigObjectBinding.h"
#include "ControlRigObjectBinding.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "ModularRig.h"
#include "Transform/TransformableHandleUtils.h"
#include "Rigs/RigHierarchyElements.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sequencer/ControlRigSequencerHelpers.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigTransformableHandle)

namespace UE::Private
{
	using RigGuard = TGuardValue_Bitfield_Cleanup<TFunction<void()>>;
	
	TSet<UControlRig*> NotifyingRigs;

	bool IsRigNotifying(const UControlRig* InControlRig)
	{
		return InControlRig ? NotifyingRigs.Contains(InControlRig) : false;
	}

	// only allow notification from one thread at a time
	FTransactionallySafeCriticalSection NotificationLock;
	
	struct FControlPoseChangedNotifier
	{
		FControlPoseChangedNotifier(const URigHierarchy* InHierarchy, const FRigControlElement* InControl, const USceneComponent* InComponent)
			: Hierarchy(InHierarchy)
			, Control(InControl)
			, Component(InComponent)
		{
			if (IsValid())
			{
				PoseVersion = Hierarchy->GetPoseVersion(Control); 
			}
		}
		
		~FControlPoseChangedNotifier()
		{
			if (IsValid() && Hierarchy->GetPoseVersion(Control) != PoseVersion)
			{
				TransformableHandleUtils::MarkComponentForEvaluation(Component);
			}
		}

	private:

		bool IsValid() const
		{
			return Hierarchy && Control && Component && TransformableHandleUtils::SkipTicking();
		}
		
		const URigHierarchy* Hierarchy = nullptr;
		const FRigControlElement* Control = nullptr;
		const USceneComponent* Component = nullptr;
		int32 PoseVersion = INDEX_NONE;
	};

	void FixModularRigControlName(UControlRig* InControlRig, UTransformableControlHandle* InOutHandle)
	{
		if (UModularRig* ModularRig = Cast<UModularRig>(InControlRig))
		{
			if (const FRigControlElement* Control = ModularRig->FindControl(InOutHandle->ControlName))
			{
				const FName& ControlName = Control->GetFName();
				if (ControlName != InOutHandle->ControlName)
				{
					InOutHandle->ControlName = ControlName;
				}
			}
		}
	}
}

/**
 * UTransformableControlHandle
 */

UTransformableControlHandle::~UTransformableControlHandle()
{
	UnregisterDelegates();
}

void UTransformableControlHandle::PostLoad()
{
	Super::PostLoad();
	RegisterDelegates();
}

bool UTransformableControlHandle::IsValid(const bool bDeepCheck) const
{
	if (!ControlRig.IsValid() || ControlName == NAME_None)
	{
		return false;
	}

	const FRigControlElement* ControlElement = ControlRig->FindControl(ControlName);
	if (!ControlElement)
	{
		return false;
	}

	if (bDeepCheck)
	{
		const USceneComponent* BoundComponent = GetBoundComponent();
		if (!BoundComponent)
		{
			return false;
		}
	}
	
	return true;
}

void UTransformableControlHandle::PreEvaluate(const bool bTick) const
{
	if (!ControlRig.IsValid() || ControlRig->IsEvaluating())
	{
		return;
	}

	if (ControlRig->IsAdditive())
	{
		if (UE::Private::IsRigNotifying(ControlRig.Get()))
		{
			return;
		}

		if (const USkeletalMeshComponent* SkeletalMeshComponent = GetSkeletalMesh())
		{
			if (!SkeletalMeshComponent->PoseTickedThisFrame())
			{
				return TickTarget();
			}
		}
	}

	if (TransformableHandleUtils::SkipTicking())
	{
		// TODO test UControlRigComponent with FAnimationEvaluationCache 
		TransformableHandleUtils::EvaluateComponent(GetSkeletalMesh(), GetEvaluationTask());
		return;
	}
	
	if (!bTick)
	{
		return ControlRig->Evaluate_AnyThread();
	}

	// else full tick
	TickTarget();
}

void UTransformableControlHandle::TickTarget() const
{
	if (!ControlRig.IsValid())
	{
		return;
	}
	
	if (ControlRig->IsAdditive() && UE::Private::IsRigNotifying(ControlRig.Get()))
	{
		return;
	}
	
	if (USkeletalMeshComponent* SkeletalMeshComponent = GetSkeletalMesh())
	{
		if (TransformableHandleUtils::SkipTicking())
		{
			TransformableHandleUtils::EvaluateComponent( GetSkeletalMesh() );
			return;
		}
		
		return TransformableHandleUtils::TickDependantComponents(SkeletalMeshComponent);
	}

	if (UControlRigComponent* ControlRigComponent = GetControlRigComponent())
	{
		// TODO test an equivalent to EvaluateSkeletalMeshComponent
		ControlRigComponent->Update();
	}
}

// NOTE should we cache the skeletal mesh and the CtrlIndex to avoid looking for if every time
// probably not for handling runtime changes
void UTransformableControlHandle::SetGlobalTransform(const FTransform& InGlobal) const
{
	const FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement)
	{
		return;
	}

	const USceneComponent* BoundComponent = GetBoundComponent();
	if (!BoundComponent)
	{
		return;
	}
	
	const FTransform ComponentTransform = TransformableHandleUtils::SkipTicking() ?
		TransformableHandleUtils::GetGlobalTransform(BoundComponent, NAME_None) : 
		BoundComponent->GetComponentTransform();

	static const FRigControlModifiedContext Context(EControlRigSetKey::Never);
	static constexpr bool bNotify = false, bSetupUndo = false, bPrintPython = false, bFixEulerFlips = false;

	{
		UE::Private::FControlPoseChangedNotifier Notifier(ControlRig->GetHierarchy(), ControlElement, BoundComponent);
	
		//use this function so we don't set the preferred angles
		const FRigElementKey& ControlKey = ControlElement->GetKey();
		ControlRig->SetControlGlobalTransform(ControlKey.Name, InGlobal.GetRelativeTransform(ComponentTransform),
			bNotify, Context, bSetupUndo, bPrintPython, bFixEulerFlips);
	}
}

void UTransformableControlHandle::SetLocalTransform(const FTransform& InLocal) const
{
	const FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	const FRigElementKey& ControlKey = ControlElement->GetKey();
	const int32 CtrlIndex = Hierarchy->GetIndex(ControlKey);
	
	{
		UE::Private::FControlPoseChangedNotifier Notifier(ControlRig->GetHierarchy(), ControlElement, GetSkeletalMesh());
		Hierarchy->SetLocalTransform(CtrlIndex, InLocal);
	}
}

// NOTE should we cache the skeletal mesh and the CtrlIndex to avoid looking for if every time
// probably not for handling runtime changes
FTransform UTransformableControlHandle::GetGlobalTransform() const
{
	const FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement)
	{
		return FTransform::Identity;
	}
	
	USceneComponent* BoundComponent = GetBoundComponent();
	if (!BoundComponent)
	{
		return FTransform::Identity;
	}

	const FTransform& ComponentTransform = BoundComponent->GetComponentTransform();
	if (TransformableHandleUtils::SkipTicking())
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundComponent))
		{
			if (!ControlRig->IsEvaluating())
			{
				const UE::Anim::FAnimationEvaluator& Evaluator = TransformableHandleUtils::EvaluateComponent( SkeletalMeshComponent, GetEvaluationTask() );
				if (BoundComponent->GetAttachParent())
				{
					return Cache * Evaluator.GetGlobalTransform(NAME_None);
				}
			}
			return Cache * ComponentTransform;
		}
	}

	const FRigElementKey& ControlKey = ControlElement->GetKey();
	const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	const int32 CtrlIndex = Hierarchy->GetIndex(ControlKey);
	return Hierarchy->GetGlobalTransform(CtrlIndex) * ComponentTransform;
}

FTransform UTransformableControlHandle::GetLocalTransform() const
{
	const FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement)
	{
		return FTransform::Identity;
	}

	if (ControlRig->IsAdditive())
	{
		return ControlRig->GetControlLocalTransform(ControlName);
	}
	
	const FRigElementKey& ControlKey = ControlElement->GetKey();
	const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	const int32 CtrlIndex = Hierarchy->GetIndex(ControlKey);

	return Hierarchy->GetLocalTransform(CtrlIndex);
}

UObject* UTransformableControlHandle::GetPrerequisiteObject() const
{
	return GetBoundComponent(); 
}

FTickFunction* UTransformableControlHandle::GetTickFunction() const
{
	USceneComponent* BoundComponent = GetBoundComponent();
	return BoundComponent ? &BoundComponent->PrimaryComponentTick : nullptr;
}

uint32 UTransformableControlHandle::ComputeHash(const UControlRig* InControlRig, const FName& InControlName)
{
	return HashCombine(GetTypeHash(InControlRig), GetTypeHash(InControlName));
}

uint32 UTransformableControlHandle::GetHash() const
{
	if (ControlRig.IsValid() && ControlName != NAME_None)
	{
		return ComputeHash(ControlRig.Get(), ControlName);
	}
	return 0;
}

TWeakObjectPtr<UObject> UTransformableControlHandle::GetTarget() const
{
	return GetBoundComponent();
}

USceneComponent* UTransformableControlHandle::GetBoundComponent() const
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = GetSkeletalMesh())
	{
		return SkeletalMeshComponent;	
	}
	return GetControlRigComponent();
}

USkeletalMeshComponent* UTransformableControlHandle::GetSkeletalMesh() const
{
	const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig.IsValid() ? ControlRig->GetObjectBinding() : nullptr;
   	return ObjectBinding ? Cast<USkeletalMeshComponent>(ObjectBinding->GetBoundObject()) : nullptr;
}

UControlRigComponent* UTransformableControlHandle::GetControlRigComponent() const
{
	const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig.IsValid() ? ControlRig->GetObjectBinding() : nullptr;
	return ObjectBinding ? Cast<UControlRigComponent>(ObjectBinding->GetBoundObject()) : nullptr;
}

bool UTransformableControlHandle::HasIndirectDependencyWith(const UTransformableHandle& InOther) const
{
	const uint32 OtherHash = InOther.GetHash();
	if (OtherHash == 0 || OtherHash == GetHash())
	{
		return false;
	}
	
	FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement)
	{
		return false;
	}
	
	const UTransformableControlHandle* OtherCtrlHandle = InOther.IsValid() ? Cast<UTransformableControlHandle>(&InOther) : nullptr;
	const FRigControlElement* OtherControlElement = OtherCtrlHandle ? OtherCtrlHandle->GetControlElement() : nullptr;
	if (!OtherCtrlHandle || !OtherControlElement || OtherCtrlHandle->ControlRig != ControlRig)
	{
		return false;
	}

	// look for common ancestor
	const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	static constexpr bool bRecursive = true;	
	const FRigBaseElementParentArray& Parents = Hierarchy->GetParents(ControlElement, bRecursive);
	const FRigBaseElementParentArray& OtherParents = Hierarchy->GetParents(OtherControlElement, bRecursive);

	int32 Index = Parents.Num(), OtherIndex = OtherParents.Num();
	auto HasCommonAncestor = [&Index, &Parents, &OtherIndex, &OtherParents, OtherControlElement, ControlElement]()
	{
		if (!Parents.IsEmpty() && !OtherParents.IsEmpty())
		{
			FRigBaseElement* CommonAncestor = nullptr;
			while (Index > 0 && OtherIndex > 0 && Parents[Index-1] == OtherParents[OtherIndex-1])
			{
				CommonAncestor = Parents[Index-1];
				--Index;
				--OtherIndex;
			}
			return CommonAncestor != nullptr;
		}
		return false;
	};

	// if common ancestor, compare depths
	if (HasCommonAncestor() && Index != OtherIndex)
	{
		return OtherIndex < Index; 
	}

	// otherwise, compare indices
	if (OtherControlElement->GetIndex() < ControlElement->GetIndex())
	{
		return true;
	}

	return false;
}

bool UTransformableControlHandle::HasDirectDependencyWith(const UTransformableHandle& InOther) const
{
	const uint32 OtherHash = InOther.GetHash();
	if (OtherHash == 0)
	{
		return false;
	}

	// check whether the other handle is one of the skeletal mesh parent
	if (const USceneComponent* BoundComponent = GetBoundComponent())
	{
		if (GetTypeHash(BoundComponent) == OtherHash)
		{
			// we cannot constrain the skeletal mesh component to one of ControlRig's controls
			return true;
		}
		
		for (const USceneComponent* Comp=BoundComponent->GetAttachParent(); Comp!=nullptr; Comp=Comp->GetAttachParent() )
		{
			const uint32 AttachParentHash = GetTypeHash(Comp);
			if (AttachParentHash == OtherHash)
			{
				return true;
			}
		}
	}
	
	FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement)
	{
		return false;
	}

	UControlRig* ControlRigPtr = ControlRig.Get();
	
	// check whether the other handle is one of the control parent within the CR hierarchy
	static constexpr bool bRecursive = true;
	const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	const FRigBaseElementParentArray AllParents = Hierarchy->GetParents(ControlElement, bRecursive);
	const bool bIsParent = AllParents.ContainsByPredicate([ControlRigPtr, OtherHash](const FRigBaseElement* Parent)
	{
		const uint32 ParentHash = ComputeHash(ControlRigPtr, Parent->GetFName());
		return ParentHash == OtherHash;		
	});

	if (bIsParent)
	{
		return true;
	}

	// otherwise, check if there are any dependency in the graph that would cause a cycle
	const TArray<FRigControlElement*> AllControls = Hierarchy->GetControls();
	const int32 IndexOfPossibleParent = AllControls.IndexOfByPredicate([ControlRigPtr, OtherHash](const FRigBaseElement* Parent)
	{
		const uint32 ChildHash = ComputeHash(ControlRigPtr, Parent->GetFName());
		return ChildHash == OtherHash;
	});

	if (IndexOfPossibleParent != INDEX_NONE)
	{
		// at this point, we know that both handles belong to the same rig
		FRigControlElement* PossibleParent = AllControls[IndexOfPossibleParent];

		auto CanCauseCycle = [ControlElement, PossibleParent](UControlRig* InControlRig, const URigHierarchy* InRigHierarchy)
		{
			if (InControlRig && InRigHierarchy)
			{
			#if WITH_EDITOR
				FRigDependenciesProviderForControlRig DependencyProvider(InControlRig);
				DependencyProvider.SetInteractiveDialogEnabled(true);
				return InRigHierarchy->CanCauseCycle(PossibleParent, ControlElement, DependencyProvider);
            #else
				return InRigHierarchy->CanCauseCycle(PossibleParent, ControlElement);
            #endif	
			}
			return false;
		};
		
		// modular test
		if (UModularRig* ModularRig = Cast<UModularRig>(ControlRigPtr))
		{
			const FName ModuleName = Hierarchy->GetModuleFName(ControlElement->GetKey());
			const FRigModuleInstance* ModuleInstance = ModuleName != NAME_None ? ModularRig->FindModule(ModuleName) : nullptr;

			const FName ParentModuleName = Hierarchy->GetModuleFName(PossibleParent->GetKey());
			const FRigModuleInstance* ParentModuleInstance = ParentModuleName != NAME_None ? ModularRig->FindModule(ParentModuleName) : nullptr;

			if (ModuleInstance && ParentModuleInstance)
			{
				if (ModuleInstance == ParentModuleInstance)
				{
					// both handles are under the same module so check dependencies within that module
					UControlRig* ModuleRig = ModuleInstance->GetRig();
					const URigHierarchy* ModuleHierarchy = ModuleRig ? ModuleRig->GetHierarchy() : nullptr;
					
					const FRigBaseElementParentArray ParentParents = ModuleHierarchy->GetParents(PossibleParent, bRecursive);
					if (ParentParents.Contains(ControlElement))
					{
						// if ControlElement is a parent of PossibleParent then it can't be one of its children
						return false;		
					}

					// NOTE: we'd like to call this here but read URigHierarchy::GetDependenciesForVM about using this function with modular rigs
					// if (CanCauseCycle(ModuleRig, ModuleHierarchy))
					// {
					// 	return true;
					// }
				}
				else
				{
					FRigModuleInstance* ParentModule = ModuleInstance->CachedParentModule;
					while (ParentModule)
					{
						if (ParentModule == ParentModuleInstance)
						{
							return true;	
						}
						ParentModule = ParentModule->CachedParentModule; 
					}
				}
			}
		}

		// default control rig test
		if (CanCauseCycle(ControlRigPtr, Hierarchy))
		{
			return true;
		}
	}

	return false;
}

// if there's no skeletal mesh bound then the handle is not valid so no need to do anything else
FTickPrerequisite UTransformableControlHandle::GetPrimaryPrerequisite(const bool bAllowThis) const
{
	if (bAllowThis)
	{
		if (FTickFunction* TickFunction = GetTickFunction())
		{
			return FTickPrerequisite(GetBoundComponent(), *TickFunction); 
		}
	}
	
	static const FTickPrerequisite DummyPrerex;
	return DummyPrerex;
}

FRigControlElement* UTransformableControlHandle::GetControlElement() const
{
	if (!ControlRig.IsValid() || ControlName == NAME_None)
	{
		return nullptr;
	}

	return ControlRig->FindControl(ControlName);
}

const UE::Anim::FAnimationEvaluationTask& UTransformableControlHandle::GetEvaluationTask() const
{
	const FRigControlElement* Control = GetControlElement();
	if (!Control)
	{
		EvaluationTask = UE::Anim::FAnimationEvaluationTask();
		return EvaluationTask;
	}

	USkeletalMeshComponent* SkeletalMeshComponent = GetSkeletalMesh();
	if (!SkeletalMeshComponent)
	{
		EvaluationTask = UE::Anim::FAnimationEvaluationTask();
		return EvaluationTask;
	}

	if (!EvaluationTask.Guid.IsValid())
	{
		EvaluationTask.Guid = FGuid::NewGuid();
	}

	if (EvaluationTask.SkeletalMeshComponent != SkeletalMeshComponent)
	{
		EvaluationTask.SkeletalMeshComponent = SkeletalMeshComponent;

		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		const int32 CtrlIndex = Hierarchy->GetIndex(Control->GetKey());
		EvaluationTask.PostEvaluationFunction = [&CacheRef = Cache, CtrlIndex, WeakHierarchy = TWeakObjectPtr<URigHierarchy>(Hierarchy)]()
		{
			const URigHierarchy* Hierarchy = WeakHierarchy.IsValid() ? WeakHierarchy.Get() : nullptr;
			if (Hierarchy && CtrlIndex > INDEX_NONE)
			{
				CacheRef = Hierarchy->GetGlobalTransform(CtrlIndex);
			}
		};
	}
	
	return EvaluationTask;
}

void UTransformableControlHandle::UnregisterDelegates() const
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif
	
	if (ControlRig.IsValid())
	{
		if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->OnModified().RemoveAll(this);
		}
		ControlRig->ControlModified().RemoveAll(this);

		if (const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding())
		{
			Binding->OnControlRigBind().RemoveAll(this);
		}
		ControlRig->ControlRigBound().RemoveAll(this);
	}
}

void UTransformableControlHandle::RegisterDelegates()
{
	UnregisterDelegates();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UTransformableControlHandle::OnObjectsReplaced);
#endif

	// make sure the CR is loaded so that we can register delegates
	if (ControlRig.IsPending())
	{
		ControlRig.LoadSynchronous();
	}
	
	if (ControlRig.IsValid())
	{
		UE::Private::FixModularRigControlName(ControlRig.Get(), this);
		
		if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->OnModified().AddUObject(this, &UTransformableControlHandle::OnHierarchyModified);
		}

		// NOTE BINDER: this has to be done before binding UTransformableControlHandle::OnControlModified
		if (!ControlRig->ControlModified().IsBoundToObject(&GetEvaluationBinding()))
		{
			ControlRig->ControlModified().AddRaw(&GetEvaluationBinding(), &FControlEvaluationGraphBinding::HandleControlModified);
		}
		
		ControlRig->ControlModified().AddUObject(this, &UTransformableControlHandle::OnControlModified);
		if (!ControlRig->ControlRigBound().IsBoundToObject(this))
		{
			ControlRig->ControlRigBound().AddUObject(this, &UTransformableControlHandle::OnControlRigBound);
		}
		OnControlRigBound(ControlRig.Get());
	}
}

void UTransformableControlHandle::OnHierarchyModified(
	ERigHierarchyNotification InNotif,
	URigHierarchy* InHierarchy,
	const FRigNotificationSubject& InSubject)
{
	if (!ControlRig.IsValid())
	{
	 	return;
	}

	const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (!Hierarchy || InHierarchy != Hierarchy)
	{
		return;
	}

	const FRigBaseElement* InElement = InSubject.Element;

	switch (InNotif)
	{
		case ERigHierarchyNotification::ElementAdded:
		{
			UE::Private::FixModularRigControlName(ControlRig.Get(), this);
			break;
		}
		case ERigHierarchyNotification::ElementRemoved:
		{
			// FIXME this leaves the constraint invalid as the element won't exist anymore
			// find a way to remove this from the constraints list 
			break;
		}
		case ERigHierarchyNotification::ElementRenamed:
		{
			const FName OldName = Hierarchy->GetPreviousName(InElement->GetKey());
			if (OldName == ControlName)
			{
				ControlName = InElement->GetFName();
			}
			break;
		}
		default:
			break;
	}
}

void UTransformableControlHandle::OnControlModified(
	UControlRig* InControlRig,
	FRigControlElement* InControl,
	const FRigControlModifiedContext& InContext)
{
	if (!InControlRig || !InControl || ControlName == NAME_None)
	{
		return;
	}

	if (bNotifying)
	{
		return;
	}

	const bool bThisControl = InControl->GetFName() == ControlName;
	const EHandleEvent Event = InContext.bConstraintUpdate ? EHandleEvent::GlobalTransformUpdated : EHandleEvent::LocalTransformUpdated;
	if (Event == EHandleEvent::LocalTransformUpdated && !bThisControl)
	{
		return;
	}
	
	UControlRig* Rig = ControlRig.Get();
	if (!Rig || Rig != InControlRig)
	{
		return;
	}

	if (HandleModified().IsBound())
	{
		if (bThisControl)
		{
			UE::TScopeLock ScopeLock(UE::Private::NotificationLock);
			
			// if that handle is wrapping InControl
			if (InContext.bConstraintUpdate)
			{
				GetEvaluationBinding().bPendingFlush = true;
			}
			
			// guard from re-entrant notification
			const UE::Private::RigGuard NotificationGuard([Rig]()
			{
				UE::Private::NotifyingRigs.Remove(Rig);
			});
			UE::Private::NotifyingRigs.Add(Rig);
			
			Notify(Event);

			if (TransformableHandleUtils::SkipTicking())
			{
				TransformableHandleUtils::MarkComponentForEvaluation(GetSkeletalMesh());
			}
		}
		else if (Event == EHandleEvent::GlobalTransformUpdated)
		{
			// the control being modified is not the one wrapped by this handle 
			if (const FRigControlElement* Control = ControlRig->FindControl(ControlName))
			{
				UE::TScopeLock ScopeLock(UE::Private::NotificationLock);
			
				if (InContext.bConstraintUpdate)
				{
					GetEvaluationBinding().bPendingFlush = true;
				}

				// guard from re-entrant notification 
				const UE::Private::RigGuard NotificationGuard([Rig]()
				{
					UE::Private::NotifyingRigs.Remove(Rig);
				});
				UE::Private::NotifyingRigs.Add(Rig);

				const UTickableConstraint* TickableConstraint = GetTypedOuter<UTickableConstraint>();
				const bool bIsConstraintActive = TickableConstraint && TickableConstraint->IsFullyActive();
				const bool bPreTick = !ControlRig->IsAdditive() && bIsConstraintActive;
				Notify(EHandleEvent::UpperDependencyUpdated, bPreTick);

				if (TransformableHandleUtils::SkipTicking())
				{
					TransformableHandleUtils::MarkComponentForEvaluation(GetSkeletalMesh());
				}
			}
		}
	}
}

void UTransformableControlHandle::UpdateControlRig(UControlRig* InControlRig)
{
	if (InControlRig && InControlRig != ControlRig)
	{
		UnregisterDelegates();
		ControlRig = InControlRig;
		RegisterDelegates();
	}
}

void UTransformableControlHandle::OnControlRigBound(UControlRig* InControlRig)
{
	if (!InControlRig)
	{
		return;
	}

	if (!ControlRig.IsValid() || ControlRig != InControlRig)
	{
		return;
	}

	if (const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding())
	{
		if (!Binding->OnControlRigBind().IsBoundToObject(this))
		{
			Binding->OnControlRigBind().AddUObject(this, &UTransformableControlHandle::OnObjectBoundToControlRig);
		}
	}
}

void UTransformableControlHandle::OnObjectBoundToControlRig(UObject* InObject)
{
	if (!ControlRig.IsValid() || !InObject)
	{
		return;
	}

	const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
	const UObject* CurrentObject = Binding ? Binding->GetBoundObject() : nullptr;
	if (CurrentObject == InObject)
	{
		const UWorld* World = GetWorld();
		if (!World)
		{
			if (const USceneComponent* BoundComponent = GetBoundComponent())
			{
				World = BoundComponent->GetWorld();
			}
		}
		
		if (World && InObject->GetWorld() == World)
		{
			Notify(EHandleEvent::ComponentUpdated);
		}
	}
}

TArrayView<FMovieSceneFloatChannel*>  UTransformableControlHandle::GetFloatChannels(const UMovieSceneSection* InSection) const
{
	return FControlRigSequencerHelpers::GetFloatChannels(ControlRig.Get(),
		ControlName, InSection);
}

TArrayView<FMovieSceneDoubleChannel*>  UTransformableControlHandle::GetDoubleChannels(const UMovieSceneSection* InSection) const
{
	static const TArrayView<FMovieSceneDoubleChannel*> EmptyChannelsView;
	return EmptyChannelsView;
}

bool UTransformableControlHandle::AddTransformKeys(const TArray<FFrameNumber>& InFrames,
	const TArray<FTransform>& InTransforms,
	const EMovieSceneTransformChannel& InChannels,
	const FFrameRate& InTickResolution,
	UMovieSceneSection* InSection,
	const bool bLocal) const
{
	if (!ControlRig.IsValid() || ControlName == NAME_None || !ControlRig->GetHierarchy())
	{
		return false;
	}

	FRigControlElement* ControlElement = ControlRig.Get()->FindControl(ControlName);
	if(!ControlElement)
	{
		return false;
	}
	
	const int32 NumFrames = InFrames.Num();
	const int32 NumTransforms = InTransforms.Num();
	const bool bUpdateDefault = NumFrames == 0 && NumTransforms == 1;
	if (NumFrames == 0)
	{
		if (!bUpdateDefault)
		{
			return false;
		}
	}
	else if (NumTransforms != NumFrames)
	{
		return false;
	}

	auto KeyframeFunc = [this, bLocal](const FTransform& InTransform, const FRigControlModifiedContext& InKeyframeContext, FRigControlElement* InControlElement)
	{
		UControlRig* InControlRig = ControlRig.Get();
		static constexpr bool bNotify = true;
		static constexpr bool bUndo = false;
		static constexpr bool bFixEuler = true;

		if (bLocal)
		{
			const FVector EulerAngle = InTransform.GetRotation().Rotator().Euler();
			
			InControlRig->GetHierarchy()->SetControlSpecifiedEulerAngle(InControlElement, EulerAngle);

			InControlRig->SetControlLocalTransform(ControlName, InTransform, bNotify, InKeyframeContext, bUndo, bFixEuler);
			if (InControlRig->IsAdditive())
			{
				InControlRig->Evaluate_AnyThread();
			}
			return;
		}
		
		InControlRig->SetControlGlobalTransform(ControlName, InTransform, bNotify, InKeyframeContext, bUndo, bFixEuler);
		if (InControlRig->IsAdditive())
		{
			InControlRig->Evaluate_AnyThread();
		}
	};

	FRigControlModifiedContext KeyframeContext;
	KeyframeContext.KeyMask = static_cast<uint32>(InChannels);
	
	const FScopedLegacyEvaluationTrack ScopedLegacyEvaluationTrack(InSection);
	
	if (bUpdateDefault)
	{
		KeyframeFunc(InTransforms[0], KeyframeContext, ControlElement);
	}
	else
	{
		KeyframeContext.SetKey = EControlRigSetKey::Always;

		for (int32 Index = 0; Index < InFrames.Num(); ++Index)
		{
			const FFrameNumber& Frame = InFrames[Index];
			KeyframeContext.LocalTime = InTickResolution.AsSeconds(FFrameTime(Frame));

			KeyframeFunc(InTransforms[Index], KeyframeContext, ControlElement);
		}
		
	}

	return true;
}

//for control rig need to check to see if the control rig is different then we may need to update it based upon what we are now bound to
void UTransformableControlHandle::ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject)
{
	if (UControlRig* InControlRig = Cast<UControlRig>(SubObject))
	{
		// nothing to do
		if (ControlRig == InControlRig)
		{
			return;
		}

		// skip resolving if the rigs don't share the same class type 
		if (ControlRig && ControlRig->GetAssetReference() != InControlRig->GetAssetReference())
		{
			return;
		}
		
		//just do one
		if (auto BoundObjectsView = ConstraintBindingID.ResolveBoundObjects(LocalSequenceID, SharedPlaybackState); BoundObjectsView.Num() > 0)
		{
			const TWeakObjectPtr<> ParentObject = BoundObjectsView[0];
			const UObject* Bindable = FControlRigObjectBinding::GetBindableObject(ParentObject.Get());
			if (InControlRig->GetObjectBinding() && InControlRig->GetObjectBinding()->GetBoundObject() == Bindable)
			{
				UpdateControlRig(InControlRig);
			}
		}
	}
}

UTransformableHandle* UTransformableControlHandle::Duplicate(UObject* NewOuter) const
{
	UTransformableControlHandle* HandleCopy = DuplicateObject<UTransformableControlHandle>(this, NewOuter, GetFName());
	HandleCopy->ControlRig = ControlRig;
	HandleCopy->ControlName = ControlName;
	return HandleCopy;
}
#if WITH_EDITOR

FString UTransformableControlHandle::GetLabel() const
{
	const USceneComponent* BoundComponent = GetBoundComponent();
	if (!BoundComponent)
	{
		if (ControlRig.IsValid())
		{
			const FString ControlRigLabel = ControlRig->GetName();
			if (const FRigControlElement* Control = ControlRig->FindControl(ControlName))
			{
				return FString::Printf(TEXT("%s(unbound)/%s"), *ControlRigLabel, *Control->GetName() );	
			}
			return FString::Printf(TEXT("%s(unbound)/%s(not found)"), *ControlRigLabel, *ControlName.ToString() );			
		}
		
		static const FString DummyLabel;
		return DummyLabel;
	}
	
	return ControlName.ToString();
}

FString UTransformableControlHandle::GetFullLabel() const
{
	const USceneComponent* BoundComponent = GetBoundComponent();
	if (!BoundComponent)
	{
		if (ControlRig.IsValid())
		{
			const FString ControlRigLabel = ControlRig->GetName();
			if (const FRigControlElement* Control = ControlRig->FindControl(ControlName))
			{
				return FString::Printf(TEXT("%s(unbound)/%s"), *ControlRigLabel, *Control->GetName() );	
			}
			return FString::Printf(TEXT("%s(unbound)/%s(not found)"), *ControlRigLabel, *ControlName.ToString() );			
		}
		
		static const FString DummyLabel;
		return DummyLabel;
	}

	if (ControlRig.IsValid())
	{
		const AActor* Actor = BoundComponent->GetOwner();
		const FString BoundLabel = Actor ? Actor->GetActorLabel() : BoundComponent->GetName();
		return FString::Printf(TEXT("%s/%s/%s"), *BoundLabel, *ControlRig->GetName(), *ControlName.ToString() );
	}
	
	const AActor* Actor = BoundComponent->GetOwner();
	const FString BoundLabel = Actor ? Actor->GetActorLabel() : BoundComponent->GetName();
	return FString::Printf(TEXT("%s/%s"), *BoundLabel, *ControlName.ToString() );
}

void UTransformableControlHandle::OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances)
{
	if (UObject* NewObject = InOldToNewInstances.FindRef(ControlRig.Get()))
	{
		if (UControlRig* NewControlRig = Cast<UControlRig>(NewObject))
		{
			UpdateControlRig(NewControlRig);
		}
	}
}

#endif

FControlEvaluationGraphBinding& UTransformableControlHandle::GetEvaluationBinding()
{
	static FControlEvaluationGraphBinding EvaluationBinding;
	return EvaluationBinding;
}

void FControlEvaluationGraphBinding::HandleControlModified(UControlRig* InControlRig, FRigControlElement* InControl, const FRigControlModifiedContext& InContext)
{
	if (!bPendingFlush || !InContext.bConstraintUpdate)
	{
		return;
	}
	
	if (!InControlRig || !InControl)
	{
		return;
	}

	// flush all pending evaluations if any
	if (UWorld* World = InControlRig->GetWorld())
	{
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.FlushEvaluationGraph();
	}
	bPendingFlush = false;
}
