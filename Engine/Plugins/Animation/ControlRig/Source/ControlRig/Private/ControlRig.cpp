// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRig.h"
#include "Engine/SkeletalMesh.h"
#include "GameFramework/Actor.h"
#include "Misc/RuntimeErrors.h"
#include "Misc/ScopeLock.h"
#include "IControlRigObjectBinding.h"
#include "HelperUtil.h"
#include "ObjectTrace.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "ControlRigObjectVersion.h"
#include "Rigs/RigHierarchyController.h"
#include "Rigs/RigPhysics.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Units/Execution/RigUnit_InteractionExecution.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimationPoseData.h"
#include "Internationalization/TextKey.h"
#include "UObject/Package.h"
#if WITH_EDITOR
#include "ControlRigModule.h"
#include "Modules/ModuleManager.h"
#include "RigVMModel/RigVMNode.h"
#include "Engine/Blueprint.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMDeveloperTypeUtils.h"
#include "Editor.h"
#include "RigVMCompiler/RigVMCompiler.h"
#endif// WITH_EDITOR
#include "ControlRigBlueprintGeneratedClass.h"
#include "ControlRigComponent.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "RigVMCore/RigVMNativized.h"
#include "UObject/UObjectIterator.h"
#include "RigVMCore/RigVMAssetUserData.h"
#include "ModularRig.h"
#include "Units/Modules/RigUnit_ConnectorExecution.h"
#include "ControlRigReplay.h"
#include "CoreGlobals.h"
#include "ControlRigRuntimeAsset.h"
#include "Rigs/FKControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRig)

#define LOCTEXT_NAMESPACE "ControlRig"

DEFINE_LOG_CATEGORY(LogControlRig);

DECLARE_STATS_GROUP(TEXT("ControlRig"), STATGROUP_ControlRig, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Control Rig Execution"), STAT_RigExecution, STATGROUP_ControlRig, );
DEFINE_STAT(STAT_RigExecution);

const FName UControlRig::OwnerComponent("OwnerComponent");

//CVar to specify if we should create a float control for each curve in the curve container
//By default we don't but it may be useful to do so for debugging
static TAutoConsoleVariable<int32> CVarControlRigCreateFloatControlsForCurves(
	TEXT("ControlRig.CreateFloatControlsForCurves"),
	0,
	TEXT("If nonzero we create a float control for each curve in the curve container, useful for debugging low level controls."),
	ECVF_Default);

//CVar to specify if we should allow debug drawing in game (during shipped game or PIE)
//By default we don't for performance 
static TAutoConsoleVariable<float> CVarControlRigEnableDrawInterfaceInGame(
	TEXT("ControlRig.EnableDrawInterfaceInGame"),
	0,
	TEXT("If nonzero debug drawing will be enabled during play."),
	ECVF_Default);


UControlRig::UControlRig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITOR
	, bIsRunningInPIE(false)
	, bEnableAnimAttributeTrace(false)
#endif 
	, DataSourceRegistry(nullptr)
#if WITH_EDITOR
	, PreviewInstance(nullptr)
#endif
	, bCopyHierarchyBeforeConstruction(true)
	, bResetInitialTransformsBeforeConstruction(true)
	, bHierarchyInitializedThisFrame(false)
	, bResetCurrentTransformsAfterConstruction(false)
	, bManipulationEnabled(false)
	, PreConstructionBracket(0)
	, PostConstructionBracket(0)
	, InteractionBracket(0)
	, InterRigSyncBracket(0)
	, ControlUndoBracketIndex(0)
	, InteractionType((uint8)EControlRigInteractionType::None)
	, bEvaluationTriggeredFromInteraction(false)
	, bInteractionJustBegan(false)
	, bIsAdditive(false)
	, bOnlyRunConstruction(false)
	, DebugBoneRadiusMultiplier(1.f)
#if WITH_EDITOR
	, bRecordSelectionPoseForConstructionMode(true)
	, bIsClearingTransientControls(false)
#endif
	, ReplayTimeIndex(INDEX_NONE)
{
	EventQueue.Add(FRigUnit_BeginExecution::EventName);

	SetRigVMExtendedExecuteContext(&RigVMExtendedExecuteContext);
}

#if WITH_EDITOR

void UControlRig::ResetRecordedTransforms(const FName& InEventName)
{
	if(const URigHierarchy* Hierarchy = GetHierarchy())
	{
		if(Hierarchy->bRecordInstructionsAtRuntime)
		{
			FControlRigExecuteContext& PublicData = GetRigVMExtendedExecuteContext().GetPublicData<FControlRigExecuteContext>();
			PublicData.ResetInstructionRecords();
		}
	}
}

#endif

void UControlRig::BeginDestroy()
{
	BeginDestroyEvent.Broadcast(this);
	BeginDestroyEvent.Clear();
	
	Super::BeginDestroy();
	SetRigVMExtendedExecuteContext(nullptr);

	PreConstructionEvent.Clear();
	PostConstructionEvent.Clear();
	PreForwardsSolveEvent.Clear();
	PostForwardsSolveEvent.Clear();
	PreAdditiveValuesApplicationEvent.Clear();

#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject) && !IsEngineExitRequested())
	{
		if(const UClass* Class = GetClass())
		{
			if(IsValid(Class))
			{
				URigHierarchy* DefaultHierarchy = nullptr;
				if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
				{
					if (!IsGarbageOrDestroyed(RuntimeAsset.GetObject()))
					{
						DefaultHierarchy = RuntimeAsset->GetHierarchy();
					}
				}
				else if(UControlRig* CDO = Class->GetDefaultObject<UControlRig>())
				{
					if (!IsGarbageOrDestroyed(CDO))
					{
						DefaultHierarchy = CDO->GetHierarchy();
					}
				}
				
				if (DefaultHierarchy)
				{
					DefaultHierarchy->UnregisterListeningHierarchy(GetHierarchy());
				}
			}
		}
	}
#endif

	TRACE_OBJECT_LIFETIME_END(this);
}

int32 UControlRig::NumOverrideAssets() const
{
	return OverrideAssets.Num();
}

const UControlRigOverrideAsset* UControlRig::GetLastOverrideAsset() const
{
	return OverrideAssets.IsEmpty() ? nullptr : OverrideAssets.Last();
}

UControlRigOverrideAsset* UControlRig::GetLastOverrideAsset()
{
	return OverrideAssets.IsEmpty() ? nullptr : OverrideAssets.Last();
}

const UControlRigOverrideAsset* UControlRig::GetOverrideAsset(int32 InIndex) const
{
	return OverrideAssets.IsValidIndex(InIndex) ? OverrideAssets[InIndex] : nullptr;
}

UControlRigOverrideAsset* UControlRig::GetOverrideAsset(int32 InIndex)
{
	return OverrideAssets.IsValidIndex(InIndex) ? OverrideAssets[InIndex] : nullptr;
}

int32 UControlRig::LinkOverrideAsset(UControlRigOverrideAsset* InAsset)
{
	if (!InAsset)
	{
		return INDEX_NONE;
	}

	if (OverrideAssets.Contains(InAsset))
	{
		return INDEX_NONE;
	}
	
	const int32 Index = OverrideAssets.Add(InAsset);
	InAsset->OnChanged().AddUObject(this, &UControlRig::HandleOverrideChanged);
	OverrideAssetsChangedDelegate.Broadcast(this);
	RequestConstruction();
	return Index;
}

bool UControlRig::UnlinkOverrideAsset(UControlRigOverrideAsset* InAsset)
{
	const int32 NumRemoved = OverrideAssets.Remove(const_cast<UControlRigOverrideAsset*>(InAsset));
	if (NumRemoved == 0)
	{
		return false;
	}
	
	if (InAsset)
	{
		InAsset->OnChanged().RemoveAll(this);
	}
		
	OverrideAssetsChangedDelegate.Broadcast(this);
	RequestConstruction();
	
	return true;
}

bool UControlRig::UnlinkAllOverrideAssets()
{
	if (OverrideAssets.IsEmpty())
	{
		return false;
	}

	for (const TObjectPtr<UControlRigOverrideAsset>& OverrideAsset : OverrideAssets)
	{
		OverrideAsset->OnChanged().RemoveAll(this);
	}
	OverrideAssets.Reset();
	OverrideAssetsChangedDelegate.Broadcast(this);
	RequestConstruction();
	
	return true;
}

bool UControlRig::IsLinkedToOverrideAsset(const UControlRigOverrideAsset* InOverrideAsset) const
{
	return OverrideAssets.Contains(InOverrideAsset);
}

void UControlRig::ApplyElementOverrides()
{
	if (!CVarControlRigEnableOverrides.GetValueOnAnyThread())
	{
		return;
	}

	if (IsReadOnly())
	{
		return;
	}
	
	URigHierarchy* Hierarchy = GetHierarchy();
	if(!Hierarchy)
	{
		return;
	}
	
	TSet<FRigControlElement*> ControlsChanged;
	
	for (const UControlRigOverrideAsset* OverrideAsset : OverrideAssets)
	{
		if (!OverrideAsset)
		{
			continue;
		}

		for (const FControlRigOverrideValue& Override : OverrideAsset->Overrides)
		{
			const FRigElementKey Key(Override.GetSubjectKey(), ERigElementType::Control);
			FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key);
			if(!ControlElement)
			{
				continue;
			}
			
			const FProperty* ShapeTransformProperty =
				FRigControlSettings::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FRigControlSettings, ShapeTransform));
			const bool bIsTransformShapeOverride = Override.ContainsProperty(ShapeTransformProperty);
			if (bIsTransformShapeOverride)
			{
				ControlElement->Settings.ShapeTransform = Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal);
			}

			// in the future let's avoid this check know on a container level
			// if this container represents changes to the control
			if (!Override.CopyToSubject(ControlElement, FRigControlElement::StaticStruct()))
			{
				continue;
			}
			ControlsChanged.Add(ControlElement);

			if (bIsTransformShapeOverride)
			{
				Hierarchy->SetControlShapeTransformByIndex(ControlElement->GetIndex(), ControlElement->Settings.ShapeTransform, true);
				Hierarchy->SetControlShapeTransformByIndex(ControlElement->GetIndex(), ControlElement->Settings.ShapeTransform, false);
			}
		}
	}
}

void UControlRig::HandleOverrideChanged(const UControlRigOverrideAsset* InOverrideAsset)
{
	if (!bSuspendOverrideAssetChangedDelegate&& InOverrideAsset && OverrideAssets.Contains(InOverrideAsset))
	{
		RequestConstruction();
	}
}

UWorld* UControlRig::GetWorld() const
{
	if (ObjectBinding.IsValid())
	{
		AActor* HostingActor = ObjectBinding->GetHostingActor();
		if (HostingActor)
		{
			return HostingActor->GetWorld();
		}

		UObject* Owner = ObjectBinding->GetBoundObject();
		if (Owner)
		{
			return Owner->GetWorld();
		}
	}
	return Super::GetWorld();
}

void UControlRig::Initialize(bool bRequestInit)
{
	TRACE_OBJECT_LIFETIME_BEGIN_WITH_OUTER(this, GetBoundOuterForTrace());

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ControlRig_Initialize);

	// Runtime-asset-backed rigs loaded from disk might arrive here with DynamicHierarchy == nullptr:
	// PostInitProperties runs before GeneratedBy is serialized, so PostInitInstance is skipped
	// at construction and PostLoad does not make up for it. InitializeVMs below would create
	// the hierarchy (via InitializeFromCDO → PostInitInstanceIfRequired), but we access
	// GetHierarchy() earlier than that, so ensure it's ready up front.
	PostInitInstanceIfRequired();

	if (!IsConstructionRequired())
	{
		SelectionBeforeInit = GetHierarchy()->GetSelectedHierarchyKeys();
	}

	InitializeVMs(bRequestInit);

	// Create the data source registry here to avoid UObject creation from Non-Game Threads
	GetDataSourceRegistry();

	// Create the Hierarchy Controller here to avoid UObject creation from Non-Game Threads
	if(!IsRigModuleInstance())
	{
		GetHierarchy()->GetController(true);
	}

#if WITH_EDITOR
	bIsRunningInPIE = GEditor && GEditor->IsPlaySessionInProgress();
#endif
	
	// should refresh mapping 
	RequestConstruction();

	if(!IsRigModuleInstance())
	{
		GetHierarchy()->OnModified().RemoveAll(this);
		GetHierarchy()->OnModified().AddUObject(this, &UControlRig::HandleHierarchyModified);
		GetHierarchy()->OnEventReceived().RemoveAll(this);
		GetHierarchy()->OnEventReceived().AddUObject(this, &UControlRig::HandleHierarchyEvent);
		GetHierarchy()->UpdateVisibilityOnProxyControls();
	}
}

void UControlRig::RestoreShapeLibrariesFromCDO()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ShapeLibraryNameMap.Reset();
		if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
		{
			ShapeLibraries = RuntimeAsset->GetShapeLibraries();
		}
		else if(UControlRig* CDO = GetClass()->GetDefaultObject<UControlRig>())
		{
			ShapeLibraries = CDO->ShapeLibraries;
		}
	}
}

void UControlRig::OnAddShapeLibrary(const FControlRigExecuteContext* InContext, const FString& InLibraryName, UControlRigShapeLibrary* InShapeLibrary, bool bLogResults)
{
	// don't ever change the CDO
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if(InShapeLibrary == nullptr)
	{
		if(InContext)
		{
			static constexpr TCHAR Message[] = TEXT("No shape library provided.");
			InContext->Log(EMessageSeverity::Error, Message);
		}
		return;
	}

	// remove the shape library in case it was there before
	ShapeLibraryNameMap.Remove(InShapeLibrary->GetName());
	ShapeLibraries.Remove(InShapeLibrary);

	// if we've removed all shape libraries - let's add the ones from the CDO back
	if (ShapeLibraries.IsEmpty())
	{
		RestoreShapeLibrariesFromCDO();
	}

	// if we are supposed to replace the library and the library name is empty
	FString LibraryName = InLibraryName;
	if(LibraryName.IsEmpty())
	{
		LibraryName = InShapeLibrary->GetName();
	}

	if(LibraryName != InShapeLibrary->GetName())
	{
		ShapeLibraryNameMap.FindOrAdd(InShapeLibrary->GetName()) = LibraryName;
	}
	ShapeLibraries.AddUnique(InShapeLibrary);

#if WITH_EDITOR
	if(bLogResults)
	{
		static constexpr TCHAR MapFormat[] = TEXT("Control Rig '%s': Shape Library Name Map: '%s' -> '%s'");
		static constexpr TCHAR LibraryFormat[] = TEXT("Control Rig '%s': Shape Library '%s' uses asset '%s'");
		static constexpr TCHAR DefaultShapeFormat[] = TEXT("Control Rig '%s': Shape Library '%s' has default shape '%s'");
		static constexpr TCHAR ShapeFormat[] = TEXT("Control Rig '%s': Shape Library '%s' contains shape %03d: '%s'");
		static constexpr TCHAR ResolvedShapeFormat[] = TEXT("Control Rig '%s': ShapeName '%s' resolved to '%s.%s'");

		const FString PathName = GetPathName();

		for(const TPair<FString, FString>& Pair: ShapeLibraryNameMap)
		{
			UE_LOG(LogControlRig, Display, MapFormat, *PathName, *Pair.Key, *Pair.Value);
		}

		const int32 NumShapeLibraries = ShapeLibraries.Num();

		TMap<FString, const UControlRigShapeLibrary*> ShapeNameToShapeLibrary;
		for(int32 Index = 0; Index <NumShapeLibraries; Index++)
		{
			const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary = ShapeLibraries[Index];
			if(ShapeLibrary.IsNull())
			{
				continue;
			}

			UE_LOG(LogControlRig, Display, LibraryFormat, *PathName, *ShapeLibrary->GetName(), *ShapeLibrary->GetPathName());

			const bool bUseNameSpace = ShapeLibraries.Num() > 1;
			const FString DefaultShapeName = UControlRigShapeLibrary::GetShapeName(ShapeLibrary.Get(), bUseNameSpace, ShapeLibraryNameMap, ShapeLibrary->DefaultShape);
			UE_LOG(LogControlRig, Display, DefaultShapeFormat, *PathName, *ShapeLibrary->GetName(), *DefaultShapeName);

			for(int32 ShapeIndex = 0; ShapeIndex < ShapeLibrary->Shapes.Num(); ShapeIndex++)
			{
				const FString ShapeName = UControlRigShapeLibrary::GetShapeName(ShapeLibrary.Get(), bUseNameSpace, ShapeLibraryNameMap, ShapeLibrary->Shapes[ShapeIndex]);
				UE_LOG(LogControlRig, Display, ShapeFormat, *PathName, *ShapeLibrary->GetName(), ShapeIndex, *ShapeName);
				ShapeNameToShapeLibrary.FindOrAdd(ShapeName, nullptr) = ShapeLibrary.Get();
			}
		}

		for(const TPair<FString, const UControlRigShapeLibrary*>& Pair : ShapeNameToShapeLibrary)
		{
			FString Left, Right = Pair.Key;
			(void)Pair.Key.Split(TEXT("."), &Left, &Right, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			UE_LOG(LogControlRig, Display, ResolvedShapeFormat, *PathName, *Pair.Key, *Pair.Value->GetName(), *Right);
		}
	}
#endif
}

bool UControlRig::OnShapeExists(const FName& InShapeName) const
{
	const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& Libraries = GetShapeLibraries();
	if (UControlRigShapeLibrary::GetShapeByName(InShapeName, GetShapeLibraries(), ShapeLibraryNameMap, false))
	{
		return true;
	}
	
	return false;
}

bool UControlRig::InitializeVM(const FName& InEventName)
{
	if(!InitializeVMs(InEventName))
	{
		return false;
	}

	RequestConstruction();

#if WITH_EDITOR
	// setup the hierarchy's controller log function
	if (URigHierarchyController* HierarchyController = GetHierarchy()->GetController(true))
	{
		HierarchyController->LogFunction = [this](EMessageSeverity::Type InSeverity, const FString& Message)
		{
			const FRigVMExecuteContext& PublicContext = GetRigVMExtendedExecuteContext().GetPublicData<>();
			if(RigVMLog)
			{
				RigVMLog->Report(InSeverity,PublicContext.GetFunctionName(),PublicContext.GetInstructionIndex(), Message);
			}
			else
			{
				LogOnce(InSeverity,PublicContext.GetInstructionIndex(), Message);
			}
		};
	}
#endif

	return true;
}

void UControlRig::Evaluate_AnyThread()
{
	if (bIsAdditive)
	{
		if (bRequiresInitExecution)
		{
			Super::Evaluate_AnyThread();
		}

		if (bOnlyRunConstruction)
		{
			return;
		}
		
		// we can have other systems trying to poke into running instances of Control Rigs
		// on the anim thread and query data, such as
		// URigVMHostSkeletalMeshComponent::RebuildDebugDrawSkeleton,
		// using a lock here to prevent them from having an inconsistent view of the rig at some
		// intermediate stage of evaluation, for example, during evaluate, we can have a call
		// to copy hierarchy, which empties the hierarchy for a short period of time
		// and we don't want other systems to see that.
		UE::TScopeLock EvaluateLock(GetEvaluateMutex());

		URigHierarchy* Hierarchy = GetHierarchy();

		// Reset all control local transforms to initial, and switch to default parents
		if (!IsReadOnly())
		{
			Hierarchy->ForEach([Hierarchy](FRigBaseElement* Element) -> bool
			{
				if (FRigControlElement* Control = Cast<FRigControlElement>(Element))
				{
					if (Control->CanTreatAsAdditive())
					{
						if (Hierarchy->GetActiveParent(Control->GetKey()) != URigHierarchy::GetDefaultParentKey())
						{
							Hierarchy->SwitchToDefaultParent(Control);
						}
						Hierarchy->SetTransform(Control, Hierarchy->GetTransform(Control, ERigTransformType::InitialLocal), ERigTransformType::CurrentLocal, true);
					}
				}
				else if (FRigNullElement* NullElement = Cast<FRigNullElement>(Element))
				{
					Hierarchy->SetTransform(NullElement, Hierarchy->GetTransform(NullElement, ERigTransformType::InitialLocal), ERigTransformType::CurrentLocal, true);
				}
				return true;
			});
		}
		
		if (PoseBeforeBackwardsSolve.Num() == 0)
		{
			// If the pose is empty, this is an indication that a new pose is coming in
			uint8 Types = (uint8)ERigElementType::Bone | (uint8)ERigElementType::Curve;
			PoseBeforeBackwardsSolve = Hierarchy->GetPose(false, (ERigElementType)Types, TArrayView<const FRigElementKey>());
		}
		else if (!IsReadOnly())
		{
			// Restore the pose from the anim sequence
			Hierarchy->SetPose(PoseBeforeBackwardsSolve);
		}

		// Let's set up a queue so that the rig knows when to reset things. We manually
		// run the events in the queue here - but the queue is still required to make sure
		// the delta time isn't reset, logs are being collected correctly etc.
		// Note: It's ok that the interaction event is in here - since we are manually
		// running the queue below it won't be run if there's no interaction.
		static const TArray<FName> AdditiveControlRigQueue = {
			FRigUnit_InverseExecution::EventName,
			FRigUnit_InteractionExecution::EventName,
			FRigUnit_PreBeginExecution::EventName,
			FRigUnit_BeginExecution::EventName,
			FRigUnit_PostBeginExecution::EventName
		};
		const TGuardValue<TArray<FName>> EventQueueGuard(EventQueueToRun, AdditiveControlRigQueue);
		
		// Backwards solve
		{
			Execute(FRigUnit_InverseExecution::EventName);
		}

		// Switch parents
		if (!IsReadOnly())
		{
			for (TPair<FRigElementKey, FRigSwitchParentInfo>& Pair : SwitchParentValues)
			{
				if (FRigBaseElement* Element = Hierarchy->Find(Pair.Key))
				{
					if (FRigBaseElement* NewParent = Hierarchy->Find(Pair.Value.NewParent))
					{
						Hierarchy->SwitchToParent(Element, NewParent, Pair.Value.bInitial, Pair.Value.bAffectChildren);
					}
				}
			}
		}

		// Store control pose after backwards solve to figure out additive local transforms based on animation
		// This needs to happen after the switch parents
		ControlsAfterBackwardsSolve = Hierarchy->GetPose(false, ERigElementType::Control, TArrayView<const FRigElementKey>());

		if (PreAdditiveValuesApplicationEvent.IsBound())
		{
			PreAdditiveValuesApplicationEvent.Broadcast(this, TEXT("Additive"));
		}

		if (!IsReadOnly())
		{
			// Apply additive controls
			for (TPair<FRigElementKey, FRigSetControlValueInfo>& Value : ControlValues)
			{
				if (FRigBaseElement* Element = Hierarchy->Find(Value.Key))
				{
					if (FRigControlElement* Control = Cast<FRigControlElement>(Element))
					{
						FRigSetControlValueInfo& Info = Value.Value;

						// A bool/enum value is not an additive property. We just overwrite the value.
						if (!Control->CanTreatAsAdditive())
						{
							const bool bSetupUndo = false; // Rely on the sequencer track to handle undo/redo
							Hierarchy->SetControlValue(Control, Info.Value, ERigControlValueType::Current, bSetupUndo, false, Info.bPrintPythonCommnds, false);
						}
						else
						{
							// Transform from animation
							const FRigControlValue PreviousValue = Hierarchy->GetControlValue(Control, ERigControlValueType::Current, false);
							const FTransform PreviousTransform = PreviousValue.GetAsTransform(Control->Settings.ControlType, Control->Settings.PrimaryAxis);

							// Additive transform from controls
							const FRigControlValue& AdditiveValue = Info.Value;
							const FTransform AdditiveTransform = AdditiveValue.GetAsTransform(Control->Settings.ControlType, Control->Settings.PrimaryAxis);

							// Add them to find the final value
							FTransform FinalTransform = AdditiveTransform * PreviousTransform;
							FinalTransform.SetRotation(FinalTransform.GetRotation().GetNormalized());

							FRigControlValue FinalValue;
							FinalValue.SetFromTransform(FinalTransform, Control->Settings.ControlType, Control->Settings.PrimaryAxis);

							const bool bSetupUndo = false; // Rely on the sequencer track to handle undo/redo
							Hierarchy->SetControlValue(Control, FinalValue, ERigControlValueType::Current, bSetupUndo, false, Info.bPrintPythonCommnds, false);
						}

						if (Info.bNotify && OnControlModified.IsBound())
						{
							OnControlModified.Broadcast(this, Control, Info.Context);
							Info.bNotify = false;
						}
					}
				}
			}
		}

		// Interaction solve
		if (InteractionType != (uint8)EControlRigInteractionType::None)
		{
			Execute(FRigUnit_InteractionExecution::EventName);
		}
		
		// Forward solve
		{
			Execute(FRigUnit_PreBeginExecution::EventName);
			Execute(FRigUnit_BeginExecution::EventName);
			Execute(FRigUnit_PostBeginExecution::EventName);
		}
	}
	else
	{
		Super::Evaluate_AnyThread();
	}
}

bool UControlRig::EvaluateSkeletalMeshComponent(double InDeltaTime)
{
	if (USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(GetObjectBinding()->GetBoundObject()))
	{
		SkelMeshComp->TickAnimation(InDeltaTime, false);

		SkelMeshComp->RefreshBoneTransforms();
		SkelMeshComp->RefreshFollowerComponents();
		SkelMeshComp->UpdateComponentToWorld();
		SkelMeshComp->FinalizeBoneTransform();
		SkelMeshComp->MarkRenderTransformDirty();
		SkelMeshComp->MarkRenderDynamicDataDirty();
		return true;
	}
	return false;
}

void UControlRig::ResetControlValues()
{
	ControlValues.Reset();
}

void UControlRig::ClearPoseBeforeBackwardsSolve()
{
	PoseBeforeBackwardsSolve.Reset();
}

TArray<FRigControlElement*> UControlRig::InvertInputPose(const TArray<FRigElementKey>& InElements, EControlRigSetKey InSetKey)
{
	TArray<FRigControlElement*> ModifiedElements;
	ModifiedElements.Reserve(ControlsAfterBackwardsSolve.Num());
	for (const FRigPoseElement& PoseElement : ControlsAfterBackwardsSolve)
	{
		if (FRigControlElement* ControlElement = Cast<FRigControlElement>(DynamicHierarchy->Get(PoseElement.Index)))
		{
			if (IsAdditive() && !ControlElement->CanTreatAsAdditive())
			{
				continue;
			}
			
			if (InElements.IsEmpty() || InElements.Contains(ControlElement->GetKey()))
			{
				FRigControlValue Value;
				Value.SetFromTransform(PoseElement.LocalTransform.Inverse(), ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
				FRigSetControlValueInfo Info = {Value, true, FRigControlModifiedContext(InSetKey), false, false, false};
				ControlValues.Add(ControlElement->GetKey(), Info);
				ModifiedElements.Add(ControlElement);
			}
		}
	}
	return ModifiedElements;
}

void UControlRig::InitializeFromCDO()
{
	// Flag the rig as needing sequencer re-initialization. Scoped to Runtime Asset-backed
	// rigs because they re-initialize the same UObject in place (same GetUniqueID()), so
	// sequencer's HasControlRigTrack(UniqueID) check would otherwise skip the refresh.
	// BP rigs get a new UObject via FBlueprintCompileReinstancer, so they don't need this.
	if (!HasAnyFlags(RF_ClassDefaultObject) && GetGeneratedByAsset().GetObject() != nullptr)
	{
		bNeedsInstanceInitialization = true;
	}

	InitializeVMsFromCDO();
	
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// copy CDO property you need to here
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (!IsRigModuleInstance())
		{
			// similar to FControlRigBlueprintCompilerContext::CopyTermDefaultsToDefaultObject,
			// where CDO is initialized from BP there,
			// we initialize all other instances of Control Rig from the CDO here
			URigHierarchy* DefaultHierarchy = nullptr;
			FRigHierarchySettings* DefaultHierarchySettings = nullptr;
			FRigElementKeyRedirector* DefaultElementKeyRedirector = nullptr;
			FRigModuleSettings* DefaultRigModuleSettings = nullptr;
			if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
			{
				DefaultHierarchy = RuntimeAsset->GetHierarchy();
				DefaultHierarchySettings = &RuntimeAsset->GetHierarchySettings();
				DefaultElementKeyRedirector = &RuntimeAsset->GetElementKeyRedirector();
				DefaultRigModuleSettings = &RuntimeAsset->GetRigModuleSettings();
			}
			else if (UControlRig* CDO = GetClass()->GetDefaultObject<UControlRig>())
			{
				DefaultHierarchy = CDO->GetHierarchy();
				DefaultHierarchySettings = &CDO->HierarchySettings;
				DefaultElementKeyRedirector = &CDO->ElementKeyRedirector;
				DefaultRigModuleSettings = &CDO->RigModuleSettings;
			}
			URigHierarchy* Hierarchy = GetHierarchy();

			// copy hierarchy
			{
				FRigHierarchyValidityBracket ValidityBracketA(Hierarchy);
				FRigHierarchyValidityBracket ValidityBracketB(DefaultHierarchy);
			
				TGuardValue<bool> Guard(Hierarchy->GetSuspendNotificationsFlag(), true);
				Hierarchy->CopyHierarchy(DefaultHierarchy);
				Hierarchy->ResetPoseToInitial(ERigElementType::All);
				bHierarchyInitializedThisFrame = true;
			}

#if WITH_EDITOR
			// current hierarchy should always mirror CDO's hierarchy whenever a change of interest happens
			DefaultHierarchy->RegisterListeningHierarchy(Hierarchy);
#endif

			// notify clients that the hierarchy has changed
			Hierarchy->Notify(ERigHierarchyNotification::HierarchyReset, {});

			// copy hierarchy settings
			if (DefaultHierarchySettings)
			{
				HierarchySettings = *DefaultHierarchySettings;
			}
			if (DefaultElementKeyRedirector)
			{
				ElementKeyRedirector = FRigElementKeyRedirector(*DefaultElementKeyRedirector, Hierarchy);
			} 
			if (DefaultRigModuleSettings)
			{
				RigModuleSettings = *DefaultRigModuleSettings;
			}
		
			// increment the procedural limit based on the number of elements in the CDO
			if(const URigHierarchy* CDOHierarchy = DefaultHierarchy)
			{
				HierarchySettings.ProceduralElementLimit += CDOHierarchy->Num();
			}
		}

		ExternalVariableDataAssetLinks.Reset();
	}
}

UControlRig::FAnimAttributeContainerPtrScope::FAnimAttributeContainerPtrScope(UControlRig* InControlRig,
	UE::Anim::FMeshAttributeContainer& InExternalContainer)
{
	ControlRig = InControlRig;
	ControlRig->ExternalAnimAttributeContainer = &InExternalContainer;
}

UControlRig::FAnimAttributeContainerPtrScope::~FAnimAttributeContainerPtrScope()
{
	// control rig should not hold on to this container since it is stack allocated
	// and should not be used outside of stack, see FPoseContext
	ControlRig->ExternalAnimAttributeContainer = nullptr;
}

AActor* UControlRig::GetHostingActor() const
{
	return ObjectBinding ? ObjectBinding->GetHostingActor() : nullptr;
}

#if WITH_EDITOR
FText UControlRig::GetCategory() const
{
	return LOCTEXT("DefaultControlRigCategory", "Animation|ControlRigs");
}

FText UControlRig::GetToolTipText() const
{
	return LOCTEXT("DefaultControlRigTooltip", "ControlRig");
}
#endif

bool UControlRig::AllConnectorsAreResolved(FString* OutFailureReason, FRigElementKey* OutConnector) const
{
	if(const URigHierarchy* Hierarchy = GetHierarchy())
	{
		// todo: introduce a cache here based on ElementKeyDirector hash and
		// topology hash of the hierarchy
		
		for(const FRigModuleConnector& Connector : GetRigModuleSettings().ExposedConnectors)
		{
			if (Connector.Settings.bOptional)
			{
				continue;
			}

			const FRigElementKey ConnectorKey(*Connector.Name, ERigElementType::Connector);
			if(const FRigElementKeyRedirector::FCachedKeyArray* Cache = ElementKeyRedirector.Find(ConnectorKey))
			{
				for(const FCachedRigElement& CachedRigElement : *Cache)
				{
					const bool bIsConnectorValid = const_cast<FCachedRigElement*>(&CachedRigElement)->UpdateCache(Hierarchy);
					if (!bIsConnectorValid)
					{
						if(OutFailureReason)
						{
							static constexpr TCHAR Format[] = TEXT("Connector '%s%s' has invalid target '%s'.");
							*OutFailureReason = FString::Printf(Format, *GetRigModulePrefix(), *Connector.Name, *CachedRigElement.GetKey().ToString());
						}
						if(OutConnector)
						{
							*OutConnector = ConnectorKey;
						}
						return false;
					}
				}
			}
			else
			{
				if(OutFailureReason)
				{
					static constexpr TCHAR Format[] = TEXT("Connector '%s' is not resolved.");
					*OutFailureReason = FString::Printf(Format, *Connector.Name);
				}
				if(OutConnector)
				{
					*OutConnector = ConnectorKey;
				}
				return false;
			}
		}
	}
	return true;
}

bool UControlRig::CanExecute() const
{
	return Super::CanExecute() && GetHierarchy() != nullptr;
}

bool UControlRig::Execute(const FName& InEventName)
{
	if(!CanExecute())
	{
		return false;
	}

	// Only top-level rigs should execute this function.
	// Rig modules/nested rigs should run through Execute_Internal
	if (InEventName != FRigUnit_PrepareForExecution::EventName)
	{
		ensureMsgf(GetTypedOuter<UControlRig>() == nullptr, TEXT("UControlRig::Execute running from a nested rig in %s"), *GetPackage()->GetPathName());
	}
	// ensureMsgf(InEventName != FRigUnit_PreBeginExecution::EventName &&
	// 					InEventName != FRigUnit_PostBeginExecution::EventName, TEXT("Requested execution of invalid event %s on top level rig in %s"), *InEventName.ToString(), *GetPackage()->GetPathName());

	bool bJustRanInit = false;
	if(bRequiresInitExecution && !IsReadOnly())
	{
		const TGuardValue<float> AbsoluteTimeGuard(AbsoluteTime, AbsoluteTime);
		const TGuardValue<float> DeltaTimeGuard(DeltaTime, DeltaTime);
		if(!InitializeVM(InEventName))
		{
			return false;
		}
		bJustRanInit = true;
	}

	// The EventQueueToRun should only be modified in URigVMHost::Evaluate_AnyThread
	// We create a temporary queue for the execution of only this event
	TConstArrayView<FName> LocalEventQueueToRun = EventQueueToRun.IsEmpty() ? EventQueue : EventQueueToRun;

	const bool bIsEventInQueue = LocalEventQueueToRun.Contains(InEventName);
	const bool bIsEventFirstInQueue = !LocalEventQueueToRun.IsEmpty() && LocalEventQueueToRun[0] == InEventName; 
	const bool bIsEventLastInQueue = !LocalEventQueueToRun.IsEmpty() && LocalEventQueueToRun.Last() == InEventName;
	const bool bIsConstructionEvent = InEventName == FRigUnit_PrepareForExecution::EventName;
	const bool bIsPostConstructionEvent = InEventName == FRigUnit_PostPrepareForExecution::EventName;
	const bool bPostConstructionEventInQueue = LocalEventQueueToRun.Contains(FRigUnit_PostPrepareForExecution::EventName) && SupportsEvent(FRigUnit_PostPrepareForExecution::EventName);
	const bool bPreForwardSolveInQueue = LocalEventQueueToRun.Contains(FRigUnit_PreBeginExecution::EventName);
	const bool bPostForwardSolveInQueue = LocalEventQueueToRun.Contains(FRigUnit_PostBeginExecution::EventName);
	const bool bIsPreForwardSolve = InEventName == FRigUnit_PreBeginExecution::EventName;
	const bool bIsForwardSolve = InEventName == FRigUnit_BeginExecution::EventName;
	const bool bIsPostForwardSolve = InEventName == FRigUnit_PostBeginExecution::EventName;
	const bool bIsInteractionEvent = InEventName == FRigUnit_InteractionExecution::EventName;

	ensure(!HasAnyFlags(RF_ClassDefaultObject));
	
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ControlRig_Execute);
	
	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	TGuardValue<TFunction<void(const FName& InEventName)>> OnExecutionReachedExitCallback(ExtendedExecuteContext.OnExecutionReachedExitCallback, TFunction<void(const FName & InEventName)>([this](const FName& InEventName)
		{
			HandleExecutionReachedExit(InEventName);
		}));

	FControlRigExecuteContext& PublicContext = ExtendedExecuteContext.GetPublicDataSafe<FControlRigExecuteContext>();

#if WITH_EDITOR
	PublicContext.SetLog(RigVMLog); // may be nullptr
#endif

	PublicContext.SetEventName(InEventName);
	PublicContext.SetDeltaTime(DeltaTime);
	PublicContext.SetAbsoluteTime(AbsoluteTime);
	PublicContext.SetFramesPerSecond(GetCurrentFramesPerSecond());
#if WITH_EDITOR
	PublicContext.SetHostBeingDebugged(bIsBeingDebugged);
#endif
#if RIGVM_TRACE_ENABLED
	PublicContext.SetHostPlayingRewindDebugTrace(bIsPlayingRewindDebugTrace);
#endif

#if UE_RIGVM_DEBUG_EXECUTION
	PublicContext.bDebugExecution = bDebugExecutionEnabled;
#endif
	
#if WITH_EDITOR
	if (!IsReadOnly())
	{
		ExtendedExecuteContext.SetInstructionVisitInfo(&InstructionVisitInfo);
		ExtendedExecuteContext.SetDebugInfo(DebugInfo, false);
		
		if (IsProfilingEnabled())
		{
			ExtendedExecuteContext.SetProfilingInfo(&ProfilingInfo);
		}
		else
		{
			ExtendedExecuteContext.SetProfilingInfo(nullptr);
		}
	}
#endif

	FRigUnitContext& Context = PublicContext.UnitContext;

	bool bEnableDrawInterface = false;
#if WITH_EDITOR
	const bool bEnabledDuringGame = CVarControlRigEnableDrawInterfaceInGame->GetInt() != 0;
	if (bEnabledDuringGame || !bIsRunningInPIE || UE_TRACE_CHANNELEXPR_IS_ENABLED(RigVMChannel))
	{
		bEnableDrawInterface = true;
	}	
#endif

	// setup the draw interface for debug drawing
	if(!bIsEventInQueue || bIsEventFirstInQueue)
	{
		// only reset the draw interface if we are not running the rig during interaction with simulation present. 
		if(InEventName != FRigUnit_InteractionExecution::EventName || !bEvaluationTriggeredFromInteraction || !ContainsSimulation())
		{
			if (!IsReadOnly())
			{
				DrawInterface.Reset();
			}
		}
	}

	if (bEnableDrawInterface)
	{
		PublicContext.SetDrawInterface(&DrawInterface);
	}
	else
	{
		PublicContext.SetDrawInterface(nullptr);
	}

	// setup the animation attribute container
	Context.AnimAttributeContainer = ExternalAnimAttributeContainer;

	// draw container contains persistent draw instructions, 
	// so we cannot call Reset(), which will clear them,
	// instead, we re-initialize them from the CDO
	if (GeneratedBy)
	{
		DrawContainer = GeneratedBy->GetDrawContainer();
	}
	else if (UControlRig* CDO = GetClass()->GetDefaultObject<UControlRig>())
	{
		DrawContainer = CDO->DrawContainer;
	}
	PublicContext.SetDrawContainer(&DrawContainer);

	// setup the data source registry
	Context.DataSourceRegistry = GetDataSourceRegistry();

	// setup the context with further fields
	Context.InteractionType = InteractionType;
	Context.ElementsBeingInteracted = ElementsBeingInteracted;
	PublicContext.Hierarchy = GetHierarchy();
	PublicContext.ControlRig = this;

	// allow access to the hierarchy
	Context.HierarchySettings = HierarchySettings;
	check(PublicContext.Hierarchy);

	// allow access to the shape libraries
	PublicContext.OnAddShapeLibraryDelegate.BindUObject(this, &UControlRig::OnAddShapeLibrary);
	PublicContext.OnShapeExistsDelegate.BindUObject(this, &UControlRig::OnShapeExists);

	// allow access to the default hierarchy to allow to reset
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (PublicContext.Hierarchy)
		{
			if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
			{
				if(URigHierarchy* DefaultHierarchy = RuntimeAsset->GetHierarchy())
				{
					PublicContext.Hierarchy->DefaultHierarchyPtr = DefaultHierarchy;
				}
			}
			else if (UControlRig* CDO = Cast<UControlRig>(GetClass()->GetDefaultObject()))
			{
				if(URigHierarchy* DefaultHierarchy = CDO->GetHierarchy())
				{
					PublicContext.Hierarchy->DefaultHierarchyPtr = DefaultHierarchy;
				}
			}
		}
	}

	// disable any controller access outside of the construction event
	FRigHierarchyEnableControllerBracket DisableHierarchyController(PublicContext.Hierarchy, bIsConstructionEvent || bIsPostConstructionEvent);

	// given the outer scene component configure
	// the transform lookups to map transforms from rig space to world space
	if (USceneComponent* SceneComponent = GetOwningSceneComponent())
	{
		PublicContext.SetOwningComponent(SceneComponent);
	}
	else
	{
		PublicContext.SetOwningComponent(nullptr);
		
		if (ObjectBinding.IsValid())
		{
			AActor* HostingActor = ObjectBinding->GetHostingActor();
			if (HostingActor)
			{
				PublicContext.SetOwningActor(HostingActor);
			}
			else if (UObject* Owner = ObjectBinding->GetBoundObject())
			{
				PublicContext.SetWorld(Owner->GetWorld());
			}
		}

		if (PublicContext.GetWorld() == nullptr)
		{
			if (UObject* Outer = GetOuter())
			{
				PublicContext.SetWorld(Outer->GetWorld());
			}
		}
	}
	
	// setup the user data
	// first using the data from the rig,
	// followed by the data coming from the skeleton
	// then the data coming from the skeletal mesh,
	// and in the future we'll also add the data from the control rig component
	if (PublicContext.IsRunningAConstructionEvent())
	{
		PublicContext.AssetUserData.Reset();
		if(const TArray<UAssetUserData*>* ControlRigUserDataArray = GetAssetUserDataArray())
		{
			PublicContext.AssetUserData.Reserve(ControlRigUserDataArray->Num());
			for(const UAssetUserData* ControlRigUserData : *ControlRigUserDataArray)
			{
				if (ControlRigUserData)
				{
					PublicContext.AssetUserData.Add(ControlRigUserData);
				}
			}
		}
	}

	// if we have any referenced elements dirty them
	// also reset the recorded read / written transforms as needed
#if WITH_EDITOR
	TSharedPtr<TGuardValue<bool>> RecordTransformsPerInstructionGuard;
#endif

	URigHierarchy* Hierarchy = GetHierarchy();
	if(Hierarchy && !IsReadOnly())
	{
		Hierarchy->UpdateReferences(&PublicContext);

#if WITH_EDITOR
		bool bRecordInstructionsAtRuntime = true;
		if(const UObject* Outer = GetOuter())
		{
			if(Outer->IsA<UControlRigComponent>())
			{
				bRecordInstructionsAtRuntime = false;
			}
		}
		
		if (IsNativeRig())
		{
			bRecordInstructionsAtRuntime = false;
		}
		
		// only record instructions when running one of the forward solve events for now
		if (!bIsPreForwardSolve && !bIsForwardSolve && !bIsPostForwardSolve)
		{
			bRecordInstructionsAtRuntime = false;
		}
		
		bool bShouldResetRecords = !bIsEventInQueue || bIsEventFirstInQueue;
		if (!bShouldResetRecords && bIsAdditive)
		{
			// In the case of layered rigs, the forward solve is not the first event in the queue, the backward solve is
			// So lets just force the reset right before the pre forward solve
			bShouldResetRecords = InEventName == FRigUnit_PreBeginExecution::EventName;
		}
		
		RecordTransformsPerInstructionGuard = MakeShared<TGuardValue<bool>>(Hierarchy->bRecordInstructionsAtRuntime, bRecordInstructionsAtRuntime);
		if (bShouldResetRecords)
		{
			ResetRecordedTransforms(InEventName);
		}
#endif
	}

	// guard against recursion
	if(IsExecuting())
	{
		UE_LOGF(LogControlRig, Warning, "%ls: Execute is being called recursively.", *GetPathName());
		return false;
	}
	if(bIsConstructionEvent || bIsPostConstructionEvent)
	{
		if(IsRunningPreConstruction() || IsRunningPostConstruction())
		{
			UE_LOGF(LogControlRig, Warning, "%ls: Construction is being called recursively.", *GetPathName());
			return false;
		}
	}

	bool bSuccess = true;

	// we'll special case the construction event here
	if (bIsConstructionEvent)
	{
		check(Hierarchy);

		// Reset our flag on construction - nodes etc are expected to set this if they contain
		// state/would behave badly with some optional updates.
		bContainsSimulation = false;

		// construction mode means that we are running the construction event
		// constantly for testing purposes.
		const bool bConstructionModeEnabled = IsConstructionModeEnabled();
		{
			if (PreConstructionForUIEvent.IsBound())
			{
				FControlRigBracketScope BracketScope(PreConstructionBracket);
				PreConstructionForUIEvent.Broadcast(this, FRigUnit_PrepareForExecution::EventName);
			}

#if WITH_EDITOR
			// apply the selection pose for the construction mode
			// we are doing this here to make sure the brackets below have the right data
			if(bConstructionModeEnabled)
			{
				ApplySelectionPoseForConstructionMode(InEventName);
			}
#endif

			// disable selection notifications from the hierarchy
			TGuardValue<bool> DisableSelectionNotifications(Hierarchy->GetController(true)->bSuspendSelectionNotifications, true);
			{
				FRigPose CurrentPose;
				// We might want to reset the input pose after construction
				if (bResetCurrentTransformsAfterConstruction && !IsReadOnly())
				{
					CurrentPose = Hierarchy->GetPose(false, ERigElementType::ToResetAfterConstructionEvent, FRigElementKeyCollection());
				}
				
				{
					// Copy the hierarchy from the default object onto this one
#if WITH_EDITOR
					FTransientControlScope TransientControlScope(Hierarchy);
#endif
					if (!bHierarchyInitializedThisFrame && !IsReadOnly())
					{
						if ((bCopyHierarchyBeforeConstruction || bResetInitialTransformsBeforeConstruction))
						{
							// maintain the initial pose if it ever was set by the client
							FRigPose InitialPose;
							if(!bResetInitialTransformsBeforeConstruction)
							{
								InitialPose = Hierarchy->GetPose(true, ERigElementType::ToResetAfterConstructionEvent, FRigElementKeyCollection());
							}

							if(bCopyHierarchyBeforeConstruction)
							{
								Hierarchy->ResetToDefault();
							}

							if(InitialPose.Num() > 0)
							{
								const TGuardValue<bool> DisableRecordingCurveChanges(Hierarchy->GetRecordCurveChangesFlag(), false);
								Hierarchy->SetPose(InitialPose, ERigTransformType::InitialLocal);
							}
						}

						{
		#if WITH_EDITOR
							TUniquePtr<FTransientControlPoseScope> TransientControlPoseScope;
							if (bConstructionModeEnabled)
							{
								// save the transient control value, it should not be constantly reset in construction mode
								TransientControlPoseScope = MakeUnique<FTransientControlPoseScope>(this);
							}
		#endif
							// reset the pose to initial such that construction event can run from a deterministic initial state
							Hierarchy->ResetPoseToInitial(ERigElementType::All);
						}
					}
					else
					{
						bHierarchyInitializedThisFrame = false;
					}

					RestoreShapeLibrariesFromCDO();

					if (PreConstructionEvent.IsBound())
					{
						FControlRigBracketScope BracketScope(PreConstructionBracket);
						PreConstructionEvent.Broadcast(this, FRigUnit_PrepareForExecution::EventName);
					}

					bSuccess = Execute_Internal(FRigUnit_PrepareForExecution::EventName);
					
				} // destroy FTransientControlScope

				if(!bPostConstructionEventInQueue)
				{
					Hierarchy->GetController()->SetHierarchySelection(SelectionBeforeInit);
					ApplyElementOverrides();
					EmitPostConstructionEventFinished();
				}

				// Reset the input pose after construction
				if (CurrentPose.Num() > 0 && !IsReadOnly())
				{
					const TGuardValue<bool> DisableRecordingCurveChanges(Hierarchy->GetRecordCurveChangesFlag(), false);
					Hierarchy->SetPose(CurrentPose, ERigTransformType::CurrentLocal);
				}
			}
		} // destroy DisableSelectionNotifications

		if (bConstructionModeEnabled && !IsReadOnly())
		{
#if WITH_EDITOR
			TUniquePtr<FTransientControlPoseScope> TransientControlPoseScope;
			if (bConstructionModeEnabled)
			{
				// save the transient control value, it should not be constantly reset in construction mode
				TransientControlPoseScope = MakeUnique<FTransientControlPoseScope>(this);
			}
#endif
			Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		}

		// synchronize the selection now with the new hierarchy after running construction
		SendSelectionBeforeInitNotifications();
	}
	else
	{
#if WITH_EDITOR
		// only set a valid first entry event when none has been set
		if (InstructionVisitInfo.GetFirstEntryEventInEventQueue() == NAME_None && !LocalEventQueueToRun.IsEmpty() && VM)
		{
			InstructionVisitInfo.SetFirstEntryEventInEventQueue(LocalEventQueueToRun[0]);
		}

		// Transform Overrride is generated using a Transient Control 
		ApplyTransformOverrideForUserCreatedBones();

		if (bEnableAnimAttributeTrace && ExternalAnimAttributeContainer != nullptr)
		{
			InputAnimAttributeSnapshot.CopyFrom(*ExternalAnimAttributeContainer);
		}
#endif

		if (PreExecutedEvent.IsBound())
		{
			FControlRigBracketScope BracketScope(ExecuteBracket);
			PreExecutedEvent.Broadcast(this, InEventName);
		}
		
		if (bIsPreForwardSolve || (bIsForwardSolve && !bPreForwardSolveInQueue))
		{
			if (PreForwardsSolveEvent.IsBound())
			{
				FControlRigBracketScope BracketScope(PreForwardsSolveBracket);
				PreForwardsSolveEvent.Broadcast(this, FRigUnit_BeginExecution::EventName);
			}

		}

		bSuccess = Execute_Internal(InEventName);

		if (bIsPostConstructionEvent)
		{
			{
				TGuardValue<bool> DisableSelectionNotifications(Hierarchy->GetController(true)->bSuspendSelectionNotifications, true);
				Hierarchy->GetController()->SetHierarchySelection(SelectionBeforeInit);
			}
			SendSelectionBeforeInitNotifications();
			ApplyElementOverrides();
			EmitPostConstructionEventFinished();
		}

#if WITH_EDITOR
		if (bEnableAnimAttributeTrace && ExternalAnimAttributeContainer != nullptr)
		{
			OutputAnimAttributeSnapshot.CopyFrom(*ExternalAnimAttributeContainer);
		}
#endif

		if (bIsPostForwardSolve || (bIsForwardSolve && !bPostForwardSolveInQueue))
		{
			if (PostForwardsSolveEvent.IsBound())
			{
				FControlRigBracketScope BracketScope(PostForwardsSolveBracket);
				PostForwardsSolveEvent.Broadcast(this, FRigUnit_BeginExecution::EventName);
			}
		}
	}

	if(!bIsEventInQueue || bIsEventLastInQueue) 
	{
		DeltaTime = 0.f;
	}

	if (ExecutedEvent.IsBound())
	{
		if (!IsAdditive() || InEventName != FRigUnit_InverseExecution::EventName)
		{
			FControlRigBracketScope BracketScope(ExecuteBracket);
			ExecutedEvent.Broadcast(this, InEventName);
		}
	}

#if WITH_EDITOR

	// for the last event in the queue - clear the log message queue
	if (RigVMLog != nullptr && bEnableLogging)
	{
		if (bJustRanInit)
		{
			RigVMLog->KnownMessages.Reset();
			LoggedMessages.Reset();
		}
		else if(bIsEventLastInQueue)
		{
			for (const FRigVMLog::FLogEntry& Entry : RigVMLog->Entries)
			{
				if (Entry.FunctionName == NAME_None || Entry.InstructionIndex == INDEX_NONE || Entry.Message.IsEmpty())
				{
					continue;
				}

				FString PerInstructionMessage = 
					FString::Printf(
						TEXT("Instruction[%d] '%s': '%s'"),
						Entry.InstructionIndex,
						*Entry.FunctionName.ToString(),
						*Entry.Message
					);

				LogOnce(Entry.Severity, Entry.InstructionIndex, PerInstructionMessage);
			}
		}
	}
#endif

	// close remaining undo brackets from hierarchy
	while(ControlUndoBracketIndex > 0)
	{
		FRigEventContext EventContext;
		EventContext.Event = ERigEvent::CloseUndoBracket;
		EventContext.SourceEventName = InEventName;
		EventContext.LocalTime = PublicContext.GetAbsoluteTime();
		HandleHierarchyEvent(Hierarchy, EventContext);
	}

	if (PublicContext.GetDrawInterface() && PublicContext.GetDrawContainer() && bIsEventLastInQueue) 
	{
		if (!IsReadOnly())
		{
			PublicContext.GetDrawInterface()->Instructions.Append(PublicContext.GetDrawContainer()->Instructions);
		}

		FRigHierarchyValidityBracket ValidityBracket(Hierarchy);
		
		Hierarchy->ForEach<FRigControlElement>([this, Hierarchy](FRigControlElement* ControlElement) -> bool
		{
			const FRigControlSettings& Settings = ControlElement->Settings;

			if (Settings.IsVisible() &&
				!Settings.bIsTransientControl &&
				Settings.bDrawLimits &&
				Settings.LimitEnabled.Contains(FRigControlLimitEnabled(true, true)))
			{
				FTransform Transform = Hierarchy->GetGlobalControlOffsetTransformByIndex(ControlElement->GetIndex());
				FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, Settings.ShapeColor, 0.f, Transform);

				switch (Settings.ControlType)
				{
					case ERigControlType::Float:
					case ERigControlType::ScaleFloat:
					{
						if(Settings.LimitEnabled[0].IsOff())
						{
							break;
						}

						FVector MinPos = FVector::ZeroVector;
						FVector MaxPos = FVector::ZeroVector;

						switch (Settings.PrimaryAxis)
						{
							case ERigControlAxis::X:
							{
								MinPos.X = Settings.MinimumValue.Get<float>();
								MaxPos.X = Settings.MaximumValue.Get<float>();
								break;
							}
							case ERigControlAxis::Y:
							{
								MinPos.Y = Settings.MinimumValue.Get<float>();
								MaxPos.Y = Settings.MaximumValue.Get<float>();
								break;
							}
							case ERigControlAxis::Z:
							{
								MinPos.Z = Settings.MinimumValue.Get<float>();
								MaxPos.Z = Settings.MaximumValue.Get<float>();
								break;
							}
						}

						Instruction.Positions.Add(MinPos);
						Instruction.Positions.Add(MaxPos);
						break;
					}
					case ERigControlType::Integer:
					{
						if(Settings.LimitEnabled[0].IsOff())
						{
							break;
						}

						FVector MinPos = FVector::ZeroVector;
						FVector MaxPos = FVector::ZeroVector;

						switch (Settings.PrimaryAxis)
						{
							case ERigControlAxis::X:
							{
								MinPos.X = (float)Settings.MinimumValue.Get<int32>();
								MaxPos.X = (float)Settings.MaximumValue.Get<int32>();
								break;
							}
							case ERigControlAxis::Y:
							{
								MinPos.Y = (float)Settings.MinimumValue.Get<int32>();
								MaxPos.Y = (float)Settings.MaximumValue.Get<int32>();
								break;
							}
							case ERigControlAxis::Z:
							{
								MinPos.Z = (float)Settings.MinimumValue.Get<int32>();
								MaxPos.Z = (float)Settings.MaximumValue.Get<int32>();
								break;
							}
						}

						Instruction.Positions.Add(MinPos);
						Instruction.Positions.Add(MaxPos);
						break;
					}
					case ERigControlType::Vector2D:
					{
						if(Settings.LimitEnabled.Num() < 2)
						{
							break;
						}
						if(Settings.LimitEnabled[0].IsOff() && Settings.LimitEnabled[1].IsOff())
						{
							break;
						}

						Instruction.PrimitiveType = ERigVMDrawSettings::LineStrip;
						FVector3f MinPos = Settings.MinimumValue.Get<FVector3f>();
						FVector3f MaxPos = Settings.MaximumValue.Get<FVector3f>();

						switch (Settings.PrimaryAxis)
						{
							case ERigControlAxis::X:
							{
								Instruction.Positions.Add(FVector(0.f, MinPos.X, MinPos.Y));
								Instruction.Positions.Add(FVector(0.f, MaxPos.X, MinPos.Y));
								Instruction.Positions.Add(FVector(0.f, MaxPos.X, MaxPos.Y));
								Instruction.Positions.Add(FVector(0.f, MinPos.X, MaxPos.Y));
								Instruction.Positions.Add(FVector(0.f, MinPos.X, MinPos.Y));
								break;
							}
							case ERigControlAxis::Y:
							{
								Instruction.Positions.Add(FVector(MinPos.X, 0.f, MinPos.Y));
								Instruction.Positions.Add(FVector(MaxPos.X, 0.f, MinPos.Y));
								Instruction.Positions.Add(FVector(MaxPos.X, 0.f, MaxPos.Y));
								Instruction.Positions.Add(FVector(MinPos.X, 0.f, MaxPos.Y));
								Instruction.Positions.Add(FVector(MinPos.X, 0.f, MinPos.Y));
								break;
							}
							case ERigControlAxis::Z:
							{
								Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, 0.f));
								Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, 0.f));
								Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, 0.f));
								Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, 0.f));
								Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, 0.f));
								break;
							}
						}
						break;
					}
					case ERigControlType::Position:
					case ERigControlType::Scale:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						FVector3f MinPos = FVector3f::ZeroVector;
						FVector3f MaxPos = FVector3f::ZeroVector;

						// we only check the first three here
						// since we only consider translation anyway
						// for scale it's also the first three
						if(Settings.LimitEnabled.Num() < 3)
						{
							break;
						}
						if(!Settings.LimitEnabled[0].IsOn() && !Settings.LimitEnabled[1].IsOn() && !Settings.LimitEnabled[2].IsOn())
						{
							break;
						}

						switch (Settings.ControlType)
						{
							case ERigControlType::Position:
							case ERigControlType::Scale:
							{
								MinPos = Settings.MinimumValue.Get<FVector3f>();
								MaxPos = Settings.MaximumValue.Get<FVector3f>();
								break;
							}
							case ERigControlType::Transform:
							{
								MinPos = Settings.MinimumValue.Get<FRigControlValue::FTransform_Float>().GetTranslation();
								MaxPos = Settings.MaximumValue.Get<FRigControlValue::FTransform_Float>().GetTranslation();
								break;
							}
							case ERigControlType::TransformNoScale:
							{
								MinPos = Settings.MinimumValue.Get<FRigControlValue::FTransformNoScale_Float>().GetTranslation();
								MaxPos = Settings.MaximumValue.Get<FRigControlValue::FTransformNoScale_Float>().GetTranslation();
								break;
							}
							case ERigControlType::EulerTransform:
							{
								MinPos = Settings.MinimumValue.Get<FRigControlValue::FEulerTransform_Float>().GetTranslation();
								MaxPos = Settings.MaximumValue.Get<FRigControlValue::FEulerTransform_Float>().GetTranslation();
								break;
							}
						}

						Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, MaxPos.Z));

						Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, MaxPos.Z));

						Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MinPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MinPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MinPos.X, MaxPos.Y, MaxPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, MinPos.Z));
						Instruction.Positions.Add(FVector(MaxPos.X, MaxPos.Y, MaxPos.Z));
						break;
					}
				}

				if (Instruction.Positions.Num() > 0)
				{
					DrawInterface.DrawInstruction(Instruction);
				}
			}

			return true;
		});
	}

	if(bIsInteractionEvent)
	{
		bInteractionJustBegan = false;
	}

	if (IsRunOnceEvent(InEventName))
	{
		RemoveRunOnceEvent(InEventName);
	}
	
	return bSuccess;
}

bool UControlRig::Execute_Internal(const FName& InEventName)
{
	if(!SupportsEvent(InEventName))
	{
		return false;
	}

	// The EventQueueToRun should only be modified in URigVMHost::Evaluate_AnyThread
	// We create a temporary queue for the execution of only this event
	TConstArrayView<FName> LocalEventQueueToRun = EventQueueToRun.IsEmpty() ? EventQueue : EventQueueToRun;

	// make sure to initialize here as well - just in case this gets
	// called without a call to ::Execute. This is already tackled by
	// the UModularRig::ExecuteQueue, but added it here as well nevertheless
	// to avoid crashes in the future.
	if(bRequiresInitExecution && !IsReadOnly())
	{
		const TGuardValue<float> AbsoluteTimeGuard(AbsoluteTime, AbsoluteTime);
		const TGuardValue<float> DeltaTimeGuard(DeltaTime, DeltaTime);
		if(!InitializeVM(InEventName))
		{
			return false;
		}
	}
	
	if(IsRigModule() && InEventName != FRigUnit_ConnectorExecution::EventName)
	{
		FString ConnectorWarning;
		if(!AllConnectorsAreResolved(&ConnectorWarning))
		{
#if WITH_EDITOR
			LogOnce(EMessageSeverity::Warning, INDEX_NONE, ConnectorWarning);
#endif
			return false;
		}
	}

	if (IsReadOnly())
	{
		return true;
	}
	
	if (VM)
	{
		FRigVMExtendedExecuteContext& Context = GetRigVMExtendedExecuteContext();

		static constexpr TCHAR InvalidatedVMFormat[] = TEXT("%s: Invalidated VM - aborting execution.");
		if(VM->IsNativized())
		{
			if(!IsValidLowLevel() ||
				!VM->IsValidLowLevel())
			{
				UE_LOG(LogControlRig, Warning, InvalidatedVMFormat, *GetClass()->GetName());
				return false;
			}
		}
		else
		{
			// sanity check the validity of the VM to ensure stability.
			if(!VM->IsContextValidForExecution(Context)
				|| !IsValidLowLevel()
				|| !VM->IsValidLowLevel()
			)
			{
				UE_LOG(LogControlRig, Warning, InvalidatedVMFormat, *GetClass()->GetName());
				return false;
			}
		}
		
#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
		const uint64 StartCycles = FPlatformTime::Cycles64();
		if(ProfilingRunsLeft <= 0)
		{
			ProfilingRunsLeft = UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM;
			AccumulatedCycles = 0;
		}
#endif
		
		URigHierarchy* Hierarchy = GetHierarchy();
		FRigHierarchyExecuteContextBracket HierarchyContextGuard(Hierarchy, &Context);

		// setup the module information
		FControlRigExecuteContext& PublicContext = Context.GetPublicDataSafe<FControlRigExecuteContext>();
		FControlRigExecuteContextRigModuleGuard RigModuleGuard(PublicContext, this);
		FRigHierarchyRedirectorGuard ElementRedirectorGuard(this);

		const bool bSuccess = VM->ExecuteVM(Context, InEventName) != ERigVMExecuteResult::Failed;

#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
		const uint64 EndCycles = FPlatformTime::Cycles64();
		const uint64 Cycles = EndCycles - StartCycles;
		AccumulatedCycles += Cycles;
		ProfilingRunsLeft--;
		if(ProfilingRunsLeft == 0)
		{
			const double Milliseconds = FPlatformTime::ToMilliseconds64(AccumulatedCycles);
			UE_LOGF(LogControlRig, Display, "%ls: %d runs took %.03lfms.", *GetClass()->GetName(), UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM, Milliseconds);
		}
#endif

		return bSuccess;
	}
	return false;
}

void UControlRig::RequestInit()
{
	RequestInitVMs();
	RequestConstruction();
}

void UControlRig::RequestConstruction()
{
	RequestRunOnceEvent(FRigUnit_PrepareForExecution::EventName);
	RequestRunOnceEvent(FRigUnit_PostPrepareForExecution::EventName);
}

bool UControlRig::IsConstructionRequired() const
{
	return IsRunOnceEvent(FRigUnit_PrepareForExecution::EventName);
}

bool UControlRig::SupportsBackwardsSolve() const
{
	return SupportsEvent(FRigUnit_InverseExecution::EventName);
}

void UControlRig::AdaptEventQueueForEvaluate(TArray<FName>& InOutEventQueueToRun)
{
	if(IsReplayEnabled())
	{
		check(Replay);

		// find all events at this particular frame
		const int32 NumTimes = Replay->InputTracks.GetNumTimes();

		int32 TimeIndex = GetReplayTimeIndex();
		if(TimeIndex >= NumTimes || TimeIndex < 0)
		{
			TimeIndex = 0;
			SetReplayTimeIndex(TimeIndex);
		}
		const double TimeAtIndex = Replay->InputTracks.GetAbsoluteTime(TimeIndex);

		InOutEventQueueToRun.Reset();
		while(TimeIndex < NumTimes)
		{
			const FName EventFromReplay = Replay->InputTracks.GetRigVMEvent(TimeIndex);
			InOutEventQueueToRun.Add(EventFromReplay);

			// also queue up the next event to run immediately if it was recorded at the same time
			if(TimeIndex != NumTimes - 1)
			{
				if(FMath::IsNearlyEqual(TimeAtIndex, Replay->InputTracks.GetAbsoluteTime(++TimeIndex)))
				{
					continue;
				}
			}
			break;
		}
		return;
	}
	
	Super::AdaptEventQueueForEvaluate(InOutEventQueueToRun);

	for (int32 i=0; i<InOutEventQueueToRun.Num(); ++i)
	{
		if (InOutEventQueueToRun[i] == FRigUnit_PrepareForExecution::EventName)
		{
			if (SupportsEvent(FRigUnit_PostPrepareForExecution::EventName))
			{
				i++; // skip construction
				InOutEventQueueToRun.Insert(FRigUnit_PostPrepareForExecution::EventName, i);
			}
		}
		if (InOutEventQueueToRun[i] == FRigUnit_BeginExecution::EventName)
		{
			if (SupportsEvent(FRigUnit_PreBeginExecution::EventName))
			{
				InOutEventQueueToRun.Insert(FRigUnit_PreBeginExecution::EventName, i);
				i++; // skip preforward 
			}
			if (SupportsEvent(FRigUnit_PostBeginExecution::EventName))
			{
				i++; // skip forward
				InOutEventQueueToRun.Insert(FRigUnit_PostBeginExecution::EventName, i);
			}
		}
	}

	if(InteractionType != (uint8)EControlRigInteractionType::None)
	{
		static const FName& InteractionEvent = FRigUnit_InteractionExecution::EventName;
		if(SupportsEvent(InteractionEvent))
		{
			if(InOutEventQueueToRun.IsEmpty())
			{
				InOutEventQueueToRun.Add(InteractionEvent);
			}
			else if(!InOutEventQueueToRun.Contains(FRigUnit_PrepareForExecution::EventName))
			{
				// insert just before the last event so the interaction runs prior to
				// forward solve or backwards solve.
				InOutEventQueueToRun.Insert(InteractionEvent, InOutEventQueueToRun.Num() - 1);
			}
		}

		if(ContainsSimulation() && bEvaluationTriggeredFromInteraction)
		{
			InOutEventQueueToRun.Remove(FRigUnit_BeginExecution::EventName);
			InOutEventQueueToRun.Remove(FRigUnit_PreBeginExecution::EventName);
			InOutEventQueueToRun.Remove(FRigUnit_PostBeginExecution::EventName);
		}
	}
}

void UControlRig::GetMappableNodeData(TArray<FName>& OutNames, TArray<FNodeItem>& OutNodeItems) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	OutNames.Reset();
	OutNodeItems.Reset();

	check(DynamicHierarchy);

	// now add all nodes
	DynamicHierarchy->ForEach<FRigBoneElement>([&OutNames, &OutNodeItems, this](FRigBoneElement* BoneElement) -> bool
    {
		OutNames.Add(BoneElement->GetFName());
		FRigElementKey ParentKey = DynamicHierarchy->GetFirstParent(BoneElement->GetKey());
		if(ParentKey.Type != ERigElementType::Bone)
		{
			ParentKey.Name = NAME_None;
		}

		const FTransform GlobalInitial = DynamicHierarchy->GetGlobalTransformByIndex(BoneElement->GetIndex(), true);
		OutNodeItems.Add(FNodeItem(ParentKey.Name, GlobalInitial));
		return true;
	});
}

UAnimationDataSourceRegistry* UControlRig::GetDataSourceRegistry()
{
	if (DataSourceRegistry)
	{
		if (DataSourceRegistry->GetOuter() != this)
		{
			DataSourceRegistry = nullptr;
		}
	}
	if (DataSourceRegistry == nullptr)
	{
		DataSourceRegistry = NewObject<UAnimationDataSourceRegistry>(this, NAME_None, RF_Transient);

		if (!IsInGameThread())
		{
			// If the object was created on a non-game thread, clear the async flag immediately, so that it can be
			// garbage collected in the future. 
			(void)DataSourceRegistry->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
		}

		if (HasAnyFlags(RF_ClassDefaultObject) && GetClass()->IsNative())
		{
			DataSourceRegistry->AddToRoot();
		}
	}

	return DataSourceRegistry;
}

#if WITH_EDITORONLY_DATA

void UControlRig::PostReinstanceCallback(const UControlRig* Old)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ObjectBinding = Old->ObjectBinding;
	Initialize();
}

#endif // WITH_EDITORONLY_DATA

TArray<UControlRig*> UControlRig::FindControlRigs(UObject* Outer, TSubclassOf<UControlRig> OptionalClass)
{
	TArray<UControlRig*> Result;
	
	if(Outer == nullptr)
	{
		return Result; 
	}
	
	AActor* OuterActor = Cast<AActor>(Outer);
	if(OuterActor == nullptr)
	{
		OuterActor = Outer->GetTypedOuter<AActor>();
	}
	
	for (TObjectIterator<UControlRig> Itr; Itr; ++Itr)
	{
		UControlRig* RigInstance = *Itr;
		if (OptionalClass == nullptr || RigInstance->GetClass()->IsChildOf(OptionalClass))
		{
			if(RigInstance->IsInOuter(Outer))
			{
				Result.Add(RigInstance);
				continue;
			}

			if(OuterActor)
			{
				if(RigInstance->IsInOuter(OuterActor))
				{
					Result.Add(RigInstance);
					continue;
				}

				if (TSharedPtr<IControlRigObjectBinding> Binding = RigInstance->GetObjectBinding())
				{
					if (AActor* Actor = Binding->GetHostingActor())
					{
						if (Actor == OuterActor)
						{
							Result.Add(RigInstance);
							continue;
						}
					}
				}
			}
		}
	}

	return Result;
}

void UControlRig::Serialize(FArchive& Ar)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("UControlRig(%s)"), *GetName()));

	Super::Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	// We are no longer storing the physics solver descriptions as part of the rig, and the struct
	// definitions have been removed. In addition, the array would have been empty when serialized
	// with the affected version, so we just need to absorb the data into a temporary array.
	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::ControlRigStoresPhysicsSolvers)
	{
		TArray<FRigPhysicsSolverDescription> TempSolvers;
		Ar << TempSolvers;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("PhysicsSolvers"));
	}

	if(Ar.IsObjectReferenceCollector())
	{
		Ar << OverrideAssets;
	}
}

void UControlRig::PostLoad()
{
	Super::PostLoad();

	if(HasAnyFlags(RF_ClassDefaultObject))
	{
		if(DynamicHierarchy)
		{
			// Some dynamic hierarchy objects have been created using NewObject<> instead of CreateDefaultSubObjects.
			// Assets from that version require the dynamic hierarchy to be flagged as below.
			DynamicHierarchy->SetFlags(DynamicHierarchy->GetFlags() | RF_Public | RF_DefaultSubObject);
		}
	}

	FRigInfluenceMapPerEvent NewInfluences;
	for(int32 MapIndex = 0; MapIndex < Influences.Num(); MapIndex++)
	{
		FRigInfluenceMap Map = Influences[MapIndex];
		FName EventName = Map.GetEventName();

		if(EventName == TEXT("Update"))
		{
			EventName = FRigUnit_BeginExecution::EventName;
		}
		else if(EventName == TEXT("Inverse"))
		{
			EventName = FRigUnit_InverseExecution::EventName;
		}
		
		NewInfluences.FindOrAdd(EventName).Merge(Map, true);
	}
	Influences = NewInfluences;

	if(DynamicHierarchy)
	{
		DynamicHierarchy->bUsePreferredEulerAngles = !bIsAdditive;
	}
}

FControlRigAssetStrongReference UControlRig::GetAssetReference() const
{
	FControlRigAssetStrongReference Result;
	if (GeneratedBy)
	{
		if (UControlRigRuntimeAsset* RuntimeAsset = Cast<UControlRigRuntimeAsset>(GeneratedBy.GetObject()))
		{
			Result.Set(RuntimeAsset);
		}
	}
	// for backwards compatibility, we cast to URigVMBlueprintGeneratedClass instead of UControlRigBlueprintGeneratedClass
	else if (URigVMBlueprintGeneratedClass* GeneratedClass = Cast<URigVMBlueprintGeneratedClass>(GetClass()))
	{
		Result.Set(TSubclassOf<UControlRig>(GeneratedClass));
	}
	else if (GetClass() == UFKControlRig::StaticClass())
	{
		Result.Set(TSubclassOf<UControlRig>(UFKControlRig::StaticClass()));
	}
	return Result;
}

const FRigModuleSettings& UControlRig::GetRigModuleSettings() const
{
	if(HasAnyFlags(RF_ClassDefaultObject))
	{
		return RigModuleSettings;
	}
	if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
	{
		return RuntimeAsset->GetRigModuleSettings();
	}
	else if (const UControlRig* CDO = Cast<UControlRig>(GetClass()->GetDefaultObject()))
	{
		return CDO->GetRigModuleSettings();
	}
	return RigModuleSettings;
}

bool UControlRig::IsRigModule() const
{
	return GetRigModuleSettings().IsValidModule(false);
}

bool UControlRig::IsRigModuleInstance() const
{
	if(IsRigModule())
	{
		return GetParentRig() != nullptr;
	}
	return false;
}

bool UControlRig::IsModularRig() const
{
	if(const UClass* Class = GetClass())
	{
		return Class->IsChildOf(UModularRig::StaticClass());
	}
	return false;
}

bool UControlRig::IsStandaloneRig() const
{
	return !IsModularRig();
}

bool UControlRig::IsNativeRig() const
{
	if(const UClass* Class = GetClass())
	{
		return Class->IsNative();
	}
	return false;
}

UControlRig* UControlRig::GetParentRig() const
{
	return GetTypedOuter<UControlRig>();
}

const FString& UControlRig::GetRigModulePrefix() const
{
	if(IsRigModule())
	{
		auto& [CachedPrefixName, CachedPrefixString] = RigModulePrefix;
		if (IsRigModuleInstance() || GetRigModuleSettings().Identifier.Name.IsEmpty())
		{
			if (!CachedPrefixName.IsEqual(GetFName(), ENameCase::CaseSensitive) || CachedPrefixString.IsEmpty())
			{
				CachedPrefixString = GetFName().ToString() + FRigHierarchyModulePath::ModuleNameSuffix;
				CachedPrefixName = GetFName();
			}
		}
		else
		{
			CachedPrefixString = GetRigModuleSettings().Identifier.Name + FRigHierarchyModulePath::ModuleNameSuffix;
		}
		return CachedPrefixString;
	}

	static const FString EmptyPrefix;
	return EmptyPrefix;
}

FRigElementKeyRedirector& UControlRig::GetElementKeyRedirector()
{
	// if we are an instance on a modular rig, use our local info
	if(IsRigModuleInstance())
	{
		return ElementKeyRedirector;
	}

	// if we are a rig module ( but not an instance on a rig)
	if(IsRigModule())
	{
		return ElementKeyRedirector;
	}

	static FRigElementKeyRedirector EmptyRedirector = FRigElementKeyRedirector();
	return EmptyRedirector;
}

void UControlRig::SetElementKeyRedirector(const FRigElementKeyRedirector InElementRedirector)
{
	ElementKeyRedirector = InElementRedirector;
}

TArray<FRigControlElement*> UControlRig::AvailableControls() const
{
	if(DynamicHierarchy)
	{
		return DynamicHierarchy->GetElementsOfType<FRigControlElement>();
	}
	return TArray<FRigControlElement*>();
}

FRigControlElement* UControlRig::FindControl(const FName& InControlName) const
{
	if(DynamicHierarchy == nullptr)
	{
		return nullptr;
	}

	// look up the control based on the stored name. 
	const FRigElementKey ControlKey(InControlName, ERigElementType::Control);
	if(FRigControlElement* Control = DynamicHierarchy->Find<FRigControlElement>(ControlKey))
	{
		return Control;
	}

	return nullptr;
}

bool UControlRig::IsConstructionModeEnabled() const
{
	return EventQueueToRun.Num() == 1 && EventQueueToRun.Contains(FRigUnit_PrepareForExecution::EventName);
}

FTransform UControlRig::SetupControlFromGlobalTransform(const FName& InControlName, const FTransform& InGlobalTransform)
{
	if (IsConstructionModeEnabled())
	{
		FRigControlElement* ControlElement = FindControl(InControlName);
		if (ControlElement && !ControlElement->Settings.bIsTransientControl)
		{
			const FTransform ParentTransform = GetHierarchy()->GetParentTransform(ControlElement, ERigTransformType::CurrentGlobal);
			const FTransform OffsetTransform = InGlobalTransform.GetRelativeTransform(ParentTransform);
			GetHierarchy()->SetControlOffsetTransform(ControlElement, OffsetTransform, ERigTransformType::InitialLocal, true, false);
			GetHierarchy()->SetControlOffsetTransform(ControlElement, OffsetTransform, ERigTransformType::CurrentLocal, true, false);

			if(URigHierarchy* DefaultHierarchy = GetHierarchy()->GetDefaultHierarchy())
			{
				if(FRigControlElement* DefaultControlElement = DefaultHierarchy->Find<FRigControlElement>(ControlElement->GetKey()))
				{
					DefaultHierarchy->SetControlOffsetTransform(DefaultControlElement, OffsetTransform, ERigTransformType::InitialLocal, true, true);
					DefaultHierarchy->SetControlOffsetTransform(DefaultControlElement, OffsetTransform, ERigTransformType::CurrentLocal, true, true);
				}
			}
		}
	}
	return InGlobalTransform;
}

void UControlRig::CreateRigControlsForCurveContainer()
{
	const bool bCreateFloatControls = CVarControlRigCreateFloatControlsForCurves->GetInt() == 0 ? false : true;
	if(bCreateFloatControls && DynamicHierarchy)
	{
		URigHierarchyController* Controller = DynamicHierarchy->GetController(true);
		if(Controller == nullptr)
		{
			return;
		}
		static const FString CtrlPrefix(TEXT("CTRL_"));

		DynamicHierarchy->ForEach<FRigCurveElement>([this, Controller](FRigCurveElement* CurveElement) -> bool
        {
			const FString Name = CurveElement->GetFName().ToString();
			
			if (Name.Contains(CtrlPrefix) && !DynamicHierarchy->Contains(FRigElementKey(*Name, ERigElementType::Curve)))
			{
				FRigControlSettings Settings;
				Settings.AnimationType = ERigControlAnimationType::AnimationChannel;
				Settings.ControlType = ERigControlType::Float;
				Settings.bIsCurve = true;
				Settings.bDrawLimits = false;

				FRigControlValue Value;
				Value.Set<float>(CurveElement->Get());

				Controller->AddControl(CurveElement->GetFName(), FRigElementKey(), Settings, Value, FTransform::Identity, FTransform::Identity); 
			}

			return true;
		});

		ControlModified().AddUObject(this, &UControlRig::HandleOnControlModified);
	}
}

void UControlRig::HandleOnControlModified(UControlRig* Subject, FRigControlElement* Control, const FRigControlModifiedContext& Context)
{
	if (Control->Settings.bIsCurve && DynamicHierarchy)
	{
		const FRigControlValue Value = DynamicHierarchy->GetControlValue(Control, IsConstructionModeEnabled() ? ERigControlValueType::Initial : ERigControlValueType::Current);
		DynamicHierarchy->SetCurveValue(FRigElementKey(Control->GetFName(), ERigElementType::Curve), Value.Get<float>());
	}	
}

bool UControlRig::IsCurveControl(const FRigControlElement* InControlElement) const
{
	return InControlElement->Settings.bIsCurve;
}

FTransform UControlRig::GetControlGlobalTransform(const FName& InControlName) const
{
	if(DynamicHierarchy == nullptr)
	{
		return FTransform::Identity;
	}
	return DynamicHierarchy->GetGlobalTransform(FRigElementKey(InControlName, ERigElementType::Control), false);
}

FRigControlValue UControlRig::GetControlValue(FRigControlElement* InControl, const ERigControlValueType& InValueType) const
{
	if (bIsAdditive && InValueType == ERigControlValueType::Current)
	{
		const int32 ControlIndex = ControlsAfterBackwardsSolve.GetIndex(InControl->GetKey());
		if (ControlIndex != INDEX_NONE)
		{
			// Booleans/Enums are not additive properties, just return the current value
			if (!InControl->CanTreatAsAdditive())
			{
				return GetHierarchy()->GetControlValue(InControl, InValueType);
			}
			
			// return local space control value (the one to be added after backwards solve)
			const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[ControlIndex];
			const FRigControlValue& CurrentValue = GetHierarchy()->GetControlValue(InControl, InValueType, false);
			const FTransform FinalTransform = CurrentValue.GetAsTransform(InControl->Settings.ControlType, InControl->Settings.PrimaryAxis);
			FTransform AdditiveTransform = FinalTransform * AnimPose.LocalTransform.Inverse();
			AdditiveTransform.SetRotation(AdditiveTransform.GetRotation().GetNormalized());
			FRigControlValue AdditiveValue;
			AdditiveValue.SetFromTransform(AdditiveTransform, InControl->Settings.ControlType, InControl->Settings.PrimaryAxis);
			return AdditiveValue;
		}
	}
	return GetHierarchy()->GetControlValue(InControl, InValueType);
}

void UControlRig::SetControlValueImpl(const FName& InControlName, const FRigControlValue& InValue, bool bNotify,
	const FRigControlModifiedContext& Context, bool bSetupUndo, bool bPrintPythonCommnds, bool bFixEulerFlips)
{
	// make sure we can run this rig as we are interacting with it.
	ResetEvaluationsLeft();
	
	const FRigElementKey Key(InControlName, ERigElementType::Control);

	FRigControlElement* ControlElement = DynamicHierarchy->Find<FRigControlElement>(Key);
	if(ControlElement == nullptr)
	{
		return;
	}
	if (bIsAdditive)
	{
		// Store the value to apply it after the backwards solve
		FRigSetControlValueInfo Info = {InValue, bNotify, Context, bSetupUndo, bPrintPythonCommnds, bFixEulerFlips};
		ControlValues.Add(ControlElement->GetKey(), Info);
		if (Context.SetKey == EControlRigSetKey::Always && IsInGameThread())
		{
			// Force evaluation to set keys in game thread
			Evaluate_AnyThread();
		}
	}
	else
	{
		DynamicHierarchy->SetControlValue(ControlElement, InValue, ERigControlValueType::Current, bSetupUndo, false, bPrintPythonCommnds, bFixEulerFlips);

		if (bNotify && OnControlModified.IsBound())
		{
			OnControlModified.Broadcast(this, ControlElement, Context);
		}
	}
}

void UControlRig::SwitchToParent(const FRigElementKey& InElementKey, const FRigElementKey& InNewParentKey, bool bInitial, bool bAffectChildren)
{
	FRigBaseElement* Element = DynamicHierarchy->Find<FRigBaseElement>(InElementKey);
	if(Element == nullptr)
	{
		return;
	}
	if (InNewParentKey.Type != ERigElementType::Reference)
	{
		FRigBaseElement* Parent = DynamicHierarchy->Find<FRigBaseElement>(InNewParentKey);
		if (Parent == nullptr)
		{
			return;
		}
	}
	if (bIsAdditive)
	{
		FRigSwitchParentInfo Info;
		Info.NewParent = InNewParentKey;
		Info.bInitial = bInitial;
		Info.bAffectChildren = bAffectChildren;
		SwitchParentValues.Add(InElementKey, Info);
	}
	else
	{
		// when simply executing space switching we need to make sure it runs through,
		// and during that we'll only respect parent child relationships (while ignoring VM dependencies).
		static const FEmptyRigDependenciesProvider EmptyRigDependenciesProvider;
		DynamicHierarchy->SwitchToParent(InElementKey, InNewParentKey, bInitial, bAffectChildren, EmptyRigDependenciesProvider, nullptr);
	}
}

bool UControlRig::SetControlGlobalTransform(const FName& InControlName, const FTransform& InGlobalTransform, bool bNotify, const FRigControlModifiedContext& Context, bool bSetupUndo, bool bPrintPythonCommands, bool bFixEulerFlips)
{
	FTransform GlobalTransform = InGlobalTransform;
	ERigTransformType::Type TransformType = ERigTransformType::CurrentGlobal;
	if (IsConstructionModeEnabled())
	{
#if WITH_EDITOR
		if(bRecordSelectionPoseForConstructionMode)
		{
			SelectionPoseForConstructionMode.FindOrAdd(FRigElementKey(InControlName, ERigElementType::Control)) = GlobalTransform;
		}
#endif
		TransformType = ERigTransformType::InitialGlobal;
		GlobalTransform = SetupControlFromGlobalTransform(InControlName, GlobalTransform);
	}

	FRigControlElement* Control = FindControl(InControlName);
	if (IsAdditive())
	{
		if (Control)
		{
			// For additive control rigs, proxy controls should not be stored and applied after the backwards solve like other controls
			// Instead, apply the transform as global, run the forward solve and let the driven controls request keying
			if (Control->Settings.AnimationType == ERigControlAnimationType::ProxyControl)
			{
				DynamicHierarchy->SetTransform(Control, GlobalTransform, ERigTransformType::CurrentGlobal, true);
				Execute(FRigUnit_BeginExecution::EventName);
				return true;
			}
		}
	}

	FRigControlValue Value = GetControlValueFromGlobalTransform(InControlName, GlobalTransform, TransformType);
	if (OnFilterControl.IsBound())
	{
		if (Control)
		{
			OnFilterControl.Broadcast(this, Control, Value);
		}
	}

	SetControlValue(InControlName, Value, bNotify, Context, bSetupUndo, bPrintPythonCommands, bFixEulerFlips);
	return true;
}

FRigControlValue UControlRig::GetControlValueFromGlobalTransform(const FName& InControlName, const FTransform& InGlobalTransform, ERigTransformType::Type InTransformType)
{
	FRigControlValue Value;

	if (FRigControlElement* ControlElement = FindControl(InControlName))
	{
		if(DynamicHierarchy)
		{
			if (bIsAdditive)
			{
				const int32 ControlIndex = ControlsAfterBackwardsSolve.GetIndex(ControlElement->GetKey());
				if (ControlIndex != INDEX_NONE)
				{
					// ParentGlobalTransform = ParentAnimationGlobalTransform + ParentAdditiveTransform
					const FTransform& ParentGlobalTransform = DynamicHierarchy->GetParentTransform(ControlElement, ERigTransformType::CurrentGlobal);

					// LocalTransform = InGlobal - ParentGlobalTransform - OffsetLocal
					const FTransform OffsetLocalTransform = InGlobalTransform.GetRelativeTransform(ParentGlobalTransform);
					const FTransform OffsetTransform = DynamicHierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentLocal);
					const FTransform LocalTransform = OffsetLocalTransform.GetRelativeTransform(OffsetTransform);

					// Additive = LocalTransform - AnimLocalTransform
					const FTransform& AnimLocalTransform = ControlsAfterBackwardsSolve[ControlIndex].LocalTransform;
					const FTransform AdditiveTransform = LocalTransform.GetRelativeTransform(AnimLocalTransform);

					Value.SetFromTransform(AdditiveTransform, ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
				}
			}
			else
			{
				FTransform Transform = DynamicHierarchy->ComputeLocalControlValue(ControlElement, InGlobalTransform, InTransformType);
				Value.SetFromTransform(Transform, ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
			}

			if (ShouldApplyLimits())
			{
				ControlElement->Settings.ApplyLimits(Value);
			}
		}
	}

	return Value;
}

void UControlRig::SetControlLocalTransform(const FName& InControlName, const FTransform& InLocalTransform, bool bNotify, const FRigControlModifiedContext& Context, bool bSetupUndo, bool bFixEulerFlips)
{
	if (FRigControlElement* ControlElement = FindControl(InControlName))
	{
		FRigControlValue Value;
		Value.SetFromTransform(InLocalTransform, ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);

		if (OnFilterControl.IsBound())
		{
			OnFilterControl.Broadcast(this, ControlElement, Value);
			
		}
		SetControlValue(InControlName, Value, bNotify, Context, bSetupUndo, false /*bPrintPython not defined!*/, bFixEulerFlips);
	}
}

FTransform UControlRig::GetControlLocalTransform(const FName& InControlName)
{
	if(DynamicHierarchy == nullptr)
	{
		return FTransform::Identity;
	}
	if (bIsAdditive)
	{
		FRigElementKey ControlKey (InControlName, ERigElementType::Control);
		if (FRigBaseElement* Element = DynamicHierarchy->Find(ControlKey))
		{
			if (FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
			{
				return GetControlValue(ControlElement, ERigControlValueType::Current).GetAsTransform(ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
			}
		}
	}
	return DynamicHierarchy->GetLocalTransform(FRigElementKey(InControlName, ERigElementType::Control));
}

FVector UControlRig::GetControlSpecifiedEulerAngle(const FRigControlElement* InControlElement, bool bIsInitial) const
{
	if (bIsAdditive)
	{
		const ERigControlValueType Type = bIsInitial ? ERigControlValueType::Initial : ERigControlValueType::Current;
		const FRigControlValue Value = GetControlValue(InControlElement->GetKey().Name);
		FRotator Rotator = Value.GetAsTransform(InControlElement->Settings.ControlType, InControlElement->Settings.PrimaryAxis).Rotator();
		return FVector(Rotator.Roll, Rotator.Pitch, Rotator.Yaw);
	}
	else
	{
		return DynamicHierarchy->GetControlSpecifiedEulerAngle(InControlElement, bIsInitial);
	}
}

const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& UControlRig::GetShapeLibraries() const
{
	const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>* LibrariesPtr = &ShapeLibraries;

	if (ShapeLibraries.IsEmpty())
	{
		if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
		{
			LibrariesPtr = &RuntimeAsset->GetShapeLibraries();
		}
		else if(!GetClass()->IsNative())
		{
			if (UControlRig* CDO = Cast<UControlRig>(GetClass()->GetDefaultObject()))
			{
				LibrariesPtr = &CDO->ShapeLibraries;
			}
		}
	}

	const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& Libraries = *LibrariesPtr;
	for(const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary : Libraries)
	{
		if (!ShapeLibrary.IsValid())
		{
			ShapeLibrary.LoadSynchronous();
		}
	}

	return Libraries;
}

void UControlRig::SelectControl(const FName& InControlName, bool bSelect, bool bSetupUndo)
{
	if(DynamicHierarchy)
	{
		if(URigHierarchyController* Controller = DynamicHierarchy->GetController(true))
		{
			Controller->SelectElement(FRigElementKey(InControlName, ERigElementType::Control), bSelect, false, bSetupUndo);
		}
	}
}

bool UControlRig::ClearControlSelection(bool bSetupUndo)
{
	if(DynamicHierarchy)
	{
		if(URigHierarchyController* Controller = DynamicHierarchy->GetController(true))
		{
			return Controller->ClearSelection(bSetupUndo);
		}
	}
	return false;
}

TArray<FName> UControlRig::GetSortedControls() const
{
	TArray<FRigControlElement*> SortedControls;
	GetControlsInOrder(SortedControls);
	TArray<FName> ControlNames;
	ControlNames.Reserve(SortedControls.Num());
	for (const FRigControlElement* ControlElement : SortedControls)
	{
		if (ControlElement)
		{
			ControlNames.Add(ControlElement->GetFName());
		}
	}
	return ControlNames;
}

TArray<FName> UControlRig::CurrentControlSelection() const
{
	TArray<FName> SelectedControlNames;

	if(DynamicHierarchy)
	{
		TArray<const FRigBaseElement*> SelectedControls = DynamicHierarchy->GetSelectedElements(ERigElementType::Control);
		for (const FRigBaseElement* SelectedControl : SelectedControls)
		{
			SelectedControlNames.Add(SelectedControl->GetFName());
		}
#if WITH_EDITOR
		for(const TSharedPtr<FRigDirectManipulationInfo>& ManipulationInfo : RigUnitManipulationInfos)
		{
			SelectedControlNames.Add(ManipulationInfo->ControlKey.Name);
		}
#endif
	}
	return SelectedControlNames;
}

bool UControlRig::IsControlSelected(const FName& InControlName)const
{
	if(DynamicHierarchy)
	{
		if(FRigControlElement* ControlElement = FindControl(InControlName))
		{
			return DynamicHierarchy->IsSelected(ControlElement);
		}
	}
	return false;
}

void UControlRig::EmitPostConstructionEventFinished()
{
	if (PostConstructionEvent.IsBound())
	{
		FControlRigBracketScope BracketScope(PostConstructionBracket);
		PostConstructionEvent.Broadcast(this, FRigUnit_PrepareForExecution::EventName);
	}
}

void UControlRig::SendSelectionBeforeInitNotifications()
{
	URigHierarchy* Hierarchy = GetHierarchy();
	const TArray<FRigHierarchyKey>& CurrentSelection = Hierarchy->GetSelectedHierarchyKeys();
	for(const FRigHierarchyKey& SelectedHierarchyKey : CurrentSelection)
	{
		if(!SelectionBeforeInit.Contains(SelectedHierarchyKey))
		{
			if(SelectedHierarchyKey.IsElement())
			{
				if(const FRigBaseElement* SelectedElement = Hierarchy->Find(SelectedHierarchyKey.GetElement()))
				{
					Hierarchy->Notify(ERigHierarchyNotification::ElementSelected, SelectedElement);
				}
			}
			if(SelectedHierarchyKey.IsComponent())
			{
				if(const FRigBaseComponent* SelectedComponent = Hierarchy->FindComponent(SelectedHierarchyKey.GetComponent()))
				{
					Hierarchy->Notify(ERigHierarchyNotification::ComponentSelected, SelectedComponent);
				}
			}
		}
	}
	for(const FRigHierarchyKey& PreviouslySelectedKey : SelectionBeforeInit)
	{
		if(!CurrentSelection.Contains(PreviouslySelectedKey))
		{
			if(PreviouslySelectedKey.IsElement())
			{
				if(const FRigBaseElement* SelectedElement = Hierarchy->Find(PreviouslySelectedKey.GetElement()))
				{
					Hierarchy->Notify(ERigHierarchyNotification::ElementDeselected, SelectedElement);
				}
			}
			if(PreviouslySelectedKey.IsComponent())
			{
				if(const FRigBaseComponent* SelectedComponent = Hierarchy->FindComponent(PreviouslySelectedKey.GetComponent()))
				{
					Hierarchy->Notify(ERigHierarchyNotification::ComponentDeselected, SelectedComponent);
				}
			}
		}
	}
}

#if RIGVM_TRACE_ENABLED

UObject* UControlRig::GetBoundOuterForTrace() const
{
	if (ObjectBinding.IsValid())
	{
		if (UObject* Owner = ObjectBinding->GetBoundObject())
		{
			return Owner;
		}
	}
	return Super::GetBoundOuterForTrace();
}

void UControlRig::TraceExecuteData(FRigVMTraceArchiveWriter& InArchive) const
{
	InArchive.UsingCustomVersion(FControlRigObjectVersion::GUID);
	
	Super::TraceExecuteData(InArchive);

	URigHierarchy* Hierarchy = GetHierarchy();
	bool bHasHierarchy = Hierarchy != nullptr;
	InArchive << bHasHierarchy;
	if (bHasHierarchy)
	{
		int64 PosBeforeHierarchy = InArchive.Tell();
		int64 PosAfterHierarchy = 0;
		InArchive << PosAfterHierarchy;
		
		Hierarchy->Serialize(InArchive);

		PosAfterHierarchy = InArchive.Tell();
		InArchive.Seek(PosBeforeHierarchy);
		InArchive << PosAfterHierarchy;
		InArchive.Seek(PosAfterHierarchy);
	}
}

void UControlRig::LoadTracedExecuteData(FRigVMTraceArchiveReader& InArchive)
{
	Super::LoadTracedExecuteData(InArchive);

	bool bHasHierarchy = false;
	InArchive << bHasHierarchy;
	if (bHasHierarchy)
	{
		int64 PosAfterHierarchy = 0;
		InArchive << PosAfterHierarchy;
		
		if (URigHierarchy* Hierarchy = GetHierarchy())
		{
			Hierarchy->Serialize(InArchive);
		}
		else
		{
			InArchive.Seek(PosAfterHierarchy);
		}
	}
}

#endif

void UControlRig::HandleHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy,
                                          const FRigNotificationSubject& InSubject)
{
	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;
	
	switch(InNotification)
	{
		case ERigHierarchyNotification::ElementSelected:
		case ERigHierarchyNotification::ElementDeselected:
		{
			bool bClearTransientControls = true;
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>((FRigBaseElement*)InElement))
			{
				const bool bSelected = InNotification == ERigHierarchyNotification::ElementSelected;
				ControlSelected().Broadcast(this, ControlElement, bSelected);

				OnControlSelected_BP.Broadcast(this, *ControlElement, bSelected);
				bClearTransientControls = !ControlElement->Settings.bIsTransientControl;
			}

#if WITH_EDITOR
			if(IsConstructionModeEnabled())
			{
				if(InElement->GetType() == ERigElementType::Control)
				{
					if(InNotification == ERigHierarchyNotification::ElementSelected)
					{
						SelectionPoseForConstructionMode.FindOrAdd(InElement->GetKey()) = GetHierarchy()->GetGlobalTransform(InElement->GetKey());
					}
					else
					{
						SelectionPoseForConstructionMode.Remove(InElement->GetKey());
					}
				}
			}
			if(bClearTransientControls)
			{
				ClearTransientControls();
			}
#endif
			break;
		}
		case ERigHierarchyNotification::ControlSettingChanged:
		case ERigHierarchyNotification::ControlShapeTransformChanged:
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>((FRigBaseElement*)InElement))
			{
				ControlModified().Broadcast(this, ControlElement, FRigControlModifiedContext(EControlRigSetKey::Never));
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

#if WITH_EDITOR

bool UControlRig::CanAddTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget, FString* OutFailureReason)
{
	if (InNode == nullptr)
	{
		if(OutFailureReason)
		{
			static const FString Reason = TEXT("Provided node is nullptr.");
			*OutFailureReason = Reason;
		}
		return false;
	}
	if (DynamicHierarchy == nullptr)
	{
		if(OutFailureReason)
		{
			static const FString Reason = TEXT("The rig does not contain a hierarchy.");
			*OutFailureReason = Reason;
		}
		return false;
	}

	const UScriptStruct* UnitStruct = InNode->GetScriptStruct();
	if(UnitStruct == nullptr)
	{
		if(OutFailureReason)
		{
			static const FString Reason = TEXT("The node is not resolved.");
			*OutFailureReason = Reason;
		}
		return false;
	}

	const TSharedPtr<FStructOnScope> NodeInstance = InNode->ConstructLiveStructInstance(this);
	if(!NodeInstance.IsValid() || !NodeInstance->IsValid())
	{
		if(OutFailureReason)
		{
			static const FString Reason = TEXT("Unexpected error: Node instance could not be constructed.");
			*OutFailureReason = Reason;
		}
		return false;
	}

	const FRigUnit* UnitInstance = GetRigUnitInstanceFromScope(NodeInstance);
	check(UnitInstance);

	TArray<FRigDirectManipulationTarget> Targets;
	if(UnitInstance->GetDirectManipulationTargets(InNode, NodeInstance, DynamicHierarchy, Targets, OutFailureReason))
	{
		return Targets.Contains(InTarget);
	}
	return false;
}

FName UControlRig::AddTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget)
{
	if(!CanAddTransientControl(InNode, InTarget, nullptr))
	{
		return NAME_None;
	}
	RemoveTransientControl(InNode, InTarget);

	URigHierarchyController* Controller = DynamicHierarchy->GetController(true);
	if(Controller == nullptr)
	{
		return NAME_None;
	}

	const UScriptStruct* UnitStruct = InNode->GetScriptStruct();
	check(UnitStruct);

	const TSharedPtr<FStructOnScope> NodeInstance = InNode->ConstructLiveStructInstance(this);
	check(NodeInstance.IsValid() && NodeInstance->IsValid());

	const FRigUnit* UnitInstance = GetRigUnitInstanceFromScope(NodeInstance);
	check(UnitInstance);

	const FName ControlName = GetNameForTransientControl(InNode, InTarget);
	FTransform ShapeTransform = FTransform::Identity;
	ShapeTransform.SetScale3D(FVector::ZeroVector);

	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Transform;
	Settings.bIsTransientControl = true;
	Settings.DisplayName = TEXT("Temporary Control");

	TSharedPtr<FRigDirectManipulationInfo> Info = MakeShareable(new FRigDirectManipulationInfo());
	Info->Target = InTarget;
	Info->Node = TWeakObjectPtr<const URigVMUnitNode>(InNode);

	FRigControlValue Value = FRigControlValue::Make(FTransform::Identity);
	UnitInstance->ConfigureDirectManipulationControl(InNode, Info, Settings, Value);

	Controller->ClearSelection();

    const FRigElementKey ControlKey = Controller->AddControl(
    	ControlName,
    	FRigElementKey(),
    	Settings,
    	Value,
    	FTransform::Identity,
    	ShapeTransform, false);

	Info->ControlKey = ControlKey;
	RigUnitManipulationInfos.Add(Info);
	SetTransientControlValue(InNode, Info);

	return Info->ControlKey.Name;
}

bool UControlRig::SetTransientControlValue(const URigVMUnitNode* InNode, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	check(InNode);
	check(DynamicHierarchy);
	
	const TSharedPtr<FStructOnScope> NodeInstance = InNode->ConstructLiveStructInstance(this);
	check(NodeInstance.IsValid());
	check(NodeInstance->IsValid());

	const UScriptStruct* UnitStruct = InNode->GetScriptStruct();
	check(UnitStruct);

	FRigUnit* UnitInstance = GetRigUnitInstanceFromScope(NodeInstance);
	check(UnitInstance);

	const FName ControlName = GetNameForTransientControl(InNode, InInfo->Target);
	const FRigControlElement* ControlElement = DynamicHierarchy->Find<FRigControlElement>({ ControlName, ERigElementType::Control });
	if(ControlElement == nullptr)
	{
		return false;
	}

	FControlRigExecuteContext& PublicContext = GetRigVMExtendedExecuteContext().GetPublicDataSafe<FControlRigExecuteContext>();
	const bool bResult = UnitInstance->UpdateHierarchyForDirectManipulation(InNode, NodeInstance, PublicContext, InInfo);
	InInfo->bInitialized = true;
	return bResult;
}

FName UControlRig::RemoveTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget)
{
	if ((InNode == nullptr) || (DynamicHierarchy == nullptr))
	{
		return NAME_None;
	}

	RigUnitManipulationInfos.RemoveAll([InTarget](const TSharedPtr<FRigDirectManipulationInfo>& Info) -> bool
	{
		return Info->Target == InTarget;
	});

	URigHierarchyController* Controller = DynamicHierarchy->GetController(true);
	if(Controller == nullptr)
	{
		return NAME_None;
	}

	const FName ControlName = GetNameForTransientControl(InNode, InTarget);
	if(FRigControlElement* ControlElement = FindControl(ControlName))
	{
		DynamicHierarchy->Notify(ERigHierarchyNotification::ElementDeselected, ControlElement);
		if(Controller->RemoveElement(ControlElement))
		{
#if WITH_EDITOR
			SelectionPoseForConstructionMode.Remove(FRigElementKey(ControlName, ERigElementType::Control));
#endif
			return ControlName;
		}
	}

	return NAME_None;
}

FName UControlRig::AddTransientControl(const FRigElementKey& InElement)
{
	if (!InElement.IsValid())
	{
		return NAME_None;
	}

	if(DynamicHierarchy == nullptr)
	{
		return NAME_None;
	}

	URigHierarchyController* Controller = DynamicHierarchy->GetController(true);
	if(Controller == nullptr)
	{
		return NAME_None;
	}

	const FName ControlName = GetNameForTransientControl(InElement);
	if(DynamicHierarchy->Contains(FRigElementKey(ControlName, ERigElementType::Control)))
	{
		SetTransientControlValue(InElement);
		return ControlName;
	}

	const int32 ElementIndex = DynamicHierarchy->GetIndex(InElement);
	if (ElementIndex == INDEX_NONE)
	{
		return NAME_None;
	}

	FTransform ShapeTransform = FTransform::Identity;
	ShapeTransform.SetScale3D(FVector::ZeroVector);

	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Transform;
	Settings.bIsTransientControl = true;
	Settings.DisplayName = TEXT("Temporary Control");

	FRigElementKey Parent;
	switch (InElement.Type)
	{
		case ERigElementType::Bone:
		{
			Parent = DynamicHierarchy->GetFirstParent(InElement);
			break;
		}
		case ERigElementType::Null:
		{
			Parent = InElement;
			break;
		}
		default:
		{
			break;
		}
	}

	const FRigElementKey ControlKey = Controller->AddControl(
        ControlName,
        Parent,
        Settings,
        FRigControlValue::Make(FTransform::Identity),
        FTransform::Identity,
        ShapeTransform, false);

	if (InElement.Type == ERigElementType::Bone)
	{
		// don't allow transient control to modify forward mode poses when we
		// already switched to the construction mode
		if (!IsConstructionModeEnabled())
		{
			if(FRigBoneElement* BoneElement = DynamicHierarchy->Find<FRigBoneElement>(InElement))
			{
				// add a modify bone AnimNode internally that the transient control controls for imported bones only
				// for user created bones, refer to UControlRig::TransformOverrideForUserCreatedBones 
				if (BoneElement->BoneType == ERigBoneType::Imported)
				{ 
					if (PreviewInstance)
					{
						PreviewInstance->ModifyBone(InElement.Name);
					}
				}
				else if (BoneElement->BoneType == ERigBoneType::User)
				{
					// add an empty entry, which will be given the correct value in
					// SetTransientControlValue(InElement);
					TransformOverrideForUserCreatedBones.FindOrAdd(InElement.Name);
				}
			}
		}
	}

	SetTransientControlValue(InElement);

	return ControlKey.Name;
}

bool UControlRig::SetTransientControlValue(const FRigElementKey& InElement)
{
	if (!InElement.IsValid())
	{
		return false;
	}

	if(DynamicHierarchy == nullptr)
	{
		return false;
	}

	const FName ControlName = GetNameForTransientControl(InElement);
	if(FRigControlElement* ControlElement = FindControl(ControlName))
	{
		if (InElement.Type == ERigElementType::Bone)
		{
			if (IsConstructionModeEnabled())
			{
				// need to get initial because that is what construction mode uses
				// specifically, when user change the initial from the details panel
				// this will allow the transient control to react to that change
				const FTransform InitialLocalTransform = DynamicHierarchy->GetInitialLocalTransform(InElement);
				DynamicHierarchy->SetTransform(ControlElement, InitialLocalTransform, ERigTransformType::InitialLocal, true, false);
				DynamicHierarchy->SetTransform(ControlElement, InitialLocalTransform, ERigTransformType::CurrentLocal, true, false);
			}
			else
			{
				const FTransform LocalTransform = DynamicHierarchy->GetLocalTransform(InElement);
				DynamicHierarchy->SetTransform(ControlElement, LocalTransform, ERigTransformType::InitialLocal, true, false);
				DynamicHierarchy->SetTransform(ControlElement, LocalTransform, ERigTransformType::CurrentLocal, true, false);

				if (FRigBoneElement* BoneElement = DynamicHierarchy->Find<FRigBoneElement>(InElement))
				{
					if (BoneElement->BoneType == ERigBoneType::Imported)
					{
						if (PreviewInstance)
						{
							if (FAnimNode_ModifyBone* Modify = PreviewInstance->FindModifiedBone(InElement.Name))
							{
								Modify->Translation = LocalTransform.GetTranslation();
								Modify->Rotation = LocalTransform.GetRotation().Rotator();
								Modify->TranslationSpace = EBoneControlSpace::BCS_ParentBoneSpace;
								Modify->RotationSpace = EBoneControlSpace::BCS_ParentBoneSpace;
							}
						}	
					}
					else if (BoneElement->BoneType == ERigBoneType::User)
					{
						if (FTransform* TransformOverride = TransformOverrideForUserCreatedBones.Find(InElement.Name))
						{
							*TransformOverride = LocalTransform;
						}	
					}
				}
			}
		}
		else if (InElement.Type == ERigElementType::Null)
		{
			const FTransform GlobalTransform = DynamicHierarchy->GetGlobalTransform(InElement);
			DynamicHierarchy->SetTransform(ControlElement, GlobalTransform, ERigTransformType::InitialGlobal, true, false);
			DynamicHierarchy->SetTransform(ControlElement, GlobalTransform, ERigTransformType::CurrentGlobal, true, false);
		}

		return true;
	}
	return false;
}

FName UControlRig::RemoveTransientControl(const FRigElementKey& InElement)
{
	if (!InElement.IsValid())
	{
		return NAME_None;
	}

	if(DynamicHierarchy == nullptr)
	{
		return NAME_None;
	}

	URigHierarchyController* Controller = DynamicHierarchy->GetController(true);
	if(Controller == nullptr)
	{
		return NAME_None;
	}

	const FName ControlName = GetNameForTransientControl(InElement);
	if(FRigControlElement* ControlElement = FindControl(ControlName))
	{
		DynamicHierarchy->Notify(ERigHierarchyNotification::ElementDeselected, ControlElement);
		if(Controller->RemoveElement(ControlElement))
		{
#if WITH_EDITOR
			SelectionPoseForConstructionMode.Remove(FRigElementKey(ControlName, ERigElementType::Control));
#endif
			return ControlName;
		}
	}

	return NAME_None;
}

FName UControlRig::GetNameForTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const
{
	check(InNode);
	check(DynamicHierarchy);
	
	const FString NodeName = InNode->GetName();
	return DynamicHierarchy->GetSanitizedName(FRigName(FString::Printf(TEXT("ControlForNode|%s|%s"), *NodeName, *InTarget.Name)));
}

FString UControlRig::GetNodeNameFromTransientControl(const FRigElementKey& InKey)
{
	FString Name = InKey.Name.ToString();
	if(Name.StartsWith(TEXT("ControlForNode|")))
	{
		Name.RightChopInline(15);
		Name.LeftInline(Name.Find(TEXT("|")));
	}
	else
	{
		return FString();
	}
	return Name;
}

FString UControlRig::GetTargetFromTransientControl(const FRigElementKey& InKey)
{
	FString Name = InKey.Name.ToString();
	if(Name.StartsWith(TEXT("ControlForNode|")))
	{
		Name.RightChopInline(15);
		Name.RightChopInline(Name.Find(TEXT("|")) + 1);
	}
	else
	{
		return FString();
	}
	return Name;
}

TSharedPtr<FRigDirectManipulationInfo> UControlRig::GetRigUnitManipulationInfoForTransientControl(
	const FRigElementKey& InKey)
{
	const TSharedPtr<FRigDirectManipulationInfo>* InfoPtr = RigUnitManipulationInfos.FindByPredicate(
		[InKey](const TSharedPtr<FRigDirectManipulationInfo>& Info) -> bool
		{
			return Info->ControlKey == InKey;
		}) ;

	if(InfoPtr)
	{
		return *InfoPtr;
	}

	return TSharedPtr<FRigDirectManipulationInfo>();
}

FName UControlRig::GetNameForTransientControl(const FRigElementKey& InElement)
{
	if (InElement.Type == ERigElementType::Control)
	{
		return InElement.Name;
	}

	const FName EnumName = *StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)InElement.Type).ToString();
	return *FString::Printf(TEXT("ControlForRigElement_%s_%s"), *EnumName.ToString(), *InElement.Name.ToString());
}

FRigElementKey UControlRig::GetElementKeyFromTransientControl(const FRigElementKey& InKey)
{
	if(InKey.Type != ERigElementType::Control)
	{
		return FRigElementKey();
	}
	
	static FString ControlRigForElementBoneName;
	static FString ControlRigForElementNullName;

	if (ControlRigForElementBoneName.IsEmpty())
	{
		ControlRigForElementBoneName = FString::Printf(TEXT("ControlForRigElement_%s_"),
            *StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)ERigElementType::Bone).ToString());
		ControlRigForElementNullName = FString::Printf(TEXT("ControlForRigElement_%s_"),
            *StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)ERigElementType::Null).ToString());
	}
	
	FString Name = InKey.Name.ToString();
	if(Name.StartsWith(ControlRigForElementBoneName))
	{
		Name.RightChopInline(ControlRigForElementBoneName.Len());
		return FRigElementKey(*Name, ERigElementType::Bone);
	}
	if(Name.StartsWith(ControlRigForElementNullName))
	{
		Name.RightChopInline(ControlRigForElementNullName.Len());
		return FRigElementKey(*Name, ERigElementType::Null);
	}
	
	return FRigElementKey();;
}

void UControlRig::ClearTransientControls()
{
	if(DynamicHierarchy == nullptr)
	{
		return;
	}

	URigHierarchyController* Controller = DynamicHierarchy->GetController(true);
	if(Controller == nullptr)
	{
		return;
	}

	if(bIsClearingTransientControls)
	{
		return;
	}
	TGuardValue<bool> ReEntryGuard(bIsClearingTransientControls, true);

	RigUnitManipulationInfos.Reset();

	const TArray<FRigControlElement*> ControlsToRemove = DynamicHierarchy->GetTransientControls();
	for (FRigControlElement* ControlToRemove : ControlsToRemove)
	{
		const FRigElementKey KeyToRemove = ControlToRemove->GetKey();
		if(Controller->RemoveElement(ControlToRemove))
		{
			SelectionPoseForConstructionMode.Remove(KeyToRemove);
		}
	}
}

void UControlRig::ApplyTransformOverrideForUserCreatedBones()
{
	if(DynamicHierarchy == nullptr)
	{
		return;
	}

	if (IsReadOnly())
	{
		return;
	}
	
	for (const auto& Entry : TransformOverrideForUserCreatedBones)
	{
		DynamicHierarchy->SetLocalTransform(FRigElementKey(Entry.Key, ERigElementType::Bone), Entry.Value, false);
	}
}

void UControlRig::ApplySelectionPoseForConstructionMode(const FName& InEventName)
{
	if (IsReadOnly())
	{
		return;
	}
	
	FRigControlModifiedContext ControlValueContext;
	ControlValueContext.EventName = InEventName;

	TGuardValue<bool> DisableRecording(bRecordSelectionPoseForConstructionMode, false);
	for(const TPair<FRigElementKey, FTransform>& Pair : SelectionPoseForConstructionMode)
	{
		const FName ControlName = Pair.Key.Name;
		const FRigControlValue Value = GetControlValueFromGlobalTransform(ControlName, Pair.Value, ERigTransformType::InitialGlobal);
		SetControlValue(ControlName, Value, true, ControlValueContext, false, false, false);
	}
}

#endif

void UControlRig::HandleHierarchyEvent(URigHierarchy* InHierarchy, const FRigEventContext& InEvent)
{
	if (RigEventDelegate.IsBound())
	{
		RigEventDelegate.Broadcast(InHierarchy, InEvent);
	}

	switch (InEvent.Event)
	{
		case ERigEvent::RequestAutoKey:
		{
			int32 Index = InHierarchy->GetIndex(InEvent.Key);
			if (Index != INDEX_NONE && InEvent.Key.Type == ERigElementType::Control)
			{
				if(FRigControlElement* ControlElement = InHierarchy->GetChecked<FRigControlElement>(Index))
				{
					FRigControlModifiedContext Context;
					Context.SetKey = EControlRigSetKey::Always;
					Context.LocalTime = InEvent.LocalTime;
					Context.EventName = InEvent.SourceEventName;
					ControlModified().Broadcast(this, ControlElement, Context);
				}
			}
			break;
		}
		case ERigEvent::OpenUndoBracket:
		case ERigEvent::CloseUndoBracket:
		{
			const bool bOpenUndoBracket = InEvent.Event == ERigEvent::OpenUndoBracket;
			ControlUndoBracketIndex = FMath::Max<int32>(0, ControlUndoBracketIndex + (bOpenUndoBracket ? 1 : -1));
			ControlUndoBracket().Broadcast(this, bOpenUndoBracket);
			break;
		}
		default:
		{
			break;
		}
	}
}

void UControlRig::GetControlsInOrder(TArray<FRigControlElement*>& SortedControls) const
{
	SortedControls.Reset();

	if(DynamicHierarchy == nullptr)
	{
		return;
	}

	SortedControls = DynamicHierarchy->GetElementsOfType<FRigControlElement>(true, true);
}

const FRigInfluenceMap* UControlRig::FindInfluenceMap(const FName& InEventName)
{
	if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
	{
		return RuntimeAsset->GetInfluences().Find(InEventName);
	}
	else if (UControlRig* CDO = GetClass()->GetDefaultObject<UControlRig>())
	{
		return CDO->Influences.Find(InEventName);
	}
	return nullptr;
}

#if WITH_EDITOR

void UControlRig::PreEditChange(FProperty* PropertyAboutToChange)
{
	// for BP user authored properties let's ignore changes since they
	// will be distributed from the BP anyway to all archetype instances.
	if(PropertyAboutToChange && !PropertyAboutToChange->IsNative())
	{
		if(const UBlueprint* Blueprint = Cast<UBlueprint>(GetClass()->ClassGeneratedBy))
		{
			const struct FBPVariableDescription* FoundVariable = Blueprint->NewVariables.FindByPredicate(
				[PropertyAboutToChange](const struct FBPVariableDescription& NewVariable)
				{
					return NewVariable.VarName == PropertyAboutToChange->GetFName();
				}
			);

			if(FoundVariable)
			{
				return;
			}
		}
	}
	
	Super::PreEditChange(PropertyAboutToChange);
}

void UControlRig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

FRigUnit* UControlRig::GetRigUnitInstanceFromScope(TSharedPtr<FStructOnScope> InScope)
{
	if(InScope.IsValid())
	{
		if(InScope->IsValid())
		{
			if(InScope->GetStruct()->IsChildOf(FRigUnit::StaticStruct()))
			{
				return (FRigUnit*)InScope->GetStructMemory(); 
			}
		}
	}
	static FStructOnScope DefaultRigUnitInstance(FRigUnit::StaticStruct());
	return (FRigUnit*)DefaultRigUnitInstance.GetStructMemory();
}

const TArray<UAssetUserData*>* UControlRig::GetAssetUserDataArray() const
{
	if(HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		if (IsRunningCookCommandlet())
		{
			return &ToRawPtrTArrayUnsafe(AssetUserData);
		}
		else
		{
			static thread_local TArray<TObjectPtr<UAssetUserData>> CachedAssetUserData;
			CachedAssetUserData.Reset();
			CachedAssetUserData.Append(AssetUserData);
			CachedAssetUserData.Append(AssetUserDataEditorOnly);
			return &ToRawPtrTArrayUnsafe(CachedAssetUserData);
		}
#else
		return &ToRawPtrTArrayUnsafe(AssetUserData);
#endif
	}

	CombinedAssetUserData.Reset();

	if (GeneratedBy)
	{
		CombinedAssetUserData.Append(*GeneratedBy->GetAssetUserDataArray());
	}
	else if (UControlRig* CDO = Cast<UControlRig>(GetClass()->GetDefaultObject(false)))
	{
		CombinedAssetUserData.Append(*CDO->GetAssetUserDataArray());
	}
	else
	{
		CombinedAssetUserData.Append(AssetUserData);
#if WITH_EDITOR
		if (!IsRunningCookCommandlet())
		{
			CombinedAssetUserData.Append(AssetUserDataEditorOnly);
		}
#endif
	}
	
	if(GetExternalAssetUserDataDelegate.IsBound())
	{
		CombinedAssetUserData.Append(GetExternalAssetUserDataDelegate.Execute());
	}

	// find the data assets on the external variable list
	TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariablesImpl(false);
	for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
	{
		if(ExternalVariable.GetMemory() != nullptr)
		{
			if(const UClass* Class = Cast<UClass>(ExternalVariable.GetCPPTypeObject()))
			{
				if(Class->IsChildOf(UDataAsset::StaticClass()))
				{
					TObjectPtr<UDataAsset>& DataAsset = *(TObjectPtr<UDataAsset>*)ExternalVariable.GetMemory();
					if(IsValid(DataAsset) && !DataAsset->GetFName().IsNone())
					{
						if(const TObjectPtr<UDataAssetLink>* ExistingDataAssetLink =
							ExternalVariableDataAssetLinks.Find(ExternalVariable.GetName()))
						{
							if(!IsValid(*ExistingDataAssetLink) ||
								(*ExistingDataAssetLink)->HasAnyFlags(EObjectFlags::RF_BeginDestroyed))
							{
								ExternalVariableDataAssetLinks.Remove(ExternalVariable.GetName());
							}
						}
						if(!ExternalVariableDataAssetLinks.Contains(ExternalVariable.GetName()))
						{
							ExternalVariableDataAssetLinks.Add(
								ExternalVariable.GetName(),
								NewObject<UDataAssetLink>(GetTransientPackage(), UDataAssetLink::StaticClass(), NAME_None, RF_Transient));
						}

						TObjectPtr<UDataAssetLink>& DataAssetLink = ExternalVariableDataAssetLinks.FindChecked(ExternalVariable.GetName());
						DataAssetLink->NameSpace = ExternalVariable.GetName().ToString();
						DataAssetLink->SetDataAsset(DataAsset);
						CombinedAssetUserData.Add(DataAssetLink);
					}
				}
			}
		}
	}

	// Propagate the outer rig user data into the child modules
	if (UControlRig* OuterCR = GetTypedOuter<UControlRig>())
	{
		if(const TArray<UAssetUserData*>* OuterUserDataArray = OuterCR->GetAssetUserDataArray())
		{
			for(UAssetUserData* OuterUserData : *OuterUserDataArray)
			{
				CombinedAssetUserData.Add(OuterUserData);
			}
		}
	}
	
	if(OuterSceneComponent.IsValid())
	{
		if(const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(OuterSceneComponent))
		{
			if(USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				if(const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
				{
					if(const TArray<UAssetUserData*>* SkeletonUserDataArray = Skeleton->GetAssetUserDataArray())
					{
						for(UAssetUserData* SkeletonUserData : *SkeletonUserDataArray)
						{
							CombinedAssetUserData.Add(SkeletonUserData);
						}
					}
				}
				if(const TArray<UAssetUserData*>* SkeletalMeshUserDataArray = SkeletalMesh->GetAssetUserDataArray())
				{
					for(UAssetUserData* SkeletalMeshUserData : *SkeletalMeshUserDataArray)
					{
						CombinedAssetUserData.Add(SkeletalMeshUserData);
					}
				}
			}
			if (const TArray<UAssetUserData*>* ActorComponentUserDataArray = SkeletalMeshComponent->GetAssetUserDataArray())
			{
				for(UAssetUserData* ActorComponentUserData : *ActorComponentUserDataArray)
				{
					CombinedAssetUserData.Add(ActorComponentUserData);
				}
			}
		}
	}

	return &ToRawPtrTArrayUnsafe(CombinedAssetUserData);
}

void UControlRig::CopyPoseFromOtherRig(UControlRig* Subject)
{
	if (IsReadOnly())
	{
		return;
	}

	check(DynamicHierarchy);
	check(Subject);
	URigHierarchy* OtherHierarchy = Subject->GetHierarchy();
	check(OtherHierarchy);

	for (FRigBaseElement* Element : *DynamicHierarchy)
	{
		FRigBaseElement* OtherElement = OtherHierarchy->Find(Element->GetKey());
		if(OtherElement == nullptr)
		{
			continue;
		}

		if(OtherElement->GetType() != Element->GetType())
		{
			continue;
		}

		if(FRigBoneElement* BoneElement = Cast<FRigBoneElement>(Element))
		{
			FRigBoneElement* OtherBoneElement = CastChecked<FRigBoneElement>(OtherElement);
			const FTransform Transform = OtherHierarchy->GetTransform(OtherBoneElement, ERigTransformType::CurrentLocal);
			DynamicHierarchy->SetTransform(BoneElement, Transform, ERigTransformType::CurrentLocal, true, false);
		}
		else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Element))
		{
			FRigCurveElement* OtherCurveElement = CastChecked<FRigCurveElement>(OtherElement);
			const float Value = OtherHierarchy->GetCurveValue(OtherCurveElement);
			DynamicHierarchy->SetCurveValue(CurveElement, Value, false);
		}
	}
}

void UControlRig::SetBoneInitialTransformsFromAnimInstance(UAnimInstance* InAnimInstance)
{
	if (IsReadOnly())
	{
		return;
	}

	FMemMark Mark(FMemStack::Get());
	FCompactPose OutPose;
	OutPose.ResetToRefPose(InAnimInstance->GetRequiredBones());
	SetBoneInitialTransformsFromCompactPose(&OutPose);
}

void UControlRig::SetBoneInitialTransformsFromAnimInstanceProxy(const FAnimInstanceProxy* InAnimInstanceProxy)
{
	if (IsReadOnly())
	{
		return;
	}

	FMemMark Mark(FMemStack::Get());
	FCompactPose OutPose;
	OutPose.ResetToRefPose(InAnimInstanceProxy->GetRequiredBones());
	SetBoneInitialTransformsFromCompactPose(&OutPose);
}

void UControlRig::SetBoneInitialTransformsFromSkeletalMeshComponent(USkeletalMeshComponent* InSkelMeshComp, bool bUseAnimInstance)
{
	if (IsReadOnly())
	{
		return;
	}

	check(InSkelMeshComp);
	check(DynamicHierarchy);
	
	if (!bUseAnimInstance && (InSkelMeshComp->GetAnimInstance() != nullptr))
	{
		SetBoneInitialTransformsFromAnimInstance(InSkelMeshComp->GetAnimInstance());
	}
	else if(InSkelMeshComp->GetSkeletalMeshAsset())
	{
		SetBoneInitialTransformsFromSkeletalMesh(InSkelMeshComp->GetSkeletalMeshAsset());
	}
}


void UControlRig::SetBoneInitialTransformsFromSkeletalMesh(USkeletalMesh* InSkeletalMesh)
{
	if (IsReadOnly())
	{
		return;
	}

	if (ensure(InSkeletalMesh))
	{ 
		SetBoneInitialTransformsFromRefSkeleton(InSkeletalMesh->GetRefSkeleton());
	}
}

void UControlRig::SetBoneInitialTransformsFromRefSkeleton(const FReferenceSkeleton& InReferenceSkeleton)
{
	if (IsReadOnly())
	{
		return;
	}

	check(DynamicHierarchy);

	DynamicHierarchy->ForEach<FRigBoneElement>([this, InReferenceSkeleton](FRigBoneElement* BoneElement) -> bool
	{
		if(BoneElement->BoneType == ERigBoneType::Imported)
		{
			const int32 BoneIndex = InReferenceSkeleton.FindBoneIndex(BoneElement->GetFName());
			if (BoneIndex != INDEX_NONE)
			{
				const FTransform LocalInitialTransform = InReferenceSkeleton.GetRefBonePose()[BoneIndex];
				DynamicHierarchy->SetTransform(BoneElement, LocalInitialTransform, ERigTransformType::InitialLocal, true, false);
				DynamicHierarchy->SetTransform(BoneElement, LocalInitialTransform, ERigTransformType::CurrentLocal, true, false);
			}
		}
		return true;
	});
	bResetInitialTransformsBeforeConstruction = false;
	RequestConstruction();
}

void UControlRig::SetBoneInitialTransformsFromCompactPose(FCompactPose* InCompactPose)
{
	if (IsReadOnly())
	{
		return;
	}

	check(InCompactPose);

	if(!InCompactPose->IsValid())
	{
		return;
	}
	if(!InCompactPose->GetBoneContainer().IsValid())
	{
		return;
	}
	
	FMemMark Mark(FMemStack::Get());

	DynamicHierarchy->ForEach<FRigBoneElement>([this, InCompactPose](FRigBoneElement* BoneElement) -> bool
		{
			if (BoneElement->BoneType == ERigBoneType::Imported)
			{
				int32 MeshIndex = InCompactPose->GetBoneContainer().GetPoseBoneIndexForBoneName(BoneElement->GetFName());
				if (MeshIndex != INDEX_NONE)
				{
					FCompactPoseBoneIndex CPIndex = InCompactPose->GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshIndex));
					if (CPIndex != INDEX_NONE)
					{
						FTransform LocalInitialTransform = InCompactPose->GetRefPose(CPIndex);
						DynamicHierarchy->SetTransform(BoneElement, LocalInitialTransform, ERigTransformType::InitialLocal, true, false);
					}
				}
			}
			return true;
		});

	bResetInitialTransformsBeforeConstruction = false;
	RequestConstruction();
}

const FRigControlElementCustomization* UControlRig::GetControlCustomization(const FRigElementKey& InControl) const
{
	check(InControl.Type == ERigElementType::Control);

	if(const FRigControlElementCustomization* Customization = ControlCustomizations.Find(InControl))
	{
		return Customization;
	}

	if(DynamicHierarchy)
	{
		if(const FRigControlElement* ControlElement = DynamicHierarchy->Find<FRigControlElement>(InControl))
		{
			return &ControlElement->Settings.Customization;
		}
	}

	return nullptr;
}

void UControlRig::SetControlCustomization(const FRigElementKey& InControl, const FRigControlElementCustomization& InCustomization)
{
	if (IsReadOnly())
	{
		return;
	}

	check(InControl.Type == ERigElementType::Control);
	
	ControlCustomizations.FindOrAdd(InControl) = InCustomization;
}

void UControlRig::PostInitInstanceIfRequired()
{
	if(GetHierarchy() == nullptr || VM == nullptr)
	{
		if(HasAnyFlags(RF_ClassDefaultObject))
		{
			PostInitInstance(nullptr);
		}
		else
		{
			if (GeneratedBy)
			{
				ensure(VM == nullptr || VM == GeneratedBy->GetVM());
				PostInitInstance(nullptr);
			}
			else
			{
				UControlRig* CDO = GetClass()->GetDefaultObject<UControlRig>();
				ensure(VM == nullptr || VM == CDO->VM);
				PostInitInstance(CDO);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void UControlRig::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(URigHierarchy::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UAnimationDataSourceRegistry::StaticClass()));
}

#endif

USceneComponent* UControlRig::GetOwningSceneComponent()
{
	if(OuterSceneComponent == nullptr)
	{
		const FControlRigExecuteContext& PublicContext = GetRigVMExtendedExecuteContext().GetPublicDataSafe<FControlRigExecuteContext>();
		const FRigUnitContext& Context = PublicContext.UnitContext;

		if (Context.DataSourceRegistry)
		{
			USceneComponent* SceneComponentFromRegistry = Context.DataSourceRegistry->RequestSource<USceneComponent>(UControlRig::OwnerComponent);
			if (SceneComponentFromRegistry)
			{
				OuterSceneComponent = SceneComponentFromRegistry;
			}
		}

		if(OuterSceneComponent == nullptr)
		{
			OuterSceneComponent = Super::GetOwningSceneComponent();
		}
	}
	return OuterSceneComponent.Get();
}

void UControlRig::PostInitInstance(URigVMHost* InCDO)
{
	const EObjectFlags SubObjectFlags =
	HasAnyFlags(RF_ClassDefaultObject) ?
		RF_Public | RF_DefaultSubObject :
		RF_Transient | RF_Transactional;

	FRigVMExtendedExecuteContext& Context = GetRigVMExtendedExecuteContext();
	
	Context.SetContextPublicDataStruct(FControlRigExecuteContext::StaticStruct());

	UpdateVMSettings();

	// set up the hierarchy
	{
		// If this is not a CDO, it should have never saved its hierarchy. However, we have found that some rigs in the past
		// did save their hierarchies. If that's the case, let's mark them as garbage and rename them before creating our own hierarchy.
		if (!HasAnyFlags(RF_ClassDefaultObject))
		{
			FGCScopeGuard Guard;
			// Don't replace the DynamicHierarchy object if it's not Local. We need to keep Remote/LocalNotReady DynamicHierarchy object so that we don't accidentally create another one with the same remote id when migrating from another server
			URigHierarchy* ObjectFound = Cast<URigHierarchy>(StaticFindObjectFast(URigHierarchy::StaticClass(), this, TEXT("DynamicHierarchy"), EFindObjectFlags::ExactClass, RF_DefaultSubObject, EInternalObjectFlags::Remote | EInternalObjectFlags::LocalNotReady));
			if (ObjectFound)
			{
				FName NewName = MakeUniqueObjectName(GetTransientPackage(), URigHierarchy::StaticClass(), TEXT("DynamicHierarchy_Deleted"));
				ObjectFound->Rename(*NewName.ToString(), GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				ObjectFound->MarkAsGarbage();
			}
		}

		if(!IsRigModuleInstance())
		{
			DynamicHierarchy = NewObject<URigHierarchy>(this, TEXT("DynamicHierarchy"), SubObjectFlags);

			if (!IsInGameThread())
			{
				// If the object was created on a non-game thread, clear the async flag immediately, so that it can be
				// garbage collected in the future. 
				(void)DynamicHierarchy->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
			}
		}
	}

#if WITH_EDITOR
	if(!IsRigModuleInstance())
	{
		const TWeakObjectPtr<UControlRig> WeakThis = this;
		DynamicHierarchy->OnUndoRedo().AddStatic(&UControlRig::OnHierarchyTransformUndoRedoWeak, WeakThis);
	}
#endif

	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		URigVM* SourceVM = nullptr;
		if (InCDO)
		{
			SourceVM = InCDO->GetVM();
		}
		else if (GeneratedBy)
		{
			SourceVM = GeneratedBy->GetVM();
		}
		
		if (SourceVM)
		{
			ensure(VM == nullptr || VM == SourceVM);
			if (VM == nullptr) // this is needed for some EngineTests, on a normal setup, the VM is set to the CDO VM already
			{
				VM = SourceVM;
			}
		}
	}
	else // we are the CDO
	{
		check(InCDO == nullptr);

		if (VM == nullptr)
		{
			VM = NewObject<URigVM>(this, TEXT("ControlRig_VM"), SubObjectFlags);

			if (!IsInGameThread())
			{
				// If the object was created on a non-game thread, clear the async flag immediately, so that it can be
				// garbage collected in the future. 
				(void)VM->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
			}
		}

		// for default objects we need to check if the CDO is rooted. specialized Control Rigs
		// such as the FK control rig may not have a root since they are part of a C++ package.

		// since the sub objects are created after the constructor
		// GC won't consider them part of the CDO, even if they have the sub object flags
		// so even if CDO is rooted and references these sub objects, 
		// it is not enough to keep them alive.
		// Hence, we have to add them to root here.
		if(GetClass()->IsNative())
		{
			VM->AddToRoot();

			if(!IsRigModuleInstance())
			{
				DynamicHierarchy->AddToRoot();
			}
		}
	}

	if(UControlRig* CDOControlRig = Cast<UControlRig>(InCDO))
	{
		if (DynamicHierarchy)
		{
			ElementKeyRedirector = FRigElementKeyRedirector(CDOControlRig->ElementKeyRedirector, DynamicHierarchy);
		}
	}

	RequestInit();
}

void UControlRig::SetDynamicHierarchy(TObjectPtr<URigHierarchy> InHierarchy)
{
	// Delete any existing hierarchy
	if (DynamicHierarchy && DynamicHierarchy->GetOuter() == this)
	{
		DynamicHierarchy->OnUndoRedo().RemoveAll(this);
		DynamicHierarchy->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		DynamicHierarchy->MarkAsGarbage();
	}
	DynamicHierarchy = InHierarchy;
}

UTransformableControlHandle* UControlRig::CreateTransformableControlHandle(
	const FName& InControlName) const
{
	const FRigControlElement* ControlElement = FindControl(InControlName);
	auto IsConstrainable = [ControlElement]()
	{
		if (!ControlElement)
		{
			return false;
		}
		
		const FRigControlSettings& ControlSettings = ControlElement->Settings;
		if (ControlSettings.ControlType == ERigControlType::Bool ||
			ControlSettings.ControlType == ERigControlType::Float ||
			ControlSettings.ControlType == ERigControlType::ScaleFloat ||
			ControlSettings.ControlType == ERigControlType::Integer)
		{
			return false;
		}

		return true;
	};

	if (!IsConstrainable())
	{
		return nullptr;
	}
	
	UTransformableControlHandle* CtrlHandle = NewObject<UTransformableControlHandle>(GetTransientPackage(), NAME_None, RF_Transactional);
	check(CtrlHandle);
	CtrlHandle->ControlRig = const_cast<UControlRig*>(this);
	CtrlHandle->ControlName = InControlName;
	
	if (ControlElement->GetFName() != InControlName)
	{
		CtrlHandle->ControlName = ControlElement->GetFName();
		
#if WITH_EDITOR
		const FRigElementKey ControlKey(InControlName, ERigElementType::Control);
		if (DynamicHierarchy->ReceivedNameSpaceBasedKeys.Contains(ControlKey))
		{
			UE_LOGF(
				LogControlRig,
				Warning,
				"%ls: control '%ls' has been accessed using a namespace based key ('%ls') when creating the control handle. Please consider updating your code.",
				*GetName(),
				*ControlElement->GetName(),
				*InControlName.ToString()
			);	
		}
#endif
	}
	
	CtrlHandle->RegisterDelegates();
	return CtrlHandle;
}

void UControlRig::OnHierarchyTransformUndoRedo(URigHierarchy* InHierarchy, const FRigElementKey& InKey, ERigTransformType::Type InTransformType, const FTransform& InTransform, bool bIsUndo)
{
	if(InKey.Type == ERigElementType::Control)
	{
		if(FRigControlElement* ControlElement = InHierarchy->Find<FRigControlElement>(InKey))
		{
			ControlModified().Broadcast(this, ControlElement, FRigControlModifiedContext(EControlRigSetKey::Never));
		}
	}
}

void UControlRig::GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& Context)
{
	Super::GenerateRequiredPluginsData(Context);

	if (const URigHierarchy* Hierarchy = GetHierarchy())
	{
		for (int32 Index = 0; Index < Hierarchy->ElementComponents.Num(); Index++)
		{
			if(Hierarchy->ElementComponents[Index].IsValid())
			{
				const FRigBaseComponent* Component = Hierarchy->ElementComponents[Index].GetPtr<FRigBaseComponent>();
				const FString PluginName = RigVMTypeUtils::GetPluginName(Component->GetScriptStruct());
				if (!PluginName.IsEmpty())
				{
					RequiredPlugins.AddUnique(PluginName);
				}
			}
		}
	}
}

UControlRig::FPoseScope::FPoseScope(UControlRig* InControlRig, ERigElementType InFilter, const TArray<FRigElementKey>& InElements, const ERigTransformType::Type InTransformType)
: ControlRig(InControlRig)
, Filter(InFilter)
, TransformType(InTransformType)
{
	check(InControlRig);
	const TArrayView<const FRigElementKey> ElementView(InElements.GetData(), InElements.Num());
	CachedPose = InControlRig->GetHierarchy()->GetPose(IsInitial(InTransformType), InFilter, ElementView);
}

UControlRig::FPoseScope::~FPoseScope()
{
	check(ControlRig);

	ControlRig->GetHierarchy()->SetPose(CachedPose, TransformType);
}

#if WITH_EDITOR

UControlRig::FTransientControlScope::FTransientControlScope(TObjectPtr<URigHierarchy> InHierarchy)
	:Hierarchy(InHierarchy)
{
	for (FRigControlElement* Control : Hierarchy->GetTransientControls())
	{
		FTransientControlInfo Info;
		Info.Name = Control->GetFName();
		Info.Parent = Hierarchy->GetFirstParent(Control->GetKey());
		Info.Settings = Control->Settings;
		// preserve whatever value that was produced by this transient control at the moment
		Info.Value = Hierarchy->GetControlValue(Control->GetKey(),ERigControlValueType::Current);
		Info.OffsetTransform = Hierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentLocal);
		Info.ShapeTransform = Hierarchy->GetControlShapeTransform(Control, ERigTransformType::CurrentLocal);
				
		SavedTransientControls.Add(Info);
	}
}

UControlRig::FTransientControlScope::~FTransientControlScope()
{
	if (URigHierarchyController* Controller = Hierarchy->GetController())
	{
		for (const FTransientControlInfo& Info : SavedTransientControls)
		{
			Controller->AddControl(
				Info.Name,
				Info.Parent,
				Info.Settings,
				Info.Value,
				Info.OffsetTransform,
				Info.ShapeTransform,
				false,
				false
			);
		}
	}
}

#endif

uint32 UControlRig::GetShapeLibraryHash() const
{
	uint32 Hash = 0;
	for(const TPair<FString, FString>& Pair : ShapeLibraryNameMap)
	{
		Hash = HashCombine(Hash, HashCombine(GetTypeHash(Pair.Key), GetTypeHash(Pair.Value)));
	}
	for(const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary : ShapeLibraries)
	{
		Hash = HashCombine(Hash, GetTypeHash(ShapeLibrary.GetUniqueID()));
	}
	return Hash;
}

int32 UControlRig::GetReplayTimeIndex() const
{
	return ReplayTimeIndex;
}

void UControlRig::SetReplayTimeIndex(int32 InReplayTimeIndex)
{
	ReplayTimeIndex = InReplayTimeIndex;
}

void UControlRig::DisableReplay()
{
	Replay = nullptr;
}

bool UControlRig::IsReplayEnabled() const
{
	return GetReplayTimeIndex() != INDEX_NONE && Replay != nullptr;
}

void UControlRig::SetReplay(UControlRigReplay* InReplay)
{
	if (IsReadOnly())
	{
		return;
	}

	if(InReplay == nullptr)
	{
		DisableReplay();
		return;
	}
	Replay = InReplay;
}

#undef LOCTEXT_NAMESPACE
