<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="UE::MovieScene::FAccumulatedControlRigValues">
    <DisplayString>{ ValuesArray }</DisplayString>
    <Expand>
      <ExpandedItem>ValuesArray</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::FControlRigParameterValues">
    <DisplayString>{ Headers }</DisplayString>
    <Expand>
      <ExpandedItem>Headers</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::MovieScene::FControlRigParameterValueHeader">

    <Intrinsic Name="Align" Expression="(uint8*)(((uint64)Val + Alignment - 1) &amp; ~(Alignment - 1))">
      <Parameter Name="Val" Type="uint8*" />
    </Intrinsic>

    <DisplayString>Type={Type}, Num={NumElements}</DisplayString>

    <Expand>
      <CustomListItems>
      <Variable Name="i" InitialValue="0" />
      <Variable Name="Names" InitialValue="(FName*)Data" />
      <Variable Name="Values" InitialValue="(uint8*)Align(Data + sizeof(FName)*Capacity)" />   <!-- Space -->

      <Size>NumElements</Size>
      <Loop>
        <If Condition="i == NumElements">
          <Break/>
        </If>
        <Item Condition="Type == 0" Name="{Names[i]}">((FMovieSceneControlRigSpaceBaseKey*)Values)[i]</Item>   <!-- Space -->
        <Item Condition="Type == 1" Name="{Names[i]}">((bool*)Values)[i]</Item>                                <!-- Parameter_Bool -->
        <Item Condition="Type == 2" Name="{Names[i]}">((uint8*)Values)[i]</Item>                               <!-- Parameter_Enum -->
        <Item Condition="Type == 3" Name="{Names[i]}">((int32*)Values)[i]</Item>                               <!-- Parameter_Integer -->
        <Item Condition="Type == 4" Name="{Names[i]}">((float*)Values)[i]</Item>                               <!-- Parameter_Scalar -->
        <Item Condition="Type == 5" Name="{Names[i]}">((FVector3f*)Values)[i]</Item>                           <!-- Parameter_Vector -->
        <Item Condition="Type == 6" Name="{Names[i]}">((FEulerTransform*)Values)[i]</Item>                     <!-- Parameter_Transform -->
        <Exec>++i</Exec>
      </Loop>
      </CustomListItems>
    </Expand>
  </Type>

</AutoVisualizer>