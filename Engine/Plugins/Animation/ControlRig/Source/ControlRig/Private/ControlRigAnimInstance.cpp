// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigAnimInstance.h"

#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigAnimInstance)

////////////////////////////////////////////////////////////////////////////////////////
void FControlRigAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimInstanceProxy::Initialize(InAnimInstance);
}

bool FControlRigAnimInstanceProxy::Evaluate(FPoseContext& Output)
{
	if (StoredTransforms.Num() == 0 && StoredCurves.Num() == 0 && !StoredAttributes.ContainsData())
	{
		return false;
	}

	FComponentSpacePoseContext CSContext(this);
	CSContext.Pose.InitPose(Output.Pose);

	TArray<FCompactPoseBoneIndex> ModifiedBones;
	
	const FBoneContainer& BoneContainer = Output.Pose.GetBoneContainer();
	
	for (TPair<FMeshPoseBoneIndex, FTransform>& StoredTransform : StoredTransforms)
	{
		const FMeshPoseBoneIndex& MeshPoseBoneIndexToModify = StoredTransform.Key;
		FSkeletonPoseBoneIndex SkeletonPoseBoneIndex = BoneContainer.GetSkeletonPoseIndexFromMeshPoseIndex(MeshPoseBoneIndexToModify);
		FCompactPoseBoneIndex CompactIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonPoseBoneIndex);
		
		if (CompactIndex.GetInt() != INDEX_NONE)
		{
			CSContext.Pose.SetComponentSpaceTransform(CompactIndex, StoredTransform.Value);
			ModifiedBones.Add(CompactIndex);
		}
	}

	FCompactPose CompactPose(Output.Pose);
	FCSPose<FCompactPose>::ConvertComponentPosesToLocalPosesSafe(CSContext.Pose, CompactPose);

	// reset to ref pose before setting the pose to ensure if we don't have any missing bones
	Output.ResetToRefPose();

	for (const FCompactPoseBoneIndex& ModifiedBone : ModifiedBones)
	{
		Output.Pose[ModifiedBone] = CompactPose[ModifiedBone];
	}

	FBlendedCurve Curve;
	UE::Anim::FCurveUtils::BuildUnsorted(Curve, StoredCurves);
	Output.Curve.Combine(Curve);

	Output.CustomAttributes.CopyFrom(StoredAttributes, Output.Pose.GetBoneContainer());

	return true;
}

void FControlRigAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	// such that AnimScriptInstance->GetUpdateCounter().HasEverBeenUpdated() is accurate
	UpdateCounter.Increment();
}

FControlRigAnimInstanceProxy::~FControlRigAnimInstanceProxy()
{
}

////////////////////////////////////////////////////////////////////////////////////////

UControlRigAnimInstance::UControlRigAnimInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = false;
}

FAnimInstanceProxy* UControlRigAnimInstance::CreateAnimInstanceProxy()
{
	return new FControlRigAnimInstanceProxy(this);
}


