// Copyright Epic Games, Inc. All Rights Reserved.

#include  "ControlRigAssetReference.h"

#include "ControlRig.h"
#include "ControlRigBlueprintGeneratedClass.h"
#include "ControlRigRuntimeAsset.h"
#include "ModularRig.h"


FControlRigAssetStrongReference::FControlRigAssetStrongReference(UObject* InAsset)
{
	if (!InAsset)
	{
		return;
	}
	if (UBlueprint* Blueprint = Cast<UBlueprint>(InAsset))
	{
		BlueprintRigClass = Blueprint->GeneratedClass.Get();
	}
	else if (UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(InAsset))
	{
		BlueprintRigClass = GeneratedClass;
	}
	else if (UControlRigRuntimeAsset* RuntimeAsset = Cast<UControlRigRuntimeAsset>(InAsset))
	{
		ControlRigAsset = RuntimeAsset;
	}
	else if (TSubclassOf<UControlRig> ControlRigClass = Cast<UClass>(InAsset)) // in the case of FKControlRig::StaticClass()
	{
		BlueprintRigClass = ControlRigClass.Get();
	}
}

FControlRigAssetStrongReference::FControlRigAssetStrongReference(TSubclassOf<UControlRig> InClass)
	: BlueprintRigClass(InClass), ControlRigAsset(nullptr)
{}

FControlRigAssetStrongReference::FControlRigAssetStrongReference(TObjectPtr<UControlRigRuntimeAsset> InRuntimeAsset)
: BlueprintRigClass(nullptr), ControlRigAsset(InRuntimeAsset)
{}

void FControlRigAssetStrongReference::Reset()
{
	BlueprintRigClass = nullptr;
	ControlRigAsset = nullptr;
}

bool FControlRigAssetStrongReference::operator==(const FControlRigAssetStrongReference& InOtherValue) const
{
	return BlueprintRigClass == InOtherValue.BlueprintRigClass && ControlRigAsset == InOtherValue.ControlRigAsset;
}

bool FControlRigAssetStrongReference::IsValid() const
{
	return BlueprintRigClass != nullptr || ControlRigAsset != nullptr;
}

FString FControlRigAssetStrongReference::GetName() const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass->GetName();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->GetName();
	}
	return FString();
}

FString FControlRigAssetStrongReference::GetPathName() const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass->GetPathName();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset.GetPathName();
	}
	return FString();
}

UObject* FControlRigAssetStrongReference::Get() const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass;
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset;
	}
	return nullptr;
}

bool FControlRigAssetStrongReference::IsRigModule() const
{
	if (BlueprintRigClass)
	{
		UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>();
		return ClassDefaultObject->IsRigModule();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->IsRigModule();
	}
	return false;
}

bool FControlRigAssetStrongReference::IsModularRig() const
{
	if (BlueprintRigClass)
	{
		UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>();
		return ClassDefaultObject->IsModularRig();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->IsModularRig();
	}
	return false;
}

UClass* FControlRigAssetStrongReference::GetRigClass() const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass.Get();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->GetRigVMHostClass();
	}
	return nullptr;
}

const FRigModuleSettings& FControlRigAssetStrongReference::GetRigModuleSettings() const
{
	if (BlueprintRigClass)
	{
		UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>();
		return ClassDefaultObject->GetRigModuleSettings();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->GetRigModuleSettings();
	}
	static FRigModuleSettings EmptySettings;
	return EmptySettings;
}

const TArray<FName>& FControlRigAssetStrongReference::GetSupportedEvents() const
{
	if (BlueprintRigClass)
	{
		UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>();
		return ClassDefaultObject->GetSupportedEvents();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->GetSupportedEventNames();
	}
	
	static TArray<FName> EmptySupportedEvents;
	return EmptySupportedEvents;
}

TArray<FRigVMExternalVariable> FControlRigAssetStrongReference::GetExternalVariables() const
{
	if (BlueprintRigClass)
	{
		UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>();
		return ClassDefaultObject->GetExternalVariables();
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->GetExternalVariables();
	}
	static TArray<FRigVMExternalVariable> EmptyExternalVariables;
	return EmptyExternalVariables;
}

FRigVMExtendedExecuteContext& FControlRigAssetStrongReference::GetRigVMExtendedExecuteContext() const
{
	if (BlueprintRigClass)
	{
		UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>();
		return ClassDefaultObject->GetRigVMExtendedExecuteContext();
	}
	if (ControlRigAsset)
	{
		return *ControlRigAsset->GetRigVMExtendedExecuteContext();
	}
	static FRigVMExtendedExecuteContext EmptyContext;
	return EmptyContext;
}

const FModularRigModel& FControlRigAssetStrongReference::GetModularRigModel() const
{
	if (IsModularRig())
	{
		if (BlueprintRigClass)
		{
			if (UModularRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UModularRig>())
			{
				return ClassDefaultObject->GetModularRigModel();
			}
		}
		if (ControlRigAsset)
		{
			return ControlRigAsset->GetModularRigModel();
		}
	}
	static FModularRigModel EmptyModel;
	return EmptyModel;
}

URigHierarchy* FControlRigAssetStrongReference::GetDefaultHierarchy() const
{
	if (BlueprintRigClass)
	{
		if (UControlRig* ClassDefaultObject = BlueprintRigClass->GetDefaultObject<UControlRig>())
		{
			return ClassDefaultObject->GetHierarchy();
		}
	}

	if (ControlRigAsset)
	{
		return ControlRigAsset->GetHierarchy();
	}
	
	return nullptr;
}

bool FControlRigAssetStrongReference::IsNative() const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass->IsNative();
	}

	if (ControlRigAsset)
	{
		return ControlRigAsset->IsNative();
	}
	
	return false;
}

void FControlRigAssetStrongReference::ForEachVariableProperty(TFunction<bool(const FProperty*)> PerVariableProperty)
{
	if (BlueprintRigClass)
	{
		for (TFieldIterator<FProperty> PropertyIt(BlueprintRigClass.Get()); PropertyIt; ++PropertyIt)
		{
			if (!PerVariableProperty(*PropertyIt))
			{
				break;
			}
		}
	}
	else if (ControlRigAsset)
	{
		FRigVMPropertyBag* PropertyBag = &ControlRigAsset->Variables;
		for (int32 PropertyIndex = 0; PropertyIndex < PropertyBag->Num(); PropertyIndex++)
		{
			if (!PerVariableProperty(PropertyBag->GetProperty(PropertyIndex)))
			{
				break;
			}
		}
	}
}

const FProperty* FControlRigAssetStrongReference::FindVariablePropertyByName(const FName& InName) const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass->FindPropertyByName(InName);
	}
	else if (ControlRigAsset)
	{
		FRigVMPropertyBag* PropertyBag = &ControlRigAsset->Variables;
		return PropertyBag->FindPropertyByName(InName);
	}
	
	return nullptr;
}

FRigVMVariant FControlRigAssetStrongReference::GetVariant() const
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = Get())
	{
		return RuntimeAsset->GetAssetVariant();
	}
	return FRigVMVariant();
}

UStruct* FControlRigAssetStrongReference::GetVariablesStruct() const
{
	if (BlueprintRigClass)
	{
		return BlueprintRigClass.Get();
	}
	if (ControlRigAsset)
	{
		return const_cast<UPropertyBag*>(ControlRigAsset->GetVariablesPropertyBag()->GetPropertyBagStruct());
	}
	
	return nullptr;
}

uint8* FControlRigAssetStrongReference::GetVariablesMemory() const
{
	if (BlueprintRigClass)
	{
		return reinterpret_cast<uint8*>(BlueprintRigClass->GetDefaultObject<UControlRig>());
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->Variables.GetMutableValue().GetMemory();
	}

	return nullptr;
}

#if WITH_EDITORONLY_DATA
UObject* FControlRigAssetStrongReference::GetEditorAsset() const
{
	if (BlueprintRigClass)
	{
		if (URigVMBlueprintGeneratedClass* GeneratedClass = Cast<URigVMBlueprintGeneratedClass>(BlueprintRigClass.Get()))
		{
			return GeneratedClass->ClassGeneratedBy;
		}
	}
	if (ControlRigAsset)
	{
		return ControlRigAsset->GetEditorOnlyData();
	}

	return nullptr;
}
#endif

bool FControlRigAssetStrongReference::IsSourceOf(UObject* InObject) const
{
	if (BlueprintRigClass)
	{
		return InObject->GetClass() == BlueprintRigClass.Get();
	}
	if (ControlRigAsset)
	{
		if (URigVMHost* RigVMHost = Cast<URigVMHost>(InObject))
		{
			return RigVMHost->GetGeneratedByAsset() == ControlRigAsset.Get();
		}
	}

	return false;
}

UControlRig* FControlRigAssetStrongReference::CreateInstance(UObject* InOuter, const FName InName, EObjectFlags InFlags) const
{
	if (BlueprintRigClass)
	{
		return NewObject<UControlRig>(InOuter, BlueprintRigClass.Get(), InName, InFlags);
	}
	if (ControlRigAsset)
	{
		return Cast<UControlRig>(ControlRigAsset->InstantiateObject(InOuter, InName, InFlags));
	}

	return nullptr;
}

uint32 GetTypeHash(const FControlRigAssetStrongReference& InSource)
{
	return HashCombine(GetTypeHash(InSource.BlueprintRigClass), GetTypeHash(InSource.ControlRigAsset));
}

FControlRigAssetSoftReference::FControlRigAssetSoftReference(UObject* InAsset)
{
	if (!InAsset)
	{
		return;
	}
	if (UBlueprint* Blueprint = Cast<UBlueprint>(InAsset))
	{
		BlueprintRigClass = Blueprint->GeneratedClass.Get();
	}
	else if (UBlueprintGeneratedClass* GeneratedClass = Cast<UBlueprintGeneratedClass>(InAsset))
	{
		BlueprintRigClass = GeneratedClass;
	}
	else if (UControlRigRuntimeAsset* RuntimeAsset = Cast<UControlRigRuntimeAsset>(InAsset))
	{
		ControlRigAsset = RuntimeAsset;
	}
	else if (TSubclassOf<UControlRig> ControlRigClass = Cast<UClass>(InAsset))
	{
		BlueprintRigClass = ControlRigClass.Get();
	}
}

FControlRigAssetSoftReference::FControlRigAssetSoftReference(TSoftClassPtr<UControlRig> InClass)
: BlueprintRigClass(InClass)
{}

FControlRigAssetSoftReference::FControlRigAssetSoftReference(TSoftObjectPtr<UControlRigRuntimeAsset> InRuntimeAsset)
: ControlRigAsset(InRuntimeAsset)
{}

bool FControlRigAssetSoftReference::operator==(const FControlRigAssetSoftReference& InOtherValue) const
{
	return BlueprintRigClass == InOtherValue.BlueprintRigClass && ControlRigAsset == InOtherValue.ControlRigAsset;
}

bool FControlRigAssetSoftReference::IsValid() const
{
	const bool bBlueprintIsValid = BlueprintRigClass.IsValid() || BlueprintRigClass.IsPending();
	const bool bAssetIsValid = ControlRigAsset.IsValid() || ControlRigAsset.IsPending();
	return bBlueprintIsValid || bAssetIsValid;
}

FString FControlRigAssetSoftReference::GetName() const
{
	if (BlueprintRigClass.IsValid() || BlueprintRigClass.IsPending())
	{
		return BlueprintRigClass.GetAssetName();
	}
	if (ControlRigAsset.IsValid() || ControlRigAsset.IsPending())
	{
		return ControlRigAsset.GetAssetName();
	}
	return FString();
}

FString FControlRigAssetSoftReference::GetPathName() const
{
	if (BlueprintRigClass.IsValid() || BlueprintRigClass.IsPending())
	{
		return BlueprintRigClass.ToSoftObjectPath().GetAssetPath().ToString();
	}
	if (ControlRigAsset.IsValid() || ControlRigAsset.IsPending())
	{
		return ControlRigAsset.ToSoftObjectPath().GetAssetPath().ToString();
	}
	return FString();
}

FRigVMVariant FControlRigAssetSoftReference::GetVariant() const
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = Get())
	{
		return RuntimeAsset->GetAssetVariant();
	}
	return FRigVMVariant();
}

FControlRigAssetStrongReference FControlRigAssetSoftReference::LoadStrongReference(bool bForceLoad) const
{
	return LoadSynchronous(bForceLoad);
}

FSoftObjectPath FControlRigAssetSoftReference::ToSoftObjectPath() const
{
	if (BlueprintRigClass.IsValid() || BlueprintRigClass.IsPending())
	{
		return BlueprintRigClass.ToSoftObjectPath();
	}
	if (ControlRigAsset.IsValid() || ControlRigAsset.IsPending())
	{
		return ControlRigAsset.ToSoftObjectPath();
	}
	return FSoftObjectPath();
}



UObject* FControlRigAssetSoftReference::Get() const
{
	if (BlueprintRigClass.IsValid())
	{
		return BlueprintRigClass.Get();
	}
	if (ControlRigAsset.IsValid())
	{
		return ControlRigAsset.Get();
	}
	return nullptr;
}

UObject* FControlRigAssetSoftReference::LoadSynchronous(bool bForceLoad) const
{
	UObject* Result = nullptr;
	if (bForceLoad || !IsAsyncLoading())
	{
		if (BlueprintRigClass.IsValid() || BlueprintRigClass.IsPending())
		{
			Result = BlueprintRigClass.LoadSynchronous();
		}
		if (!Result)
		{
			if (ControlRigAsset.IsValid() || ControlRigAsset.IsPending())
			{
				Result = ControlRigAsset.LoadSynchronous();
			}
		}
	}
	else
	{
		// Dont load, just return the object if it is available
		Result = Get();
	}
	return Result;
}

uint32 GetTypeHash(const FControlRigAssetSoftReference& InSource)
{
	return HashCombine(GetTypeHash(InSource.BlueprintRigClass), GetTypeHash(InSource.ControlRigAsset));
}

#undef UE_API
