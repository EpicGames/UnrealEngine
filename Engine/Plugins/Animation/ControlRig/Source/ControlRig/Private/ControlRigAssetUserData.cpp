// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigAssetUserData.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigAssetUserData)

void UControlRigShapeLibraryLink::SetShapeLibrary(TSoftObjectPtr<UControlRigShapeLibrary> InShapeLibrary)
{
	InvalidateCache();
	ShapeLibrary = InShapeLibrary;
	ShapeLibraryCached = InShapeLibrary.Get();
	ShapeNames.Reset();
	if(ShapeLibraryCached)
	{
		for(const FControlRigShapeDefinition& Shape : ShapeLibraryCached->Shapes)
		{
			ShapeNames.Add(Shape.ShapeName);
		}
	}
}

const UNameSpacedUserData::FUserData* UControlRigShapeLibraryLink::GetUserData(const FString& InPath, FString* OutErrorMessage) const
{
	// rely on super to look it up from the cache
	if(const FUserData* ResultFromSuper = Super::GetUserData(InPath))
	{
		return ResultFromSuper;
	}

	if(ShapeLibrary)
	{
		if(InPath.Equals(GET_MEMBER_NAME_STRING_CHECKED(UControlRigShapeLibraryLink, ShapeLibrary), ESearchCase::CaseSensitive))
		{
			UpdateShapeLibraryCache();
			static const FProperty* ShapeLibraryProperty = FindPropertyByName(StaticClass(), GET_MEMBER_NAME_CHECKED(UControlRigShapeLibraryLink, ShapeLibraryCached));
			const FUserData UserData(InPath, ShapeLibraryProperty, static_cast<const uint8*>(ShapeLibraryCached));
			return StoreCacheForUserData(UserData);
		}
		if(InPath.Equals(DefaultShapePath, ESearchCase::CaseSensitive))
		{
			static const FProperty* DefaultShapeProperty = FindPropertyByName(FControlRigShapeDefinition::StaticStruct(), GET_MEMBER_NAME_CHECKED(FControlRigShapeDefinition, ShapeName));
			return StoreCacheForUserData({InPath, DefaultShapeProperty, reinterpret_cast<const uint8*>(&ShapeLibrary->DefaultShape.ShapeName)});
		}
		if(InPath.Equals(ShapeNamesPath, ESearchCase::CaseSensitive))
		{
			static const FProperty* ShapeNamesProperty = FindPropertyByName(StaticClass(), GET_MEMBER_NAME_CHECKED(UControlRigShapeLibraryLink, ShapeNames));
			return StoreCacheForUserData({InPath, ShapeNamesProperty, reinterpret_cast<const uint8*>(&ShapeNames)});
		}
		if(OutErrorMessage && OutErrorMessage->IsEmpty())
		{
			(*OutErrorMessage) = FString::Printf(PathNotFoundFormat, *InPath);
		}
		return nullptr;
	}

	if(OutErrorMessage && OutErrorMessage->IsEmpty())
	{
		(*OutErrorMessage) = FString::Printf(ShapeLibraryNullFormat, *InPath);
	}
	return nullptr;
}

const TArray<const UNameSpacedUserData::FUserData*>& UControlRigShapeLibraryLink::GetUserDataArray(const FString& InParentPath, FString* OutErrorMessage) const
{
	const TArray<const FUserData*>& ResultFromSuper = Super::GetUserDataArray(InParentPath);
	if(!ResultFromSuper.IsEmpty())
	{
		return ResultFromSuper;
	}

	if(ShapeLibrary)
	{
		// UControlRigShapeLibraryLink doesn't offer any arrays other than the top level 
		if(InParentPath.IsEmpty())
		{
			const FUserData* ShapeLibraryUserData = GetUserData(GET_MEMBER_NAME_STRING_CHECKED(UControlRigShapeLibraryLink, ShapeLibrary), OutErrorMessage);
			const FUserData* DefaultShapeUserData = GetUserData(DefaultShapePath, OutErrorMessage);
			const FUserData* ShapeNamesUserData = GetUserData(ShapeNamesPath, OutErrorMessage);
			check(ShapeLibraryUserData);
			check(DefaultShapeUserData);
			check(ShapeNamesUserData);
			return StoreCacheForUserDataArray(InParentPath, {ShapeLibraryUserData, DefaultShapeUserData, ShapeNamesUserData});
		}
		return EmptyUserDatas;
	}

	if(OutErrorMessage && OutErrorMessage->IsEmpty())
	{
		(*OutErrorMessage) = FString::Printf(ShapeLibraryNullFormat, *InParentPath);
	}
	return EmptyUserDatas;
}

void UControlRigShapeLibraryLink::Serialize(FArchive& Ar)
{
	// Treat the cached ptr as transient unless we're cooking this out.
	const bool bIsSavingAssetToStorage = Ar.IsSaving() && Ar.IsPersistent() && !Ar.IsCooking(); 
	UControlRigShapeLibrary* SavedShapeLibrary = ShapeLibraryCached; 
	if (bIsSavingAssetToStorage)
	{
		ShapeLibraryCached = nullptr;
	}
	
	Super::Serialize(Ar);
	
	if (bIsSavingAssetToStorage)
	{
		ShapeLibraryCached = SavedShapeLibrary;
	}
}

#if WITH_EDITOR

void UControlRigShapeLibraryLink::PostLoad()
{
	Super::PostLoad();

	ShapeLibraryCached = ShapeLibrary.Get();
	UpdateShapeLibraryCache();
	SetShapeLibrary(ShapeLibrary);
}

void UControlRigShapeLibraryLink::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static const FProperty* ShapeLibraryProperty= FindPropertyByName(GetClass(), GET_MEMBER_NAME_CHECKED(UControlRigShapeLibraryLink, ShapeLibrary));
	if(PropertyChangedEvent.Property == ShapeLibraryProperty ||
		PropertyChangedEvent.MemberProperty == ShapeLibraryProperty)
	{
		SetShapeLibrary(ShapeLibrary);
	}
}

#endif

void UControlRigShapeLibraryLink::UpdateShapeLibraryCache() const
{
	if(ShapeLibraryCached != ShapeLibrary && !ShapeLibrary.IsNull())
	{
		// We need to check if the mount point exists - since the shape library link may
		// refer to an editor-only asset in a runtime game.
		const FString PackagePath = ShapeLibrary.GetLongPackageName();
		const FName PluginMountPoint = FPackageName::GetPackageMountPoint(PackagePath, false);
		if (FPackageName::MountPointExists(PluginMountPoint.ToString()))
		{
			const FString ObjectPath = ShapeLibrary.ToString();

			// load without throwing additional warnings / errors
			ShapeLibraryCached = LoadObject<UControlRigShapeLibrary>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
		}
	}
}
