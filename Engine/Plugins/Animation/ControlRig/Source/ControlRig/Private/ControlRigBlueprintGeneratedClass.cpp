// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigBlueprintGeneratedClass.h"
#include "Units/Control/RigUnit_Control.h"
#include "ControlRigObjectVersion.h"
#include "ControlRig.h"
#include "ModularRig.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigBlueprintGeneratedClass)

UControlRigBlueprintGeneratedClass::UControlRigBlueprintGeneratedClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UControlRigBlueprintGeneratedClass::Serialize(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("UControlRigBlueprintGeneratedClass(%s)"), *GetName()));

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	// don't use URigVMBlueprintGeneratedClass
	// to avoid backwards compat issues.
	UBlueprintGeneratedClass::Serialize(Ar);

	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::SwitchedToRigVM)
	{
		return;
	}

	// for debugging purposes we'll give this VM a name that's useful.
	static TAtomic<uint32> NumVMs{ 0 };
	static constexpr TCHAR Format[] = TEXT("%s_VM_%u");
	const FString VMDebugName = FString::Printf(Format, *GetName(), uint32(++NumVMs));
	URigVM* VM = NewObject<URigVM>(GetTransientPackage(), *VMDebugName);

	if (UControlRig* CDO = Cast<UControlRig>(GetDefaultObject(true)))
	{
		const bool bIsSerializingToDisk = URigVM::IsSerializingToDisk(Ar);
		if (Ar.IsSaving() && bIsSerializingToDisk && CDO->VM)
		{
			VM->CopyDataForSerialization(CDO->VM);
			VM->CreateLocalizedRegistryIfRequired();
		}
	}
	
	VM->Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("VM"));

	if (UControlRig* CDO = Cast<UControlRig>(GetDefaultObject(false)))
	{
		if (Ar.IsLoading() && CDO->VM)
		{
			CDO->VM->CopyDataForSerialization(VM);
		}
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::StoreFunctionsInGeneratedClass)
	{
		return;
	}
	
	Ar << GraphFunctionStore;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("GraphFunctionStore"));
}

bool UControlRigBlueprintGeneratedClass::IsControlRigModule() const
{
	return RigModuleSettings.IsValidModule();
}

FControlRigAssetStrongReference UControlRigBlueprintGeneratedClass::GetControlRigAssetReference()
{
	FControlRigAssetStrongReference AssetReference;
	AssetReference.Set(TSubclassOf<UControlRig>(this));
	return AssetReference;
}

UClass* UControlRigBlueprintGeneratedClass::GetRigVMHostClass() const
{
	if (UControlRig* CDO = GetCDO())
	{
		return CDO->GetClass();
	}
	return UControlRig::StaticClass();
}

FRigElementKeyRedirector& UControlRigBlueprintGeneratedClass::GetElementKeyRedirector()
{
	if (UControlRig* CDO = GetCDO())
	{
		return CDO->ElementKeyRedirector;
	}
	static FRigElementKeyRedirector EmptyRedirector;
	return EmptyRedirector;
}

TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& UControlRigBlueprintGeneratedClass::GetShapeLibraries()
{
	if (UControlRig* CDO = GetCDO())
	{
		return CDO->ShapeLibraries;
	}
	static TArray<TSoftObjectPtr<UControlRigShapeLibrary>> Empty;
	return Empty;
}

TMap<FRigElementKey, FRigElementKeyCollection>& UControlRigBlueprintGeneratedClass::GetArrayConnectionMap()
{
	static TMap<FRigElementKey, FRigElementKeyCollection> Empty;
	return Empty;
}

TSoftObjectPtr<UObject>& UControlRigBlueprintGeneratedClass::GetSourceCurveImport()
{
	static TSoftObjectPtr<UObject> Empty;
	return Empty;
}

TSoftObjectPtr<UObject>& UControlRigBlueprintGeneratedClass::GetSourceHierarchyImport()
{
	static TSoftObjectPtr<UObject> Empty;
	return Empty;
}

UControlRig* UControlRigBlueprintGeneratedClass::GetCDO() const
{
	return Cast<UControlRig>(GetDefaultObject(false));
}

URigHierarchy* UControlRigBlueprintGeneratedClass::GetHierarchy()
{
	if (UControlRig* CDO = GetCDO())
	{
		return CDO->GetHierarchy();
	}
	return nullptr;
}

void UControlRigBlueprintGeneratedClass::SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy)
{
	if (UControlRig* CDO = GetCDO())
	{
		CDO->SetDynamicHierarchy(InHierarchy);
	}
}

FModularRigSettings& UControlRigBlueprintGeneratedClass::GetModularRigSettings()
{
	if (UModularRig* ModularRigCDO = Cast<UModularRig>(GetCDO()))
	{
		// ModularRigSettings is public in UModularRig
		return ModularRigCDO->ModularRigSettings;
	}
	// Return a static default if CDO is not a ModularRig (shouldn't happen for modular rigs)
	static FModularRigSettings DefaultSettings;
	return DefaultSettings;
}

FRigHierarchySettings& UControlRigBlueprintGeneratedClass::GetHierarchySettings()
{
	if (UControlRig* CDO = GetCDO())
	{
		return CDO->HierarchySettings;
	}
	// Return a static default if CDO is not available (shouldn't happen in practice)
	static FRigHierarchySettings DefaultSettings;
	return DefaultSettings;
}

FRigInfluenceMapPerEvent& UControlRigBlueprintGeneratedClass::GetInfluences()
{
	if (UControlRig* CDO = GetCDO())
	{
		// Access the Influences member from CDO (friend class can access protected members)
		return CDO->Influences;
	}
	// Return a static default if CDO is not available
	static FRigInfluenceMapPerEvent DefaultInfluences;
	return DefaultInfluences;
}

FModularRigModel& UControlRigBlueprintGeneratedClass::GetModularRigModel()
{
	if (UModularRig* ModularRigCDO = Cast<UModularRig>(GetCDO()))
	{
		// GetModularRigModel() returns const, but interface requires non-const reference
		// This is safe because we're accessing the CDO's member
		return const_cast<FModularRigModel&>(ModularRigCDO->GetModularRigModel());
	}
	// Return a static default if CDO is not a ModularRig (shouldn't happen for modular rigs)
	static FModularRigModel DefaultModel;
	return DefaultModel;
}

