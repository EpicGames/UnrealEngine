// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigComponent.h"
#include "Engine/SkeletalMesh.h"
#include "MeshElementCollector.h"
#include "SceneView.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Units/Execution/RigUnit_Hierarchy.h"
#include "Misc/ScopeLock.h"

#include "SkeletalDebugRendering.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "AnimCustomInstanceHelper.h"
#include "ControlRigObjectBinding.h"
#include "PrimitiveDrawingUtils.h"
#include "Math/ControlRigMathLibrary.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigComponent)

// CVar to disable control rig execution within a component
static TAutoConsoleVariable<int32> CVarControlRigDisableExecutionComponent(TEXT("ControlRig.DisableExecutionInComponent"), 0, TEXT("if nonzero we disable the execution of Control Rigs inside a ControlRigComponent."));

FControlRigAnimInstanceProxy* FControlRigComponentMappedElement::GetAnimProxyOnGameThread() const
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SceneComponent))
	{
		if (UControlRigAnimInstance* AnimInstance = Cast<UControlRigAnimInstance>(SkeletalMeshComponent->GetAnimInstance()))
		{
			return AnimInstance->GetControlRigProxyOnGameThread();
		}
	}

	return nullptr;
}


TAutoConsoleVariable<bool> CVarControlRigDisableTickDependencyCheck(
	TEXT("ControlRig.DisableTickDependencyCheck"),
	false,
	TEXT("Disables tick dependency checking, which prevents cycles. Use carefully.\n")
	);


////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
TMap<FString, TSharedPtr<SNotificationItem>> UControlRigComponent::EditorNotifications;
#endif

struct FSkeletalMeshToMap
{
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;
	TArray<FControlRigComponentMappedBone> Bones;
	TArray<FControlRigComponentMappedCurve> Curves;
};

FTransactionallySafeCriticalSection gPendingSkeletalMeshesLock;
TMap<TWeakObjectPtr<UControlRigComponent>, TArray<FSkeletalMeshToMap> > gPendingSkeletalMeshes;

UControlRigComponent::UControlRigComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.TickGroup = TG_PrePhysics;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bTickEvenWhenPaused = true;
	bTickInEditor = true;
	bAutoActivate = true;

	ControlRig = nullptr;
	bResetTransformBeforeTick = true;
	bResetInitialsBeforeConstruction = true;
	bUpdateRigOnTick = true;
	bUpdateInEditor = true;
	bDrawBones = true;
	bShowDebugDrawing = false;
	bIsInsideInitializeBracket = false;
	bWantsInitializeComponent = true;
	
	bEnableLazyEvaluation = false;
	LazyEvaluationPositionThreshold = 0.1f;
	LazyEvaluationRotationThreshold = 0.5f;
	LazyEvaluationScaleThreshold = 0.01f;
	bNeedsEvaluation = true;

	bNeedToInitialize = false;
}

#if WITH_EDITOR
void UControlRigComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UControlRigComponent, ControlRigAssetReference))
	{
		ControlRig = nullptr;
		SetupControlRigIfRequired();
	}
	else if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UControlRigComponent, MappedElements))
	{
		ValidateMappingData();
	}
}

void UControlRigComponent::PostLoad()
{
	Super::PostLoad();
}
#endif

void UControlRigComponent::BeginDestroy()
{
	Super::BeginDestroy();

	UE::TScopeLock Lock(gPendingSkeletalMeshesLock);
	gPendingSkeletalMeshes.Remove(this);
}

void UControlRigComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	if (Ar.IsLoading())
	{
		if (ControlRigClass_DEPRECATED)
		{
			ControlRigAssetReference.Set(ControlRigClass_DEPRECATED);
			ControlRigClass_DEPRECATED = nullptr;
		}
	}

	// after compile, we have to reinitialize
	// because it needs new execution code
	// since memory has changed, similar to FAnimNode_ControlRig::PostSerialize
	if (Ar.IsObjectReferenceCollector())
	{
		if (ControlRig)
		{
			ControlRig->Initialize();
		}
	}
}

void UControlRigComponent::OnRegister()
{
	Super::OnRegister();
	
	ControlRig = nullptr;

	{
		UE::TScopeLock Lock(gPendingSkeletalMeshesLock);
		gPendingSkeletalMeshes.FindOrAdd(this);
	}

	// call to Initialize() should be delayed until TickComponent
	
	// Initialize() here directly is not enough because
	// in case of PIE, OnRegister() is called before BP native events like
	// OnPreInitialize are bound, which does not happen until
	// 1. Re-Registration ( Sequencer / Level Editor / non- PIE use cases, and there is no way to detect it here)
	// 2. InitializeComponent() (PIE + Cooked build only, not called for editor use cases)

	// so to avoid calling Initialize() in both OnRegister() and InitializeComponent()
	// we have to delay Initialize() until TickComponent, which is called all the time
	// in both editor + cooked.

	bNeedToInitialize = true;

	if (AActor* Actor = GetOwner())
	{
		Actor->PrimaryActorTick.bStartWithTickEnabled = true;
		Actor->PrimaryActorTick.bCanEverTick = true;
		Actor->PrimaryActorTick.bTickEvenWhenPaused = true;
	}
}

void UControlRigComponent::OnUnregister()
{
	Super::OnUnregister();

	bool bBeginDestroyed = URigVMHost::IsGarbageOrDestroyed(this);
	if (!bBeginDestroyed)
	{
		if (AActor* Actor = GetOwner())
		{
			bBeginDestroyed = URigVMHost::IsGarbageOrDestroyed(Actor) || Actor->IsActorBeingDestroyed();
		}
	}

	if (!bBeginDestroyed)
	{
		for (TPair<USkeletalMeshComponent*, FCachedSkeletalMeshComponentSettings>& Pair : CachedSkeletalMeshComponentSettings)
		{
			if (Pair.Key)
			{
				if (Pair.Key->IsValidLowLevel() &&
					!URigVMHost::IsGarbageOrDestroyed(Pair.Key) &&
					IsValid(Pair.Key) &&
					!Pair.Key->IsUnreachable())
				{
					Pair.Value.Apply(Pair.Key);
				}
			}
		}
	}
	else
	{
		UE::TScopeLock Lock(gPendingSkeletalMeshesLock);
		gPendingSkeletalMeshes.Remove(this);
	}

	CachedSkeletalMeshComponentSettings.Reset();
}

void UControlRigComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	if (bNeedToInitialize)
	{
		Initialize();
		bNeedToInitialize = false;
	}
	
	if(!bUpdateRigOnTick)
	{
		return;
	}

	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		UE::TScopeLock Lock(gPendingSkeletalMeshesLock);
		TArray<FSkeletalMeshToMap>* PendingSkeletalMeshes = gPendingSkeletalMeshes.Find(this);

		if (PendingSkeletalMeshes != nullptr && PendingSkeletalMeshes->Num() > 0)
		{
			for (const FSkeletalMeshToMap& SkeletalMeshToMap : *PendingSkeletalMeshes)
			{
				AddMappedSkeletalMesh(
					SkeletalMeshToMap.SkeletalMeshComponent.Get(),
					SkeletalMeshToMap.Bones,
					SkeletalMeshToMap.Curves
				);
			}

			PendingSkeletalMeshes->Reset();
		}
	}

	Update(DeltaTime);
}

FPrimitiveSceneProxy* UControlRigComponent::CreateSceneProxy()
{
	return new FControlRigSceneProxy(this);
}

FBoxSphereBounds UControlRigComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BBox(ForceInit);

	if (ControlRig)
	{
		// Get bounding box for the debug drawings if they are drawn 
		if (bShowDebugDrawing)
		{ 
			const FRigVMDrawInterface& DrawInterface = ControlRig->GetDrawInterface();

			for (int32 InstructionIndex = 0; InstructionIndex < DrawInterface.Num(); InstructionIndex++)
			{
				const FRigVMDrawInstruction& Instruction = DrawInterface[InstructionIndex];

				FTransform Transform = Instruction.Transform * GetComponentToWorld();
				for (const FVector& Position : Instruction.Positions)
				{
					BBox += Transform.TransformPosition(Position);
				}
			}
		}

		const FTransform Transform = GetComponentToWorld();

		// Get bounding box for bones
		if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->ForEach<FRigTransformElement>([&BBox, Transform, Hierarchy](FRigTransformElement* TransformElement) -> bool
			{
				BBox += Transform.TransformPosition(Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentGlobal).GetLocation());
				return true;
			});
		}
	}

	if (BBox.IsValid)
	{
		// Points are in world space, so no need to transform.
		return FBoxSphereBounds(BBox);
	}
	else
	{
		const FVector BoxExtent(1.f);
		return FBoxSphereBounds(LocalToWorld.GetLocation(), BoxExtent, 1.f);
	}
}


UControlRig* UControlRigComponent::GetControlRig()
{
	return SetupControlRigIfRequired();
}

bool UControlRigComponent::CanExecute()
{
	if(CVarControlRigDisableExecutionComponent->GetInt() != 0)
	{
		return false;
	}
	
	if(UControlRig* CR = GetControlRig())
	{
		return CR->CanExecute();
	}
	
	return false;
}

float UControlRigComponent::GetAbsoluteTime() const
{
	if(ControlRig)
	{
		return ControlRig->AbsoluteTime;
	}
	return 0.f;
}

void UControlRigComponent::OnPreInitialize_Implementation(UControlRigComponent* Component)
{
	OnPreInitializeDelegate.Broadcast(Component);
}

void UControlRigComponent::OnPostInitialize_Implementation(UControlRigComponent* Component)
{
	ValidateMappingData();
	OnPostInitializeDelegate.Broadcast(Component);
}

void UControlRigComponent::OnPreConstruction_Implementation(UControlRigComponent* Component)
{
	OnPreConstructionDelegate.Broadcast(Component);
}

void UControlRigComponent::OnPostConstruction_Implementation(UControlRigComponent* Component)
{
	OnPostConstructionDelegate.Broadcast(Component);
}

void UControlRigComponent::OnPreForwardsSolve_Implementation(UControlRigComponent* Component)
{
	OnPreForwardsSolveDelegate.Broadcast(Component);
}

void UControlRigComponent::OnPostForwardsSolve_Implementation(UControlRigComponent* Component)
{
	OnPostForwardsSolveDelegate.Broadcast(Component);
}

void UControlRigComponent::Initialize()
{
	if(bIsInsideInitializeBracket)
	{
		return;
	}

	TGuardValue<bool> InitializeBracket(bIsInsideInitializeBracket, true);
	
	if (!UserDefinedElements.IsEmpty())
	{
		MappedElements = UserDefinedElements;
	}
	ValidateMappingData();

#if WITH_EDITOR
	TickDependencyErrorMessages.Reset();

	if (bUpdateInEditor)
	{
		FEditorScriptExecutionGuard AllowScripts;
		OnPreInitialize(this);
	}
	else
#endif
	{
		OnPreInitialize(this);
	}
	
	if(UControlRig* CR = SetupControlRigIfRequired())
	{
		if (CR->IsInitializing())
		{
			ReportError(TEXT("Initialize is being called recursively."));
		}
		else
		{
			CR->DrawInterface.Reset();
			CR->GetHierarchy()->ResetPoseToInitial(ERigElementType::All);
			CR->RequestInit();
		}
	}

	// we want to make sure all components driven by control rig component tick
	// after the control rig component such that by the time they tick they can
	// send the latest data for rendering
	// also need to make sure all components driving the control rig component tick before
	// the control rig component ticks
	TMap<TObjectPtr<USceneComponent>, EControlRigComponentMapDirection> MappedComponents;
#if WITH_EDITOR
	TSet<TObjectPtr<USceneComponent>> MappedComponentsWithErrors;
#endif
	for (const FControlRigComponentMappedElement& MappedElement : MappedElements)
	{
		if (MappedElement.SceneComponent)
		{
			if (EControlRigComponentMapDirection* Direction = MappedComponents.Find(MappedElement.SceneComponent))
			{
				if (*Direction != MappedElement.Direction)
				{
					// elements from the same component should not be mapped to both input and output
#if WITH_EDITOR
					MappedComponentsWithErrors.Add(MappedElement.SceneComponent);
#endif
					continue;
				}
			}

			// input elements should tick before ControlRig updates
			// output elements should tick after ControlRig updates
#if WITH_EDITOR
			FString TickDependencyErrorMessage;
			FString* TickDependencyErrorMessagePtr = &TickDependencyErrorMessage;
#else
			static FString* TickDependencyErrorMessagePtr = nullptr;
#endif
			if (MappedElement.Direction == EControlRigComponentMapDirection::Output)
			{
				if(AllowsTickDependency(MappedElement.SceneComponent, this, TickDependencyErrorMessagePtr))
				{
					MappedElement.SceneComponent->AddTickPrerequisiteComponent(this);
				}
			}
			else
			{
				if(AllowsTickDependency(this, MappedElement.SceneComponent, TickDependencyErrorMessagePtr))
				{
					AddTickPrerequisiteComponent(MappedElement.SceneComponent);
				}
			}

#if WITH_EDITOR
			if(!TickDependencyErrorMessage.IsEmpty())
			{
				// only throw the error message if the user has picked a particular component to set up the dependency for
				if(!MappedElement.ComponentReference.ComponentProperty.IsNone() || MappedElement.ComponentReference.OtherActor.IsValid())
				{
					if(!TickDependencyErrorMessages.Contains(TickDependencyErrorMessage))
					{
						UE_LOGF(LogControlRig, Warning, "%ls", *TickDependencyErrorMessage);
						TickDependencyErrorMessages.Add(TickDependencyErrorMessage);
					}
				}
			}
#endif

			MappedComponents.Add(MappedElement.SceneComponent) = MappedElement.Direction;
			
			// make sure that the animation is updated so that bone transforms are updated on the mapped component
			// (otherwise, FControlRigAnimInstanceProxy::Evaluate is never called when moving a control in the editor)
			if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(MappedElement.SceneComponent))
			{
				Component->SetUpdateAnimationInEditor(bUpdateInEditor);
			}
		}
	}

#if WITH_EDITOR
	for (const TObjectPtr<USceneComponent>& ErrorComponent : MappedComponentsWithErrors)
	{
		FString Message = FString::Printf(
				TEXT("Elements from the same component (%s) should not be mapped to both input and output,"
					" because it creates ambiguity when inferring tick order."), *ErrorComponent.GetName());
		UE_LOGF(LogControlRig, Warning, "%ls: %ls", *GetPathName(), *Message);
	}
#endif
}

void UControlRigComponent::Update(float DeltaTime)
{
	if(UControlRig* CR = SetupControlRigIfRequired())
	{
		if(!CanExecute())
		{
			return;
		}
		
		if (CR->IsExecuting() || CR->IsInitializing())
		{
			ReportError(TEXT("Update is being called recursively."));
		}
		else
		{
			CR->SetDeltaTime(DeltaTime);
			CR->bResetInitialTransformsBeforeConstruction = bResetInitialsBeforeConstruction;

			// todo: set log
			// todo: set external data providers

			if (bResetTransformBeforeTick)
			{
				CR->GetHierarchy()->ResetPoseToInitial(ERigElementType::Bone);
			}

#if WITH_EDITOR
			// if we are recording any change - let's clear the undo stack
			if(URigHierarchy* Hierarchy = CR->GetHierarchy())
			{
				if(Hierarchy->IsTracingChanges())
				{
					Hierarchy->ResetTransformStack();
				}
			}
#endif

			TransferInputs();

			if(bNeedsEvaluation)
			{
#if WITH_EDITOR
				if(URigHierarchy* Hierarchy = CR->GetHierarchy())
				{
					if(Hierarchy->IsTracingChanges())
					{
						Hierarchy->StorePoseForTrace(TEXT("UControlRigComponent::BeforeEvaluate"));
					}
				}
#endif

				// Animation will only evaluate if DeltaTime > 0
				if (CR->IsAdditive() && DeltaTime > 0)
				{
					CR->ClearPoseBeforeBackwardsSolve();
				}

				{
					// Necessary for FStackAttributeContainer that uses a FAnimStackAllocator (TMemStackAllocator) which allocates from FMemStack.
					// When allocating memory from FMemStack we need to explicitly use FMemMark to ensure items are freed when the scope exits. 
					TempAttributeContainer.Empty();
					UControlRig::FAnimAttributeContainerPtrScope AttributeScope(CR, TempAttributeContainer);
				
					CR->Evaluate_AnyThread();
				}

#if WITH_EDITOR
				if(URigHierarchy* Hierarchy = CR->GetHierarchy())
				{
					if(Hierarchy->IsTracingChanges())
					{
						Hierarchy->StorePoseForTrace(TEXT("UControlRigComponent::AfterEvaluate"));
					}
				}
#endif
			}
		} 

		if (bShowDebugDrawing)
		{
			if (CR->DrawInterface.Instructions.Num() > 0)
			{ 
				MarkRenderStateDirty();
			}
		}

#if WITH_EDITOR
		// if we are recording any change - dump the trace to file
		if(URigHierarchy* Hierarchy = CR->GetHierarchy())
		{
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->DumpTransformStackToFile();
			}
		}
#endif
	} 
}

TArray<FName> UControlRigComponent::GetElementNames(ERigElementType ElementType)
{
	TArray<FName> Names;

	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		for (FRigBaseElement* Element : *CR->GetHierarchy())
		{
			if(Element->IsTypeOf(ElementType))
			{
				Names.Add(Element->GetFName());
			}
		}
	}

	return Names;
}

bool UControlRigComponent::DoesElementExist(FName Name, ERigElementType ElementType)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		return CR->GetHierarchy()->GetIndex(FRigElementKey(Name, ElementType)) != INDEX_NONE;
	}
	return false;
}

void UControlRigComponent::ClearMappedElements()
{
	if (!EnsureCalledOutsideOfBracket(TEXT("ClearMappedElements")))
	{
		return;
	}

	MappedElements.Reset();
	MappedElements = UserDefinedElements;
	ValidateMappingData();
	Initialize();
}

void UControlRigComponent::SetMappedElements(TArray<FControlRigComponentMappedElement> NewMappedElements)
{
	if (!EnsureCalledOutsideOfBracket(TEXT("SetMappedElements")))
	{
		return;
	}

	MappedElements = NewMappedElements;
	ValidateMappingData();
	Initialize();
}

void UControlRigComponent::AddMappedElements(TArray<FControlRigComponentMappedElement> NewMappedElements)
{
	if (!EnsureCalledOutsideOfBracket(TEXT("AddMappedElements")))
	{
		return;
	}

	MappedElements.Append(NewMappedElements);
	ValidateMappingData();
	Initialize();
}

void UControlRigComponent::AddMappedComponents(TArray<FControlRigComponentMappedComponent> Components)
{
	if (!EnsureCalledOutsideOfBracket(TEXT("AddMappedComponents")))
	{
		return;
	}

	TArray<FControlRigComponentMappedElement> ElementsToMap;

	for (const FControlRigComponentMappedComponent& ComponentToMap : Components)
	{
		if (ComponentToMap.Component == nullptr ||
			ComponentToMap.ElementName.IsNone())
		{
			continue;
		}

		USceneComponent* Component = ComponentToMap.Component;

		FControlRigComponentMappedElement ElementToMap;
		ElementToMap.ComponentReference.OtherActor = Component->GetOwner() != GetOwner() ? Component->GetOwner() : nullptr;
		ElementToMap.ComponentReference.ComponentProperty = GetComponentNameWithinActor(Component);

		ElementToMap.ElementName = ComponentToMap.ElementName;
		ElementToMap.ElementType = ComponentToMap.ElementType;
		ElementToMap.Direction = ComponentToMap.Direction;

		ElementsToMap.Add(ElementToMap);
	}

	AddMappedElements(ElementsToMap);
}

void UControlRigComponent::AddMappedSkeletalMesh(USkeletalMeshComponent* SkeletalMeshComponent, TArray<FControlRigComponentMappedBone> Bones, TArray<
                                                 FControlRigComponentMappedCurve> Curves, const EControlRigComponentMapDirection InDirection)
{
	if (SkeletalMeshComponent == nullptr)
	{
		return;
	}

	if (!EnsureCalledOutsideOfBracket(TEXT("AddMappedSkeletalMesh")))
	{
		return;
	}

	UControlRig* CR = SetupControlRigIfRequired();
	if (CR == nullptr)
	{
		// if we don't have a valid rig yet - delay it until tick component
		FSkeletalMeshToMap PendingMesh;
		PendingMesh.SkeletalMeshComponent = SkeletalMeshComponent;
		PendingMesh.Bones = Bones;
		PendingMesh.Curves = Curves;

		UE::TScopeLock Lock(gPendingSkeletalMeshesLock);
		TArray<FSkeletalMeshToMap>& PendingSkeletalMeshes = gPendingSkeletalMeshes.FindOrAdd(this);
		PendingSkeletalMeshes.Add(PendingMesh);
		return;
	}

	TArray<FControlRigComponentMappedElement> ElementsToMap;
	TArray<FControlRigComponentMappedBone> BonesToMap = Bones;
	if (BonesToMap.Num() == 0)
	{
		if (const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			if (const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
			{
				CR->GetHierarchy()->ForEach<FRigBoneElement>([SkeletalMeshComponent, &BonesToMap](FRigBoneElement* BoneElement) -> bool
				{
					if (SkeletalMeshComponent->GetBoneIndex(BoneElement->GetFName()) != INDEX_NONE)
					{
						FControlRigComponentMappedBone BoneToMap;
						BoneToMap.Source = BoneElement->GetFName();
						BoneToMap.Target = BoneElement->GetFName();
						BonesToMap.Add(BoneToMap);
					}
					return true;
				});
			}
			else
			{
				ReportError(FString::Printf(TEXT("%s does not have a Skeleton set."), *SkeletalMesh->GetPathName()));
			}
		}
	}

	TArray<FControlRigComponentMappedCurve> CurvesToMap = Curves;
	if (CurvesToMap.Num() == 0)
	{
		if (USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			if (USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
			{
				CR->GetHierarchy()->ForEach<FRigCurveElement>([Skeleton, &CurvesToMap](FRigCurveElement* CurveElement) -> bool
                {
					FControlRigComponentMappedCurve CurveToMap;
					CurveToMap.Source = CurveElement->GetFName();
					CurveToMap.Target = CurveElement->GetFName();
					CurvesToMap.Add(CurveToMap);
					return true;
				});
			}
			else
			{
				ReportError(FString::Printf(TEXT("%s does not have a Skeleton set."), *SkeletalMesh->GetPathName()));
			}
		}
	}

	for (const FControlRigComponentMappedBone& BoneToMap : BonesToMap)
	{
		if (BoneToMap.Source.IsNone() ||
			BoneToMap.Target.IsNone())
		{
			continue;
		}

		FControlRigComponentMappedElement ElementToMap;
		ElementToMap.ComponentReference.OtherActor = SkeletalMeshComponent->GetOwner() != GetOwner() ? SkeletalMeshComponent->GetOwner() : nullptr;
		ElementToMap.ComponentReference.ComponentProperty = GetComponentNameWithinActor(SkeletalMeshComponent);

		ElementToMap.ElementName = BoneToMap.Source;
		ElementToMap.ElementType = ERigElementType::Bone;
		ElementToMap.TransformName = BoneToMap.Target;
		ElementToMap.Direction = InDirection;

		ElementsToMap.Add(ElementToMap);
	}

	for (const FControlRigComponentMappedCurve& CurveToMap : CurvesToMap)
	{
		if (CurveToMap.Source.IsNone() ||
			CurveToMap.Target.IsNone())
		{
			continue;
		}

		FControlRigComponentMappedElement ElementToMap;
		ElementToMap.ComponentReference.OtherActor = SkeletalMeshComponent->GetOwner() != GetOwner() ? SkeletalMeshComponent->GetOwner() : nullptr;
		ElementToMap.ComponentReference.ComponentProperty = GetComponentNameWithinActor(SkeletalMeshComponent);

		ElementToMap.ElementName = CurveToMap.Source;
		ElementToMap.ElementType = ERigElementType::Curve;
		ElementToMap.TransformName = CurveToMap.Target;

		ElementsToMap.Add(ElementToMap);
	}

	AddMappedElements(ElementsToMap);
}

void UControlRigComponent::AddMappedCompleteSkeletalMesh(USkeletalMeshComponent* SkeletalMeshComponent, const EControlRigComponentMapDirection InDirection)
{
	AddMappedSkeletalMesh(SkeletalMeshComponent, TArray<FControlRigComponentMappedBone>(), TArray<FControlRigComponentMappedCurve>(), InDirection);
}

void UControlRigComponent::SetBoneInitialTransformsFromSkeletalMesh(USkeletalMesh* InSkeletalMesh)
{
	if (InSkeletalMesh)
	{
		if (UControlRig* CR = SetupControlRigIfRequired())
		{
			CR->SetBoneInitialTransformsFromSkeletalMesh(InSkeletalMesh);
			bResetInitialsBeforeConstruction = false;
		}
	}
}

FTransform UControlRigComponent::GetBoneTransform(FName BoneName, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		const int32 BoneIndex = CR->GetHierarchy()->GetIndex(FRigElementKey(BoneName, ERigElementType::Bone));
		if (BoneIndex != INDEX_NONE)
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				return CR->GetHierarchy()->GetLocalTransform(BoneIndex);
			}
			else
			{
				FTransform RootTransform = CR->GetHierarchy()->GetGlobalTransform(BoneIndex);
				ConvertTransformFromRigSpace(RootTransform, Space);
				return RootTransform;
			}
		}
	}

	return FTransform::Identity;
}

FTransform UControlRigComponent::GetInitialBoneTransform(FName BoneName, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		const int32 BoneIndex = CR->GetHierarchy()->GetIndex(FRigElementKey(BoneName, ERigElementType::Bone));
		if (BoneIndex != INDEX_NONE)
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				return CR->GetHierarchy()->GetInitialLocalTransform(BoneIndex);
			}
			else
			{
				FTransform RootTransform = CR->GetHierarchy()->GetInitialGlobalTransform(BoneIndex);
				ConvertTransformFromRigSpace(RootTransform, Space);
				return RootTransform;
			}
		}
	}

	return FTransform::Identity;
}

void UControlRigComponent::SetBoneTransform(FName BoneName, FTransform Transform, EControlRigComponentSpace Space, float Weight, bool bPropagateToChildren)
{
	if (Weight <= SMALL_NUMBER)
	{
		return;
	}

	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		int32 BoneIndex = CR->GetHierarchy()->GetIndex(FRigElementKey(BoneName, ERigElementType::Bone));
		if (BoneIndex != INDEX_NONE)
		{
			ConvertTransformToRigSpace(Transform, Space);

			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				if (Weight >= 1.f - SMALL_NUMBER)
				{
					CR->GetHierarchy()->SetLocalTransform(BoneIndex, Transform, bPropagateToChildren);
				}
				else
				{
					FTransform PreviousTransform = CR->GetHierarchy()->GetLocalTransform(BoneIndex);
					FTransform BlendedTransform = FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, Weight);
					CR->GetHierarchy()->SetLocalTransform(BoneIndex, Transform, bPropagateToChildren);
				}
			}
			else
			{
				if (Weight >= 1.f - SMALL_NUMBER)
				{
					CR->GetHierarchy()->SetGlobalTransform(BoneIndex, Transform, bPropagateToChildren);
				}
				else
				{
					FTransform PreviousTransform = CR->GetHierarchy()->GetGlobalTransform(BoneIndex);
					FTransform BlendedTransform = FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, Weight);
					CR->GetHierarchy()->SetGlobalTransform(BoneIndex, Transform, bPropagateToChildren);
				}
			}
		}
	}
}

void UControlRigComponent::SetInitialBoneTransform(FName BoneName, FTransform InitialTransform, EControlRigComponentSpace Space, bool bPropagateToChildren)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		const int32 BoneIndex = CR->GetHierarchy()->GetIndex(FRigElementKey(BoneName, ERigElementType::Bone));
		if (BoneIndex != INDEX_NONE)
		{
			if(!CR->IsRunningPreConstruction() && !CR->IsRunningPostConstruction())
			{
				ReportError(TEXT("SetInitialBoneTransform should only be called during OnPreConstruction / OnPostConstruction."));
				return;
			}

			ConvertTransformToRigSpace(InitialTransform, Space);

			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				const int32 ParentIndex = CR->GetHierarchy()->GetFirstParent(BoneIndex);
				if(ParentIndex != INDEX_NONE)
				{
					InitialTransform = InitialTransform * CR->GetHierarchy()->GetInitialGlobalTransform(ParentIndex);
				}
			}

			CR->GetHierarchy()->SetInitialGlobalTransform(BoneIndex, InitialTransform, bPropagateToChildren);
		}
	}	
}

bool UControlRigComponent::GetControlBool(FName ControlName)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (ControlElement->Settings.ControlType == ERigControlType::Bool)
			{
				return CR->GetHierarchy()->GetControlValue(ControlElement, ERigControlValueType::Current).Get<bool>();
			}
		}
	}

	return false;
}

float UControlRigComponent::GetControlFloat(FName ControlName)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (ControlElement->Settings.ControlType == ERigControlType::Float || ControlElement->Settings.ControlType == ERigControlType::ScaleFloat)
			{
				return CR->GetHierarchy()->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
			}
		}
	}

	return 0.f;
}

int32 UControlRigComponent::GetControlInt(FName ControlName)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (ControlElement->Settings.ControlType == ERigControlType::Integer)
			{
				return CR->GetHierarchy()->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int>();
			}
		}
	}

	return 0.f;
}

FVector2D UControlRigComponent::GetControlVector2D(FName ControlName)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (ControlElement->Settings.ControlType == ERigControlType::Vector2D)
			{
				const FVector3f Value = CR->GetHierarchy()->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
				return FVector2D(Value.X, Value.Y);
			}
		}
	}

	return FVector2D::ZeroVector;
}

FVector UControlRigComponent::GetControlPosition(FName ControlName, EControlRigComponentSpace Space)
{
	return GetControlTransform(ControlName, Space).GetLocation();
}

FRotator UControlRigComponent::GetControlRotator(FName ControlName, EControlRigComponentSpace Space)
{
	return GetControlTransform(ControlName, Space).GetRotation().Rotator();
}

FVector UControlRigComponent::GetControlScale(FName ControlName, EControlRigComponentSpace Space)
{
	return GetControlTransform(ControlName, Space).GetScale3D();
}

FTransform UControlRigComponent::GetControlTransform(FName ControlName, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			FTransform Transform;
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				Transform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentLocal);
			}
			else
			{
				Transform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal);
				ConvertTransformFromRigSpace(Transform, Space);
			}
			return Transform;
		}
	}

	return FTransform::Identity;
}

void UControlRigComponent::SetControlBool(FName ControlName, bool Value)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		CR->SetControlValue<bool>(ControlName, Value);
	}
}

void UControlRigComponent::SetControlFloat(FName ControlName, float Value)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		CR->SetControlValue<float>(ControlName, Value);
	}
}

void UControlRigComponent::SetControlInt(FName ControlName, int32 Value)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		CR->SetControlValue<int32>(ControlName, Value);
	}
}

void UControlRigComponent::SetControlVector2D(FName ControlName, FVector2D Value)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		CR->SetControlValue<FVector2D>(ControlName, Value);
	}
}

void UControlRigComponent::SetControlPosition(FName ControlName, FVector Value, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			FTransform InputTransform = FTransform::Identity;
			InputTransform.SetLocation(Value);

			FTransform PreviousTransform = FTransform::Identity;

			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				PreviousTransform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentLocal); 
			}
			else
			{
				PreviousTransform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal); 
				ConvertTransformToRigSpace(InputTransform, Space);
			}

			PreviousTransform.SetTranslation(InputTransform.GetTranslation());
			SetControlTransform(ControlName, PreviousTransform, Space);
		}
	}
}

void UControlRigComponent::SetControlRotator(FName ControlName, FRotator Value, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			FTransform InputTransform = FTransform::Identity;
			InputTransform.SetRotation(FQuat(Value));

			FTransform PreviousTransform = FTransform::Identity;

			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				PreviousTransform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentLocal); 
			}
			else
			{
				PreviousTransform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal); 
				ConvertTransformToRigSpace(InputTransform, Space);
			}

			PreviousTransform.SetRotation(InputTransform.GetRotation());
			SetControlTransform(ControlName, PreviousTransform, Space);
		}
	}
}

void UControlRigComponent::SetControlScale(FName ControlName, FVector Value, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			FTransform InputTransform = FTransform::Identity;
			InputTransform.SetScale3D(Value);

			FTransform PreviousTransform = FTransform::Identity;

			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				PreviousTransform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentLocal); 
			}
			else
			{
				PreviousTransform = CR->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal); 
				ConvertTransformToRigSpace(InputTransform, Space);
			}

			PreviousTransform.SetScale3D(InputTransform.GetScale3D());
			SetControlTransform(ControlName, PreviousTransform, Space);
		}
	}
}

void UControlRigComponent::SetControlTransform(FName ControlName, FTransform Value, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				CR->GetHierarchy()->SetTransform(ControlElement, Value, ERigTransformType::CurrentLocal, true);
			}
			else
			{
				ConvertTransformToRigSpace(Value, Space);
				CR->GetHierarchy()->SetTransform(ControlElement, Value, ERigTransformType::CurrentGlobal, true);
			}
		}
	}
}

FTransform UControlRigComponent::GetControlOffset(FName ControlName, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				return CR->GetHierarchy()->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentLocal);
			}
			else
			{
				FTransform RootTransform = CR->GetHierarchy()->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentGlobal);
				ConvertTransformFromRigSpace(RootTransform, Space);
				return RootTransform;
			}
		}
	}

	return FTransform::Identity;
}

void UControlRigComponent::SetControlOffset(FName ControlName, FTransform OffsetTransform, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigControlElement* ControlElement = CR->GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
		{
			if (Space != EControlRigComponentSpace::LocalSpace)
			{
				ConvertTransformToRigSpace(OffsetTransform, Space);
			}

			CR->GetHierarchy()->SetControlOffsetTransform(ControlElement, OffsetTransform,
				Space == EControlRigComponentSpace::LocalSpace ? ERigTransformType::CurrentLocal : ERigTransformType::CurrentGlobal, true, false); 
		}
	}
}

FTransform UControlRigComponent::GetSpaceTransform(FName SpaceName, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigNullElement* NullElement = CR->GetHierarchy()->Find<FRigNullElement>(FRigElementKey(SpaceName, ERigElementType::Control)))
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				return CR->GetHierarchy()->GetTransform(NullElement, ERigTransformType::CurrentLocal);
			}
			else
			{
				FTransform RootTransform = CR->GetHierarchy()->GetTransform(NullElement, ERigTransformType::CurrentGlobal);
				ConvertTransformFromRigSpace(RootTransform, Space);
				return RootTransform;
			}
		}
	}

	return FTransform::Identity;
}

FTransform UControlRigComponent::GetInitialSpaceTransform(FName SpaceName, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigNullElement* NullElement = CR->GetHierarchy()->Find<FRigNullElement>(FRigElementKey(SpaceName, ERigElementType::Control)))
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				return CR->GetHierarchy()->GetTransform(NullElement, ERigTransformType::InitialLocal);
			}
			else
			{
				FTransform RootTransform = CR->GetHierarchy()->GetTransform(NullElement, ERigTransformType::InitialGlobal);
				ConvertTransformFromRigSpace(RootTransform, Space);
				return RootTransform;
			}
		}
	}

	return FTransform::Identity;
}

void UControlRigComponent::SetInitialSpaceTransform(FName SpaceName, FTransform InitialTransform, EControlRigComponentSpace Space)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if(FRigNullElement* NullElement = CR->GetHierarchy()->Find<FRigNullElement>(FRigElementKey(SpaceName, ERigElementType::Control)))
		{
			if (Space == EControlRigComponentSpace::LocalSpace)
			{
				CR->GetHierarchy()->SetTransform(NullElement, InitialTransform, ERigTransformType::InitialLocal, true, false);
			}
			else
			{
				ConvertTransformToRigSpace(InitialTransform, Space);
				CR->GetHierarchy()->SetTransform(NullElement, InitialTransform, ERigTransformType::InitialGlobal, true, false);
			}
		}
	}
}

UControlRig* UControlRigComponent::SetupControlRigIfRequired()
{
	if(ControlRig != nullptr)
	{
		if (!ControlRigAssetReference.IsSourceOf(ControlRig))
		{
			SetControlRig(nullptr);
		}
		else
		{
			return ControlRig;
		}
	}

	if(ControlRigAssetReference.IsValid())
	{
		UControlRig* NewControlRig = ControlRigAssetReference.CreateInstance(this);

		SetControlRig(NewControlRig);

		if (ControlRigCreatedEvent.IsBound())
		{
			ControlRigCreatedEvent.Broadcast(this);
		}

		ValidateMappingData();
	}

	return ControlRig;
}
void UControlRigComponent::SetControlRig(UControlRig* InControlRig)
{
	if (ControlRig != InControlRig)
	{
		if (ControlRig)
		{
			ControlRig->OnInitialized_AnyThread().RemoveAll(this);
			ControlRig->OnPreConstruction_AnyThread().RemoveAll(this);
			ControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
			ControlRig->OnPreForwardsSolve_AnyThread().RemoveAll(this);
			ControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
			ControlRig->OnExecuted_AnyThread().RemoveAll(this);

			// rename the previous rig.
			// GC will pick it up eventually - since we won't have any
			// owning pointers to it anymore.
			ControlRig->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			ControlRig->MarkAsGarbage();
		}
		ControlRig = InControlRig;
	}

	if (ControlRig)
	{
		ControlRig->OnInitialized_AnyThread().AddUObject(this, &UControlRigComponent::HandleControlRigInitializedEvent);
		ControlRig->OnPreConstruction_AnyThread().AddUObject(this, &UControlRigComponent::HandleControlRigPreConstructionEvent);
		ControlRig->OnPostConstruction_AnyThread().AddUObject(this, &UControlRigComponent::HandleControlRigPostConstructionEvent);
		ControlRig->OnPreForwardsSolve_AnyThread().AddUObject(this, &UControlRigComponent::HandleControlRigPreForwardsSolveEvent);
		ControlRig->OnPostForwardsSolve_AnyThread().AddUObject(this, &UControlRigComponent::HandleControlRigPostForwardsSolveEvent);
		ControlRig->OnExecuted_AnyThread().AddUObject(this, &UControlRigComponent::HandleControlRigExecutedEvent);

		ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, this);
		if(ObjectBinding.IsValid())
		{
			ControlRig->SetObjectBinding(ObjectBinding);
		}

		ControlRig->Initialize();
	}
}

void UControlRigComponent::SetControlRigClass(TSubclassOf<UControlRig> InControlRigClass)
{
	FControlRigAssetStrongReference AssetReference(InControlRigClass);
	SetControlRigAssetReference(AssetReference);
}

void UControlRigComponent::SetControlRigAssetReference(FControlRigAssetStrongReference InControlRigAssetReference)
{
	if (ControlRigAssetReference != InControlRigAssetReference)
	{
		SetControlRig(nullptr);
		ControlRigAssetReference = InControlRigAssetReference;
		Initialize();
	}
}

void UControlRigComponent::SetObjectBinding(UObject* InObjectToBind)
{
	if(!ObjectBinding.IsValid())
	{
		ObjectBinding = MakeShared<FControlRigObjectBinding>();
	}
	ObjectBinding->BindToObject(InObjectToBind);

	if(UControlRig* CR = SetupControlRigIfRequired())
	{
		CR->SetObjectBinding(ObjectBinding);
	}
}

void UControlRigComponent::ValidateMappingData()
{
	TMap<USkeletalMeshComponent*, FCachedSkeletalMeshComponentSettings> NewCachedSettings;

	if(ControlRig)
	{
		for (FControlRigComponentMappedElement& MappedElement : MappedElements)
		{
			MappedElement.ElementIndex = INDEX_NONE;
			MappedElement.SubIndex = INDEX_NONE;

			AActor* MappedOwner = !MappedElement.ComponentReference.OtherActor.IsValid() ? GetOwner() : MappedElement.ComponentReference.OtherActor.Get();
			MappedElement.SceneComponent = Cast<USceneComponent>(MappedElement.ComponentReference.GetComponent(MappedOwner));

			// try again with the path to the component
			if (MappedElement.SceneComponent == nullptr)
			{
				FSoftComponentReference TempReference;
				TempReference.PathToComponent = MappedElement.ComponentReference.ComponentProperty.ToString();
				if (USceneComponent* TempSceneComponent = Cast<USceneComponent>(TempReference.GetComponent(MappedOwner)))
				{
					MappedElement.ComponentReference = TempReference;
					MappedElement.SceneComponent = TempSceneComponent;
				}
			}

			if (MappedElement.SceneComponent == nullptr ||
				MappedElement.SceneComponent == this ||
				MappedElement.ElementName.IsNone())
			{
				continue;
			}

			// cache the scene component also in the override component to avoid on further relying on names
			MappedElement.ComponentReference.OverrideComponent = MappedElement.SceneComponent;

			if (MappedElement.Direction == EControlRigComponentMapDirection::Output && MappedElement.Weight <= SMALL_NUMBER)
			{
				continue;
			}

			FRigElementKey Key(MappedElement.ElementName, MappedElement.ElementType);
			MappedElement.ElementIndex = ControlRig->GetHierarchy()->GetIndex(Key);
			MappedElement.SubIndex = MappedElement.TransformIndex;

			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MappedElement.SceneComponent))
			{
				//MappedElement.Space = EControlRigComponentSpace::ComponentSpace;

				MappedElement.SubIndex = INDEX_NONE;
				if (MappedElement.TransformIndex >= 0 && MappedElement.TransformIndex < SkeletalMeshComponent->GetNumBones())
				{
					MappedElement.SubIndex = MappedElement.TransformIndex;
				}
				else if (!MappedElement.TransformName.IsNone())
				{
					if (USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
					{
						if (USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
						{
							if (MappedElement.ElementType != ERigElementType::Curve)
							{
								// this is really getting the FMeshPoseBoneIndex
								MappedElement.SubIndex = SkeletalMeshComponent->GetBoneIndex(MappedElement.TransformName);
							}
						}
						else
						{
							ReportError(FString::Printf(TEXT("%s does not have a Skeleton set."), *SkeletalMesh->GetPathName()));
						}
					}

					// if we didn't find the bone, disable this mapped element
					if (MappedElement.ElementType != ERigElementType::Curve)
					{
						if (MappedElement.SubIndex == INDEX_NONE)
						{
							MappedElement.ElementIndex = INDEX_NONE;
							continue;
						}
					}
				}

				if (MappedElement.Direction == EControlRigComponentMapDirection::Output)
				{
					if (!NewCachedSettings.Contains(SkeletalMeshComponent))
					{
						FCachedSkeletalMeshComponentSettings PreviousSettings(SkeletalMeshComponent);
						NewCachedSettings.Add(SkeletalMeshComponent, PreviousSettings);
					}

					//If the animinstance is a sequencer instance don't replace it that means we are already running an animation on the skeleton
					//and don't want to replace the anim instance.
					if (Cast<ISequencerAnimationSupport>(SkeletalMeshComponent->GetAnimInstance()) == nullptr)
					{
						SkeletalMeshComponent->SetAnimInstanceClass(UControlRigAnimInstance::StaticClass());
					}
				}
			}
		}
	}

	// for the skeletal mesh components we no longer map, let's remove it
	for (TPair<USkeletalMeshComponent*, FCachedSkeletalMeshComponentSettings>& Pair : CachedSkeletalMeshComponentSettings)
	{
		FCachedSkeletalMeshComponentSettings* NewCachedSetting = NewCachedSettings.Find(Pair.Key);
		if (NewCachedSetting)
		{
			*NewCachedSetting = Pair.Value;
		}
		else
		{
			Pair.Value.Apply(Pair.Key);
		}
	}

	CachedSkeletalMeshComponentSettings = NewCachedSettings;
}

void UControlRigComponent::TransferInputs()
{
	bNeedsEvaluation = true;
	if (ControlRig)
	{
		InputElementIndices.Reset();
		InputTransforms.Reset();
		
		for (FControlRigComponentMappedElement& MappedElement : MappedElements)
		{
			if (MappedElement.ElementIndex == INDEX_NONE || MappedElement.Direction == EControlRigComponentMapDirection::Output)
			{
				continue;
			}

			if (!IsValid(MappedElement.SceneComponent))
			{
				continue;
			}

			FTransform Transform = FTransform::Identity;
			if (MappedElement.SubIndex >= 0)
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MappedElement.SceneComponent))
				{
					Transform = SkeletalMeshComponent->GetBoneTransform(MappedElement.SubIndex, FTransform::Identity);
					if(MappedElement.Space == EControlRigComponentSpace::WorldSpace)
					{
						Transform = Transform * MappedElement.SceneComponent->GetComponentToWorld();
					}
				}
				else if (UInstancedStaticMeshComponent* InstancingComponent = Cast<UInstancedStaticMeshComponent>(MappedElement.SceneComponent))
				{
					if (MappedElement.SubIndex < InstancingComponent->GetNumRenderInstances())
					{
						InstancingComponent->GetInstanceTransform(MappedElement.SubIndex, Transform, true);
					}
					else
					{
						continue;
					}
				}
			}
			else
			{
				Transform = MappedElement.SceneComponent->GetComponentToWorld();
			}

			Transform = Transform * MappedElement.Offset;

			ConvertTransformToRigSpace(Transform, MappedElement.Space);

			InputElementIndices.Add(MappedElement.ElementIndex);
			InputTransforms.Add(Transform);
		}

		if(bEnableLazyEvaluation && InputTransforms.Num() > 0)
		{
			if(LastInputTransforms.Num() == InputTransforms.Num())
			{
				bNeedsEvaluation = false;

				const float PositionU = FMath::Abs(LazyEvaluationPositionThreshold);
				const float RotationU = FMath::Abs(LazyEvaluationRotationThreshold);
				const float ScaleU = FMath::Abs(LazyEvaluationScaleThreshold);

				for(int32 Index=0;Index<InputElementIndices.Num();Index++)
				{
					if(!FRigUnit_PoseGetDelta::AreTransformsEqual(
						InputTransforms[Index],
						LastInputTransforms[Index],
						PositionU,
						RotationU,
						ScaleU))
					{
						bNeedsEvaluation = true;
						break;
					}
				}
			}

			LastInputTransforms = InputTransforms;
		}

		if(bNeedsEvaluation)
		{
			for(int32 Index=0;Index<InputElementIndices.Num();Index++)
			{
				ControlRig->GetHierarchy()->SetGlobalTransform(InputElementIndices[Index], InputTransforms[Index]);
			}
		}

		if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->ResetChangedCurveIndices();
			
#if WITH_EDITOR
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("UControlRigComponent::TransferInputs"));
			}
#endif
		}
	}
}

void UControlRigComponent::TransferOutputs()
{
	if (ControlRig)
	{
		if (ControlRig->IsAdditive())
		{
			if (MappedElements.Num() > 0)
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(MappedElements[0].SceneComponent))
				{
					if (USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
					{
						USceneComponent* LastComponent = nullptr;
						FControlRigAnimInstanceProxy* Proxy = nullptr;
						
						TArray<FTransform> BoneSpaceTransforms;
						BoneSpaceTransforms.SetNumUninitialized(SkeletalMesh->GetRefSkeleton().GetNum());
						for (const FControlRigComponentMappedElement& MappedElement : MappedElements)
						{
							if (LastComponent != MappedElement.SceneComponent || Proxy == nullptr)
							{
								Proxy = MappedElement.GetAnimProxyOnGameThread();
								if (Proxy)
								{
									Proxy->StoredTransforms.Reset();
									Proxy->StoredCurves.Reset();
									Proxy->StoredAttributes.Empty();
									LastComponent = MappedElement.SceneComponent;
								}
							}
							
							if (MappedElement.ElementIndex == INDEX_NONE || MappedElement.Direction == EControlRigComponentMapDirection::Input)
							{
								continue;
							}

							if (MappedElement.ElementType == ERigElementType::Bone ||
								MappedElement.ElementType == ERigElementType::Control ||
								MappedElement.ElementType == ERigElementType::Null)
							{
								if (MappedElement.SubIndex >= 0 && BoneSpaceTransforms.IsValidIndex(MappedElement.SubIndex))
								{
									FTransform Transform = ControlRig->GetHierarchy()->GetLocalTransform(MappedElement.ElementIndex);
									BoneSpaceTransforms[MappedElement.SubIndex] = Transform;
								}
							}
						}

						TArray<FTransform> OutSpaceBases;
						OutSpaceBases.SetNumUninitialized(BoneSpaceTransforms.Num());
						SkeletalMesh->FillComponentSpaceTransforms(BoneSpaceTransforms, SkeletalMeshComponent->FillComponentSpaceTransformsRequiredBones, SkeletalMeshComponent->GetEditableComponentSpaceTransforms());

						if (Proxy)
						{
							Proxy->StoredAttributes.CopyFrom(TempAttributeContainer);
						}	
					}
				}
			}
		}
		else
		{
			USceneComponent* LastComponent = nullptr;
			FControlRigAnimInstanceProxy* Proxy = nullptr;

			for (const FControlRigComponentMappedElement& MappedElement : MappedElements)
			{
				if (LastComponent != MappedElement.SceneComponent || Proxy == nullptr)
				{
					Proxy = MappedElement.GetAnimProxyOnGameThread();
					if (Proxy)
					{
						Proxy->StoredTransforms.Reset();
						Proxy->StoredCurves.Reset();
						Proxy->StoredAttributes.Empty();
						LastComponent = MappedElement.SceneComponent;
					}
				}
			}

			for (const FControlRigComponentMappedElement& MappedElement : MappedElements)
			{
				if (MappedElement.ElementIndex == INDEX_NONE || MappedElement.Direction == EControlRigComponentMapDirection::Input)
				{
					continue;
				}

				if (!IsValid(MappedElement.SceneComponent))
				{
					continue;
				}

				if (MappedElement.ElementType == ERigElementType::Bone ||
					MappedElement.ElementType == ERigElementType::Control ||
					MappedElement.ElementType == ERigElementType::Null)
				{
					FTransform Transform = ControlRig->GetHierarchy()->GetGlobalTransform(MappedElement.ElementIndex);
					ConvertTransformFromRigSpace(Transform, MappedElement.Space);

					Transform = Transform * MappedElement.Offset;

					if (MappedElement.SubIndex >= 0)
					{
						if (LastComponent != MappedElement.SceneComponent || Proxy == nullptr)
						{
							Proxy = MappedElement.GetAnimProxyOnGameThread();
							if (Proxy)
							{
								LastComponent = MappedElement.SceneComponent;
							}
						}

						if (Proxy && (MappedElement.SceneComponent != nullptr))
						{
							if(MappedElement.Space == EControlRigComponentSpace::WorldSpace)
							{
								Transform = Transform.GetRelativeTransform(MappedElement.SceneComponent->GetComponentToWorld());
							}
							Proxy->StoredTransforms.FindOrAdd(FMeshPoseBoneIndex(MappedElement.SubIndex)) = Transform;
						}
						else if (UInstancedStaticMeshComponent* InstancingComponent = Cast<UInstancedStaticMeshComponent>(MappedElement.SceneComponent))
						{
							if (MappedElement.SubIndex < InstancingComponent->GetNumRenderInstances())
							{
								if (MappedElement.Weight < 1.f - SMALL_NUMBER)
								{
									FTransform Previous = FTransform::Identity;
									InstancingComponent->GetInstanceTransform(MappedElement.SubIndex, Previous, true);
									Transform = FControlRigMathLibrary::LerpTransform(Previous, Transform, FMath::Clamp<float>(MappedElement.Weight, 0.f, 1.f));
								}
								InstancingComponent->UpdateInstanceTransform(MappedElement.SubIndex, Transform, true, true, true);
							}
						}
					}
					else
					{
						if (MappedElement.Weight < 1.f - SMALL_NUMBER)
						{
							FTransform Previous = MappedElement.SceneComponent->GetComponentToWorld();
							Transform = FControlRigMathLibrary::LerpTransform(Previous, Transform, FMath::Clamp<float>(MappedElement.Weight, 0.f, 1.f));
						}
						MappedElement.SceneComponent->SetWorldTransform(Transform);
					}
				}
				else if (MappedElement.ElementType == ERigElementType::Curve)
				{
					if (MappedElement.ElementIndex >= 0)
					{
						if (LastComponent != MappedElement.SceneComponent || Proxy == nullptr)
						{
							Proxy = MappedElement.GetAnimProxyOnGameThread();
							if (Proxy)
							{
								LastComponent = MappedElement.SceneComponent;
							}
						}

						if (Proxy)
						{
							Proxy->StoredCurves.FindOrAdd(MappedElement.TransformName) = ControlRig->GetHierarchy()->GetCurveValue(MappedElement.ElementIndex);
						}
					}
				}
			}

			if (Proxy)
			{
				Proxy->StoredAttributes.CopyFrom(TempAttributeContainer);
			}			
		}

#if WITH_EDITOR
		if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			if(Hierarchy->IsTracingChanges())
			{
				Hierarchy->StorePoseForTrace(TEXT("UControlRigComponent::TransferOutputs"));
			}
		}
#endif
	}
}

FName UControlRigComponent::GetComponentNameWithinActor(UActorComponent* InComponent)
{
	check(InComponent);
	
	FName ComponentProperty = InComponent->GetFName(); 

	if(AActor* Owner = InComponent->GetOwner())
	{
		// we need to see if the owner stores this component as a property
		for (TFieldIterator<FProperty> PropertyIt(Owner->GetClass()); PropertyIt; ++PropertyIt)
		{
			if(const FObjectPropertyBase* Property = CastField<FObjectPropertyBase>(*PropertyIt))
			{
				if(Property->GetObjectPropertyValue_InContainer(Owner) == InComponent)
				{
					ComponentProperty = Property->GetFName();
					break;
				}
			}
		}

#if WITH_EDITOR

		// validate that the property storage will return the right component.
		// this is a sanity check ensuring that ComponentReference will find the right component later.
		FObjectPropertyBase* Property = FindFProperty<FObjectPropertyBase>(Owner->GetClass(), ComponentProperty);
		if(Property != nullptr)
		{
			UActorComponent* FoundComponent = Cast<UActorComponent>(Property->GetObjectPropertyValue_InContainer(Owner));
			check(FoundComponent == InComponent);
		}
		
#endif
	}
	return ComponentProperty;
}

void UControlRigComponent::HandleControlRigInitializedEvent(URigVMHost* InControlRig, const FName& InEventName)
{
#if WITH_EDITOR
	if (bUpdateInEditor)
	{
		FEditorScriptExecutionGuard AllowScripts;
		OnPostInitialize(this);
	}
	else
#endif
	{
		OnPostInitialize(this);
	}
}

void UControlRigComponent::HandleControlRigPreConstructionEvent(UControlRig* InControlRig, const FName& InEventName)
{
	TArray<USkeletalMeshComponent*> ComponentsToTick;

	USceneComponent* LastComponent = nullptr;
	FControlRigAnimInstanceProxy* Proxy = nullptr;

	for (FControlRigComponentMappedElement& MappedElement : MappedElements)
	{
		if (LastComponent != MappedElement.SceneComponent || Proxy == nullptr)
		{
			Proxy = MappedElement.GetAnimProxyOnGameThread();
			if (Proxy)
			{
				Proxy->StoredTransforms.Reset();
				Proxy->StoredCurves.Reset();
				Proxy->StoredAttributes.Empty();
				LastComponent = MappedElement.SceneComponent;
			}
		}

		if (USkeletalMeshComponent* Component = Cast< USkeletalMeshComponent>(MappedElement.SceneComponent))
		{
			ComponentsToTick.AddUnique(Component);
		}
	}

	for (USkeletalMeshComponent* SkeletalMeshComponent : ComponentsToTick)
	{
		SkeletalMeshComponent->TickAnimation(0.f, false);
		SkeletalMeshComponent->RefreshBoneTransforms();
		SkeletalMeshComponent->RefreshFollowerComponents();
		SkeletalMeshComponent->UpdateComponentToWorld();
		SkeletalMeshComponent->FinalizeBoneTransform();
		SkeletalMeshComponent->MarkRenderTransformDirty();
		SkeletalMeshComponent->MarkRenderDynamicDataDirty();
	}

#if WITH_EDITOR
	if (bUpdateInEditor)
	{
		FEditorScriptExecutionGuard AllowScripts;
		OnPreConstruction(this);
	}
	else
#endif
	{
		OnPreConstruction(this);
	}
}

void UControlRigComponent::HandleControlRigPostConstructionEvent(UControlRig* InControlRig, const FName& InEventName)
{
#if WITH_EDITOR
	if (bUpdateInEditor)
	{
		FEditorScriptExecutionGuard AllowScripts;
		OnPostConstruction(this);
	}
	else
#endif
	{
		OnPostConstruction(this);
	}
}

void UControlRigComponent::HandleControlRigPreForwardsSolveEvent(UControlRig* InControlRig, const FName& InEventName)
{
#if WITH_EDITOR
	if (bUpdateInEditor)
	{
		FEditorScriptExecutionGuard AllowScripts;
		OnPreForwardsSolve(this);
	}
	else
#endif
	{
		OnPreForwardsSolve(this);
	}
}

void UControlRigComponent::HandleControlRigPostForwardsSolveEvent(UControlRig* InControlRig, const FName& InEventName)
{
#if WITH_EDITOR
	if (bUpdateInEditor)
	{
		FEditorScriptExecutionGuard AllowScripts;
		OnPostForwardsSolve(this);
	}
	else
#endif
	{
		OnPostForwardsSolve(this);
	}
}

void UControlRigComponent::HandleControlRigExecutedEvent(URigVMHost* InControlRig, const FName& InEventName)
{
	TransferOutputs();
}

void UControlRigComponent::ConvertTransformToRigSpace(FTransform& InOutTransform, EControlRigComponentSpace FromSpace)
{
	switch (FromSpace)
	{
		case EControlRigComponentSpace::WorldSpace:
		{
			InOutTransform = InOutTransform.GetRelativeTransform(GetComponentToWorld());
			break;
		}
		case EControlRigComponentSpace::ActorSpace:
		{
			InOutTransform = InOutTransform.GetRelativeTransform(GetRelativeTransform());
			break;
		}
		case EControlRigComponentSpace::ComponentSpace:
		case EControlRigComponentSpace::RigSpace:
		case EControlRigComponentSpace::LocalSpace:
		default:
		{
			// nothing to do
			break;
		}
	}
}

void UControlRigComponent::ConvertTransformFromRigSpace(FTransform& InOutTransform, EControlRigComponentSpace ToSpace)
{
	switch (ToSpace)
	{
		case EControlRigComponentSpace::WorldSpace:
		{
			InOutTransform = InOutTransform * GetComponentToWorld();
			break;
		}
		case EControlRigComponentSpace::ActorSpace:
		{
			InOutTransform = InOutTransform * GetRelativeTransform();
			break;
		}
		case EControlRigComponentSpace::ComponentSpace:
		case EControlRigComponentSpace::RigSpace:
		case EControlRigComponentSpace::LocalSpace:
		default:
		{
			// nothing to do
			break;
		}
	}
}

bool UControlRigComponent::EnsureCalledOutsideOfBracket(const TCHAR* InCallingFunctionName)
{
	if (UControlRig* CR = SetupControlRigIfRequired())
	{
		if (CR->IsRunningPreConstruction())
		{
			if (InCallingFunctionName)
			{
				ReportError(FString::Printf(TEXT("%s cannot be called during the PreConstructionEvent - use ConstructionScript instead."), InCallingFunctionName));
				return false;
			}
			else
			{
				ReportError(TEXT("Cannot be called during the PreConstructionEvent - use ConstructionScript instead."));
				return false;
			}
		}

		if (CR->IsRunningPostConstruction())
		{
			if (InCallingFunctionName)
			{
				ReportError(FString::Printf(TEXT("%s cannot be called during the PostConstructionEvent - use ConstructionScript instead."), InCallingFunctionName));
				return false;
			}
			else
			{
				ReportError(TEXT("Cannot be called during the PostConstructionEvent - use ConstructionScript instead."));
				return false;
			}
		}

		if (CR->IsInitializing())
		{
			if (InCallingFunctionName)
			{
				ReportError(FString::Printf(TEXT("%s cannot be called during the InitEvent - use ConstructionScript instead."), InCallingFunctionName));
				return false;
			}
			else
			{
				ReportError(TEXT("Cannot be called during the InitEvent - use ConstructionScript instead."));
				return false;
			}
		}

		if (CR->IsExecuting())
		{
			if (InCallingFunctionName)
			{
				ReportError(FString::Printf(TEXT("%s cannot be called during the ForwardsSolveEvent - use ConstructionScript instead."), InCallingFunctionName));
				return false;
			}
			else
			{
				ReportError(TEXT("Cannot be called during the ForwardsSolveEvent - use ConstructionScript instead."));
				return false;
			}
		}
	}
	
	return true;
}

void UControlRigComponent::ReportError(const FString& InMessage)
{
	UE_LOGF(LogControlRig, Warning, "%ls: %ls", *GetPathName(), *InMessage);

#if WITH_EDITOR

	if (GetWorld()->IsEditorWorld())
	{
		const TSharedPtr<SNotificationItem>* ExistingItemPtr = EditorNotifications.Find(InMessage);
		if (ExistingItemPtr)
		{
			const TSharedPtr<SNotificationItem>& ExistingItem = *ExistingItemPtr;
			if (ExistingItem.IsValid())
			{
				if (ExistingItem->HasActiveTimers())
				{
					return;
				}
				else
				{
					EditorNotifications.Remove(InMessage);
				}
			}
		}

		FNotificationInfo Info(FText::FromString(InMessage));
		Info.bUseSuccessFailIcons = true;
		Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
		Info.bFireAndForget = true;
		Info.bUseThrobber = true;
		Info.FadeOutDuration = 8.0f;
		Info.ExpireDuration = Info.FadeOutDuration;
		TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);

		EditorNotifications.Add(InMessage, NotificationPtr);
	}
#endif
}

bool UControlRigComponent::HasTickDependency(const UObject* InDependent, const UObject* InDependency, FString* OutErrorMessage)
{
	TSet<const UObject*> Visited;
	return HasTickDependency_Impl(InDependent, InDependency, Visited, OutErrorMessage);
}

bool UControlRigComponent::HasTickDependency_Impl(const UObject* InDependent, const UObject* InDependency, TSet<const UObject*>& InOutVisited, FString* OutErrorMessage)
{
	if(InDependent == InDependency)
	{
		return true;
	}

	// avoid cycles / infinite recursion
	if(InOutVisited.Contains(InDependent))
	{
		return false;
	}
	InOutVisited.Add(InDependent);
	
	if(const USceneComponent* DependencyComponent = Cast<USceneComponent>(InDependency))
	{
		if(const USceneComponent* DependentComponent = Cast<USceneComponent>(InDependent))
		{
			if(const AActor* DependencyActor = DependencyComponent->GetOwner())
			{
				if(const AActor* DependentActor = DependentComponent->GetOwner())
				{
					if(const USceneComponent* DependentParentComponent = DependentComponent->GetAttachParent())
					{
						if(HasTickDependency_Impl(DependentParentComponent, InDependency, InOutVisited, OutErrorMessage))
						{
#if WITH_EDITOR
							if(DependencyActor == DependentActor)
							{
								if(DependentParentComponent == InDependency)
								{
									if(OutErrorMessage)
									{
										if(!OutErrorMessage->IsEmpty())
										{
											*OutErrorMessage += TEXT("\n");
										}
										static constexpr TCHAR Format[] = TEXT("The dependency '%s' is a parent component of dependent '%s'.");
										*OutErrorMessage += FString::Printf(Format, *InDependency->GetPathName(), *InDependent->GetPathName());
									}
								}
							}
#endif
							return true;
						}
					}
				}
			}
		}
		else if(const AActor* DependencyActor = Cast<AActor>(InDependency))
		{
			if(DependencyActor->GetComponents().Contains(DependentComponent))
			{
#if WITH_EDITOR
				if(OutErrorMessage)
				{
					if(!OutErrorMessage->IsEmpty())
					{
						*OutErrorMessage += TEXT("\n");
					}
					static constexpr TCHAR Format[] = TEXT("The actor '%s' owns dependent component '%s'.");
					*OutErrorMessage += FString::Printf(Format, *DependencyActor->GetPathName(), *InDependent->GetPathName());
				}
#endif
				return true;
			}
		}
	}

	// iterate all existing tick dependency components
	if(const UActorComponent* DependentActorComponent = Cast<UActorComponent>(InDependent))
	{
		const TArray<struct FTickPrerequisite>& Prerequisites = DependentActorComponent->PrimaryComponentTick.GetPrerequisites();
		for(const struct FTickPrerequisite& Prerequisite : Prerequisites)
		{
			if(Prerequisite.PrerequisiteObject.IsValid())
			{
				if(const UObject* TickDependency = Prerequisite.PrerequisiteObject.Get())
				{
					if(TickDependency == InDependency)
					{
#if WITH_EDITOR
						if(OutErrorMessage)
						{
							if(!OutErrorMessage->IsEmpty())
							{
								*OutErrorMessage += TEXT("\n");
							}
							static constexpr TCHAR Format[] = TEXT("The dependency '%s' is a prerequisite of dependent'%s'.");
							*OutErrorMessage += FString::Printf(Format, *InDependency->GetPathName(), *InDependent->GetPathName());
						}
#endif
						return true;
					}
					
					if(HasTickDependency_Impl(TickDependency, InDependency, InOutVisited, OutErrorMessage))
					{
#if WITH_EDITOR
						if(OutErrorMessage)
						{
							if(!OutErrorMessage->IsEmpty())
							{
								*OutErrorMessage += TEXT("\n");
							}
							static constexpr TCHAR Format[] = TEXT("The dependency '%s' is a indirect prerequisite of dependent '%s'.");
							*OutErrorMessage += FString::Printf(Format, *InDependency->GetPathName(), *InDependent->GetPathName());
						}
#endif
						return true;
					}
				}
			}
		}
	}
	
	return false;
}

bool UControlRigComponent::AllowsTickDependency(const UObject* InDependent, const UObject* InDependency, FString* OutErrorMessage)
{
	if (CVarControlRigDisableTickDependencyCheck.GetValueOnAnyThread())
	{
		return true;
	}
	
	if(InDependent == InDependency)
	{
#if WITH_EDITOR
		if(OutErrorMessage)
		{
			if(!OutErrorMessage->IsEmpty())
			{
				*OutErrorMessage += TEXT("\n");
			}
			static constexpr TCHAR Format[] = TEXT("The dependency and dependent '%s' are identical.");
			*OutErrorMessage += FString::Printf(Format, *InDependent->GetPathName());
		}
#endif
		return false;
	}
	
	// don't allow to depend on an inner
	if(InDependency->IsInOuter(InDependent))
	{
#if WITH_EDITOR
		if(OutErrorMessage)
		{
			if(!OutErrorMessage->IsEmpty())
			{
				*OutErrorMessage += TEXT("\n");
			}
			static constexpr TCHAR Format[] = TEXT("The dependency '%s' is within the dependent '%s'.");
			*OutErrorMessage += FString::Printf(Format, *InDependency->GetPathName(), *InDependent->GetPathName());
		}
#endif
		return false;
	}
	
	if(HasTickDependency(InDependent, InDependency, OutErrorMessage))
	{
#if WITH_EDITOR
		if(OutErrorMessage)
		{
			if(!OutErrorMessage->IsEmpty())
			{
				*OutErrorMessage += TEXT("\n");
			}
			static constexpr TCHAR Format[] = TEXT("The dependent '%s' already depends on dependency '%s'.");
			*OutErrorMessage += FString::Printf(Format, *InDependent->GetPathName(), *InDependency->GetPathName());
		}
#endif
		return false;
	}
	
	if(HasTickDependency(InDependency, InDependent, OutErrorMessage))
	{
#if WITH_EDITOR
		if(OutErrorMessage)
		{
			if(!OutErrorMessage->IsEmpty())
			{
				*OutErrorMessage += TEXT("\n");
			}
			static constexpr TCHAR Format[] = TEXT("The dependency '%s' depends on dependent '%s'. This would cause a cycle.");
			*OutErrorMessage += FString::Printf(Format, *InDependency->GetPathName(), *InDependent->GetPathName());
		}
#endif
		return false;
	}

	return true;
}

FControlRigSceneProxy::FControlRigSceneProxy(const UControlRigComponent* InComponent)
: FPrimitiveSceneProxy(InComponent)
, ControlRigComponent(InComponent)
{
	bWillEverBeLit = false;
}

SIZE_T FControlRigSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FControlRigSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	if (ControlRigComponent->ControlRig == nullptr)
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

#if UE_ENABLE_DEBUG_DRAWING
			FPrimitiveDrawInterface* PDI = Collector.GetDebugPDI(ViewIndex);
#else
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
#endif

			bool bShouldDrawBones = ControlRigComponent->bDrawBones && ControlRigComponent->ControlRig != nullptr;

			// make sure to check if we are within a preview / editor world
			// or the console variable draw bones is turned on
			if (bShouldDrawBones)
			{
				if (UWorld* World = ControlRigComponent->GetWorld())
				{
					if (!World->IsPreviewWorld())
					{
						const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;
						bShouldDrawBones = EngineShowFlags.Bones != 0;
					}
				}
			}

			if (bShouldDrawBones)
			{
				const float RadiusMultiplier = ControlRigComponent->ControlRig->GetDebugBoneRadiusMultiplier();
				const FTransform Transform = ControlRigComponent->GetComponentToWorld();
				const float MaxDrawRadius = ControlRigComponent->Bounds.SphereRadius * 0.02f;

				URigHierarchy* Hierarchy = ControlRigComponent->ControlRig->GetHierarchy();
				Hierarchy->ForEach<FRigBoneElement>([PDI, Hierarchy, Transform, MaxDrawRadius, RadiusMultiplier](FRigBoneElement* BoneElement) -> bool
                {
                    const int32 ParentIndex = Hierarchy->GetFirstParent(BoneElement->GetIndex());
					const FLinearColor LineColor = FLinearColor::White;

					FVector Start, End;
					if (ParentIndex >= 0)
					{
						Start = Hierarchy->GetGlobalTransform(ParentIndex).GetLocation();
						End = Hierarchy->GetGlobalTransform(BoneElement->GetIndex()).GetLocation();
					}
					else
					{
						Start = FVector::ZeroVector;
						End = Hierarchy->GetGlobalTransform(BoneElement->GetIndex()).GetLocation();
					}

					Start = Transform.TransformPosition(Start);
					End = Transform.TransformPosition(End);

					const float BoneLength = (End - Start).Size();
					// clamp by bound, we don't want too long or big
					const float Radius = FMath::Clamp<float>(BoneLength * 0.05f, 0.1f, MaxDrawRadius) * RadiusMultiplier;

					//Render Sphere for bone end point and a cone between it and its parent.
					SkeletalDebugRendering::DrawWireBone(PDI, Start, End, LineColor, SDPG_Foreground, Radius);

					return true;
				});
			}

			if (ControlRigComponent->bShowDebugDrawing)
			{ 
				const FRigVMDrawInterface& DrawInterface = ControlRigComponent->ControlRig->GetDrawInterface();

				for (int32 InstructionIndex = 0; InstructionIndex < DrawInterface.Num(); InstructionIndex++)
				{
					const FRigVMDrawInstruction& Instruction = DrawInterface[InstructionIndex];
					if (Instruction.Positions.Num() == 0)
					{
						continue;
					}

					FTransform InstructionTransform = Instruction.Transform * ControlRigComponent->GetComponentToWorld();
					switch (Instruction.PrimitiveType)
					{
						case ERigVMDrawSettings::Points:
						{
							for (const FVector& Point : Instruction.Positions)
							{
								PDI->DrawPoint(InstructionTransform.TransformPosition(Point), Instruction.Color, Instruction.Thickness, SDPG_Foreground);
							}
							break;
						}
						case ERigVMDrawSettings::Lines:
						{
							const TArray<FVector>& Points = Instruction.Positions;
							PDI->AddReserveLines(SDPG_Foreground, Points.Num() / 2, false, Instruction.Thickness > SMALL_NUMBER);
							for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex += 2)
							{
								PDI->DrawLine(InstructionTransform.TransformPosition(Points[PointIndex]), InstructionTransform.TransformPosition(Points[PointIndex + 1]), Instruction.Color, SDPG_Foreground, Instruction.Thickness);
							}
							break;
						}
						case ERigVMDrawSettings::LineStrip:
						{
							const TArray<FVector>& Points = Instruction.Positions;
							PDI->AddReserveLines(SDPG_Foreground, Points.Num() - 1, false, Instruction.Thickness > SMALL_NUMBER);
							for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
							{
								PDI->DrawLine(InstructionTransform.TransformPosition(Points[PointIndex]), InstructionTransform.TransformPosition(Points[PointIndex + 1]), Instruction.Color, SDPG_Foreground, Instruction.Thickness);
							}
							break;
						}
					}
				}
			}
		}
	}
}

/**
*  Returns a struct that describes to the renderer when to draw this proxy.
*	@param		Scene view to use to determine our relevence.
*  @return		View relevance struct
*/
FPrimitiveViewRelevance FControlRigSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance ViewRelevance;
	ViewRelevance.bDrawRelevance = IsShown(View);
	ViewRelevance.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	ViewRelevance.bSeparateTranslucency = ViewRelevance.bNormalTranslucency = true;
	return ViewRelevance;
}

uint32 FControlRigSceneProxy::GetMemoryFootprint(void) const
{
	return(sizeof(*this) + GetAllocatedSize());
}

uint32 FControlRigSceneProxy::GetAllocatedSize(void) const
{
	return FPrimitiveSceneProxy::GetAllocatedSize();
}

