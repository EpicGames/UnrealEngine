// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigControlActor.h"
#include "Components/StaticMeshComponent.h"
#include "ControlRigGizmoLibrary.h"
#include "IControlRigObjectBinding.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigControlActor)

#define LOCTEXT_NAMESPACE "ControlRigControlActor"

AControlRigControlActor::AControlRigControlActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRefreshOnTick(true)
	, bIsSelectable(true)
	, ColorParameter(TEXT("Color"))
	, bCastShadows(false)
{
	ActorRootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent0"));
	SetRootComponent(ActorRootComponent);

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bTickEvenWhenPaused = true;
	PrimaryActorTick.TickGroup = TG_PostUpdateWork;

	PrimaryActorTick.bStartWithTickEnabled = true;
	bAllowTickBeforeBeginPlay = true;
	
	SetActorEnableCollision(false);
	

	Refresh();
}

AControlRigControlActor::~AControlRigControlActor()
{
	RemoveUnbindDelegate();
}

void AControlRigControlActor::RemoveUnbindDelegate() const
{
	if (ControlRig)
	{
		if (TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding())
		{
			Binding->OnControlRigUnbind().RemoveAll(this);
		}
	}
}

void AControlRigControlActor::HandleControlRigUnbind()
{
	Clear();
	Refresh();
}

#if WITH_EDITOR

void AControlRigControlActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property &&
		(	PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AControlRigControlActor, ActorToTrack) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AControlRigControlActor, ControlRigClass) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AControlRigControlActor, MaterialOverride) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AControlRigControlActor, ColorParameter) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AControlRigControlActor, bCastShadows)))
	{
		ResetControlActor();
		Refresh();
	}
}

#endif

void AControlRigControlActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	if (bRefreshOnTick)
	{
		Refresh();
	}
}

void AControlRigControlActor::Clear()
{
	if (ActorRootComponent)
	{
		TArray<USceneComponent*> ChildComponents;
		ActorRootComponent->GetChildrenComponents(true, ChildComponents);
		for (USceneComponent* Child : ChildComponents)
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Child))
			{
				if (IsValid(StaticMeshComponent))
				{
					Components.AddUnique(StaticMeshComponent);	
				}
			}
		}

		for (USceneComponent* Component : Components)
		{
			if (IsValid(Component))
			{
				Component->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
				Component->UnregisterComponent();
				Component->DestroyComponent();	
			}
		}
	}

	ControlNames.Reset();
	ShapeTransforms.Reset();
	Components.Reset();
	Materials.Reset();
}

void AControlRigControlActor::ResetControlActor()
{
	RemoveUnbindDelegate();
	ControlRig = nullptr;
	Clear();
}

void AControlRigControlActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	ResetControlActor();
	Super::EndPlay(EndPlayReason);
}

void AControlRigControlActor::BeginDestroy()
{
	// since end play is not always called, we have to clear the delegate here
	// clearing it at destructor might be too late as in some cases, the control rig was already GCed
	ResetControlActor();
	Super::BeginDestroy();
}

void AControlRigControlActor::Refresh()
{
	if (ActorToTrack == nullptr)
	{
		return;
	}

	// workaround to make sure we get a clean reset when the object was initialized from an existing template instance but some arrays didn't get populated somehow...
	if (ControlNames.IsEmpty() || ControlNames.Num() != Components.Num() || ControlNames.Num() != Materials.Num() || ControlNames.Num() != ShapeTransforms.Num())
	{
		ResetControlActor();
	}

	if (ControlRig == nullptr)
	{
		TArray<UControlRig*> Rigs = UControlRig::FindControlRigs(ActorToTrack, ControlRigClass);
		if(Rigs.Num() > 0)
		{
			ControlRig = Rigs[0];
		}
		
		if (ControlRig == nullptr)
		{
			return;
		}

		RemoveUnbindDelegate();
		if (const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding())
		{
			Binding->OnControlRigUnbind().AddUObject(this, &AControlRigControlActor::HandleControlRigUnbind);
		}

		const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& ShapeLibraries = ControlRig->GetShapeLibraries();
		if(ShapeLibraries.IsEmpty())
		{
			return;
		}
		
		// disable collision again
		SetActorEnableCollision(false);

		for(const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary : ShapeLibraries)
		{
			ShapeLibrary->DefaultMaterial.LoadSynchronous();
		}

		const FShapeProxyComponentProviderRegistry& ProviderRegistry = FShapeProxyComponentProviderRegistry::Get();
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		Hierarchy->ForEach<FRigControlElement>([this, ShapeLibraries, Hierarchy, &ProviderRegistry](FRigControlElement* ControlElement) -> bool
        {
			if (!ControlElement->Settings.SupportsShape())
			{
				return true;
			}

			switch (ControlElement->Settings.ControlType)
			{
				case ERigControlType::Float:
				case ERigControlType::ScaleFloat:
				case ERigControlType::Integer:
				case ERigControlType::Vector2D:
				case ERigControlType::Position:
				case ERigControlType::Scale:
				case ERigControlType::Rotator:
				case ERigControlType::Transform:
				case ERigControlType::TransformNoScale:
				case ERigControlType::EulerTransform:
				{
					if (const FControlRigShapeDefinition* ShapeDef = UControlRigShapeLibrary::GetShapeByName(ControlElement->Settings.ShapeName, ShapeLibraries, ControlRig->ShapeLibraryNameMap))
					{
						UMaterialInterface* BaseMaterial = nullptr;
						if (MaterialOverride && !ColorParameter.IsEmpty())
						{
							BaseMaterial = MaterialOverride;
							ColorParameterName = *ColorParameter;
						}
						else
						{
							if(!ShapeDef->Library.IsValid())
							{
								return true;
							}

							if(ShapeDef->Library->DefaultMaterial.IsValid())
							{
								BaseMaterial = ShapeDef->Library->DefaultMaterial.Get();
							}
							else
							{
								BaseMaterial = ShapeDef->Library->DefaultMaterial.LoadSynchronous();
							}
							ColorParameterName = ShapeDef->Library->MaterialColorParameter;
						}

						UPrimitiveComponent* Component = [ShapeDef, Outer = ActorRootComponent, &ProviderRegistry]() -> UPrimitiveComponent*
						{
							if (UObject* Proxy = ShapeDef->GetShapeProxy())
							{
								return ProviderRegistry.GetNewProxyComponent(Proxy, Outer);	
							}
							return nullptr;
						}();
						
						if (Component)
						{
							Component->SetupAttachment(ActorRootComponent);
							Component->RegisterComponent();
							Component->bCastStaticShadow = bCastShadows;
							Component->bCastDynamicShadow = bCastShadows;

							UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(BaseMaterial, Component);
							for (int32 Index = 0; Index < Component->GetNumMaterials(); ++Index)
							{
								Component->SetMaterial(Index, MaterialInstance);
							}

							ControlNames.Add(ControlElement->GetFName());
							ShapeTransforms.Add(ShapeDef->Transform * Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal));
							Components.Add(Component);
							Materials.Add(MaterialInstance);
						}
					}
				}
				default:
				{
					break;
				}
			}
			return true;
		});
	}

	for (int32 GizmoIndex = 0; GizmoIndex < ControlNames.Num(); GizmoIndex++)
	{
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

		USceneComponent* Component = Components[GizmoIndex];
		const FRigElementKey ControlKey(ControlNames[GizmoIndex], ERigElementType::Control);
		const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(ControlKey);
		const bool bVisible = ControlElement && ControlElement->Settings.IsVisible();

		if (Component && Component->IsVisible() != bVisible)
		{
			Component->SetVisibility(bVisible);
		}
		if (Component == nullptr || ControlElement == nullptr)
		{
			continue;
		}

		FTransform ControlTransform = ControlRig->GetControlGlobalTransform(ControlNames[GizmoIndex]);
		Component->SetRelativeTransform(ShapeTransforms[GizmoIndex] * ControlTransform);
		Materials[GizmoIndex]->SetVectorParameterValue(ColorParameterName, FVector(ControlElement->Settings.ShapeColor));
	}
}


#undef LOCTEXT_NAMESPACE

