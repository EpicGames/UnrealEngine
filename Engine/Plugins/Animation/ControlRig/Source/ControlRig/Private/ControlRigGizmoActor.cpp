// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigGizmoActor.h"
#include "ControlRig.h"
#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Engine/CollisionProfile.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigGizmoActor)

namespace UE::ControlRigShape::Private
{
	const TCHAR* SupportAlphaName = TEXT("ControlRig.SupportAlpha");

	static bool bSupportAlpha = false;
	static FAutoConsoleVariableRef CVarSupportAlpha(
	   SupportAlphaName,
	   bSupportAlpha,
	   TEXT("Support for the alpha channel of the control colors.")
	   );
}

AControlRigShapeActor::AControlRigShapeActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ControlRigIndex(INDEX_NONE)
	, ControlRig(nullptr)
	, ControlName(NAME_None)
	, ShapeName(NAME_None)
	, OverrideColor(0, 0, 0, 0)
	, OffsetTransform(FTransform::Identity)
	, bSelected(false)
	, bHovered(false)
	, bSelectable(true)
	, CurrentColor(FLinearColor::Red)
{
	ActorRootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent0"));
	RootComponent = ActorRootComponent;
}

EProxyUpdateType AControlRigShapeActor::EnsureProxyComponent(UObject* InProxy)
{
	const FShapeProxyComponentProviderRegistry& ProviderRegistry = FShapeProxyComponentProviderRegistry::Get();
	if (InProxy && ProviderRegistry.IsShapeProxySupported(InProxy->GetClass()))
	{
		const UClass* ExpectedComponentClass = ProviderRegistry.GetProxyComponentClass(InProxy->GetClass());
		if (!ProxyComponent)
		{
			return AllocateProxyComponent(InProxy);
		}
		
		if (ProxyComponent->GetClass() != ExpectedComponentClass)
		{
			RemoveProxyComponent();
			return AllocateProxyComponent(InProxy);
		}
		
		return UpdateProxyComponent(InProxy);
	}
	
	return RemoveProxyComponent();
}

EProxyUpdateType AControlRigShapeActor::RemoveProxyComponent()
{
	if (ProxyComponent.Get())
	{
		ProxyComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
		ProxyComponent->UnregisterComponent();
		ProxyComponent->SetVisibility(false);		
		ProxyComponent->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		ProxyComponent->MarkAsGarbage();
		ProxyComponent = nullptr;
		return EProxyUpdateType::Removed;
	}
	return EProxyUpdateType::None;
}

EProxyUpdateType AControlRigShapeActor::AllocateProxyComponent(UObject* InShapeProxy)
{
	ensure(!ProxyComponent.Get());
	
	const FShapeProxyComponentProviderRegistry& ProviderRegistry = FShapeProxyComponentProviderRegistry::Get();
	ProxyComponent = ProviderRegistry.GetNewProxyComponent(InShapeProxy, this);
	
	if (ensure(IsValid(ProxyComponent)))
	{
		ProxyComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		ProxyComponent->Mobility = EComponentMobility::Movable;
		ProxyComponent->SetGenerateOverlapEvents(false);
#if WITH_EDITORONLY_DATA
		ProxyComponent->HitProxyPriority = HPP_Wireframe;
#endif

#if WITH_EDITOR
		ProxyComponent->bAlwaysAllowTranslucentSelect = true;
#endif
		ProxyComponent->SetupAttachment(RootComponent);
		ProxyComponent->bCastStaticShadow = false;
		ProxyComponent->bCastDynamicShadow = false;
		ProxyComponent->bSelectable = true;
		ProxyComponent->RegisterComponent();
		return EProxyUpdateType::Created;
	}
	return EProxyUpdateType::None;
}

EProxyUpdateType AControlRigShapeActor::UpdateProxyComponent(UObject* InShapeProxy) const
{
	const FShapeProxyComponentProviderRegistry& ProviderRegistry = FShapeProxyComponentProviderRegistry::Get();
	if (InShapeProxy && ProviderRegistry.IsShapeProxySupported(InShapeProxy->GetClass()))
	{
		ProviderRegistry.UpdateProxyComponent(InShapeProxy, ProxyComponent.Get());
		return EProxyUpdateType::Modified;
	}
	return EProxyUpdateType::None;
}

UMaterialInstanceDynamic* AControlRigShapeActor::GetProxyMaterial() const
{
	constexpr int32 MatIndex = 0;
	return ProxyComponent ? Cast<UMaterialInstanceDynamic>(ProxyComponent->GetMaterial(MatIndex)) : nullptr;
}

//client should check to see if it's seletable based upon how the selection occurs (viewport or outliner,etc..)
void AControlRigShapeActor::SetSelected(bool bInSelected)
{
	if(bSelected != bInSelected)
	{
		bSelected = bInSelected;
		FEditorScriptExecutionGuard Guard;
		OnSelectionChanged(bSelected);
		
		if (bSelected != bHovered)
		{
			SetHovered(bSelected);
		}
	}
}

bool AControlRigShapeActor::IsSelectedInEditor() const
{
	return bSelected;
}

bool AControlRigShapeActor::IsSelectable() const
{
	return bSelectable;
}

//we no longer set the StaticMeshComponent bSelectable flag since that drives if it can move with a gizmo or not
void AControlRigShapeActor::SetSelectable(bool bInSelectable)
{
	if (bSelectable != bInSelectable)
	{
		bSelectable = bInSelectable;
		if (!bSelectable)
		{
			SetSelected(false);
		}

		FEditorScriptExecutionGuard Guard;
		OnEnabledChanged(bInSelectable);
	}
}

void AControlRigShapeActor::SetHovered(bool bInHovered)
{
	if (bInHovered != bHovered)
	{
		bHovered = bInHovered;
		
		if(UMaterialInstanceDynamic* Material = HoveredParameterName.IsNone() ? nullptr : GetProxyMaterial())
		{
			Material->SetScalarParameterValue(HoveredParameterName, bHovered ? 1.f : 0.f);
		}
		
		FEditorScriptExecutionGuard Guard;
		OnHoveredChanged(bHovered);
	}
}

bool AControlRigShapeActor::IsHovered() const
{
	return bHovered;
}

void AControlRigShapeActor::SetHoveredColor(const FLinearColor& InColor, const bool bForce)
{
	if (UMaterialInstanceDynamic* Material = HoveredColorParameterName.IsNone() ? nullptr : GetProxyMaterial())
	{
		if (bForce || InColor != HoveredColor)
		{
			Material->SetVectorParameterValue(HoveredColorParameterName, InColor);
			HoveredColor = InColor;
		}
	}
}

void AControlRigShapeActor::SetShapeColor(const FLinearColor& InColor, const bool bForce)
{
	if (UMaterialInstanceDynamic* Material = ColorParameterName.IsNone() ? nullptr : GetProxyMaterial())
	{
		if (bForce || InColor != CurrentColor)
		{
			if (UE::ControlRigShape::Private::bSupportAlpha)
			{
				Material->SetVectorParameterValue(ColorParameterName, InColor);
			}
			else
			{
				Material->SetVectorParameterValue(ColorParameterName, FVector(InColor));
			}
			CurrentColor = InColor;
		}
	}
}

bool AControlRigShapeActor::UpdateControlSettings(
	ERigHierarchyNotification InNotif,
	UControlRig* InControlRig,
	const FRigControlElement* InControlElement,
	bool bHideManipulators,
	bool bIsInLevelEditor)
{
	check(InControlElement);

	const FRigControlSettings& ControlSettings = InControlElement->Settings;
	
	// if this actor is not supposed to exist
	if(!ControlSettings.SupportsShape())
	{
		return false;
	}

	const bool bShapeNameUpdated = ShapeName != ControlSettings.ShapeName;
	bool bShapeTransformChanged = InNotif == ERigHierarchyNotification::ControlShapeTransformChanged;
	const bool bLookupShape = bShapeNameUpdated || bShapeTransformChanged;
	
	FTransform MeshTransform = FTransform::Identity;

	// update the shape used for the control
	if(bLookupShape)
	{
		const TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries = InControlRig->GetShapeLibraries();
		if (const FControlRigShapeDefinition* ShapeDef = UControlRigShapeLibrary::GetShapeByName(ControlSettings.ShapeName, ShapeLibraries, InControlRig->ShapeLibraryNameMap))
		{
			MeshTransform = ShapeDef->Transform;

			if(bShapeNameUpdated)
			{
				if (UObject* ShapeProxy = ShapeDef->GetShapeProxy())
				{
					EProxyUpdateType UpdateType = EProxyUpdateType::None;
					UPrimitiveComponent* NewProxyComponent = GetProxyComponent(ShapeProxy, &UpdateType);
					if (!NewProxyComponent || UpdateType == EProxyUpdateType::Removed)
					{
						return false;
					}
					
					if (UpdateType == EProxyUpdateType::Created || UpdateType == EProxyUpdateType::Modified)
					{
						bShapeTransformChanged = true;
						ShapeName = ControlSettings.ShapeName;	
					}
					
					if (UpdateType == EProxyUpdateType::Created)
					{
						if (ShapeDef->Library.Get())
						{
							(void)ShapeDef->Library->DefaultMaterial.LoadSynchronous();
						}
					
						if (UMaterial* Material = ShapeDef->Library->DefaultMaterial.Get())
						{
							UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, this);
							MaterialInstance->SetVectorParameterValue(ShapeDef->Library->MaterialColorParameter, FVector(ControlSettings.ShapeColor));
							for (int32 Index = 0; Index < NewProxyComponent->GetNumMaterials(); ++Index)
							{
								NewProxyComponent->SetMaterial(Index, MaterialInstance);
							}
						}
						
						if (ShapeDef->PostSetupFunction)
						{
							ShapeDef->PostSetupFunction(this);
						}
					}
				}
				else
				{
					return false;
				}
			}
		}
	}

	// update the shape transform
	if(bShapeTransformChanged && ProxyComponent.Get())
	{
		const FTransform ShapeTransform = InControlElement->GetShapeTransform().Get(ERigTransformType::CurrentLocal);
		ProxyComponent->SetRelativeTransform(MeshTransform * ShapeTransform);
	}
	
	// update the shape color
	SetShapeColor(ControlSettings.ShapeColor);

	return true;
}

// FControlRigShapeHelper START

namespace FControlRigShapeHelper
{
	const FActorSpawnParameters& GetDefaultSpawnParameter()
	{
		static FActorSpawnParameters ActorSpawnParameters;
#if WITH_EDITOR
		ActorSpawnParameters.bTemporaryEditorActor = true;
		ActorSpawnParameters.bHideFromSceneOutliner = true;
#endif
		ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		ActorSpawnParameters.ObjectFlags = RF_Transient;
		return ActorSpawnParameters;
	}

	// create shape from custom staticmesh, may deprecate this unless we come up with better usage
	AControlRigShapeActor* CreateShapeActor(UWorld* InWorld, UStaticMesh* InStaticMesh, const FControlShapeActorCreationParam& CreationParam)
	{
		if (InWorld)
		{
			AControlRigShapeActor* ShapeActor = CreateDefaultShapeActor(InWorld, CreationParam);
			if (ShapeActor)
			{
				if (InStaticMesh)
				{
					ensure(ShapeActor->GetProxyComponent<UStaticMeshComponent>(InStaticMesh));
				}

				return ShapeActor;
			}
		}

		return nullptr;
	}

	AControlRigShapeActor* CreateShapeActor(UWorld* InWorld, TSubclassOf<AControlRigShapeActor> InClass, const FControlShapeActorCreationParam& CreationParam)
	{
		AControlRigShapeActor* ShapeActor = InWorld->SpawnActor<AControlRigShapeActor>(InClass, GetDefaultSpawnParameter());
		if (ShapeActor)
		{
			// set transform
			ShapeActor->SetActorTransform(CreationParam.SpawnTransform);
			return ShapeActor;
		}

		return nullptr;
	}

	AControlRigShapeActor* CreateDefaultShapeActor(UWorld* InWorld, const FControlShapeActorCreationParam& CreationParam)
	{
		AControlRigShapeActor* ShapeActor = InWorld->SpawnActor<AControlRigShapeActor>(AControlRigShapeActor::StaticClass(), CreationParam.SpawnTransform, GetDefaultSpawnParameter());
		if (ShapeActor)
		{
			ShapeActor->ControlRigIndex = CreationParam.ControlRigIndex;
			ShapeActor->ControlRig = CreationParam.ControlRig;
			ShapeActor->ControlName = CreationParam.ControlName;
			ShapeActor->ShapeName = CreationParam.ShapeName;
			ShapeActor->SetSelectable(CreationParam.bSelectable);

#if WITH_EDITOR
			ShapeActor->SetActorLabel(CreationParam.ControlName.ToString(), false);
#endif // WITH_EDITOR

			UObject* ProxyObject = [&CreationParam]()
			{
				if (!CreationParam.Proxy.IsValid())
				{
					(void)CreationParam.Proxy.LoadSynchronous();
				}
				return CreationParam.Proxy.Get();
			}();
			
			if (ProxyObject)
			{
				UPrimitiveComponent* Component = ShapeActor->GetProxyComponent(ProxyObject);
				if (ensure(Component))
				{
					Component->DestroyPhysicsState();
					Component->SetRelativeTransform(CreationParam.MeshTransform * CreationParam.ShapeTransform);
					
					// update materials
					UMaterial* Material = [&CreationParam]()
					{
						if (!CreationParam.Material.IsValid())
						{
							(void)CreationParam.Material.LoadSynchronous();
						}
						return CreationParam.Material.Get();
					}();
					
					if (Material)
					{
						UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(Material, ShapeActor);
						
						ShapeActor->ColorParameterName = CreationParam.ColorParameterName;
						MaterialInstance->SetVectorParameterValue(CreationParam.ColorParameterName, FVector(CreationParam.Color));
						
						if (!CreationParam.HoveredParameterName.IsNone())
						{
							ShapeActor->HoveredParameterName = CreationParam.HoveredParameterName;
							MaterialInstance->SetScalarParameterValue(ShapeActor->HoveredParameterName, 0.f);
						}
						
						if (!CreationParam.HoveredColorParameterName.IsNone())
						{
							ShapeActor->HoveredColorParameterName = CreationParam.HoveredColorParameterName;
							MaterialInstance->SetVectorParameterValue(ShapeActor->HoveredColorParameterName, CreationParam.HoveredColor);
						}
		
						for (int32 Index = 0; Index < Component->GetNumMaterials(); ++Index)
						{
							Component->SetMaterial(Index, MaterialInstance);
						}
					}
				}
			}
			return ShapeActor;
		}

		return nullptr;
	}
}

void AControlRigShapeActor::SetGlobalTransform(const FTransform& InTransform)
{
	if (RootComponent)
	{
		RootComponent->SetRelativeTransform(InTransform, false, nullptr, ETeleportType::TeleportPhysics);
	}
}

FTransform AControlRigShapeActor::GetGlobalTransform() const
{
	if (RootComponent)
	{
		return RootComponent->GetRelativeTransform();
	}

	return FTransform::Identity;
}

// FControlRigShapeHelper END

