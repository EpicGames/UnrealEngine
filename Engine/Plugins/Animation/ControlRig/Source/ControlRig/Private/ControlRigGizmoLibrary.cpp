// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigGizmoLibrary.h"

#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigGizmoLibrary)

#define LOCTEXT_NAMESPACE "ControlRigGizmoLibrary"

UControlRigShapeLibrary::UControlRigShapeLibrary()
{
#if WITH_EDITOR
	XRayMaterial = LoadObject<UMaterial>(nullptr, TEXT("/ControlRig/Controls/ControlRigXRayMaterial.ControlRigXRayMaterial"));
#endif
}

void UControlRigShapeLibrary::PostLoad()
{
	Super::PostLoad();
	
	if (!DefaultShape.StaticMesh_DEPRECATED.IsNull())
	{
		DefaultShape.ShapeProxy = DefaultShape.StaticMesh_DEPRECATED;
		DefaultShape.StaticMesh_DEPRECATED.Reset();		
	}
	
	for (FControlRigShapeDefinition& Shape : Shapes)
	{
		if (!Shape.StaticMesh_DEPRECATED.IsNull())
		{
			Shape.ShapeProxy = Shape.StaticMesh_DEPRECATED;
			Shape.StaticMesh_DEPRECATED.Reset();
		}
	}
}

#if WITH_EDITOR

// UObject interface
void UControlRigShapeLibrary::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property->GetName() == TEXT("ShapeName"))
	{
		FProperty* MemberProperty = PropertyChangedEvent.PropertyChain.GetHead()->GetValue();
		if (MemberProperty->GetName() == TEXT("DefaultShape"))
		{
			DefaultShape.ShapeName = FControlRigShapeDefinition().ShapeName;
			GetUpdatedNameList(true);
		}
		else if(MemberProperty->GetName() == TEXT("Shapes"))
		{
			if (Shapes.Num() == 0)
			{
				return;
			}

			int32 ShapeIndexEdited = PropertyChangedEvent.GetArrayIndex(TEXT("Shapes"));
			if (Shapes.IsValidIndex(ShapeIndexEdited))
			{
				TArray<FName> Names;
				Names.Add(DefaultShape.ShapeName);
				for (int32 ShapeIndex = 0; ShapeIndex < Shapes.Num(); ShapeIndex++)
				{
					if (ShapeIndex != ShapeIndexEdited)
					{
						Names.Add(Shapes[ShapeIndex].ShapeName);
					}
				}

				FName DesiredName = Shapes[ShapeIndexEdited].ShapeName;
				FString Name = DesiredName.ToString();
				int32 Suffix = 0;
				while (Names.Contains(*Name))
				{
					Suffix++;
					Name = FString::Printf(TEXT("%s_%d"), *DesiredName.ToString(), Suffix);
				}
				Shapes[ShapeIndexEdited].ShapeName = *Name;
			}
			GetUpdatedNameList(true);
		}
	}
	else if (PropertyChangedEvent.Property->GetName() == TEXT("Shapes"))
	{
		TArray<FName> Names;
		Names.Add(DefaultShape.ShapeName);
		for (int32 ShapeIndex = 0; ShapeIndex < Shapes.Num(); ShapeIndex++)
		{
			FName DesiredName = Shapes[ShapeIndex].ShapeName;
			FString Name = DesiredName.ToString();
			int32 Suffix = 0;
			while (Names.Contains(*Name))
			{
				Suffix++;
				Name = FString::Printf(TEXT("%s_%d"), *DesiredName.ToString(), Suffix);
			}
			Shapes[ShapeIndex].ShapeName = *Name;

			Names.Add(Shapes[ShapeIndex].ShapeName);
		}
		GetUpdatedNameList(true);
	}
}

#endif

const FControlRigShapeDefinition* UControlRigShapeLibrary::GetShapeByName(const FName& InName, bool bUseDefaultIfNotFound) const
{
	UControlRigShapeLibrary* MutableThis = (UControlRigShapeLibrary*)this;
	if (InName == MutableThis->DefaultShape.ShapeName)
	{
		MutableThis->DefaultShape.Library = MutableThis;
		return &MutableThis->DefaultShape;
	}

	for (int32 ShapeIndex = 0; ShapeIndex < MutableThis->Shapes.Num(); ShapeIndex++)
	{
		if (MutableThis->Shapes[ShapeIndex].ShapeName == InName)
		{
			MutableThis->Shapes[ShapeIndex].Library = MutableThis;
			return &MutableThis->Shapes[ShapeIndex];
		}
	}

	if (bUseDefaultIfNotFound)
	{
		MutableThis->DefaultShape.Library = MutableThis;
		return &DefaultShape;
	}

	return nullptr;
}

const FControlRigShapeDefinition* UControlRigShapeLibrary::GetShapeByName(const FName& InName, const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& InShapeLibraries, const TMap<FString, FString>& InLibraryNameMap, bool bUseDefaultIfNotFound)
{
	const FString InString = InName.ToString();
	FString Left, Right;
	FName RightName;
	if(!InString.Split(TEXT("."), &Left, &Right))
	{
		Left = FString();
		Right = InString;
		RightName = InName;
	}
	else
	{
		RightName = *Right;
	}

	// we'll do this in two passes - first with the whole namespace
	// and then without - to allow to find a shape also if it has been specified
	// without a namespace all-together.
	for(int32 Pass = 0; Pass < 2; Pass++)
	{
		// we need to walk backwards since shape libraries added later need to take precedence
		for(int32 Index = InShapeLibraries.Num() - 1; Index >= 0; Index--)
		{
			const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary = InShapeLibraries[Index];
			if(!ShapeLibrary.IsValid())
			{
				continue;
			}

			FString ShapeLibraryName = ShapeLibrary->GetName();
			if(const FString* RemappedName = InLibraryNameMap.Find(ShapeLibraryName))
			{
				ShapeLibraryName = *RemappedName;
			}

			if(ShapeLibraryName.Equals(Left) || Left.IsEmpty())
			{
				// only fall back on the default shape for the very last shape library
				const bool bFallBackToDefaultShape = (Pass > 0) && (Index == 0) && bUseDefaultIfNotFound;
				if(const FControlRigShapeDefinition* Shape = ShapeLibrary->GetShapeByName(RightName, bFallBackToDefaultShape))
				{
					return Shape;
				}
			}
		}

		// remove the namespace for the next pass
		Left.Reset();
	}

	return nullptr;
}

const FString UControlRigShapeLibrary::GetShapeName(const UControlRigShapeLibrary* InShapeLibrary, bool bUseNameSpace, const TMap<FString, FString>& InLibraryNameMap, const FControlRigShapeDefinition& InShape)
{
	FString LibraryName = InShapeLibrary->GetName();
	if(const FString* RemappedName = InLibraryNameMap.Find(LibraryName))
	{
		LibraryName = *RemappedName;
	}
		
	const FString NameSpace = bUseNameSpace ? LibraryName + TEXT(".") : FString();
	return NameSpace + InShape.ShapeName.ToString();
}

const TArray<FName> UControlRigShapeLibrary::GetUpdatedNameList(bool bReset)
{
	if (bReset)
	{
		NameList.Reset();
	}

	if (NameList.Num() != Shapes.Num())
	{
		NameList.Reset();
		for (const FControlRigShapeDefinition& Shape : Shapes)
		{
			NameList.Add(Shape.ShapeName);
		}
		NameList.Sort(FNameLexicalLess());
	}

	return NameList;
}

UObject* FControlRigShapeDefinition::GetShapeProxyObject(const bool bLoadPending) const
{
	if (bLoadPending)
	{
		(void)ShapeProxy.LoadSynchronous();
	}
	return ShapeProxy.Get();	
}

FShapeProxyComponentProviderRegistry& FShapeProxyComponentProviderRegistry::Get()
{
	static FShapeProxyComponentProviderRegistry Singleton;
	return Singleton;
}

void FShapeProxyComponentProviderRegistry::RegisterDefaultProviders()
{
	FShapeProxyComponentProviderRegistry& Registry = Get();
	
	// static mesh
	Registry.RegisterProvider<UStaticMesh, UStaticMeshComponent>(
		[](UObject* Proxy, UObject* Outer) -> UPrimitiveComponent*
		{
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Proxy))
			{
				const FName UniqueName = MakeUniqueObjectName(Outer, UStaticMeshComponent::StaticClass(), TEXT("StaticMeshComponent_CtrlProxy"));
				UStaticMeshComponent* StaticMeshComponent = NewObject<UStaticMeshComponent>(Outer, UniqueName);
				StaticMeshComponent->bUseDefaultCollision = false;
				StaticMeshComponent->SetStaticMesh(StaticMesh);
				return StaticMeshComponent;
			}
			return nullptr;
		},
		[](UObject* Proxy, UObject* Component)
		{
			UStaticMesh* StaticMesh = Cast<UStaticMesh>(Proxy);
			UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
			if (StaticMesh && StaticMeshComponent && StaticMeshComponent->GetStaticMesh() != StaticMesh)
			{
				StaticMeshComponent->SetStaticMesh(StaticMesh);
				return true;
			}
			return false;
		});
	
	// skeletal mesh
	Registry.RegisterProvider<USkeletalMesh, USkeletalMeshComponent>(
		[](UObject* Proxy, UObject* Outer) -> UPrimitiveComponent*
		{
			if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Proxy))
			{
				const FName UniqueName = MakeUniqueObjectName(Outer, USkeletalMeshComponent::StaticClass(), TEXT("SkeletalMeshComponent_CtrlProxy"));
				USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(Outer, UniqueName);
				SkeletalMeshComponent->SetSkeletalMesh(SkeletalMesh);
				return SkeletalMeshComponent;
			}
			return nullptr; 
		},
		[](UObject* Proxy, UObject* Component)
		{
			USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Proxy);
			USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component);
			if (SkeletalMesh && SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset() != SkeletalMesh)
			{
				SkeletalMeshComponent->SetSkeletalMesh(SkeletalMesh);
				return true;
			}
			return false;
		});
}

bool FShapeProxyComponentProviderRegistry::IsShapeProxySupported(const UClass* InProxyClass) const
{
	if (!InProxyClass)
	{
		LogNull();
		return false;
	}
	
	const bool bSupported = ShapeProxyToComponent.Contains(InProxyClass) && ProxyComponentFunctions.Contains(InProxyClass);
	if (!bSupported)
	{
		LogOnce(InProxyClass);
		return false;
	}
	
	return true;
}

const UClass* FShapeProxyComponentProviderRegistry::GetProxyComponentClass(const UClass* InProxyClass) const
{
	if (!InProxyClass)
	{
		LogNull();
		return nullptr;
	}
	
	UClass* const* ComponentClass =ShapeProxyToComponent.Find(InProxyClass);
	if (!ComponentClass)
	{
		LogOnce(InProxyClass);
		return nullptr;
	}
	
	return *ComponentClass;
}

UPrimitiveComponent* FShapeProxyComponentProviderRegistry::GetNewProxyComponent(UObject* InProxy, UObject* InOuter) const
{
	if (!InProxy)
	{
		LogNull();
		return nullptr;
	}
	
	const FProxyFunctions* ProxyFunctions = ProxyComponentFunctions.Find(InProxy->GetClass());
	if (!(ProxyFunctions && ProxyFunctions->CreateFunction))
	{
		LogOnce(InProxy->GetClass());
		return nullptr;
	}
	
	return ProxyFunctions->CreateFunction(InProxy, InOuter);
}

bool FShapeProxyComponentProviderRegistry::UpdateProxyComponent(UObject* InProxy, UObject* InComponent) const
{
	if (!InProxy || !InComponent)
	{
		LogNull();
		return false;
	}
	
	const FProxyFunctions* ProxyFunctions = ProxyComponentFunctions.Find(InProxy->GetClass());
	if (!(ProxyFunctions && ProxyFunctions->UpdateFunction))
	{
		LogOnce(InProxy->GetClass());
		return false;
	}
	return ProxyFunctions->UpdateFunction(InProxy, InComponent);
}

void FShapeProxyComponentProviderRegistry::LogOnce(const UClass* InClass) const
{
	if (!InClass)
	{
		return LogNull();
	}
	
	if (!QueriedUnregisteredClass.Contains(InClass))
	{
		UE_LOGF(LogTemp, Warning, "No shape proxy provider found for class %ls. Did you call RegisterProvider for that class?", *InClass->GetName());
		QueriedUnregisteredClass.Add(InClass);
	}
}

void FShapeProxyComponentProviderRegistry::LogNull()
{
	UE_LOGF(LogTemp, Warning, "Null proxy provided.");
}

#undef LOCTEXT_NAMESPACE

