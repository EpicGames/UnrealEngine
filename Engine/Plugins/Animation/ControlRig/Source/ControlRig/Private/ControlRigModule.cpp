// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigModule.h"

#include "ControlRigGizmoActor.h"
#include "Transform/TransformableRegistry.h"
#include "Modules/ModuleManager.h"
#include "ILevelSequenceModule.h"
#include "ControlRigObjectVersion.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "UObject/DevObjectVersion.h"
#include "ControlRig.h"

// Unique Control Rig Object version id
const FGuid FControlRigObjectVersion::GUID(0xA7820CFB, 0x20A74359, 0x8C542C14, 0x9623CF50);
// Register ControlRig custom version with Core
FDevVersionRegistration GRegisterControlRigObjectVersion(FControlRigObjectVersion::GUID, FControlRigObjectVersion::LatestVersion, TEXT("Dev-ControlRig"));

void FControlRigModule::StartupModule()
{
	ManipulatorMaterial = LoadObject<UMaterial>(nullptr, TEXT("/ControlRig/M_Manip.M_Manip"));

	RegisterTransformableCustomization();
	
	FShapeProxyComponentProviderRegistry& ProviderRegistry = FShapeProxyComponentProviderRegistry::Get();
	ProviderRegistry.RegisterDefaultProviders();
}

void FControlRigModule::ShutdownModule()
{
}

void FControlRigModule::RegisterTransformableCustomization() const
{
	// load the module to trigger default objects registration
	static const FName ConstraintsModuleName(TEXT("Constraints"));
	FModuleManager::Get().LoadModule(ConstraintsModuleName);
	
	// register UControlRig and AControlRigShapeActor
	auto CreateControlHandle = [](UObject* InObject, const FName& InControlName)->UTransformableHandle*
	{
		if (const UControlRig* ControlRig = Cast<UControlRig>(InObject))
		{
			return ControlRig->CreateTransformableControlHandle(InControlName);
		}
		return nullptr;
	};

	auto GetControlHash = [](const UObject* InObject, const FName& InControlName)->uint32
	{
		if (const UControlRig* ControlRig = Cast<UControlRig>(InObject))
		{
			return UTransformableControlHandle::ComputeHash(ControlRig, InControlName);
		}
		return 0;
	};
	
	auto CreateControlHandleFromActor = [CreateControlHandle](UObject* InObject, const FName&)->UTransformableHandle*
	{
		if (const AControlRigShapeActor* ControlActor = Cast<AControlRigShapeActor>(InObject))
		{
			return CreateControlHandle(ControlActor->ControlRig.Get(), ControlActor->ControlName);
		}
		return nullptr;
	};

	auto GetControlHashFromActor = [GetControlHash](const UObject* InObject, const FName&)->uint32
	{
		if (const AControlRigShapeActor* ControlActor = Cast<AControlRigShapeActor>(InObject))
		{
			return GetControlHash(ControlActor->ControlRig.Get(), ControlActor->ControlName);
		}
		return 0;
	};

	// Register UTransformableControlHandle and has function to handle control thru the transform constraints system
	// as AControlRigShapeActor is only available if the ControlRig plugin is loaded.
	FTransformableRegistry& Registry = FTransformableRegistry::Get();
	Registry.Register(AControlRigShapeActor::StaticClass(), CreateControlHandleFromActor, GetControlHashFromActor);
	Registry.Register(UControlRig::StaticClass(), CreateControlHandle, GetControlHash);
}

IMPLEMENT_MODULE(FControlRigModule, ControlRig)
