// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "IControlRigModule.h"
#include "Materials/Material.h"

class FControlRigModule : public IControlRigModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	UMaterial* ManipulatorMaterial;
	
private:
	FDelegateHandle OnCreateMovieSceneObjectSpawnerHandle;

	void RegisterTransformableCustomization() const;
};
