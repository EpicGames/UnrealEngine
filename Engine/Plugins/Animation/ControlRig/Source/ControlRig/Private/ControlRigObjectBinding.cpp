// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigObjectBinding.h"
#include "GameFramework/Actor.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ControlRigComponent.h"

FControlRigObjectBinding::~FControlRigObjectBinding()
{
}

void FControlRigObjectBinding::BindToObject(UObject* InObject)
{
	BoundObject = GetBindableObject(InObject);
	ControlRigBind.Broadcast(BoundObject.Get());
}

void FControlRigObjectBinding::UnbindFromObject()
{
	BoundObject = nullptr;

	ControlRigUnbind.Broadcast();
}

bool FControlRigObjectBinding::IsBoundToObject(UObject* InObject) const
{
	return InObject != nullptr && BoundObject.Get() == GetBindableObject(InObject);
}

UObject* FControlRigObjectBinding::GetBoundObject() const
{
	return BoundObject.Get();
}

AActor* FControlRigObjectBinding::GetHostingActor() const
{
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject.Get()))
	{
		return SceneComponent->GetOwner();
	}

	return nullptr;
}

UObject* FControlRigObjectBinding::GetBindableObject(UObject* InObject) 
{
	// If we are binding to an actor, find the first skeletal mesh component
	if (AActor* Actor = Cast<AActor>(InObject))
	{
		if (UControlRigComponent* ControlRigComponent = Actor->FindComponentByClass<UControlRigComponent>())
		{
			return ControlRigComponent;
		}
		else if (USkeletalMeshComponent* SkeletalMeshComponent = Actor->FindComponentByClass<USkeletalMeshComponent>())
		{
			return SkeletalMeshComponent;
		}
	}
	else if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(InObject))
	{
		return ControlRigComponent;
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InObject))
	{
		return SkeletalMeshComponent;
	}
	else if (USkeleton* Skeleton = Cast<USkeleton>(InObject))
	{
		return Skeleton;
	}
	else if (UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
	{
		return GetBindableObject(ActorComponent->GetOuter());
	}

	return nullptr;
}
