// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigOverride.h"
#include "ControlRig.h"
#include "RigVMCore/RigVMMemoryCommon.h"
#include "ControlRigObjectVersion.h"

#if WITH_EDITOR
#include "PropertyPath.h"
#include "AssetToolsModule.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/CoreRedirects.h"
#endif
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigOverride)

TAutoConsoleVariable<bool> CVarControlRigEnableOverrides(TEXT("ControlRig.Overrides"), false, TEXT("Enables overrides for use in Control Rig"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
{
	for (TObjectIterator<UControlRigOverrideAsset> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		UControlRigOverrideAsset* Override = *It;
		if(IsValid(Override))
		{
			Override->BroadcastChanged();
		}
	}
}));

////////////////////////////////////////////////////////////////////////////////
// FControlRigOverrideValue
////////////////////////////////////////////////////////////////////////////////

FControlRigOverrideValue::FControlRigOverrideValue(const FString& InPath, const UStruct* InOwnerStruct, const void* InSubjectPtr, const FName& InSubjectKey)
	: SubjectKey(InSubjectKey)
	, Hash(0)
	, DataPropertyHash(0)
{
	ensure(InOwnerStruct);
	ensureMsgf(InPath.Contains(TEXT(".")) == false, TEXT("Paths for overrides should use the -> separator."));

	if(SetPropertiesFromPath(InPath, InOwnerStruct))
	{
		if(InSubjectPtr)
		{
			(void)SetFromSubject(InSubjectPtr);
		}
	}
}

FControlRigOverrideValue::FControlRigOverrideValue(const FString& InPath, const UStruct* InOwnerStruct, const FString& InValueAsString, const FName& InSubjectKey, const FControlRigOverrideValueErrorPipe::TReportFunction& InReportFunction)
	: SubjectKey(InSubjectKey)
	, Hash(0)
	, DataPropertyHash(0)
{
	if(SetPropertiesFromPath(InPath, InOwnerStruct))
	{
		SetFromString(InValueAsString, InReportFunction);
	}
}

FControlRigOverrideValue::FControlRigOverrideValue(const FControlRigOverrideValue& InOther)
{
	*this = InOther;
}

FControlRigOverrideValue::~FControlRigOverrideValue()
{
	Reset();
}

FControlRigOverrideValue& FControlRigOverrideValue::operator=(const FControlRigOverrideValue& InOther)
{
	Reset();
	Path = InOther.Path;
	SubjectKey = InOther.SubjectKey;
	CachedStringValue = InOther.CachedStringValue;
	Properties = InOther.Properties;
	Hash = InOther.Hash;

	if(GetLeafProperty() && InOther.IsDataValid())
	{
		if(const void* OtherDataPtr = InOther.GetData())
		{
			void* DataPtr = AllocateDataIfRequired();
			check(DataPtr);
			CopyValue(DataPtr, OtherDataPtr);
		}
	}

	return *this;
}

bool FControlRigOverrideValue::Serialize(FArchive& Ar)
{
	UStruct* OwnerStruct = nullptr;
	bool bStoresOnlyPathAndLeafProperty = true;
	
	if(Ar.IsLoading())
	{
		Reset();
		bStoresOnlyPathAndLeafProperty = Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::OverridesStorePathAndLeafPropertyOnly;
	}
	else
	{
		if (const FProperty* RootProperty = GetRootProperty())
		{
			OwnerStruct = RootProperty->GetOwnerStruct();
		}
	}

	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	Ar << SubjectKey;

	bool bEncounteredError = false;

	auto ReportError = [&OwnerStruct, &bEncounteredError](const FString& InPath, const FString& InMessage)
	{
		if (OwnerStruct)
		{
			UE_LOGF(LogControlRig, Error, "[%ls] Error when loading override for path '%ls': '%ls'", *OwnerStruct->GetPathName(), *InPath, *InMessage);
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Error when loading override for path '%ls': '%ls'", *InPath, *InMessage);
		}
		bEncounteredError = true;
	};

	if(Ar.IsLoading())
	{
		TOptional<FString> TempPath;

		if (bStoresOnlyPathAndLeafProperty)
		{
			FSoftObjectPath OwnerStructPath;
			Ar << OwnerStructPath;
			OwnerStructPath.FixupCoreRedirects();
			OwnerStruct = Cast<UStruct>(OwnerStructPath.ResolveObject());
			Ar << TempPath;
			if (!OwnerStruct || !UE::IsSavingPackage())
			{
				StaticLoadObject(UObject::StaticClass(), nullptr, *OwnerStructPath.ToString());
				OwnerStruct = Cast<UStruct>(OwnerStructPath.ResolveObject());
			}
			if (!OwnerStruct)
			{
				ReportError(TempPath.Get(TEXT("Cannot resolve owner.")), OwnerStructPath.ToString());
			}
		}
		else
		{
			int32 NumPropertiesToLoad = 0;
			Ar << NumPropertiesToLoad;

			int32 NumValidProperties = 0;
			
			TArray<FPropertyInfo> TempProperties;
			TempProperties.Reserve(NumPropertiesToLoad);

			for (int32 Index = 0; Index < NumPropertiesToLoad; Index++)
			{
				FPropertyInfo Info;
				Ar << Info.Property;
				Ar << Info.ArrayIndex;
				if (Info.Property.IsValid())
				{
					if (Index == 0)
					{
						OwnerStruct = Info.Property.Get()->GetOwnerStruct();
					}
					TempProperties.Add(Info);
					NumValidProperties++;
				}
			}

			TempPath = GetPathFromProperties(TempProperties);

			if (NumValidProperties != NumPropertiesToLoad)
			{
				Reset();
				ReportError(TempPath.Get(TEXT("Unknown path")), TEXT("Unexpected number of properties on path."));
			}

		}

		SerializedPath = TempPath;

		if(!OwnerStruct || !SetPropertiesFromPath(TempPath.Get(FString()), OwnerStruct))
		{
			Reset();
			ReportError(TempPath.Get(TEXT("Unknown path")), TEXT("Path is not valid."));
		}
	}
	else
	{
		FSoftObjectPath OwnerStructPath = OwnerStruct;
		Ar << OwnerStructPath;
		Ar << Path;
	}

	const int64 ArchivePositionBeforeData = Ar.Tell();

	int64 OffsetForData = 0;
	Ar << OffsetForData;
	
	if(Ar.IsLoading())
	{
		// previously the data skip offset was stored as an int32, we need to convert it over
		if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::OverridesStoreDatSkipOffsetAsInt64)
		{
			const int64 ArchivePositionCurrent = Ar.Tell();
			Ar.Seek(ArchivePositionBeforeData);
			int32 OffsetForDataAsInt32 = 0;
			Ar << OffsetForDataAsInt32;
			OffsetForData = OffsetForDataAsInt32;
			Ar.Seek(ArchivePositionCurrent);
		}
		
		ensure(OffsetForData > 0);
		bool bSkipData = true;
		
		if (!bEncounteredError)
		{
			if (const FProperty* LeafProperty = GetLeafProperty())
			{
				uint32 PropertyHash = 0;
				bool bIsUsingPotentiallyInvalidHash = false;

				if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::OverridesStoreLeafPropertyHashOnly)
				{
					Ar << PropertyHash;
				}
				else
				{
					bIsUsingPotentiallyInvalidHash = true;

					if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::OverridesStoreTOCDataForProperties)
					{
						int32 PropertySize = 0;
						Ar << PropertySize;
						Ar << PropertyHash;
					}
				}

				if (bIsUsingPotentiallyInvalidHash)
				{
					// update this to the new way of representing hashes / value addressing 
					DataPropertyHash = PropertyHash = GetPropertyHash(); 
				}

				if (GetPropertyHash() == PropertyHash)
				{
					void* DataPtr = AllocateDataIfRequired();
					LeafProperty->SerializeItem(FStructuredArchiveFromArchive(Ar).GetSlot(), DataPtr);

					FString ValueAsString;
					LeafProperty->ExportTextItem_Direct(ValueAsString, DataPtr, DataPtr, nullptr, PPF_None, nullptr);
					CachedStringValue = ValueAsString;

					bSkipData = false;
				}
				else
				{
					DataPropertyHash = 0;
					DataArray.Reset();
					const FString Error = FString::Printf(TEXT("The property type doesn't match. GetPropertyHash(%d), serialized PropertyHash(%d)"), GetPropertyHash(), PropertyHash);
					ReportError(Path.Get(TEXT("Unknown path")), Error);
				}
			}
		}

		if (bSkipData)
		{
			// we can't deserialize the data - skip it
			Ar.Seek(ArchivePositionBeforeData + OffsetForData);
		}

		UpdateHash();
	}
	else
	{
		if(const FProperty* LeafProperty = GetLeafProperty())
		{
			uint32 PropertyHash = DataPropertyHash;
			Ar << PropertyHash;
			
			void *DataPtr = AllocateDataIfRequired();
			LeafProperty->SerializeItem(FStructuredArchiveFromArchive(Ar).GetSlot(), DataPtr, nullptr);
		}
		
		const int64 ArchivePositionAfterData = Ar.Tell();
		int64 DataToSkip = ArchivePositionAfterData - ArchivePositionBeforeData;
		Ar.Seek(ArchivePositionBeforeData);
		Ar << DataToSkip;
		Ar.Seek(ArchivePositionAfterData);
	}

	return true;
}

bool FControlRigOverrideValue::IsValid() const
{
	const bool bIsSet = Path.IsSet() && CachedStringValue.IsSet() && (GetData() != nullptr);
	if (!bIsSet || !IsDataValid())
	{
		return false;
	}
	for (const FPropertyInfo& PropertyInfo : Properties)
	{
		if (!PropertyInfo.Property.IsValid())
		{
			return false;
		}
	}
	return true;
}

bool FControlRigOverrideValue::IsDataValid() const
{
	if(GetLeafProperty())
	{
		return IsDataValid(DataArray, DataPropertyHash);
	}
	return false;
}

bool FControlRigOverrideValue::IsDataValid(const TArray<uint8, TAlignedHeapAllocator<16>>& InDataArray, const uint32& InOutPropertyHash) const
{
	if (const FProperty* LeafProperty = GetLeafProperty())
	{
		if (InDataArray.Num() == LeafProperty->GetSize())
		{
			if (GetPropertyHash() == InOutPropertyHash)
			{
				return true;
			}
		}
	}
	return false;
}

uint32 FControlRigOverrideValue::GetPropertyHash() const
{
	uint32 PropertyHash = GetTypeHash(GetSerializedPath());
	if (const FProperty* LeafProperty = GetLeafProperty())
	{
		PropertyHash = HashCombine(PropertyHash, GetPropertyHash(LeafProperty));
	}
	return PropertyHash;
}

#define UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(PropertyType) \
if (InProperty->IsA<PropertyType>()) \
{ \
	static const FString PropertyString = TEXT(#PropertyType); \
	static const uint32 PropertyHash = GetTypeHash(PropertyString); \
	return PropertyHash; \
}

uint32 FControlRigOverrideValue::GetPropertyHash(const FProperty* InProperty) const
{
	if (InProperty == nullptr)
	{
		return 0;
	}
	
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FBoolProperty);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FUInt16Property);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FUInt32Property);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FUInt64Property);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FInt8Property);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FInt16Property);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FIntProperty);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FInt64Property);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FFloatProperty);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FDoubleProperty);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FNameProperty);
	UE_CONTROLRIGOVERRIDE_PROPERTYHASHCASE(FStrProperty);
	
	if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		static const FString PropertyString = TEXT("ByteProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		if (const UEnum* Enum = ByteProperty->Enum)
		{
			return HashCombine(PropertyHash, GetTypeHash(Enum->GetName()));
		}
		return PropertyHash;
	}
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		static const FString PropertyString = TEXT("EnumProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		if (const UEnum* Enum = EnumProperty->GetEnum())
		{
			return HashCombine(PropertyHash, GetTypeHash(Enum->GetName()));
		}
		return PropertyHash;
	}
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		static const FString PropertyString = TEXT("ArrayProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		return HashCombine(PropertyHash, GetPropertyHash(ArrayProperty->Inner));
	}
	if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
	{
		static const FString PropertyString = TEXT("SetProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		return HashCombine(PropertyHash, GetPropertyHash(SetProperty->ElementProp));
	}
	if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
	{
		static const FString PropertyString = TEXT("MapProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		return HashCombine(PropertyHash,
			GetPropertyHash(MapProperty->KeyProp),
			GetPropertyHash(MapProperty->ValueProp));
	}
	if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty))
	{
		static const FString PropertyString = TEXT("ObjectProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		if (const UClass* Class = ObjectProperty->PropertyClass)
		{
			return HashCombine(PropertyHash, GetTypeHash(Class->GetName()));
		}
		return PropertyHash;
	}
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		static const FString PropertyString = TEXT("StructProperty");
		static const uint32 PropertyHash = GetTypeHash(PropertyString);
		if (const UStruct* Struct = StructProperty->Struct)
		{
			return HashCombine(PropertyHash, GetTypeHash(Struct->GetName()));
		}
		return PropertyHash;
	}

	return 0;
}

void FControlRigOverrideValue::Reset()
{
	FreeDataIfRequired();
	CachedStringValue.Reset();
	Properties.Reset();
	Hash = 0;
	SerializedPath.Reset();
}

const FName& FControlRigOverrideValue::GetSubjectKey() const
{
	static const FName InvalidName = FName(NAME_None);
	return SubjectKey.Get(InvalidName);
}

const FString& FControlRigOverrideValue::GetPath() const
{
	static const FString EmptyPath;
	return Path.Get(EmptyPath);
}

const FString& FControlRigOverrideValue::GetSerializedPath() const
{
	if (SerializedPath.IsSet())
	{
		return SerializedPath.GetValue();
	}
	return GetPath(); 
}

uint8* FControlRigOverrideValue::SubjectPtrToValuePtr(const void* InSubjectPtr, bool bResizeArrays) const
{
	if(Properties.IsEmpty())
	{
		return nullptr;
	}

	if(InSubjectPtr == nullptr)
	{
		return nullptr;
	}

	void* ValuePtr = const_cast<void*>(InSubjectPtr);
	
	for (const FPropertyInfo& Info : Properties)
	{
		const FProperty* Property = Info.Property.Get();
		if (!Property)
		{
			return nullptr;
		}

		const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		if(ArrayProperty == nullptr)
		{
			ArrayProperty = CastField<FArrayProperty>(Property->GetOwnerProperty());
		}

		if(Info.ArrayIndex != INDEX_NONE)
		{
			check(ArrayProperty);
			
			ValuePtr = ArrayProperty->ContainerPtrToValuePtr<void*>(ValuePtr);

			FScriptArrayHelper ArrayHelper(ArrayProperty, ValuePtr);
			if(ArrayHelper.Num() <= Info.ArrayIndex)
			{
				if(bResizeArrays)
				{
					ArrayHelper.Resize(Info.ArrayIndex + 1);
				}
				else
				{
					return nullptr;
				}
			}
			
			ValuePtr = ArrayHelper.GetRawPtr(Info.ArrayIndex);
			continue;
		}
		
		if(ArrayProperty)
		{
			Property = ArrayProperty;
		}

		ValuePtr = Property->ContainerPtrToValuePtr<void*>(ValuePtr);
	}

	return static_cast<uint8*>(ValuePtr);
}

const FProperty* FControlRigOverrideValue::GetRootProperty() const
{
	if(Properties.IsEmpty())
	{
		return nullptr;
	}
	return Properties[0].Property.Get();
}

FProperty* FControlRigOverrideValue::FindProperty(const UStruct* InStruct, const FString& InNameOrDisplayName)
{
	bool bWasRedirected = false;
	return FindProperty(InStruct, InNameOrDisplayName, bWasRedirected);
}

FProperty* FControlRigOverrideValue::FindProperty(const UStruct* InStruct, const FString& InNameOrDisplayName, bool& bWasRedirected)
{
	if(InStruct == nullptr)
	{
		return  nullptr;
	}

	if(InNameOrDisplayName.IsEmpty())
	{
		return nullptr;
	}

	FName PropertyName = *InNameOrDisplayName;
	const FCoreRedirectObjectName OldObjectName(PropertyName, InStruct->GetFName(), *InStruct->GetOutermost()->GetPathName());
	const FCoreRedirectObjectName NewObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Property, OldObjectName);
	if (NewObjectName.IsValid() && OldObjectName != NewObjectName)
	{
		PropertyName = NewObjectName.ObjectName;
		bWasRedirected = true;
	}
	
	if(FProperty* Property = InStruct->FindPropertyByName(PropertyName))
	{
		return Property;
	}

#if WITH_EDITORONLY_DATA
	const FText DisplayName = FText::FromString(InNameOrDisplayName);
	for (TFieldIterator<FProperty> PropertyIt(InStruct); PropertyIt; ++PropertyIt)
	{
		if(FProperty* PropertyOnStruct = *PropertyIt)
		{
			if(PropertyOnStruct->GetDisplayNameText().EqualToCaseIgnored(DisplayName))
			{
				return PropertyOnStruct;
			}
		}
	}
#endif

	return nullptr;	
}

uint32 GetTypeHash(const FControlRigOverrideValue& InOverride)
{
	return InOverride.Hash;
}

void FControlRigOverrideValue::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	if(const FProperty* RootProperty = GetRootProperty())
	{
		if (UStruct* OwnerStruct = RootProperty->GetOwnerStruct())
		{
			TObjectPtr<UObject> OwnerStructObj = OwnerStruct;
			Collector.AddReferencedObject(OwnerStructObj);
		}
	}
	
	if (const FProperty* LeafProperty = GetLeafProperty())
	{
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(LeafProperty))
		{
			if (const FObjectProperty* ObjProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
			{
				if (const void* DataPtr = GetData())
				{
					FScriptArrayHelper ArrayHelper(ArrayProperty, DataPtr);
					for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
					{
						if (const void* ElementData = ArrayHelper.GetRawPtr(Index))
						{
							if (TObjectPtr<UObject> ObjPtr = ObjProperty->GetObjectPtrPropertyValue(ElementData))
							{
								Collector.AddReferencedObject(ObjPtr);
							}
						}
					}
				}
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
			{
				if (const void* DataPtr = GetData())
				{
					FScriptArrayHelper ArrayHelper(ArrayProperty, DataPtr);
					for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
					{
						UScriptStruct* Struct = StructProperty->Struct;
						void* StructData = ArrayHelper.GetRawPtr(Index);
						Collector.AddPropertyReferencesWithStructARO(Struct, StructData);
					}
				}
			}
		}
		else
		{
			if (const FObjectProperty* ObjProperty = CastField<FObjectProperty>(LeafProperty))
			{
				if (const void* DataPtr = GetData())
				{
					if (TObjectPtr<UObject> ObjPtr = ObjProperty->GetObjectPtrPropertyValue(DataPtr))
					{
						Collector.AddReferencedObject(ObjPtr);
					}
				}
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(LeafProperty))
			{
				if (const void* DataPtr = GetData())
				{
					Collector.AddPropertyReferencesWithStructARO(StructProperty->Struct, const_cast<void*>(DataPtr));
				}
			}
		}
	}
}

void FControlRigOverrideValue::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	if(const FProperty* RootProperty = GetRootProperty())
	{
		if (UStruct* OwnerStruct = RootProperty->GetOwnerStruct())
		{
			OutDeps.Add(OwnerStruct);
		}
	}
}

const FProperty* FControlRigOverrideValue::GetLeafProperty() const
{
	FProperty* LeafProperty = const_cast<FControlRigOverrideValue*>(this)->GetLeafProperty();
	return const_cast<FProperty*>(LeafProperty);
}

FProperty* FControlRigOverrideValue::GetLeafProperty()
{
	if(Properties.IsEmpty())
	{
		return nullptr;
	}
	
	if(FProperty* LeafProperty = Properties.Last().Property.Get())
	{
		const int32& ArrayIndex = Properties.Last().ArrayIndex;
		if(ArrayIndex != INDEX_NONE)
		{
			if(FArrayProperty* ArrayProperty = CastField<FArrayProperty>(LeafProperty))
			{
				LeafProperty = ArrayProperty->Inner;
			}
		}
		else
		{
			if(FArrayProperty* ArrayProperty = CastField<FArrayProperty>(LeafProperty->GetOwnerProperty()))
			{
				LeafProperty = ArrayProperty;
			}
		}

		return LeafProperty;
	}

	return nullptr;
}

int32 FControlRigOverrideValue::GetNumProperties() const
{
	return Properties.Num();
}

const FProperty* FControlRigOverrideValue::GetProperty(int32 InIndex) const
{
	if(Properties[InIndex].Property.IsValid())
	{
		return Properties[InIndex].Property.Get();
	}
	return nullptr;
}

int32 FControlRigOverrideValue::GetArrayIndex(int32 InIndex) const
{
	return Properties[InIndex].ArrayIndex;
}

bool FControlRigOverrideValue::ContainsProperty(const FProperty* InProperty) const
{
	for(int32 Index = 0; Index < Properties.Num(); Index++)
	{
		if(Properties[Index].Property.IsValid())
		{
			if(Properties[Index].Property.Get() == InProperty)
			{
				return true;
			}
		}
	}
	return false;
}

void* FControlRigOverrideValue::GetData()
{
	if(DataArray.IsEmpty())
	{
		return nullptr;
	}
	return DataArray.GetData();
}

const void* FControlRigOverrideValue::GetData() const
{
	if(DataArray.IsEmpty())
	{
		return nullptr;
	}
	return DataArray.GetData();
}

bool FControlRigOverrideValue::SetPropertiesFromPath(const FString& InPath, const UStruct* InOwnerStruct)
{
	if (InPath.IsEmpty() || InOwnerStruct == nullptr)
	{
		Path.Reset();
		SerializedPath.Reset();
		return false;
	}
	
	Path = InPath;

	FString PropertyNameString;
	FString ArrayIndexString;
	bool bIsParsingPropertyName = true;
	const UStruct* OwnerStruct = InOwnerStruct;
	bool bWasDirected = false;

	auto ProcessPropertyName = [this, &PropertyNameString, &OwnerStruct, &bWasDirected]() -> bool
	{
		FString StringToProcess;
		Swap(StringToProcess, PropertyNameString);
		if(!StringToProcess.IsEmpty())
		{
			if(FProperty* Property = FindProperty(OwnerStruct, StringToProcess, bWasDirected))
			{
				FProperty* PropertyForPath = Property;
				
				if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					Property = ArrayProperty->Inner;

					if(Properties.IsEmpty())
					{
						PropertyForPath = Property;
					}
				}
				if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					OwnerStruct = StructProperty->Struct;
				}

				Properties.Add({PropertyForPath, INDEX_NONE});

				return true;
			}
		}
		return false;
	};
	
	auto ProcessArrayIndex = [this, &ArrayIndexString]() -> bool
	{
		FString StringToProcess;
		Swap(StringToProcess, ArrayIndexString);
		if(!StringToProcess.IsEmpty())
		{
			Properties.Last().ArrayIndex = FCString::Atoi(*StringToProcess);
			return true;
		}
		return false;
	};

	// Parse the provided Property path. Path may look like
	// Settings[2]->Color->R resulting in three segments, the
	// first one with an array index != INDEX_NONE
	int32 CharIndex = 0;
	while(CharIndex < InPath.Len())
	{
		// potentially start parsing an array index
		if(InPath[CharIndex] == TEXT('['))
		{
			if(!ProcessPropertyName())
			{
				Reset();
				return false;
			}
			bIsParsingPropertyName = false;
			CharIndex++;
			continue;;
		}
		
		// potentially end parsing an array index
		if(InPath[CharIndex] == TEXT(']'))
		{
			if(!ProcessArrayIndex())
			{
				Reset();
				return false;
			}
			CharIndex++;
			continue;;
		}

		// given a path separator parse the property name and store it.
		// if bIsParsingPropertyName == false we already processed the property name
		// and have been parsing an array index
		if(InPath.Mid(CharIndex, PathSeparatorLength).Equals(PathSeparator, ESearchCase::CaseSensitive))
		{
			if(bIsParsingPropertyName)
			{
				if(!ProcessPropertyName())
				{
					Reset();
					return false;
				}
			}
			bIsParsingPropertyName = true;
			CharIndex += PathSeparatorLength;
			continue;
		}

		if(bIsParsingPropertyName)
		{
			PropertyNameString += InPath[CharIndex];
		}
		else
		{
			ArrayIndexString += InPath[CharIndex];
		}
		CharIndex++;
	}

	if(!PropertyNameString.IsEmpty() && !ProcessPropertyName())
	{
		Reset();
		return false;
	}
	if(!ArrayIndexString.IsEmpty() && !ProcessArrayIndex())
	{
		Reset();
		return false;
	}

	if (bWasDirected)
	{
		Path = GetPathFromProperties();
	}
	
	return true;
}

FString FControlRigOverrideValue::GetPathFromProperties() const
{
	return GetPathFromProperties(Properties);
}

FString FControlRigOverrideValue::GetPathFromProperties(const TArray<FPropertyInfo>& InProperties)
{
	// rebuild path from properties
	TStringBuilder<256> PathBuilder;
	bool bFirstProperty = true;

	const int32 NumProperties = InProperties.Num();
	for(int32 Index = 0; Index < NumProperties; Index++)
	{
		if(const FProperty* Property = InProperties[Index].Property.Get())
		{
			if(!bFirstProperty)
			{
				PathBuilder.Append(FControlRigOverrideValue::PathSeparator);
			}
			else
			{
				bFirstProperty = false;
			}
			PathBuilder.Append(Property->GetName());
			if(InProperties[Index].ArrayIndex != INDEX_NONE)
			{
				PathBuilder.Appendf(TEXT("[%d]"),  InProperties[Index].ArrayIndex);
			}
		}
		else
		{
			PathBuilder.Reset();
			break;
		}
	}

	if (PathBuilder.Len() > 0)
	{
		return FString(PathBuilder);
	}

	return FString();
}

void* FControlRigOverrideValue::AllocateDataIfRequired()
{
	return AllocateDataIfRequired(DataArray, DataPropertyHash);
}

void FControlRigOverrideValue::FreeDataIfRequired()
{
	FreeDataIfRequired(DataArray, DataPropertyHash);
}

void* FControlRigOverrideValue::AllocateDataIfRequired(TArray<uint8, TAlignedHeapAllocator<16>>& InOutDataArray, uint32& InOutPropertyHash) const
{
	if(const FProperty* LeafProperty = GetLeafProperty())
	{
		if(InOutDataArray.IsEmpty() || !IsDataValid(InOutDataArray, InOutPropertyHash))
		{
			InOutDataArray.Reset();
			InOutDataArray.AddZeroed(LeafProperty->GetSize());
			InOutPropertyHash = GetPropertyHash();
			LeafProperty->InitializeValue(InOutDataArray.GetData());
		}
	}
	return InOutDataArray.GetData();
}

void FControlRigOverrideValue::FreeDataIfRequired(TArray<uint8, TAlignedHeapAllocator<16>>& InOutDataArray, uint32& InOutPropertyHash) const
{
	if(IsDataValid())
	{
		if(void* DataPtr = InOutDataArray.GetData())
		{
			if(const FProperty* LeafProperty = GetLeafProperty())
			{
				if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(LeafProperty))
				{
					FScriptArrayHelper ArrayHelper(ArrayProperty, DataPtr);
					ArrayHelper.DestroyContainer_Unsafe();
				}
				else
				{
					LeafProperty->DestroyValue(DataPtr);
				}
			}
		}
	}
	InOutDataArray.Reset();
	InOutPropertyHash = 0;
}

void FControlRigOverrideValue::CopyValue(void* InDestPtr, const void* InSourcePtr) const
{
	check(InDestPtr && InSourcePtr);
	
	const FProperty* LeafProperty = GetLeafProperty();
	check(LeafProperty);

	if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(LeafProperty))
	{
		FScriptArrayHelper SourceArray(ArrayProperty, InSourcePtr);
		FScriptArrayHelper DestArray(ArrayProperty, InDestPtr);
		DestArray.Resize(SourceArray.Num());

		for(int32 Index = 0; Index < SourceArray.Num(); Index++)
		{
			const void* SourceElement = SourceArray.GetRawPtr(Index);
			void* DestElement = DestArray.GetRawPtr(Index);
			if(SourceElement && DestElement)
			{
				ArrayProperty->Inner->CopyCompleteValue(DestElement, SourceElement);
			}
		}
	}
	else
	{
		LeafProperty->CopyCompleteValue(InDestPtr, InSourcePtr);
	}
}

void FControlRigOverrideValue::UpdateHash()
{
	Hash = GetTypeHash(SubjectKey);
	if(IsValid())
	{
		for(const FPropertyInfo& PropertyInfo : Properties)
		{
			Hash = HashCombine(Hash, GetTypeHash(PropertyInfo.Property));
			Hash = HashCombine(Hash, GetTypeHash(PropertyInfo.ArrayIndex));
		}
		Hash = HashCombine(Hash, GetTypeHash(CachedStringValue));
	}
}

const FString& FControlRigOverrideValue::ToString() const
{
	static const FString EmptyPath;
	return CachedStringValue.Get(EmptyPath);
}

bool FControlRigOverrideValue::SetFromString(const FString& InValue, const FControlRigOverrideValueErrorPipe::TReportFunction& InReportFunction)
{
	if(InValue != CachedStringValue.Get(FString()))
	{
		if(const FProperty* LeafProperty = GetLeafProperty())
		{
			TArray<uint8, TAlignedHeapAllocator<16>> TempDataArray;
			uint32 TempPropertyHash = 0;
			void* TempDataPtr = AllocateDataIfRequired(TempDataArray, TempPropertyHash); 

			// first try to import the string value using the property.
			// we need to react to any message coming in from the import since missing parentheses for
			// example are reported at verbosity level Log.
			FControlRigOverrideValueErrorPipe ErrorPipe(ELogVerbosity::Log, InReportFunction);
			LeafProperty->ImportText_Direct(*InValue, TempDataPtr, nullptr, PPF_None, &ErrorPipe);
			if(ErrorPipe.GetNumErrors() == 0)
			{
				FreeDataIfRequired(DataArray, DataPropertyHash);
				Swap(DataArray, TempDataArray);
				Swap(DataPropertyHash, TempPropertyHash);
				
				CachedStringValue = InValue;
				UpdateHash();
				return true;
			}
			FreeDataIfRequired(TempDataArray, TempPropertyHash);

			// we've hit a problem when importing the value
			if(const FStructProperty* StructProperty = CastField<FStructProperty>(LeafProperty))
			{
				// potentially we have a default value for a math type here.
				const UStruct* Struct = StructProperty->Struct;

				if(Struct == TBaseStructure<FVector2D>::Get())
				{
					FString ValueString = InValue;
					for(int32 TryIndex = 0; TryIndex < 2; TryIndex++)
					{
						if(InitFromLegacyString<FVector2D>(ValueString, FVector2D::ZeroVector))
						{
							return true;
						}

						// potentially we have the value as "1.000,2.000"
						if(TryIndex == 0)
						{
							TArray<FString> Parts;
							if(RigVMStringUtils::SplitString(ValueString, TEXT(","), Parts))
							{
								if(Parts.Num() == 2)
								{
									ValueString = FString::Printf(TEXT("(X=%s,Y=%s)"), *Parts[0], *Parts[1]);
									continue;
								}
							}
						}
						
						break;
					}
				}

				if(Struct == TBaseStructure<FVector>::Get())
				{
					FString ValueString = InValue;
					for(int32 TryIndex = 0; TryIndex < 2; TryIndex++)
					{
						if(InitFromLegacyString<FVector>(ValueString, FVector::ZeroVector))
						{
							return true;
						}
					
						// potentially we have the value as "1.000,2.000,3.000"
						if(TryIndex == 0)
						{
							TArray<FString> Parts;
							if(RigVMStringUtils::SplitString(ValueString, TEXT(","), Parts))
							{
								if(Parts.Num() == 3)
								{
									ValueString = FString::Printf(TEXT("(X=%s,Y=%s,Z=%s)"), *Parts[0], *Parts[1], *Parts[2]);
									continue;
								}
							}
						}
						
						break;
					}
				}

				if(Struct == TBaseStructure<FRotator>::Get())
				{
					FString ValueString = InValue;
					for(int32 TryIndex = 0; TryIndex < 2; TryIndex++)
					{
						if(InitFromLegacyString<FRotator>(ValueString, FRotator::ZeroRotator))
						{
							return true;
						}

						// potentially we have the value as "1.000,2.000,3.000"
						if(TryIndex == 0)
						{
							TArray<FString> Parts;
							if(RigVMStringUtils::SplitString(ValueString, TEXT(","), Parts))
							{
								if(Parts.Num() == 3)
								{
									ValueString = FString::Printf(TEXT("(P=%s,Y=%s,R=%s)"), *Parts[0], *Parts[1], *Parts[2]);
									continue;
								}
							}
						}
						
						break;
					}
				}

				if(Struct == TBaseStructure<FQuat>::Get())
				{
					if(InitFromLegacyString<FQuat>(InValue, FQuat::Identity))
					{
						return true;
					}
				}

				if(Struct == TBaseStructure<FTransform>::Get())
				{
					if(InitFromLegacyString<FTransform>(InValue, FTransform::Identity))
					{
						return true;
					}
				}
			}

			if(SubjectKey.IsSet())
			{
				UE_LOGF(LogControlRig, Warning, "Unable to import default value '%ls' for override '%ls' on subject '%ls'.", *InValue, *GetPath(), *SubjectKey.GetValue().ToString());
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Unable to import default value '%ls' for override '%ls'", *InValue, *GetPath());
			}
		}
	}
	return false;
}

bool FControlRigOverrideValue::operator==(const FControlRigOverrideValue& InOtherValue) const
{
	if(Hash != InOtherValue.Hash)
	{
		return false;
	}
	return IdenticalValue(InOtherValue.GetData());
}

bool FControlRigOverrideValue::Identical(const FControlRigOverrideValue& InOtherValue) const
{
	return *this == InOtherValue;
}

bool FControlRigOverrideValue::IdenticalValue(const void* InValuePtr) const
{
	if(!IsValid() || (InValuePtr == nullptr))
	{
		return false;
	}

	// IsValid checks the validity of the data pointer and leaf property
	const FProperty* LeafProperty = GetLeafProperty();
	check(LeafProperty);
	return LeafProperty->Identical(GetData(), InValuePtr);
}

bool FControlRigOverrideValue::IdenticalValueInSubject(const void* InSubjectPtr) const
{
	return IdenticalValue(SubjectPtrToValuePtr(InSubjectPtr, false));
}

bool FControlRigOverrideValue::CopyToSubject(void* InSubjectPtr, const UStruct* InSubjectStruct) const
{
	if(InSubjectStruct)
	{
		if(const FProperty* RootProperty = GetRootProperty())
		{
			if(RootProperty->GetOwnerStruct() != InSubjectStruct)
			{
				return false;
			}
		}
	}

	if(const void* SourcePtr = GetData())
	{
		if(void* DestPtr = SubjectPtrToValuePtr(InSubjectPtr, true))
		{
			CopyValue(DestPtr, SourcePtr);
			return true;
		}
	}

	return false;
}

bool FControlRigOverrideValue::SetFromSubject(const void* InSubjectPtr, const UStruct* InSubjectStruct)
{
	if(InSubjectStruct)
	{
		if(const FProperty* RootProperty = GetRootProperty())
		{
			if(RootProperty->GetOwnerStruct() != InSubjectStruct)
			{
				return false;
			}
		}
	}
	
	if(const FProperty* LeafProperty = GetLeafProperty())
	{
		void* DestPtr = AllocateDataIfRequired();
		if(InSubjectPtr && DestPtr)
		{
			if(const void* SourcePtr = SubjectPtrToValuePtr(InSubjectPtr, false))
			{
				FString ValueAsString;
				CopyValue(DestPtr, SourcePtr);
				LeafProperty->ExportTextItem_Direct(ValueAsString, DestPtr, DestPtr, nullptr, PPF_None, nullptr);
				CachedStringValue = ValueAsString;
				UpdateHash();
				return true;
			}
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
// FControlRigOverrideContainer
////////////////////////////////////////////////////////////////////////////////

void FControlRigOverrideContainer::Reset()
{
	Values.Reset();
	HashIndexLookup.Reset();
	SubjectIndexLookup.Reset();
	InvalidateCache();
}

void FControlRigOverrideContainer::Empty()
{
	Values.Empty();
	HashIndexLookup.Empty();
	SubjectIndexLookup.Reset();
	InvalidateCache();
}

void FControlRigOverrideContainer::Reserve(int32 InNum)
{
	Values.Reserve(InNum);
	HashIndexLookup.Reserve(InNum);
}

bool FControlRigOverrideContainer::UsesKeyForSubject() const
{
	return bUsesKeyForSubject;
}

void FControlRigOverrideContainer::SetUsesKeyForSubject(bool InUsesKeyForSubject)
{
	bUsesKeyForSubject = InUsesKeyForSubject;
	RebuildLookup();
	InvalidateCache();
}

int32 FControlRigOverrideContainer::Add(const FControlRigOverrideValue& InValue)
{
	if(!InValue.IsValid())
	{
		return INDEX_NONE;
	}
	
	const FString& Path = InValue.GetPath();
	if(!Path.IsEmpty())
	{
		if(!Contains(Path, InValue.GetSubjectKey()))
		{
			// don't allow to add overrides for child paths of
			// existing parent paths
			if(ContainsParentPathOf(Path, InValue.GetSubjectKey()))
			{
				return INDEX_NONE;
			}
			
			// remove child paths when adding parent overrides
			if(ContainsChildPathOf(Path, InValue.GetSubjectKey()))
			{
				TArray<int32> IndicesToRemove;
				for(int32 Index = 0; Index < Values.Num(); Index++)
				{
					const FString& ChildPath = Values[Index].GetPath();
					if(IsChildPathOf(ChildPath, Path))
					{
						IndicesToRemove.Add(Index);
					}
				}

				if(!IndicesToRemove.IsEmpty())
				{
					// remove in reverse
					for(int32 Index = IndicesToRemove.Num() - 1; Index >= 0; Index--)
					{
						Values.RemoveAt(IndicesToRemove[Index]);
					}
					RebuildLookup();
				}
			}
			
			const int32 Index = Values.Add(InValue);
			const uint32 Hash = GetLookupHash(Path, InValue.GetSubjectKey());
			HashIndexLookup.Add(Hash, Index);
			SubjectIndexLookup.FindOrAdd(InValue.GetSubjectKey()).Add(Index);

			InvalidateCache();
			return Index;
		}
	}
	return INDEX_NONE;
}

const FControlRigOverrideValue* FControlRigOverrideContainer::FindOrAdd(const FControlRigOverrideValue& InValue)
{
	const FString& Path = InValue.GetPath();
	if(!Path.IsEmpty())
	{
		if(FControlRigOverrideValue* ExistingValue = Find(Path, InValue.GetSubjectKey()))
		{
			*ExistingValue = InValue;
			return ExistingValue;
		}

		const int32 NewIndex = Add(InValue);
		if(IsValidIndex(NewIndex))
		{
			return &Values[NewIndex];
		}
	}
	return nullptr;
}

bool FControlRigOverrideContainer::Remove(const FControlRigOverrideValue& InValue)
{
	return Remove(InValue.GetPath(), InValue.GetSubjectKey());
}

bool FControlRigOverrideContainer::Remove(const FString& InPath, const FName& InSubjectKey)
{
	const int32 Index = GetIndex(InPath, InSubjectKey);
	if(IsValidIndex(Index))
	{
		Values.RemoveAt(Index);
		RebuildLookup();
		InvalidateCache();
		return true;
	}
	return false;
}

bool FControlRigOverrideContainer::RemoveAll(const FName& InSubjectKey)
{
	if(IsEmpty())
	{
		return false;
	}
	if(!InSubjectKey.IsNone())
	{
		if(const TArray<int32>* IndicesPtr = GetIndicesForSubject(InSubjectKey))
		{
			for(int32 Index = IndicesPtr->Num() - 1; Index >= 0; Index--)
			{
				Values.RemoveAt(IndicesPtr->operator[](Index));
			}
			RebuildLookup();
			InvalidateCache();
			return true;
		}
		return false;
	}
	else
	{
		Reset();
	}
	return true;
}

int32 FControlRigOverrideContainer::GetIndex(const FString& InPath, const FName& InSubjectKey) const
{
	const uint32 Hash = GetLookupHash(InPath, InSubjectKey);
	if(const int32* Index = HashIndexLookup.Find(Hash))
	{
		return *Index;
	}
	return INDEX_NONE;
}

const TArray<int32>* FControlRigOverrideContainer::GetIndicesForSubject(const FName& InSubjectKey) const
{
	return SubjectIndexLookup.Find(InSubjectKey);
}

const FControlRigOverrideValue* FControlRigOverrideContainer::Find(const FString& InPath, const FName& InSubjectKey) const
{
	const int32 Index = GetIndex(InPath, InSubjectKey);
	if(IsValidIndex(Index))
	{
		return &Values[Index];
	}
	return nullptr;
}

FControlRigOverrideValue* FControlRigOverrideContainer::Find(const FString& InPath, const FName& InSubjectKey)
{
	const int32 Index = GetIndex(InPath, InSubjectKey);
	if(IsValidIndex(Index))
	{
		return &Values[Index];
	}
	return nullptr;
}

const FControlRigOverrideValue& FControlRigOverrideContainer::FindChecked(const FString& InPath, const FName& InSubjectKey) const
{
	const int32 Index = GetIndex(InPath, InSubjectKey);
	check(IsValidIndex(Index));
	return Values[Index];
}

FControlRigOverrideValue& FControlRigOverrideContainer::FindChecked(const FString& InPath, const FName& InSubjectKey)
{
	const int32 Index = GetIndex(InPath, InSubjectKey);
	check(IsValidIndex(Index));
	return Values[Index];
}

TArray<FName> FControlRigOverrideContainer::GenerateSubjectArray() const
{
	TArray<FName> Keys;
	SubjectIndexLookup.GenerateKeyArray(Keys);
	return Keys;
}

bool FControlRigOverrideContainer::Contains(const FString& InPath, const FName& InSubjectKey) const
{
	return IsValidIndex(GetIndex(InPath, InSubjectKey));
}

bool FControlRigOverrideContainer::ContainsParentPathOf(const FString& InChildPath, const FName& InSubjectKey) const
{
	if(InChildPath.IsEmpty())
	{
		return false;
	}

	const uint32 Hash = GetLookupHash(InChildPath, InSubjectKey);
	if(const bool* CachedResult = ContainsParentPathCache.Find(Hash))
	{
		return *CachedResult;
	}

	const FString ParentPath = GetParentPath(InChildPath);
	const bool bResult = !ParentPath.IsEmpty() && (Contains(ParentPath, InSubjectKey) || ContainsParentPathOf(ParentPath, InSubjectKey));
	ContainsParentPathCache.Add(Hash, bResult);
	return bResult;
}

bool FControlRigOverrideContainer::ContainsChildPathOf(const FString& InParentPath, const FName& InSubjectKey) const
{
	if(InParentPath.IsEmpty())
	{
		return false;
	}
	
	const uint32 Hash = GetLookupHash(InParentPath, InSubjectKey);
	if(const bool* CachedResult = ContainsChildPathCache.Find(Hash))
	{
		return *CachedResult;
	}

	for(const FControlRigOverrideValue& Value : Values)
	{
		if(bUsesKeyForSubject)
		{
			if(Value.GetSubjectKey() != InSubjectKey)
			{
				continue;
			}
		}
		const FString& Path = Value.GetPath();
		if(IsChildPathOf(Path, InParentPath))
		{
			ContainsChildPathCache.Add(Hash, true);
			return true;
		}
	}

	ContainsChildPathCache.Add(Hash, false);
	return false;
}

bool FControlRigOverrideContainer::ContainsAnyPathForSubject(const FName& InSubjectKey) const
{
	return SubjectIndexLookup.Contains(InSubjectKey);
}

bool FControlRigOverrideContainer::ContainsPathForAnySubject(const FString& InPath) const
{
	for(const FControlRigOverrideValue& Override : Values)
	{
		if(Override.GetPath() == InPath)
		{
			return true;
		}
	}
	return false;
}

bool FControlRigOverrideContainer::Contains(const FControlRigOverrideValue& InOverrideValue) const
{
	return Contains(InOverrideValue.GetPath(), InOverrideValue.GetSubjectKey());
}

bool FControlRigOverrideContainer::ContainsParentPathOf(const FControlRigOverrideValue& InOverrideValue) const
{
	return ContainsParentPathOf(InOverrideValue.GetPath(), InOverrideValue.GetSubjectKey());
}

bool FControlRigOverrideContainer::ContainsChildPathOf(const FControlRigOverrideValue& InOverrideValue) const
{
	return ContainsChildPathOf(InOverrideValue.GetPath(), InOverrideValue.GetSubjectKey());
}

void FControlRigOverrideContainer::CopyToSubject(void* InSubjectPtr, const UStruct* InSubjectStruct, const FName& InSubjectKey) const
{
	for(const FControlRigOverrideValue& Override : Values)
	{
		if(bUsesKeyForSubject)
		{
			if(Override.GetSubjectKey() != InSubjectKey)
			{
				continue;
			}
		}
		(void)Override.CopyToSubject(InSubjectPtr, InSubjectStruct);
	}
}

void FControlRigOverrideContainer::SetFromSubject(const void* InSubjectPtr, const UStruct* InSubjectStruct, const FName& InSubjectKey)
{
	for(FControlRigOverrideValue& Override : Values)
	{
		if(bUsesKeyForSubject)
		{
			if(Override.GetSubjectKey() != InSubjectKey)
			{
				continue;
			}
		}
		(void)Override.SetFromSubject(InSubjectPtr, InSubjectStruct);
	}
}

bool FControlRigOverrideContainer::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	if (!Ar.IsLoading())
	{
		Values.RemoveAll([](const FControlRigOverrideValue& Value) -> bool {
			return !Value.IsValid();
		});
	}
		
	Ar << bUsesKeyForSubject;

	const bool bStoresOnlyPathAndLeafProperty = Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::OverridesStorePathAndLeafPropertyOnly;
	if (!bStoresOnlyPathAndLeafProperty)
	{
		Ar << Values;
	}
	else
	{
		int32 NumValues = Values.Num();
		Ar << NumValues;
		
		if (Ar.IsLoading())
		{
			Values.Reset();
			Values.AddDefaulted(NumValues);
		}
		
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			Values[Index].Serialize(Ar);
		}
	}
	
	if(Ar.IsLoading())
	{
		Values.RemoveAll([](const FControlRigOverrideValue& Value) -> bool {
			return !Value.IsValid();
		});

		RebuildLookup();
		InvalidateCache();
	}
	return true;
}

void FControlRigOverrideContainer::RebuildLookup()
{
	HashIndexLookup.Reset();
	HashIndexLookup.Reserve(Values.Num());
	SubjectIndexLookup.Reset();

	for(int32 Index = 0; Index < Values.Num(); Index++)
	{
		if(Values[Index].IsValid())
		{
			const FString& Path = Values[Index].GetPath();
			if(!Path.IsEmpty())
			{
				const uint32 Hash = GetLookupHash(Path, Values[Index].GetSubjectKey());
				HashIndexLookup.Add(Hash, Index);
				SubjectIndexLookup.FindOrAdd(Values[Index].GetSubjectKey()).Add(Index);
			}
		}
	}
}

void FControlRigOverrideContainer::InvalidateCache() const
{
	ContainsParentPathCache.Reset();
	ContainsChildPathCache.Reset();
}

#if WITH_EDITOR
void UControlRigOverrideAsset::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	UObject::PostTransacted(TransactionEvent);

	if(TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		for (TObjectIterator<UControlRig> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			UControlRig* ControlRig = *It;
			if(IsValid(ControlRig))
			{
				if(ControlRig->IsLinkedToOverrideAsset(this))
				{
					ControlRig->RequestConstruction();
				}
			}
		}

	}
}
#endif

UControlRigOverrideAsset* UControlRigOverrideAsset::CreateOverrideAsset(const FString& InLongName)
{
	UControlRigOverrideAsset* Asset = nullptr;
#if WITH_EDITOR
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FString UniquePackageName;
	FString UniqueAssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(InLongName, TEXT(""), UniquePackageName, UniqueAssetName);
	if (UniquePackageName.EndsWith(UniqueAssetName))
	{
		UniquePackageName = UniquePackageName.LeftChop(UniqueAssetName.Len() + 1);
	}
	Asset = Cast<UControlRigOverrideAsset>(AssetToolsModule.Get().CreateAsset(*UniqueAssetName, *UniquePackageName, UControlRigOverrideAsset::StaticClass(), nullptr));
	check(Asset);
#endif
	return Asset;
}

UControlRigOverrideAsset* UControlRigOverrideAsset::CreateOverrideAssetInDeveloperFolder(const UControlRig* InSubject)
{
	UControlRigOverrideAsset* Asset = nullptr;
#if WITH_EDITOR
	FString DeveloperPathWithoutSlash = FPackageName::FilenameToLongPackageName(FPaths::GameUserDeveloperDir()).LeftChop(1);
	FString ShortName = InSubject ? InSubject->GetAssetReference().Get()->GetOutermost()->GetName() : TEXT("Default");
	ShortName = FPaths::GetBaseFilename(ShortName) + TEXT("_Override");
	const FString LongName = FPaths::Combine(DeveloperPathWithoutSlash, TEXT("Overrides"), ShortName);
	Asset = CreateOverrideAsset(LongName);
#endif
	return Asset;
}

void UControlRigOverrideAsset::BroadcastChanged()
{
	OverrideChangedDelegate.Broadcast(this);
}

void FControlRigOverrideContainer::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	for(const FControlRigOverrideValue& Override : Values)
	{
		if(const FProperty* RootProperty = Override.GetRootProperty())
		{
			if(UClass* Class = RootProperty->GetOwnerClass())
			{
				TWeakObjectPtr<UClass> WeakClass(Class);
				Collector.AddReferencedObject(WeakClass);
			}
			else if(UStruct* Struct = RootProperty->GetOwnerStruct())
			{
				TWeakObjectPtr<UStruct> WeakStruct(Struct);
				Collector.AddReferencedObject(WeakStruct);
			}

			Override.AddStructReferencedObjects(Collector);
		}
	}
}

void FControlRigOverrideContainer::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	for(FControlRigOverrideValue& Override : Values)
	{
		Override.GetPreloadDependencies(OutDeps);
	}
}

bool FControlRigOverrideContainer::Identical(const FControlRigOverrideContainer* Other, uint32 PortFlags) const
{
	if(Other == nullptr)
	{
		return false;
	}
	if(Num() != Other->Num())
	{
		return false;
	}
	for(int32 Index = 0; Index < Num(); Index++)
	{
		const FControlRigOverrideValue& Value = operator[](Index);
		const FControlRigOverrideValue& OtherValue = Other->operator[](Index);
		if(!Value.Identical(OtherValue))
		{
			return false;
		}
	}
	return true;
}

FString FControlRigOverrideContainer::GetParentPath(const FString& InChildPath)
{
	int32 LastSeparatorIndex = InChildPath.Find(FControlRigOverrideValue::PathSeparator, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

	int32 LastArrayIndex = INDEX_NONE;
	if(InChildPath.FindLastChar(FControlRigOverrideValue::ArraySeparator, LastArrayIndex))
	{
		LastSeparatorIndex = FMath::Max(LastSeparatorIndex, LastArrayIndex);
	}

	if(LastSeparatorIndex != INDEX_NONE)
	{
		return InChildPath.Left(LastSeparatorIndex);
	}

	return FString();
}

bool FControlRigOverrideContainer::IsChildPathOf(const FString& InChildPath, const FString& InParentPath)
{
	if(!InChildPath.IsEmpty())
	{
		// this is true for both cases of child paths - one being separated by -> and the other being separated by [
		if(InChildPath.Len() > InParentPath.Len() + FControlRigOverrideValue::PathSeparatorLength)
		{
			const FStringView ChildPathView(InChildPath);
			if(ChildPathView.StartsWith(InParentPath))
			{
				if(ChildPathView[InParentPath.Len()] == FControlRigOverrideValue::ArraySeparator)
				{
					return true;
				}
				if(ChildPathView.Mid(InParentPath.Len(), FControlRigOverrideValue::PathSeparatorLength) == FControlRigOverrideValue::PathSeparator)
				{
					return true;
				}
			}
		}
	}
	return false;
}

#if WITH_EDITOR
TSharedPtr<FPropertyPath> FControlRigOverrideValue::ToPropertyPath() const
{
	if(Properties.IsEmpty())
	{
		return nullptr;
	}
	
	TSharedPtr<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();

	for(const FPropertyInfo& PropertyInfo : Properties)
	{
		PropertyPath->AddProperty({PropertyInfo.Property, PropertyInfo.ArrayIndex});
	}

	return PropertyPath;
}
#endif

uint32 GetTypeHash(const FControlRigOverrideContainer& InContainer)
{
	uint32 Hash = GetTypeHash(InContainer.bUsesKeyForSubject);
	for(const FControlRigOverrideValue& Override : InContainer.Values)
	{
		Hash = HashCombine(Hash, GetTypeHash(Override));
	}
	return Hash;
}
