// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigProfiling.h"

DEFINE_LOG_CATEGORY_STATIC(ControlRigProfilingLog, Log, All);

FControlRigStats GControlRigStats;
double GControlRigStatsCounter = 0;

FControlRigStats& FControlRigStats::Get()
{
	return GControlRigStats;
}

void FControlRigStats::Clear()
{
	Stack.Reset();
	Counters.Reset();
}

double& FControlRigStats::RetainCounter(const TCHAR* Key)
{
	FName Name(Key, FNAME_Find);
	if (Name == NAME_None)
	{
		Name = FName(Key, FNAME_Add);
	}
	return RetainCounter(Name);
}

double& FControlRigStats::RetainCounter(const FName& Key)
{
	if (!Enabled)
	{
		return GControlRigStatsCounter;
	}
	FName Path = Key;
	if (Stack.Num() > 0)
	{
		Path = FName(*FString::Printf(TEXT("%s.%s"), *Stack.Top().ToString(), *Key.ToString()));
	}
	Stack.Add(Path);
	TUniquePtr<double>& Slot = GControlRigStats.Counters.FindOrAdd(Path);
	if (!Slot)
	{
		Slot = MakeUnique<double>(0.0);
	}
	return *Slot;
}

void FControlRigStats::ReleaseCounter()
{
	if (!Enabled)
	{
		return;
	}
	if (Stack.Num() > 0)
	{
		Stack.Pop(EAllowShrinking::No);
	}
}

void FControlRigStats::Dump()
{
	struct FCounterTree
	{
		FName Name;
		double Seconds;
		TArray<FCounterTree*> Children;

		~FCounterTree()
		{
			for (FCounterTree* Counter : Children)
			{
				delete(Counter);
			}
		}

		int32 GetHash() const
		{
			return INT32_MAX - (int32)(Seconds * 100000.f);
		}

		FCounterTree* FindChild(const FName& InName)
		{
			for (FCounterTree* Child : Children)
			{
				if (Child->Name == InName)
				{
					return Child;
				}
			}
			return nullptr;
		}

		FCounterTree* FindOrAddChild(const FName& InName)
		{
			if (FCounterTree* Child = FindChild(InName))
			{
				return Child;
			}

			FString Path = InName.ToString();
			FString Left, Right;
			if (Path.Split(TEXT("."), &Left, &Right))
			{
				if (FCounterTree* Child = FindChild(*Left))
				{
					return Child->FindOrAddChild(*Right);;
				}
				FCounterTree* Child = new(FCounterTree);
				Child->Seconds = 0.0;
				Children.Add(Child);
				Child->Name = *Left;
				return Child->FindOrAddChild(*Right);
			}

			FCounterTree* Child = new(FCounterTree);
			Child->Seconds = 0.0;
			Children.Add(Child);
			Child->Name = InName;
			return Child;
		}

		void Dump(const FString& Indent = TEXT(""))
		{
			if (Seconds > SMALL_NUMBER)
			{
				UE_LOGF(ControlRigProfilingLog, Display, "%ls%08.3f ms -  %ls", *Indent, float(Seconds * 1000.0), *Name.ToString());
			}
			else
			{
				UE_LOGF(ControlRigProfilingLog, Display, "%ls%ls", *Indent, *Name.ToString());
			}

			TArray<FCounterTree*> LocalChildren = Children;
			TArray<int32> Order;
			for (FCounterTree* Child : LocalChildren)
			{
				Order.Add(Order.Num());
			}

			Algo::SortBy(Order, [&LocalChildren](int32 Index) -> int32
			{
				return LocalChildren[Index]->GetHash();
			});

			for(int32 Index : Order)
			{
				LocalChildren[Index]->Dump(Indent + TEXT("   "));
			}
		}
	};
	
	FCounterTree Tree;
	Tree.Name = TEXT("Control Rig Profiling");
	Tree.Seconds = 0.0;
	for (const TPair<FName, TUniquePtr<double>>& Pair : Counters)
	{
		FCounterTree* Child = Tree.FindOrAddChild(Pair.Key);
		Child->Seconds = Pair.Value ? *Pair.Value : 0.0;
	}

	Tree.Dump();
}

FControlRigSimpleScopeSecondsCounter::FControlRigSimpleScopeSecondsCounter(const TCHAR* InName)
	: FSimpleScopeSecondsCounter(FControlRigStats::Get().RetainCounter(InName), FControlRigStats::Get().Enabled)
{
}

FControlRigSimpleScopeSecondsCounter::~FControlRigSimpleScopeSecondsCounter()
{
	FControlRigStats::Get().ReleaseCounter();
}

