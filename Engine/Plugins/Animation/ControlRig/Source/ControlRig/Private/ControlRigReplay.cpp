// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigReplay.h"
#include "ControlRig.h"
#include "Rigs/RigHierarchyController.h"
#include "ControlRigObjectVersion.h"
#include "Engine/SkeletalMesh.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/RigUnitContext.h"
#include "HAL/PlatformTime.h"
#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "Engine/World.h"
#include "Kismet/KismetSystemLibrary.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigReplay)
#define LOCTEXT_NAMESPACE "ControlRigReplay"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FControlRigReplayTracks
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FControlRigReplayTracks::Serialize(FArchive& InArchive)
{
	if(!FSampleTrackHost::Serialize(InArchive))
	{
		return false;
	}

	if(InArchive.IsLoading())
	{
		SampleTrackIndex = FSampleTrackIndex(*GetContainer());
	}
	return true;
}

void FControlRigReplayTracks::Reset()
{
	FSampleTrackHost::Reset();
	SampleTrackIndex = FSampleTrackIndex();
}

bool FControlRigReplayTracks::IsEmpty() const
{
	return GetContainer()->GetNumTimes() == 0 || GetContainer()->NumTracks() == 0;
}

void FControlRigReplayTracks::StoreRigVMEvent(const FName& InName)
{
	const TSharedPtr<FSampleTrack<FName>> EventTrack = GetContainer()->FindOrAddTrack<FName>(RigVMEventName, FSampleTrackBase::ETrackType_Name);
	EventTrack->AddSample(InName);
}

FName FControlRigReplayTracks::GetRigVMEvent(int32 InTimeIndex) const
{
	if(const TSharedPtr<const FSampleTrack<FName>> EventTrack = GetContainer()->FindTrack<FName>(RigVMEventName))
	{
		return EventTrack->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
	}
	return NAME_None;
}

void FControlRigReplayTracks::StoreInteraction(uint8 InInteractionMode, const TArray<FRigElementKey>& InElementsBeingInteracted)
{
	const TSharedPtr<FSampleTrack<uint32>> InteractionTypeTrack = GetContainer()->FindOrAddTrack<uint32>(InteractionTypeName, FSampleTrackBase::ETrackType_Uint32);
	const TSharedPtr<FSampleTrack<TArray<FRigElementKey>>> ElementsBeingInteractedTrack = GetContainer()->FindOrAddTrack<TArray<FRigElementKey>>(ElementsBeingInteractedName, FSampleTrackBase::ETrackType_ElementKeyArray);
	InteractionTypeTrack->AddSample(static_cast<uint32>(InInteractionMode));
	ElementsBeingInteractedTrack->AddSample(InElementsBeingInteracted);
}

TTuple<uint8, TArray<FRigElementKey>> FControlRigReplayTracks::GetInteraction(int32 InTimeIndex) const
{
	const TSharedPtr<const FSampleTrack<uint32>> InteractionTypeTrack = GetContainer()->FindTrack<uint32>(InteractionTypeName);
	const TSharedPtr<const FSampleTrack<TArray<FRigElementKey>>> ElementsBeingInteractedTrack = GetContainer()->FindTrack<TArray<FRigElementKey>>(ElementsBeingInteractedName);
	if(InteractionTypeTrack.IsValid() && ElementsBeingInteractedTrack.IsValid())
	{
		return {static_cast<uint8>(InteractionTypeTrack->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex)), ElementsBeingInteractedTrack->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex)};
	}
	return {0, TArray<FRigElementKey>()};
}

void FControlRigReplayTracks::StoreHierarchy(URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys, bool bStorePose,
                                             bool bStoreComponents, bool bStoreMetadata)
{
	// for the first frame store the expected hierarchy topology in the output
	if(!bIsInput)
	{
		const TSharedPtr<FSampleTrack<uint32>> TopologyHashTrack = GetContainer()->FindOrAddTrack<uint32>(TopologyHashName, FSampleTrackBase::ETrackType_Uint32);
		const TSharedPtr<FSampleTrack<uint32>> MetadataVersionTrack = GetContainer()->FindOrAddTrack<uint32>(MetadataVersionName, FSampleTrackBase::ETrackType_Uint32);
		TopologyHashTrack->AddSample(InHierarchy->GetTopologyHash());
		MetadataVersionTrack->AddSample(InHierarchy->GetMetadataTagVersion());

		if(GetNumTimes() == 1)
		{
			const TSharedPtr<FSampleTrack<TArray<FRigElementKey>>> ElementKeysTrack = GetContainer()->FindOrAddTrack<TArray<FRigElementKey>>(ElementKeysName, FSampleTrackBase::ETrackType_ElementKeyArray);
			const TSharedPtr<FSampleTrack<TArray<int32>>> ParentIndicesTrack = GetContainer()->FindOrAddTrack<TArray<int32>>(ParentIndicesName, FSampleTrackBase::ETrackType_Int32Array);

			TArray<FRigElementKey> ElementKeys = InHierarchy->GetAllKeys();
			FilterElementKeys(ElementKeys);
			
			TArray<int32> ParentIndices;
			ParentIndices.Reserve(ElementKeys.Num());
			for(const FRigElementKey& ElementKey : ElementKeys)
			{
				if(ElementKey.Type == ERigElementType::Connector)
				{
					ParentIndices.Add(INDEX_NONE);
				}
				else
				{
					ParentIndices.Add(ElementKeys.Find(InHierarchy->GetDefaultParent(ElementKey)));
				}
			}

			ElementKeysTrack->AddSample(ElementKeys);
			ParentIndicesTrack->AddSample(ParentIndices);
		}
	}

	ForEachElement(InHierarchy, InKeys, [this, InHierarchy, bStorePose, bStoreComponents, bStoreMetadata](FRigBaseElement* InElement, bool& bSuccess)
	{
		if(bStorePose)
		{
			StorePose(InHierarchy, InElement);
		}
		if(bStoreComponents)
		{
			StoreComponents(InHierarchy, InElement);
		}
		if(bStoreMetadata)
		{
			StoreMetaData(InHierarchy, InElement);
		}
	});
}

bool FControlRigReplayTracks::RestoreHierarchy(int32 InTimeIndex, URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys,
	TReportFunction InReportFunction, bool bRestorePose, bool bRestoreComponents, bool bRestoreMetadata) const
{
	return ForEachElement(InHierarchy, InKeys,
	[this, InTimeIndex, InHierarchy, InReportFunction, bRestorePose, bRestoreComponents, bRestoreMetadata](FRigBaseElement* InElement, bool& bSuccess)
	{
		if(bRestorePose)
		{
			if(!RestorePose(InTimeIndex, InHierarchy, InElement, InReportFunction))
			{
				bSuccess = false;
			}
		}
		if(bRestoreComponents)
		{
			if(!RestoreComponents(InTimeIndex, InHierarchy, InElement, InReportFunction))
			{
				bSuccess = false;
			}
		}
		if(bRestoreMetadata)
		{
			if(!RestoreMetaData(InTimeIndex, InHierarchy, InElement, InReportFunction))
			{
				bSuccess = false;
			}
		}
	});
}

void FControlRigReplayTracks::StorePose(URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys)
{
	ForEachElement(InHierarchy, InKeys, [this, InHierarchy](FRigBaseElement* InElement, bool& bSuccess)
	{
		StorePose(InHierarchy, InElement);
	});
}

void FControlRigReplayTracks::StorePose(URigHierarchy* InHierarchy, FRigBaseElement* InElement)
{
	if(!InElement->IsA<FRigTransformElement>() && !InElement->IsA<FRigCurveElement>() && !InElement->IsA<FRigConnectorElement>() && !InElement->IsA<FRigReferenceElement>())
	{
		return;
	}

	FSampleTrackContainer* Storage = GetContainer();
	const FName& TrackName = GetTrackName(InElement->GetKey());

	if(const FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(InElement))
	{
		if(bIsInput)
		{
			if(const UControlRig* ControlRig = Cast<UControlRig>(InHierarchy->GetOuter()))
			{
				const FName& ConnectorTrackName = *(TrackName.ToString() + TEXT("ConnectorTargets"));
				const TSharedPtr<FSampleTrack<TArray<FRigElementKey>>> ConnectorTrack = StaticCastSharedPtr<FSampleTrack<TArray<FRigElementKey>>>(Storage->FindOrAddTrack(ConnectorTrackName, FSampleTrackBase::ETrackType_ElementKeyArray));
			
				TArray<FRigElementKey> Targets;
				const FRigElementKeyRedirector Redirector = ControlRig->GetElementKeyRedirector();
				if(const FRigElementKeyRedirector::FCachedKeyArray* Cache = Redirector.Find(ConnectorElement->GetKey()))
				{
					Targets.Append(FRigElementKeyRedirector::Convert(*Cache));
				}
				ConnectorTrack->AddSample(Targets);
			}
		}
		return;
	}

	if(FRigCurveElement* Curve = Cast<FRigCurveElement>(InElement))
	{
		const TSharedPtr<FSampleTrack<float>> Track = StaticCastSharedPtr<FSampleTrack<float>>(Storage->FindOrAddTrack(TrackName, FSampleTrackBase::ETrackType_Float));
		Track->AddSample(InHierarchy->GetCurveValue(Curve));
		return;
	}
	
	TSharedPtr<FSampleTrack<FTransform3f>> Track = Storage->FindTrack<FTransform3f>(TrackName);
	if(!Track.IsValid())
	{
		Track = Storage->AddTransformTrack(TrackName);
		SampleTrackIndex.Update(*Storage);
	}
	const FTransform LocalTransform = InHierarchy->GetLocalTransform(InElement->GetIndex());
	Track->AddSample(FTransform3f(LocalTransform));
}

bool FControlRigReplayTracks::RestorePose(int32 InTimeIndex, URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys, TReportFunction InReportFunction) const
{
	return ForEachElement(InHierarchy, InKeys, [this, InTimeIndex, InHierarchy, InReportFunction](FRigBaseElement* InElement, bool& bSuccess)
	{
		if(!RestorePose(InTimeIndex, InHierarchy, InElement, InReportFunction))
		{
			bSuccess = false;
		}
	});
}

bool FControlRigReplayTracks::RestorePose(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, TReportFunction InReportFunction) const
{
	if(!InElement->IsA<FRigTransformElement>() && !InElement->IsA<FRigCurveElement>() && !InElement->IsA<FRigReferenceElement>())
	{
		return true;
	}

	const FSampleTrackContainer* Storage = GetContainer();
	const FName& TrackName = GetTrackName(InElement->GetKey());

	if(FRigCurveElement* Curve = Cast<FRigCurveElement>(InElement))
	{
		const TSharedPtr<const FSampleTrack<float>> Track = Storage->FindTrack<float>(TrackName);
		if(!Track)
		{
			if(InReportFunction)
			{
				InReportFunction(EMessageSeverity::Warning, TrackName, TEXT("Track not found."));
			}
			return false;
		}
		InHierarchy->SetCurveValue(Curve, Track->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex));
		return true;
	}
	
	if(const TSharedPtr<const FSampleTrack<FTransform3f>> Track = Storage->FindTrack<FTransform3f>(TrackName))
	{
		const FTransform3f& Transform = Track->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
		InHierarchy->SetLocalTransform(InElement->GetIndex(), FTransform(Transform));
	}
	else
	{
		if(InReportFunction)
		{
			InReportFunction(EMessageSeverity::Warning, TrackName, TEXT("Track not found."));
		}
		return false;
	}
	return true;
}

void FControlRigReplayTracks::StoreComponents(URigHierarchy* InHierarchy, const TArrayView<FRigComponentKey>& InKeys)
{
	ForEachComponent(InHierarchy, InKeys, [this, InHierarchy](FRigBaseComponent* InComponent, bool& bSuccess)
	{
		StoreComponent(InHierarchy, InComponent);
	});
}

void FControlRigReplayTracks::StoreComponents(URigHierarchy* InHierarchy, FRigBaseElement* InElement)
{
	TArray<FRigComponentKey> ComponentKeys = InElement->GetComponentKeys();
	if(!ComponentKeys.IsEmpty())
	{
		ForEachComponent(InHierarchy, ComponentKeys, [this, InHierarchy](FRigBaseComponent* InComponent, bool& bSuccess)
		{
			StoreComponent(InHierarchy, InComponent);
		});
	}
}

void FControlRigReplayTracks::StoreComponent(URigHierarchy* InHierarchy, FRigBaseComponent* InComponent)
{
	FSampleTrackContainer* Storage = GetContainer();
	const FName& TrackName = GetTrackName(InComponent->GetKey());
	TSharedPtr<FSampleTrack<FInstancedStruct>> Track = Storage->FindTrack<FInstancedStruct>(TrackName);
	if(Track)
	{
		check(Track->GetTrackType() == FSampleTrackBase::ETrackType_Struct);
		check(Track->GetScriptStruct() == InComponent->GetScriptStruct());
	}
	else
	{
		Track = Storage->AddStructTrack(TrackName, InComponent->GetScriptStruct());
		SampleTrackIndex.Update(*Storage);
	}
	FInstancedStruct Struct(InComponent->GetScriptStruct());
	Struct.GetScriptStruct()->CopyScriptStruct(Struct.GetMutableMemory(), InComponent);
	Track->AddSample(Struct);
}

bool FControlRigReplayTracks::RestoreComponents(int32 InTimeIndex, URigHierarchy* InHierarchy, const TArrayView<FRigComponentKey>& InKeys, TReportFunction InReportFunction) const
{
	return ForEachComponent(InHierarchy, InKeys, [this, InTimeIndex, InHierarchy, InReportFunction](FRigBaseComponent* InComponent, bool& bSuccess)
	{
		if(!RestoreComponent(InTimeIndex, InHierarchy, InComponent, InReportFunction))
		{
			bSuccess = false;
		}
	});
}

bool FControlRigReplayTracks::RestoreComponents(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, TReportFunction InReportFunction) const
{
	TArray<FRigComponentKey> ComponentKeys = InElement->GetComponentKeys();
	if(!ComponentKeys.IsEmpty())
	{
		return ForEachComponent(InHierarchy, ComponentKeys, [this, InTimeIndex, InHierarchy, InReportFunction](FRigBaseComponent* InComponent, bool& bSuccess)
		{
			if(!RestoreComponent(InTimeIndex, InHierarchy, InComponent, InReportFunction))
			{
				bSuccess = false;
			}
		});
	}
	return true;
}

bool FControlRigReplayTracks::RestoreComponent(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseComponent* InComponent, TReportFunction InReportFunction) const
{
	const FSampleTrackContainer* Storage = GetContainer();
	const FName& TrackName = GetTrackName(InComponent->GetKey());
	if(const TSharedPtr<const FSampleTrack<FInstancedStruct>> Track = Storage->FindTrack<FInstancedStruct>(TrackName))
	{
		if(Track->GetScriptStruct() != InComponent->GetScriptStruct())
		{
			if(InReportFunction)
			{
				InReportFunction(EMessageSeverity::Error, TrackName, TEXT("Component doesn't match track scriptstruct"));
			}
			return false;
		}
		TGuardValue<int32> GuardIndexInHierarchy(InComponent->IndexInHierarchy, InComponent->IndexInHierarchy);
		TGuardValue<int32> GuardIndexInElement(InComponent->IndexInElement, InComponent->IndexInElement);
		TGuardValue<bool> GuardSelected(InComponent->bSelected, InComponent->bSelected);

		const FInstancedStruct& Struct = Track->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
		Struct.GetScriptStruct()->CopyScriptStruct(InComponent, Struct.GetMemory());
	}
	else
	{
		if(InReportFunction)
		{
			InReportFunction(EMessageSeverity::Warning, TrackName, TEXT("Track not found."));
		}
		return false;
	}
	return true;
}

void FControlRigReplayTracks::StoreMetaData(URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys)
{
	ForEachElement(InHierarchy, InKeys, [this, InHierarchy](FRigBaseElement* InElement, bool& bSuccess)
	{
		StoreMetaData(InHierarchy, InElement);
	});
}

void FControlRigReplayTracks::StoreMetaData(URigHierarchy* InHierarchy, FRigBaseElement* InElement)
{
	if(!InHierarchy->HasMetadata(InElement))
	{
		return;
	}
	
	FSampleTrackContainer* Storage = GetContainer();
	const TArray<FName> MetadataNames = InHierarchy->GetMetadataNames(InElement->GetKey());
	const FName& TrackName = GetTrackName(InElement->GetKey());
	const FName MetadataNamesTrackName = *(TrackName.ToString() + TEXT("MetadataNames"));
	TSharedPtr<FSampleTrack<TArray<FName>>> Track = StaticCastSharedPtr<FSampleTrack<TArray<FName>>>(Storage->FindOrAddTrack(MetadataNamesTrackName, FSampleTrackBase::ETrackType_NameArray));
	if(Track)
	{
		Track->AddSample(MetadataNames);
	}

	for(const FName& MetadataName : MetadataNames)
	{
		if(FRigBaseMetadata* Metadata = InElement->GetMetadata(MetadataName))
		{
			StoreMetaData(InHierarchy, InElement, Metadata);
		}
	}
}

void FControlRigReplayTracks::StoreMetaData(URigHierarchy* InHierarchy, FRigBaseElement* InElement, FRigBaseMetadata* InMetadata)
{
	FSampleTrackContainer* Storage = GetContainer();
	const FName TrackName = GetTrackName(InElement->GetKey(), InMetadata->GetName());
	const FSampleTrackBase::ETrackType TrackType = GetTrackTypeFromMetadataType(InMetadata->GetType());
	TSharedPtr<FSampleTrackBase> Track = Storage->FindTrack(TrackName);
	if(Track)
	{
		check(Track->GetTrackType() == TrackType);
	}
	else
	{
		Track = Storage->AddTrack(TrackName, TrackType);
		SampleTrackIndex.Update(*Storage);
	}
	
	switch(InMetadata->GetType())
	{
		case ERigMetadataType::Bool:
		{
			const bool& Value = static_cast<FRigBoolMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<bool>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::BoolArray:
		{
			const TArray<bool>& Value = static_cast<FRigBoolArrayMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<TArray<bool>>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::Float:
		{
			const float& Value = static_cast<FRigFloatMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<float>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::FloatArray:
		{
			const TArray<float>& Value = static_cast<FRigFloatArrayMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<TArray<float>>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::Int32:
		{
			const int32& Value = static_cast<FRigInt32Metadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<int32>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::Int32Array:
		{
			const TArray<int32>& Value = static_cast<FRigInt32ArrayMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<TArray<int32>>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::Name:
		{
			const FName& Value = static_cast<FRigNameMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<FName>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::NameArray:
		{
			const TArray<FName>& Value = static_cast<FRigNameArrayMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<TArray<FName>>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::Vector:
		{
			const FVector& Value = static_cast<FRigVectorMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<FVector3f>>(Track)->AddSample(FVector3f(Value));
			break;
		}
		case ERigMetadataType::VectorArray:
		{
			const TArray<FVector>& Value = static_cast<FRigVectorArrayMetadata*>(InMetadata)->GetValue();
			TArray<FVector3f> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FVector& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue);
			}
			StaticCastSharedPtr<FSampleTrack<TArray<FVector3f>>>(Track)->AddSample(ConvertedValue);
			break;
		}
		case ERigMetadataType::Rotator:
		{
			const FRotator& Value = static_cast<FRigRotatorMetadata*>(InMetadata)->GetValue();
			StaticCastSharedPtr<FSampleTrack<FVector3f>>(Track)->AddSample(FVector3f(Value.Euler()));
			break;
		}
		case ERigMetadataType::RotatorArray:
		{
			const TArray<FRotator>& Value = static_cast<FRigRotatorArrayMetadata*>(InMetadata)->GetValue();
			TArray<FVector3f> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FRotator& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue.Euler());
			}
			StaticCastSharedPtr<FSampleTrack<TArray<FVector3f>>>(Track)->AddSample(ConvertedValue);
			break;
		}
		case ERigMetadataType::Quat:
		{
			const FQuat& Value = static_cast<FRigQuatMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<FQuat4f>>(Track)->AddSample(FQuat4f(Value));
			break;
		}
		case ERigMetadataType::QuatArray:
		{
			const TArray<FQuat>& Value = static_cast<FRigQuatArrayMetadata*>(InMetadata)->GetValue(); 
			TArray<FQuat4f> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FQuat& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue);
			}
			StaticCastSharedPtr<FSampleTrack<TArray<FQuat4f>>>(Track)->AddSample(ConvertedValue);
			break;
		}
		case ERigMetadataType::Transform:
		{
			const FTransform& Value = static_cast<FRigTransformMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<FTransform3f>>(Track)->AddSample(FTransform3f(Value));
			break;
		}
		case ERigMetadataType::TransformArray:
		{
			const TArray<FTransform>& Value = static_cast<FRigTransformArrayMetadata*>(InMetadata)->GetValue(); 
			TArray<FTransform3f> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FTransform& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue);
			}
			StaticCastSharedPtr<FSampleTrack<TArray<FTransform3f>>>(Track)->AddSample(ConvertedValue);
			break;
		}
		case ERigMetadataType::LinearColor:
		{
			const FLinearColor& Value = static_cast<FRigLinearColorMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<FLinearColor>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::LinearColorArray:
		{
			const TArray<FLinearColor>& Value = static_cast<FRigLinearColorArrayMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<TArray<FLinearColor>>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::RigElementKey:
		{
			const FRigElementKey& Value = static_cast<FRigElementKeyMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<FRigElementKey>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::RigElementKeyArray:
		{
			const TArray<FRigElementKey>& Value = static_cast<FRigElementKeyArrayMetadata*>(InMetadata)->GetValue(); 
			StaticCastSharedPtr<FSampleTrack<TArray<FRigElementKey>>>(Track)->AddSample(Value);
			break;
		}
		case ERigMetadataType::Invalid:
		{
			break;
		}
	}
}

bool FControlRigReplayTracks::RestoreMetaData(int32 InTimeIndex, URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys, TReportFunction InReportFunction) const
{
	return ForEachElement(InHierarchy, InKeys, [this, InTimeIndex, InHierarchy, InReportFunction](FRigBaseElement* InElement, bool& bSuccess)
	{
		if(!RestoreMetaData(InTimeIndex, InHierarchy, InElement, InReportFunction))
		{
			bSuccess = false;
		}
	});
}

bool FControlRigReplayTracks::RestoreMetaData(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, TReportFunction InReportFunction) const
{
	const TArray<FName>& StoredMetadataNames = GetMetadataNames(InTimeIndex, SampleTrackIndex, InElement->GetKey(), InReportFunction);
	
	bool bSuccess = true;
	for(const FName& StoredMetadataName : StoredMetadataNames)
	{
		if(!RestoreMetaData(InTimeIndex, InHierarchy, InElement, StoredMetadataName, InReportFunction))
		{
			bSuccess = false;
		}
	}
	return bSuccess;
}

const TArray<FName> FControlRigReplayTracks::GetMetadataNames(int32 InTimeIndex, FSampleTrackIndex& OutSampleTrackIndex, const FRigElementKey& InElementKey, TReportFunction InReportFunction) const
{
	const FSampleTrackContainer* Storage = GetContainer();
	static const TArray<FName> EmptyMetadataNames;

	const FName& TrackName = GetTrackName(InElementKey);
	const FName MetadataNamesTrackName = *(TrackName.ToString() + TEXT("MetadataNames"));
	const TSharedPtr<const FSampleTrack<TArray<FName>>> Track = Storage->FindTrack<TArray<FName>>(MetadataNamesTrackName);
	if(!Track.IsValid())
	{
		return EmptyMetadataNames;
	}
	if(Track->GetTrackType() != FSampleTrackBase::ETrackType_NameArray)
	{
		if(InReportFunction)
		{
			InReportFunction(EMessageSeverity::Error, TrackName, TEXT("Track has incorrect type."));
		}
		return EmptyMetadataNames;
	}

	return Track->GetValueAtTimeIndex(InTimeIndex, OutSampleTrackIndex);
}

bool FControlRigReplayTracks::RestoreMetaData(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, const FName& InMetadataName, TReportFunction InReportFunction) const
{
	const FSampleTrackContainer* Storage = GetContainer();
	const FName TrackName = GetTrackName(InElement->GetKey(), InMetadataName);
	const TSharedPtr<const FSampleTrackBase> Track = Storage->FindTrack(TrackName);
	if(!Track.IsValid())
	{
		if(InReportFunction)
		{
			InReportFunction(EMessageSeverity::Warning, TrackName, TEXT("Track not found."));
		}
		return false;
	}

	const ERigMetadataType ExpectedMetadataType = GetMetadataTypeFromTrackType(Track->GetTrackType());
	FRigBaseMetadata* Metadata = InElement->GetMetadata(InMetadataName);
	if(Metadata == nullptr)
	{
		if(ExpectedMetadataType != ERigMetadataType::Invalid)
		{
			Metadata = InHierarchy->GetMetadataForElement(InElement, InMetadataName, ExpectedMetadataType, true);
		}
		else
		{
			if(InReportFunction)
			{
				InReportFunction(EMessageSeverity::Warning, TrackName, TEXT("Cannot create metadata. Invalid metadata type."));
			}
			return false;
		}
	}
	else if(Metadata->GetType() != ExpectedMetadataType)
	{
		if(InReportFunction)
		{
			InReportFunction(EMessageSeverity::Error, TrackName, TEXT("Track has incorrect type."));
		}
		return false;
	}
	
	switch(Metadata->GetType())
	{
		case ERigMetadataType::Bool:
		{
			const bool& Value = StaticCastSharedPtr<const FSampleTrack<bool>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigBoolMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::BoolArray:
		{
			const TArray<bool>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<bool>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigBoolArrayMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::Float:
		{
			const float& Value = StaticCastSharedPtr<const FSampleTrack<float>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigFloatMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::FloatArray:
		{
			const TArray<float>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<float>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigFloatArrayMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::Int32:
		{
			const int32& Value = StaticCastSharedPtr<const FSampleTrack<int32>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigInt32Metadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::Int32Array:
		{
			const TArray<int32>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<int32>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigInt32ArrayMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::Name:
		{
			const FName& Value = StaticCastSharedPtr<const FSampleTrack<FName>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigNameMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::NameArray:
		{
			const TArray<FName>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FName>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigNameArrayMetadata*>(Metadata)->SetValue(Value);
			break;
		}
		case ERigMetadataType::Vector:
		{
			const FVector3f& Value = StaticCastSharedPtr<const FSampleTrack<FVector3f>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigVectorMetadata*>(Metadata)->SetValue(FVector(Value));
			break;
		}
		case ERigMetadataType::VectorArray:
		{
			const TArray<FVector3f>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FVector3f>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			TArray<FVector> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FVector3f& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue);
			}
			static_cast<FRigVectorArrayMetadata*>(Metadata)->SetValue(ConvertedValue);
			break;
		}
		case ERigMetadataType::Rotator:
		{
			const FVector3f& Value = StaticCastSharedPtr<const FSampleTrack<FVector3f>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigRotatorMetadata*>(Metadata)->SetValue(FRotator::MakeFromEuler(FVector(Value)));
			break;
		}
		case ERigMetadataType::RotatorArray:
		{
			const TArray<FVector3f>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FVector3f>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			TArray<FRotator> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FVector3f& SingleValue : Value)
			{
				ConvertedValue.Add(FRotator::MakeFromEuler(FVector(SingleValue)));
			}
			static_cast<FRigRotatorArrayMetadata*>(Metadata)->SetValue(ConvertedValue);
			break;
		}
		case ERigMetadataType::Quat:
		{
			const FQuat4f& Value = StaticCastSharedPtr<const FSampleTrack<FQuat4f>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigQuatMetadata*>(Metadata)->SetValue(FQuat(Value)); 
			break;
		}
		case ERigMetadataType::QuatArray:
		{
			const TArray<FQuat4f>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FQuat4f>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			TArray<FQuat> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FQuat4f& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue);
			}
			static_cast<FRigQuatArrayMetadata*>(Metadata)->SetValue(ConvertedValue); 
			break;
		}
		case ERigMetadataType::Transform:
		{
			const FTransform3f& Value = StaticCastSharedPtr<const FSampleTrack<FTransform3f>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigTransformMetadata*>(Metadata)->SetValue(FTransform(Value)); 
			break;
		}
		case ERigMetadataType::TransformArray:
		{
			const TArray<FTransform3f>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FTransform3f>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			TArray<FTransform> ConvertedValue;
			ConvertedValue.Reserve(Value.Num());
			for(const FTransform3f& SingleValue : Value)
			{
				ConvertedValue.Emplace(SingleValue);
			}
			static_cast<FRigTransformArrayMetadata*>(Metadata)->SetValue(ConvertedValue); 
			break;
		}
		case ERigMetadataType::LinearColor:
		{
			const FLinearColor& Value = StaticCastSharedPtr<const FSampleTrack<FLinearColor>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigLinearColorMetadata*>(Metadata)->SetValue(Value); 
			break;
		}
		case ERigMetadataType::LinearColorArray:
		{
			const TArray<FLinearColor>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FLinearColor>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigLinearColorArrayMetadata*>(Metadata)->SetValue(Value); 
			break;
		}
		case ERigMetadataType::RigElementKey:
		{
			const FRigElementKey& Value = StaticCastSharedPtr<const FSampleTrack<FRigElementKey>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigElementKeyMetadata*>(Metadata)->SetValue(Value); 
			break;
		}
		case ERigMetadataType::RigElementKeyArray:
		{
			const TArray<FRigElementKey>& Value = StaticCastSharedPtr<const FSampleTrack<TArray<FRigElementKey>>>(Track)->GetValueAtTimeIndex(InTimeIndex, SampleTrackIndex);
			static_cast<FRigElementKeyArrayMetadata*>(Metadata)->SetValue(Value); 
			break;
		}
		case ERigMetadataType::Invalid:
		{
			if(InReportFunction)
			{
				InReportFunction(EMessageSeverity::Error, TrackName, TEXT("Unsupported Metadata Type."));
			}
			return false;
		}
	}
	return true;
}

void FControlRigReplayTracks::StoreVariables(URigVMHost* InHost)
{
	TArray<FName> VariableNames;
	
	FSampleTrackContainer* Storage = GetContainer();
	for (TFieldIterator<FProperty> PropertyIt(InHost->GetClass()); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		if(Property->IsNative())
		{
			continue;
		}

		const UScriptStruct* ScriptStruct = nullptr;
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			ScriptStruct = StructProperty->Struct;
		}
		else if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			if(const FStructProperty* InnerStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
			{
				ScriptStruct = InnerStructProperty->Struct;
			}
		}

		const FName& TrackName = GetTrackName(Property);
		const FSampleTrackBase::ETrackType TrackType = GetTrackTypeFromProperty(Property);
		TSharedPtr<FSampleTrackBase> Track = Storage->FindOrAddTrack(TrackName, TrackType, ScriptStruct);
		const uint8* Memory = Property->ContainerPtrToValuePtr<uint8>(InHost);
		Track->AddSampleFromProperty(Property, Memory);

		VariableNames.Add(Property->GetFName());
	}

	if(GetNumTimes() == 1 && !VariableNames.IsEmpty())
	{
		const TSharedPtr<FSampleTrack<TArray<FName>>> Track = StaticCastSharedPtr<FSampleTrack<TArray<FName>>>(Storage->FindOrAddTrack(VariableNamesName, FSampleTrackBase::ETrackType_NameArray, nullptr));
		Track->AddSample(VariableNames);
	}
}

bool FControlRigReplayTracks::RestoreVariables(int32 InTimeIndex, URigVMHost* InHost, TReportFunction InReportFunction) const
{
	const FSampleTrackContainer* Storage = GetContainer();
	bool bSuccess = true;
	for (TFieldIterator<FProperty> PropertyIt(InHost->GetClass()); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		if(Property->IsNative())
		{
			continue;
		}

		const FName& TrackName = GetTrackName(Property);
		const FSampleTrackBase::ETrackType TrackType = GetTrackTypeFromProperty(Property);
		const TSharedPtr<const FSampleTrackBase> Track = Storage->FindTrack(TrackName);
		if(Track)
		{
			check(Track->GetTrackType() == TrackType);
			uint8* Memory = Property->ContainerPtrToValuePtr<uint8>(InHost);
			Track->GetSampleForProperty(InTimeIndex, SampleTrackIndex, Property, Memory);
		}
		else
		{
			if(InReportFunction)
			{
				InReportFunction(EMessageSeverity::Warning, TrackName, TEXT("Track not found."));
			}
			bSuccess = false;
		}
	}
	return bSuccess;
}

bool FControlRigReplayTracks::ForEachElement(URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys, TFunction<void(FRigBaseElement*, bool&)> InFunction)
{
	bool bSuccess = true;
	if(InKeys.IsEmpty())
	{
		InHierarchy->Traverse([InFunction, &bSuccess](FRigBaseElement* InElement, bool& bContinue)
		{
			InFunction(InElement, bSuccess);
			bContinue = true;
		}, true);
	}
	else
	{
		for(const FRigElementKey& Key : InKeys)
		{
			if(FRigBaseElement* Element = InHierarchy->Find(Key))
			{
				InFunction(Element, bSuccess);
			}
		}
	}
	return bSuccess;
}

bool FControlRigReplayTracks::ForEachComponent(URigHierarchy* InHierarchy, const TArrayView<FRigComponentKey>& InKeys,
	TFunction<void(FRigBaseComponent*, bool&)> InFunction)
{
	TArrayView<FRigComponentKey> View = InKeys;
	TArray<FRigComponentKey> AllComponentKeys;
	if(View.IsEmpty())
	{
		AllComponentKeys = InHierarchy->GetAllComponentKeys();
		View = TArrayView<FRigComponentKey>(AllComponentKeys);
	}

	bool bSuccess = true;
	for(const FRigComponentKey& Key : View)
	{
		if(FRigBaseComponent* Component = InHierarchy->FindComponent(Key))
		{
			InFunction(Component, bSuccess);
		}
	}
	return bSuccess;
}

void FControlRigReplayTracks::FilterElementKeys(TArray<FRigElementKey>& InOutElementKeys)
{
	InOutElementKeys.RemoveAll([](const FRigElementKey& ElementKey) -> bool
	{
		return ElementKey.Type == ERigElementType::Reference;
	});
}

const FName& FControlRigReplayTracks::GetTrackName(const FRigElementKey& InElementKey) const
{
	if(const FName* ExistingTrackName = ElementKeyToTrackName.Find(InElementKey))
	{
		return *ExistingTrackName;
	}
	const FName TrackName = *InElementKey.ToString();
	return ElementKeyToTrackName.Add(InElementKey, TrackName);
}

const FName& FControlRigReplayTracks::GetTrackName(const FRigComponentKey& InComponentKey) const
{
	if(const FName* ExistingTrackName = ComponentKeyToTrackName.Find(InComponentKey))
	{
		return *ExistingTrackName;
	}
	const FName TrackName = *InComponentKey.ToString();
	return ComponentKeyToTrackName.Add(InComponentKey, TrackName);
}

const FName& FControlRigReplayTracks::GetTrackName(const FRigElementKey& InElementKey, const FName& InMetadataName) const
{
	const TTuple<FRigElementKey,FName> MetadataKey(InElementKey, InMetadataName);
	if(const FName* ExistingTrackName = MetadataToTrackName.Find(MetadataKey))
	{
		return *ExistingTrackName;
	}
	TStringBuilder<256> Builder;
	Builder.Append(InElementKey.ToString());
	Builder.AppendChar(TEXT(':'));
	Builder.Append(InMetadataName.ToString());
	const FName TrackName(Builder);
	return MetadataToTrackName.Add(MetadataKey, TrackName);
}

const FName& FControlRigReplayTracks::GetTrackName(const FProperty* InProperty) const
{
	check(InProperty);
	if(const FName* ExistingTrackName = PropertyNameToTrackName.Find(InProperty->GetFName()))
	{
		return *ExistingTrackName;
	}
	TStringBuilder<256> Builder;
	Builder.Append(TEXT("Variable:"));
	Builder.Append(InProperty->GetName());
	const FName TrackName(Builder);
	return PropertyNameToTrackName.Add(InProperty->GetFName(), TrackName);
}

FSampleTrackBase::ETrackType FControlRigReplayTracks::GetTrackTypeFromMetadataType(ERigMetadataType InMetadataType)
{
	switch(InMetadataType)
	{
		case ERigMetadataType::Bool:
		{
			return FSampleTrackBase::ETrackType_Bool;
		}
		case ERigMetadataType::BoolArray:
		{
			return FSampleTrackBase::ETrackType_BoolArray;
		}
		case ERigMetadataType::Float:
		{
			return FSampleTrackBase::ETrackType_Float;
		}
		case ERigMetadataType::FloatArray:
		{
			return FSampleTrackBase::ETrackType_FloatArray;
		}
		case ERigMetadataType::Int32:
		{
			return FSampleTrackBase::ETrackType_Int32;
		}
		case ERigMetadataType::Int32Array:
		{
			return FSampleTrackBase::ETrackType_Int32Array;
		}
		case ERigMetadataType::Name:
		{
			return FSampleTrackBase::ETrackType_Name;
		}
		case ERigMetadataType::NameArray:
		{
			return FSampleTrackBase::ETrackType_NameArray;
		}
		case ERigMetadataType::Vector:
		case ERigMetadataType::Rotator:
		{
			return FSampleTrackBase::ETrackType_Vector3f;
		}
		case ERigMetadataType::VectorArray:
		case ERigMetadataType::RotatorArray:
		{
			return FSampleTrackBase::ETrackType_Vector3fArray;
		}
		case ERigMetadataType::Quat:
		{
			return FSampleTrackBase::ETrackType_Quatf;
		}
		case ERigMetadataType::QuatArray:
		{
			return FSampleTrackBase::ETrackType_QuatfArray;
		}
		case ERigMetadataType::Transform:
		{
			return FSampleTrackBase::ETrackType_Transformf;
		}
		case ERigMetadataType::TransformArray:
		{
			return FSampleTrackBase::ETrackType_TransformfArray;
		}
		case ERigMetadataType::LinearColor:
		{
			return FSampleTrackBase::ETrackType_LinearColor;
		}
		case ERigMetadataType::LinearColorArray:
		{
			return FSampleTrackBase::ETrackType_LinearColorArray;
		}
		case ERigMetadataType::RigElementKey:
		{
			return FSampleTrackBase::ETrackType_ElementKey;
		}
		case ERigMetadataType::RigElementKeyArray:
		{
			return FSampleTrackBase::ETrackType_ElementKeyArray;
		}
		case ERigMetadataType::Invalid:
		{
			break;
		}
	}
	return FSampleTrackBase::ETrackType_Unknown;
}

ERigMetadataType FControlRigReplayTracks::GetMetadataTypeFromTrackType(FSampleTrackBase::ETrackType InTrackType)
{
	switch(InTrackType)
	{
		case FSampleTrackBase::ETrackType_Bool:
		{
			return ERigMetadataType::Bool;
		}
		case FSampleTrackBase::ETrackType_BoolArray:
		{
			return ERigMetadataType::BoolArray;
		}
		case FSampleTrackBase::ETrackType_Float:
		{
			return ERigMetadataType::Float;
		}
		case FSampleTrackBase::ETrackType_FloatArray:
		{
			return ERigMetadataType::FloatArray;
		}
		case FSampleTrackBase::ETrackType_Int32:
		{
			return ERigMetadataType::Int32;
		}
		case FSampleTrackBase::ETrackType_Int32Array:
		{
			return ERigMetadataType::Int32Array;
		}
		case FSampleTrackBase::ETrackType_Name:
		{
			return ERigMetadataType::Name;
		}
		case FSampleTrackBase::ETrackType_NameArray:
		{
			return ERigMetadataType::NameArray;
		}
		case FSampleTrackBase::ETrackType_Vector3f:
		{
			return ERigMetadataType::Vector;
		}
		case FSampleTrackBase::ETrackType_Vector3fArray:
		{
			return ERigMetadataType::VectorArray;
		}
		case FSampleTrackBase::ETrackType_Quatf:
		{
			return ERigMetadataType::Quat;
		}
		case FSampleTrackBase::ETrackType_QuatfArray:
		{
			return ERigMetadataType::QuatArray;
		}
		case FSampleTrackBase::ETrackType_Transformf:
		{
			return ERigMetadataType::Transform;
		}
		case FSampleTrackBase::ETrackType_TransformfArray:
		{
			return ERigMetadataType::TransformArray;
		}
		case FSampleTrackBase::ETrackType_LinearColor:
		{
			return ERigMetadataType::LinearColor;
		}
		case FSampleTrackBase::ETrackType_LinearColorArray:
		{
			return ERigMetadataType::LinearColorArray;
		}
		case FSampleTrackBase::ETrackType_ElementKey:
		{
			return ERigMetadataType::RigElementKey;
		}
		case FSampleTrackBase::ETrackType_ElementKeyArray:
		{
			return ERigMetadataType::RigElementKeyArray;
		}
		case FSampleTrackBase::ETrackType_Uint32:
		case FSampleTrackBase::ETrackType_String:
		case FSampleTrackBase::ETrackType_ComponentKey:
		case FSampleTrackBase::ETrackType_Struct:
		case FSampleTrackBase::ETrackType_Uint32Array:
		case FSampleTrackBase::ETrackType_StringArray:
		case FSampleTrackBase::ETrackType_ComponentKeyArray:
		case FSampleTrackBase::ETrackType_StructArray:
		case FSampleTrackBase::ETrackType_Unknown:
		{
			break;
		}
	}
	return ERigMetadataType::Invalid;
}

FSampleTrackBase::ETrackType FControlRigReplayTracks::GetTrackTypeFromProperty(const FProperty* InProperty)
{
	check(InProperty);

	static constexpr uint8 SingleToArrayOffset = uint8(FSampleTrackBase::ETrackType_BoolArray)-uint8(FSampleTrackBase::ETrackType_Bool);

	bool bIsArray = false;
	if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		InProperty = ArrayProperty->Inner;
		bIsArray = true;
	}

	FSampleTrackBase::ETrackType TrackType = FSampleTrackBase::ETrackType_Unknown; 
	
	if(InProperty->IsA<FBoolProperty>())
	{
		TrackType = FSampleTrackBase::ETrackType_Bool;
	}
	else if(InProperty->IsA<FFloatProperty>() || InProperty->IsA<FDoubleProperty>())
	{
		TrackType = FSampleTrackBase::ETrackType_Float;
	}
	else if(InProperty->IsA<FIntProperty>() || InProperty->IsA<FInt16Property>())
	{
		TrackType = FSampleTrackBase::ETrackType_Int32;
	}
	else if(InProperty->IsA<FUInt32Property>() || InProperty->IsA<FUInt16Property>() ||  InProperty->IsA<FByteProperty>() || InProperty->IsA<FEnumProperty>())
	{
		TrackType = FSampleTrackBase::ETrackType_Uint32;
	}
	else if(InProperty->IsA<FNameProperty>())
	{
		TrackType = FSampleTrackBase::ETrackType_Name;
	}
	else if(InProperty->IsA<FStrProperty>())
	{
		TrackType = FSampleTrackBase::ETrackType_String;
	}
	else if(InProperty->IsA<FStructProperty>())
	{
		TrackType = FSampleTrackBase::ETrackType_Struct;
	}

	if(bIsArray)
	{
		TrackType = (FSampleTrackBase::ETrackType)(uint8(TrackType) + SingleToArrayOffset);
	}

	return TrackType;
}

const TArray<FRigElementKey> FControlRigReplayTracks::GetElementKeys() const
{
	const TSharedPtr<const FSampleTrack<TArray<FRigElementKey>>> ElementKeysTrack = GetContainer()->FindTrack<TArray<FRigElementKey>>(ElementKeysName);
	if(ElementKeysTrack)
	{
		static FSampleTrackIndex SingletonTrackIndex = FSampleTrackIndex::MakeSingleton();
		return ElementKeysTrack->GetValueAtTimeIndex(0, SingletonTrackIndex);
	}
	static const TArray<FRigElementKey> EmptyElementKeys;
	return EmptyElementKeys;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// UControlRigReplay
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const FText UControlRigReplay::LiveStatus = LOCTEXT("LiveStatus", "Live"); 
const FText UControlRigReplay::LiveStatusTooltip = LOCTEXT("LiveStatusTooltip", "The replay is not affecting the rig."); 
const FText UControlRigReplay::ReplayInputsStatus = LOCTEXT("ReplayInputsStatus", "Replay Inputs");
const FText UControlRigReplay::ReplayInputsStatusTooltip = LOCTEXT("ReplayInputsStatusTooltip", "The replay's input data is applied first, then the rig runs.");
const FText UControlRigReplay::GroundTruthStatus = LOCTEXT("GroundTruthStatus", "Ground Truth");
const FText UControlRigReplay::GroundTruthStatusTooltip = LOCTEXT("GroundTruthStatusTooltip", "The results from the replay override the rig completely.");

void UControlRigReplay::BeginDestroy()
{
	UObject::BeginDestroy();
}

void UControlRigReplay::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);
	UObject::Serialize(Ar);
}

UControlRigReplay* UControlRigReplay::CreateNewAsset(FString InDesiredPackagePath, FString InBlueprintPathName, UClass* InAssetClass)
{
#if WITH_EDITOR
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FString UniquePackageName;
	FString UniqueAssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(InDesiredPackagePath, TEXT(""), UniquePackageName, UniqueAssetName);

	if (UniquePackageName.EndsWith(UniqueAssetName))
	{
		UniquePackageName = UniquePackageName.LeftChop(UniqueAssetName.Len() + 1);
	}

	UObject* NewAsset = AssetToolsModule.Get().CreateAsset(*UniqueAssetName, *UniquePackageName, InAssetClass, nullptr);
	if(NewAsset)
	{
		// make sure the package is never cooked.
		UPackage* Package = NewAsset->GetOutermost();
		Package->SetPackageFlags(Package->GetPackageFlags() | PKG_EditorOnly);
			
		if(UControlRigReplay* TestData = Cast<UControlRigReplay>(NewAsset))
		{
			TestData->ControlRigObjectPath = InBlueprintPathName;
			return TestData;
		}
	}
#endif
	return nullptr;
}

FVector2D UControlRigReplay::GetTimeRange() const
{
	return FVector2D(OutputTracks.GetContainer()->GetTimeRange());
}

bool UControlRigReplay::StartRecording(UControlRig* InControlRig)
{
	if(InControlRig == nullptr)
	{
		return false;
	}

	(void)StopRecording();
	(void)StopReplay();

	InputTracks.Reset();
	OutputTracks.Reset();
	
	RecordControlRig = InControlRig;
	ClearDelegates(InControlRig);

	TimeAtStartOfRecording = TimeOfLastFrame = FPlatformTime::Seconds();
	bStoreVariablesDuringPreEvent = true;
	InControlRig->SetAbsoluteAndDeltaTime(0.f, InControlRig->GetDeltaTime());

	PreConstructionHandle = InControlRig->OnPreConstruction_AnyThread().AddLambda(
		[this](UControlRig* InControlRig, const FName& InEventName)
		{
			if (InEventName != FRigUnit_PrepareForExecution::EventName)
			{
				return;
			}
			// store the first frame of variables
			InputTracks.AddTimeSample(InControlRig->GetAbsoluteTime(), InControlRig->GetDeltaTime());
			InputTracks.StoreVariables(InControlRig);
			bStoreVariablesDuringPreEvent = false;
		}
	);

	PreEventHandle = InControlRig->OnPreExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig)
			{
				return;
			}

			if (InEventName == FRigUnit_PrepareForExecution::EventName ||
				InEventName == FRigUnit_PostPrepareForExecution::EventName)
			{
				return;
			}

			if(!ControlRig->SupportsEvent(InEventName))
			{
				return;
			}

			if(bStoreVariablesDuringPreEvent)
			{
				InputTracks.AddTimeSample(ControlRig->GetAbsoluteTime(), ControlRig->GetDeltaTime());
				InputTracks.StoreVariables(ControlRig);
			}
			InputTracks.StoreRigVMEvent(InEventName);
			InputTracks.StoreInteraction(ControlRig->InteractionType, ControlRig->ElementsBeingInteracted);
			InputTracks.StoreHierarchy(ControlRig->GetHierarchy());

			bStoreVariablesDuringPreEvent = true;
		}
	);

	PostEventHandle = InControlRig->OnExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig)
			{
				return;
			}
			
			if (InEventName == FRigUnit_PrepareForExecution::EventName ||
				InEventName == FRigUnit_PostPrepareForExecution::EventName)
			{
				return;
			}
			
			const double CurrentTime = FPlatformTime::Seconds(); 
			const double DisplayDeltaTime = CurrentTime - TimeOfLastFrame;
			TimeOfLastFrame = CurrentTime;

			OutputTracks.AddTimeSample(InputTracks.GetLastAbsoluteTime(), InputTracks.GetLastDeltaTime());
			OutputTracks.StoreVariables(ControlRig);
			OutputTracks.StoreHierarchy(ControlRig->GetHierarchy());
			const double RecordingDuration = OutputTracks.GetLastAbsoluteTime();

#if WITH_EDITOR
			static constexpr TCHAR PrintStringFormat[] = TEXT("Recorded time... %.02f");
			UKismetSystemLibrary::PrintString(ControlRig->GetWorld(), FString::Printf(PrintStringFormat, static_cast<float>(RecordingDuration)), true, false, FLinearColor::White, DisplayDeltaTime * 0.5f);
#endif

			if(DesiredRecordingDuration >= -KINDA_SMALL_NUMBER)
			{
				if(RecordingDuration >= DesiredRecordingDuration)
				{
					StopRecording();
				}
			}
		}
	);

	if(InputTracks.GetContainer()->NumTracks() == 0)
	{
		InControlRig->RequestInit();
	}
	
	return true;
}

bool UControlRigReplay::StopRecording()
{
	if(UControlRig* ControlRig = RecordControlRig.Get())
	{
		ClearDelegates(ControlRig);
		RecordControlRig.Reset();
		InputTracks.Compact();
		OutputTracks.Compact();
		TimeAtStartOfRecording = -1.0;
		DesiredRecordingDuration = -1.0;
		return true;
	}
	return false;
}

EControlRigReplayPlaybackMode UControlRigReplay::GetPlaybackMode() const
{
	if(IsReplaying())
	{
		return PlaybackMode;
	}
	return EControlRigReplayPlaybackMode::Live;
}

void UControlRigReplay::SetPlaybackMode(EControlRigReplayPlaybackMode InMode)
{
	if(InMode >= EControlRigReplayPlaybackMode::Max)
	{
		InMode = EControlRigReplayPlaybackMode::ReplayInputs;
	}

	if(PlaybackMode == InMode)
	{
		return;
	}

	if(InMode == EControlRigReplayPlaybackMode::Live)
	{
		StopReplay();
	}
	else
	{
		PlaybackMode = InMode; 
	}
}

bool UControlRigReplay::StartReplay(UControlRig* InControlRig, EControlRigReplayPlaybackMode InMode)
{
	if(IsReplaying() && ReplayControlRig.Get() == InControlRig)
	{
		if(InMode != GetPlaybackMode())
		{
			SetPlaybackMode(InMode);
			return true;
		}
		if(bReplayPaused)
		{
			bReplayPaused = false;
			return true;
		}
	}
	
	StopRecording();
	StopReplay();

	if(InMode == EControlRigReplayPlaybackMode::Live)
	{
		SetPlaybackMode(InMode);
		return true;
	}

	if(InControlRig == nullptr)
	{
		return false;
	}

	if(InputTracks.GetNumTimes() != OutputTracks.GetNumTimes())
	{
		return false;
	}

	bStoreVariablesDuringPreEvent = true;

	if(InputTracks.IsEmpty() || OutputTracks.IsEmpty())
	{
		return false;
	}

	InControlRig->SetReplay(this);
	
	PreConstructionHandle = InControlRig->OnPreConstruction_AnyThread().AddLambda(
		[this](UControlRig* InControlRig, const FName& InEventName)
		{
			if (InEventName != FRigUnit_PrepareForExecution::EventName)
			{
				return;
			}
			// restore the first set of variables
			InputTracks.RestoreVariables(0, InControlRig);
			bStoreVariablesDuringPreEvent = false;
		}
	);

	PreEventHandle = InControlRig->OnPreExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig
				|| InEventName == FRigUnit_PrepareForExecution::EventName
				|| InEventName == FRigUnit_PostPrepareForExecution::EventName)
			{
				return;
			}

			int32 ReplayTimeIndex = ControlRig->GetReplayTimeIndex();

			if(GetPlaybackMode() == EControlRigReplayPlaybackMode::ReplayInputs)
			{
				ReplayTimeIndex = FMath::Clamp(ReplayTimeIndex, 0, InputTracks.GetNumTimes() - 1);
				
				const FName EventNameFromReplay = InputTracks.GetRigVMEvent(ReplayTimeIndex);
				if (InEventName != EventNameFromReplay)
				{
					return;
				}

				const TTuple<uint8, TArray<FRigElementKey>> Interaction = InputTracks.GetInteraction(ReplayTimeIndex);
				ControlRig->InteractionType = Interaction.Get<0>();
				ControlRig->ElementsBeingInteracted = Interaction.Get<1>();

				ControlRig->SetAbsoluteAndDeltaTime(
					InputTracks.GetAbsoluteTime(ReplayTimeIndex),
					InputTracks.GetDeltaTime(ReplayTimeIndex)
				);

				if(bStoreVariablesDuringPreEvent)
				{
					InputTracks.RestoreVariables(ReplayTimeIndex, ControlRig);
				}
				InputTracks.RestoreHierarchy(ReplayTimeIndex, ControlRig->GetHierarchy());
			}
			bStoreVariablesDuringPreEvent = true;
		}
	);

	PostEventHandle = InControlRig->OnExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig
				 || InEventName == FRigUnit_PrepareForExecution::EventName
				 || InEventName == FRigUnit_PostPrepareForExecution::EventName)
			{
				return;
			}

			int32 ReplayTimeIndex = ControlRig->GetReplayTimeIndex();

			if(GetPlaybackMode() == EControlRigReplayPlaybackMode::ReplayInputs)
			{
				ReplayTimeIndex = FMath::Clamp(ReplayTimeIndex, 0, InputTracks.GetNumTimes() - 1);
				
				ControlRig->SetAbsoluteAndDeltaTime(
					InputTracks.GetAbsoluteTime(ReplayTimeIndex),
					InputTracks.GetDeltaTime(ReplayTimeIndex)
				);

				// only validate results during the first play through
				const FName EventNameFromReplay = InputTracks.GetRigVMEvent(ReplayTimeIndex);
				if(InEventName == EventNameFromReplay)
				{
					const int32 OutputTimeIndex = ReplayTimeIndex;

					(void)ValidateExpectedResults(OutputTimeIndex, OutputTracks.SampleTrackIndex, ControlRig,
						[ControlRig](EMessageSeverity::Type InSeverity, const FString InMessage)
						{
							switch(InSeverity)
							{
								case EMessageSeverity::Warning:
								case EMessageSeverity::PerformanceWarning:
								{
									UE_LOGF(LogControlRig, Warning, "%ls", *InMessage);
									break;
								}
								case EMessageSeverity::Error:
								{
									UE_LOGF(LogControlRig, Error, "%ls", *InMessage);
									break;
								}
								case EMessageSeverity::Info:
								default:
								{
									UE_LOGF(LogControlRig, Display, "%ls", *InMessage);
									break;
								}
							}
						}
					);
				}
				
				if(!bReplayPaused)
				{
					// loop the animation
					if(ReplayTimeIndex >= InputTracks.GetNumTimes() - 1)
					{
						ControlRig->SetReplayTimeIndex(0);
					}
					else
					{
						ControlRig->SetReplayTimeIndex(ReplayTimeIndex + 1);
					}
				}
			}
			else
			{
				ReplayTimeIndex = FMath::Clamp(ReplayTimeIndex, 0, OutputTracks.GetNumTimes() - 1);

				ControlRig->SetAbsoluteAndDeltaTime(
					OutputTracks.GetAbsoluteTime(ReplayTimeIndex),
					OutputTracks.GetDeltaTime(ReplayTimeIndex)
				);
				
				OutputTracks.RestoreVariables(ReplayTimeIndex, ControlRig);
				OutputTracks.RestoreHierarchy(ReplayTimeIndex, ControlRig->GetHierarchy());

				if(!bReplayPaused)
				{
					// loop the animation
					if(ReplayTimeIndex >= OutputTracks.GetNumTimes() - 1)
					{
						ControlRig->SetReplayTimeIndex(0);
					}
					else
					{
						ControlRig->SetReplayTimeIndex(ReplayTimeIndex + 1);
					}
				}
			}
		}
	);

	InControlRig->RequestInit();

	ReplayControlRig = InControlRig;
	SetPlaybackMode(InMode);

	return true;
}

bool UControlRigReplay::StopReplay()
{
	if(UControlRig* ControlRig = ReplayControlRig.Get())
	{
		ClearDelegates(ControlRig);
		ControlRig->DisableReplay();
		ControlRig->InteractionType = 0;
		ControlRig->ElementsBeingInteracted.Reset();
		ReplayControlRig.Reset();
		bReplayPaused = false;
		PlaybackMode = EControlRigReplayPlaybackMode::Live;
		return true;
	}
	return false;
}

bool UControlRigReplay::PauseReplay()
{
	if(!IsReplaying())
	{
		return false;
	}

	bReplayPaused = true;
	return false;
}

bool UControlRigReplay::IsReplaying() const
{
	return ReplayControlRig.IsValid();
}

bool UControlRigReplay::IsPaused() const
{
	return bReplayPaused;
}

bool UControlRigReplay::IsRecording() const
{
	return RecordControlRig.IsValid();
}

bool UControlRigReplay::IsValidForTesting() const
{
	return !InputTracks.IsEmpty() &&
		!OutputTracks.IsEmpty() &&
		InputTracks.GetNumTimes() == OutputTracks.GetNumTimes() &&
		ControlRigObjectPath.IsValid();
}

bool UControlRigReplay::HasValidationErrors() const
{
	return !LastValidationWarningsAndErrors.IsEmpty();
}

const TArray<FString>& UControlRigReplay::GetValidationErrors() const
{
	return LastValidationWarningsAndErrors;
}

bool UControlRigReplay::PerformTest(UControlRig* InSubject, TFunction<void(EMessageSeverity::Type, const FString&)> InLogFunction) const
{
	// if we have nothing to check we can consider this test successful
	if(InputTracks.IsEmpty() || OutputTracks.IsEmpty())
	{
		return true;
	}

	if(!FMath::IsNearlyEqual(InputTracks.GetAbsoluteTime(0), OutputTracks.GetAbsoluteTime(0)) ||
		!FMath::IsNearlyEqual(InputTracks.GetLastAbsoluteTime(), OutputTracks.GetLastAbsoluteTime()))
	{
		if(InLogFunction)
		{
			InLogFunction(EMessageSeverity::Error, TEXT("Test Replay is corrupt. Input and Output time ranges don't match."));
		}
		return false;
	}

	bool bSuccess = true;

	auto ReportFunction = [&bSuccess, InLogFunction](EMessageSeverity::Type Severity, const FName& Key, const FString& Message)
	{
		if(InLogFunction)
		{
			if(Key.IsNone())
			{
				InLogFunction(Severity, Message);
			}
			else
			{
				InLogFunction(Severity, FString::Printf(TEXT("%s: %s"), *Key.ToString(), *Message));
			}
		}
		if(Severity == EMessageSeverity::Error)
		{
			bSuccess = false;
		}
	};

	URigHierarchy* Hierarchy = InSubject->GetHierarchy();

	// set up the rig by restoring variables and then running construction 
	InSubject->RequestInit();
	InputTracks.RestoreVariables(0, InSubject, ReportFunction);

	InSubject->SetAbsoluteAndDeltaTime(InputTracks.GetAbsoluteTime(0), InputTracks.GetDeltaTime(0));
	
	InSubject->EventQueue = {FRigUnit_BeginExecution::EventName};

	// make sure to import the hierarchy the same way it is imported in the control rig editor and apply the connectors
	InSubject->OnPreConstruction_AnyThread().AddUObject(this, &UControlRigReplay::HandlePreconstructionForTest);
	
	InSubject->Evaluate_AnyThread();

	FSampleTrackIndex SampleTrackIndex(OutputTracks.GetContainer()->NumTracks());
	
	for(int32 InputTimeIndex = 0; InputTimeIndex < InputTracks.GetNumTimes(); InputTimeIndex++)
	{
		InSubject->EventQueue = {InputTracks.GetRigVMEvent(InputTimeIndex)};
		const TTuple<uint8, TArray<FRigElementKey>> Interaction = InputTracks.GetInteraction(InputTimeIndex);
		InSubject->InteractionType = Interaction.Get<0>();
		InSubject->ElementsBeingInteracted = Interaction.Get<1>();
		
		if(InputTimeIndex > 0)
		{
			InputTracks.RestoreVariables(InputTimeIndex, InSubject, ReportFunction);
		}
		
		InputTracks.RestoreHierarchy(InputTimeIndex, Hierarchy, {}, ReportFunction);
		
		InSubject->SetAbsoluteAndDeltaTime(
			InputTracks.GetAbsoluteTime(InputTimeIndex),
			InputTracks.GetDeltaTime(InputTimeIndex));
		
		InSubject->Evaluate_AnyThread();

		const int OutTimeIndex = InputTimeIndex;
		if(!ValidateExpectedResults(OutTimeIndex, SampleTrackIndex, InSubject, InLogFunction))
		{
			bSuccess = false;
		}
	}

	return bSuccess;
}

bool UControlRigReplay::ValidateExpectedResults(
	int32 InPlaybackTimeIndex,
	FSampleTrackIndex& OutSampleTrackIndex,
	UControlRig* InSubject,
	TFunction<void(EMessageSeverity::Type, const FString&)> InLogFunction) const
{
	URigHierarchy* Hierarchy = InSubject->GetHierarchy();
	const TArray<FRigElementKey>& StoredElementKeys = OutputTracks.GetElementKeys();
	LastValidationWarningsAndErrors.Reset();

	auto LocalLogFunction = [InLogFunction, this](EMessageSeverity::Type InSeverity, const FString& InMessage)
	{
		if(InLogFunction)
		{
			InLogFunction(InSeverity, InMessage);
		}
		if(InSeverity == EMessageSeverity::Warning || InSeverity == EMessageSeverity::Error)
		{
			LastValidationWarningsAndErrors.Add(InMessage);
		}
	};

	bool bSuccess = true;
	if(bValidateHierarchyTopology && InPlaybackTimeIndex == 0)
	{
		TArray<FRigElementKey> CurrentElementKeys = Hierarchy->GetAllKeys();
		FControlRigReplayTracks::FilterElementKeys(CurrentElementKeys);
		
		if(StoredElementKeys.Num() != CurrentElementKeys.Num())
		{
			LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Number of elements in hierarchy (%d) and number of elements in replay (%d) don't match."), CurrentElementKeys.Num(), StoredElementKeys.Num()));
			bSuccess = false;
		}

		for(const FRigElementKey& StoredKey : StoredElementKeys)
		{
			if(!CurrentElementKeys.Contains(StoredKey))
			{
				LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Hierarchy is missing element %s expected by the replay."), *StoredKey.ToString()));
				bSuccess = false;
			}
		}

		for(const FRigElementKey& CurrentKey : CurrentElementKeys)
		{
			if(!StoredElementKeys.Contains(CurrentKey))
			{
				LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Hierarchy contains element %s which is not part of the replay."), *CurrentKey.ToString()));
				bSuccess = false;
			}
		}

		const TSharedPtr<const FSampleTrack<TArray<int32>>> ParentIndicesTrack = OutputTracks.GetContainer()->FindTrack<TArray<int32>>(FControlRigReplayTracks::ParentIndicesName);
		const TArray<int32> StoredParentIndices = ParentIndicesTrack->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);

		for(int32 Index = 0; Index < CurrentElementKeys.Num(); Index++)
		{
			const FRigElementKey& CurrentKey = CurrentElementKeys[Index];
			const int32 StoredIndex = StoredElementKeys.Find(CurrentKey);
			if(StoredIndex == INDEX_NONE)
			{
				continue;
			}
			if(Hierarchy->Find<FRigTransformElement>(CurrentKey) == nullptr)
			{
				continue;
			}
			const int32 CurrentParentIndex = CurrentKey.Type == ERigElementType::Connector ? INDEX_NONE : CurrentElementKeys.Find(Hierarchy->GetDefaultParent(CurrentKey));
			const int32 StoredParentIndex = StoredParentIndices[StoredIndex];

			if(CurrentParentIndex == INDEX_NONE && StoredParentIndex != INDEX_NONE)
			{
				LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Element %s is at root currently, but is parented to %s in the replay."), *CurrentKey.ToString(), *StoredElementKeys[StoredParentIndex].ToString()));
				bSuccess = false;
			}
			else if(CurrentParentIndex != INDEX_NONE && StoredParentIndex == INDEX_NONE)
			{
				LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Element %s is parent to %s currently, but was root in the replay."), *CurrentKey.ToString(), *CurrentElementKeys[CurrentParentIndex].ToString()));
				bSuccess = false;
			}
			else if(CurrentParentIndex != INDEX_NONE && StoredParentIndex != INDEX_NONE)
			{
				if(CurrentElementKeys[CurrentParentIndex] != StoredElementKeys[StoredParentIndex])
				{
					LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Element %s is parent to %s currently, but was parented to %s in the replay."), *CurrentKey.ToString(), *CurrentElementKeys[CurrentParentIndex].ToString(), *StoredElementKeys[StoredParentIndex].ToString()));
					bSuccess = false;
				}
			}
		}
	}

	if(bValidatePose)
	{
		for(const FRigElementKey& StoredKey : StoredElementKeys)
		{
			if(StoredKey.Type == ERigElementType::Connector)
			{
				continue;
			}
			
			if(const FRigBaseElement* Element = Hierarchy->Find(StoredKey))
			{
				const FName& TrackName = OutputTracks.GetTrackName(StoredKey);
				if(Element->IsA<FRigCurveElement>())
				{
					if(const TSharedPtr<const FSampleTrack<float>> Track = OutputTracks.GetContainer()->FindTrack<float>(TrackName))
					{
						const float& StoredValue = Track->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
						const float CurrentValue = Hierarchy->GetCurveValueByIndex(Element->GetIndex());
						if(!FMath::IsNearlyEqual(StoredValue, CurrentValue, static_cast<float>(Tolerance)))
						{
							LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Curve %s has value %.03f, expected by the replay: %.03f."), InPlaybackTimeIndex, *StoredKey.ToString(), CurrentValue, StoredValue));
							bSuccess = false;
						}
					}
				}
				else if(Element->IsA<FRigTransformElement>())
				{
					if(const TSharedPtr<const FSampleTrack<FTransform3f>> Track = OutputTracks.GetContainer()->FindTrack<FTransform3f>(TrackName))
					{
						const FTransform StoredValue = FTransform(Track->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex));
						const FTransform CurrentValue = Hierarchy->GetLocalTransform(Element->GetIndex());
						
						if(!StoredValue.GetLocation().Equals(CurrentValue.GetLocation(), static_cast<float>(Tolerance)))
						{
							LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Element %s has position %s, expected %s by the replay."), InPlaybackTimeIndex, *StoredKey.ToString(), *CurrentValue.GetLocation().ToString(), *StoredValue.GetLocation().ToString()));
							bSuccess = false;
						}
						if(!StoredValue.Rotator().EqualsOrientation(CurrentValue.Rotator(), static_cast<float>(Tolerance)))
						{
							LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Element %s has rotation %s, expected %s by the replay."), InPlaybackTimeIndex, *StoredKey.ToString(), *CurrentValue.Rotator().ToString(), *StoredValue.Rotator().ToString()));
							bSuccess = false;
						}
						if(!StoredValue.GetScale3D().Equals(CurrentValue.GetScale3D(), static_cast<float>(Tolerance)))
						{
							LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Element %s has scale %s, expected %s by the replay."), InPlaybackTimeIndex, *StoredKey.ToString(), *CurrentValue.GetScale3D().ToString(), *StoredValue.GetScale3D().ToString()));
							bSuccess = false;
						}
					}
				}
			}
		}
	}

	if(bValidateMetadata)
	{
		for(const FRigElementKey& StoredKey : StoredElementKeys)
		{
			if(const FRigBaseElement* Element = Hierarchy->Find(StoredKey))
			{
				auto TrackNameBasedLogFunction = [InLogFunction](EMessageSeverity::Type InSeverity, const FName& InTrackName, const FString& InMessage)
				{
					InLogFunction(InSeverity, FString::Printf(TEXT("%s: %s"), *InTrackName.ToString(), *InMessage));
				};
				
				const TArray<FName>& StoredMetadataNames = OutputTracks.GetMetadataNames(InPlaybackTimeIndex, OutSampleTrackIndex, StoredKey, TrackNameBasedLogFunction);

				for(const FName& StoredMetadataName : StoredMetadataNames)
				{
					const FName& TrackName = OutputTracks.GetTrackName(StoredKey, StoredMetadataName);
					const TSharedPtr<const FSampleTrackBase> MetadataTrack = OutputTracks.GetContainer()->FindTrack(TrackName);
					if(MetadataTrack.IsValid())
					{
						ERigMetadataType ExpectedMetadataType = OutputTracks.GetMetadataTypeFromTrackType(MetadataTrack->GetTrackType());
						if(const FRigBaseMetadata* Metadata = Element->GetMetadata(StoredMetadataName, ExpectedMetadataType))
						{
							switch(ExpectedMetadataType)
							{
								case ERigMetadataType::Bool:
								{
									const bool& StoredValue = StaticCastSharedPtr<const FSampleTrack<bool>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const bool& CurrentValue = static_cast<const FRigBoolMetadata*>(Metadata)->GetValue();
									if(StoredValue != CurrentValue)
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::BoolArray:
								{
									const TArray<bool>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<bool>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<bool>& CurrentValue = static_cast<const FRigBoolArrayMetadata*>(Metadata)->GetValue();
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(StoredValue[StoredValueIndex] != CurrentValue[StoredValueIndex])
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::Float:
								{
									const float& StoredValue = StaticCastSharedPtr<const FSampleTrack<float>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const float& CurrentValue = static_cast<const FRigFloatMetadata*>(Metadata)->GetValue();
									if(FMath::IsNearlyEqual(StoredValue, CurrentValue, static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::FloatArray:
								{
									const TArray<float>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<float>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<float>& CurrentValue = static_cast<const FRigFloatArrayMetadata*>(Metadata)->GetValue();
										if(StoredValue.Num() != CurrentValue.Num())
										{
											TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
											bSuccess = false;
										}
										else
										{
											for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
											{
												if(FMath::IsNearlyEqual(StoredValue[StoredValueIndex], CurrentValue[StoredValueIndex], static_cast<float>(Tolerance)))
												{
													TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
													bSuccess = false;
												}
											}
										}
										break;
								}
								case ERigMetadataType::Int32:
								{
									const int32& StoredValue = StaticCastSharedPtr<const FSampleTrack<int32>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const int32& CurrentValue = static_cast<const FRigInt32Metadata*>(Metadata)->GetValue();
									if(StoredValue != CurrentValue)
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::Int32Array:
								{
									const TArray<int32>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<int32>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<int32>& CurrentValue = static_cast<const FRigInt32ArrayMetadata*>(Metadata)->GetValue();
										if(StoredValue.Num() != CurrentValue.Num())
										{
											TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
											bSuccess = false;
										}
										else
										{
											for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
											{
												if(StoredValue[StoredValueIndex] != CurrentValue[StoredValueIndex])
												{
													TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
													bSuccess = false;
												}
											}
										}
										break;
								}
								case ERigMetadataType::Name:
								{
									const FName& StoredValue = StaticCastSharedPtr<const FSampleTrack<FName>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const FName& CurrentValue = static_cast<const FRigNameMetadata*>(Metadata)->GetValue();
									if(StoredValue != CurrentValue)
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::NameArray:
								{
									const TArray<FName>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FName>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FName>& CurrentValue = static_cast<const FRigNameArrayMetadata*>(Metadata)->GetValue();
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(StoredValue[StoredValueIndex] != CurrentValue[StoredValueIndex])
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::Vector:
								{
									const FVector3f& StoredValue = StaticCastSharedPtr<const FSampleTrack<FVector3f>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const FVector3f& CurrentValue = FVector3f(static_cast<const FRigVectorMetadata*>(Metadata)->GetValue());
									if(!StoredValue.Equals(CurrentValue, static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::VectorArray:
								{
									const TArray<FVector3f>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FVector3f>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FVector>& CurrentValue = static_cast<const FRigVectorArrayMetadata*>(Metadata)->GetValue();
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(!StoredValue[StoredValueIndex].Equals(FVector3f(CurrentValue[StoredValueIndex]), static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::Rotator:
								{
									FRotator StoredValue = FRotator::MakeFromEuler(FVector(StaticCastSharedPtr<const FSampleTrack<FVector3f>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex)));
									const FRotator& CurrentValue = static_cast<const FRigRotatorMetadata*>(Metadata)->GetValue();
									if(!StoredValue.EqualsOrientation(CurrentValue, static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::RotatorArray:
								{
									const TArray<FVector3f>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FVector3f>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FRotator>& CurrentValue = static_cast<const FRigRotatorArrayMetadata*>(Metadata)->GetValue();
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(!FRotator::MakeFromEuler(FVector(StoredValue[StoredValueIndex])).EqualsOrientation((CurrentValue[StoredValueIndex]), static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::Quat:
								{
									const FQuat4f& StoredValue = StaticCastSharedPtr<const FSampleTrack<FQuat4f>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const FQuat4f& CurrentValue = FQuat4f(static_cast<const FRigQuatMetadata*>(Metadata)->GetValue()); 
									if(!StoredValue.Equals(CurrentValue, static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::QuatArray:
								{
									const TArray<FQuat4f>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FQuat4f>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FQuat>& CurrentValue = static_cast<const FRigQuatArrayMetadata*>(Metadata)->GetValue();
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(!StoredValue[StoredValueIndex].Equals(FQuat4f(CurrentValue[StoredValueIndex]), static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::Transform:
								{
									const FTransform3f& StoredValue = StaticCastSharedPtr<const FSampleTrack<FTransform3f>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const FTransform3f& CurrentValue = FTransform3f(static_cast<const FRigTransformMetadata*>(Metadata)->GetValue()); 
									if(!StoredValue.GetLocation().Equals(CurrentValue.GetLocation(), static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata location value doesn't match."));
										bSuccess = false;
									}
									if(!StoredValue.Rotator().EqualsOrientation(CurrentValue.Rotator(), static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata rotation value doesn't match."));
										bSuccess = false;
									}
									if(!StoredValue.GetScale3D().Equals(CurrentValue.GetScale3D(), static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata scale value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::TransformArray:
								{
									const TArray<FTransform3f>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FTransform3f>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FTransform>& CurrentValue = static_cast<const FRigTransformArrayMetadata*>(Metadata)->GetValue();
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											const FTransform3f CurrentTransform(CurrentValue[StoredValueIndex]);
											if(!StoredValue[StoredValueIndex].GetLocation().Equals(CurrentTransform.GetLocation(), static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array locationvalue element doesn't match."));
												bSuccess = false;
											}
											if(!StoredValue[StoredValueIndex].Rotator().EqualsOrientation(CurrentTransform.Rotator(), static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array rotation value element doesn't match."));
												bSuccess = false;
											}
											if(!StoredValue[StoredValueIndex].GetScale3D().Equals(CurrentTransform.GetScale3D(), static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array scale value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::LinearColor:
								{
									const FLinearColor& StoredValue = StaticCastSharedPtr<const FSampleTrack<FLinearColor>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const FLinearColor& CurrentValue = static_cast<const FRigLinearColorMetadata*>(Metadata)->GetValue(); 
									if(!StoredValue.Equals(CurrentValue, static_cast<float>(Tolerance)))
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::LinearColorArray:
								{
									const TArray<FLinearColor>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FLinearColor>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FLinearColor>& CurrentValue = static_cast<const FRigLinearColorArrayMetadata*>(Metadata)->GetValue(); 
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(!StoredValue[StoredValueIndex].Equals(CurrentValue[StoredValueIndex], static_cast<float>(Tolerance)))
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::RigElementKey:
								{
									const FRigElementKey& StoredValue = StaticCastSharedPtr<const FSampleTrack<FRigElementKey>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const FRigElementKey& CurrentValue = static_cast<const FRigElementKeyMetadata*>(Metadata)->GetValue(); 
									if(StoredValue != CurrentValue)
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata value doesn't match."));
										bSuccess = false;
									}
									break;
								}
								case ERigMetadataType::RigElementKeyArray:
								{
									const TArray<FRigElementKey>& StoredValue = StaticCastSharedPtr<const FSampleTrack<TArray<FRigElementKey>>>(MetadataTrack)->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
									const TArray<FRigElementKey>& CurrentValue = static_cast<const FRigElementKeyArrayMetadata*>(Metadata)->GetValue(); 
									if(StoredValue.Num() != CurrentValue.Num())
									{
										TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value num doesn't match."));
										bSuccess = false;
									}
									else
									{
										for(int32 StoredValueIndex = 0; StoredValueIndex < StoredValue.Num(); StoredValueIndex++)
										{
											if(StoredValue[StoredValueIndex] != CurrentValue[StoredValueIndex])
											{
												TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Metadata array value element doesn't match."));
												bSuccess = false;
											}
										}
									}
									break;
								}
								case ERigMetadataType::Invalid:
								{
									TrackNameBasedLogFunction(EMessageSeverity::Error, TrackName, TEXT("Unsupported Metadata Type."));
									return false;
								}
								
							}
						}
						else
						{
							TrackNameBasedLogFunction(EMessageSeverity::Warning, TrackName, TEXT("Cannot find metadata on element."));
							bSuccess = false;
						}
					}
					else
					{
						TrackNameBasedLogFunction(EMessageSeverity::Warning, TrackName, TEXT("Track not found."));
						bSuccess = false;
					}
				}
			}
		}
	}

	if(bValidateVariables)
	{
		TArray<FName> CurrentVariableNames;
		for (TFieldIterator<FProperty> PropertyIt(InSubject->GetClass()); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			if(Property->IsNative())
			{
				continue;
			}

			const UScriptStruct* ScriptStruct = nullptr;
			if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				ScriptStruct = StructProperty->Struct;
			}
			else if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				if(const FStructProperty* InnerStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
				{
					ScriptStruct = InnerStructProperty->Struct;
				}
			}

			const FName& TrackName = OutputTracks.GetTrackName(Property);
			const TSharedPtr<const FSampleTrackBase> Track = OutputTracks.GetContainer()->FindTrack(TrackName);

			if(Track)
			{
				const FSampleTrackBase::ETrackType TrackType = OutputTracks.GetTrackTypeFromProperty(Property);
				if(Track->GetTrackType() == TrackType && Track->GetScriptStruct() == ScriptStruct)
				{
					TArray<uint8> ValueMemory;
					ValueMemory.AddUninitialized(Property->GetSize());
					Property->InitializeValue(ValueMemory.GetData());
					Track->GetSampleForProperty(InPlaybackTimeIndex, OutSampleTrackIndex, Property, ValueMemory.GetData());

					const uint8* CurrentMemory = Property->ContainerPtrToValuePtr<uint8>(InSubject);
					if(!Property->Identical(ValueMemory.GetData(), CurrentMemory, PPF_None))
					{
						LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Variable '%s' value doesn't match replay."), InPlaybackTimeIndex, *Property->GetName()));
						bSuccess = false;
					}
					Property->DestroyValue(ValueMemory.GetData());
				}
				else
				{
					if(InPlaybackTimeIndex == 0 && InLogFunction)
					{
						LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Track for Variable '%s' has incorrect type."), InPlaybackTimeIndex, *Property->GetName()));
					}
					bSuccess = false;
				}
			}
			else
			{
				if(InPlaybackTimeIndex == 0 && InLogFunction)
				{
					LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Track for Variable '%s' is missing in replay."), InPlaybackTimeIndex, *Property->GetName()));
				}
				bSuccess = false;
			}
			
			CurrentVariableNames.Add(Property->GetFName());
		}

		if(InPlaybackTimeIndex == 0)
		{
			if(const TSharedPtr<const FSampleTrack<TArray<FName>>> VariableNamesTrack = OutputTracks.GetContainer()->FindTrack<TArray<FName>>(FControlRigReplayTracks::VariableNamesName))
			{
				const TArray<FName>& StoredVariableNames = VariableNamesTrack->GetValueAtTimeIndex(InPlaybackTimeIndex, OutSampleTrackIndex);
				for(const FName& StoredVariableName : StoredVariableNames)
				{
					if(!CurrentVariableNames.Contains(StoredVariableName))
					{
						LocalLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame [%04d]: Variable '%s' for stored track is missing."), InPlaybackTimeIndex, *StoredVariableName.ToString()));
						bSuccess = false;
					}
				}
			}
		}
	}
	
	return bSuccess;
}

void UControlRigReplay::ClearDelegates(UControlRig* InControlRig)
{
	if(InControlRig)
	{
		if(PreConstructionHandle.IsValid())
		{
			InControlRig->OnPreConstruction_AnyThread().Remove(PreConstructionHandle);
			PreConstructionHandle.Reset();
		}
		if(PreEventHandle.IsValid())
		{
			InControlRig->OnPreExecuted_AnyThread().Remove(PreEventHandle);
			PreEventHandle.Reset();
		}
		if(PostEventHandle.IsValid())
		{
			InControlRig->OnExecuted_AnyThread().Remove(PostEventHandle);
			PostEventHandle.Reset();
		}
	}
}

void UControlRigReplay::HandlePreconstructionForTest(UControlRig* InRig, const FName& InEventName) const
{
	InRig->OnPreConstruction_AnyThread().RemoveAll(this);
	
	if(InRig->IsRigModule())
	{
		if(Cast<USkeletalMesh>(PreviewSkeletalMeshObjectPath.TryLoad()))
		{
			if(USkeletalMesh* PreviewSkeletalMesh = Cast<USkeletalMesh>(PreviewSkeletalMeshObjectPath.TryLoad()))
			{
				if(URigHierarchy* Hierarchy = InRig->GetHierarchy())
				{
					if(URigHierarchyController* Controller = Hierarchy->GetController(true))
					{
						const TArray<FRigSocketState> SocketStates = InRig->GetHierarchy()->GetSocketStates();
						Controller->ImportPreviewSkeletalMesh(PreviewSkeletalMesh, false, false, false, false);
						InRig->GetHierarchy()->RestoreSocketsFromStates(SocketStates);
					}
				}
			}
		}
	}

	// restore the connectors
	const TArray<FRigElementKey> ConnectorKeys = InRig->GetHierarchy()->GetConnectorKeys();
	FRigElementKeyRedirector::FKeyMap ConnectorMap;
	for(const FRigElementKey& ConnectorKey : ConnectorKeys)
	{
		const FName& TrackName = InputTracks.GetTrackName(ConnectorKey);
		const FName& ConnectorTrackName = *(TrackName.ToString() + TEXT("ConnectorTargets"));

		if(const TSharedPtr<const FSampleTrack<TArray<FRigElementKey>>> Track = InputTracks.GetContainer()->FindTrack<TArray<FRigElementKey>>(ConnectorTrackName))
		{
			FRigElementKeyRedirector::FKeyArray Targets;
			Targets.Append(Track->GetValueAtTimeIndex(0, InputTracks.SampleTrackIndex));
			ConnectorMap.Add(ConnectorKey, Targets);
		}
	}
	if(!ConnectorMap.IsEmpty())
	{
		InRig->SetElementKeyRedirector(FRigElementKeyRedirector(ConnectorMap, InRig->GetHierarchy()));
		InRig->GetHierarchy()->ElementKeyRedirector = &InRig->GetElementKeyRedirector(); 
	}

}

#undef LOCTEXT_NAMESPACE
