// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigRuntimeAsset.h"

#include "ControlRig.h"
#include "ModularRig.h"
#include "UObject/ObjectSaveContext.h"

TScriptInterface<IControlRigRuntimeAssetInterface> IControlRigRuntimeAssetInterface::GetInterfaceOuter(const UObject* InObject)
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RigVMAsset = IRigVMRuntimeAssetInterface::GetInterfaceOuter(InObject))
	{
		if (RigVMAsset.GetObject()->Implements<UControlRigRuntimeAssetInterface>())
		{
			return RigVMAsset.GetObject();
		}
	}
	return nullptr;
}

FControlRigAssetStrongReference UControlRigRuntimeAsset::GetControlRigAssetReference()
{
	FControlRigAssetStrongReference AssetReference;
	AssetReference.Set(this);
	return AssetReference;
}

UClass* UControlRigRuntimeAsset::GetRigVMHostClass() const
{
	if (ControlRigType == EControlRigType::ModularRig)
	{
		return UModularRig::StaticClass();
	}
	return UControlRig::StaticClass();
}

void UControlRigRuntimeAsset::UpdateSupportedEventNames()
{
	if (IsModularRig())
	{
		SupportedEventNames = ModularRigModel.GetSupportedEvents();
	}
	else
	{
		Super::UpdateSupportedEventNames();
	}
}

void UControlRigRuntimeAsset::PostLoad()
{
	Super::PostLoad();
}

void UControlRigRuntimeAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	if(Ar.IsLoading())
	{
		ModularRigModel.UpdateCachedChildren();
		ModularRigModel.Connections.UpdateFromConnectionList();
	}
}

void UControlRigRuntimeAsset::PostInitProperties()
{
	Super::PostInitProperties();
	
	ExecuteContext.SetContextPublicDataStruct(FControlRigExecuteContext::StaticStruct());
}

void UControlRigRuntimeAsset::Initialize(UClass* InEditorOnlyDataClass)
{
	Super::Initialize(InEditorOnlyDataClass);
	
	ExecuteContext.SetContextPublicDataStruct(FControlRigExecuteContext::StaticStruct());
}

URigVMHost* UControlRigRuntimeAsset::InstantiateObject(UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	UControlRig* Host = nullptr;
	if (ControlRigType == EControlRigType::ModularRig)
	{
		Host = NewObject<UModularRig>(InOuter, GetRigVMHostClass(), InName, InFlags);
	}
	else
	{
		Host = NewObject<UControlRig>(InOuter, GetRigVMHostClass(), InName, InFlags);
	}
	Host->GeneratedBy = TScriptInterface<IRigVMRuntimeAssetInterface>(this);
	InitializeInstance(Host);
	return Host;
}

bool UControlRigRuntimeAsset::InitializeInstance(URigVMHost* InInstance)
{
	bool bSuccess = false;
	if (UControlRig* ControlRig = Cast<UControlRig>(InInstance))
	{
		if (Super::InitializeInstance(InInstance))
		{
			if (!ControlRig->IsRigModuleInstance())
			{
				ControlRig->DynamicHierarchy->CopyHierarchy(Hierarchy);
				ControlRig->ElementKeyRedirector = FRigElementKeyRedirector(ElementKeyRedirector, ControlRig->DynamicHierarchy);
			}
			ControlRig->HierarchySettings = HierarchySettings;
			ControlRig->RigModuleSettings = RigModuleSettings;
			
			bSuccess = true;
		}
		
		if (UModularRig* ModularRig = Cast<UModularRig>(InInstance))
		{
			ModularRig->ModularRigSettings = ModularRigSettings;
			ModularRig->ModularRigModel = ModularRigModel;
		}
	}
	return bSuccess;
}

void UControlRigRuntimeAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	URigVMRuntimeAsset::PreSave(ObjectSaveContext);
#if WITH_EDITORONLY_DATA
	EditorAsset->PreSave(ObjectSaveContext);
#endif
}

#if WITH_EDITOR
void UControlRigRuntimeAsset::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	URigVMRuntimeAsset::PostTransacted(TransactionEvent);
	EditorAsset->PostTransacted(TransactionEvent);
}
#endif

void UControlRigRuntimeAsset::PostDuplicate(bool bDuplicateForPIE)
{
	URigVMRuntimeAsset::PostDuplicate(bDuplicateForPIE);
#if WITH_EDITORONLY_DATA
	EditorAsset->PostDuplicate(bDuplicateForPIE);
#endif
}

void UControlRigRuntimeAsset::PostRename(UObject* OldOuter, const FName OldName)
{
	URigVMRuntimeAsset::PostRename(OldOuter, OldName);
}
