// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigTestData.h"
#include "ControlRig.h"
#include "ControlRigObjectVersion.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "HAL/PlatformTime.h"
#if WITH_EDITOR
#include "AssetToolsModule.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigTestData)

bool FControlRigTestDataFrame::Store(UControlRig* InControlRig, bool bInitial)
{
	if(InControlRig == nullptr)
	{
		return false;
	}

	AbsoluteTime = InControlRig->GetAbsoluteTime();
	DeltaTime = InControlRig->GetDeltaTime();
	Pose = InControlRig->GetHierarchy()->GetPose(bInitial);
	Variables.Reset();

	const TArray<FRigVMExternalVariable> ExternalVariables = InControlRig->GetExternalVariables(); 
	for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
	{
		FControlRigReplayVariable VariableData;
		VariableData.Name = ExternalVariable.GetName();
		VariableData.CPPType = ExternalVariable.GetExtendedCPPType();

		if(ExternalVariable.GetProperty() && ExternalVariable.GetMemory())
		{
			ExternalVariable.GetProperty()->ExportText_Direct(
				VariableData.Value,
				ExternalVariable.GetMemory(),
				ExternalVariable.GetMemory(),
				nullptr,
				PPF_None,
				nullptr
			);
		}

		Variables.Add(VariableData);
	}

	MetadataMap = InControlRig->GetHierarchy()->CopyMetadata();
	TArray<uint8> UncompressedBytes;
	FMemoryWriter ArchiveWriter(UncompressedBytes);
	ArchiveWriter.UsingCustomVersion(FControlRigObjectVersion::GUID);

	ArchiveWriter << MetadataMap;
	Metadata = UncompressedBytes;

	return true;
}

bool FControlRigTestDataFrame::Restore(UControlRig* InControlRig, bool bInitial) const
{
	if(InControlRig == nullptr)
	{
		return false;
	}

	URigHierarchy* Hierarchy = InControlRig->GetHierarchy();

	// check if the pose can be applied
	for(const FRigPoseElement& PoseElement : Pose.Elements)
	{
		const FRigElementKey& Key = PoseElement.Index.GetKey(); 
		if(!Hierarchy->Contains(Key))
		{
			UE_LOGF(LogControlRig, Error, "Control Rig does not contain hierarchy element '%ls'. Please re-create the test data asset.", *Key.ToString());
			return false;
		}
	}

	Hierarchy->SetPose(Pose, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal);
	bool bSuccess = RestoreVariables(InControlRig);
	bSuccess &= RestoreMetadata(InControlRig);

	return bSuccess;
}

bool FControlRigTestDataFrame::RestoreVariables(UControlRig* InControlRig) const
{
	class FControlRigTestDataFrame_ErrorPipe : public FOutputDevice
	{
	public:

		TArray<FString> Errors;

		FControlRigTestDataFrame_ErrorPipe()
			: FOutputDevice()
		{
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			Errors.Add(FString::Printf(TEXT("Error convert to string: %s"), V));
		}
	};

	const TArray<FRigVMExternalVariable> ExternalVariables = InControlRig->GetExternalVariables();

	if(ExternalVariables.Num() != Variables.Num())
	{
		UE_LOGF(LogControlRig, Error, "Variable data does not match the Rig. Please re-create the test data asset.");
		return false;
	}
	
	for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
	{
		if(ExternalVariable.GetMemory() == nullptr || ExternalVariable.GetProperty() == nullptr)
		{
			UE_LOGF(LogControlRig, Error, "Variable '%ls' is not valid.", *ExternalVariable.GetName().ToString());
			return false;
		}
		
		const FControlRigReplayVariable* VariableData = Variables.FindByPredicate(
			[ExternalVariable](const FControlRigReplayVariable& InVariable) -> bool
			{
				return InVariable.Name == ExternalVariable.GetName() &&
					InVariable.CPPType == ExternalVariable.GetExtendedCPPType();
			}
		);

		if(VariableData)
		{
			FControlRigTestDataFrame_ErrorPipe ErrorPipe;
			ExternalVariable.GetProperty()->ImportText_Direct(
				*VariableData->Value,
				const_cast<uint8*>(ExternalVariable.GetMemory()),
				nullptr,
				PPF_None,
				&ErrorPipe
			);

			if(!ErrorPipe.Errors.IsEmpty())
			{
				for(const FString& ImportError : ErrorPipe.Errors)
				{
					UE_LOGF(LogControlRig, Error, "Import Error for Variable '%ls': %ls", *ExternalVariable.GetName().ToString(), *ImportError);
				}
				return false;
			}
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Variable data for '%ls' is not part of the test file. Please re-create the test data asset.", *ExternalVariable.GetName().ToString());
			return false;
		}
	}

	return true;
}

bool FControlRigTestDataFrame::RestoreMetadata(UControlRig* InControlRig) const
{
	bool bSuccess = true;

	if (MetadataMap.IsEmpty())
	{
		if (Metadata.IsEmpty())
		{
			bTestMetadata = false;
		}
		else
		{
			FMemoryReader ArchiveReader(Metadata);
			ArchiveReader.SetCustomVersions(ArchiveCustomVersions);
			ArchiveReader << MetadataMap;
		}
	}

	if (bTestMetadata)
	{
		bSuccess &= InControlRig->GetHierarchy()->SetMetadata(MetadataMap);
	}
		
	return bSuccess;
}

void UControlRigTestData::BeginDestroy()
{
	UObject::BeginDestroy();

	TArray<FControlRigTestDataFrame> Frames;
	Frames.Add(Initial);
	Frames.Append(InputFrames);
	Frames.Append(OutputFrames);

	for (FControlRigTestDataFrame& Frame : Frames)
	{
		for (TPair<FRigElementKey, URigHierarchy::FMetadataStorage>& Pair : Frame.MetadataMap)
		{
			Pair.Value.Reset();
		}
		Frame.MetadataMap.Reset();
	}
}

void UControlRigTestData::Serialize(FArchive& Ar)
{
	UControlRigReplay::Serialize(Ar);
	LastFrameIndex = INDEX_NONE;

	Initial.ArchiveCustomVersions = Ar.GetCustomVersions();
	Initial.bTestMetadata = !Initial.Metadata.IsEmpty();
	for (int32 IO=0; IO<2; IO++)
	{
		TArray<FControlRigTestDataFrame>& Frames = (IO == 0) ? InputFrames : OutputFrames;
		for (FControlRigTestDataFrame& Frame : Frames)
		{
			Frame.ArchiveCustomVersions = Ar.GetCustomVersions();
			Frame.bTestMetadata = !Frame.Metadata.IsEmpty();
		}
	}
	
	// If pose is older than RigPoseWithParentKey, set the active parent of all poses to invalid key
	if (GetLinkerCustomVersion(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RigPoseWithParentKey)
	{
		for (FRigPoseElement& Element : Initial.Pose)
		{
			Element.ActiveParent = FRigElementKey();
		}
		for (FControlRigTestDataFrame& Frame : InputFrames)
		{
			for (FRigPoseElement& Element : Frame.Pose)
			{
				Element.ActiveParent = FRigElementKey();
			}
		}
		for (FControlRigTestDataFrame& Frame : OutputFrames)
		{
			for (FRigPoseElement& Element : Frame.Pose)
			{
				Element.ActiveParent = FRigElementKey();
			}
		}
	}

	if (EventQueue.IsEmpty())
	{
		EventQueue = {FRigUnit_BeginExecution::EventName};
	}
}

FVector2D UControlRigTestData::GetTimeRange() const
{
	if(OutputFrames.IsEmpty())
	{
		return FVector2D::ZeroVector;
	}
	return FVector2D(OutputFrames[0].AbsoluteTime, OutputFrames.Last().AbsoluteTime);
}

int32 UControlRigTestData::GetFrameIndexForTime(double InSeconds, bool bInput) const
{
	const TArray<FControlRigTestDataFrame>& Frames = bInput ? InputFrames : OutputFrames;

	if(LastFrameIndex == INDEX_NONE)
	{
		LastFrameIndex = 0;
	}

	while(Frames.IsValidIndex(LastFrameIndex) && Frames[LastFrameIndex].AbsoluteTime < InSeconds)
	{
		LastFrameIndex++;
	}

	while(Frames.IsValidIndex(LastFrameIndex) && Frames[LastFrameIndex].AbsoluteTime > InSeconds)
	{
		LastFrameIndex--;
	}

	LastFrameIndex = FMath::Clamp(LastFrameIndex, 0, Frames.Num() - 1);

	return LastFrameIndex;
}

bool UControlRigTestData::StartRecording(UControlRig* InControlRig)
{
	if(InControlRig == nullptr)
	{
		return false;
	}

	RecordControlRig = InControlRig;
	StopReplay();
	ClearDelegates(InControlRig);

	EventQueue = InControlRig->EventQueue;
	TimeAtStartOfRecording = FPlatformTime::Seconds();

	PreConstructionHandle = InControlRig->OnPreConstruction_AnyThread().AddLambda(
		[this](UControlRig* InControlRig, const FName& InEventName)
		{
			Initial.Store(InControlRig, true);
		}
	);

	PreEventHandle = InControlRig->OnPreExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig)
			{
				return;
			}

			if (InEventName == FRigUnit_PrepareForExecution::EventName)
			{
				return;
			}

			
			if (InEventName != EventQueue[0])
			{
				return;
			}
			
			FControlRigTestDataFrame Frame;
			Frame.Store(ControlRig);

			// reapply the variable data. we are doing this to make sure that
			// the results in the rig are the same during a recording and replay.
			Frame.RestoreVariables(ControlRig);
			
			InputFrames.Add(MoveTemp(Frame));
		}
	);

	PostEventHandle = InControlRig->OnExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig)
			{
				return;
			}

			if (InEventName == FRigUnit_PrepareForExecution::EventName)
			{
				return;
			}

			if (InEventName != EventQueue.Last())
			{
				return;
			}
			
			FControlRigTestDataFrame Frame;
			Frame.Store(ControlRig);
			OutputFrames.Add(MoveTemp(Frame));
			LastFrameIndex = INDEX_NONE;
			(void)MarkPackageDirty();

			const double TimeNow = FPlatformTime::Seconds();
			const double TimeDelta = TimeNow - TimeAtStartOfRecording;
			if(DesiredRecordingDuration <= TimeDelta)
			{
				DesiredRecordingDuration = 0.0;

				// Once clear delegates is called, we no longer have access to this pointer
				StopRecording();
			}
		}
	);

	// if this is the first frame
	if (InputFrames.IsEmpty())
	{
		InControlRig->RequestInit();
	}

	return true;
}

bool UControlRigTestData::StopRecording()
{
	if(RecordControlRig.IsValid())
	{
		ClearDelegates(RecordControlRig.Get());
		RecordControlRig.Reset();
		return true;
	}
	return false;
}

bool UControlRigTestData::StartReplay(UControlRig* InControlRig, EControlRigReplayPlaybackMode InMode)
{
	StopRecording();
	StopReplay();
	ClearDelegates(InControlRig);

	if(InControlRig == nullptr)
	{
		return false;
	}

	InControlRig->EventQueue = EventQueue;
	bIsApplyingOutputs = true;

	if(InputFrames.IsEmpty() || OutputFrames.IsEmpty())
	{
		return false;
	}

	// reset the control rig's absolute time
	InControlRig->SetAbsoluteAndDeltaTime(InputFrames[0].AbsoluteTime, InputFrames[0].DeltaTime);
	
	PreConstructionHandle = InControlRig->OnPreConstruction_AnyThread().AddLambda(
		[this](UControlRig* InControlRig, const FName& InEventName)
		{
			Initial.Restore(InControlRig, true);
		}
	);

	PreEventHandle = InControlRig->OnPreExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig || InEventName == FRigUnit_PrepareForExecution::EventName)
			{
				return;
			}

			if (InEventName != EventQueue[0])
			{
				return;
			}
			
			// loop the animation data
			if(ControlRig->GetAbsoluteTime() < GetTimeRange().X - SMALL_NUMBER ||
				ControlRig->GetAbsoluteTime() > GetTimeRange().Y + SMALL_NUMBER)
			{
				ControlRig->SetAbsoluteAndDeltaTime(GetTimeRange().X, ControlRig->GetDeltaTime());
			}
			
			const int32 FrameIndex = GetFrameIndexForTime(ControlRig->GetAbsoluteTime(), true);
			const FControlRigTestDataFrame& Frame = InputFrames[FrameIndex];
			Frame.Restore(ControlRig, false);

			if(Frame.DeltaTime > SMALL_NUMBER)
			{
				ControlRig->SetDeltaTime(Frame.DeltaTime);
			}
		}
	);

	PostEventHandle = InControlRig->OnExecuted_AnyThread().AddLambda(
		[this](URigVMHost* InRigVMHost, const FName& InEventName)
		{
			UControlRig* ControlRig = Cast<UControlRig>(InRigVMHost);
			if (!ControlRig || InEventName == FRigUnit_PrepareForExecution::EventName)
			{
				return;
			}

			if (InEventName != EventQueue.Last())
			{
				return;
			}
			
			const FRigPose CurrentPose = ControlRig->GetHierarchy()->GetPose();

			const int32 FrameIndex = LastFrameIndex; 
			const FControlRigTestDataFrame& Frame = OutputFrames[FrameIndex];

			if (Frame.bTestMetadata)
			{
				const TMap<FRigElementKey, URigHierarchy::FMetadataStorage>& ExpectedMetadata = Frame.MetadataMap;
			   URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
#if WITH_EDITOR
			   FRigVMLog* Log = ControlRig->GetLog();
			   if (Log)
			   {
				   Hierarchy->ForEach([ExpectedMetadata, Hierarchy, Log, FrameIndex](const FRigBaseElement* Element)
				   {
					   TArray<FName> MetadataNames = Hierarchy->GetMetadataNames(Element->GetKey());
					   const URigHierarchy::FMetadataStorage* ExpectedElementMetadata = ExpectedMetadata.Find(Element->GetKey());
					   if (ExpectedElementMetadata)
					   {
						   if (ExpectedElementMetadata->MetadataMap.Num() != MetadataNames.Num())
						   {
							   const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: number of metadata elements does not match"), *Element->GetKey().ToString());
							   Log->Report(EMessageSeverity::Error, TEXT("TestData"), FrameIndex, Message);
						   }
						   for(TTuple<FName, FRigBaseMetadata*> Pair : ExpectedElementMetadata->MetadataMap)
						   {
							   if(FRigBaseMetadata* Value = Hierarchy->FindMetadataForElement(Element, Pair.Key, Pair.Value->GetType()))
							   {
								   const FProperty* Property = Value->GetValueProperty();
								   check(Value->IsValid());
								   check(Pair.Value->IsValid());
								   if(!Property->Identical(Value->GetValueData(), Pair.Value->GetValueData()))
								   {
									   FString Expected, Received;
									   Property->ExportText_Direct(Expected, Pair.Value->GetValueData(), nullptr, nullptr, PPF_None, nullptr);
									   Property->ExportText_Direct(Received, Value->GetValueData(), nullptr, nullptr, PPF_None, nullptr);
									   const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: property %s expected %s, but found %s"), *Element->GetKey().ToString(), *Pair.Key.ToString(), *Expected, *Received);
									   Log->Report(EMessageSeverity::Error, TEXT("TestData"), FrameIndex, Message);
								   }
							   }
							   else
							   {
								   const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: property %s expected, but was not found"), *Element->GetKey().ToString(), *Pair.Key.ToString());
								   Log->Report(EMessageSeverity::Error, TEXT("TestData"), FrameIndex, Message);
							   }
						   }
					   }
					   else if (Hierarchy->HasMetadata(Element))
					   {
						   const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: number of metadata elements does not match"), *Element->GetKey().ToString());
						   Log->Report(EMessageSeverity::Error, TEXT("TestData"), FrameIndex, Message);
					   }
					   return true;
				   });
			   }
#endif
			}
			
			if(bIsApplyingOutputs)
			{
				Frame.Restore(ControlRig, false);
			}

			const FRigPose& ExpectedPose = Frame.Pose;

			// draw differences of the pose result of the rig onto the screen
			FRigVMDrawInterface& DrawInterface = ControlRig->GetDrawInterface();
			for(const FRigPoseElement& ExpectedPoseElement : ExpectedPose)
			{
				const int32 CurrentPoseIndex = CurrentPose.GetIndex(ExpectedPoseElement.Index.GetKey());
				if(CurrentPoseIndex != INDEX_NONE)
				{
					const FRigPoseElement& CurrentPoseElement = CurrentPose[CurrentPoseIndex];

					if(!CurrentPoseElement.LocalTransform.Equals(ExpectedPoseElement.LocalTransform, 0.001f))
					{
						DrawInterface.DrawAxes(
							FTransform::Identity,
							bIsApplyingOutputs ? CurrentPoseElement.GlobalTransform : ExpectedPoseElement.GlobalTransform,
							bIsApplyingOutputs ? FLinearColor::Red : FLinearColor::Green,
							15.0f,
							1.0f
						);
					}
				}
			}
		}
	);

	InControlRig->RequestInit();

	ReplayControlRig = InControlRig;
	return true;
}

bool UControlRigTestData::StopReplay()
{
	if(UControlRig* ControlRig = ReplayControlRig.Get())
	{
		ClearDelegates(ControlRig);
		ReplayControlRig.Reset();
		return true;
	}
	return false;
}

bool UControlRigTestData::IsValidForTesting() const
{
	return InputFrames.Num() == OutputFrames.Num();
}

bool UControlRigTestData::PerformTest(UControlRig* InSubject, TFunction<void(EMessageSeverity::Type, const FString&)> InLogFunction) const
{
	bool bSuccess = true;

	// initialize
	InSubject->Initialize();
	Initial.Restore(InSubject, true);

	// run the construction event
	InSubject->SetEventQueue({FRigUnit_PrepareForExecution::EventName});
	InSubject->Evaluate_AnyThread();

	// now run all of the frames
	InSubject->SetEventQueue({FRigUnit_BeginExecution::EventName});
	for(int32 FrameIndex = 0; FrameIndex < FMath::Min<int32>(InputFrames.Num(), OutputFrames.Num()); FrameIndex++)
	{
		const FControlRigTestDataFrame& InputFrame = InputFrames[FrameIndex];
		const FControlRigTestDataFrame& OutputFrame = OutputFrames[FrameIndex];
		
		InSubject->SetAbsoluteAndDeltaTime(InputFrame.AbsoluteTime, InputFrame.DeltaTime);
		if(!InputFrame.Restore(InSubject))
		{
			bSuccess = false;
		}
		
		InSubject->Evaluate_AnyThread();

		// skip the frame's test results
		if(FramesToSkip.Contains(FrameIndex))
		{
			continue;
		}

		const FRigPose CurrentPose = InSubject->GetHierarchy()->GetPose();
		const FRigPose& ExpectedPose = OutputFrame.Pose;

		for(const FRigPoseElement& ExpectedPoseElement : ExpectedPose)
		{
			const int32 CurrentPoseIndex = CurrentPose.GetIndex(ExpectedPoseElement.Index.GetKey());
			if(CurrentPoseIndex != INDEX_NONE)
			{
				const FRigPoseElement& CurrentPoseElement = CurrentPose[CurrentPoseIndex];

				if(!CurrentPoseElement.LocalTransform.Equals(ExpectedPoseElement.LocalTransform, Tolerance))
				{
					InLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame[%03d]: Expected Rig Element '%s' doesn't match. (%s vs expected %s)."),
						FrameIndex,
						*ExpectedPoseElement.Index.GetKey().ToString(),
						*CurrentPoseElement.LocalTransform.ToString(),
						*ExpectedPoseElement.LocalTransform.ToString()
					));
					bSuccess = false;
				}
			}
			else
			{
				InLogFunction(EMessageSeverity::Error, FString::Printf(TEXT("Frame[%03d]: Expected Rig Element '%s' is missing."),
					FrameIndex,
					*ExpectedPoseElement.Index.GetKey().ToString()
				));
				bSuccess = false;
			}
		}

		if (OutputFrame.bTestMetadata)
		{
			const TMap<FRigElementKey, URigHierarchy::FMetadataStorage>& ExpectedMetadata = OutputFrame.MetadataMap;
			URigHierarchy* Hierarchy = InSubject->GetHierarchy();
#if WITH_EDITOR
			Hierarchy->ForEach([ExpectedMetadata, Hierarchy, InLogFunction, FrameIndex](const FRigBaseElement* Element)
			{
				const TArray<FName> MetadataNames = Hierarchy->GetMetadataNames(Element->GetKey());
				const URigHierarchy::FMetadataStorage* ExpectedElementMetadata = ExpectedMetadata.Find(Element->GetKey());
				if (ExpectedElementMetadata)
				{
					if (ExpectedElementMetadata->MetadataMap.Num() != MetadataNames.Num())
					{
						const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: number of metadata elements does not match"), *Element->GetKey().ToString());
						InLogFunction(EMessageSeverity::Error, Message);
					}
					for(TTuple<FName, FRigBaseMetadata*> Pair : ExpectedElementMetadata->MetadataMap)
					{
						if(FRigBaseMetadata* Value = Hierarchy->FindMetadataForElement(Element, Pair.Key, Pair.Value->GetType()))
						{
							const FProperty* Property = Value->GetValueProperty();
							check(Value->IsValid());
							check(Pair.Value->IsValid());
							if(!Property->Identical(Value->GetValueData(), Pair.Value->GetValueData()))
							{
								FString Expected, Received;
								Property->ExportText_Direct(Expected, Pair.Value->GetValueData(), nullptr, nullptr, PPF_None, nullptr);
								Property->ExportText_Direct(Received, Value->GetValueData(), nullptr, nullptr, PPF_None, nullptr);
								const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: property %s expected %s, but found %s"), *Element->GetKey().ToString(), *Pair.Key.ToString(), *Expected, *Received);
								InLogFunction(EMessageSeverity::Error, Message);
							}
						}
						else
						{
							const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: property %s expected, but was not found"), *Element->GetKey().ToString(), *Pair.Key.ToString());
							InLogFunction(EMessageSeverity::Error, Message);
						}
					}
				}
				else if (Hierarchy->HasMetadata(Element))
				{
					const FString Message = FString::Printf(TEXT("Metadata mismatch in element %s: number of metadata elements does not match"), *Element->GetKey().ToString());
					InLogFunction(EMessageSeverity::Error, Message);
				}
				return true;
			});
#endif
		}

		const TArray<FRigVMExternalVariable> ExternalVariables = InSubject->GetExternalVariables(); 
		for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
		{
			const FControlRigReplayVariable* VariableData = OutputFrame.Variables.FindByPredicate(
				[ExternalVariable](const FControlRigReplayVariable& InVariable) -> bool
				{
					return InVariable.Name == ExternalVariable.GetName() &&
						InVariable.CPPType == ExternalVariable.GetExtendedCPPType();
				}
			);

			// cases of missing variables etc are already caught in the test by the
			// FControlRigTestDataFrame::Restore earlier.
			
			if(VariableData)
			{
				FString CurrentValue;
				ExternalVariable.GetProperty()->ExportText_Direct(
					CurrentValue,
					ExternalVariable.GetMemory(),
					ExternalVariable.GetMemory(),
					nullptr,
					PPF_None,
					nullptr
				);

				if(CurrentValue != VariableData->Value)
				{
					UE_LOGF(LogControlRig, Error, "Variable '%ls' doesn't match: %ls vs expected '%ls'",
						*ExternalVariable.GetName().ToString(),
						*CurrentValue,
						*VariableData->Value
					);
					bSuccess = false;
				}
			}

		}
	}

	return bSuccess;
}

