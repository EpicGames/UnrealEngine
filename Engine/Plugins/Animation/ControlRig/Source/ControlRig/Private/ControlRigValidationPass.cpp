// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigValidationPass.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigValidationPass)

////////////////////////////////////////////////////////////////////////////////
// FControlRigValidationContext
////////////////////////////////////////////////////////////////////////////////

FControlRigValidationContext::FControlRigValidationContext()
	: DrawInterface(nullptr)
{
}

void FControlRigValidationContext::Clear()
{
	ClearDelegate.ExecuteIfBound();
}

void FControlRigValidationContext::Report(EMessageSeverity::Type InSeverity, const FString& InMessage)
{
	Report(InSeverity, FRigElementKey(), FLT_MAX, InMessage);
}

void FControlRigValidationContext::Report(EMessageSeverity::Type InSeverity, const FRigElementKey& InKey, const FString& InMessage)
{
	Report(InSeverity, InKey, FLT_MAX, InMessage);
}

void FControlRigValidationContext::Report(EMessageSeverity::Type InSeverity, const FRigElementKey& InKey, float InQuality, const FString& InMessage)
{
	ReportDelegate.ExecuteIfBound(InSeverity, InKey, InQuality, InMessage);
}

FString FControlRigValidationContext::GetDisplayNameForEvent(const FName& InEventName) const
{
	FString DisplayName = InEventName.ToString();

#if WITH_EDITOR
	if(InEventName == FRigUnit_BeginExecution::EventName)
	{
		FRigUnit_BeginExecution::StaticStruct()->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayName);
	}
	else if (InEventName == FRigUnit_InverseExecution::EventName)
	{
		FRigUnit_InverseExecution::StaticStruct()->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayName);
	}
	else if (InEventName == FRigUnit_PrepareForExecution::EventName)
	{
		FRigUnit_PrepareForExecution::StaticStruct()->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayName);
	}
#endif

	return DisplayName;
}

////////////////////////////////////////////////////////////////////////////////
// UControlRigValidator
////////////////////////////////////////////////////////////////////////////////

UControlRigValidator::UControlRigValidator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UControlRigValidationPass* UControlRigValidator::FindPass(UClass* InClass) const
{
	for (UControlRigValidationPass* Pass : Passes)
	{
		if (Pass->GetClass() == InClass)
		{
			return Pass;
		}
	}
	return nullptr;
}

UControlRigValidationPass* UControlRigValidator::AddPass(UClass* InClass)
{
	check(InClass);

	if (UControlRigValidationPass* ExistingPass = FindPass(InClass))
	{
		return ExistingPass;
	}

	UControlRigValidationPass* NewPass = NewObject<UControlRigValidationPass>(this, InClass);
	Passes.Add(NewPass);

	NewPass->OnSubjectChanged(GetControlRig(), &ValidationContext);
	NewPass->OnInitialize(GetControlRig(), &ValidationContext);

	return NewPass;
}

void UControlRigValidator::RemovePass(UClass* InClass)
{
	for (UControlRigValidationPass* Pass : Passes)
	{
		if (Pass->GetClass() == InClass)
		{
			Passes.Remove(Pass);
			Pass->Rename(nullptr, GetTransientPackage());
			return;
		}
	}
}

void UControlRigValidator::SetControlRig(UControlRig* InControlRig)
{
	if (UControlRig* ControlRig = WeakControlRig.Get())
	{
		ControlRig->OnInitialized_AnyThread().RemoveAll(this);
		ControlRig->OnExecuted_AnyThread().RemoveAll(this);
	}

	ValidationContext.DrawInterface = nullptr;
	WeakControlRig = InControlRig;

	if (UControlRig* ControlRig = WeakControlRig.Get())
	{
		OnControlRigInitialized(ControlRig, FRigUnit_BeginExecution::EventName);
		ControlRig->OnInitialized_AnyThread().AddUObject(this, &UControlRigValidator::OnControlRigInitialized);
		ControlRig->OnExecuted_AnyThread().AddUObject(this, &UControlRigValidator::OnControlRigExecuted);
		ValidationContext.DrawInterface = &ControlRig->DrawInterface;
	}
}

void UControlRigValidator::OnControlRigInitialized(URigVMHost* Subject, const FName& EventName)
{
	if (UControlRig* ControlRig = WeakControlRig.Get())
	{
		if (ControlRig != Subject)
		{
			return;
		}

		ValidationContext.Clear();

		for (int32 PassIndex = 0; PassIndex < Passes.Num(); PassIndex++)
		{
			Passes[PassIndex]->OnInitialize(ControlRig, &ValidationContext);
		}
	}
}

void UControlRigValidator::OnControlRigExecuted(URigVMHost* Subject, const FName& EventName)
{
	if (UControlRig* ControlRig = WeakControlRig.Get())
	{
		if (ControlRig != Subject)
		{
			return;
		}

		for (int32 PassIndex = 0; PassIndex < Passes.Num(); PassIndex++)
		{
			Passes[PassIndex]->OnEvent(ControlRig, EventName, &ValidationContext);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
// UControlRigValidationPass
////////////////////////////////////////////////////////////////////////////////

UControlRigValidationPass::UControlRigValidationPass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

