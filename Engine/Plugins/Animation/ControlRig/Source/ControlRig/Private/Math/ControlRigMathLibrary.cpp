// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/ControlRigMathLibrary.h"
#include "TwoBoneIK.h"

void FControlRigMathLibrary::SolveBasicTwoBoneIK(FTransform& BoneA, FTransform& BoneB, FTransform& Effector, const FVector& PoleVector, const FVector& PrimaryAxis, const FVector& SecondaryAxis, float SecondaryAxisWeight, float BoneALength, float BoneBLength, bool bEnableStretch, float StretchStartRatio, float StretchMaxRatio)
{
	FVector RootPos = BoneA.GetLocation();
	FVector ElbowPos = BoneB.GetLocation();
	FVector EffectorPos = Effector.GetLocation();

	AnimationCore::SolveTwoBoneIK(RootPos, ElbowPos, EffectorPos, PoleVector, EffectorPos, ElbowPos, EffectorPos, BoneALength, BoneBLength, bEnableStretch, StretchStartRatio, StretchMaxRatio);

	BoneB.SetLocation(ElbowPos);
	Effector.SetLocation(EffectorPos);

	FVector Axis = BoneA.TransformVectorNoScale(PrimaryAxis);
	FVector Target1 = BoneB.GetLocation() - BoneA.GetLocation();
	bool bIsFullyExtended = false;

	FVector BoneBLocation = BoneB.GetLocation();
	if (!Target1.IsNearlyZero() && !Axis.IsNearlyZero())
	{
		Target1 = Target1.GetSafeNormal();

		// Compute if the elbow is placed in the segment between the root and the effector (the limb is completely extended)
		{
			FVector BaseDirection = (EffectorPos - RootPos).GetSafeNormal();
			if (FMath::Abs(FVector::DotProduct(BaseDirection, Target1) - 1.0f) < KINDA_SMALL_NUMBER)
			{
				bIsFullyExtended = true;
			}
		}
		
		FQuat Rotation1 = FQuat::FindBetweenNormals(Axis, Target1);
		BoneA.SetRotation((Rotation1 * BoneA.GetRotation()).GetNormalized());

		Axis = BoneA.TransformVectorNoScale(SecondaryAxis);

		if (SecondaryAxisWeight > SMALL_NUMBER)
		{
			FVector Target2 =  bIsFullyExtended ? PoleVector - RootPos : BoneBLocation - (Effector.GetLocation() + BoneA.GetLocation()) * 0.5f;
			if (!Target2.IsNearlyZero() && !Axis.IsNearlyZero())
			{
				Target2 = Target2 - FVector::DotProduct(Target2, Target1) * Target1;
				Target2 = Target2.GetSafeNormal();

				FQuat Rotation2 = FQuat::FindBetweenNormals(Axis, Target2);
				if (!FMath::IsNearlyEqual(SecondaryAxisWeight, 1.f))
				{
					FVector RotationAxis = Rotation2.GetRotationAxis();
					float RotationAngle = Rotation2.GetAngle();
					Rotation2 = FQuat(RotationAxis, RotationAngle * FMath::Clamp<float>(SecondaryAxisWeight, 0.f, 1.f));
				}
				BoneA.SetRotation((Rotation2 * BoneA.GetRotation()).GetNormalized());
			}
		}
	}

	Axis = BoneB.TransformVectorNoScale(PrimaryAxis);
	Target1 = Effector.GetLocation() - BoneBLocation;
	if (!Target1.IsNearlyZero() && !Axis.IsNearlyZero())
	{
		Target1 = Target1.GetSafeNormal();
		FQuat Rotation1 = FQuat::FindBetweenNormals(Axis, Target1);
		BoneB.SetRotation((Rotation1 * BoneB.GetRotation()).GetNormalized());

		if (SecondaryAxisWeight > SMALL_NUMBER)
		{
			Axis = BoneB.TransformVectorNoScale(SecondaryAxis);
			FVector Target2 = bIsFullyExtended ?  PoleVector - BoneBLocation : BoneBLocation - (Effector.GetLocation() + BoneA.GetLocation()) * 0.5f;
			if (!Target2.IsNearlyZero() && !Axis.IsNearlyZero())
			{
				Target2 = Target2 - FVector::DotProduct(Target2, Target1) * Target1;
				Target2 = Target2.GetSafeNormal();

				FQuat Rotation2 = FQuat::FindBetweenNormals(Axis, Target2);
				if (!FMath::IsNearlyEqual(SecondaryAxisWeight, 1.f))
				{
					FVector RotationAxis = Rotation2.GetRotationAxis();
					float RotationAngle = Rotation2.GetAngle();
					Rotation2 = FQuat(RotationAxis, RotationAngle * FMath::Clamp<float>(SecondaryAxisWeight, 0.f, 1.f));
				}
				BoneB.SetRotation((Rotation2 * BoneB.GetRotation()).GetNormalized());
			}
		}
	}
}
