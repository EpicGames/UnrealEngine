// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CRSimContainer)

void FCRSimContainer::Reset()
{
	AccumulatedTime = TimeLeftForStep = 0.f;
}

void FCRSimContainer::ResetTime()
{
	AccumulatedTime = TimeLeftForStep = 0.f;
}

void FCRSimContainer::StepVerlet(float InDeltaTime, float InBlend)
{
	TimeLeftForStep -= InDeltaTime;

	if (TimeLeftForStep >= 0.f)
	{
		return;
	}

	while (TimeLeftForStep < 0.f)
	{
		TimeLeftForStep += TimeStep;
		AccumulatedTime += TimeStep;
		CachePreviousStep();
		IntegrateVerlet(InBlend);
	}
}

void FCRSimContainer::StepSemiExplicitEuler(float InDeltaTime)
{
	TimeLeftForStep -= InDeltaTime;

	if (TimeLeftForStep >= 0.f)
	{
		return;
	}

	while (TimeLeftForStep < 0.f)
	{
		TimeLeftForStep += TimeStep;
		AccumulatedTime += TimeStep;
		CachePreviousStep();
		IntegrateSemiExplicitEuler();
	}
}

