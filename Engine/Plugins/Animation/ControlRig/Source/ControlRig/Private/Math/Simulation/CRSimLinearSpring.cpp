// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimLinearSpring.h"
#include "Math/Simulation/CRSimUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CRSimLinearSpring)

void FCRSimLinearSpring::CalculateForPoints(const FRigVMSimPoint& InPointA, const FRigVMSimPoint& InPointB, FVector& ForceA, FVector& ForceB) const
{
	ForceA = ForceB = FVector::ZeroVector;

	float WeightA = 0.f, WeightB = 0.f;
	FCRSimUtils::ComputeWeightsFromMass(InPointA.Mass, InPointB.Mass, WeightA, WeightB);
	if (WeightA + WeightB <= SMALL_NUMBER)
	{
		return;
	}

	const FVector Direction = InPointA.Position - InPointB.Position;
	const float Distance = Direction.Size();
	if (Distance < SMALL_NUMBER)
	{
		return;
	}

	const FVector Displacement = Direction * (Equilibrium - Distance) / Distance;
	ForceA = Displacement * Coefficient * WeightA;
	ForceB = -Displacement * Coefficient * WeightB;
}

