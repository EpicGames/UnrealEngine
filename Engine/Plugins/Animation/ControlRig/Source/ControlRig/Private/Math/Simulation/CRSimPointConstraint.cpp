// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimPointConstraint.h"
#include "Math/Simulation/CRSimUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CRSimPointConstraint)

void FCRSimPointConstraint::Apply(FRigVMSimPoint& OutPointA, FRigVMSimPoint& OutPointB) const
{
	switch (Type)
	{
		case ECRSimConstraintType::Distance:
		case ECRSimConstraintType::DistanceFromA:
		case ECRSimConstraintType::DistanceFromB:
		{
			float WeightA = 0.f, WeightB = 0.f;
			switch (Type)
			{
				case ECRSimConstraintType::Distance:
				{
					FCRSimUtils::ComputeWeightsFromMass(OutPointA.Mass, OutPointB.Mass, WeightA, WeightB);
					break;
				}
				case ECRSimConstraintType::DistanceFromA:
				{
					WeightA = 0.f;
					WeightB = 1.f;
					break;
				}
				default:
				case ECRSimConstraintType::DistanceFromB:
				{
					WeightA = 1.f;
					WeightB = 0.f;
					break;
				}
			}
			if (WeightA + WeightB <= SMALL_NUMBER)
			{
				return;
			}

			const float Equilibrium = DataA.X;
			const FVector Direction = OutPointA.Position - OutPointB.Position;
			const float Distance = Direction.Size();
			if (Distance < SMALL_NUMBER)
			{
				break;
			}
			const FVector Displacement = Direction * (Equilibrium - Distance) / Distance;
			OutPointA.Position += Displacement * WeightA;
			OutPointB.Position -= Displacement * WeightB;
			break;
		}
		case ECRSimConstraintType::Plane:
		{
			const FVector& PlanePoint = DataA;
			const FVector& PlaneNormal = DataB;
			FVector Direction = OutPointA.Position - PlanePoint;
			OutPointA.Position -= FVector::DotProduct(PlaneNormal, Direction) * PlaneNormal;
			break;
		}
	}
}

