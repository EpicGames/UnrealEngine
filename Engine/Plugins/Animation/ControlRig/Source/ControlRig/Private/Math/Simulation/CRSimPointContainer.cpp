// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimPointContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CRSimPointContainer)

void FCRSimPointContainer::Reset()
{
	FCRSimContainer::Reset();
	Points.Reset();
	Springs.Reset();
	Constraints.Reset();
	PreviousStep.Reset();
}

void FCRSimPointContainer::ResetTime()
{
	FCRSimContainer::ResetTime();
	PreviousStep.Reset();
	for (FRigVMSimPoint& Point : Points)
	{
		Point.LinearVelocity = FVector::ZeroVector;
	}
}

FRigVMSimPoint FCRSimPointContainer::GetPointInterpolated(int32 InIndex) const
{
	if (TimeLeftForStep <= SMALL_NUMBER || PreviousStep.Num() != Points.Num())
	{
		return Points[InIndex];
	}

	float T = 1.f - TimeLeftForStep / TimeStep;
	const FRigVMSimPoint& PrevPoint = PreviousStep[InIndex];
	FRigVMSimPoint Point = Points[InIndex];
	Point.Position = FMath::Lerp<FVector>(PrevPoint.Position, Point.Position, T);
	Point.Size = FMath::Lerp<float>(PrevPoint.Size, Point.Size, T);
	Point.LinearVelocity = FMath::Lerp<FVector>(PrevPoint.LinearVelocity, Point.LinearVelocity, T);
	return Point;
}

void FCRSimPointContainer::CachePreviousStep()
{
	PreviousStep = Points;
	for (FRigVMSimPoint& Point : Points)
	{
		Point.LinearVelocity = FVector::ZeroVector;
	}
}

void FCRSimPointContainer::IntegrateVerlet(float InBlend)
{
	IntegrateSprings();
	IntegrateForcesAndVolumes();
	IntegrateVelocityVerlet(InBlend);
	ApplyConstraints();
}

void FCRSimPointContainer::IntegrateSemiExplicitEuler()
{
	IntegrateSprings();
	IntegrateForcesAndVolumes();
	IntegrateVelocitySemiExplicitEuler();
	ApplyConstraints();
}

void FCRSimPointContainer::IntegrateSprings()
{
	for (int32 SpringIndex = 0; SpringIndex < Springs.Num(); SpringIndex++)
	{
		const FCRSimLinearSpring& Spring = Springs[SpringIndex];
		if (Spring.SubjectA == INDEX_NONE || Spring.SubjectB == INDEX_NONE)
		{
			continue;
		}
		const FRigVMSimPoint& PrevPointA = PreviousStep[Spring.SubjectA];
		const FRigVMSimPoint& PrevPointB = PreviousStep[Spring.SubjectB];

		FVector ForceA = FVector::ZeroVector;
		FVector ForceB = FVector::ZeroVector;
		Spring.CalculateForPoints(PrevPointA, PrevPointB, ForceA, ForceB);

		FRigVMSimPoint& PointA = Points[Spring.SubjectA];
		FRigVMSimPoint& PointB = Points[Spring.SubjectB];
		PointA.LinearVelocity += ForceA;
		PointB.LinearVelocity += ForceB;
	}
}

void FCRSimPointContainer::IntegrateForcesAndVolumes()
{
	for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
	{
		for (int32 ForceIndex = 0; ForceIndex < Forces.Num(); ForceIndex++)
		{
			Points[PointIndex].LinearVelocity += Forces[ForceIndex].Calculate(PreviousStep[PointIndex], TimeStep);
		}
		for (int32 VolumeIndex = 0; VolumeIndex < CollisionVolumes.Num(); VolumeIndex++)
		{
			Points[PointIndex].LinearVelocity += CollisionVolumes[VolumeIndex].CalculateForPoint(PreviousStep[PointIndex], TimeStep);
		}
	}
}

void FCRSimPointContainer::IntegrateVelocityVerlet(float InBlend)
{
	for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
	{
		Points[PointIndex] = PreviousStep[PointIndex].IntegrateVerlet(Points[PointIndex].LinearVelocity, InBlend, TimeStep);
	}
}

void FCRSimPointContainer::IntegrateVelocitySemiExplicitEuler()
{
	for (int32 PointIndex = 0; PointIndex < Points.Num(); PointIndex++)
	{
		Points[PointIndex] = PreviousStep[PointIndex].IntegrateSemiExplicitEuler(Points[PointIndex].LinearVelocity, TimeStep);
	}
}

void FCRSimPointContainer::ApplyConstraints()
{
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ConstraintIndex++)
	{
		const FCRSimPointConstraint& Constraint = Constraints[ConstraintIndex];
		if (Constraint.SubjectA == INDEX_NONE)
		{
			continue;
		}

		FRigVMSimPoint& PointA = Points[Constraint.SubjectA];
		FRigVMSimPoint& PointB = Points[Constraint.SubjectB == INDEX_NONE ? Constraint.SubjectA : Constraint.SubjectB];
		Constraint.Apply(PointA, PointB);
	}
}

