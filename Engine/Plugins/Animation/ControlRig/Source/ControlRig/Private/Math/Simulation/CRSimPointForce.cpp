// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimPointForce.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CRSimPointForce)

FVector FCRSimPointForce::Calculate(const FRigVMSimPoint& InPoint, float InDeltaTime) const
{
	FVector Force = FVector::ZeroVector;
	if(InPoint.Mass > SMALL_NUMBER)
	{
		switch(ForceType)
		{
			case ECRSimPointForceType::Direction:
			{
				if(bNormalize)
				{
					Force = Vector.GetSafeNormal();
				}
				else
				{
					Force = Vector;
				}
				Force = Force * Coefficient;
				break;
			}
		}
	}
	return Force;
}

