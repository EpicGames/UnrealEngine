// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimSoftCollision.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CRSimSoftCollision)

float FCRSimSoftCollision::CalculateFalloff(const FCRSimSoftCollision& InCollision, const FVector& InPosition, float InSize, FVector& OutDirection)
{
	const FTransform& Transform = InCollision.Transform;

	float Distance = 0.f;
	OutDirection = FVector::ZeroVector;

	switch(InCollision.ShapeType)
	{
		case ECRSimSoftCollisionType::Plane:
		{
			OutDirection = Transform.TransformVectorNoScale(FVector(0.f, 0.f, 1.f));
			Distance = FMath::Max<float>(Transform.InverseTransformPosition(InPosition).Z - InSize, 0.f);
			break;
		}
		case ECRSimSoftCollisionType::Sphere:
		{
			OutDirection = InPosition - Transform.GetLocation();
			Distance = FMath::Max(OutDirection.Size() - InSize, 0.f);
			if(Distance > SMALL_NUMBER)
			{
				OutDirection = OutDirection.GetSafeNormal();
			}
			break;
		}
		case ECRSimSoftCollisionType::Cone:
		{
			const FVector Direction = InPosition - Transform.GetLocation();
			const FVector Tip = Transform.TransformPositionNoScale(FVector(0.f, 0.f, 1.f) * Direction.Size());
			OutDirection = InPosition - Tip;
			Distance = 0.f;
			if(!OutDirection.IsNearlyZero())
			{
				Distance = OutDirection.Size();
				OutDirection = OutDirection.GetSafeNormal();
				FVector ShrunkDirection = OutDirection * FMath::Max(Distance - InSize, 0.f);
				if (!ShrunkDirection.IsNearlyZero())
				{
					FVector Position = Tip + ShrunkDirection;
					Distance = FControlRigMathLibrary::AngleBetween(Position - Transform.GetLocation(), Transform.TransformVectorNoScale(FVector(0.f, 0.f, 1.f)));
					Distance = FMath::RadiansToDegrees(Distance) * 2.f; // min and max express half of the cone
				}
			}
			break;
		}
	}

	if (InCollision.bInverted)
	{
		OutDirection = -OutDirection;
	}

	if (InCollision.MaximumDistance <= InCollision.MinimumDistance)
	{
		if (Distance < InCollision.MinimumDistance)
		{
			return InCollision.bInverted ? 0.f : 1.f;
		}
		return InCollision.bInverted ? 1.f : 0.f;
	}

	float Ratio = 0.f;
	if (Distance < InCollision.MinimumDistance)
	{
		Ratio = 1.f;
	}
	else if (Distance < InCollision.MaximumDistance)
	{
		Ratio = (Distance - InCollision.MinimumDistance) / (InCollision.MaximumDistance - InCollision.MinimumDistance);
		Ratio = 1.f - FMath::Clamp<float>(Ratio, 0.f, 1.f);
	}

	if (InCollision.bInverted)
	{
		Ratio = 1.f - Ratio;
	}

	return FControlRigMathLibrary::EaseFloat(Ratio, InCollision.FalloffType);
}

FVector FCRSimSoftCollision::CalculateForPoint(const FRigVMSimPoint& InPoint, float InDeltaTime) const
{
	FVector Force = FVector::ZeroVector;
	if(InPoint.Mass > SMALL_NUMBER)
	{
		float Falloff = CalculateFalloff(*this, InPoint.Position, InPoint.Size, Force);
		Force = Force * Falloff * Coefficient;
	}
	return Force;
}


