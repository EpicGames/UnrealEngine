// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Simulation/CRSimUtils.h"

void FCRSimUtils::ComputeWeightsFromMass(float MassA, float MassB, float& OutWeightA, float& OutWeightB)
{
	const bool bIsDynamicA = MassA > SMALL_NUMBER;
	const bool bIsDynamicB = MassB > SMALL_NUMBER;

	OutWeightA = 0.f;
	OutWeightB = 0.f;

	if (bIsDynamicA && !bIsDynamicB)
	{
		OutWeightA = 1.f;
		OutWeightB = 0.f;
		return;
	}

	if (!bIsDynamicA && bIsDynamicB)
	{
		OutWeightA = 0.f;
		OutWeightB = 1.f;
		return;
	}

	float CombinedMass = MassA + MassB;
	if (CombinedMass > SMALL_NUMBER)
	{
		OutWeightA = FMath::Clamp<float>(MassB / CombinedMass, 0.f, 1.f);
		OutWeightB = FMath::Clamp<float>(MassA / CombinedMass, 0.f, 1.f);
	}
}
