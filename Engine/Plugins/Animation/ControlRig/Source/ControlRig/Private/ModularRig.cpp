// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRig.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_InteractionExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRigObjectBinding.h"
#include "Rigs/RigHierarchyController.h"
#include "ControlRigComponent.h"
#include "ControlRigRuntimeAsset.h"
#include "RigVMRuntimeAsset.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Units/Modules/RigUnit_ConnectorExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularRig)

#define LOCTEXT_NAMESPACE "ModularRig"

////////////////////////////////////////////////////////////////////////////////
// FModuleInstanceHandle
////////////////////////////////////////////////////////////////////////////////

FModuleInstanceHandle::FModuleInstanceHandle(const UModularRig* InModularRig, const FName& InModuleName)
: ModularRig(const_cast<UModularRig*>(InModularRig))
, ModuleName(InModuleName)
{
}

FModuleInstanceHandle::FModuleInstanceHandle(const UModularRig* InModularRig, const FRigModuleInstance* InModule)
: ModularRig(const_cast<UModularRig*>(InModularRig))
, ModuleName(InModule->Name)
{
}

const FRigModuleInstance* FModuleInstanceHandle::Get() const
{
	if(const UModularRig* ResolvedRig = ModularRig.Get())
	{
		return ResolvedRig->FindModule(ModuleName);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
// UModularRig
////////////////////////////////////////////////////////////////////////////////

UModularRig::UModularRig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UModularRig::BeginDestroy()
{
	Super::BeginDestroy();
	
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReinstanced.RemoveAll(this);
#endif
}

UControlRig* FRigModuleInstance::GetRig() const
{
	if(IsValid(RigPtr))
	{
		return RigPtr;
	}

	// reset the cache if it is not valid
	return RigPtr = nullptr;
}

void FRigModuleInstance::SetRig(UControlRig* InRig)
{
	UControlRig* PreviousRig = GetRig();
	if(PreviousRig && (PreviousRig != InRig))
	{
		UModularRig::DiscardModuleRig(PreviousRig);
	}

	// update the cache
	RigPtr = InRig;
	if (InRig)
	{
		RigAssetReference = InRig->GetAssetReference();
	}
	else
	{
		RigAssetReference.Reset();
	}
	
}

bool FRigModuleInstance::ContainsRig(const UControlRig* InRig) const
{
	if(InRig == nullptr)
	{
		return false;
	}
	if(RigPtr == InRig)
	{
		return true;
	}
	return false;
}

const FRigModuleReference* FRigModuleInstance::GetModuleReference() const
{
	if(const UControlRig* Rig = GetRig())
	{
		if(const UModularRig* ModularRig = Cast<UModularRig>(Rig->GetParentRig()))
		{
			const FModularRigModel& Model = ModularRig->GetModularRigModel();
			return Model.FindModule(Name);
		}
	}
	return nullptr;
}

const FRigModuleInstance* FRigModuleInstance::GetParentModule() const
{
	if(ParentModuleName.IsNone())
	{
		return nullptr;
	}
	if(CachedParentModule)
	{
		return CachedParentModule;
	}
	if(const UControlRig* Rig = GetRig())
	{
		if(const UModularRig* ModularRig = Cast<UModularRig>(Rig->GetParentRig()))
		{
			return ModularRig->FindModule(ParentModuleName);
		}
	}
	return nullptr;
}

const FRigModuleInstance* FRigModuleInstance::GetRootModule() const
{
	if(ParentModuleName.IsNone())
	{
		return this;
	}
	if(const FRigModuleInstance* ParentModule = GetParentModule())
	{
		return ParentModule->GetRootModule();
	}
	return nullptr;
}

const FRigConnectorElement* FRigModuleInstance::FindPrimaryConnector() const
{
	if(const UControlRig* Rig = GetRig())
	{
		if(const URigHierarchy* Hierarchy = Rig->GetHierarchy())
		{
			PrimaryConnector.UpdateCache(Hierarchy);
			if(PrimaryConnector.IsValid())
			{
				return Cast<FRigConnectorElement>(PrimaryConnector.GetElement());
			}

			const TArray<FRigConnectorElement*> AllConnectors = Hierarchy->GetConnectors();
			for(const FRigConnectorElement* Connector : AllConnectors)
			{
				if(Connector->IsPrimary())
				{
					const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connector->GetKey());
					if(!ConnectorModuleName.IsNone())
					{
						if(ConnectorModuleName == Name)
						{
							PrimaryConnector.UpdateCache(Connector->GetKey(), Hierarchy);
							return Cast<FRigConnectorElement>(PrimaryConnector.GetElement());
						}
					}
				}
			}
		}
	}
	return nullptr;
}

TArray<const FRigConnectorElement*> FRigModuleInstance::FindConnectors() const
{
	TArray<const FRigConnectorElement*> Connectors;
	if(const UControlRig* Rig = GetRig())
	{
		if(const URigHierarchy* Hierarchy = Rig->GetHierarchy())
		{
			const TArray<FRigConnectorElement*> AllConnectors = Hierarchy->GetConnectors();
			for(const FRigConnectorElement* Connector : AllConnectors)
			{
				const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connector->GetKey());
				if(!ConnectorModuleName.IsNone())
				{
					if(ConnectorModuleName == Name)
					{
						Connectors.Add(Connector);
					}
				}
			}
		}
	}
	return Connectors;
}

bool FRigModuleInstance::IsRootModule() const
{
	return ParentModuleName.IsNone();
}

FString FRigModuleInstance::GetModulePrefix() const
{
	return Name.ToString() + FRigHierarchyModulePath::ModuleNameSuffix;
}

FString FRigModuleInstance::GetModulePath_Deprecated() const
{
	return ParentPath_DEPRECATED.IsEmpty() ?
		Name.ToString() :
		URigHierarchy::JoinNameSpace_Deprecated(ParentPath_DEPRECATED, Name.ToString());
}

bool FRigModuleInstance::HasChildModule(const FName& InModuleName, bool bRecursive) const
{
	if(InModuleName.IsNone())
	{
		return false;
	}
	for(const FRigModuleInstance* ChildModule : CachedChildren)
	{
		if(ChildModule->Name == InModuleName)
		{
			return true;
		}
		if(bRecursive)
		{
			if(ChildModule->HasChildModule(InModuleName, bRecursive))
			{
				return true;
			}
		}
	}
	return false;
}

void FRigModuleInstance::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (RigClass_DEPRECATED)
		{
			RigAssetReference.Set(RigClass_DEPRECATED);
			RigClass_DEPRECATED = nullptr;
		}
	}
}

uint32 GetTypeHash(const FRigModuleExecutionElement& InElement)
{
	uint32 Hash = GetTypeHash(InElement.EventName);
	Hash = HashCombine(Hash, GetTypeHash(InElement.ModuleName));
	Hash = HashCombine(Hash, GetTypeHash(InElement.bExecuted));
	return Hash;
}

uint32 GetTypeHash(const FRigModuleExecutionQueue& InQueue)
{
	uint32 Hash = 0;
	for (const FRigModuleExecutionElement& Element : InQueue.Elements)
	{
		Hash = HashCombine(Hash, GetTypeHash(Element));
	}
	return Hash;
}

void UModularRig::PostInitProperties()
{
	Super::PostInitProperties();

	ModularRigModel.UpdateCachedChildren();
	ModularRigModel.Connections.UpdateFromConnectionList();
	UpdateSupportedEvents();
}

void UModularRig::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	ModularRigModel.ForEachModule([&OutDeps](const FRigModuleReference* Module) -> bool
	{
		OutDeps.AddUnique(Module->ControlRigAssetReference.Get());
		return true;
	});
}

void UModularRig::PostInitInstance(URigVMHost* InCDO)
{
	Super::PostInitInstance(InCDO);

#if WITH_EDITOR
	if (IsInGameThread())
	{
		FCoreUObjectDelegates::OnObjectsReinstanced.AddUObject(this, &UModularRig::OnObjectsReplaced);
	}
#endif
}

void UModularRig::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
		ModularRigModel.UpdateCachedChildren();
		ModularRigModel.Connections.UpdateFromConnectionList();
		UpdateCachedChildren();
	}
}

void UModularRig::PostLoad()
{
	Super::PostLoad();
	ModularRigModel.UpdateCachedChildren();
	ModularRigModel.Connections.UpdateFromConnectionList();
}

void UModularRig::InitializeVMs(bool bRequestInit)
{
	URigVMHost::Initialize(bRequestInit);
	ForEachModule([bRequestInit](const FRigModuleInstance* Module) -> bool
	{
		if (UControlRig* ModuleRig = Module->GetRig())
		{
			ModuleRig->InitializeVMs(bRequestInit);
		}
		return true;
	});
}

bool UModularRig::InitializeVMs(const FName& InEventName)
{
	URigVMHost::InitializeVM(InEventName);
	UpdateModuleHierarchyFromCDO();

	ForEachModule([&InEventName](const FRigModuleInstance* Module) -> bool
	{
		if (UControlRig* ModuleRig = Module->GetRig())
		{
			ModuleRig->InitializeVMs(InEventName);
		}
		return true;
	});
	return true;
}

void UModularRig::InitializeFromCDO()
{
	Super::InitializeFromCDO();
	UpdateModuleHierarchyFromCDO();
}

void UModularRig::UpdateModuleHierarchyFromCDO()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// keep the previous rigs around
		check(PreviousModuleRigs.IsEmpty());
		for (const FRigModuleInstance& Module : Modules)
        {
			if(UControlRig* ModuleRig = Module.GetRig())
			{
				if(IsValid(ModuleRig))
				{
					PreviousModuleRigs.Add(Module.Name, ModuleRig);
				}
			}
        }

		// don't destroy the rigs when resetting
		ResetModules(false);

		// the CDO owns the model - when we ask for the model we'll always
		// get the model from the CDO. we'll now add UObject module instances
		// for each module (data only) reference in the model.
		// Note: The CDO does not contain any UObject module instances itself.
		const FModularRigModel& Model = GetModularRigModel();
		Model.ForEachModule([this, &Model](const FRigModuleReference* InModuleReference) -> bool
		{
			check(InModuleReference);
			FControlRigAssetStrongReference AssetReference = InModuleReference->ControlRigAssetReference.Get();
			if (IsInGameThread() && !AssetReference.IsValid())
			{
				AssetReference = InModuleReference->ControlRigAssetReference.LoadStrongReference(/*bForceLoad=*/true);
			}
			
			if (AssetReference.IsValid())
			{
				(void)AddModuleInstance(
					InModuleReference->Name,
					AssetReference,
					FindModule(InModuleReference->ParentModuleName),
					Model.Connections.GetModuleConnectionMap(InModuleReference->Name),
					InModuleReference->ConfigOverrides);
			}

			// continue to the next module
			return true;
		});

		// discard any remaining rigs
		for(const TPair<FName, UControlRig*>& Pair : PreviousModuleRigs)
		{
			DiscardModuleRig(Pair.Value);
		}
		PreviousModuleRigs.Reset();

		// update the module variable bindings now - since for this all
		// modules have to exist first
		ForEachModule([this, &Model](const FRigModuleInstance* Module) -> bool
		{
			if(const FRigModuleReference* ModuleReference = Model.FindModule(Module->Name))
			{
				(void)SetModuleVariableBindings(ModuleReference->Name, ModuleReference->Bindings);
			}
			if(UControlRig* ModuleRig = Module->GetRig())
			{
				ModuleRig->Initialize();
			}
			return true;
		});

		UpdateCachedChildren();
		UpdateSupportedEvents();
	}
}

bool UModularRig::Execute_Internal(const FName& InEventName)
{
	if (VM)
	{
		FRigVMExtendedExecuteContext& ModularRigContext = GetRigVMExtendedExecuteContext();
		const FControlRigExecuteContext& PublicContext = ModularRigContext.GetPublicDataSafe<FControlRigExecuteContext>();
		const FRigUnitContext& UnitContext = PublicContext.UnitContext;
		const URigHierarchy* Hierarchy = GetHierarchy();

		ForEachModule([this, &InEventName, Hierarchy, &UnitContext](FRigModuleInstance* Module) -> bool
		{
			if (const UControlRig* ModuleRig = Module->GetRig())
			{
				if (!ModuleRig->SupportsEvent(InEventName))
				{
					return true;
				}

				// Only emit interaction event on this module if any of the interaction elements
				// belong to the module's namespace
				if (InEventName == FRigUnit_InteractionExecution::EventName)
				{
					const bool bIsInteracting = UnitContext.ElementsBeingInteracted.ContainsByPredicate(
						[Module, Hierarchy](const FRigElementKey& InteractionElement)
						{
							return Module->Name == Hierarchy->GetModuleFName(InteractionElement);
						});
					if (!bIsInteracting)
					{
						return true;
					}
				}

				ExecutionQueue.Add(FRigModuleExecutionElement(Module, InEventName));
			}
			return true;
		});

#if WITH_EDITOR
		if (EventQueueToRun.IsEmpty() || EventQueueToRun[0] == InEventName)
		{
			LastExecutedElements.Reset();
		}
#endif

		ExecuteQueue();

		if (bAccumulateTime)
		{
			AbsoluteTime += DeltaTime;
		}

#if WITH_EDITOR
		// copy the elements over while applying a lock
		if (EventQueueToRun.IsEmpty() || EventQueueToRun.Last() == InEventName)
		{
			UE::TWriteScopeLock WriteLock(LastExecutionQueueLock);
			LastExecutionQueue.Elements = LastExecutedElements;
			LastExecutedElements.Reset();
		}
#endif

		return true;
	}
	return false;
}

void UModularRig::Evaluate_AnyThread()
{
	ResetExecutionQueue();
	Super::Evaluate_AnyThread();
}

bool UModularRig::SupportsEvent(const FName& InEventName) const
{
	return GetSupportedEvents().Contains(InEventName);
}

const TArray<FName>& UModularRig::GetSupportedEvents() const
{
	if (SupportedEvents.IsEmpty())
	{
		UpdateSupportedEvents();
	}
	return SupportedEvents;
}

void UModularRig::GetControlsInOrder(TArray<FRigControlElement*>& SortedControls) const
{
	SortedControls.Reset();

	if(DynamicHierarchy == nullptr)
	{
		return;
	}

	TMap<FName, TArray<FRigControlElement*>> ControlsByModule;

	DynamicHierarchy->Traverse([&](FRigBaseElement* Element, bool& bContinue)
	{
		if (FRigControlElement* Control = Cast<FRigControlElement>(Element))
		{
			const FName ModuleName = DynamicHierarchy->GetModuleFName(Control->GetKey());
			ControlsByModule.FindOrAdd(ModuleName).AddUnique(Control);
		}
		bContinue = true;
	});

	ForEachModule([&ControlsByModule, &SortedControls](const FRigModuleInstance* Module) -> bool
	{
		if (const TArray<FRigControlElement*>* Controls = ControlsByModule.Find(Module->Name))
		{
			SortedControls.Append(*Controls);
		}
		return true;
	});
	
	if (TArray<FRigControlElement*>* Controls = ControlsByModule.Find(NAME_None))
	{
		SortedControls.Append(*Controls);
	}
}

const FModularRigSettings& UModularRig::GetModularRigSettings() const
{
	if(HasAnyFlags(RF_ClassDefaultObject))
	{
		return ModularRigSettings;
	}
	if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GeneratedBy.GetObject())
	{
		return RuntimeAsset->GetModularRigSettings();
	}
	else if (const UModularRig* CDO = Cast<UModularRig>(GetClass()->GetDefaultObject()))
	{
		return CDO->GetModularRigSettings();
	}
	return ModularRigSettings;
}

void UModularRig::ExecuteQueue()
{
	FRigVMExtendedExecuteContext& Context = GetRigVMExtendedExecuteContext();
	FControlRigExecuteContext& PublicContext = Context.GetPublicDataSafe<FControlRigExecuteContext>();
	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TMap<FRigModuleInstance*, FFirstEntryEventGuard> FirstModuleEvent;
#endif

	int32 PostConstructionSpawnIndex = INDEX_NONE;
	TArray<FRigModuleInstance*> ModulesWithPendingSpawnIndex;
	
	while(ExecutionQueue.IsValidIndex(ExecutionQueueFront))
	{
		FRigModuleExecutionElement& ExecutionElement = ExecutionQueue[ExecutionQueueFront];
		if (FRigModuleInstance* ModuleInstance = ExecutionElement.ModuleInstance)
		{
#if WITH_EDITOR
			LastExecutedElements.Add(ExecutionElement);
			LastExecutedElements.Last().bExecuted = false;
			LastExecutedElements.Last().DurationInMicroSeconds = 0;

			const uint64 StartCycles = FPlatformTime::Cycles64();
#endif
			
			if (UControlRig* ModuleRig = ModuleInstance->GetRig())
			{
				// Let the module know of the spawn index the hierarchy is currently going through
				if(ExecutionElement.EventName == FRigUnit_PrepareForExecution::EventName)
				{
					ModuleInstance->ConstructionSpawnStartIndex = PostConstructionSpawnIndex = Hierarchy->GetNextSpawnIndex();
					ModulesWithPendingSpawnIndex.Add(ModuleInstance);
				}
				else if(ExecutionElement.EventName == FRigUnit_PostPrepareForExecution::EventName)
				{
					ModuleInstance->PostConstructionSpawnStartIndex = Hierarchy->GetNextSpawnIndex();
					ModulesWithPendingSpawnIndex.Remove(ModuleInstance);
				}

				if (!ModuleRig->SupportsEvent(ExecutionElement.EventName))
				{
					ExecutionQueueFront++;
					continue;
				}

				// Make sure the hierarchy has the correct element redirector from this module rig
				FRigHierarchyRedirectorGuard ElementRedirectorGuard(ModuleRig);

				FRigVMExtendedExecuteContext& RigExtendedExecuteContext= ModuleRig->GetRigVMExtendedExecuteContext();

				// Make sure the hierarchy has the correct execute context with the rig module namespace
				FRigHierarchyExecuteContextBracket ExecuteContextBracket(Hierarchy, &RigExtendedExecuteContext);

				FControlRigExecuteContext& RigPublicContext = RigExtendedExecuteContext.GetPublicDataSafe<FControlRigExecuteContext>();
				FRigUnitContext& RigUnitContext = RigPublicContext.UnitContext;
				RigUnitContext = PublicContext.UnitContext;

				// forward important context info to each module
				RigPublicContext.SetEventName(ExecutionElement.EventName);
				RigPublicContext.SetDrawInterface(PublicContext.GetDrawInterface());
				RigPublicContext.SetDrawContainer(PublicContext.GetDrawContainer());
				RigPublicContext.RigModuleInstance = ExecutionElement.ModuleInstance;
				RigPublicContext.SetAbsoluteTime(PublicContext.GetAbsoluteTime());
				RigPublicContext.SetDeltaTime(PublicContext.GetDeltaTime());
				RigPublicContext.SetWorld(PublicContext.GetWorld());
				RigPublicContext.SetOwningActor(PublicContext.GetOwningActor());
				RigPublicContext.SetOwningComponent(PublicContext.GetOwningComponent());
#if WITH_EDITOR
				RigPublicContext.SetLog(PublicContext.GetLog());
#endif
				RigPublicContext.SetFramesPerSecond(PublicContext.GetFramesPerSecond());
#if WITH_EDITOR
				RigPublicContext.SetHostBeingDebugged(bIsBeingDebugged);
#endif
				RigPublicContext.SetToWorldSpaceTransform(PublicContext.GetToWorldSpaceTransform());
				RigPublicContext.OnAddShapeLibraryDelegate = PublicContext.OnAddShapeLibraryDelegate;
				RigPublicContext.OnShapeExistsDelegate = PublicContext.OnShapeExistsDelegate;
				RigPublicContext.RuntimeSettings = PublicContext.RuntimeSettings;

#if WITH_EDITOR
				// redirect the records to be recorded to the top level context
				const TGuardValue<FInstructionRecordContainer*> RecordsGuard(RigPublicContext.Records, PublicContext.Records);
				
				if (!FirstModuleEvent.Contains(ModuleInstance))
				{
					//ModuleRig->InstructionVisitInfo.FirstEntryEventInQueue = ExecutionElement.EventName;
					FirstModuleEvent.Add(ModuleInstance, FFirstEntryEventGuard(&ModuleRig->InstructionVisitInfo, ExecutionElement.EventName));
				}
#endif

				// re-initialize the module in case only the VM side got recompiled.
				// this happens when the user relies on auto recompilation when editing the
				// module (dependency) graph - by changing a value, add / remove nodes or links.
				if(ModuleRig->IsInitRequired())
				{
					const TGuardValue<float> AbsoluteTimeGuard(ModuleRig->AbsoluteTime, ModuleRig->AbsoluteTime);
					const TGuardValue<float> DeltaTimeGuard(ModuleRig->DeltaTime, ModuleRig->DeltaTime);
					if(!ModuleRig->InitializeVM(ExecutionElement.EventName))
					{
						ExecutionQueueFront++;
						continue;
					}

					// put the variable defaults back
					if(const FRigModuleReference* ModuleReference = GetModularRigModel().FindModule(ExecutionElement.ModuleName))
					{
						(void) ModuleReference->ConfigOverrides.CopyToSubject(ModuleRig->GetVariablesMemory(), ModuleRig->GetVariablesStruct(), ModuleRig->GetFName());
					}
				}

				// Update the interaction elements to show only the ones belonging to this module
				RigUnitContext.ElementsBeingInteracted = RigUnitContext.ElementsBeingInteracted.FilterByPredicate(
					[&ExecutionElement, Hierarchy](const FRigElementKey& Key)
				{
					return ExecutionElement.ModuleName == Hierarchy->GetModuleFName(Key);
				});
				RigUnitContext.InteractionType = RigUnitContext.ElementsBeingInteracted.IsEmpty() ?
					(uint8) EControlRigInteractionType::None
					: RigUnitContext.InteractionType;

				// Make sure the module's rig has the correct user data
				// The rig will combine the user data of the
				// - skeleton
				// - skeletalmesh
				// - SkeletalMeshComponent
				// - default control rig module
				// - outer modular rig
				// - external variables
				if (RigPublicContext.IsRunningAConstructionEvent())
				{
					RigPublicContext.AssetUserData.Reset();
					if(const TArray<UAssetUserData*>* ControlRigUserDataArray = ModuleRig->GetAssetUserDataArray())
					{
						RigPublicContext.AssetUserData.Reserve(ControlRigUserDataArray->Num());
						for(const UAssetUserData* ControlRigUserData : *ControlRigUserDataArray)
						{
							if (ControlRigUserData)
							{
								RigPublicContext.AssetUserData.Add(ControlRigUserData);
							}
						}
					}
				}

				// Copy variable bindings
				UpdateModuleVariables(ExecutionElement.ModuleInstance);
			
				ModuleRig->Execute_Internal(ExecutionElement.EventName);
				ExecutionElement.bExecuted = true;

#if WITH_EDITOR
				if (!LastExecutedElements.IsEmpty())
				{
					const uint64 Cycles = FPlatformTime::Cycles64() - StartCycles;
					LastExecutedElements.Last().bExecuted = true;
					LastExecutedElements.Last().DurationInMicroSeconds = static_cast<double>(Cycles) * FPlatformTime::GetSecondsPerCycle() * 1000.0 * 1000.0;
				}
#endif

				// Copy result of Connection event to the ModularRig's unit context
				if (ExecutionElement.EventName == FRigUnit_ConnectorExecution::EventName)
				{
					PublicContext.UnitContext.ConnectionResolve = RigPublicContext.UnitContext.ConnectionResolve;
				}
			}
		}
		
		ExecutionQueueFront++;
	}

	for(FRigModuleInstance* Module : ModulesWithPendingSpawnIndex)
	{
		Module->PostConstructionSpawnStartIndex = Hierarchy->GetNextSpawnIndex();
	}
}

void UModularRig::ResetExecutionQueue()
{
	ExecutionQueue.Reset();
	ExecutionQueueFront = 0;
#if WITH_EDITOR
	LastExecutedElements.Reset();
#endif
}

#if WITH_EDITOR

FRigModuleExecutionQueue UModularRig::GetLastExecutionQueue() const
{
	UE::TReadScopeLock ReadLock(LastExecutionQueueLock);
	// this performs a copy
	return LastExecutionQueue;
}

#endif

void UModularRig::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	if(Modules.IsEmpty())
	{
		return;
	}
	
	bool bPerformedChange = false;
	for(const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
	{
		UObject* NewObject = Pair.Value;
		if((NewObject == nullptr) || (NewObject->GetOuter() != this) || !NewObject->IsA<UControlRig>())
		{
			continue;
		}

		UControlRig* NewRig = CastChecked<UControlRig>(NewObject);

		// relying on GetFName since URigVMHost is overloading GetName()
		const FName& ModuleName = NewRig->GetFName();

		// if we find a matching module update it.
		// FRigModuleInstance::SetRig takes care of disregarding the previous module instance.
		if(FRigModuleInstance* Module = const_cast<FRigModuleInstance*>(FindModule(ModuleName)))
		{
			Module->SetRig(NewRig);
			NewRig->bCopyHierarchyBeforeConstruction = false;
			NewRig->SetDynamicHierarchy(GetHierarchy());
			NewRig->Initialize(true);
			bPerformedChange = true;
		}
	}

	if(bPerformedChange)
	{
		UpdateSupportedEvents();
		RequestInit();
	}
}

void UModularRig::ResetModules(bool bDestroyModuleRigs)
{
	for (FRigModuleInstance& Module : Modules)
	{
		Module.CachedChildren.Reset();
		Module.PrimaryConnector.Reset();

		if(bDestroyModuleRigs)
		{
			if (const UControlRig* ModuleRig = Module.GetRig())
			{
				check(ModuleRig->GetOuter() == this);
				// takes care of renaming / moving the rig to the transient package
				Module.SetRig(nullptr);
			}
		}
	}
	
	RootModules.Reset();
	Modules.Reset();
	SupportedEvents.Reset();
}

const FModularRigModel& UModularRig::GetModularRigModel() const
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = GetGeneratedByAsset().GetObject())
		{
			return RuntimeAsset->GetModularRigModel();
		}
		else
		{
			const UModularRig* CDO = GetClass()->GetDefaultObject<UModularRig>();
			return CDO->GetModularRigModel();
		}
	}
	return ModularRigModel;
}

void UModularRig::UpdateCachedChildren()
{
	bool bContainsDeprecatedParentPaths = false;
	for (FRigModuleInstance& Module : Modules)
	{
		if(!Module.ParentPath_DEPRECATED.IsEmpty())
		{
			bContainsDeprecatedParentPaths = true;
			break;
		}
	}

	TMap<FName, FRigModuleInstance*> NameToModule;
	NameToModule.Reserve(Modules.Num());

	TMap<FString, FRigModuleInstance*> PathToModule;
	if(bContainsDeprecatedParentPaths)
	{
		PathToModule.Reserve(Modules.Num());
	}

	for (FRigModuleInstance& Module : Modules)
	{
		Module.CachedParentModule = nullptr;
		Module.CachedChildren.Reset();
		NameToModule.Add(Module.Name, &Module);

		if(bContainsDeprecatedParentPaths)
		{
			PathToModule.Add(Module.GetModulePath_Deprecated(), &Module);
		}
	}
	
	RootModules.Reset();
	for (FRigModuleInstance& Module : Modules)
	{
		if (Module.IsRootModule())
		{
			RootModules.Add(&Module);
		}
		else
		{
			if (FRigModuleInstance** ParentModule = NameToModule.Find(Module.ParentModuleName))
			{
				Module.CachedParentModule = *ParentModule;
				(*ParentModule)->CachedChildren.Add(&Module);
			}
			else if (FRigModuleInstance** ParentModuleByPath = PathToModule.Find(Module.GetModulePath_Deprecated()))
			{
				Module.CachedParentModule = *ParentModuleByPath;
				(*ParentModuleByPath)->CachedChildren.Add(&Module);
			}
		}
	}
}

void UModularRig::UpdateSupportedEvents() const
{
	SupportedEvents.Reset();
	const FModularRigModel& Model = GetModularRigModel();
	SupportedEvents = Model.GetSupportedEvents();
}

TArray<FString> UModularRig::GetModulePaths() const
{
	TArray<FString> Paths;
	Paths.Reserve(Modules.Num());
	ForEachModule([&Paths](const FRigModuleInstance* Module) -> bool
	{
		// don't need to use AddUnique since module paths are
		// guaranteed to be unique already.
		Paths.Add(Module->GetModuleReference()->GetModulePath());
		return true;
	});
	return Paths;
}

TArray<FName> UModularRig::GetModuleNames() const
{
	TArray<FName> Names;
	Names.Reserve(Modules.Num());
	ForEachModule([&Names](const FRigModuleInstance* Module) -> bool
	{
		// module names are unique
		Names.Add(Module->Name);
		return true;
	});
	return Names;
}

void UModularRig::GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& Context)
{
	Super::GenerateRequiredPluginsData(Context);

	ForEachModule([this](const FRigModuleInstance* Module) -> bool
	{
		if(UControlRig* ModuleRig = Module->GetRig())
		{
			ModuleRig->GenerateRequiredPluginsData(ModuleRig->GetRigVMExtendedExecuteContext());

			const TArray<FString>& RequiredPluginsForModule = ModuleRig->GetRequiredPlugins();
			for (const FString& RequiredPluginForModule : RequiredPluginsForModule)
			{
				RequiredPlugins.AddUnique(RequiredPluginForModule);
			}
		}
		return true;
	});
}

FRigModuleInstance* UModularRig::AddModuleInstance(const FName& InModuleName, FControlRigAssetStrongReference InModuleSource, const FRigModuleInstance* InParent,
                                                   const FRigElementKeyRedirector::FKeyMap& InConnectionMap, const FControlRigOverrideContainer& InConfigValues) 
{
	// Make sure there are no name clashes
	FName ParentName = NAME_None;
	if (InParent)
	{
		for (const FRigModuleInstance* Child : InParent->CachedChildren)
		{
			if (Child->Name == InModuleName)
			{
				return nullptr;
			}
		}
		ParentName = InParent->Name;
	}
	else
	{
		for (const FRigModuleInstance* RootModule : RootModules)
		{
			if (RootModule->Name == InModuleName)
			{
				return nullptr;
			}
		}
	}

	// For now, lets only allow rig modules
	if (!InModuleSource.IsRigModule())
	{
		return nullptr;
	}

	// after this add_getref we shouldn't access the InParent directly
	// since the pointer likely is garbage at this point.
	FRigModuleInstance& NewModule = Modules.Add_GetRef(FRigModuleInstance());
	NewModule.Name = InModuleName;
	NewModule.ParentModuleName = ParentName;

	UControlRig* NewModuleRig = nullptr;

	// reuse existing module rig instances first
	if(UControlRig** ExistingModuleRigPtr = PreviousModuleRigs.Find(InModuleName))
	{
		if(UControlRig* ExistingModuleRig = *ExistingModuleRigPtr)
		{
			// again relying on GetFName since RigVMHost overloads GetName
			if(ExistingModuleRig->GetFName() == InModuleName && InModuleSource.IsSourceOf(ExistingModuleRig))
			{
				NewModuleRig = ExistingModuleRig;
			}
			else
			{
				DiscardModuleRig(ExistingModuleRig);
			}
			PreviousModuleRigs.Remove(InModuleName);
		}
	}

	if(NewModuleRig == nullptr)
	{
		NewModuleRig = InModuleSource.CreateInstance(this, InModuleName);
	}
	else
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = NewModuleRig->GetGeneratedByAsset())
		{
			TArray<FRigVMExternalVariable> ExternalVariables = RuntimeAsset->GetExternalVariables();
			for (FRigVMExternalVariable Variable : ExternalVariables)
			{
				const FProperty* Property = Variable.GetProperty();
				uint8* TargetData = Property->ContainerPtrToValuePtr<uint8>(NewModuleRig->GetVariablesMemory());
				Property->CopyCompleteValue(TargetData, Variable.GetMemory());
			}
		}
		else
		{
			// make sure to reset its public variables back to the value from the CDO
			const UControlRig* CDO = NewModuleRig->GetClass()->GetDefaultObject<UControlRig>();
			for (TFieldIterator<FProperty> PropertyIt( NewModuleRig->GetClass()); PropertyIt; ++PropertyIt)
			{
				const FProperty* Property = *PropertyIt;
				if(Property->IsNative())
				{
					continue;
				}
				Property->CopyCompleteValue_InContainer(NewModuleRig, CDO);
			}
		}
	}
	
	NewModule.SetRig(NewModuleRig);

	UpdateCachedChildren();
	const TArray<FName> NewModuleSupportedEvents = NewModule.GetRig()->GetSupportedEvents();
	for (const FName& EventName : NewModuleSupportedEvents)
	{
		SupportedEvents.AddUnique(EventName);
	}

	// Configure module
	{
		URigHierarchy* Hierarchy = GetHierarchy();
		FRigVMExtendedExecuteContext& ModuleContext = NewModuleRig->GetRigVMExtendedExecuteContext();
		FControlRigExecuteContext& ModulePublicContext = ModuleContext.GetPublicDataSafe<FControlRigExecuteContext>();
		NewModuleRig->RequestInit();
		NewModuleRig->bCopyHierarchyBeforeConstruction = false;
		NewModuleRig->SetDynamicHierarchy(Hierarchy);
		ModulePublicContext.Hierarchy = Hierarchy;
		ModulePublicContext.ControlRig = this;
		ModulePublicContext.RigModulePrefix = NewModuleRig->GetRigModulePrefix();
		ModulePublicContext.RigModulePrefixHash = GetTypeHash(ModulePublicContext.RigModulePrefix);
		NewModuleRig->SetElementKeyRedirector(FRigElementKeyRedirector(InConnectionMap, Hierarchy));

		UControlRig* ModuleRig = NewModule.GetRig();
		InConfigValues.CopyToSubject(ModuleRig->GetVariablesMemory(), ModuleRig->GetVariablesStruct(), ModuleRig->GetFName());
	}

	return &NewModule;
}

bool UModularRig::SetModuleVariableBindings(const FName& InModuleName, const TMap<FName, FString>& InVariableBindings)
{
	if(FRigModuleInstance* Module = const_cast<FRigModuleInstance*>(FindModule(InModuleName)))
	{
		Module->VariableBindings.Reset();
		
		for (const TPair<FName, FString>& Pair : InVariableBindings)
		{
			FString SourceModuleName, SourceVariableName = Pair.Value;
			(void)FRigHierarchyModulePath(Pair.Value).Split(&SourceModuleName, &SourceVariableName);
			FRigVMExternalVariable SourceVariable;
			if (SourceModuleName.IsEmpty())
			{
				// Use GetVariableByName directly: it already routes through GetVariablesMemory(),
				// which for BP-independent rigs points at the dynamic UPropertyBag (Variables.GetMutableValue().GetMemory())
				SourceVariable = GetVariableByName(*SourceVariableName);
			}
			else if(const FRigModuleInstance* SourceModule = FindModule(*SourceModuleName))
			{
				SourceVariable = SourceModule->GetRig()->GetPublicVariableByName(*SourceVariableName);
			}

			if(SourceVariable.GetProperty() == nullptr)
			{
				// todo: report error
				return false;
			}
			
			SourceVariable.SetName(*Pair.Value); // Adapt the name of the variable to contain the full path
			Module->VariableBindings.Add(Pair.Key, SourceVariable);
		}
		return true;
	}
	return false;
}

void UModularRig::UpdateModuleVariables(const FRigModuleInstance* InModule)
{
	if(const UControlRig* ModuleRig = InModule->GetRig())
	{
		for (const TPair<FName, FRigVMExternalVariable>& Pair : InModule->VariableBindings)
		{
			FRigVMExternalVariable TargetVariable = ModuleRig->GetPublicVariableByName(Pair.Key);
			if(ensure(TargetVariable.GetProperty()))
			{
				if (RigVMTypeUtils::AreCompatible(Pair.Value.GetProperty(), TargetVariable.GetProperty()))
				{
					Pair.Value.GetProperty()->CopyCompleteValue(TargetVariable.GetMemory(), Pair.Value.GetMemory());
				}
			}
		}
	}
}

void UModularRig::DiscardModuleRig(UControlRig* InControlRig)
{
	if(InControlRig)
	{
		// rename the previous rig.
		// GC will pick it up eventually - since we won't have any
		// owning pointers to it anymore.
		InControlRig->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		InControlRig->MarkAsGarbage();
	}
}

const FRigModuleInstance* UModularRig::FindModule(const FName& InModuleName) const
{
	return const_cast<UModularRig*>(this)->FindModule(InModuleName);
}

FRigModuleInstance* UModularRig::FindModule(const FName& InModuleName)
{
	FRigModuleInstance* Result = Modules.FindByPredicate([InModuleName](const FRigModuleInstance& Module)
	{
		return Module.Name == InModuleName;
	});
	if(Result == nullptr)
	{
		if(InModuleName.ToString().EndsWith(FRigHierarchyModulePath::ModuleNameSuffix))
		{
			// we should remove this and make sure clients call with proper names
			Result = FindModule(*InModuleName.ToString().LeftChop(1));
		}
	}
	return Result;
}

FRigModuleInstance* UModularRig::FindModule_Deprecated(const FString& InModulePath)
{
	return Modules.FindByPredicate([InModulePath](const FRigModuleInstance& Module)
	{
		return Module.GetModulePath_Deprecated().Equals(InModulePath, ESearchCase::IgnoreCase);
	});
}

const FRigModuleInstance* UModularRig::FindModule_Deprecated(const FString& InModulePath) const
{
	return const_cast<UModularRig*>(this)->FindModule_Deprecated(InModulePath);
}

const FRigModuleInstance* UModularRig::FindModule(const UControlRig* InModuleInstance) const
{
	return const_cast<UModularRig*>(this)->FindModule(InModuleInstance);
}

FRigModuleInstance* UModularRig::FindModule(const UControlRig* InModuleInstance)
{
	FRigModuleInstance* FoundModule = nullptr;
	ForEachModule([InModuleInstance, &FoundModule](FRigModuleInstance* Module) -> bool
	{
		if(Module->GetRig() == InModuleInstance)
		{
			FoundModule = Module;
			// don't continue ForEachModule
			return false;
		}
		return true;
	});

	return FoundModule;
}

const FRigModuleInstance* UModularRig::FindModule(const FRigBaseElement* InElement) const
{
	return const_cast<UModularRig*>(this)->FindModule(InElement);
}

FRigModuleInstance* UModularRig::FindModule(const FRigBaseElement* InElement)
{
	if(InElement)
	{
		return FindModule(InElement->GetKey());
	}
	return nullptr;
}

const FRigModuleInstance* UModularRig::FindModule(const FRigElementKey& InElementKey) const
{
	return const_cast<UModularRig*>(this)->FindModule(InElementKey);
}

FRigModuleInstance* UModularRig::FindModule(const FRigElementKey& InElementKey)
{
	if(const URigHierarchy* Hierarchy = GetHierarchy())
	{
		const FName ModuleName = Hierarchy->GetModuleFName(InElementKey);
		if(!ModuleName.IsNone())
		{
			return FindModule(ModuleName);
		}
	}
	return nullptr;
}

const UControlRig* UModularRig::GetModuleRig_Deprecated(FString InModulePath) const
{
	return const_cast<UModularRig*>(this)->GetModuleRig(InModulePath);
}

UControlRig* UModularRig::GetModuleRig(FString InModulePath)
{
	if(const FRigModuleInstance* Module = FindModule_Deprecated(InModulePath))
	{
		UpdateModuleVariables(Module);
		if(UControlRig* ModuleRig = Module->GetRig())
		{
			return ModuleRig;
		}
		UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't contain a rig instance.", *InModulePath);
		return nullptr;
	}
	UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't exist.", *InModulePath);
	return nullptr;
}

const UControlRig* UModularRig::GetModuleRigByName(FName InModuleName) const
{
	return const_cast<UModularRig*>(this)->GetModuleRigByName(InModuleName);
}

UControlRig* UModularRig::GetModuleRigByName(FName InModuleName)
{
	if(const FRigModuleInstance* Module = FindModule(InModuleName))
	{
		UpdateModuleVariables(Module);
		if(UControlRig* ModuleRig = Module->GetRig())
		{
			return ModuleRig;
		}
		UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't contain a rig instance.", *InModuleName.ToString());
		return nullptr;
	}
	UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't exist.", *InModuleName.ToString());
	return nullptr;
}

FString UModularRig::GetParentPathForBP(FString InModulePath) const
{
	if (const FRigModuleInstance* Module = FindModule(*InModulePath))
	{
		if(const FRigModuleReference* Reference = Module->GetModuleReference())
		{
			return Reference->GetModulePath();
		}
	}
	return FString();
}

FName UModularRig::GetParentModuleNameForBP(FName InModuleName) const
{
	return GetParentModuleName(InModuleName);
}

FName UModularRig::GetParentModuleName(const FName& InModuleName) const
{
	if (const FRigModuleInstance* Module = FindModule(InModuleName))
	{
		return Module->ParentModuleName;
	}
	return NAME_None;
}

void UModularRig::ForEachModule(TFunctionRef<bool(FRigModuleInstance*)> PerModuleFunction, bool bDepthFirst)
{
	if (bDepthFirst)
	{
		for (FRigModuleInstance* Module : RootModules)
		{
			if (!TraverseModules(Module, PerModuleFunction))
			{
				break;
			}
		}
	}
	else
	{
		TArray<FRigModuleInstance*> ModuleInstances;
		ModuleInstances.Reserve(Modules.Num());
		ModuleInstances.Append(RootModules);
		
		for (int32 ModuleIndex = 0; ModuleIndex < ModuleInstances.Num(); ++ModuleIndex)
		{
			if (!PerModuleFunction(ModuleInstances[ModuleIndex]))
			{
				break;
			}
			ModuleInstances.Append(ModuleInstances[ModuleIndex]->CachedChildren);
		}
	}
}

void UModularRig::ForEachModule(TFunctionRef<bool(const FRigModuleInstance*)> PerModuleFunction, bool bDepthFirst) const
{
	const_cast<UModularRig*>(this)->ForEachModule([&PerModuleFunction](FRigModuleInstance* InModuleInstance) -> bool
	{
		return PerModuleFunction(InModuleInstance);
	}, bDepthFirst);
}

bool UModularRig::TraverseModules(FRigModuleInstance* InModuleInstance, TFunctionRef<bool(FRigModuleInstance*)> PerModule)
{
	if (!PerModule(InModuleInstance))
	{
		return false;
	}
	for (FRigModuleInstance* ChildModule : InModuleInstance->CachedChildren)
	{
		if (!TraverseModules(ChildModule, PerModule))
		{
			return false;
		}
	}
	return true;
}

void UModularRig::ExecuteConnectorEvent(const FRigElementKey& InConnector, const FRigModuleInstance* InModuleInstance, const FRigElementKeyRedirector* InRedirector, TArray<FRigElementResolveResult>& InOutCandidates)
{
	if (!InModuleInstance)
	{
		InOutCandidates.Reset();
		return;
	}

	if (!InRedirector)
	{
		InOutCandidates.Reset();
		return;
	}
	
	FRigModuleInstance* Module = Modules.FindByPredicate([InModuleInstance](FRigModuleInstance& Instance)
	{
		return &Instance == InModuleInstance;
	});
	if (!Module)
	{
		InOutCandidates.Reset();
		return;
	}

	const TArray<FRigElementResolveResult> Candidates = InOutCandidates;
	
	FControlRigExecuteContext& PublicContext = GetRigVMExtendedExecuteContext().GetPublicDataSafe<FControlRigExecuteContext>();

	FString ShortConnectorName = InConnector.Name.ToString();
	ShortConnectorName.RemoveFromStart(Module->GetModulePrefix());
	TGuardValue<FRigElementKey> ConnectorGuard(PublicContext.UnitContext.ConnectionResolve.Connector, FRigElementKey(*ShortConnectorName, InConnector.Type));
	TGuardValue<TArray<FRigElementResolveResult>> CandidatesGuard(PublicContext.UnitContext.ConnectionResolve.Matches, Candidates);
	TGuardValue<TArray<FRigElementResolveResult>> MatchesGuard(PublicContext.UnitContext.ConnectionResolve.Excluded, {});
	
	TGuardValue<FRigElementKeyRedirector> RedirectorGuard(ElementKeyRedirector, *InRedirector);
	ExecuteEventOnModule(FRigUnit_ConnectorExecution::EventName, Module);
	ExecuteQueue();

	InOutCandidates = PublicContext.UnitContext.ConnectionResolve.Matches;
}

TArray<FName> UModularRig::GetEventsForAllModules() const
{
	TArray<FName> Events;
	ForEachModule([&Events](const FRigModuleInstance* Module) -> bool
	{
		if(const UControlRig* ModuleRig = Module->GetRig())
		{
			const TArray<FName> ModuleSupportedEvents = ModuleRig->GetSupportedEvents();
			for(const FName& SupportedEvent : ModuleSupportedEvents)
			{
				Events.AddUnique(SupportedEvent);
			}
		}
		return true;
	});
	return Events;
}

TArray<FName> UModularRig::GetEventsForModule(FString InModulePath) const
{
	if(const UControlRig* ModuleRig = GetModuleRig_Deprecated(InModulePath))
	{
		return ModuleRig->GetSupportedEvents();
	}
	return TArray<FName>();
}

TArray<FName> UModularRig::GetEventsForModuleByName(FName InModuleName) const
{
	if(const UControlRig* ModuleRig = GetModuleRigByName(InModuleName))
	{
		return ModuleRig->GetSupportedEvents();
	}
	return TArray<FName>();
}

TArray<FName> UModularRig::ExecuteEventOnAllModules(FName InEvent)
{
	TArray<FRigModuleExecutionElement> QueueForEvent;

	ForEachModule([&QueueForEvent, &InEvent](FRigModuleInstance* Module) -> bool
	{
		if(const UControlRig* ModuleRig = Module->GetRig())
		{
			if(ModuleRig->SupportsEvent(InEvent))
			{
				QueueForEvent.Emplace(Module, InEvent);
			}
		}
		return true;
	});

	TArray<FName> ModulesWhichRanEvent;
	if(QueueForEvent.IsEmpty())
	{
		UE_LOGF(LogControlRig, Error, "Event '%ls' is not supported by any module on this modular rig.", *InEvent.ToString());
		return ModulesWhichRanEvent;
	}
	
	TGuardValue<TArray<FRigModuleExecutionElement>> ExecutionGuard(ExecutionQueue, QueueForEvent);
	TGuardValue<int32> ExecutionFrontGuard(ExecutionQueueFront, 0);
	ExecuteQueue();

	for(const FRigModuleExecutionElement& ExecutionElement : ExecutionQueue)
	{
		if(ExecutionElement.bExecuted)
		{
			ModulesWhichRanEvent.Add(ExecutionElement.ModuleName);
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Module '%ls' did not run event '%ls' successfully.", *ExecutionElement.ModuleName.ToString(), *InEvent.ToString());
		}
	}
		
	return ModulesWhichRanEvent;
}

bool UModularRig::ExecuteEventOnModuleForBP(FName InEvent, FString InModulePath)
{
	if(FRigModuleInstance* Module = FindModule_Deprecated(InModulePath))
	{
		if(const UControlRig* ModuleRig = Module->GetRig())
		{
			if(ModuleRig->SupportsEvent(InEvent))
			{
				return ExecuteEventOnModule(InEvent, Module); 
			}
			UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't support the event '%ls'.", *InModulePath, *InEvent.ToString());
			return false;
		}
		UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't contain a rig instance.", *InModulePath);
		return false;
	}
	UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't exist.", *InModulePath);
	return false;
}

bool UModularRig::ExecuteEventOnModuleByNameForBP(FName InEvent, FName InModuleName)
{
	if(FRigModuleInstance* Module = FindModule(InModuleName))
	{
		if(const UControlRig* ModuleRig = Module->GetRig())
		{
			if(ModuleRig->SupportsEvent(InEvent))
			{
				return ExecuteEventOnModule(InEvent, Module); 
			}
			UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't support the event '%ls'.", *InModuleName.ToString(), *InEvent.ToString());
			return false;
		}
		UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't contain a rig instance.", *InModuleName.ToString());
		return false;
	}
	UE_LOGF(LogControlRig, Error, "Module '%ls' doesn't exist.", *InModuleName.ToString());
	return false;
}

bool UModularRig::ExecuteEventOnModule(const FName& InEvent, FRigModuleInstance* InModule)
{
	check(InModule);

	TArray<FRigModuleExecutionElement> QueueForEvent;
	QueueForEvent.Emplace(InModule, InEvent);
	TGuardValue<TArray<FRigModuleExecutionElement>> ExecutionGuard(ExecutionQueue, QueueForEvent);
	TGuardValue<int32> ExecutionFrontGuard(ExecutionQueueFront, 0);
	ExecuteQueue();
	return ExecutionQueue[0].bExecuted;
}

#undef LOCTEXT_NAMESPACE
