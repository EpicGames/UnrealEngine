// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigController.h"

#include "ControlRig.h"
#include "ModularRig.h"
#include "ModularRigModel.h"
#include "ModularRigRuleManager.h"
#if WITH_EDITOR
#include "PropertyPath.h"
#endif
#include "ControlRigRuntimeAsset.h"
#include "Misc/DefaultValueHelper.h"
#include "Rigs/RigHierarchyController.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularRigController)

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#endif

UModularRigController::UModularRigController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Model(nullptr)
	, bSuspendNotifications(false)
	, bAutomaticReparenting(true)
{
}

FName UModularRigController::AddModule(const FName& InModuleName, TSubclassOf<UControlRig> InClass, const FName& InParentModuleName, bool bSetupUndo)
{
	if (!InClass)
	{
		UE_LOGF(LogControlRig, Error, "Invalid InClass");
		return NAME_None;
	}

	UControlRig* ClassDefaultObject = InClass->GetDefaultObject<UControlRig>();
	if (!ClassDefaultObject->IsRigModule())
	{
		UE_LOGF(LogControlRig, Error, "Class %ls is not a rig module", *InClass->GetClassPathName().ToString());
		return NAME_None;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "AddModuleTransaction", "Add Module"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif 

	const FName SanitizedName = GetSafeNewName(InModuleName);
	Model->Modules.Add(FRigModuleReference(SanitizedName, TSoftClassPtr<UControlRig>(InClass), InParentModuleName, Model));
	FRigModuleReference* NewModule = &Model->Modules.Last();

	Model->UpdateCachedChildren();

	Notify(EModularRigNotification::ModuleAdded, NewModule);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	return NewModule->GetFName();
}

FName UModularRigController::AddModuleFromAssetReference(const FName& InModuleName, FControlRigAssetStrongReference InHandle, const FName& InParentModuleName, bool bSetupUndo)
{
	if (!InHandle.IsValid())
    {
    	UE_LOGF(LogControlRig, Error, "Invalid InClass");
    	return NAME_None;
    }

    if (!InHandle.IsRigModule())
    {
    	UE_LOGF(LogControlRig, Error, "Class %ls is not a rig module", *InHandle.GetPathName());
    	return NAME_None;
    }

#if WITH_EDITOR
    TSharedPtr<FScopedTransaction> TransactionPtr;
    if (bSetupUndo)
    {
    	TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "AddModuleTransaction", "Add Module"), !GIsTransacting);
    	GetOuter()->Modify();
    }
#endif 

    const FName SanitizedName = GetSafeNewName(InModuleName);
    Model->Modules.Add(FRigModuleReference(SanitizedName, InHandle.Get(), InParentModuleName, Model));
    FRigModuleReference* NewModule = &Model->Modules.Last();

    Model->UpdateCachedChildren();

    Notify(EModularRigNotification::ModuleAdded, NewModule);

#if WITH_EDITOR
    TransactionPtr.Reset();
#endif

    return NewModule->GetFName();
}

FRigModuleReference* UModularRigController::FindModule(const FName& InModuleName)
{
	return Model->FindModule(InModuleName);
}

const FRigModuleReference* UModularRigController::FindModule(const FName& InModuleName) const
{
	return const_cast<UModularRigController*>(this)->FindModule(InModuleName);
}

FRigModuleReference UModularRigController::GetModuleReference(FName InModuleName) const
{
	if(const FRigModuleReference* Module = FindModule(InModuleName))
	{
		return *Module;
	}
	return FRigModuleReference();
}

TArray<FRigElementKey> UModularRigController::GetConnectorsForModule(FName InModuleName) const
{
	TArray<FRigElementKey> ConnectorsForModule;

	const FString ModuleNameString = InModuleName.ToString();
	if(const FRigModuleReference* Module = FindModule(InModuleName))
	{
		const TArray<FRigModuleConnector>& ExposedConnectors = Module->ControlRigAssetReference.LoadStrongReference().GetRigModuleSettings().ExposedConnectors;
		for(const FRigModuleConnector& ExposedConnector : ExposedConnectors)
		{
			ConnectorsForModule.Emplace(FRigHierarchyModulePath(ModuleNameString, ExposedConnector.Name).GetPathFName(), ERigElementType::Connector);
		}
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *ModuleNameString);
	}

	return ConnectorsForModule;
}

bool UModularRigController::CanConnectConnectorToElement(const FRigElementKey& InConnectorKey, const FRigElementKey& InTargetKey, FText& OutErrorMessage)
{
	return CanConnectConnectorToElements(InConnectorKey, {InTargetKey}, OutErrorMessage);
}

bool UModularRigController::CanConnectConnectorToElements(const FRigElementKey& InConnectorKey, const TArray<FRigElementKey>& InTargetKeys,
	FText& OutErrorMessage)
{
	const FRigHierarchyModulePath ConnectorModulePath(InConnectorKey.Name);
	if (!ConnectorModulePath.IsValid())
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Connector %s does not contain a module / namespace"), *InConnectorKey.ToString()));
		return false;
	}
	
	FRigModuleReference* Module = FindModule(ConnectorModulePath.GetModuleFName());
	if (!Module)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find module %s"), *ConnectorModulePath.GetModuleNameString()));
		return false;
	}
	
	const FControlRigAssetStrongReference ModuleAssetReference = Module->ControlRigAssetReference.LoadStrongReference();
	const FRigModuleConnector* ModuleConnector = ModuleAssetReference.GetRigModuleSettings().ExposedConnectors.FindByPredicate(
		[&ConnectorModulePath](FRigModuleConnector& Connector)
		{
			return Connector.Name == ConnectorModulePath.GetElementFName();
		});
	if (!ModuleConnector)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find connector %s in class %s"), *ConnectorModulePath.GetElementNameString(), *ModuleAssetReference.GetPathName()));
		return false;
	}

	if (InTargetKeys.IsEmpty())
	{
		OutErrorMessage = FText::FromString(TEXT("Invalid empty target array"));
		return false;
	}

	for(const FRigElementKey& InTargetKey : InTargetKeys)
	{
		if (!InTargetKey.IsValid())
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Invalid target %s in class %s"), *InTargetKey.ToString(), *ModuleAssetReference.GetPathName()));
			return false;
		}

		if (InTargetKey == InConnectorKey)
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Cannot resolve connector %s to itself in class %s"), *InTargetKey.ToString(), *ModuleAssetReference.GetPathName()));
			return false;
		}

		FRigElementKey CurrentTarget = Model->Connections.FindTargetFromConnector(InConnectorKey);
		if (CurrentTarget.IsValid() && InTargetKey == CurrentTarget)
		{
			return true; // Nothing to do
		}

		if (!ModuleConnector->IsPrimary())
		{
			const FRigModuleConnector* PrimaryModuleConnector = ModuleAssetReference.GetRigModuleSettings().ExposedConnectors.FindByPredicate(
			[](const FRigModuleConnector& Connector)
			{
				return Connector.IsPrimary();
			});

			const FRigHierarchyModulePath PrimaryConnectorPath = ConnectorModulePath.ReplaceElementName(PrimaryModuleConnector->Name);
			const FRigElementKey PrimaryConnectorKey(PrimaryConnectorPath.GetPathFName(), ERigElementType::Connector);
			const FRigElementKey PrimaryTarget = Model->Connections.FindTargetFromConnector(PrimaryConnectorKey);
			if (!PrimaryTarget.IsValid())
			{
				OutErrorMessage = FText::FromString(FString::Printf(TEXT("Cannot resolve connector %s because primary connector is not resolved"), *InConnectorKey.ToString()));
				return false;
			}
		}
	}

#if WITH_EDITOR
	
	// Make sure the connection is valid
	{
		UModularRig* ModularRig = GetDebuggedModularRig();
		if (!ModularRig)
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find debugged modular rig in %s"), *GetOuter()->GetPathName()));
			return false;
		}

		URigHierarchy* Hierarchy = ModularRig->GetHierarchy();
		if (!Hierarchy)
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find hierarchy in %s"), *ModularRig->GetPathName()));
			return false;
		}

		const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(Hierarchy->Find(InConnectorKey));
		if (!Connector)
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find connector %s"), *InConnectorKey.ToString()));
			return false;
		}

		if(!Connector->IsArrayConnector())
		{
			if(InTargetKeys.Num() > 1)
			{
				OutErrorMessage = FText::FromString(FString::Printf(TEXT("Connector %s can only be resolved to one target - it is not an array connector."), *InConnectorKey.ToString()));
				return false;
			}
		}

		UModularRigRuleManager* RuleManager = Hierarchy->GetRuleManager();
		if (!RuleManager)
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not get rule manager")));
			return false;
		}

		const FRigModuleInstance* ModuleInstance = ModularRig->FindModule(Module->GetFName());
		FModularRigResolveResult RuleResults = RuleManager->FindMatches(Connector, ModuleInstance, ModularRig->GetElementKeyRedirector());
		for(const FRigElementKey& InTargetKey : InTargetKeys)
		{
			if (!RuleResults.ContainsMatch(InTargetKey))
			{
				OutErrorMessage = FText::FromString(FString::Printf(TEXT("The target %s is not a valid match for connector %s"), *InTargetKey.ToString(), *InConnectorKey.ToString()));
				return false;
			}
		}
	}
#endif
	return true;
}

bool UModularRigController::ConnectConnectorToElement(const FRigElementKey& InConnectorKey, const FRigElementKey& InTargetKey, bool bSetupUndo, bool bAutoResolveOtherConnectors, bool bCheckValidConnection)
{
	return ConnectConnectorToElements(InConnectorKey, {InTargetKey}, bSetupUndo, bAutoResolveOtherConnectors, bCheckValidConnection);
}

bool UModularRigController::ConnectConnectorToElements(const FRigElementKey& InConnectorKey, const TArray<FRigElementKey>& InTargetKeys,
	bool bSetupUndo, bool bAutoResolveOtherConnectors, bool bCheckValidConnection)
{
	const FRigElementKeyRedirector::FKeyArray TargetKeys(InTargetKeys);
	if(TargetKeys.IsEmpty())
	{
		UE_LOGF(LogControlRig, Error, "Could not connect %ls: Target Keys array is empty.", *InConnectorKey.ToString());
		return false;
	}
	
	FText ErrorMessage;
	auto GenerateTargetKeyString = [&TargetKeys]() -> FString
	{
		FString TargetKeyString = TargetKeys[0].ToString();
		if(TargetKeys.Num() > 1)
		{
			for(int32 Index = 1; Index < TargetKeys.Num(); Index++)
			{
				TargetKeyString = TargetKeyString + TEXT(", ") + TargetKeys[Index].ToString();
			}
			TargetKeyString = TEXT("{") + TargetKeyString + TEXT("}");
		}
		return TargetKeyString;
	};
	
	if (bCheckValidConnection && !CanConnectConnectorToElements(InConnectorKey, InTargetKeys, ErrorMessage))
	{
		FString TargetKeyString = GenerateTargetKeyString();
		UE_LOGF(LogControlRig, Error, "Could not connect %ls to %ls: %ls", *InConnectorKey.ToString(), *TargetKeyString, *ErrorMessage.ToString());
		return false;
	}
	

	const FRigHierarchyModulePath ConnectorModulePath(InConnectorKey.Name);
	FRigModuleReference* Module = FindModule( ConnectorModulePath.GetModuleFName());

	FRigElementKey CurrentTarget = Model->Connections.FindTargetFromConnector(InConnectorKey);

#if WITH_EDITOR
	TOptional<FName> TargetModuleName;
	UModularRig* ModularRig = GetDebuggedModularRig();
	if(ModularRig)
	{
		if (URigHierarchy* Hierarchy = ModularRig->GetHierarchy())
		{
			for(const FRigElementKey& TargetKey : TargetKeys)
			{
				if(TargetModuleName.IsSet())
				{
					if(TargetModuleName.GetValue() != Hierarchy->GetModuleFName(TargetKey))
					{
						TargetModuleName.Reset();
						break;
					}
				}
				else
				{
					TargetModuleName = Hierarchy->GetModuleFName(TargetKey);
				}
			}
		}
	}
	
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ConnectModuleToElementTransaction", "Connect to Element"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif 

	// First disconnect before connecting to anything else. This might disconnect other secondary/optional connectors.
	TMap<FRigElementKey, TArray<FRigElementKey>> PreviousConnections;
	{
		TGuardValue<bool> NotificationsGuard(bSuspendNotifications, true);
		
		if (CurrentTarget.IsValid())
		{
			const TGuardValue<bool> DisableAutomaticReparenting(bAutomaticReparenting, false);
			DisconnectConnector_Internal(InConnectorKey, false, &PreviousConnections, bSetupUndo);
		}

		Model->Connections.AddConnection(InConnectorKey, TargetKeys);

		// restore previous connections if possible
		for(const TPair<FRigElementKey, TArray<FRigElementKey>>& PreviousConnection : PreviousConnections)
		{
			if(!Model->Connections.HasConnection(PreviousConnection.Key))
			{
				FText ErrorMessageForPreviousConnection;
				if(CanConnectConnectorToElements(PreviousConnection.Key, PreviousConnection.Value, ErrorMessageForPreviousConnection))
				{
					(void)ConnectConnectorToElements(PreviousConnection.Key, PreviousConnection.Value, bSetupUndo, false, false);
				}
			}
		}
	}

	Notify(EModularRigNotification::ConnectionChanged, Module);

#if WITH_EDITOR
	if (ModularRig)
	{
		//TGuardValue<bool> NotificationsGuard(bSuspendNotifications, true);

		if (URigHierarchy* Hierarchy = ModularRig->GetHierarchy())
		{
			bool bResolvedPrimaryConnector = false;
			if(const FRigConnectorElement* PrimaryConnector = Module->FindPrimaryConnector(Hierarchy))
			{
				bResolvedPrimaryConnector = PrimaryConnector->GetKey() == InConnectorKey;
			}

			// automatically re-parent the module in the module tree as well
			if(bAutomaticReparenting)
			{
				if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(InConnectorKey))
				{
					if(Connector->IsPrimary())
					{
						if(!TargetModuleName.Get(NAME_None).IsNone())
						{
							const FName ModuleName = Module->Name;
							if(ReparentModule(ModuleName, TargetModuleName.GetValue(), bSetupUndo))
							{
								Module = FindModule(ModuleName);
							}
						}
					}
				}
			}

			if (Module && bAutoResolveOtherConnectors && bResolvedPrimaryConnector)
			{
				(void)AutoConnectModules( {Module->Name}, false, bSetupUndo);
			}
		}
	}
#endif

	// todo: Check if this is even necessary, for now it is disabled 
	// const TArray<FRigElementKey> DisconnectedConnectors = DisconnectCyclicConnectors();
	//
	// // If the connection that was attempted generated cycles, try to reestablish the previous connections
	// if (DisconnectedConnectors.Contains(InConnectorKey))
	// {
	// 	FString TargetKeyString = GenerateTargetKeyString();
	// 	UE_LOGF(LogControlRig, Error, "Could not connect %ls to %ls: cycles detected", *InConnectorKey.ToString(), *TargetKeyString);
	// 	
	// 	Model->Connections = ConnectionsSnapshot; // reestablish the previous connections
	// }

	Notify(EModularRigNotification::ConnectionChanged, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	
	return true;
}

bool UModularRigController::AddTargetToArrayConnector(const FRigElementKey& InConnectorKey, const FRigElementKey& InTargetKey, bool bSetupUndo,
	bool bAutoResolveOtherConnectors, bool bCheckValidConnection)
{
	TArray<FRigElementKey> ExistingTargets = Model->Connections.FindTargetsFromConnector(InConnectorKey);
	ExistingTargets.Add(InTargetKey);
	
	if(bCheckValidConnection)
	{
		return ConnectConnectorToElements(InConnectorKey, ExistingTargets, bSetupUndo, bAutoResolveOtherConnectors, bCheckValidConnection);
	}

	const FRigHierarchyModulePath ConnectorModulePath(InConnectorKey.Name);
	const FRigModuleReference* Module = FindModule(ConnectorModulePath.GetModuleFName());
	if(Module == nullptr)
	{
		return false;
	}

	FRigElementKeyRedirector::FKeyArray TargetKeyArray;
	TargetKeyArray.Append(ExistingTargets);

	UObject* Blueprint = Cast<UObject>(GetOuter());
#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "AddTargetToArrayConnector", "Add Target to Array Connector"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif 

	Model->Connections.RemoveConnection(InConnectorKey);
	Model->Connections.AddConnection(InConnectorKey, TargetKeyArray);

	Notify(EModularRigNotification::ConnectionChanged, Module);

	return true;
}

bool UModularRigController::DisconnectConnector(const FRigElementKey& InConnectorKey, bool bDisconnectSubModules, bool bSetupUndo)
{
	return DisconnectConnector_Internal(InConnectorKey, bDisconnectSubModules, nullptr, bSetupUndo);
};

bool UModularRigController::DisconnectConnector_Internal(const FRigElementKey& InConnectorKey, bool bDisconnectSubModules,
	TMap<FRigElementKey, TArray<FRigElementKey>>* OutRemovedConnections, bool bSetupUndo)
{
	const FRigHierarchyModulePath ConnectorModulePath(InConnectorKey.Name);
	if (!ConnectorModulePath.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Connector %ls does not contain a namespace", *InConnectorKey.ToString());
		return false;
	}
	
	FRigModuleReference* Module = FindModule(ConnectorModulePath.GetModuleFName());
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *ConnectorModulePath.GetModuleNameString());
		return false;
	}

	const FName ConnectorName = ConnectorModulePath.GetElementFName();
	const FRigModuleConnector* ModuleConnector = Module->ControlRigAssetReference.LoadStrongReference().GetRigModuleSettings().ExposedConnectors.FindByPredicate(
		[ConnectorName](FRigModuleConnector& Connector)
		{
			return Connector.Name == ConnectorName;
		});
	if (!ModuleConnector)
	{
		UE_LOGF(LogControlRig, Error, "Could not find connector %ls in class %ls", *ConnectorName.ToString(), *Module->ControlRigAssetReference.GetPathName());
		return false;
	}

	UObject* Blueprint = GetOuter();
	if(!Model->Connections.HasConnection(InConnectorKey))
	{
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ConnectModuleToElementTransaction", "Connect to Element"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif 

	if(OutRemovedConnections)
	{
		OutRemovedConnections->Add(InConnectorKey, Model->Connections.FindTargetsFromConnector(InConnectorKey));
	}
	Model->Connections.RemoveConnection(InConnectorKey);

	if (ModuleConnector->IsPrimary())
	{
		// Remove connections from module and child modules
		TArray<FRigElementKey> ConnectionsToRemove;
		for (const FModularRigSingleConnection& Connection : Model->Connections)
		{
			if (FRigHierarchyModulePath(Connection.Connector.Name).HasModuleName(ConnectorModulePath.GetModuleName()))
			{
				ConnectionsToRemove.Add(Connection.Connector);
			}
		}
		for (const FRigElementKey& ToRemove : ConnectionsToRemove)
		{
			if(OutRemovedConnections)
			{
				OutRemovedConnections->Add(ToRemove, Model->Connections.FindTargetsFromConnector(ToRemove));
			}
			Model->Connections.RemoveConnection(ToRemove);
		}
	}
	else if (!ModuleConnector->IsOptional() && bDisconnectSubModules)
	{
		// Remove connections from child modules
		TArray<FRigElementKey> ConnectionsToRemove;
		const FName ConnectorModuleFName = ConnectorModulePath.GetModuleFName();
		for (const FModularRigSingleConnection& Connection : Model->Connections)
		{
			const FRigHierarchyModulePath OtherModulePath(Connection.Connector.Name);
			if(Model->IsModuleParentedTo(OtherModulePath.GetModuleFName(), ConnectorModuleFName))
			{
				ConnectionsToRemove.Add(Connection.Connector);
			}
		}
		for (const FRigElementKey& ToRemove : ConnectionsToRemove)
		{
			if(OutRemovedConnections)
			{
				OutRemovedConnections->Add(ToRemove, Model->Connections.FindTargetsFromConnector(ToRemove));
			}
			Model->Connections.RemoveConnection(ToRemove);
		}
	}

	// todo: Make sure all the rest of the connections are still valid

	// un-parent the module if we've disconnected the primary
	if(bAutomaticReparenting)
	{
		if(ModuleConnector->IsPrimary() && !Module->IsRootModule())
		{
			(void)ReparentModule(Module->Name, NAME_None, bSetupUndo);
		}
	}

	Notify(EModularRigNotification::ConnectionChanged, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	
	return true;
}

TArray<FRigElementKey> UModularRigController::DisconnectCyclicConnectors(bool bSetupUndo)
{
	TArray<FRigElementKey> DisconnectedConnectors;

#if WITH_EDITOR
	const UObject* Blueprint = GetOuter();
	check(Blueprint);

	const UModularRig* ModularRig = GetDebuggedModularRig();
	if (!ModularRig)
	{
		return DisconnectedConnectors;
	}
	
	const URigHierarchy* Hierarchy = ModularRig->GetHierarchy();
	if (!Hierarchy)
	{
		return DisconnectedConnectors;
	}

	TArray<FRigElementKey> ConnectorsToDisconnect;
	for (const FModularRigSingleConnection& Connection : Model->Connections)
	{
		const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connection.Connector);
		if(const FRigModuleReference* ConnectorModule = Model->FindModule(ConnectorModuleName))
		{
			const FRigConnectorElement* ConnectorElement = Hierarchy->Find<FRigConnectorElement>(Connection.Connector, false);
			if(ConnectorElement == nullptr)
			{
				continue;
			}

			const FRigModuleInstance* ConnectorModuleInstance = ModularRig->FindModule(ConnectorModuleName);
			if(ConnectorModuleInstance == nullptr)
			{
				continue;
			}

			const int32& SpawnStartIndex =
				ConnectorElement->IsPostConstructionConnector() ?
				ConnectorModuleInstance->PostConstructionSpawnStartIndex :
				ConnectorModuleInstance->ConstructionSpawnStartIndex;

			for(const FRigElementKey& Target : Connection.Targets)
			{
				const FName TargetModuleName = Hierarchy->GetModuleFName(Target);

				// targets in the base hierarchy are always allowed
				if(TargetModuleName.IsNone())
				{
					continue;
				}

				const FRigModuleReference* TargetModule = Model->FindModule(TargetModuleName);
				if(TargetModule == nullptr || ConnectorModule == TargetModule)
				{
					continue;
				}

				const int32 TargetSpawnIndex = Hierarchy->GetSpawnIndex(Target);
				if(TargetSpawnIndex != INDEX_NONE)
				{
					if(TargetSpawnIndex >= SpawnStartIndex)
					{
						ConnectorsToDisconnect.Add(Connection.Connector);
						break;
					}
				}

				// alternatively if we don't have a valid spawn index relay back to module parenting rules
				else if(!Model->IsModuleParentedTo(ConnectorModule, TargetModule))
				{
					ConnectorsToDisconnect.Add(Connection.Connector);
					break;
				}
			}
		}
	}

	for(const FRigElementKey& ConnectorToDisconnect : ConnectorsToDisconnect)
	{
		if(DisconnectConnector(ConnectorToDisconnect, false, bSetupUndo))
		{
			DisconnectedConnectors.Add(ConnectorToDisconnect);
		}
	}
#endif

	return DisconnectedConnectors;
}

bool UModularRigController::AutoConnectSecondaryConnectors(const TArray<FRigElementKey>& InConnectorKeys, bool bReplaceExistingConnections, bool bSetupUndo)
{
#if WITH_EDITOR

	UObject* Blueprint = GetOuter();
	if(Blueprint == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "ModularRigController is not nested under blueprint.");
		return false;
	}

	const UModularRig* ModularRig = GetDebuggedModularRig();
	if (!ModularRig)
	{
		UE_LOGF(LogControlRig, Error, "Could not find debugged modular rig in %ls", *Blueprint->GetPathName());
		return false;
	}
	
	URigHierarchy* Hierarchy = ModularRig->GetHierarchy();
	if (!Hierarchy)
	{
		UE_LOGF(LogControlRig, Error, "Could not find hierarchy in %ls", *ModularRig->GetPathName());
		return false;
	}

	for(const FRigElementKey& ConnectorKey : InConnectorKeys)
	{
		if(ConnectorKey.Type != ERigElementType::Connector)
		{
			UE_LOGF(LogControlRig, Error, "Could not find debugged modular rig in %ls", *Blueprint->GetPathName());
			return false;
		}
		const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(ConnectorKey);
		if(Connector == nullptr)
		{
			UE_LOGF(LogControlRig, Error, "Cannot find connector %ls in %ls", *ConnectorKey.ToString(), *Blueprint->GetPathName());
			return false;
		}
		if(Connector->IsPrimary())
		{
			UE_LOGF(LogControlRig, Warning, "Provided connector %ls in %ls is a primary connector. It will be skipped during auto resolval.", *ConnectorKey.ToString(), *Blueprint->GetPathName());
		}
	}

	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "AutoResolveSecondaryConnectors", "Auto-Resolve Connectors"), !GIsTransacting);
	}

	GetOuter()->Modify();

	bool bResolvedAllConnectors = true;
	for(const FRigElementKey& ConnectorKey : InConnectorKeys)
	{
		const FName ModuleName = Hierarchy->GetModuleFName(ConnectorKey);
		if(ModuleName.IsNone())
		{
			UE_LOGF(LogControlRig, Error, "Connector %ls has no associated module", *ConnectorKey.ToString());
			bResolvedAllConnectors = false;
			continue;
		}

		const FRigModuleReference* Module = Model->FindModule(ModuleName);
		if(Module == nullptr)
		{
			UE_LOGF(LogControlRig, Error, "Could not find module %ls", *ModuleName.ToString());
			bResolvedAllConnectors = false;
			continue;
		}

		const FRigConnectorElement* PrimaryConnector = Module->FindPrimaryConnector(Hierarchy);
		if(PrimaryConnector == nullptr)
		{
			UE_LOGF(LogControlRig, Error, "Module %ls has no primary connector", *ModuleName.ToString());
			bResolvedAllConnectors = false;
			continue;
		}
		
		const FRigElementKey PrimaryConnectorKey = PrimaryConnector->GetKey();
		if(ConnectorKey == PrimaryConnectorKey)
		{
			// silently skip primary connectors
			continue;
		}
		
		if(!Model->Connections.HasConnection(PrimaryConnectorKey))
		{
			UE_LOGF(LogControlRig, Warning, "Module %ls's primary connector is not resolved", *ModuleName.ToString());
			bResolvedAllConnectors = false;
			continue;
		}

		const UModularRigRuleManager* RuleManager = Hierarchy->GetRuleManager();
		const FRigModuleInstance* ModuleInstance = ModularRig->FindModule(Module->Name);
		
		if (bReplaceExistingConnections || !Model->Connections.HasConnection(ConnectorKey))
		{
			if (const FRigConnectorElement* OtherConnectorElement = Cast<FRigConnectorElement>(Hierarchy->Find(ConnectorKey)))
			{
				FModularRigResolveResult RuleResults = RuleManager->FindMatches(OtherConnectorElement, ModuleInstance, ModularRig->GetElementKeyRedirector());

				bool bFoundMatch = false;
				if (RuleResults.GetMatches().Num() == 1)
				{
					Model->Connections.AddConnection(ConnectorKey, {RuleResults.GetMatches()[0].GetKey()});
					Notify(EModularRigNotification::ConnectionChanged, Module);
					bFoundMatch = true;
				}
				else
				{
					for (const FRigElementResolveResult& Result : RuleResults.GetMatches())
					{
						if (Result.GetState() == ERigElementResolveState::DefaultTarget)
						{
							Model->Connections.AddConnection(ConnectorKey, {Result.GetKey()});
							Notify(EModularRigNotification::ConnectionChanged, Module);
							bFoundMatch = true;
							break;
						}
					}
				}

				if(!bFoundMatch)
				{
					bResolvedAllConnectors = false;
				}
			}
		}
	}

	TransactionPtr.Reset();

	return bResolvedAllConnectors;

#else
	
	return false;

#endif
}

bool UModularRigController::AutoConnectModules(const TArray<FName>& InModuleNames, bool bReplaceExistingConnections, bool bSetupUndo)
{
#if WITH_EDITOR
	TArray<FRigElementKey> ConnectorKeys;

	UObject* Asset = GetOuter();
	const UModularRig* ModularRig = GetDebuggedModularRig();
	if (!ModularRig)
	{
		UE_LOGF(LogControlRig, Error, "Could not find debugged modular rig in %ls", *Asset->GetPathName());
		return false;
	}
	
	const URigHierarchy* Hierarchy = ModularRig->GetHierarchy();
	if (!Hierarchy)
	{
		UE_LOGF(LogControlRig, Error, "Could not find hierarchy in %ls", *ModularRig->GetPathName());
		return false;
	}

	for(const FName& ModuleName : InModuleNames)
	{
		const FRigModuleReference* Module = FindModule(ModuleName);
		if (!Module)
		{
			UE_LOGF(LogControlRig, Error, "Could not find module %ls", *ModuleName.ToString());
			return false;
		}

		const TArray<const FRigConnectorElement*> Connectors = Module->FindConnectors(Hierarchy);
		for(const FRigConnectorElement* Connector : Connectors)
		{
			if(Connector->IsSecondary())
			{
				ConnectorKeys.Add(Connector->GetKey());
			}
		}
	}

	return AutoConnectSecondaryConnectors(ConnectorKeys, bReplaceExistingConnections, bSetupUndo);

#else

	return false;

#endif
}

bool UModularRigController::SetConfigValueInModule(const FName& InModuleName, const FName& InVariableName, const FString& InValue, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

	if (!Module->ControlRigAssetReference.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Class defined in module %ls is not valid", *InModuleName.ToString());
		return false;
	}

	const FControlRigOverrideValue OverrideValue(InVariableName.ToString(), Module->ControlRigAssetReference.LoadStrongReference().GetVariablesStruct(), InValue, Module->Name);
	return SetConfigValueInModule(InModuleName, OverrideValue, bSetupUndo);
}

bool UModularRigController::SetConfigValueInModule(const FName& InModuleName, const FControlRigOverrideValue& InValue, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

	if (!Module->ControlRigAssetReference.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Class defined in module %ls is not valid", *InModuleName.ToString());
		return false;
	}

	if(!InValue.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Provided value is not valid.");
		return false;
	}

	const FString& Path = InValue.GetPath();

	// we cannot set a config value if we already have a value set on a parent path.
	// so for example you can't override Color.R if Color itself has been overridden already.
	if(Module->ConfigOverrides.ContainsParentPathOf(InValue))
	{
		UE_LOGF(LogControlRig, Error, "Cannot set a config value for '%ls' for the module instance class %ls since there's a value on a parent already.", *Path, *Module->ControlRigAssetReference.GetPathName());
		return false;
	}

	if(InValue.GetRootProperty()->GetOwnerUObject() != Module->ControlRigAssetReference.LoadStrongReference().GetVariablesStruct())
	{
		UE_LOGF(LogControlRig, Error, "Provided path '%ls' does not belong to the module instance class %ls.", *Path, *Module->ControlRigAssetReference.GetPathName());
		return false;
	}

	if (InValue.GetRootProperty()->HasAllPropertyFlags(CPF_BlueprintReadOnly))
	{
		UE_LOGF(LogControlRig, Error, "The target property %ls in module %ls is read only", *InValue.GetRootProperty()->GetName(), *InModuleName.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ConfigureModuleValueTransaction", "Configure Module Value"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif
	
	// remove all nested config values (child properties under the provided ones)
	{
		TArray<FString> KeysToRemove;
		for(const FControlRigOverrideValue& Override : Module->ConfigOverrides)
		{
			const FString& ChildPath = Override.GetPath();
			if(FControlRigOverrideContainer::IsChildPathOf(ChildPath, Path))
			{
				KeysToRemove.Add(ChildPath);
			}
		}
		for(const FString& KeyToRemove : KeysToRemove)
		{
			Module->ConfigOverrides.Remove(KeyToRemove, Module->Name);
		}
	}

	Module->ConfigOverrides.FindOrAdd(InValue);

	Notify(EModularRigNotification::ModuleConfigValueChanged, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	return true;
}

bool UModularRigController::ResetConfigValueInModule(const FName& InModuleName, const FString& InPath, bool bClearOverride, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

	if (!Module->ControlRigAssetReference.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Class defined in module %ls is not valid", *InModuleName.ToString());
		return false;
	}

	if(InPath.IsEmpty())
	{
		UE_LOGF(LogControlRig, Error, "Provided path is not valid.");
		return false;
	}

	const FControlRigAssetStrongReference AssetReference = Module->ControlRigAssetReference.LoadStrongReference();
	const FControlRigOverrideValue DefaultValue(InPath, AssetReference.GetVariablesStruct(), AssetReference.GetVariablesMemory());
	if(!DefaultValue.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Provided path '%ls' does not belong to the module instance class %ls.", *InPath, *AssetReference.GetPathName());
		return false;
	}

	if(!bClearOverride)
	{
		SetConfigValueInModule(InModuleName, DefaultValue, bSetupUndo);
		return true;
	}
	
	const FString PathPrefix = InPath + TEXT("->");

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ConfigureModuleValueTransaction", "Configure Module Value"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif
	
	// remove all nested config values (child properties under the provided ones)
	{
		TArray<FString> KeysToRemove;
		for(const FControlRigOverrideValue& Override : Module->ConfigOverrides)
		{
			const FString& ChildPath = Override.GetPath();
			if(InPath == ChildPath || FControlRigOverrideContainer::IsChildPathOf(ChildPath, InPath))
			{
				KeysToRemove.Add(ChildPath);
			}
		}
		for(const FString& KeyToRemove : KeysToRemove)
		{
			Module->ConfigOverrides.Remove(KeyToRemove, Module->Name);
		}
	}

	Notify(EModularRigNotification::ModuleConfigValueChanged, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	
	return true;
}

TArray<FString> UModularRigController::GetPossibleBindings(const FName& InModuleName, const FName& InVariableName)
{
	TArray<FString> PossibleBindings;
	const FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		return PossibleBindings;
	}

	if (!Module->ControlRigAssetReference.IsValid())
	{
		return PossibleBindings;
	}

	const FProperty* TargetProperty = Module->ControlRigAssetReference.LoadStrongReference().FindVariablePropertyByName(InVariableName);
	if (!TargetProperty)
	{
		return PossibleBindings;
	}

	if (TargetProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly | CPF_DisableEditOnInstance))
	{
		return PossibleBindings;
	}

	// Add possible blueprint variables
	FControlRigAssetStrongReference Source(GetOuter());
	if(Source.IsValid())
	{
		TArray<FRigVMExternalVariable> Variables = Source.GetExternalVariables();
		for (const FRigVMExternalVariable& Variable : Variables)
		{
			FText ErrorMessage;
			const FString VariableName = Variable.GetName().ToString();
			if (CanBindModuleVariable(InModuleName, InVariableName, VariableName, ErrorMessage))
			{
				PossibleBindings.Add(VariableName);
			}
		}
	}

	// Add possible module variables
	Model->ForEachModule([this, &PossibleBindings, &InModuleName, &InVariableName](const FRigModuleReference* InModule) -> bool
	{
		const FName CurModuleName = InModule->Name;
		if (InModuleName != CurModuleName)
		{
			FControlRigAssetStrongReference AssetReference = InModule->ControlRigAssetReference.LoadStrongReference();
			if (AssetReference.IsValid())
			{
				TArray<FRigVMExternalVariable> Variables = AssetReference.GetExternalVariables();
			   for (const FRigVMExternalVariable& Variable : Variables)
			   {
				   FText ErrorMessage;
				   const FRigHierarchyModulePath SourceVariablePath(CurModuleName, Variable.GetName());
				   if (CanBindModuleVariable(InModuleName, InVariableName, SourceVariablePath, ErrorMessage))
				   {
					   PossibleBindings.Add(SourceVariablePath);
				   }
			   }
			}
		}		
		return true;
	});

	return PossibleBindings;
}

bool UModularRigController::CanBindModuleVariable(const FName& InModuleName, const FName& InVariableName, const FString& InSourcePath, FText& OutErrorMessage)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find module %s"), *InModuleName.ToString()));
		return false;
	}

	FControlRigAssetStrongReference AssetReference = Module->ControlRigAssetReference.LoadStrongReference();
	if (!AssetReference.IsValid())
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Class defined in module %s is not valid"), *InModuleName.ToString()));
		return false;
	}

	// Use FindVariablePropertyByName (not GetVariablesStruct()->FindPropertyByName): for BP-independent
	// rigs the variables live in a UPropertyBag where the generated property name is sanitized (e.g.
	// spaces become underscores), while the binding paths use the unsanitized DisplayName. Only
	// FindVariablePropertyByName routes the lookup through the bag's name sanitization.
	const FProperty* TargetProperty = AssetReference.FindVariablePropertyByName(InVariableName);
	if (!TargetProperty)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find variable %s in module %s"), *InVariableName.ToString(), *InModuleName.ToString()));
		return false;
	}

	if (TargetProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly | CPF_DisableEditOnInstance))
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("The target variable %s in module %s is read only"), *InVariableName.ToString(), *InModuleName.ToString()));
		return false;
	}

	FString SourceModuleName, SourceVariableName = InSourcePath;
	(void)FRigHierarchyModulePath(InSourcePath).Split(&SourceModuleName, &SourceVariableName);

	FRigModuleReference* SourceModule = nullptr;
	if (!SourceModuleName.IsEmpty())
	{
		SourceModule = FindModule(*SourceModuleName);
		if (!SourceModule)
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find source module %s"), *SourceModuleName));
			return false;
		}

		if(Model->IsModuleParentedTo(SourceModule->Name, InModuleName))
		{
			OutErrorMessage = FText::FromString(FString::Printf(TEXT("Cannot bind variable of module %s to a variable of module %s because the source module is a child of the target module"), *InModuleName.ToString(), *SourceModuleName));
			return false;
		}
	}

	const FProperty* SourceProperty = nullptr;
	if (SourceModule)
	{
		SourceProperty = SourceModule->ControlRigAssetReference.LoadStrongReference().FindVariablePropertyByName(*SourceVariableName);
	}
	else
	{
		FControlRigAssetStrongReference Source(GetOuter());
		if(Source.IsValid())
		{
			SourceProperty = Source.FindVariablePropertyByName(*SourceVariableName);
		}
	}
	if (!SourceProperty)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Could not find source variable %s"), *InSourcePath));
		return false;
	}

	if (!RigVMTypeUtils::AreCompatible(SourceProperty, TargetProperty))
	{
		const FString SourcePath = (SourceModuleName.IsEmpty()) ? SourceVariableName : FRigHierarchyModulePath(SourceModuleName, SourceVariableName).GetPath();
		const FString TargetPath = FString::Printf(TEXT("%s.%s"), *InModuleName.ToString(), *InVariableName.ToString());
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Property %s of type %s and %s of type %s are not compatible"), *SourcePath, *SourceProperty->GetCPPType(), *TargetPath, *TargetProperty->GetCPPType()));
		return false;
	}

	return true;
}

bool UModularRigController::BindModuleVariable(const FName& InModuleName, const FName& InVariableName, const FString& InSourcePath, bool bSetupUndo)
{
	FText ErrorMessage;
	if (!CanBindModuleVariable(InModuleName, InVariableName, InSourcePath, ErrorMessage))
	{
		UE_LOGF(LogControlRig, Error, "Could not bind module variable %ls : %ls", *FRigHierarchyModulePath(InModuleName, InVariableName).GetPath(), *ErrorMessage.ToString());
		return false;
	}
	
	FRigModuleReference* Module = FindModule(InModuleName);
	const FProperty* TargetProperty = Module->ControlRigAssetReference.LoadStrongReference().FindVariablePropertyByName(InVariableName);

	FString SourceModuleName, SourceVariableName = InSourcePath;
	(void)FRigHierarchyModulePath(InSourcePath).Split(&SourceModuleName, &SourceVariableName);

	FString SourcePath = (SourceModuleName.IsEmpty()) ? SourceVariableName : FRigHierarchyModulePath(SourceModuleName, SourceVariableName).GetPath();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "BindModuleVariableTransaction", "Bind Module Variable"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif

	FString& SourceStr = Module->Bindings.FindOrAdd(InVariableName);
	SourceStr = SourcePath;

	Notify(EModularRigNotification::ModuleConfigValueChanged, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	return true;
}

bool UModularRigController::UnBindModuleVariable(const FName& InModuleName, const FName& InVariableName, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

	if (!Module->Bindings.Contains(InVariableName))
	{
		UE_LOGF(LogControlRig, Error, "Variable %ls in module %ls is not bound", *InVariableName.ToString(), *InModuleName.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "BindModuleVariableTransaction", "Bind Module Variable"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif

	Module->Bindings.Remove(InVariableName);

	Notify(EModularRigNotification::ModuleConfigValueChanged, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	return true;
}

bool UModularRigController::DeleteModule(const FName& InModuleName, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "DeleteModuleTransaction", "Delete Module"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif

	(void)DeselectModule(Module->Name);

	TArray<FName> ModuleNamesToDelete;
	ModuleNamesToDelete.Add(Module->Name);

	for (int32 Index = 0; Index < ModuleNamesToDelete.Num(); Index++)
	{
		const FRigModuleReference* ModuleToDelete = Model->FindModule(ModuleNamesToDelete[Index]);
		if (!ModuleToDelete)
		{
			continue;
		}
		Algo::Transform(ModuleToDelete->CachedChildren, ModuleNamesToDelete, [](const FRigModuleReference* Child){ return Child->Name; });
	}

	Algo::Reverse(ModuleNamesToDelete);

	for (const FName& ModuleName : ModuleNamesToDelete)
	{
		const FRigModuleReference* ModuleToDelete = Model->FindModule(ModuleName);
		Model->DeletedModules.Add(*Module);
		Model->DeletedModules.Last().CachedChildren.Reset();
		Model->Modules.RemoveSingle(*ModuleToDelete);
	}
	
	Model->UpdateCachedChildren();

	for (const FName& ModuleName : ModuleNamesToDelete)
	{
		const FString ModuleNameString = ModuleName.ToString();

		// Fix connections
		{
			TSet<FRigElementKey> ToRemove;
			for (FModularRigSingleConnection& Connection : Model->Connections)
			{
				const FRigHierarchyModulePath ConnectionModulePath(Connection.Connector.Name);
				if (ConnectionModulePath.HasModuleName(ModuleNameString))
				{
					ToRemove.Add(Connection.Connector);
				}

				for(const FRigElementKey& Target : Connection.Targets)
				{
					const FRigHierarchyModulePath TargetModulePath(Target.Name);
					if (TargetModulePath.HasModuleName(ModuleNameString))
					{
						ToRemove.Add(Connection.Connector);
						break;
					}
				}
			}
			for (FRigElementKey& KeyToRemove : ToRemove)
			{
				Model->Connections.RemoveConnection(KeyToRemove);
			}
			Model->Connections.UpdateFromConnectionList();
		}

		// Fix bindings
		for (FRigModuleReference& Reference : Model->Modules)
		{
			Reference.Bindings = Reference.Bindings.FilterByPredicate([ModuleNameString](const TPair<FName, FString>& Binding)
			{
				const FRigHierarchyModulePath BindingModulePath(Binding.Value);
				if (BindingModulePath.HasModuleName(ModuleNameString))
				{
					return false;
				}
				return true;
			});
		}
	}

	for (const FRigModuleReference& DeletedModule : Model->DeletedModules)
	{
		Notify(EModularRigNotification::ModuleRemoved, &DeletedModule);
	}

	Model->DeletedModules.Reset();

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	return false;
}

FName UModularRigController::RenameModule(const FName& InModuleName, const FName& InNewName, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return NAME_None;
	}

	const FName OldName = Module->Name;
	const FName NewName = InNewName;
	if (OldName == NewName)
	{
		return Module->Name;
	}

	FText ErrorMessage;
	if (!CanRenameModule(InModuleName, InNewName, ErrorMessage))
	{
		UE_LOGF(LogControlRig, Error, "Could not rename module %ls: %ls", *InModuleName.ToString(), *ErrorMessage.ToString());
		return NAME_None;
	}
	
#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "RenameModuleTransaction", "Rename Module"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif
	
	const int32 SelectionIndex = Model->SelectedModuleNames.Find(OldName);
	if(SelectionIndex != INDEX_NONE)
	{
		Notify(EModularRigNotification::ModuleDeselected, Module);
	}

	Module->PreviousName = Module->Name;
	Module->Name = InNewName;
	TArray<FRigModuleReference*> Children;
	Children.Append(Module->CachedChildren);
	for (int32 i=0; i<Children.Num(); ++i)
	{
		FRigModuleReference* Child = Children[i];
		if(Child->ParentModuleName == OldName)
		{
			Child->ParentModuleName = NewName;
		}
		Children.Append(Child->CachedChildren);
	}

	// Fix connections
	{
		for (FModularRigSingleConnection& Connection : Model->Connections)
		{
			FRigHierarchyModulePath ConnectorModulePath(Connection.Connector.Name);
			if (ConnectorModulePath.ReplaceModuleNameInline(OldName, NewName))
			{
				Connection.Connector.Name = ConnectorModulePath.GetPathFName();
			}
			for(FRigElementKey& Target : Connection.Targets)
			{
				FRigHierarchyModulePath TargetModulePath(Target.Name);
				if (TargetModulePath.ReplaceModuleNameInline(OldName, NewName))
				{
					Target.Name = TargetModulePath.GetPathFName();
				}
			}
		}
		Model->Connections.UpdateFromConnectionList();
	}

	// Fix bindings
	for (FRigModuleReference& Reference : Model->Modules)
	{
		for (TPair<FName, FString>& Binding : Reference.Bindings)
		{
			FRigHierarchyModulePath BindingModulePath(Binding.Value);
			if (BindingModulePath.ReplaceModuleNameInline(OldName, NewName))
			{
				Binding.Value = BindingModulePath.GetPath();
			}
		};
	}

	// fix overrides
	for(FControlRigOverrideValue& Override : Module->ConfigOverrides)
	{
		Override.SubjectKey = Module->Name;
	}

	// make sure to update our backwards compat code from path to module name
	for(TPair<FRigHierarchyModulePath, FName>& Pair : Model->PreviousModulePaths)
	{
		if(Pair.Value == Module->PreviousName)
		{
			Pair.Value = Module->Name;
		}
	}

	Notify(EModularRigNotification::ModuleRenamed, Module);

	if(SelectionIndex != INDEX_NONE)
	{
		Model->SelectedModuleNames[SelectionIndex] = NewName;
		Notify(EModularRigNotification::ModuleSelected, Module);
	}

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	
	return NewName;
}

bool UModularRigController::CanRenameModule(const FName& InModuleName, const FName& InNewName, FText& OutErrorMessage) const
{
	if (InNewName.IsNone() || InNewName.ToString().IsEmpty())
	{
		OutErrorMessage = FText::FromString(TEXT("Name is empty."));
		return false;
	}

	if(InNewName.ToString().Contains(FRigHierarchyModulePath::NamespaceSeparator_Deprecated))
	{
		OutErrorMessage = NSLOCTEXT("ModularRigController", "NameContainsNamespaceSeparator", "Name contains namespace separator ':'.");
		return false;
	}

	if(InNewName.ToString().Contains(FRigHierarchyModulePath::ModuleNameSuffix))
	{
		OutErrorMessage = NSLOCTEXT("ModularRigController", "NameContainsModuleSuffix", "Name contains module suffix '/'.");
		return false;
	}

	const FRigModuleReference* Module = const_cast<UModularRigController*>(this)->FindModule(InModuleName);
	if (!Module)
	{
		OutErrorMessage = FText::FromString(FString::Printf(TEXT("Module %s not found."), *InModuleName.ToString()));
		return false;
	}

	FString ErrorMessage;
	if(!IsNameAvailable(InNewName, &ErrorMessage))
	{
		OutErrorMessage = FText::FromString(ErrorMessage);
		return false;
	}
	return true;
}

bool UModularRigController::ReparentModule(const FName& InModuleName, const FName& InNewParentModuleName, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

	const FRigModuleReference* NewParentModule = FindModule(InNewParentModuleName);
	const FName PreviousParentModuleName = Module->ParentModuleName;
	if(PreviousParentModuleName == InNewParentModuleName)
	{
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ReparentModuleTransaction", "Reparent Module"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif
	
	// if the new parent is currently a descendant of the module, then first reparent the new parent to the current parent of the module
	if (Model->IsModuleParentedTo(InNewParentModuleName, InModuleName))
	{
		if (!ReparentModule(InNewParentModuleName, Module->ParentModuleName, bSetupUndo))
		{
			UE_LOGF(LogControlRig, Error, "Fail to reparent %ls under %ls", *InNewParentModuleName.ToString(), *Module->ParentModuleName.ToString());
			return false;
		}
	}

	Module->PreviousParentName = Module->ParentModuleName;
	Module->ParentModuleName = (NewParentModule) ? NewParentModule->Name : NAME_None;

	Model->UpdateCachedChildren();

	// since we've reparented the module now we should clear out all connectors which are cyclic
	(void)DisconnectCyclicConnectors(bSetupUndo);

	Notify(EModularRigNotification::ModuleReparented, Module);

#if WITH_EDITOR
 	TransactionPtr.Reset();
#endif
	
	return true;
}

bool UModularRigController::ReorderModule(const FName& InModuleName, int32 InModuleIndex, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}
	Model->UpdateCachedChildren();

	TArray<FRigModuleReference*> ChildModules;
	
	const FRigModuleReference* ParentModule = Module->GetParentModule();
	if(ParentModule)
	{
		ChildModules = ParentModule->CachedChildren;
	}
	else
	{
		ChildModules = Model->RootModules;
	}

	if(ChildModules.Num() == 1)
	{
		UE_LOGF(LogControlRig, Error, "Cannot reorder sole module %ls", *InModuleName.ToString());
		return false;
	}

	if(InModuleIndex < 0 || InModuleIndex >= ChildModules.Num())
	{
		UE_LOGF(LogControlRig, Error, "Cannot reorder module %ls, index is out of range (0 to %d)", *InModuleName.ToString(), ChildModules.Num() - 1);
		return false;
	}

	int32 CurrentIndex = ChildModules.Find(Module);
	if(CurrentIndex == InModuleIndex)
	{
		UE_LOGF(LogControlRig, Error, "Cannot reorder module %ls, module is already at desired index", *InModuleName.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ReorderModuleTransaction", "Reorder Module"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif

	// create a flat list of module name
	TArray<FName> AllModuleNames;
	for(const FRigModuleReference& ModuleReference : Model->Modules)
	{
		AllModuleNames.Add(ModuleReference.Name);
	}

	// now create a list of child names
	TArray<FName> ChildModuleNames;
	for(const FRigModuleReference* ChildModule : ChildModules)
	{
		ChildModuleNames.Add(ChildModule->Name);
	}

	// move the module up
	while(CurrentIndex > InModuleIndex)
	{
		ChildModuleNames.Swap(CurrentIndex, CurrentIndex-1);
		AllModuleNames.Swap(AllModuleNames.Find(ChildModuleNames[CurrentIndex]), AllModuleNames.Find(ChildModuleNames[CurrentIndex-1]));
		CurrentIndex--;
	}

	// move the module down
	while(CurrentIndex < InModuleIndex)
	{
		ChildModuleNames.Swap(CurrentIndex, CurrentIndex+1);
		AllModuleNames.Swap(AllModuleNames.Find(ChildModuleNames[CurrentIndex]), AllModuleNames.Find(ChildModuleNames[CurrentIndex+1]));
		CurrentIndex++;
	}

	// Sort the modules by their new name order
	Algo::SortBy(Model->Modules, [AllModuleNames](const FRigModuleReference& Module) -> int32
	{
		return AllModuleNames.IndexOfByKey(Module.Name);
	});

	Model->UpdateCachedChildren();

	Notify(EModularRigNotification::ModuleReordered, Module);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	
	return true;
}

FName UModularRigController::MirrorModule(const FName& InModuleName, const FRigVMMirrorSettings& InSettings, bool bSetupUndo)
{
	FRigModuleReference* OriginalModule = FindModule(InModuleName);
	if (!OriginalModule || !OriginalModule->ControlRigAssetReference.IsValid())
	{
		return NAME_None;
	}

	FString NewModuleNameString = OriginalModule->Name.ToString();
	if (!InSettings.SearchString.IsEmpty())
	{
		NewModuleNameString = NewModuleNameString.Replace(*InSettings.SearchString, *InSettings.ReplaceString, ESearchCase::CaseSensitive);
		NewModuleNameString = GetSafeNewName(FRigName(NewModuleNameString)).ToString();
	}

	// Before any changes, gather all the information we need from the OriginalModule, as the pointer might become invalid afterwards
	const FRigElementKeyRedirector::FKeyMap OriginalConnectionMap = Model->Connections.GetModuleConnectionMap(InModuleName);
	const TMap<FName, FString> OriginalBindings = OriginalModule->Bindings;
	const FControlRigAssetStrongReference OriginalSource = OriginalModule->ControlRigAssetReference.LoadStrongReference();
	const FName OriginalParentName = OriginalModule->ParentModuleName;
	const FControlRigOverrideContainer OriginalConfigValues = OriginalModule->ConfigOverrides;

	FModularRigControllerCompileBracketScope CompileBracketScope(this);

	const FName NewModuleName = AddModuleFromAssetReference(*NewModuleNameString, OriginalSource, OriginalParentName, bSetupUndo);
	FRigModuleReference* NewModule = FindModule(NewModuleName);
	if (!NewModule)
	{
		return NAME_None;
	}

	for (const  FRigElementKeyRedirector::FKeyPair& Pair : OriginalConnectionMap)
	{
		for(const FRigElementKey& Target : Pair.Value)
		{
			FString OriginalTargetName = Target.Name.ToString();
			FString NewTargetName = OriginalTargetName.Replace(*InSettings.SearchString, *InSettings.ReplaceString, ESearchCase::CaseSensitive);
			FRigElementKey NewTargetKey(*NewTargetName, Target.Type);

			const FRigHierarchyModulePath NewConnectorPath(NewModuleName.ToString(), Pair.Key.Name.ToString());
			FRigElementKey NewConnectorKey(NewConnectorPath.GetPathFName(), ERigElementType::Connector);
			ConnectConnectorToElement(NewConnectorKey, NewTargetKey, bSetupUndo, false, false);
		}
	}

	for (const TPair<FName, FString>& Pair : OriginalBindings)
	{
		FString NewSourcePath = Pair.Value.Replace(*InSettings.SearchString, *InSettings.ReplaceString, ESearchCase::CaseSensitive);
		BindModuleVariable(NewModuleName, Pair.Key, NewSourcePath, bSetupUndo);
	}

	TSet<FString> ConfigValueSet;
#if WITH_EDITOR
	for (TFieldIterator<FProperty> PropertyIt(OriginalSource.GetVariablesStruct()); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		
		// skip advanced properties for now
		if (Property->HasAnyPropertyFlags(CPF_AdvancedDisplay))
		{
			continue;
		}

		// skip non-public properties for now
		const bool bIsPublic = Property->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst);
		const bool bIsInstanceEditable = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
		if(!bIsPublic || !bIsInstanceEditable)
		{
			continue;
		}

		const FString CPPType = Property->GetCPPType();
		bool bIsVector;
		if (CPPType == TEXT("FVector"))
		{
			bIsVector = true;
		}
		else if (CPPType == TEXT("FTransform"))
		{
			bIsVector = false;
		}
		else
		{
			continue;
		}

		FString NewValueStr;
		if (const FControlRigOverrideValue* OriginalValue = OriginalConfigValues.Find(Property->GetName(), OriginalModule->Name))
		{
			if (bIsVector)
			{
				FVector Value;
				FBlueprintEditorUtils::PropertyValueFromString_Direct(Property, *OriginalValue->ToString(), (uint8*)&Value);
				Value = InSettings.MirrorVector(Value);
				FBlueprintEditorUtils::PropertyValueToString_Direct(Property, (uint8*)&Value, NewValueStr, nullptr);
			}
			else
			{
				FTransform Value;
				FBlueprintEditorUtils::PropertyValueFromString_Direct(Property, *OriginalValue->ToString(), (uint8*)&Value);
				Value = InSettings.MirrorTransform(Value);
				FBlueprintEditorUtils::PropertyValueToString_Direct(Property, (uint8*)&Value, NewValueStr, nullptr);
			}
		}
		else
		{
			if (uint8* Memory = OriginalSource.GetVariablesMemory())
			{
				if (bIsVector)
				{
					FVector NewVector = *Property->ContainerPtrToValuePtr<FVector>(Memory);
					NewVector = InSettings.MirrorVector(NewVector);
					FBlueprintEditorUtils::PropertyValueToString_Direct(Property, (uint8*)&NewVector, NewValueStr, nullptr);
				}
				else
				{
					FTransform NewTransform = *Property->ContainerPtrToValuePtr<FTransform>(Memory);
					NewTransform = InSettings.MirrorTransform(NewTransform);
					FBlueprintEditorUtils::PropertyValueToString_Direct(Property, (uint8*)&NewTransform, NewValueStr, nullptr);
				}
			}
		}

		ConfigValueSet.Add(Property->GetName());

		const FControlRigOverrideValue NewValue(Property->GetName(), OriginalSource.GetVariablesStruct(), NewValueStr, NewModuleName);
		SetConfigValueInModule(NewModuleName, NewValue, bSetupUndo);
	}
#endif

	// Add any other config value that was set in the original module, but was not mirrored
	for (const FControlRigOverrideValue& OriginalOverride : OriginalConfigValues)
	{
		if (!ConfigValueSet.Contains(OriginalOverride.GetPath()))
		{
			SetConfigValueInModule(NewModuleName, OriginalOverride, bSetupUndo);
		}
	}
	
	return NewModuleName;
}

bool UModularRigController::SwapModuleClass(const FName& InModuleName, TSubclassOf<UControlRig> InNewClass, bool bSetupUndo)
{
	return SwapModuleSource(InModuleName, FControlRigAssetStrongReference(InNewClass), bSetupUndo);
}

bool UModularRigController::SwapModuleSource(const FName& InModuleName, FControlRigAssetStrongReference InNewSource, bool bSetupUndo)
{
	FRigModuleReference* Module = FindModule(InModuleName);
	if (!Module)
	{
		UE_LOGF(LogControlRig, Error, "Could not find module %ls", *InModuleName.ToString());
		return false;
	}

	if (!InNewSource.IsValid())
	{
		UE_LOGF(LogControlRig, Error, "Invalid InClass");
		return false;
	}

	if (!InNewSource.IsRigModule())
	{
		UE_LOGF(LogControlRig, Error, "Class %ls is not a rig module", *InNewSource.GetPathName());
		return false;
	}

	if (Module->ControlRigAssetReference.Get() == InNewSource.Get())
	{
		// Nothing to do here
		return true;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "SwapModuleClassTransaction", "Swap Module Class"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif

	Module->ControlRigAssetReference = InNewSource.Get();

	// Remove invalid connectors/connections
	{
		const TArray<FModularRigSingleConnection>& Connections = Model->Connections.GetConnectionList();
		const TArray<FRigModuleConnector>& ExposedConnectors = InNewSource.GetRigModuleSettings().ExposedConnectors;

		TArray<FRigElementKey> ConnectionsToRemove;
		for (const FModularRigSingleConnection& Connection : Connections)
		{
			const FRigHierarchyModulePath ConnectorModulePath(Connection.Connector.Name);
			if (ConnectorModulePath.HasModuleName(InModuleName))
			{
				if (!ExposedConnectors.ContainsByPredicate([&ConnectorModulePath](const FRigModuleConnector& Exposed)
				{
				   return ConnectorModulePath.HasElementName(Exposed.Name);
				}))
				{
					ConnectionsToRemove.Add(Connection.Connector);
					continue;
				}

				for(const FRigElementKey& Target : Connection.Targets)
				{
					FText ErrorMessage;
					if (!CanConnectConnectorToElement(Connection.Connector, Target, ErrorMessage))
					{
						ConnectionsToRemove.Add(Connection.Connector);
						break;
					}
				}
			}
		}

		for (const FRigElementKey& ToRemove : ConnectionsToRemove)
		{
			DisconnectConnector(ToRemove, false, bSetupUndo);
		}
	}

	// Remove config values and bindings that are not supported anymore
	RefreshModuleVariables();

	Notify(EModularRigNotification::ModuleClassChanged, Module);
	
#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	return true;
}

bool UModularRigController::SwapModulesOfClass(TSubclassOf<UControlRig> InOldClass, TSubclassOf<UControlRig> InNewClass, bool bSetupUndo)
{
	return SwapModulesWithSource(FControlRigAssetStrongReference(InOldClass), FControlRigAssetStrongReference(InNewClass), bSetupUndo);
}

bool UModularRigController::SwapModulesWithSource(FControlRigAssetStrongReference InOldSource, FControlRigAssetStrongReference InNewSource, bool bSetupUndo)
{
#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "SwapModulesOfClassTransaction", "Swap Modules of Class"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif
	
	Model->ForEachModule([this, &InOldSource, &InNewSource, bSetupUndo](const FRigModuleReference* Module) -> bool
	{
		if (Module->ControlRigAssetReference.Get() == InOldSource.Get())
		{
			SwapModuleSource(Module->Name, InNewSource, bSetupUndo);
		}
		return true;
	});
	
#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
	
	return true;
}

bool UModularRigController::SelectModule(const FName& InModuleName, const bool InSelected)
{
	const bool bCurrentlySelected = Model->SelectedModuleNames.Contains(InModuleName);
	if(bCurrentlySelected == InSelected)
	{
		return false;
	}

	const FRigModuleReference* Module = FindModule(InModuleName);
	if(Module == nullptr)
	{
		return false;
	}

	if(InSelected)
	{
		Model->SelectedModuleNames.Add(InModuleName);
	}
	else
	{
		Model->SelectedModuleNames.Remove(InModuleName);
	}

	Notify(InSelected ? EModularRigNotification::ModuleSelected : EModularRigNotification::ModuleDeselected, Module);
	return true;
}

bool UModularRigController::DeselectModule(const FName& InModuleName)
{
	return SelectModule(InModuleName, false);
}

bool UModularRigController::SetModuleSelection(const TArray<FName>& InModuleNames)
{
	bool bResult = false;
	const TArray<FName> OldSelection = GetSelectedModules();

	for(const FName& PreviouslySelectedModule : OldSelection)
	{
		if(!InModuleNames.Contains(PreviouslySelectedModule))
		{
			if(DeselectModule(PreviouslySelectedModule))
			{
				bResult = true;
			}
		}
	}
	for(const FName& NewModuleToSelect : InModuleNames)
	{
		if(!OldSelection.Contains(NewModuleToSelect))
		{
			if(SelectModule(NewModuleToSelect))
			{
				bResult = true;
			}
		}
	}

	return bResult;
}

TArray<FName> UModularRigController::GetAllModules() const
{
	TArray<FName> Names;
	Model->ForEachModule([&Names](const FRigModuleReference* Module) -> bool
	{
		Names.Add(Module->Name);
		return true;
	});
	return Names;
}

TArray<FName> UModularRigController::GetSelectedModules() const
{
	return Model->SelectedModuleNames;
}

void UModularRigController::RefreshModuleVariables(bool bSetupUndo)
{
	Model->ForEachModule([this, bSetupUndo](const FRigModuleReference* Element) -> bool
	{
		TGuardValue<bool> NotificationsGuard(bSuspendNotifications, true);
		RefreshModuleVariables(Element, bSetupUndo);
		return true;
	});
}

void UModularRigController::RefreshModuleVariables(const FRigModuleReference* InModule, bool bSetupUndo)
{
	if (!InModule)
	{
		return;
	}
	
	// avoid dead class pointers
	const FControlRigAssetStrongReference ModuleSource = InModule->ControlRigAssetReference.LoadStrongReference();
	if(!ModuleSource.IsValid())
	{
		return;
	}

	// Make sure the provided module belongs to our ModularRigModel
	const FName ModuleName = InModule->Name;
	FRigModuleReference* Module = FindModule(ModuleName);
	if (Module != InModule)
	{
		return;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "RefreshModuleVariablesTransaction", "Refresh Module Variables"), !GIsTransacting);
		GetOuter()->Modify();
	}
#endif

	for (TFieldIterator<FProperty> PropertyIt(ModuleSource.GetVariablesStruct()); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		
		// remove advanced, private or not editable properties
		const bool bIsAdvanced = Property->HasAnyPropertyFlags(CPF_AdvancedDisplay);
		const bool bIsPublic = Property->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst);
		const bool bIsInstanceEditable = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
		if (bIsAdvanced || !bIsPublic || !bIsInstanceEditable)
		{
			FString PropertyName = Property->GetName();
			FString PropertyPrefix = PropertyName + TEXT("->");

			TArray<FString> ConfigValuesToRemove;
			TArray<FName> BindingsToRemove;
			for (const FControlRigOverrideValue& Override : Module->ConfigOverrides)
			{
				const FString& KeyString = Override.GetPath();
				if(KeyString == PropertyName || KeyString.StartsWith(PropertyPrefix))
				{
					ConfigValuesToRemove.Add(KeyString);
				}
			}
			for (const TPair<FName, FString>& Pair : Module->Bindings)
			{
				const FString KeyString = Pair.Key.ToString();
				if(KeyString == PropertyName || KeyString.StartsWith(PropertyPrefix))
				{
					BindingsToRemove.Add(Pair.Key);
				}
			}

			for(const FString& ConfigValueToRemove : ConfigValuesToRemove)
			{
				Module->ConfigOverrides.Remove(ConfigValueToRemove, ModuleName);
			}
			for(const FName& BindingToRemove : BindingsToRemove)
			{
				Module->Bindings.Remove(BindingToRemove);
			}
		}
	}

	// Make sure all the types are valid
	const FControlRigOverrideContainer ConfigOverrides = Module->ConfigOverrides;
	const TMap<FName, FString> Bindings = Module->Bindings;
	Module->ConfigOverrides.Reset();
	Module->Bindings.Reset();
	for (const FControlRigOverrideValue& Override : ConfigOverrides)
	{
		// Make sure the Override properties belong to the module class
		FControlRigOverrideValue CorrectedOverride = Override;
		if (CorrectedOverride.GetRootProperty() == nullptr)
		{
			UE_LOGF(LogControlRig, Display, "Cannot determine root property '%ls' in instance class %ls.", *CorrectedOverride.GetPath(), *Module->ControlRigAssetReference.GetPathName());
			continue;
		}
		if (CorrectedOverride.GetLeafProperty() == nullptr)
		{
			UE_LOGF(LogControlRig, Display, "Cannot determine leaf property '%ls' in instance class %ls.", *CorrectedOverride.GetPath(), *Module->ControlRigAssetReference.GetPathName());
			continue;
		}
		if(Override.GetRootProperty()->GetOwnerUObject() != Module->ControlRigAssetReference.Get())
		{
			for (int32 i=0; i<Override.Properties.Num(); i++)
			{
				const FControlRigOverrideValue::FPropertyInfo& Property = Override.Properties[i];
				if (const FProperty* TargetProperty = Module->ControlRigAssetReference.LoadStrongReference().FindVariablePropertyByName(Property.Property->GetFName()))
				{
					CorrectedOverride.Properties[i].Property = const_cast<FProperty*>(TargetProperty);
				}
				else
				{
					UE_LOGF(LogControlRig, Error, "Provided path '%ls' does not belong to the module instance class %ls.", *ModuleName.ToString(), *Module->ControlRigAssetReference.GetPathName());
					continue;
				}
			}
		}
		SetConfigValueInModule(ModuleName, CorrectedOverride, false);
	}
	for (const TPair<FName, FString>& Pair : Bindings)
	{
		BindModuleVariable(ModuleName, Pair.Key, Pair.Value, false);
	}

	// If the module is the source of another module's binding, make sure it is still a valid binding
	Model->ForEachModule([this, InModule, &ModuleName, ModuleSource](const FRigModuleReference* OtherModule) -> bool
	{
		if (InModule == OtherModule)
		{
			return true;
		}
		TArray<FName> BindingsToRemove;
		for (const TPair<FName, FString>& Binding : OtherModule->Bindings)
		{
			const FRigHierarchyModulePath BindingModulePath(Binding.Value);
			if (BindingModulePath.HasModuleName(ModuleName))
			{
				if (const FProperty* Property = ModuleSource.FindVariablePropertyByName(BindingModulePath.GetElementFName()))
				{
					// remove advanced, private or not editable properties
					const bool bIsAdvanced = Property->HasAnyPropertyFlags(CPF_AdvancedDisplay);
					const bool bIsPublic = Property->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst);
					const bool bIsInstanceEditable = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
					if (bIsAdvanced || !bIsPublic || !bIsInstanceEditable)
					{
						BindingsToRemove.Add(Binding.Key);
					}
					else 
					{
						FText ErrorMessage;
						if (!CanBindModuleVariable(OtherModule->Name, Binding.Key, Binding.Value, ErrorMessage))
						{
							BindingsToRemove.Add(Binding.Key);
						}
					}
				}
			}
		}

		for (const FName& ToRemove : BindingsToRemove)
		{
			UnBindModuleVariable(OtherModule->Name, ToRemove);
		}
		return true;
	});
	
#if WITH_EDITOR
	TransactionPtr.Reset();
#endif
}

FString UModularRigController::ExportModuleSettingsToString(TArray<FName> InModuleNames) const
{
	FModularRigModuleSettingsSetForClipboard Content;
	for(const FName& ModuleName : InModuleNames)
	{
		if(const FRigModuleReference* Module = FindModule(ModuleName))
		{
			if(Content.Settings.Contains(ModuleName))
			{
				continue;
			}
			
			const UStruct* ControlRigClass = Module->ControlRigAssetReference.LoadStrongReference().GetVariablesStruct();
			if(!ControlRigClass)
			{
				UE_LOGF(LogControlRig, Error, "Module '%ls' does not have a valid control rig class (%ls) associated.", *ModuleName.ToString(), *Module->ControlRigAssetReference.GetPathName());
				return FString();
			}

			FModularRigModuleSettingsForClipboard& Settings = Content.Settings.Add(ModuleName);
			
			Settings.ModuleClass = Module->ControlRigAssetReference.ToSoftObjectPath();

			// store the overrides as configured by the user
			for(const FControlRigOverrideValue& Override : Module->ConfigOverrides)
			{
				if(Override.IsValid())
				{
					Settings.Overrides.Add(Override.GetPath(), Override.ToString());
				}
			}

			// also store all of the defaults for the rig (including the changes introduced by the current overrides
			for (TFieldIterator<FProperty> PropertyIt( ControlRigClass ); PropertyIt; ++PropertyIt)
			{
				const FProperty* Property = *PropertyIt;
				if(Property->IsNative())
				{
					continue;
				}

				const FString PropertyPath = Property->GetName();

				// store the default
				const FControlRigOverrideValue Default(PropertyPath, ControlRigClass, Module->ControlRigAssetReference.LoadStrongReference().GetVariablesMemory());
				if(Default.IsValid())
				{
					Settings.Defaults.Add(Default.GetPath(), Default.ToString());
				}
			}

			// store the bindings as well
			Settings.Bindings = Module->Bindings;
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Module '%ls' not found.", *ModuleName.ToString());
			return FString();
		}
	}

	FString ContentAsString;
	FModularRigModuleSettingsSetForClipboard::StaticStruct()->ExportText(ContentAsString, &Content, &Content, nullptr, PPF_None, nullptr);
	return ContentAsString; 
}

bool UModularRigController::ImportModuleSettingsFromString(FString InContent, TArray<FName> InOptionalModuleNames, bool bSetupUndo)
{
	FControlRigOverrideValueErrorPipe ErrorPipe(ELogVerbosity::Warning, [this](const TCHAR* V, ELogVerbosity::Type Verbosity)
	{
		UE_LOGF(LogControlRig, Warning, "Error during import: %ls", V);
	});
	
	FModularRigModuleSettingsSetForClipboard Content;
	FModularRigModuleSettingsSetForClipboard::StaticStruct()->ImportText(*InContent, &Content, nullptr, PPF_None, &ErrorPipe, FModularRigModuleSettingsForClipboard::StaticStruct()->GetName(), true);
	if(ErrorPipe.GetNumErrors() > 0)
	{
		return false;
	}

	TArray<FName> ContentModuleNames;
	Content.Settings.GenerateKeyArray(ContentModuleNames);

	const TArray<FName> ModuleNames = InOptionalModuleNames.IsEmpty() ? ContentModuleNames : InOptionalModuleNames;
	if(ModuleNames.Num() != ContentModuleNames.Num())
	{
		UE_LOGF(LogControlRig, Error, "The number of modules selected (%d) doesn't match the number of modules on the clipboard (%d).", ModuleNames.Num(), ContentModuleNames.Num());
		return false;
	}
	
	TMap<FName, FName> ModuleNameLookup, InvModuleNameLookup;
	for(int32 Index=0;Index<ModuleNames.Num();Index++)
	{
		ModuleNameLookup.Add(ModuleNames[Index], ContentModuleNames[Index]);
		InvModuleNameLookup.Add(ContentModuleNames[Index], ModuleNames[Index]);
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if (bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("ModularRigController", "ImportModuleSettingsTransaction", "Import Module Settings"), !GIsTransacting);
	}
#endif

	TArray<FName> AffectedModules;
	auto RecordChange = [this, &AffectedModules](const FName& InModuleName)
	{
#if WITH_EDITOR
		GetOuter()->Modify();
#endif
		AffectedModules.AddUnique(InModuleName);
	};

	Notify(EModularRigNotification::InteractionBracketOpened, nullptr);

	for(const FName& InputModuleName : ModuleNames)
	{
		if(FRigModuleReference* Module = FindModule(InputModuleName))
		{
			const FName& ModuleName = ModuleNameLookup.FindChecked(InputModuleName);
			
			if(!Content.Settings.Contains(ModuleName))
			{
				UE_LOGF(LogControlRig, Error, "There are no settings provided in the content for module '%ls'.", *ModuleName.ToString());
				continue;
			}

			const FControlRigAssetSoftReference ControlRigSoftClass = Module->ControlRigAssetReference;
			const FControlRigAssetStrongReference ControlRigClass = ControlRigSoftClass.LoadStrongReference();
			if(!ControlRigClass.IsValid())
			{
				UE_LOGF(LogControlRig, Error, "Module '%ls' does not have a valid control rig class (%ls) associated.", *ModuleName.ToString(), *Module->ControlRigAssetReference.GetPathName());
				return false;
			}
			
			if(Module->ControlRigAssetReference.ToSoftObjectPath() != Content.Settings.FindChecked(ModuleName).ModuleClass)
			{
				UE_LOGF(LogControlRig, Warning, "Classes for Module '%ls' don't match between the current rig and the clipboard content. Still attempting to apply settings.", *ModuleName.ToString());
			}

			// remove all overrides
			if(!Module->ConfigOverrides.IsEmpty())
			{
				RecordChange(Module->Name);
				Module->ConfigOverrides.Reset();
			}

			FModularRigModuleSettingsForClipboard& Settings = Content.Settings.FindChecked(ModuleName);

			// first compare if the defaults are different between the copied content and now 
			UControlRig* TemporaryRig = ControlRigClass.CreateInstance(GetTransientPackage());
			for(const TPair<FString,FString>& Pair : Settings.Defaults)
			{
				// if we have a top level override - let's not worry about this one
				if(Settings.Overrides.Contains(Pair.Key))
				{
					continue;
				}

				const FControlRigOverrideValueErrorPipe::TReportFunction ReportFunction = [ModuleName, Pair](const TCHAR* V, ELogVerbosity::Type Verbosity)
				{
					UE_LOGF(LogControlRig, Warning, "Problem during import of property '%ls' for module '%ls': %ls", *Pair.Key, *ModuleName.ToString(), V);
				};

				const FControlRigOverrideValue CopiedDefault(Pair.Key, ControlRigClass.GetVariablesStruct(), Pair.Value, Module->Name, ReportFunction);
				if(!CopiedDefault.IsValid())
				{
					UE_LOGF(LogControlRig, Warning, "Cannot apply top level override Module '%ls' with path '%ls'.", *ModuleName.ToString(), *Pair.Key);
					continue;
				}
				
				const FControlRigOverrideValue DefaultOverride(CopiedDefault.GetPath(), ControlRigClass.GetVariablesStruct(), ControlRigClass.GetVariablesMemory(), Module->Name);
				if(DefaultOverride.Identical(CopiedDefault))
				{
					continue;
				}
				
				// copy the original default to the temporary rig
				//CopiedDefault.CopyToUObject(TemporaryRig);
				(void) CopiedDefault.CopyToSubject(TemporaryRig->GetVariablesMemory(), TemporaryRig->GetVariablesStruct());

				// copy all other overrides that potentially sit under there
				for(const TPair<FString,FString>& OverridePair : Settings.Overrides)
				{
					if(!FControlRigOverrideContainer::IsChildPathOf(OverridePair.Key, DefaultOverride.GetPath()))
					{
						continue;
					}
					const FControlRigOverrideValue ChildOverride(OverridePair.Key, ControlRigClass.GetVariablesStruct(), OverridePair.Value, Module->Name, ReportFunction);
					if(!ChildOverride.IsValid())
					{
						continue;
					}

					// ChildOverride.CopyToUObject(TemporaryRig);
					(void) ChildOverride.CopyToSubject(TemporaryRig->GetVariablesMemory(), TemporaryRig->GetVariablesStruct());
				}

				// construct the new override from the temporary rig, combining the copied default and any additional child override
				const FControlRigOverrideValue CombinedOverride(CopiedDefault.GetPath(), ControlRigClass.GetVariablesStruct(), TemporaryRig, Module->Name);
				if(!CombinedOverride.IsValid())
				{
					continue;
				}
				
				RecordChange(Module->Name);
				Module->ConfigOverrides.Add(CombinedOverride);
			}
			
			// now apply all of the user provided overrides as well
			for(const TPair<FString,FString>& Pair : Settings.Overrides)
			{
				const FControlRigOverrideValueErrorPipe::TReportFunction ReportFunction = [ModuleName, Pair](const TCHAR* V, ELogVerbosity::Type Verbosity)
				{
					UE_LOGF(LogControlRig, Warning, "Problem during import of property '%ls' for module '%ls': %ls", *Pair.Key, *ModuleName.ToString(), V);
				};
				
				const FControlRigOverrideValue UserProvidedOverride(Pair.Key, ControlRigClass.GetVariablesStruct(), Pair.Value, Module->Name, ReportFunction);
				if(!UserProvidedOverride.IsValid())
				{
					UE_LOGF(LogControlRig, Warning, "Cannot apply top level override Module '%ls' with path '%ls'.", *ModuleName.ToString(), *Pair.Key);
					continue;
				}

				// this fails in case there's already an override on the parent property path.
				// if there's already an override on .Color, the secondary override on say .Color.R is ignored.
				RecordChange(Module->Name);
				Module->ConfigOverrides.Add(UserProvidedOverride);
			}

			Module->Bindings.Reset();

			// apply the bindings
			for(const TPair<FName, FString>& Pair : Settings.Bindings)
			{
				// potentially remap the source path within the provided set
				FRigHierarchyModulePath Path = Pair.Value;
				if(const FName* RemappedName = InvModuleNameLookup.Find(Path.GetModuleFName()))
				{
					Path.SetModuleName(*RemappedName);
				}
				RecordChange(Module->Name);
				if(!BindModuleVariable(Module->Name, Pair.Key, Path.GetPath(), bSetupUndo))
				{
					UE_LOGF(LogControlRig, Warning, "Cannot recreate binding for module '%ls', property '%ls' to '%ls'.", *Module->Name.ToString(), *Pair.Key.ToString(), *Path.GetPath());
				}
			}
		}
	}

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	if(!AffectedModules.IsEmpty())
	{
		for(const FName& ModuleName : AffectedModules)
		{
			if(const FRigModuleReference* Module = FindModule(ModuleName))
			{
				Notify(EModularRigNotification::ModuleConfigValueChanged, Module);
			}
		}
	}
	Notify(EModularRigNotification::InteractionBracketClosed, nullptr);

	return !AffectedModules.IsEmpty();
}

void UModularRigController::SanitizeName(FRigName& InOutName, bool bAllowNameSpaces)
{
	// Sanitize the name
	FString SanitizedNameString = InOutName.GetName();
	bool bChangedSomething = false;
	for (int32 i = 0; i < SanitizedNameString.Len(); ++i)
	{
		TCHAR& C = SanitizedNameString[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||					 // Any letter
			(C == '_') || (C == '-') || (C == '.') || (C == '|') ||	 // _  - .  | anytime
			(FChar::IsDigit(C)) ||									 // 0-9 anytime
			((i > 0) && (C== ' '));									 // Space after the first character to support virtual bones

		if (!bGoodChar)
		{
			if(bAllowNameSpaces && C == FRigHierarchyModulePath::ModuleNameSuffixChar)
			{
				continue;
			}
			
			C = '_';
			bChangedSomething = true;
		}
	}

	if (SanitizedNameString.Len() > GetMaxNameLength())
	{
		SanitizedNameString.LeftChopInline(SanitizedNameString.Len() - GetMaxNameLength());
		bChangedSomething = true;
	}

	if(bChangedSomething)
	{
		InOutName.SetName(SanitizedNameString);
	}
}

FRigName UModularRigController::GetSanitizedName(const FRigName& InName, bool bAllowNameSpaces)
{
	FRigName Name = InName;
	SanitizeName(Name, bAllowNameSpaces);
	return Name;
}

bool UModularRigController::IsNameAvailable(const FRigName& InDesiredName, FString* OutErrorMessage, const FRigModuleReference* InModuleToSkip) const
{
	const FRigName DesiredName = GetSanitizedName(InDesiredName, false);
	if(DesiredName != InDesiredName)
	{
		if(OutErrorMessage)
		{
			static const FString ContainsInvalidCharactersMessage = TEXT("Name contains invalid characters.");
			*OutErrorMessage = ContainsInvalidCharactersMessage;
		}
		return false;
	}

	static const FString NameAlreadyInUse = TEXT("This name is already in use.");

	// the default is to have unique names per module
	for(const FRigModuleReference& Module : Model->Modules)
	{
		if(&Module == InModuleToSkip)
		{
			continue;
		}
		if(Module.Name == DesiredName.GetFName())
		{
			if(OutErrorMessage)
			{
				*OutErrorMessage = NameAlreadyInUse;
			}
			return false;
		}
	}
	return true;
}

FRigName UModularRigController::GetSafeNewName(const FRigName& InDesiredName, const FRigModuleReference* InModuleToSkip) const
{
	bool bSafeToUse = false;

	// create a copy of the desired name so that the string conversion can be cached
	const FRigName DesiredName = GetSanitizedName(InDesiredName, false);
	FRigName NewName = DesiredName;
	int32 Index = 0;
	while (!bSafeToUse)
	{
		bSafeToUse = true;
		if(!IsNameAvailable(NewName, nullptr, InModuleToSkip))
		{
			bSafeToUse = false;
			NewName = FString::Printf(TEXT("%s_%d"), *DesiredName.ToString(), ++Index);
		}
	}
	return NewName;
}

void UModularRigController::Notify(const EModularRigNotification& InNotification, const FRigModuleReference* InElement)
{
	if(!bSuspendNotifications)
	{
		ModifiedEvent.Broadcast(InNotification, InElement);
	}
}

UModularRig* UModularRigController::GetDebuggedModularRig()
{
#if WITH_EDITOR
	
	if(const UBlueprint* Blueprint = Cast<UBlueprint>(GetOuter()))
	{
		if(UModularRig* ModularRig = Cast<UModularRig>(Blueprint->GetObjectBeingDebugged()))
		{
			return ModularRig;
		}
		if(const UClass* Class = Blueprint->GeneratedClass)
		{
			if(UModularRig* CDO = Cast<UModularRig>(Class->GetDefaultObject(true)))
			{
				return CDO;
			}
		}
	}
	else if (URigVMRuntimeAsset* RuntimeAsset = Cast<URigVMRuntimeAsset>(GetOuter()))
	{
		if(UModularRig* ModularRig = Cast<UModularRig>(RuntimeAsset->GetObjectBeingDebugged()))
		{
			return ModularRig;
		}
	}
#endif
	return nullptr;
}

const UModularRig* UModularRigController::GetDebuggedModularRig() const
{
	return const_cast<UModularRigController*>(this)->GetDebuggedModularRig();
}

FModularRigControllerCompileBracketScope::FModularRigControllerCompileBracketScope(UModularRigController* InController)
	: Controller(InController), bSuspendNotifications(InController->bSuspendNotifications)
{
	check(InController);
	
	if (bSuspendNotifications)
	{
		return;
	}
	InController->Notify(EModularRigNotification::InteractionBracketOpened, nullptr);
}

FModularRigControllerCompileBracketScope::~FModularRigControllerCompileBracketScope()
{
	check(Controller);
	if (bSuspendNotifications)
	{
		return;
	}
	Controller->Notify(EModularRigNotification::InteractionBracketClosed, nullptr);
}
