// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigModel.h"
#include "ModularRigController.h"
#include "ModularRig.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularRigModel)

FString FRigModuleReference::GetElementPrefix() const
{
	return GetName() + FRigHierarchyModulePath::ModuleNameSuffix;
}

FRigHierarchyModulePath FRigModuleReference::GetModulePath() const
{
	if(ParentModuleName.IsNone())
	{
		if(!ParentPath_DEPRECATED.IsEmpty())
		{
			return URigHierarchy::JoinNameSpace_Deprecated(ParentPath_DEPRECATED, Name.ToString()); 
		}
	}
	else if(Model)
	{
		if(const FRigModuleReference* ParentModule = GetParentModule())
		{
			return FRigHierarchyModulePath(ParentModule->GetModulePath(), Name.ToString()); 
		}
	}
	return Name.ToString();
}

const FRigModuleReference* FRigModuleReference::GetParentModule() const
{
	if(ParentModuleName.IsNone())
	{
		if(!ParentPath_DEPRECATED.IsEmpty())
		{
			return Model->FindModuleByPath(ParentPath_DEPRECATED);
		}
	}
	else if(Model)
	{
		return Model->GetParentModule(this);
	}
	return nullptr;
}

const FRigModuleReference* FRigModuleReference::GetRootModule() const
{
	if(ParentModuleName.IsNone())
	{
		return this;
	}
	if(const FRigModuleReference* ParentModule = GetParentModule())
	{
		return ParentModule->GetRootModule();
	}
	return nullptr;
}

const FRigConnectorElement* FRigModuleReference::FindPrimaryConnector(const URigHierarchy* InHierarchy) const
{
	if(InHierarchy)
	{
		const TArray<FRigConnectorElement*> AllConnectors = InHierarchy->GetConnectors();
		for(const FRigConnectorElement* Connector : AllConnectors)
		{
			if(Connector->IsPrimary())
			{
				const FName ModuleName = InHierarchy->GetModuleFName(Connector->GetKey());
				if(Name == ModuleName)
				{
					return Connector;
				}
			}
		}
	}
	return nullptr;
}

TArray<const FRigConnectorElement*> FRigModuleReference::FindConnectors(const URigHierarchy* InHierarchy) const
{
	TArray<const FRigConnectorElement*> Connectors;
	if(InHierarchy)
	{
		const FName MyModuleName = GetFName();
		const TArray<FRigConnectorElement*> AllConnectors = InHierarchy->GetConnectors();
		for(const FRigConnectorElement* Connector : AllConnectors)
		{
			const FName ModuleName = InHierarchy->GetModuleFName(Connector->GetKey());
			if(!ModuleName.IsNone())
			{
				if(ModuleName == MyModuleName)
				{
					Connectors.Add(Connector);
				}
			}
		}
	}
	return Connectors;
}

void FRigModuleReference::PatchModelsOnLoad()
{
	if (const UClass* ClassPtr = ControlRigAssetReference.LoadStrongReference().GetBlueprintClass())
	{
		if(!ConfigValues_DEPRECATED.IsEmpty())
		{
			ConfigOverrides.Reset();
			ConfigOverrides.SetUsesKeyForSubject(false);
			ConfigOverrides.Reserve(ConfigValues_DEPRECATED.Num());
			for(const TPair<FName, FString>& Pair : ConfigValues_DEPRECATED)
			{
				ConfigOverrides.Add(FControlRigOverrideValue(Pair.Key.ToString(), ClassPtr, Pair.Value, Name));
			}
			ConfigValues_DEPRECATED.Reset();
		}
	}
}

FRigElementKeyRedirector::FKeyArray FModularRigSingleConnection::GetTargetArray() const
{
	FRigElementKeyRedirector::FKeyArray Result;
	Result.Append(Targets);
	return Result;
}

FRigElementKeyRedirector::FKeyMap FModularRigConnections::GetModuleConnectionMap(const FName& InModuleName) const
{
	const FString InModuleNameString = InModuleName.ToString();
	
	FRigElementKeyRedirector::FKeyMap Result;
	for (const FModularRigSingleConnection& Connection : ConnectionList)
	{
		const FRigHierarchyModulePath ModulePath(Connection.Connector.Name);

		// Exactly the same path (do not return connectors from child modules)
		if (ModulePath.HasModuleName(InModuleNameString))
		{
			Result.Add(FRigElementKey(ModulePath.GetElementFName(), ERigElementType::Connector), Connection.GetTargetArray());
		}
	}
	return Result;
}

void FModularRigConnections::PatchOnLoad(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName)
{
	for(FModularRigSingleConnection& Connection : ConnectionList)
	{
		if(Connection.Targets.IsEmpty() || !Connection.Targets[0].IsValid())
		{
			if(Connection.Target_DEPRECATED.IsValid())
			{
				Connection.Targets.Reset();
				Connection.Targets.Add(Connection.Target_DEPRECATED);
				Connection.Target_DEPRECATED.Reset();
			}
		}

		Connection.Connector.ConvertToModuleNameFormatInline(InModulePathToModuleName);
		for(FRigElementKey& Target : Connection.Targets)
		{
			Target.ConvertToModuleNameFormatInline(InModulePathToModuleName);
		}
	}
}

void FModularRigModel::PatchModelsOnLoad()
{
	bool bHasDeprecatedParentPath = false;
	for(int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
	{
		Modules[ModuleIndex].Model = this;
		bHasDeprecatedParentPath = bHasDeprecatedParentPath || !Modules[ModuleIndex].ParentPath_DEPRECATED.IsEmpty();
		
		if (Modules[ModuleIndex].Class_DEPRECATED.IsValid() || Modules[ModuleIndex].Class_DEPRECATED.IsPending())
		{
			Modules[ModuleIndex].ControlRigAssetReference.Set(Modules[ModuleIndex].Class_DEPRECATED);
			Modules[ModuleIndex].Class_DEPRECATED = nullptr;
		}
	}
	
	if (Connections.IsEmpty())
	{
		ForEachModule([this](const FRigModuleReference* Module) -> bool
		{
			FString ElementPrefix = Module->GetElementPrefix();
			for (const TTuple<FRigElementKey, FRigElementKey>& Connection : Module->Connections_DEPRECATED)
			{
				const FString ConnectorPath = FString::Printf(TEXT("%s%s"), *ElementPrefix, *Connection.Key.Name.ToString());
				FRigElementKey ConnectorKey(*ConnectorPath, ERigElementType::Connector);
				Connections.AddConnection(ConnectorKey, {Connection.Value});
			}
			return true;
		});
	}

	// if we need to introduce a unique name list we'll have to also fill the previous module path name list
	if(bHasDeprecatedParentPath)
	{
		PreviousModulePaths.Reset();

		TArray<FString> Paths;
		TArray<FName> Names;
		Paths.Reserve(Modules.Num());
		Names.Reserve(Modules.Num());
		
		// temporarily rename all modules
		for(int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
		{
			FRigModuleReference& Module = Modules[ModuleIndex];
			Paths.Add(Module.GetModulePath());
			Names.Add(Module.bShortNameBasedOnPath_DEPRECATED ? Module.Name : *Module.ShortName_DEPRECATED);
			Module.Name = *FString::Printf(TEXT("____TEMPORARY_MODULE_NAME_%03d"), ModuleIndex);
		}

		// start renaming it again
		for(int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
		{
			FRigModuleReference& Module = Modules[ModuleIndex];
			Module.Name = GetController()->GetSafeNewName(Names[ModuleIndex]);
			PreviousModulePaths.Add(Paths[ModuleIndex], Module.Name);
		}

		// update the parent module name
		for(int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
		{
			FRigModuleReference& Module = Modules[ModuleIndex];
			if(!Module.ParentPath_DEPRECATED.IsEmpty())
			{
				if (FName* ParentPath = PreviousModulePaths.Find(Module.ParentPath_DEPRECATED))
				{
					Module.ParentModuleName = *ParentPath;
					Module.ParentPath_DEPRECATED.Reset();
				}
			}
		}

		// update the module bindings
		for(int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
		{
			FRigModuleReference& Module = Modules[ModuleIndex];
			for(TPair<FName, FString>& Binding : Module.Bindings)
			{
				FRigHierarchyModulePath BindingModulePath(Binding.Value);
				if(BindingModulePath.ConvertToModuleNameFormatInline(&PreviousModulePaths))
				{
					Binding.Value = BindingModulePath;
				}
			}
		}
	}
	UpdateCachedChildren();

	Connections.PatchOnLoad(&PreviousModulePaths);
	Connections.UpdateFromConnectionList();

	for(int32 ModuleIndex = 0; ModuleIndex < Modules.Num(); ModuleIndex++)
	{
		FRigModuleReference& Module = Modules[ModuleIndex];
		Module.PatchModelsOnLoad();
	}
}

UModularRigController* FModularRigModel::GetController(bool bCreateIfNeeded)
{
	if (bCreateIfNeeded && Controller == nullptr)
	{
		const FName SafeControllerName = *FString::Printf(TEXT("%s_ModularRig_Controller"), *GetOuter()->GetPathName());
		UModularRigController* NewController = NewObject<UModularRigController>(GetOuter(), UModularRigController::StaticClass(), SafeControllerName);
		NewController->SetModel(this);
		Controller = NewController;
	}
	return Cast<UModularRigController>(Controller);
}

void FModularRigModel::SetOuterClientHost(UObject* InOuterClientHost)
{
	OuterModelHost = InOuterClientHost;
}

void FModularRigModel::SetOuterModelHost(UObject* InOuterModelHost)
{
	OuterModelHost = InOuterModelHost;
}

void FModularRigModel::UpdateCachedChildren()
{
	TMap<FName, FRigModuleReference*> ModuleByName;
	ModuleByName.Reserve(Modules.Num());

	// also offer a backwards compatible lookup
	TMap<FString, FRigModuleReference*> ModuleByPath;
	
	for (FRigModuleReference& Module : Modules)
	{
		Module.CachedChildren.Reset();

		if(Module.ParentPath_DEPRECATED.IsEmpty())
		{
			ModuleByName.Add(Module.GetFName(), &Module);
		}
		else
		{
			ModuleByPath.Add(Module.GetModulePath(), &Module);
		}
	}
	
	RootModules.Reset();
	for (FRigModuleReference& Module : Modules)
	{
		if (!Module.HasParentModule())
		{
			RootModules.Add(&Module);
		}
		else
		{
			if (FRigModuleReference** ParentModule = ModuleByName.Find(Module.ParentModuleName))
			{
				(*ParentModule)->CachedChildren.Add(&Module);
			}
			else if (FRigModuleReference** ParentModuleByPath = ModuleByPath.Find(Module.ParentPath_DEPRECATED))
			{
				(*ParentModuleByPath)->CachedChildren.Add(&Module);
			}
		}
	}
}

FRigModuleReference* FModularRigModel::FindModule(const FName& InModuleName)
{
	if(InModuleName.IsNone())
	{
		return nullptr;
	}
	
	FRigModuleReference* FoundModule = Modules.FindByPredicate([InModuleName](const FRigModuleReference& Module)
	{
		return Module.Name == InModuleName;
	});

	if(FoundModule == nullptr)
	{
		FoundModule = const_cast<FRigModuleReference*>(FindModuleByPath(InModuleName.ToString()));
#if WITH_EDITOR
		if(FoundModule && !IsLoading())
		{
			const int32 NumReceivedOldModulePaths = ReceivedOldModulePaths.Num(); 
			if(ReceivedOldModulePaths.AddUnique(InModuleName) == NumReceivedOldModulePaths)
			{
				if(const UObject* Outer = GetOuter())
				{
					if(!IsRunningCookCommandlet() && !IsRunningCookOnTheFly())
					{
						UE_LOGF(
							LogControlRig,
							Warning,
							"%ls: Module '%ls' has been accessed using an old module path ('%ls'). Please consider updating your code.",
							*Outer->GetPathName(),
							*FoundModule->GetName(),
							*InModuleName.ToString()
						);
					}
				}
			}
		}
#endif
	}
	
	return FoundModule;
}

const FRigModuleReference* FModularRigModel::FindModule(const FName& InName) const
{
	FRigModuleReference* Module = const_cast<FModularRigModel*>(this)->FindModule(InName);
	return Module;
}

const FRigModuleReference* FModularRigModel::FindModuleByPath(const FString& InModulePath) const
{
	if(InModulePath.IsEmpty())
	{
		return nullptr;
	}
	
	return Modules.FindByPredicate([InModulePath](const FRigModuleReference& Module)
	{
		return Module.GetModulePath() == InModulePath;
	});
}

FRigModuleReference* FModularRigModel::GetParentModule(const FName& InName)
{
	if(const FRigModuleReference* Module = FindModule(InName))
	{
		return GetParentModule(Module);
	}
	return nullptr;
}

const FRigModuleReference* FModularRigModel::GetParentModule(const FName& InName) const
{
	FRigModuleReference* Module = const_cast<FModularRigModel*>(this)->GetParentModule(InName);
	return const_cast<FRigModuleReference*>(Module);
}

FRigModuleReference* FModularRigModel::GetParentModule(const FRigModuleReference* InChildModule)
{
	if(InChildModule)
	{
		if(!InChildModule->ParentModuleName.IsNone())
		{
			return FindModule(InChildModule->ParentModuleName);
		}
	}
	return nullptr;
}

const FRigModuleReference* FModularRigModel::GetParentModule(const FRigModuleReference* InChildModule) const
{
	FRigModuleReference* Module = const_cast<FModularRigModel*>(this)->GetParentModule(InChildModule);
	return const_cast<FRigModuleReference*>(Module);
}

bool FModularRigModel::IsModuleParentedTo(const FName& InChildModuleName, const FName& InParentModuleName) const
{
	const FRigModuleReference* ChildModule = FindModule(InChildModuleName);
	const FRigModuleReference* ParentModule = FindModule(InParentModuleName);
	if(ChildModule == nullptr || ParentModule == nullptr)
	{
		return false;
	}
	return IsModuleParentedTo(ChildModule, ParentModule);
}

bool FModularRigModel::IsModuleParentedTo(const FRigModuleReference* InChildModule, const FRigModuleReference* InParentModule) const
{
	if(InChildModule == nullptr || InParentModule == nullptr)
	{
		return false;
	}
	if(InChildModule == InParentModule)
	{
		return false;
	}
	TArray<const FRigModuleReference*> VisitedModules;
	while(InChildModule)
	{
		if (VisitedModules.Contains(InChildModule))
		{
			// we have found a cycle, lets prevent an infinite loop
			break;
		}
		VisitedModules.Add(InChildModule);
		
		if(InChildModule == InParentModule)
		{
			return true;
		}
		InChildModule = GetParentModule(InChildModule);
	}

	return false;
}

TArray<const FRigModuleReference*> FModularRigModel::FindModuleInstancesOfClass(const FString& InModuleClassPath) const
{
	TArray<const FRigModuleReference*> Result;
	FString ModuleClassPath = InModuleClassPath;
	ModuleClassPath.RemoveFromEnd(TEXT("_C"));
	ForEachModule([&Result, &ModuleClassPath](const FRigModuleReference* Module) -> bool
	{
		FString PackageName = Module->ControlRigAssetReference.ToSoftObjectPath().GetAssetPathString();
		PackageName.RemoveFromEnd(TEXT("_C"));
		if (PackageName == ModuleClassPath)
		{
			Result.Add(Module);
		}
		return true;
	});
	return Result;
}

TArray<const FRigModuleReference*> FModularRigModel::FindModuleInstancesOfClass(const FAssetData& InModuleAsset) const
{
	return FindModuleInstancesOfClass(InModuleAsset.GetObjectPathString());
}

TArray<const FRigModuleReference*> FModularRigModel::FindModuleInstancesOfClass(TSoftClassPtr<UControlRig> InClass) const
{
	return FindModuleInstancesOfClass(*InClass.ToString());
}

TArray<FName> FModularRigModel::GetSupportedEvents() const
{
	TArray<FName> Result;
	ForEachModule([&Result](const FRigModuleReference* Module) -> bool
	{
		FControlRigAssetStrongReference AssetReference = Module->ControlRigAssetReference.LoadStrongReference();
		if (AssetReference.IsValid())
		{
			TArray<FName> Events = AssetReference.GetSupportedEvents();
			for (const FName& Event : Events)
			{
				Result.AddUnique(Event);
			}
		}
		return true;
	});
	return Result;
}

void FModularRigModel::ForEachModule(TFunction<bool(const FRigModuleReference*)> PerModule, bool bDepthFirst) const
{
	if (bDepthFirst)
	{
		for(const FRigModuleReference* RootModule : RootModules)
		{
			if (!TraverseModules(RootModule, PerModule))
			{
				break;
			}
		}
	}
	else
	{
		TArray<FRigModuleReference*> ModuleInstances = RootModules;
		for (int32 Index=0; Index < ModuleInstances.Num(); ++Index)
		{
			if (!PerModule(ModuleInstances[Index]))
			{
				break;
			}
			ModuleInstances.Append(ModuleInstances[Index]->CachedChildren);
		}
	}
}

bool FModularRigModel::TraverseModules(const FRigModuleReference* InModuleReference, TFunction<bool(const FRigModuleReference*)> PerModule)
{
	if (!PerModule(InModuleReference))
	{
		return false;
	}
	for (const FRigModuleReference* ChildModule : InModuleReference->CachedChildren)
	{
		if (!TraverseModules(ChildModule, PerModule))
		{
			return false;
		}
	}
	return true;
}

TArray<FName> FModularRigModel::SortModuleNames(const TArray<FName>& InModuleNames) const
{
	TArray<FName> SortedModuleNames;
	ForEachModule([&InModuleNames, &SortedModuleNames](const FRigModuleReference* Module) -> bool
	{
		if(InModuleNames.Contains(Module->GetFName()))
		{
			SortedModuleNames.AddUnique(Module->GetFName());
		}
		return true;
	});
	return SortedModuleNames;
}
