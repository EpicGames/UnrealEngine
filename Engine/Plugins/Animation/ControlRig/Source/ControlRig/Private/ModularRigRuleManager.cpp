// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigRuleManager.h"

#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularRigRuleManager)

#define LOCTEXT_NAMESPACE "ModularRigRuleManager"

FModularRigResolveResult UModularRigRuleManager::FindMatches(FWorkData& InWorkData) const
{
	check(InWorkData.Result);
	FModularRigResolveResult& Result = *InWorkData.Result;

	if(!Hierarchy.IsValid())
	{
		static const FText MissingHierarchyMessage = LOCTEXT("MissingHierarchyMessage", "The rule manager is missing the hierarchy.");
		Result.Message = MissingHierarchyMessage;
		Result.State = EModularRigResolveState::Error;
		return Result;
	}

	if (UControlRig* ControlRig = Hierarchy->GetTypedOuter<UControlRig>())
	{
		if (ControlRig->IsConstructionRequired())
		{
			const FRigElementKey ConnectorKey = InWorkData.Connector ? InWorkData.Connector->GetKey() : FRigElementKey();
			
			ControlRig->Execute(FRigUnit_PrepareForExecution::EventName);

			// restore the connector. executing the control rig may have destroyed the previous connector element
			InWorkData.Connector = Hierarchy->Find<FRigConnectorElement>(ConnectorKey);
			if(ConnectorKey.IsValid() && InWorkData.Connector == nullptr)
			{
				return Result;
			}
		}
	}

	// start with a full set of possible targets
	TArray<bool> VisitedElement;
	VisitedElement.AddZeroed(Hierarchy->Num());
	Result.Matches.Reserve(Hierarchy->Num());
	Hierarchy->Traverse([&VisitedElement, &Result](const FRigBaseElement* Element, bool& bContinue)
	{
		if(!VisitedElement[Element->GetIndex()])
		{
			VisitedElement[Element->GetIndex()] = true;
			Result.Matches.Emplace(Element->GetKey(), ERigElementResolveState::PossibleTarget, FText());
		}
		bContinue = true;
	}, true);

	InWorkData.Hierarchy = Hierarchy.Get();
	ResolveConnector(InWorkData);
	return Result;;
}

FModularRigResolveResult UModularRigRuleManager::FindMatches(
	const FRigConnectorElement* InConnector,
	const FRigModuleInstance* InModule,
	const FRigElementKeyRedirector& InResolvedConnectors) const
{
	FModularRigResolveResult Result;
	Result.Connector = InConnector->GetKey();

	FWorkData WorkData;
	WorkData.Hierarchy = Hierarchy.Get();
	WorkData.Connector = InConnector;
	WorkData.Module = InModule;
	WorkData.ResolvedConnectors = &InResolvedConnectors;
	WorkData.Result = &Result;

	return FindMatches(WorkData);
}

FModularRigResolveResult UModularRigRuleManager::FindMatches(const FRigModuleConnector* InConnector) const
{
	FModularRigResolveResult Result;
	Result.Connector = FRigElementKey(*InConnector->Name, ERigElementType::Connector);

	FWorkData WorkData;
	WorkData.Hierarchy = Hierarchy.Get();
	WorkData.ModuleConnector = InConnector;
	WorkData.Result = &Result;

	return FindMatches(WorkData);
}

FModularRigResolveResult UModularRigRuleManager::FindMatchesForPrimaryConnector(const FRigModuleInstance* InModule) const
{
	static const FRigElementKeyRedirector EmptyRedirector;
	FRigConnectionRuleInput RuleInput;
	RuleInput.Hierarchy = Hierarchy.Get();
	RuleInput.Module = InModule;
	RuleInput.Redirector = &EmptyRedirector;

	FModularRigResolveResult Result;
	
	if(const FRigConnectorElement* PrimaryConnector = RuleInput.FindPrimaryConnector(&Result.Message))
	{
		Result = FindMatches(PrimaryConnector, InModule, EmptyRedirector);
	}
	else
	{
		Result.State = EModularRigResolveState::Error;
	}

	return Result;
}

TArray<FModularRigResolveResult> UModularRigRuleManager::FindMatchesForSecondaryConnectors(const FRigModuleInstance* InModule, const FRigElementKeyRedirector& InResolvedConnectors) const
{
	FRigConnectionRuleInput RuleInput;
	RuleInput.Hierarchy = Hierarchy.Get();
	RuleInput.Module = InModule;
	RuleInput.Redirector = &InResolvedConnectors;

	TArray<FModularRigResolveResult> Results;

	const TArray<const FRigConnectorElement*> SecondaryConnectors = RuleInput.FindSecondaryConnectors(false /* optional */);
	for(const FRigConnectorElement* SecondaryConnector : SecondaryConnectors)
	{
		Results.Add(FindMatches(SecondaryConnector, InModule, InResolvedConnectors));
	}

	return Results;
}

TArray<FModularRigResolveResult> UModularRigRuleManager::FindMatchesForOptionalConnectors(const FRigModuleInstance* InModule, const FRigElementKeyRedirector& InResolvedConnectors) const
{
	FRigConnectionRuleInput RuleInput;
	RuleInput.Hierarchy = Hierarchy.Get();
	RuleInput.Module = InModule;
	RuleInput.Redirector = &InResolvedConnectors;

	TArray<FModularRigResolveResult> Results;

	const TArray<const FRigConnectorElement*> OptionalConnectors = RuleInput.FindSecondaryConnectors(true /* optional */);
	for(const FRigConnectorElement* OptionalConnector : OptionalConnectors)
	{
		Results.Add(FindMatches(OptionalConnector, InModule, InResolvedConnectors));
	}

	return Results;
}

void UModularRigRuleManager::FWorkData::Filter(TFunction<void(FRigElementResolveResult&)> PerMatchFunction)
{
	const TArray<FRigElementResolveResult> PreviousMatches = Result->Matches;;
	Result->Matches.Reset();
	for(const FRigElementResolveResult& PreviousMatch : PreviousMatches)
	{
		FRigElementResolveResult Match = PreviousMatch;
		PerMatchFunction(Match);
		if(Match.IsValid())
		{
			Result->Matches.Add(Match);
		}
		else
		{
			Result->Excluded.Add(Match);
		}
	}
}

void UModularRigRuleManager::SetHierarchy(const URigHierarchy* InHierarchy)
{
	check(InHierarchy);
	Hierarchy = InHierarchy;
}

void UModularRigRuleManager::ResolveConnector(FWorkData& InOutWorkData)
{
	FilterIncompatibleTypes(InOutWorkData);
	FilterInvalidModules(InOutWorkData);
	FilterByConnectorRules(InOutWorkData);
	FilterByConnectorEvent(InOutWorkData);

	if(InOutWorkData.Result->Matches.IsEmpty())
	{
		InOutWorkData.Result->State = EModularRigResolveState::Error;
	}
	else
	{
		InOutWorkData.Result->State = EModularRigResolveState::Success;
	}
}

void UModularRigRuleManager::FilterIncompatibleTypes(FWorkData& InOutWorkData)
{
	InOutWorkData.Filter([](FRigElementResolveResult& Result)
	{
		if(Result.GetKey().Type == ERigElementType::Curve)
		{
			static const FText CurveInvalidTargetMessage = LOCTEXT("CannotConnectToCurves", "Cannot connect to curves.");
			Result.SetInvalidTarget(CurveInvalidTargetMessage);
		}
		if(Result.GetKey().Type == ERigElementType::Connector)
		{
			static const FText CurveInvalidTargetMessage = LOCTEXT("CannotConnectToConnectors", "Cannot connect to connectors.");
			Result.SetInvalidTarget(CurveInvalidTargetMessage);
		}
	});
}

void UModularRigRuleManager::FilterInvalidModules(FWorkData& InOutWorkData)
{
	if(InOutWorkData.Connector == nullptr)
	{
		return;
	}
	
	const FName ModuleName = InOutWorkData.Hierarchy->GetModuleFName(InOutWorkData.Connector->GetKey());
	if(ModuleName.IsNone())
	{
		return;
	}
	
	InOutWorkData.Filter([ModuleName, InOutWorkData](FRigElementResolveResult& Result)
	{
		const FName MatchModuleName = InOutWorkData.Hierarchy->GetModuleFName(Result.GetKey());
		if(MatchModuleName == ModuleName)
		{
			static const FText CannotConnectWithinNameSpaceMessage = LOCTEXT("CannotConnectWithinNameSpace", "Cannot connect within the same namespace.");
			Result.SetInvalidTarget(CannotConnectWithinNameSpaceMessage);
		}
		else
		{
			if(InOutWorkData.Module)
			{
				const int32 SpawnStartIndex =
					InOutWorkData.Connector->IsPostConstructionConnector() ?
						InOutWorkData.Module->PostConstructionSpawnStartIndex :
						InOutWorkData.Module->ConstructionSpawnStartIndex;

				const int32 TargetSpawnIndex = InOutWorkData.Hierarchy->GetSpawnIndex(Result.GetKey());

				if(SpawnStartIndex != INDEX_NONE && TargetSpawnIndex >= SpawnStartIndex)
				{
					static const FText CannotConnectBelowNameSpaceMessage = LOCTEXT("CannotConnectLaterSpawnElement", "Cannot connect to an element spawn later than the connector's module.");
					Result.SetInvalidTarget(CannotConnectBelowNameSpaceMessage);
					return;
				}

				// if we don't have a spawn index at all - let's rely on module hierarchy only.
				if(SpawnStartIndex == INDEX_NONE && InOutWorkData.Module->HasChildModule(MatchModuleName, true))
				{
					static const FText CannotConnectBelowNameSpaceMessage = LOCTEXT("CannotConnectBelowNameSpace", "Cannot connect to element below the connector's namespace.");
					Result.SetInvalidTarget(CannotConnectBelowNameSpaceMessage);
				}
			}
		}
	});
}

void UModularRigRuleManager::FilterByConnectorRules(FWorkData& InOutWorkData)
{
	check(InOutWorkData.Connector != nullptr || InOutWorkData.ModuleConnector != nullptr);
	
	const TArray<FRigConnectionRuleStash>& Rules =
		InOutWorkData.Connector ? InOutWorkData.Connector->Settings.Rules : InOutWorkData.ModuleConnector->Settings.Rules;
	
	for(const FRigConnectionRuleStash& Stash : Rules)
	{
		if (!Stash.IsValid())
		{
			continue;
		}
		TSharedPtr<FStructOnScope> Storage;
		const FRigConnectionRule* Rule = Stash.Get(Storage);

		FRigConnectionRuleInput RuleInput;
		RuleInput.Hierarchy = InOutWorkData.Hierarchy;
		RuleInput.Redirector = InOutWorkData.ResolvedConnectors;
		RuleInput.Module = InOutWorkData.Module;
		
		InOutWorkData.Filter([Rule, RuleInput](FRigElementResolveResult& Result)
		{
			const FRigBaseElement* Target = RuleInput.Hierarchy->Find(Result.GetKey());
			check(Target);
			Result = Rule->Resolve(Target, RuleInput);
		});
	}
}

void UModularRigRuleManager::FilterByConnectorEvent(FWorkData& InOutWorkData)
{
	// this may be null during unit tests
	if(InOutWorkData.Module == nullptr)
	{
		return;
	}

	if(InOutWorkData.Connector == nullptr)
	{
		return;
	}
	
	// see if we are nested below a modular rig
	UModularRig* ModularRig = InOutWorkData.Hierarchy->GetTypedOuter<UModularRig>();
	if(ModularRig == nullptr)
	{
		return;
	}
	
	FModularRigResolveResult* Result = InOutWorkData.Result;

	ModularRig->ExecuteConnectorEvent(InOutWorkData.Connector->GetKey(), InOutWorkData.Module, InOutWorkData.ResolvedConnectors, InOutWorkData.Result->Matches);
	
	// move the default match to the front of the list
	if(!Result->Matches.IsEmpty())
	{
		const int32 DefaultMatchIndex = Result->Matches.IndexOfByPredicate([](const FRigElementResolveResult& ElementResult) -> bool
		{
			return ElementResult.State == ERigElementResolveState::DefaultTarget;
		}) ;

		if(DefaultMatchIndex != INDEX_NONE)
		{
			const FRigElementResolveResult DefaultResult = Result->Matches[DefaultMatchIndex];
			Result->Matches.RemoveAt(DefaultMatchIndex);
			Result->Matches.Insert(DefaultResult, 0);
		};
	}
}

#undef LOCTEXT_NAMESPACE
