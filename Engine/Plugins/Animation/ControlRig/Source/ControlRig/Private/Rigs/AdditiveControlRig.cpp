// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/AdditiveControlRig.h"
#include "Animation/SmartName.h"
#include "Engine/SkeletalMesh.h"
#include "IControlRigObjectBinding.h"
#include "Components/SkeletalMeshComponent.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/Execution/RigUnit_BeginExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AdditiveControlRig)

#define LOCTEXT_NAMESPACE "AdditiveControlRig"

UAdditiveControlRig::UAdditiveControlRig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCopyHierarchyBeforeConstruction = false;
	bResetInitialTransformsBeforeConstruction = false;
}

FName UAdditiveControlRig::GetControlName(const FName& InBoneName)
{
	if (InBoneName != NAME_None)
	{
		return FName(*(InBoneName.ToString() + TEXT("_CONTROL")));
	}

	// if bone name is coming as none, we don't append
	return NAME_None;
}

FName UAdditiveControlRig::GetNullName(const FName& InBoneName)
{
	if (InBoneName != NAME_None)
	{
		return FName(*(InBoneName.ToString() + TEXT("_NULL")));
	}

	// if bone name is coming as none, we don't append
	return NAME_None;
}

bool UAdditiveControlRig::Execute_Internal(const FName& InEventName)
{
	if (InEventName == FRigUnit_BeginExecution::EventName)
	{
		FControlRigExecuteContext ExecuteContext;
		ExecuteContext.Hierarchy = GetHierarchy();
		ExecuteContext.ControlRig = this;
		ExecuteContext.SetEventName(InEventName);
		ExecuteContext.UnitContext = FRigUnitContext();

		for (FRigUnit_AddBoneTransform& Unit : AddBoneRigUnits)
		{
			FName ControlName = GetControlName(Unit.Bone);
			const int32 Index = GetHierarchy()->GetIndex(FRigElementKey(ControlName, ERigElementType::Control));
			Unit.Transform = GetHierarchy()->GetLocalTransform(Index);
			Unit.Execute(ExecuteContext);
		}
	}
	else if(InEventName == FRigUnit_PrepareForExecution::EventName)
	{
		if(const TSharedPtr<IControlRigObjectBinding> Binding = GetObjectBinding())
		{
			if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Binding->GetBoundObject()))
			{
				CreateRigElements(SkeletalMeshComponent->GetSkeletalMeshAsset());
			}
		}

		// add units and initialize
		AddBoneRigUnits.Reset();

		GetHierarchy()->ForEach<FRigBoneElement>([&](FRigBoneElement* BoneElement) -> bool
		{
			FRigUnit_AddBoneTransform NewUnit;
			NewUnit.Bone = BoneElement->GetFName();
			NewUnit.bPropagateToChildren = true;
			AddBoneRigUnits.Add(NewUnit);
			return true;
		});
	}

	return true;
}

void UAdditiveControlRig::Initialize(bool bInitRigUnits /*= true*/)
{
	PostInitInstanceIfRequired();
	
	Super::Initialize(bInitRigUnits);

	if (GetObjectBinding() == nullptr)
	{
		return;
	}

	Execute_Internal(FRigUnit_PrepareForExecution::EventName);
}

void UAdditiveControlRig::CreateRigElements(const FReferenceSkeleton& InReferenceSkeleton, const USkeleton* InSkeleton)
{
	PostInitInstanceIfRequired();
	
	GetHierarchy()->Reset();
	if (URigHierarchyController* Controller = GetHierarchy()->GetController(true))
	{
		Controller->ImportBones(InReferenceSkeleton, NAME_None, false, false, true, false);

		if (InSkeleton)
		{
			InSkeleton->ForEachCurveMetaData([Controller](const FName& InCurveName, const FCurveMetaData& InMetaData)
			{
				Controller->AddCurve(InCurveName, 0.f, false);
			});
		}

		// add control for all bone hierarchy
		GetHierarchy()->ForEach<FRigBoneElement>([&](FRigBoneElement* BoneElement) -> bool
        {
            const FName BoneName = BoneElement->GetFName();
            const int32 ParentIndex = GetHierarchy()->GetFirstParent(BoneElement->GetIndex());
            const FName NullName = GetNullName(BoneName);// name conflict?
            const FName ControlName = GetControlName(BoneName); // name conflict?

            FRigElementKey NullKey;
		
            if (ParentIndex != INDEX_NONE)
            {
                FTransform GlobalTransform = GetHierarchy()->GetGlobalTransform(BoneElement->GetIndex());
                FTransform ParentTransform = GetHierarchy()->GetGlobalTransform(ParentIndex);
                FTransform LocalTransform = GlobalTransform.GetRelativeTransform(ParentTransform);
                NullKey = Controller->AddNull(NullName, GetHierarchy()->GetKey(ParentIndex), LocalTransform, false, false);
            }
            else
            {
                FTransform GlobalTransform = GetHierarchy()->GetGlobalTransform(BoneElement->GetIndex());
                NullKey = Controller->AddNull(NullName, FRigElementKey(), GlobalTransform, true, false);
            }

            FRigControlSettings Settings;
            Settings.DisplayName = BoneName;
			Settings.ControlType = ERigControlType::Transform;
            Controller->AddControl(ControlName, NullKey, Settings, FRigControlValue::Make(FTransform::Identity), FTransform::Identity, FTransform::Identity, false);

            return true;
        });
	}
}

void UAdditiveControlRig::CreateRigElements(const USkeletalMesh* InReferenceMesh)
{
	if (InReferenceMesh)
	{
		const USkeleton* Skeleton = InReferenceMesh->GetSkeleton();
		CreateRigElements(InReferenceMesh->GetRefSkeleton(), Skeleton);
	}
}

#undef LOCTEXT_NAMESPACE



