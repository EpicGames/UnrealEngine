// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/AnimationHierarchy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationHierarchy)

FTransformConstraint* FConstraintNodeData::FindConstraint(const FName& TargetNode)
{
	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		if (Constraints[ConstraintIdx].TargetNode == TargetNode)
		{
			return &Constraints[ConstraintIdx];
		}
	}

	return nullptr;
}

void FConstraintNodeData::AddConstraint(const FTransformConstraint& TransformConstraint)
{
	FTransformConstraint* ExistingConstraint = FindConstraint(TransformConstraint.TargetNode);
	if (ExistingConstraint)
	{
		*ExistingConstraint = TransformConstraint;
	}
	else
	{
		Constraints.Add(TransformConstraint);
	}
}

void FConstraintNodeData::DeleteConstraint(const FName& TargetNode)
{
	for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ++ConstraintIdx)
	{
		if (Constraints[ConstraintIdx].TargetNode == TargetNode)
		{
			Constraints.RemoveAt(ConstraintIdx);
			--ConstraintIdx;
		}
	}
}

