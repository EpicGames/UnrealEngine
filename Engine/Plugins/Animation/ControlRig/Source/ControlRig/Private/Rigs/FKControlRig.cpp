// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/FKControlRig.h"
#include "Animation/SmartName.h"
#include "AutoRTFM.h"
#include "Containers/StaticArray.h"
#include "Engine/SkeletalMesh.h"
#include "IControlRigObjectBinding.h"
#include "Components/SkeletalMeshComponent.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Settings/ControlRigSettings.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FKControlRig)

#define LOCTEXT_NAMESPACE "OverrideControlRig"

UFKControlRig::UFKControlRig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ApplyMode(EControlRigFKRigExecuteMode::Replace)
	, CachedToggleApplyMode(EControlRigFKRigExecuteMode::Replace)
{
	bCopyHierarchyBeforeConstruction = false;
	bResetInitialTransformsBeforeConstruction = false;
}

FName UFKControlRig::GetControlName(const FName& InName, const ERigElementType& InType)
{
	if (InName != NAME_None)
	{
		if((InType == ERigElementType::Bone || InType == ERigElementType::Curve))
		{
			using NameToControlMappingCache = TStaticArray<TMap<FName, FName>, 2>;
			UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(NameToControlMappingCache, NameToControlMappings);
			TMap<FName, FName>& NameToControlMapping = NameToControlMappings[InType == ERigElementType::Bone ? 0 : 1];
			if (const FName* CachedName = NameToControlMapping.Find(InName))
			{
				return *CachedName;
			}
			else
			{
				UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(FString, ScratchString);

				// ToString performs ScratchString.Reset() internally
				InName.ToString(ScratchString);
				// Force lower-case names
				ScratchString.ToLowerInline();

				if (InType == ERigElementType::Curve)
				{
					static const TCHAR* CurvePostFix = TEXT("_CURVE");
					ScratchString.Append(CurvePostFix);
				}

				static const TCHAR* ControlPostFix = TEXT("_CONTROL");
				ScratchString.Append(ControlPostFix);
				return NameToControlMapping.Add(InName, FName(*ScratchString));
			}
		}
		else if (InType == ERigElementType::Control)
		{
			checkSlow(InName.ToString().Contains(TEXT("_CONTROL")));
			return InName;
		}	
	}

	// if control name is coming as none, we don't append the postfix
	return NAME_None;
}

FName UFKControlRig::GetControlTargetName(const FName& InName, const ERigElementType& InType)
{
	if (InName != NAME_None)
	{
		check(InType == ERigElementType::Bone || InType == ERigElementType::Curve);

		using NameToTargetMappingCache = TStaticArray<TMap<FName, FName>, 2>;
		UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(NameToTargetMappingCache, NameToTargetMappings);
		TMap<FName, FName>& NameToTargetMapping = NameToTargetMappings[InType == ERigElementType::Bone ? 0 : 1];
		if (const FName* CachedName = NameToTargetMapping.Find(InName))
		{
			return *CachedName;
		}
		else
		{
			UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(FString, ScratchString);
			
			// ToString performs ScratchString.Reset() internally
			InName.ToString(ScratchString);

			int32 StartPostFix;
			if (InType == ERigElementType::Curve)
			{
				static const TCHAR* CurvePostFix = TEXT("_CURVE_CONTROL");
				StartPostFix = ScratchString.Find(CurvePostFix, ESearchCase::CaseSensitive);
			}
			else
			{
				static const TCHAR* ControlPostFix = TEXT("_CONTROL");
				StartPostFix = ScratchString.Find(ControlPostFix, ESearchCase::CaseSensitive);
			}

			FString ControlTargetString = FString::ConstructFromPtrSize(*ScratchString, StartPostFix != INDEX_NONE ? StartPostFix : ScratchString.Len());

			// Force lower-case names
			ControlTargetString.ToLowerInline();

			return NameToTargetMapping.Add(InName, FName(ControlTargetString));
		}
	}

	// If incoming name is NONE, or not correctly formatted according to expected post-fix return NAME_None
	return NAME_None;
}

bool UFKControlRig::Execute_Internal(const FName& InEventName)
{
#if WITH_EDITOR
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		if(Hierarchy->IsTracingChanges())
		{
			Hierarchy->StorePoseForTrace(TEXT("UFKControlRig::BeginExecuteUnits"));
		}
		Hierarchy->bRecordInstructionsAtRuntime = false;
	}
#endif	

	if (InEventName == FRigUnit_BeginExecution::EventName)
	{
		GetHierarchy()->ForEach<FRigBoneElement>([&](FRigBoneElement* BoneElement) -> bool
        {
			const FName ControlName = GetControlName(BoneElement->GetFName(), BoneElement->GetType());
			const FRigElementKey ControlKey(ControlName, ERigElementType::Control);
			const int32 ControlIndex = GetHierarchy()->GetIndex(ControlKey);
			if (ControlIndex != INDEX_NONE && GetControlActive(ControlIndex))
			{
				FRigControlElement* Control = GetHierarchy()->Get<FRigControlElement>(ControlIndex);
				const FTransform LocalTransform = GetHierarchy()->GetLocalTransform(ControlIndex);
				switch (ApplyMode)
				{
					case EControlRigFKRigExecuteMode::Replace:
					{
						const FTransform OffsetTransform = GetHierarchy()->GetControlOffsetTransform(Control, ERigTransformType::InitialLocal);
						FTransform Transform = LocalTransform * OffsetTransform;
						Transform.NormalizeRotation();
						GetHierarchy()->SetTransform(BoneElement, Transform, ERigTransformType::CurrentLocal, true, false);
						break;
					}
					case EControlRigFKRigExecuteMode::Additive:
					{
						const FTransform PreviousTransform = GetHierarchy()->GetTransform(BoneElement, ERigTransformType::CurrentLocal);
						FTransform Transform = LocalTransform * PreviousTransform;
						Transform.NormalizeRotation();
						GetHierarchy()->SetTransform(BoneElement, Transform, ERigTransformType::CurrentLocal, true, false);
						break;
					}
					case EControlRigFKRigExecuteMode::Direct:
					{
						GetHierarchy()->SetTransform(BoneElement, LocalTransform, ERigTransformType::CurrentLocal, true, false);
						break;
					}
				}
			}
			return true;
		});

		GetHierarchy()->ForEach<FRigCurveElement>([&](FRigCurveElement* CurveElement) -> bool
        {
			const FName ControlName = GetControlName(CurveElement->GetFName(), CurveElement->GetType());
			const FRigElementKey ControlKey(ControlName, ERigElementType::Control);
			const int32 ControlIndex = GetHierarchy()->GetIndex(ControlKey);
			
			if (ControlIndex != INDEX_NONE && GetControlActive(ControlIndex))
			{
				const float CurveValue = GetHierarchy()->GetControlValue(ControlIndex).Get<float>();

				switch (ApplyMode)
				{
					case EControlRigFKRigExecuteMode::Replace:
					case EControlRigFKRigExecuteMode::Direct:
					{
						GetHierarchy()->SetCurveValue(CurveElement, CurveValue, false /*bSetupUndo*/);
						break;
					}
					case EControlRigFKRigExecuteMode::Additive:
					{
						const float PreviousValue = GetHierarchy()->GetCurveValue(CurveElement);
						GetHierarchy()->SetCurveValue(CurveElement, PreviousValue + CurveValue, false /*bSetupUndo*/);
						break;
					}
				}
			}
			return true;
		});
	}
	else if (InEventName == FRigUnit_InverseExecution::EventName)
	{
		const bool bNotify = true;
		const FRigControlModifiedContext Context = FRigControlModifiedContext();
		const bool bSetupUndo = false;

		GetHierarchy()->Traverse([&](FRigBaseElement* InElement, bool& bContinue)
		{
			if(!InElement->IsA<FRigBoneElement>())
			{
				bContinue = false;
				return;
			}

			FRigBoneElement* BoneElement = CastChecked<FRigBoneElement>(InElement);

			const FName ControlName = GetControlName(BoneElement->GetFName(), BoneElement->GetType());
			const FRigElementKey ControlKey(ControlName, ERigElementType::Control);
			const int32 ControlIndex = GetHierarchy()->GetIndex(ControlKey);
			
			switch (ApplyMode)
			{
				case EControlRigFKRigExecuteMode::Replace:
				case EControlRigFKRigExecuteMode::Additive:
				{
					// during inversion we assume Replace Mode
					FRigControlElement* Control = GetHierarchy()->GetChecked<FRigControlElement>(ControlIndex);
					const FTransform Offset = GetHierarchy()->GetControlOffsetTransform(Control, ERigTransformType::InitialLocal);
					const FTransform Current = GetHierarchy()->GetTransform(BoneElement, ERigTransformType::CurrentLocal);
			
					FTransform Transform = Current.GetRelativeTransform(Offset);
					Transform.NormalizeRotation();

					SetControlValue(ControlName, FRigControlValue::Make(FEulerTransform(Transform)), bNotify, Context, bSetupUndo, false, true);
					break;
				}

				case EControlRigFKRigExecuteMode::Direct:
				{
					FTransform Transform = GetHierarchy()->GetTransform(BoneElement, ERigTransformType::CurrentLocal);
					Transform.NormalizeRotation();
					SetControlValue(ControlName, FRigControlValue::Make(FEulerTransform(Transform)), bNotify, Context, bSetupUndo, false, true);

					break;
				}
			}

		}, true);

		GetHierarchy()->ForEach<FRigCurveElement>([&](FRigCurveElement* CurveElement) -> bool
        {
			const FName ControlName = GetControlName(CurveElement->GetFName(), CurveElement->GetType());
			const FRigElementKey ControlKey(ControlName, ERigElementType::Control);
			const int32 ControlIndex = GetHierarchy()->GetIndex(ControlKey);

			const float CurveValue = GetHierarchy()->GetCurveValue(CurveElement);
			SetControlValue(ControlName, FRigControlValue::Make(CurveValue), bNotify, Context, bSetupUndo);

			return true;
		});
	}

#if WITH_EDITOR
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		if(Hierarchy->IsTracingChanges())
		{
			Hierarchy->StorePoseForTrace(TEXT("UFKControlRig::EndExecuteUnits"));
			Hierarchy->DumpTransformStackToFile();
		}
	}
#endif
	
	return true;
}

void UFKControlRig::SetBoneInitialTransformsFromSkeletalMeshComponent(USkeletalMeshComponent* InSkelMeshComp,
                                                                      bool bUseAnimInstance)
{
	Super::SetBoneInitialTransformsFromSkeletalMeshComponent(InSkelMeshComp, bUseAnimInstance);

#if WITH_EDITOR
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		if(Hierarchy->IsTracingChanges())
		{
			Hierarchy->StorePoseForTrace(TEXT("UFKControlRig::SetBoneInitialTransformsFromSkeletalMeshComponent"));
		}
	}
#endif	

	SetControlOffsetsFromBoneInitials();
}

void UFKControlRig::Initialize(bool bInitRigUnits /*= true*/)
{
	PostInitInstanceIfRequired();

	Super::Initialize(bInitRigUnits);

#if WITH_EDITOR
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		Hierarchy->bRecordInstructionsAtRuntime = false;
	}
#endif // WITH_EDITOR

	if(const TSharedPtr<IControlRigObjectBinding> Binding = GetObjectBinding())
	{
		// we do this after Initialize because Initialize will copy from CDO. 
		// create hierarchy from the incoming skeleton
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Binding->GetBoundObject()))
		{
			CreateRigElements(SkeletalMeshComponent->GetSkeletalMeshAsset());
		}
		else if (USkeleton* Skeleton = Cast<USkeleton>(Binding->GetBoundObject()))
		{
			const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
			CreateRigElements(RefSkeleton, Skeleton);
		}
	}
}

TArray<FName> UFKControlRig::GetControlNames()
{
	TArray<FRigControlElement*> Controls;
	GetControlsInOrder(Controls);

	TArray<FName> Names;
	for (FRigControlElement* ControlElement : Controls) 
	{
		Names.Add(ControlElement->GetFName());
	}
	return Names;
}

bool UFKControlRig::GetControlActive(int32 Index) const
{
	if (Index >= 0 && Index < IsControlActive.Num())
	{
		return IsControlActive[Index];
	}
	return false;
}

void UFKControlRig::SetControlActive(int32 Index, bool bActive)
{
	if (Index >= 0 && Index < IsControlActive.Num())
	{
		IsControlActive[Index] = bActive;
	}
}

void UFKControlRig::SetControlActive(const TArray<FFKBoneCheckInfo>& BoneChecks)
{
	for (const FFKBoneCheckInfo& Info : BoneChecks)
	{
		SetControlActive(Info.BoneID, Info.bActive);
	}
}

void UFKControlRig::CreateRigElements(const FReferenceSkeleton& InReferenceSkeleton, const USkeleton* InSkeleton)
{
	PostInitInstanceIfRequired();

	GetHierarchy()->Reset();
	if (URigHierarchyController* Controller = GetHierarchy()->GetController(true))
	{
		// determine the max element name length
		int32 MaxElementName = GetHierarchy()->GetMaxNameLength();
		for (int32 Index = 0; Index < InitializationOptions.CurveNames.Num(); ++Index)
		{
			MaxElementName = FMath::Max<int32>(MaxElementName, InitializationOptions.CurveNames[Index].GetStringLength() + 14 ); // _CURVE_CONTROL suffix  
		}
		if (InSkeleton)
		{
			InSkeleton->ForEachCurveMetaData([&MaxElementName](const FName& InCurveName, const FCurveMetaData&)
			{
				MaxElementName = FMath::Max<int32>(MaxElementName, InCurveName.GetStringLength() + 14 ); // _CURVE_CONTROL suffix  
			});
		}

		const TArray<FMeshBoneInfo> BoneInfos = InReferenceSkeleton.GetRefBoneInfo();
		for (const FMeshBoneInfo& BoneInfo : BoneInfos)
		{
			MaxElementName = FMath::Max<int32>(MaxElementName, BoneInfo.Name.GetStringLength() + 8 ); // _CONTROL suffix  
		}
		GetHierarchy()->SetMaxNameLength(MaxElementName);
		
		Controller->ImportBones(InReferenceSkeleton, NAME_None, false, false, true, false);

		if (InitializationOptions.bGenerateCurveControls)
		{
			// try to add curves for manually specified names
			if (InitializationOptions.CurveNames.Num() > 0)
            {
            	for (int32 Index = 0; Index < InitializationOptions.CurveNames.Num(); ++Index)
            	{
            		Controller->AddCurve(InitializationOptions.CurveNames[Index], 0.f, false);
            	}
            }
			// add all curves found on the skeleton
            else if (InSkeleton && InitializationOptions.bImportCurves)
            {
            	InSkeleton->ForEachCurveMetaData([Controller](const FName& InCurveName, const FCurveMetaData& InMetaData)
				{
					Controller->AddCurve(InCurveName, 0.f, false);
				});
            }
		}

		// add control for all bone hierarchy 
		if (InitializationOptions.bGenerateBoneControls)
		{
			auto CreateControlForBoneElement = [this, Controller](const FRigBoneElement* BoneElement)
			{
				const FName BoneName = BoneElement->GetFName();
				const FName ControlName = GetControlName(BoneName, BoneElement->GetType());
				const FRigElementKey ParentKey = GetHierarchy()->GetFirstParent(BoneElement->GetKey());

				FRigControlSettings Settings;
				Settings.ControlType = ERigControlType::EulerTransform;
				Settings.DisplayName = BoneName;
					
				Controller->AddControl(ControlName, ParentKey, Settings, FRigControlValue::Make(FEulerTransform::Identity), FTransform::Identity, FTransform::Identity, false);
			};
			
			if (InitializationOptions.BoneNames.Num())
			{
				for (const FName& BoneName : InitializationOptions.BoneNames)
				{
					FRigElementKey BoneElementKey(BoneName, ERigElementType::Bone);
					if (const FRigBoneElement* BoneElement = GetHierarchy()->Find<FRigBoneElement>(BoneElementKey))
					{
						CreateControlForBoneElement(BoneElement);
					}
				}
			}
			else
			{
				GetHierarchy()->ForEach<FRigBoneElement>([&](const FRigBoneElement* BoneElement) -> bool
				{
					CreateControlForBoneElement(BoneElement);
					return true;
				});
			}

			SetControlOffsetsFromBoneInitials();
		}
		
		if (InitializationOptions.bGenerateCurveControls)
		{
			GetHierarchy()->ForEach<FRigCurveElement>([&](const FRigCurveElement* CurveElement) -> bool
			{
				const FName ControlName = GetControlName(CurveElement->GetFName(), CurveElement->GetType());

				FRigControlSettings Settings;
				Settings.ControlType = ERigControlType::Float;
				Settings.DisplayName = CurveElement->GetFName();

				const FName DisplayCurveControlName(*(CurveElement->GetFName().ToString() + TEXT(" Curve")));
				Settings.DisplayName = DisplayCurveControlName;
				Controller->AddControl(ControlName, FRigElementKey(), Settings, FRigControlValue::Make(CurveElement->Get()), FTransform::Identity, FTransform::Identity, false);
				
				return true;
			});
		}

		RefreshActiveControls();
	}
}

void UFKControlRig::SetControlOffsetsFromBoneInitials()
{
	GetHierarchy()->Traverse([&](FRigBaseElement* InElement, bool& bContinue)
	{
		if(!InElement->IsA<FRigBoneElement>())
		{
			bContinue = false;
			return;
		}

		const FRigBoneElement* BoneElement = CastChecked<FRigBoneElement>(InElement);
		const FName BoneName = BoneElement->GetFName();
		const FName ControlName = GetControlName(BoneName, BoneElement->GetType());

		FRigControlElement* ControlElement = GetHierarchy()->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control));
		if(ControlElement == nullptr)
		{
			return;
		}
			
		const FRigElementKey ParentKey = GetHierarchy()->GetFirstParent(BoneElement->GetKey());

		FTransform OffsetTransform;
		if (ParentKey.IsValid())
		{
			FTransform GlobalTransform = GetHierarchy()->GetGlobalTransformByIndex(BoneElement->GetIndex(), true);
			FTransform ParentTransform = GetHierarchy()->GetGlobalTransform(ParentKey, true);
			OffsetTransform = GlobalTransform.GetRelativeTransform(ParentTransform);
		}
		else
		{
			OffsetTransform = GetHierarchy()->GetLocalTransformByIndex(BoneElement->GetIndex(), true);
		}

		OffsetTransform.NormalizeRotation();

		GetHierarchy()->SetControlOffsetTransform(ControlElement, OffsetTransform, ERigTransformType::InitialLocal, false, false, true);

	}, true);
}

void UFKControlRig::RefreshActiveControls()
{
	if (const URigHierarchy* Hierarchy = GetHierarchy())
	{
		const int32 NumControls = Hierarchy->Num();
		if (IsControlActive.Num() != NumControls)
		{
			IsControlActive.Empty(NumControls);
			IsControlActive.SetNum(NumControls);
			for (bool& bIsActive : IsControlActive)
			{
				bIsActive = true;
			}
		}
	}
}

void UFKControlRig::CreateRigElements(const USkeletalMesh* InReferenceMesh)
{
	if (InReferenceMesh)
	{
		const USkeleton* Skeleton = InReferenceMesh->GetSkeleton();
		CreateRigElements(InReferenceMesh->GetRefSkeleton(), Skeleton);
	}
}

void UFKControlRig::SetApplyMode(EControlRigFKRigExecuteMode InMode)
{
	if (ApplyMode == InMode)
	{
		return;
	}
	
	ApplyMode = InMode;

	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		if (ApplyMode == EControlRigFKRigExecuteMode::Additive)
		{
			FRigControlModifiedContext Context;
			Context.SetKey = EControlRigSetKey::Never;
			const bool bSetupUndo = false;

			Hierarchy->ForEach<FRigControlElement>([&](FRigControlElement* ControlElement) -> bool
			{
				if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					SetControlValue<FEulerTransform>(ControlElement->GetFName(), FEulerTransform::Identity, true, Context, bSetupUndo);
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::Float)
				{
					SetControlValue<float>(ControlElement->GetFName(), 0.f, true, Context, bSetupUndo);
				}

				return true;
			});
		}
		else
		{
			FRigControlModifiedContext Context;
			Context.SetKey = EControlRigSetKey::Never;
			const bool bSetupUndo = false;

			Hierarchy->ForEach<FRigControlElement>([&](FRigControlElement* ControlElement) -> bool
			{
				if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					const FRigControlValue::FEulerTransform_Float InitValue =
						GetHierarchy()->GetControlValue(ControlElement, ERigControlValueType::Initial).Get<FRigControlValue::FEulerTransform_Float>();
					SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlElement->GetFName(), InitValue, true, Context, bSetupUndo);
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::Float)
				{
					const float InitValue = GetHierarchy()->GetControlValue(ControlElement, ERigControlValueType::Initial).Get<float>();
					SetControlValue<float>(ControlElement->GetFName(), InitValue, true, Context, bSetupUndo);
				}

				return true;
			});
		}
	}
}

void UFKControlRig::ToggleApplyMode()
{
	EControlRigFKRigExecuteMode ModeToApply = ApplyMode;
	if (ApplyMode == EControlRigFKRigExecuteMode::Additive)
	{
		ModeToApply = CachedToggleApplyMode;
	}
	else
	{
		CachedToggleApplyMode = ApplyMode;
		ModeToApply = EControlRigFKRigExecuteMode::Additive;
	}
	SetApplyMode(ModeToApply);
}

#undef LOCTEXT_NAMESPACE



