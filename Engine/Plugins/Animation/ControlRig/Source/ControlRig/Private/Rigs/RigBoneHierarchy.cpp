// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigBoneHierarchy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigBoneHierarchy)

////////////////////////////////////////////////////////////////////////////////
// FRigBoneHierarchy
////////////////////////////////////////////////////////////////////////////////

FRigBoneHierarchy::FRigBoneHierarchy()
{
}

FRigBone& FRigBoneHierarchy::Add(const FName& InNewName, const FName& InParentName, ERigBoneType InType, const FTransform& InInitTransform, const FTransform& InLocalTransform, const FTransform& InGlobalTransform)
{
	FRigBone NewBone;
	NewBone.Name = InNewName;
	NewBone.ParentIndex = INDEX_NONE; // we no longer support parent index lookup
	NewBone.ParentName = InParentName;
	NewBone.InitialTransform = InInitTransform;
	NewBone.LocalTransform = InLocalTransform;
	NewBone.GlobalTransform = InGlobalTransform;
	NewBone.Type = InType;

	const int32 Index = Bones.Add(NewBone);
	return Bones[Index];
}
