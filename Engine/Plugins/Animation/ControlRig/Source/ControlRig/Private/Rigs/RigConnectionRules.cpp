// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigConnectionRules.h"
#include "Rigs/RigHierarchyElements.h"
#include "Rigs/RigHierarchy.h"
#include "ModularRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigConnectionRules)

////////////////////////////////////////////////////////////////////////////////
// FRigConnectionRuleStash
////////////////////////////////////////////////////////////////////////////////

FRigConnectionRuleStash::FRigConnectionRuleStash()
{
}

FRigConnectionRuleStash::FRigConnectionRuleStash(const FRigConnectionRule* InRule)
{
	check(InRule);
	ScriptStructPath = InRule->GetScriptStruct()->GetPathName();
	InRule->GetScriptStruct()->ExportText(ExportedText, InRule, InRule, nullptr, PPF_None, nullptr);
}

void FRigConnectionRuleStash::Save(FArchive& Ar)
{
	Ar << ScriptStructPath;
	Ar << ExportedText;
}

void FRigConnectionRuleStash::Load(FArchive& Ar)
{
	Ar << ScriptStructPath;
	Ar << ExportedText;
}

bool FRigConnectionRuleStash::IsValid() const
{
	return !ScriptStructPath.IsEmpty() && !ExportedText.IsEmpty();
}

UScriptStruct* FRigConnectionRuleStash::GetScriptStruct() const
{
	if(!ScriptStructPath.IsEmpty())
	{
		return FindObject<UScriptStruct>(nullptr, *ScriptStructPath);
	}
	return nullptr;
}

TSharedPtr<FStructOnScope> FRigConnectionRuleStash::Get() const
{
	class FErrorPipe : public FOutputDevice
	{
	public:

		int32 NumErrors;

		FErrorPipe()
			: FOutputDevice()
			, NumErrors(0)
		{
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			NumErrors++;
		}
	};

	UScriptStruct* ScriptStruct = GetScriptStruct();
	check(ScriptStruct);
	TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(ScriptStruct));
	FErrorPipe ErrorPipe;
	ScriptStruct->ImportText(*ExportedText, StructOnScope->GetStructMemory(), nullptr, PPF_None, &ErrorPipe, ScriptStruct->GetName());
	return StructOnScope;
}

const FRigConnectionRule* FRigConnectionRuleStash::Get(TSharedPtr<FStructOnScope>& InOutStorage) const
{
	InOutStorage = Get();
	if(InOutStorage.IsValid() && InOutStorage->IsValid())
	{
		check(InOutStorage->GetStruct()->IsChildOf(FRigConnectionRule::StaticStruct()));
		return reinterpret_cast<const FRigConnectionRule*>(InOutStorage->GetStructMemory());
	}
	return nullptr;
}

bool FRigConnectionRuleStash::operator==(const FRigConnectionRuleStash& InOther) const
{
	return ScriptStructPath.Equals(InOther.ScriptStructPath, ESearchCase::CaseSensitive) &&
		ExportedText.Equals(InOther.ExportedText, ESearchCase::CaseSensitive);
}

uint32 GetTypeHash(const FRigConnectionRuleStash& InRuleStash)
{
	return HashCombine(GetTypeHash(InRuleStash.ScriptStructPath), GetTypeHash(InRuleStash.ExportedText));
}

////////////////////////////////////////////////////////////////////////////////
// FRigConnectionRuleInput
////////////////////////////////////////////////////////////////////////////////

const FRigConnectorElement* FRigConnectionRuleInput::FindPrimaryConnector(FText* OutErrorMessage) const
{
	check(Hierarchy);

	const bool bIsModularRig = Hierarchy->GetTypedOuter<UModularRig>() != nullptr;
	if (bIsModularRig)
	{
		check(Module);
	}
	const FRigConnectorElement* PrimaryConnector = nullptr;
	Hierarchy->ForEach<FRigConnectorElement>(
		[this, &PrimaryConnector, bIsModularRig](FRigConnectorElement* Connector) -> bool
		{
			if (Connector->IsPrimary())
			{
				const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connector->GetKey());

				if (!bIsModularRig ||
					Module->Name == ConnectorModuleName)
				{
					PrimaryConnector = Connector;
					return false; // stop the search
				}
			}
			return true; // continue the search
		}
	);

	if (PrimaryConnector == nullptr)
	{
		if (OutErrorMessage)
		{
			if (Module)
			{
				static constexpr TCHAR Format[] = TEXT("No primary connector found for module '%s'.");
				*OutErrorMessage = FText::FromString(FString::Printf(Format, *Module->Name.ToString()));
			}
			else
			{
				static constexpr TCHAR Format[] = TEXT("No primary connector found in Control Rig Module.");
			}
		}
	}

	return PrimaryConnector;
}

TArray<const FRigConnectorElement*> FRigConnectionRuleInput::FindSecondaryConnectors(bool bOptional, FText* OutErrorMessage) const
{
	check(Hierarchy);

	const bool bIsModularRig = Hierarchy->GetTypedOuter<UModularRig>() != nullptr;
	if (bIsModularRig)
	{
		check(Module);
	}

	TArray<const FRigConnectorElement*> SecondaryConnectors;
	Hierarchy->ForEach<FRigConnectorElement>(
		[this, bOptional, bIsModularRig, &SecondaryConnectors](const FRigConnectorElement* Connector) -> bool
		{
			if (Connector->IsSecondary() && Connector->IsOptional() == bOptional)
			{
				const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connector->GetKey());

				if (!bIsModularRig ||
					Module->Name == ConnectorModuleName)
				{
					SecondaryConnectors.Add(Connector);
				}
			}
			return true; // continue the search
		}
	);

	return SecondaryConnectors;
}

const FRigTransformElement* FRigConnectionRuleInput::ResolveConnector(const FRigConnectorElement* InConnector, FText* OutErrorMessage) const
{
	check(Redirector);

	if(const FRigElementKeyRedirector::FCachedKeyArray* Cache = Redirector->Find(InConnector->GetKey()))
	{
		if(!Cache->IsEmpty())
		{
			const FCachedRigElement& CachedElement = Cache->operator[](0);
			if(const_cast<FCachedRigElement&>(CachedElement).UpdateCache(GetHierarchy()))
			{
				return Cast<FRigTransformElement>(CachedElement.GetElement());
			}
		}
	}

	if(OutErrorMessage)
	{
		static constexpr TCHAR Format[] = TEXT("Resolved target not found for connector '%s'.");
		*OutErrorMessage = FText::FromString(FString::Printf(Format, *InConnector->GetName()));
	}
	return nullptr;
}

const FRigTransformElement* FRigConnectionRuleInput::ResolvePrimaryConnector(FText* OutErrorMessage) const
{
	if(const FRigConnectorElement* Connector = FindPrimaryConnector(OutErrorMessage))
	{
		return ResolveConnector(Connector, OutErrorMessage);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
// FRigConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetInvalidTarget(FText());
	return Result;
}

////////////////////////////////////////////////////////////////////////////////
// FRigAndConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigAndConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetPossibleTarget();
	
	TSharedPtr<FStructOnScope> Storage;
	for(const FRigConnectionRuleStash& ChildRule : ChildRules)
	{
		if (!ChildRule.IsValid())
		{
			continue;
		}
		if(const FRigConnectionRule* Rule = ChildRule.Get(Storage))
		{
			Result = Rule->Resolve(InTarget, InRuleInput);
			if(!Result.IsValid())
			{
				return Result;
			}
		}
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////
// FRigOrConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigOrConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetPossibleTarget();
	
	TSharedPtr<FStructOnScope> Storage;
	for(const FRigConnectionRuleStash& ChildRule : ChildRules)
	{
		if (!ChildRule.IsValid())
		{
			continue;
		}
		if(const FRigConnectionRule* Rule = ChildRule.Get(Storage))
		{
			Result = Rule->Resolve(InTarget, InRuleInput);
			if(Result.IsValid())
			{
				return Result;
			}
		}
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////
// FRigTypeConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigTypeConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetPossibleTarget();

	if(!InTarget->GetKey().IsTypeOf(ElementType))
	{
		const FString ExpectedType = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)ElementType).ToString();
		static constexpr TCHAR Format[] = TEXT("Element '%s' is not of the expected type (%s).");
		Result.SetInvalidTarget(FText::FromString(FString::Printf(Format, *InTarget->GetKey().ToString(), *ExpectedType)));
	}
	
	return Result;
}

////////////////////////////////////////////////////////////////////////////////
// FRigTagConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigTagConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetPossibleTarget();

	if(!InRuleInput.GetHierarchy()->HasTag(InTarget->GetKey(), Tag))
	{
		static constexpr TCHAR Format[] = TEXT("Element '%s' does not contain tag '%s'.");
		Result.SetInvalidTarget(FText::FromString(FString::Printf(Format, *InTarget->GetKey().ToString(), *Tag.ToString())));
	}
	
	return Result;
}

////////////////////////////////////////////////////////////////////////////////
// FRigChildOfPrimaryConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigChildOfPrimaryConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetPossibleTarget();

	// find the primary resolved target
	FText ErrorMessage;
	const FRigTransformElement* PrimaryTarget = InRuleInput.ResolvePrimaryConnector(&ErrorMessage);
	if(PrimaryTarget == nullptr)
	{
		Result.SetInvalidTarget(ErrorMessage);
		return Result;
	}

	static constexpr TCHAR IsNotAChildOfFormat[] = TEXT("Target '%s' is not a child of the primary.");
	static constexpr TCHAR IsAlreadyUsedForPrimaryFormat[] = TEXT("Target '%s' is already used for the primary.");
	if(InTarget == PrimaryTarget)
	{
		Result.SetInvalidTarget(FText::FromString(FString::Printf(IsAlreadyUsedForPrimaryFormat, *InTarget->GetKey().ToString())));
		return Result;
	}

	// for sockets we use the parent for resolve
	if(PrimaryTarget->GetType() == ERigElementType::Socket)
	{
		if(const FRigTransformElement* FirstParent =
			Cast<FRigTransformElement>(InRuleInput.GetHierarchy()->GetFirstParent(PrimaryTarget)))
		{
			PrimaryTarget = FirstParent;

			if(InTarget == PrimaryTarget)
			{
				Result.SetInvalidTarget(FText::FromString(FString::Printf(IsNotAChildOfFormat, *InTarget->GetKey().ToString())));
				return Result;
			}
		}
	}

	static const FEmptyRigDependenciesProvider EmptyRigDependenciesProvider;
	if(!InRuleInput.GetHierarchy()->IsParentedTo(
		const_cast<FRigBaseElement*>(InTarget), const_cast<FRigTransformElement*>(PrimaryTarget), EmptyRigDependenciesProvider))
	{
		Result.SetInvalidTarget(FText::FromString(FString::Printf(IsNotAChildOfFormat, *InTarget->GetKey().ToString())));
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////
// FRigArraySizeConnectionRule
////////////////////////////////////////////////////////////////////////////////

FRigElementResolveResult FRigArraySizeConnectionRule::Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const
{
	// The Array Size Rule does not restrict targets
	FRigElementResolveResult Result(InTarget->GetKey());
	Result.SetPossibleTarget();

	return Result;
}

TOptional<int32> FRigArraySizeConnectionRule::GetMinNumConnections() const
{
	if (bMinEnabled)
	{
		return MinNumConnections;
	}

	return TOptional<int32>{};
}

TOptional<int32> FRigArraySizeConnectionRule::GetMaxNumConnections() const
{
	if (bMaxEnabled)
	{
		return MaxNumConnections;
	}

	return TOptional<int32>{};
}
