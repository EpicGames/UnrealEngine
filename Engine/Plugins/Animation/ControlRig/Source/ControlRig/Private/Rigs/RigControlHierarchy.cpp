// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigControlHierarchy.h"
#include "Rigs/RigHierarchyElements.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigControlHierarchy)

////////////////////////////////////////////////////////////////////////////////
// FRigControl
////////////////////////////////////////////////////////////////////////////////

void FRigControl::ApplyLimits(FRigControlValue& InOutValue) const
{
	FRigControlSettings Settings;
	Settings.ControlType = ControlType;
	Settings.SetupLimitArrayForType(bLimitTranslation, bLimitRotation, bLimitScale);
	InOutValue.ApplyLimits(Settings.LimitEnabled, ControlType, MinimumValue, MaximumValue);
}

FTransform FRigControl::GetTransformFromValue(ERigControlValueType InValueType) const
{
	return GetValue(InValueType).GetAsTransform(ControlType, PrimaryAxis);
}

void FRigControl::SetValueFromTransform(const FTransform& InTransform, ERigControlValueType InValueType)
{
	GetValue(InValueType).SetFromTransform(InTransform, ControlType, PrimaryAxis);

	if (InValueType == ERigControlValueType::Current)
	{
		ApplyLimits(GetValue(InValueType));
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigControlHierarchy
////////////////////////////////////////////////////////////////////////////////

FRigControlHierarchy::FRigControlHierarchy()
{
}

FRigControl& FRigControlHierarchy::Add(
    const FName& InNewName,
    ERigControlType InControlType,
    const FName& InParentName,
    const FName& InSpaceName,
    const FTransform& InOffsetTransform,
    const FRigControlValue& InValue,
    const FName& InGizmoName,
    const FTransform& InGizmoTransform,
    const FLinearColor& InGizmoColor
)
{
	FRigControl NewControl;
	NewControl.Name = InNewName;
	NewControl.ControlType = InControlType;
	NewControl.ParentIndex = INDEX_NONE; // we don't support indices
	NewControl.ParentName = InParentName;
	NewControl.SpaceIndex = INDEX_NONE;
	NewControl.SpaceName = InSpaceName;
	NewControl.OffsetTransform = InOffsetTransform;
	NewControl.InitialValue = InValue;
	NewControl.Value = FRigControlValue();
	NewControl.GizmoName = InGizmoName;
	NewControl.GizmoTransform = InGizmoTransform;
	NewControl.GizmoColor = InGizmoColor;

	if (!NewControl.InitialValue.IsValid())
	{
		NewControl.SetValueFromTransform(FTransform::Identity, ERigControlValueType::Initial);
	}

	const int32 Index = Controls.Add(NewControl);
	return Controls[Index];
}

void FRigControlHierarchy::PostLoad()
{
#if WITH_EDITORONLY_DATA
	for (FRigControl& Control : Controls)
	{
		for (int32 ValueType = 0; ValueType <= (int32)ERigControlValueType::Maximum; ValueType++)
		{
			FRigControlValue& Value = Control.GetValue((ERigControlValueType)ValueType);
			if (!Value.IsValid())
			{
				Value.GetRef<FRigControlValue::FTransform_Float>() = Value.Storage_DEPRECATED;
			}
		}
	}
#endif
}
