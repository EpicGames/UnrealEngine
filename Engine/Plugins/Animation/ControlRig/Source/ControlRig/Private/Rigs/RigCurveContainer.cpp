// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigCurveContainer.h"
#include "Rigs/RigHierarchyContainer.h"
#include "HelperUtil.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigCurveContainer)

////////////////////////////////////////////////////////////////////////////////
// FRigCurveContainer
////////////////////////////////////////////////////////////////////////////////

FRigCurveContainer::FRigCurveContainer()
{
}

FRigCurve& FRigCurveContainer::Add(const FName& InNewName, float InValue)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

    FRigCurve NewCurve;
	NewCurve.Name = InNewName;
	NewCurve.Value = InValue;
	FName NewCurveName = NewCurve.Name;
	const int32 Index = Curves.Add(NewCurve);
	return Curves[Index];
}

