// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigDependency.h"
#include "ModularRig.h"
#include "Units/Execution/RigUnit_BeginExecution.h"

const TRigHierarchyDependencyMap& FEmptyRigDependenciesProvider::GetRigHierarchyDependencies() const
{
	static const TRigHierarchyDependencyMap Empty;
	return Empty;
}

const TRigHierarchyDependencyMap& FEmptyRigDependenciesProvider::GetReverseRigHierarchyDependencies() const
{
	static const TRigHierarchyDependencyMap Empty;
	return Empty;
}

FRigDependenciesProviderForVM::FRigDependenciesProviderForVM(UControlRig* InControlRig, const FName& InEventName, bool InFollowVariables)
	: WeakControlRig(InControlRig)
	, EventName(InEventName)
	, bFollowVariables(InFollowVariables)
{
}

// UE_DISABLE_OPTIMIZATION

const TRigHierarchyDependencyMap& FRigDependenciesProviderForVM::GetRigHierarchyDependencies() const
{
	static const TRigHierarchyDependencyMap Empty;

#if WITH_EDITOR
	UControlRig* ControlRig = GetControlRig();
	if (!ControlRig)
	{
		return Empty;
	}

	const URigVM* VM = ControlRig->GetVM();
	if (!VM)
	{
		return Empty;
	}

	const FRigVMExtendedExecuteContext& ExtendedExecuteContext = ControlRig->GetRigVMExtendedExecuteContext();
	const FControlRigExecuteContext& Context = ExtendedExecuteContext.GetPublicData<FControlRigExecuteContext>();
	const uint32 CurrentRecordsHash = Context.GetInstructionRecordsHash(); 
	if (!CachedDependencies.IsEmpty() && RecordsHash == CurrentRecordsHash)
	{
		return CachedDependencies;
	}
	
	CachedDependencies.Reset();

	ComputeDependencies(Context, VM, NAME_None, CachedDependencies);

	RecordsHash = CurrentRecordsHash;
	ReverseRecordsHash = UINT32_MAX;
	
#endif

	return CachedDependencies;
}

const TRigHierarchyDependencyMap& FRigDependenciesProviderForVM::GetReverseRigHierarchyDependencies() const
{
	const TRigHierarchyDependencyMap& Dependencies = GetRigHierarchyDependencies();
	if (ReverseRecordsHash == RecordsHash)
	{
		return CachedReverseDependencies;
	}

	CachedReverseDependencies.Reset();
	CachedReverseDependencies.Reserve(Dependencies.Num());
	for (const TPair<FRigHierarchyRecord, TSet<FRigHierarchyRecord>>& Pair : Dependencies)
	{
		for (const FRigHierarchyRecord& DependentRecord : Pair.Value)
		{
			CachedReverseDependencies.FindOrAdd(DependentRecord).Add(Pair.Key);
		}
	}

	ReverseRecordsHash = RecordsHash;
	return CachedReverseDependencies;
}

void FRigDependenciesProviderForVM::InvalidateCache()
{
	CachedDependencies.Reset();
	CachedReverseDependencies.Reset();
	RecordsHash = ReverseRecordsHash = UINT32_MAX;
}

UControlRig* FRigDependenciesProviderForVM::GetControlRig() const
{
	if (WeakControlRig.IsValid())
	{
		return WeakControlRig.Get();
	}
	return nullptr;
}

void FRigDependenciesProviderForVM::ComputeDependencies(const FControlRigExecuteContext& InContext, const URigVM* InVM, const FName& InVMName, TRigHierarchyDependencyMap& OutDependencies) const
{
#if WITH_EDITOR
	
	FName EventNameToUse = EventName;
	if(EventNameToUse.IsNone())
	{
		EventNameToUse = FRigUnit_BeginExecution().GetEventName();
	}

	// if the VM doesn't implement the given event
	if(!InVM->ContainsEntry(EventNameToUse))
	{
		return;
	}

	const FRigVMInstructionArray Instructions = InVM->GetByteCode().GetInstructions();
	const FInstructionRecordContainer& InstructionRecords = InContext.GetInstructionRecords();
	TArray<TArray<FInstructionRecord>> ReadRecordsPerInstruction, WrittenRecordsPerInstruction;
	
	// Find the max instruction index
	{
		int32 MaxInstructionIndex = 0;
		for(int32 RecordType = 0; RecordType < 2; RecordType++)
		{
			const TArray<FInstructionRecord>& Records = RecordType == 0 ? InstructionRecords.ReadRecords : InstructionRecords.WrittenRecords; 
			for(int32 RecordIndex = 0; RecordIndex < Records.Num(); RecordIndex++)
			{
				const FInstructionRecord& Record = Records[RecordIndex];
				if (Record.VM != InVMName)
				{
					continue;
				}
				MaxInstructionIndex = FMath::Max(MaxInstructionIndex, Record.InstructionIndex);
			}
		}
		ReadRecordsPerInstruction.AddZeroed(MaxInstructionIndex+1);
		WrittenRecordsPerInstruction.AddZeroed(MaxInstructionIndex+1);
	}

	// fill lookup tables per instruction / element
	for(int32 RecordType = 0; RecordType < 2; RecordType++)
	{
		const TArray<FInstructionRecord>& Records = RecordType == 0 ? InstructionRecords.ReadRecords : InstructionRecords.WrittenRecords; 
		TArray<TArray<FInstructionRecord>>& PerInstruction = RecordType == 0 ? ReadRecordsPerInstruction : WrittenRecordsPerInstruction;

		for(int32 RecordIndex = 0; RecordIndex < Records.Num(); RecordIndex++)
		{
			const FInstructionRecord& Record = Records[RecordIndex];
			if (Record.VM != InVMName)
			{
				continue;
			}
			PerInstruction[Record.InstructionIndex].Add(Record);
		}
	}

	// for each written transform / metadata on an instruction
	TArray<FRigHierarchyRecord> WrittenVariables;
	TMap<FRigVMOperand, TArray<int32>> InputOperandToInstructions;
	TMap<FRigVMOperand, TArray<int32>> OutputOperandToInstructions;

	// use a combination of a set and an array to ensure a faster uniqueness while being able to iterate over the last elements added
	TArray<TTuple<int32, FRigHierarchyRecord>> InstructionsToVisit;
	TMap<int32, TSet<FRigHierarchyRecord>> UniqueInstructionsToVisit;
	
	auto ResetLastInstruction = [&InstructionsToVisit, &UniqueInstructionsToVisit](const int32 InNewInstruction, const FRigHierarchyRecord& InNewRecord)
	{
		if (InstructionsToVisit.Num() == 1)
		{
			// avoid reallocation and set value directly
			InstructionsToVisit[0] = {InNewInstruction, InNewRecord};
		}
		else
		{
			InstructionsToVisit.Reset();
			InstructionsToVisit.Emplace(InNewInstruction, InNewRecord);
		}

		UniqueInstructionsToVisit.Reset();
	};

	auto FindOrAddInstruction = [&InstructionsToVisit, &UniqueInstructionsToVisit](TConstArrayView<int32> InInstructions, const FRigHierarchyRecord& InRecord)
	{
		if (InInstructions.IsEmpty())
		{
			return;
		}

		const uint32 NewRecordHash = GetTypeHash(InRecord);
		
		for (int32 NewInstruction : InInstructions)
		{
			if (InstructionsToVisit.Num() == 1)
			{
				const int32 LastInstruction = InstructionsToVisit[0].Get<0>();
				const FRigHierarchyRecord LastRecord = InstructionsToVisit[0].Get<1>();

				// check if the {Instruction, InRecord} is equal to the last instruction and continue if so, as there's nothing to do
				if (NewInstruction == LastInstruction && InRecord == LastRecord)
				{
					continue;
				}
				else
				{
					// otherwise, add the last instruction / last record hash to maintain uniqueness and proceed
					UniqueInstructionsToVisit.Reset();
					UniqueInstructionsToVisit.Emplace(LastInstruction).Emplace(LastRecord);
				}
			}

			TSet<FRigHierarchyRecord>& UniqueRecords = UniqueInstructionsToVisit.FindOrAdd(NewInstruction);
			if (UniqueRecords.IsEmpty())
			{
				UniqueRecords.AddByHash(NewRecordHash, InRecord);
				InstructionsToVisit.Emplace(NewInstruction, InRecord);
			}
			else
			{
				bool bAlreadyInSet = false;
				UniqueRecords.FindOrAddByHash(NewRecordHash, InRecord, &bAlreadyInSet);
				if (!bAlreadyInSet)
				{
					InstructionsToVisit.Emplace(NewInstruction, InRecord);
				}
			}
	   }
	};

	for(int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		FRigVMOperandArray InputOperands = InVM->GetByteCode().GetInputOperands(InstructionIndex);
		for (const FRigVMOperand& Operand : InputOperands)
		{
			if (!bFollowVariables && Operand.GetMemoryType() == ERigVMMemoryType::External)
			{
				continue;
			}
			const FRigVMOperand OperandNoRegisterOffset(Operand.GetMemoryType(), Operand.GetRegisterIndex());
			InputOperandToInstructions.FindOrAdd(OperandNoRegisterOffset).Add(InstructionIndex);
		}

		FRigVMOperandArray OutputOperands = InVM->GetByteCode().GetOutputOperands(InstructionIndex);
		for (const FRigVMOperand& Operand : OutputOperands)
		{
			if (!bFollowVariables && Operand.GetMemoryType() == ERigVMMemoryType::External)
			{
				continue;
			}
			const FRigVMOperand OperandNoRegisterOffset(Operand.GetMemoryType(), Operand.GetRegisterIndex());
			OutputOperandToInstructions.FindOrAdd(OperandNoRegisterOffset).Add(InstructionIndex);
		}
	}

	for(int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		WrittenVariables.Reset();

		if (bFollowVariables)
		{
			if (ReadRecordsPerInstruction.IsValidIndex(InstructionIndex))
			{
				const TArray<FInstructionRecord>& ReadRecords = ReadRecordsPerInstruction[InstructionIndex];
				for(int32 ReadRecordIndex = 0; ReadRecordIndex < ReadRecords.Num(); ReadRecordIndex++)
				{
					const FInstructionRecord& ReadRecord = ReadRecords[ReadRecordIndex];

					const FRigHierarchyRecord ReadElementRecord(
						ReadRecord.MetadataName.IsNone() ? FRigHierarchyRecord::EType_RigElement : FRigHierarchyRecord::EType_Metadata,
						ReadRecord.ElementIndex,
						ReadRecord.MetadataName);

					ResetLastInstruction(InstructionIndex, ReadElementRecord);

					// follow the output operands through the bytecode to determine variable writes of this instruction
					for(int32 InstructionToVisitIndex = 0; InstructionToVisitIndex < InstructionsToVisit.Num(); InstructionToVisitIndex++)
					{
						const int32 InstructionToVisit = InstructionsToVisit[InstructionToVisitIndex].Get<0>();

						const FRigVMOperandArray OutputOperands = InVM->GetByteCode().GetOutputOperands(InstructionToVisit);
						TSet<FRigVMOperand> OutputOperandsNoRegisterOffset;
						OutputOperandsNoRegisterOffset.Reserve(OutputOperands.Num());
						for(const FRigVMOperand& OutputOperand : OutputOperands)
						{
							OutputOperandsNoRegisterOffset.Emplace({OutputOperand.GetMemoryType(), OutputOperand.GetRegisterIndex()});
						}
						
						for(const FRigVMOperand& OutputOperandNoRegisterOffset : OutputOperandsNoRegisterOffset)
						{
							FRigHierarchyRecord LastRecord = InstructionsToVisit[InstructionToVisitIndex].Get<1>();

							if (OutputOperandNoRegisterOffset.GetMemoryType() == ERigVMMemoryType::External)
							{
								const FRigHierarchyRecord VariableRecord(FRigHierarchyRecord::EType_Variable, OutputOperandNoRegisterOffset.GetRegisterIndex(), ReadRecord.VM);
								WrittenVariables.AddUnique(VariableRecord);
								LastRecord = VariableRecord;
							}

							if(const TArray<int32>* InstructionsWithOutputOperand = InputOperandToInstructions.Find(OutputOperandNoRegisterOffset))
							{
								FindOrAddInstruction(*InstructionsWithOutputOperand, LastRecord);
							}
						}
					}

					for (const FRigHierarchyRecord& WrittenVariableRecord : WrittenVariables)
					{
						OutDependencies.FindOrAdd(WrittenVariableRecord).Add(ReadElementRecord);
					}
				}
			}
		}

		if (WrittenRecordsPerInstruction.IsValidIndex(InstructionIndex))
		{
			const TArray<FInstructionRecord>& WrittenRecords = WrittenRecordsPerInstruction[InstructionIndex];
			for(int32 WrittenRecordIndex = 0; WrittenRecordIndex < WrittenRecords.Num(); WrittenRecordIndex++)
			{
				const FInstructionRecord& WrittenRecord = WrittenRecords[WrittenRecordIndex];

				const FRigHierarchyRecord InstructionRecord(
					FRigHierarchyRecord::EType_Instruction,
					WrittenRecord.InstructionIndex,
					WrittenRecord.VM);

				const FRigHierarchyRecord TargetRecord(
					WrittenRecord.MetadataName.IsNone() ? FRigHierarchyRecord::EType_RigElement : FRigHierarchyRecord::EType_Metadata,
					WrittenRecord.ElementIndex,
					WrittenRecord.MetadataName);

				OutDependencies.FindOrAdd(TargetRecord).Add(InstructionRecord);

				ResetLastInstruction(InstructionIndex, InstructionRecord);

				// follow the input operands through the bytecode to determine inputs of this instruction
				for(int32 InstructionToVisitIndex = 0; InstructionToVisitIndex < InstructionsToVisit.Num(); InstructionToVisitIndex++)
				{
					const int32 InstructionToVisit = InstructionsToVisit[InstructionToVisitIndex].Get<0>();

					if (ReadRecordsPerInstruction.IsValidIndex(InstructionToVisit))
					{
						const TArray<FInstructionRecord>& ReadRecords = ReadRecordsPerInstruction[InstructionToVisit];
						for(int32 ReadRecordIndex = 0; ReadRecordIndex < ReadRecords.Num(); ReadRecordIndex++)
						{
							const FInstructionRecord& ReadRecord = ReadRecords[ReadRecordIndex];
							const FRigHierarchyRecord SourceRecord(
								ReadRecord.MetadataName.IsNone() ? FRigHierarchyRecord::EType_RigElement : FRigHierarchyRecord::EType_Metadata,
								ReadRecord.ElementIndex,
								ReadRecord.MetadataName);
							OutDependencies.FindOrAdd(InstructionsToVisit[InstructionToVisitIndex].Get<1>()).Add(SourceRecord);
						}
					}

					const FRigVMOperandArray InputOperands = InVM->GetByteCode().GetInputOperands(InstructionToVisit);
					for(const FRigVMOperand InputOperand : InputOperands)
					{
						FRigHierarchyRecord LastRecord = InstructionsToVisit[InstructionToVisitIndex].Get<1>();

						if (InputOperand.GetMemoryType() == ERigVMMemoryType::External)
						{
							if (!bFollowVariables)
							{
								continue;
							}
							const FRigHierarchyRecord VariableRecord(FRigHierarchyRecord::EType_Variable, InputOperand.GetRegisterIndex(), WrittenRecord.VM);
							OutDependencies.FindOrAdd(InstructionsToVisit[InstructionToVisitIndex].Get<1>()).Add(VariableRecord);
							LastRecord = VariableRecord;
						}
						
						const FRigVMOperand InputOperandNoRegisterOffset(InputOperand.GetMemoryType(), InputOperand.GetRegisterIndex());

						if(const TArray<int32>* InstructionsWithOutputOperand = OutputOperandToInstructions.Find(InputOperandNoRegisterOffset))
						{
							FindOrAddInstruction(*InstructionsWithOutputOperand, LastRecord);
						}
					}
				}
			}
		}
	}
#endif
}

FRigDependenciesProviderForControlRig::FRigDependenciesProviderForControlRig(UControlRig* InControlRig, const FName& InEventName, bool InFollowVariables)
	: FRigDependenciesProviderForVM(InControlRig, InEventName, InFollowVariables)
{
}

const TRigHierarchyDependencyMap& FRigDependenciesProviderForControlRig::GetRigHierarchyDependencies() const
{
	if (const UModularRig* ModularRig = Cast<UModularRig>(GetControlRig()))
	{
		const FRigVMExtendedExecuteContext& ExtendedExecuteContext = ModularRig->GetRigVMExtendedExecuteContext();
		const FControlRigExecuteContext& Context = ExtendedExecuteContext.GetPublicData<FControlRigExecuteContext>();
		const uint32 CurrentRecordsHash = Context.GetInstructionRecordsHash(); 
	
		if (!CachedDependencies.IsEmpty() && RecordsHash == CurrentRecordsHash)
		{
			return CachedDependencies;
		}

		CachedDependencies.Reset();
		CachedReverseDependencies.Reset();
		
		ModularRig->ForEachModule([this, Context](const FRigModuleInstance* InModule) -> bool
		{
			if (UControlRig* ModularRig = InModule->GetRig())
			{
				if (const URigVM* VM = ModularRig->GetVM())
				{
					ComputeDependencies(Context, VM, InModule->Name, CachedDependencies);
				}
			}
			return true;
		});

		RecordsHash = CurrentRecordsHash;
		ReverseRecordsHash = UINT32_MAX;
		return CachedDependencies;
	}
	return FRigDependenciesProviderForVM::GetRigHierarchyDependencies();
}

// UE_ENABLE_OPTIMIZATION
