// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchy.h"

#include "AnimationCoreLibrary.h"
#include "Async/TaskGraphInterfaces.h"
#include "ControlRig.h"

#include "Engine/World.h"
#include "Rigs/RigHierarchyElements.h"
#include "Rigs/RigHierarchyController.h"
#include "ModularRigRuleManager.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "Misc/ScopeLock.h"
#include "UObject/AnimObjectVersion.h"
#include "ControlRigObjectVersion.h"
#include "IControlRigObjectBinding.h"
#include "ModularRig.h"
#include "Algo/Count.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/Compression.h"
#include "UObject/ICookInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchy)

LLM_DEFINE_TAG(Animation_ControlRig);

#if WITH_EDITOR
#include "RigVMPythonUtils.h"
#include "ScopedTransaction.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/TransactionObjectEvent.h"
#include "JsonObjectConverter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Algo/Transform.h"
#include "Algo/Sort.h"

static FCriticalSection GRigHierarchyStackTraceMutex;
static char GRigHierarchyStackTrace[65536];
static void RigHierarchyCaptureCallStack(FString& OutCallstack, uint32 NumCallsToIgnore)
{
	UE::TScopeLock ScopeLock(GRigHierarchyStackTraceMutex);
	GRigHierarchyStackTrace[0] = 0;
	FPlatformStackWalk::StackWalkAndDump(GRigHierarchyStackTrace, 65535, 1 + NumCallsToIgnore);
	OutCallstack = ANSI_TO_TCHAR(GRigHierarchyStackTrace);
}

// CVar to record all transform changes 
static TAutoConsoleVariable<int32> CVarControlRigHierarchyTraceAlways(TEXT("ControlRig.Hierarchy.TraceAlways"), 0, TEXT("if nonzero we will record all transform changes."));
static TAutoConsoleVariable<int32> CVarControlRigHierarchyTraceCallstack(TEXT("ControlRig.Hierarchy.TraceCallstack"), 0, TEXT("if nonzero we will record the callstack for any trace entry.\nOnly works if(ControlRig.Hierarchy.TraceEnabled != 0)"));
static TAutoConsoleVariable<int32> CVarControlRigHierarchyTracePrecision(TEXT("ControlRig.Hierarchy.TracePrecision"), 3, TEXT("sets the number digits in a float when tracing hierarchies."));
static TAutoConsoleVariable<int32> CVarControlRigHierarchyTraceOnSpawn(TEXT("ControlRig.Hierarchy.TraceOnSpawn"), 0, TEXT("sets the number of frames to trace when a new hierarchy is spawned"));
TAutoConsoleVariable<bool> CVarControlRigHierarchyEnableRotationOrder(TEXT("ControlRig.Hierarchy.EnableRotationOrder"), true, TEXT("enables the rotation order for controls"));
TAutoConsoleVariable<bool> CVarControlRigHierarchyEnableModules(TEXT("ControlRig.Hierarchy.Modules"), true, TEXT("enables the modular rigging functionality"));
static TAutoConsoleVariable<bool> CVarControlRigHierarchyTreatAvailableSpaceSafe(TEXT("ControlRig.Hierarchy.TreatAvailableSpacesAsSafeForSpaceSwitching"), true, TEXT("Controls will be able to space switch to the list of available spaces, even if the space switch would cause a cycle."));
static TAutoConsoleVariable<bool> CVarControlRigHierarchyAllowToIgnoreCycles(TEXT("ControlRig.Hierarchy.AllowToIgnoreCycles"), true, TEXT("With this option turned on users can ignore cycles by clicking ignore in a modal dialog during the user interaction."));
static int32 sRigHierarchyLastTrace = INDEX_NONE;

// A console command to trace a single frame / single execution for a control rig anim node / control rig component
FAutoConsoleCommandWithWorldAndArgs FCmdControlRigHierarchyTraceFrames
(
	TEXT("ControlRig.Hierarchy.Trace"),
	TEXT("Traces changes in a hierarchy for a provided number of executions (defaults to 1).\nYou can use ControlRig.Hierarchy.TraceCallstack to enable callstack tracing as part of this."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
	{
		int32 NumFrames = 1;
		if(InParams.Num() > 0)
		{
			NumFrames = FCString::Atoi(*InParams[0]);
		}
		
		TArray<UObject*> Instances;
		URigHierarchy::StaticClass()->GetDefaultObject()->GetArchetypeInstances(Instances);

		for(UObject* Instance : Instances)
		{
			if (Instance->HasAnyFlags(RF_ClassDefaultObject))
			{
				continue;
			}
			
			// we'll just trace all of them for now
			//if(Instance->GetWorld() == InWorld)
			if(Instance->GetTypedOuter<UControlRig>() != nullptr)
			{
				CastChecked<URigHierarchy>(Instance)->TraceFrames(NumFrames);
			}
		}
	})
);

#endif

////////////////////////////////////////////////////////////////////////////////
// URigHierarchy
////////////////////////////////////////////////////////////////////////////////

#if URIGHIERARCHY_ENSURE_CACHE_VALIDITY
bool URigHierarchy::bEnableValidityCheckbyDefault = true;
#else
bool URigHierarchy::bEnableValidityCheckbyDefault = false;
#endif

const FLazyName URigHierarchy::DefaultParentKeyLabel = TEXT("Parent");
const FLazyName URigHierarchy::WorldSpaceKeyLabel = TEXT("World");

URigHierarchy::URigHierarchy()
: TopologyVersion(0)
, ParentWeightVersion(0)
, MetadataVersion(0)
, MetadataTagVersion(0)
, bEnableDirtyPropagation(true)
, Elements()
, bRecordCurveChanges(true)
, ElementIndexLookup()
, TransformStackIndex(0)
, bTransactingForTransformChange(false)
, MaxNameLength(GetDefaultMaxNameLength())
, bIsInteracting(false)
, LastInteractedKey()
, bSuspendNotifications(false)
, bSuspendMetadataNotifications(false)
, bSuspendNameSpaceKeyWarnings(false)
, HierarchyController(nullptr)
, bIsControllerAvailable(true)
, ResetPoseHash(INDEX_NONE)
, bIsCopyingHierarchy(false)
#if WITH_EDITOR
, bPropagatingChange(false)
, bForcePropagation(false)
, TraceFramesLeft(0)
, TraceFramesCaptured(0)
#endif
, bEnableCacheValidityCheck(bEnableValidityCheckbyDefault)
, HierarchyForCacheValidation()
, bUsePreferredEulerAngles(true)
, bAllowNameSpaceWhenSanitizingName(false)
, ExecuteContext(nullptr)
#if WITH_EDITOR
, bRecordInstructionsAtRuntime(true)
#endif
, ElementKeyRedirector(nullptr)
, ElementBeingDestroyed(nullptr)
{
	Reset();
#if WITH_EDITOR
	TraceFrames(CVarControlRigHierarchyTraceOnSpawn->GetInt());
#endif
}

void URigHierarchy::BeginDestroy()
{
	// reset needs to be in begin destroy since
	// reset touches a UObject member of this hierarchy,
	// which will be GCed by the time this hierarchy reaches destructor
	Reset();
	Super::BeginDestroy();
}

void URigHierarchy::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		// remove due to FPIEFixupSerializer hitting this checkNoEntry();
	}
}

void URigHierarchy::AddReferencedObjects(UObject* InpThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InpThis, Collector);

	URigHierarchy* pThis = static_cast<URigHierarchy*>(InpThis);
	UE::TScopeLock Lock(pThis->ElementsLock);
	for (FRigBaseElement* Element : pThis->Elements)
	{
		Collector.AddPropertyReferencesWithStructARO(Element->GetScriptStruct(), Element, pThis);
	}
}

void URigHierarchy::Save(FArchive& Ar)
{
	UE::TScopeLock Lock(ElementsLock);
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("URigHierarchy(%s)"), *GetName()));

	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	if(Ar.IsTransacting())
	{
		Ar << TransformStackIndex;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("TransformStackIndex"));
		Ar << bTransactingForTransformChange;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("bTransactingForTransformChange"));
		
		if(bTransactingForTransformChange)
		{
			return;
		}

		TArray<FRigHierarchyKey> SelectedKeys = GetSelectedHierarchyKeys();
		Ar << SelectedKeys;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("SelectedKeys"));
	}

	// make sure all parts of pose are valid.
	// this ensures cache validity.
	EnsureCacheValidity();

	// by computing all transforms we can be sure that initial & current,
	// and local & global each are correct.
	ComputeAllTransforms();

	FRigHierarchySerializationSettings SerializationSettings(Ar);
	if(SerializationSettings.bIsSerializingToPackage)
	{
		SerializationSettings.bSerializeGlobalTransform = false;
		SerializationSettings.bSerializeCurrentTransform = false;
		SerializationSettings.bUseCompressedArchive = true;
	}
	
	SerializationSettings.Save(Ar);

	FArchive* ArchiveForElements = &Ar;

	TArray<uint8> UncompressedBytes;
	TArray<FName> UniqueNames;
	FRigHierarchyMemoryWriter MemoryWriter(UncompressedBytes, UniqueNames, Ar.IsPersistent());
	if(SerializationSettings.bUseCompressedArchive)
	{
		MemoryWriter.UsingCustomVersion(FAnimObjectVersion::GUID);
		MemoryWriter.UsingCustomVersion(FControlRigObjectVersion::GUID);
		
		ArchiveForElements = &MemoryWriter;
	}

	int32 ElementCount = Elements.Num();
	*ArchiveForElements << ElementCount;

	for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
	{
		FRigBaseElement* Element = Elements[ElementIndex];

		// store the key
		FRigElementKey Key = Element->GetKey();
		*ArchiveForElements << Key;
	}

	SerializationSettings.SerializationPhase = FRigHierarchySerializationSettings::StaticData;
	for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
	{
		FRigBaseElement* Element = Elements[ElementIndex];
		Element->Serialize(*ArchiveForElements, SerializationSettings);
	}

	SerializationSettings.SerializationPhase = FRigHierarchySerializationSettings::InterElementData;
	for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
	{
		FRigBaseElement* Element = Elements[ElementIndex];
		Element->Serialize(*ArchiveForElements, SerializationSettings);
	}

	*ArchiveForElements << PreviousHierarchyNameMap;
	*ArchiveForElements << PreviousHierarchyParentMap;

	{
		TMap<FRigElementKey, FMetadataStorage> ElementMetadataToSave;
		
		for (const FRigBaseElement* Element: Elements)
		{
			if (ElementMetadata.IsValidIndex(Element->MetadataStorageIndex))
			{
				ElementMetadataToSave.Add(Element->Key, ElementMetadata[Element->MetadataStorageIndex]);
			}
		}
		
		*ArchiveForElements << ElementMetadataToSave;
	}

	// save all components
	int32 NumComponents = 0;
	for(FInstancedStruct& InstancedStruct : ElementComponents)
	{
		if(InstancedStruct.IsValid())
		{
			NumComponents++;
		}
	}
	*ArchiveForElements << NumComponents;
	if(NumComponents > 0)
	{
		TArray<const UScriptStruct*> ComponentScriptStructs;
		for(FInstancedStruct& InstancedStruct : ElementComponents)
		{
			if(InstancedStruct.IsValid())
			{
				FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
				UScriptStruct* ScriptStruct = Component->GetScriptStruct();
				ComponentScriptStructs.AddUnique(ScriptStruct);

				// if the reference collector is running also provide the script struct itself.
				// this makes sure that dependency plugins will be loaded and understood as a dependency
				// of this asset.
				if(Ar.IsObjectReferenceCollector())
				{
					Ar << ScriptStruct;
				}
			}
		}

		TArray<FString> ScriptStructNames;
		ScriptStructNames.Reserve(ComponentScriptStructs.Num());
		for(const UScriptStruct* ScriptStruct : ComponentScriptStructs)
		{
			ScriptStructNames.Add(ScriptStruct->GetStructCPPName());
		}
		*ArchiveForElements << ScriptStructNames;
		
		for(FInstancedStruct& InstancedStruct : ElementComponents)
		{
			if(InstancedStruct.IsValid())
			{
				FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
				const UScriptStruct* ScriptStruct = Component->GetScriptStruct();
				
				int32 IndexOfScriptStruct = ComponentScriptStructs.Find(ScriptStruct);
				check(ComponentScriptStructs.IsValidIndex(IndexOfScriptStruct));
				*ArchiveForElements << IndexOfScriptStruct;

				// we store the archive position after the component. we do this so we can recover from
				// loading content which cannot resolve the component's script struct.
				int64 ArchivePositionBeforeComponent = ArchiveForElements->GetArchiveState().Tell();
				int64 ArchivePositionAfterComponent = 0; // now known yet
				*ArchiveForElements << ArchivePositionAfterComponent;

				// serialize the component and take note of the pos of the archive
				Component->Serialize(*ArchiveForElements);
				ArchivePositionAfterComponent = ArchiveForElements->GetArchiveState().Tell();

				// save this again so we can skip during load
				ArchiveForElements->Seek(ArchivePositionBeforeComponent);
				*ArchiveForElements << ArchivePositionAfterComponent;
				ArchiveForElements->Seek(ArchivePositionAfterComponent);
			}
		}
	}

	if(SerializationSettings.bUseCompressedArchive)
	{
		// It is possible for compression to actually increase the size of the data, so we over allocate here to handle that.
		int32 UncompressedSize = UncompressedBytes.Num();
		int32 CompressedSize = FCompression::CompressMemoryBound(NAME_Oodle, UncompressedSize);

		TArray<uint8> CompressedBytes;
		CompressedBytes.SetNumUninitialized(CompressedSize);

		bool bStoreCompressedBytes = FCompression::CompressMemory(NAME_Oodle, CompressedBytes.GetData(), CompressedSize, UncompressedBytes.GetData(), UncompressedBytes.Num(), COMPRESS_BiasMemory);
		if(bStoreCompressedBytes)
		{
			if(CompressedSize < UncompressedSize)
			{
				CompressedBytes.SetNum(CompressedSize);
			}
			else
			{
				bStoreCompressedBytes = false;
			}
		}

		Ar << UniqueNames;
		Ar << UncompressedSize;
		Ar << bStoreCompressedBytes;
		if(bStoreCompressedBytes)
		{
			Ar << CompressedBytes;
		}
		else
		{
			Ar << UncompressedBytes;
		}
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("CompressedElements"));
	}
}

void URigHierarchy::Load(FArchive& Ar)
{
	UE::TScopeLock Lock(ElementsLock);
	
	// unlink any existing pose adapter
	UnlinkPoseAdapter();

	TArray<FRigHierarchyKey> SelectedKeys;
	if(Ar.IsTransacting())
	{
		bool bOnlySerializedTransformStackIndex = false;
		Ar << TransformStackIndex;
		Ar << bOnlySerializedTransformStackIndex;
		
		if(bOnlySerializedTransformStackIndex)
		{
			return;
		}

		Ar << SelectedKeys;
	}

	// If a controller is found where the outer is this hierarchy, make sure it is configured correctly
	{
		TArray<UObject*> ChildObjects;
		GetObjectsWithOuter(this, ChildObjects, EGetObjectsFlags::None);
		ChildObjects = ChildObjects.FilterByPredicate([](UObject* Object)
			{ return Object->IsA<URigHierarchyController>();});
		if (!ChildObjects.IsEmpty())
		{
			ensure(ChildObjects.Num() == 1); // there should only be one controller
			bIsControllerAvailable = true;
			HierarchyController = Cast<URigHierarchyController>(ChildObjects[0]);
			HierarchyController->SetHierarchy(this);
		}
	}
	
	Reset();

	FRigHierarchySerializationSettings SerializationSettings(Ar);
	if(Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyCompactTransformSerialization)
	{
		SerializationSettings.Load(Ar);
	}
	else if(Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyCompressElements)
	{
		Ar << SerializationSettings.bUseCompressedArchive;
	}
	SerializationSettings.SerializationPhase = FRigHierarchySerializationSettings::StaticData;

	FArchive* ArchiveForElements = &Ar;

	TUniquePtr<FRigHierarchyMemoryReader> MemoryReader;
	TArray<uint8> UncompressedBytes;
	TArray<uint8> CompressedBytes;
	TArray<FName> UniqueNames;
	if(SerializationSettings.bUseCompressedArchive)
	{
		// this block is not going to be entered if we are in an older asset
		// ( < FControlRigObjectVersion::RigHierarchyCompressElements ) since
		// bUseCompressedArchive can only be true in that case.
		Ar << UniqueNames;
		
		int32 UncompressedSize = 0;
		Ar << UncompressedSize;

		bool bStoreCompressedBytes = false;
		Ar << bStoreCompressedBytes;

		Ar << CompressedBytes;

		if(bStoreCompressedBytes)
		{
			UncompressedBytes.SetNumUninitialized(UncompressedSize);

			bool bSuccess = FCompression::UncompressMemory(NAME_Oodle, UncompressedBytes.GetData(), UncompressedSize, CompressedBytes.GetData(), CompressedBytes.Num());
			verify(bSuccess);
		}

		MemoryReader = MakeUnique<FRigHierarchyMemoryReader>(bStoreCompressedBytes ? UncompressedBytes : CompressedBytes, UniqueNames, Ar.IsPersistent());
		ArchiveForElements = MemoryReader.Get();
		
		// force to use the versions from within the main archive
		ArchiveForElements->SetCustomVersions(Ar.GetCustomVersions());
	}

	int32 ElementCount = 0;
	*ArchiveForElements << ElementCount;

	PoseVersionPerElement.Reset();

	int32 NumTransforms = 0;
	int32 NumDirtyStates = 0;
	int32 NumCurves = 0;

	ElementIndexLookup.Reserve(ElementCount);
	
	const bool bAllocateStoragePerElement = ArchiveForElements->CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RigHierarchyIndirectElementStorage;
	for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
	{
		FRigElementKey Key;
		*ArchiveForElements << Key;

		FRigBaseElement* Element = MakeElement(Key.Type);
		check(Element);

		Element->SubIndex = Num(Key.Type);
		Element->Index = Elements.Add(Element);
		ElementsPerType[RigElementTypeToFlatIndex(Key.Type)].Add(Element);
		ElementIndexLookup.Add(Key, Element->Index);

		if(bAllocateStoragePerElement)
		{
			AllocateDefaultElementStorage(Element, false);
			Element->Load(*ArchiveForElements, SerializationSettings);
		}
		else
		{
			NumTransforms += Element->GetNumTransforms();
			NumDirtyStates += Element->GetNumTransforms();
			NumCurves += Element->GetNumCurves();
		}
	}

	if(bAllocateStoragePerElement)
	{
		// update all storage pointers now that we've created
		// all elements.
		UpdateElementStorage();
	}
	else // if we can allocate the storage as one big buffer...
	{
		const TArray<int32, TInlineAllocator<4>> TransformIndices = ElementTransforms.Allocate(NumTransforms, FTransform::Identity);
		const TArray<int32, TInlineAllocator<4>> DirtyStateIndices = ElementDirtyStates.Allocate(NumDirtyStates, false);
		const TArray<int32, TInlineAllocator<4>> CurveIndices = ElementCurves.Allocate(NumCurves, 0.f);
		int32 UsedTransformIndex = 0;
		int32 UsedDirtyStateIndex = 0;
		int32 UsedCurveIndex = 0;

		ElementTransforms.Shrink();
		ElementDirtyStates.Shrink();
		ElementCurves.Shrink();

		for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
		{
			FRigBaseElement* Element = Elements[ElementIndex];
			
			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
			{
				TransformElement->PoseStorage.Initial.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseStorage.Current.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseStorage.Initial.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseStorage.Current.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				TransformElement->PoseDirtyState.Current.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				TransformElement->PoseDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				TransformElement->PoseDirtyState.Current.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];

				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
				{
					ControlElement->OffsetStorage.Initial.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetStorage.Current.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetStorage.Initial.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetStorage.Current.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->OffsetDirtyState.Current.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->OffsetDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->OffsetDirtyState.Current.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeStorage.Initial.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeStorage.Current.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeStorage.Initial.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeStorage.Current.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeDirtyState.Current.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeDirtyState.Current.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				}
			}
			else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Element))
			{
				CurveElement->StorageIndex = CurveIndices[UsedCurveIndex++];
			}

			Element->LinkStorage(ElementTransforms.GetStorage(), ElementDirtyStates.GetStorage(), ElementCurves.GetStorage());
			Element->Load(*ArchiveForElements, SerializationSettings);
		}
	}
	IncrementTopologyVersion();

	SerializationSettings.SerializationPhase = FRigHierarchySerializationSettings::InterElementData;
	for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
	{
		FRigBaseElement* Element = Elements[ElementIndex];
		Element->Load(*ArchiveForElements, SerializationSettings);
	}

	IncrementTopologyVersion();

	for(int32 ElementIndex = 0; ElementIndex < ElementCount; ElementIndex++)
	{
		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Elements[ElementIndex]))
		{
			FRigBaseElementParentArray CurrentParents = GetParents(TransformElement, false);
			for (FRigBaseElement* CurrentParent : CurrentParents)
			{
				if(FRigTransformElement* TransformParent = Cast<FRigTransformElement>(CurrentParent))
				{
					TransformParent->ElementsToDirty.AddUnique(TransformElement);
				}
			}
		}
	}

	if(Ar.IsTransacting())
	{
		for(const FRigHierarchyKey& SelectedKey : SelectedKeys)
		{
			if(SelectedKey.IsElement())
			{
				if(FRigBaseElement* Element = Find<FRigBaseElement>(SelectedKey.GetElement()))
				{
					Element->bSelected = true;
					OrderedSelection.Add(SelectedKey);
				}
			}
			else if(SelectedKey.IsComponent())
			{
				if(FRigBaseComponent* Component = FindComponent(SelectedKey.GetComponent()))
				{
					Component->bSelected = true;
					OrderedSelection.Add(SelectedKey);
				}
			}
		}
	}

	if (ArchiveForElements->CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyStoringPreviousNames)
	{
		if (ArchiveForElements->CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyPreviousNameAndParentMapUsingHierarchyKey)
		{
			*ArchiveForElements << PreviousHierarchyNameMap;
			*ArchiveForElements << PreviousHierarchyParentMap;
		}
		else
		{
			TMap<FRigElementKey, FRigElementKey> PreviousNameMap;
			TMap<FRigElementKey, FRigElementKey> PreviousParentMap;
			*ArchiveForElements << PreviousNameMap;
			*ArchiveForElements << PreviousParentMap;

			PreviousHierarchyNameMap.Reset();
			for(const TPair<FRigElementKey, FRigElementKey>& Pair : PreviousNameMap)
			{
				PreviousHierarchyNameMap.Add(Pair.Key, Pair.Value);
			}

			PreviousHierarchyParentMap.Reset();
			for(const TPair<FRigElementKey, FRigElementKey>& Pair : PreviousParentMap)
			{
				PreviousHierarchyParentMap.Add(Pair.Key, Pair.Value);
			}
		}
	}
	else
	{
		PreviousHierarchyNameMap.Reset();
		PreviousHierarchyParentMap.Reset();
	}

	if (ArchiveForElements->CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyStoresElementMetadata)
	{
		ElementMetadata.Reset();
		TMap<FRigElementKey, FMetadataStorage> LoadedElementMetadata;
		
		*ArchiveForElements << LoadedElementMetadata;
		for (TPair<FRigElementKey, FMetadataStorage>& Entry: LoadedElementMetadata)
		{
			FRigBaseElement* Element = Find(Entry.Key);
			Element->MetadataStorageIndex = ElementMetadata.Num();
			ElementMetadata.Add(MoveTemp(Entry.Value));
		}
	}

	if (ArchiveForElements->CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyStoresComponents)
	{
		ElementComponents.Reset();
		ComponentIndexLookup.Reset();

		int32 NumComponents = 0;
		*ArchiveForElements << NumComponents;

		if(NumComponents > 0)
		{
			ElementComponents.AddZeroed(NumComponents);
			ComponentIndexLookup.Reserve(NumComponents);

			TArray<FString> ScriptStructNames;
			*ArchiveForElements << ScriptStructNames;

			TArray<UScriptStruct*> ComponentScriptStructs;
			ComponentScriptStructs.AddZeroed(ScriptStructNames.Num());

			for(int32 ScriptStructIndex = 0; ScriptStructIndex < ScriptStructNames.Num(); ScriptStructIndex++)
			{
				// this lookup makes sure to avoid loading things syncronously,
				// and it also relies on FindGlobally - which internally uses
				// the core redirects in case the structure has been renamed.
				if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(RigVMTypeUtils::ObjectFromCPPType(ScriptStructNames[ScriptStructIndex])))
				{
					ComponentScriptStructs[ScriptStructIndex] = ScriptStruct;
				}
			}

			for (int32 ComponentIndex = 0; ComponentIndex < ElementComponents.Num(); ComponentIndex++)
			{
				int32 IndexOfScriptStruct = 0;
				*ArchiveForElements << IndexOfScriptStruct;
				check(ComponentScriptStructs.IsValidIndex(IndexOfScriptStruct));

				// we know the position of the archive so we can skip the data
				// in case we cannot resolve the component
				int64 ArchivePositionAfterComponent = 0;
				*ArchiveForElements << ArchivePositionAfterComponent;

				UScriptStruct* ScriptStruct = ComponentScriptStructs[IndexOfScriptStruct];
				if (ScriptStruct == nullptr)
				{
					ArchiveForElements->Seek(ArchivePositionAfterComponent);
					continue;
				}

				ArchiveForElements->Preload(ScriptStruct);

				FInstancedStruct& InstancedStruct = ElementComponents[ComponentIndex];
				InstancedStruct.InitializeAs(ScriptStruct);
				
				FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
				Component->Serialize(*ArchiveForElements);

				Component->IndexInHierarchy = ComponentIndex;

				FRigBaseElement* Element = Find(Component->GetElementKey());
				if (ensure(Element))
				{
					Component->Element = Element;
					Component->IndexInElement = Element->ComponentIndices.Add(Component->IndexInHierarchy);

					ComponentIndexLookup.Add(Component->GetKey(), ComponentIndex);
				}
				else
				{
					UE_LOGF(
						LogControlRig,
						Warning,
						"%ls: Can't find an owning element for component '%ls.%ls' - this component will not be loaded.",
						*GetPathName(),
						*Component->GetElementKey().Name.ToString(),
						*Component->GetKey().Name.ToString()
					);
				}
			}
		}

		IncrementTopologyVersion();
	}

	// set the max name length but respect the length of elements + components
	SetMaxNameLength(GetDefaultMaxNameLength());
	
	(void)SortElementStorage();
}

void URigHierarchy::PostLoad()
{
	UObject::PostLoad();

	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	CleanupInvalidCaches();

	Notify(ERigHierarchyNotification::HierarchyReset, {});
}

#if WITH_EDITORONLY_DATA
void URigHierarchy::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(URigHierarchyController::StaticClass()));
}
#endif

void URigHierarchy::Reset()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	Reset_Impl(true);
}

void URigHierarchy::Reset_Impl(bool bResetElements)
{
	TopologyVersion = 0;
	ParentWeightVersion = 0;
	MetadataVersion = 0;
	bEnableDirtyPropagation = true;
	NonUniqueShortNamesCache.Reset();

	if(bResetElements)
	{
		UE::TScopeLock Lock(ElementsLock);

		// remove the metadata first. elements hold an integer pointing to the
		// element metadata for each element - so if this is empty it won't cause
		// any further deallocation / removal.
		ElementMetadata.Reset([](int32 InIndex, FMetadataStorage* MetadataStorage)
		{
			for (TTuple<FName, FRigBaseMetadata*>& Item: MetadataStorage->MetadataMap)
			{
				FRigBaseMetadata::DestroyMetadata(&Item.Value);
			} 
		});

		// clear these first - the indices pointing to these
		// won't be valid - which in turn won't cause any deallocation
		ElementTransforms.Reset();
		ElementDirtyStates.Reset();
		ElementCurves.Reset();

		// clearing the components first will avoid cost of looping over
		// each component per elements and destroying them one by onw
		ElementComponents.Reset();

		// walk in reverse since certain elements might not have been allocated themselves
		for(int32 ElementIndex = Elements.Num() - 1; ElementIndex >= 0; ElementIndex--)
		{
			DestroyElement(Elements[ElementIndex],
				false /* don't clear components */,
				false /* don't clear element storage */,
				false /* don't destroy metadata */);
		}

		// Free batch blocks whose head was removed from Elements by RemoveElement.
		for (FRigBaseElement* OrphanedHead : OrphanedBatchHeads)
		{
			DestroyElement(OrphanedHead,
				false /* don't clear components */,
				false /* don't clear element storage */,
				false /* don't destroy metadata */);
		}
		OrphanedBatchHeads.Reset();

		Elements.Reset();
		ElementsPerType.Reset();
		const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();
		for (int32 EnumTypeBit = (int32)ERigElementType::First; EnumTypeBit <= (int32)ERigElementType::Last; EnumTypeBit <<= 1)
		{
			const int32 ElementTypeIndex = RigElementTypeToFlatIndex(static_cast<ERigElementType>(EnumTypeBit));
			if (ElementTypeIndex != INDEX_NONE) // Skip deprecated types
			{
				ElementsPerType.Add(TArray<FRigBaseElement*>());
			}
		}
		ElementIndexLookup.Reset();
		ComponentIndexLookup.Reset();
	}

	ResetPoseHash = INDEX_NONE;
	ResetPoseIsFilteredOut.Reset();
	ElementsToRetainLocalTransform.Reset();
	DefaultParentPerElement.Reset();
	OrderedSelection.Reset();
	PoseVersionPerElement.Reset();
	ElementDependencyCache.Reset();
	ResetChangedCurveIndices();

	ChildElementOffsetAndCountCache.Reset();
	ChildElementCache.Reset();
	ChildElementCacheTopologyVersion = std::numeric_limits<uint32>::max();


	{
		FGCScopeGuard Guard;
		Notify(ERigHierarchyNotification::HierarchyReset, {});
	}

	if(HierarchyForCacheValidation)
	{
		HierarchyForCacheValidation->Reset();
	}

	SetMaxNameLength(GetDefaultMaxNameLength());
}

#if WITH_EDITOR

void URigHierarchy::ForEachListeningHierarchy(TFunctionRef<void(const FRigHierarchyListener&)> PerListeningHierarchyFunction)
{
	CleanupListeningHierarchies();
	for(int32 Index = 0; Index < ListeningHierarchies.Num(); Index++)
	{
		if (ListeningHierarchies.IsValidIndex(Index))
		{
			if (ListeningHierarchies[Index].Hierarchy.IsValid())
			{
				PerListeningHierarchyFunction(ListeningHierarchies[Index]);
			}
		}
	}
}

#endif

void URigHierarchy::ResetToDefault()
{
	UE::TScopeLock Lock(ElementsLock);
	
	if(DefaultHierarchyPtr.IsValid())
	{
		if(URigHierarchy* DefaultHierarchy = DefaultHierarchyPtr.Get())
		{
			CopyHierarchy(DefaultHierarchy);
			// For RuntimeAssets DefaultHierarchy is the live asset hierarchy and can carry stale
			// dirty flags from prior execution or editor interaction. Resolve them before construction.
			// Not needed at runtime: cooked packages strip current and global transforms, so
			// deserialized state is always consistently dirty.
#if WITH_EDITOR
			ComputeAllTransforms();
#endif
			return;
		}
	}
	Reset();
}

void URigHierarchy::CopyHierarchy(URigHierarchy* InHierarchy)
{
	check(InHierarchy);

	const TGuardValue<bool> MarkCopyingHierarchy(bIsCopyingHierarchy, true);
	
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	UE::TScopeLock Lock(ElementsLock);
	if(Elements.Num() == 0 && InHierarchy->Elements.Num () == 0)
	{
		return;
	}

	// remember the previous selection
	const TArray<FRigHierarchyKey> PreviousSelection = GetSelectedHierarchyKeys();

	// unlink any existing pose adapter
	UnlinkPoseAdapter();

	// check if we really need to do a deep copy all over again.
	// for rigs which contain more elements (likely procedural elements)
	// we'll assume we can just remove superfluous elements (from the end of the lists).
	bool bReallocateElements = Elements.Num() < InHierarchy->Elements.Num();
	if(!bReallocateElements)
	{
		const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();
		for (int32 EnumTypeBit = (int32)ERigElementType::First; EnumTypeBit <= (int32)ERigElementType::Last; EnumTypeBit <<= 1)
		{
			const int32 ElementTypeIndex = RigElementTypeToFlatIndex(static_cast<ERigElementType>(EnumTypeBit));
			if (ElementTypeIndex != INDEX_NONE) // Skip deprecated types
			{
				check(ElementsPerType.IsValidIndex(ElementTypeIndex));
				check(InHierarchy->ElementsPerType.IsValidIndex(ElementTypeIndex));
				if(ElementsPerType[ElementTypeIndex].Num() != InHierarchy->ElementsPerType[ElementTypeIndex].Num())
				{
					bReallocateElements = true;
					break;
				}
			}
		}

		// make sure that we have the elements in the right order / type
		if(!bReallocateElements)
		{
			for(int32 Index = 0; Index < InHierarchy->Elements.Num(); Index++)
			{
				if((Elements[Index]->GetKey().Type != InHierarchy->Elements[Index]->GetKey().Type) ||
					(Elements[Index]->SubIndex != InHierarchy->Elements[Index]->SubIndex))
				{
					bReallocateElements = true;
					break;
				}
			}
		}
	}

	{
		TGuardValue<bool> SuspendMetadataNotifications(bSuspendMetadataNotifications, true);
		Reset_Impl(bReallocateElements);

		static const TArray<int32> StructureSizePerType = {
			sizeof(FRigBoneElement),
			sizeof(FRigNullElement),
			sizeof(FRigControlElement),
			sizeof(FRigCurveElement),
			sizeof(FRigReferenceElement),
			sizeof(FRigConnectorElement),
			sizeof(FRigSocketElement),
		};

		int32 NumTransforms = 0;
		int32 NumDirtyStates = 0;
		int32 NumCurves = 0;

		if(bReallocateElements)
		{
			// Allocate the elements in batches to improve performance
			TArray<uint8*> NewElementsPerType;
			for(int32 ElementTypeIndex = 0; ElementTypeIndex < InHierarchy->ElementsPerType.Num(); ElementTypeIndex++)
			{
				const ERigElementType ElementType = FlatIndexToRigElementType(ElementTypeIndex);
				int32 StructureSize = 0;

				const int32 Count = InHierarchy->ElementsPerType[ElementTypeIndex].Num();
				if(Count && ElementType != ERigElementType::Connector && ElementType != ERigElementType::Socket)
				{
					FRigBaseElement* ElementMemory = MakeElement(ElementType, Count, &StructureSize);
					verify(StructureSize == StructureSizePerType[ElementTypeIndex]);
					NewElementsPerType.Add(reinterpret_cast<uint8*>(ElementMemory));
				}
				else
				{
					NewElementsPerType.Add(nullptr);
				}
			
				ElementsPerType[ElementTypeIndex].Reserve(Count);
			}

			Elements.Reserve(InHierarchy->Elements.Num());
			ElementIndexLookup = InHierarchy->ElementIndexLookup;

			for(int32 Index = 0; Index < InHierarchy->Num(); Index++)
			{
				const FRigBaseElement* Source = InHierarchy->Get(Index);
				const FRigElementKey& Key = Source->Key;

				const int32 ElementTypeIndex = RigElementTypeToFlatIndex(Key.Type);
				const ERigElementType ElementType = FlatIndexToRigElementType(ElementTypeIndex);
				
				FRigBaseElement* Target = nullptr;
				if (ElementType == ERigElementType::Connector || ElementType == ERigElementType::Socket)
				{
					Target = MakeElement(ElementType, 1);
				}
				else
				{
					const int32 SubIndex = Num(Key.Type);
					const int32 StructureSize = StructureSizePerType[ElementTypeIndex];
					check(NewElementsPerType[ElementTypeIndex] != nullptr);
					Target = reinterpret_cast<FRigBaseElement*>(&NewElementsPerType[ElementTypeIndex][StructureSize * SubIndex]);
				}

				Target->InitializeFrom(Source); // warning, this copies index and subindex from source, among other things

				NumTransforms += Target->GetNumTransforms();
				NumDirtyStates += Target->GetNumTransforms();
				NumCurves += Target->GetNumCurves();

				Target->Index = Elements.Add(Target);
				Target->SubIndex = ElementsPerType[ElementTypeIndex].Add(Target);
				Target->ComponentIndices.Reset();
			
				IncrementPoseVersion(Index);

				check(Source->Index == Index);
				check(Target->Index == Index);
			}
		}
		else
		{
			// remove the superfluous elements
			for(int32 ElementIndex = Elements.Num() - 1; ElementIndex >= InHierarchy->Elements.Num(); ElementIndex--)
			{
				DestroyElement(Elements[ElementIndex]);
			}

			// shrink the containers accordingly
			Elements.SetNum(InHierarchy->Elements.Num());
			const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();
			for (int32 EnumTypeBit = (int32)ERigElementType::First; EnumTypeBit <= (int32)ERigElementType::Last; EnumTypeBit <<= 1)
			{
				const int32 ElementTypeIndex = RigElementTypeToFlatIndex(static_cast<ERigElementType>(EnumTypeBit));
				if (ElementTypeIndex != INDEX_NONE) // Skip deprecated types
				{
					ElementsPerType[ElementTypeIndex].SetNum(InHierarchy->ElementsPerType[ElementTypeIndex].Num());
				}
			}

			for(int32 Index = 0; Index < InHierarchy->Num(); Index++)
			{
				const FRigBaseElement* Source = InHierarchy->Get(Index);
				FRigBaseElement* Target = Elements[Index];

				check(Target->Key.Type == Source->Key.Type);
				Target->InitializeFrom(Source);
				Target->ComponentIndices.Reset();

				NumTransforms += Target->GetNumTransforms();
				NumDirtyStates += Target->GetNumTransforms();
				NumCurves += Target->GetNumCurves();

				IncrementPoseVersion(Index);
			}

			ElementIndexLookup = InHierarchy->ElementIndexLookup;
		}

		ElementTransforms.Reset();
		ElementDirtyStates.Reset();
		ElementCurves.Reset();
		
		const TArray<int32, TInlineAllocator<4>> TransformIndices = ElementTransforms.Allocate(NumTransforms, FTransform::Identity);
		const TArray<int32, TInlineAllocator<4>> DirtyStateIndices = ElementDirtyStates.Allocate(NumDirtyStates, false);
		const TArray<int32, TInlineAllocator<4>> CurveIndices = ElementCurves.Allocate(NumCurves, 0.f);
		int32 UsedTransformIndex = 0;
		int32 UsedDirtyStateIndex = 0;
		int32 UsedCurveIndex = 0;

		ElementTransforms.Shrink();
		ElementDirtyStates.Shrink();
		ElementCurves.Shrink();

		// Copy all the element subclass data and all elements' metadata over.
		for(int32 Index = 0; Index < InHierarchy->Num(); Index++)
		{
			const FRigBaseElement* Source = InHierarchy->Get(Index);
			FRigBaseElement* Target = Elements[Index];

			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Target))
			{
				TransformElement->PoseStorage.Initial.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseStorage.Current.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseStorage.Initial.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseStorage.Current.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
				TransformElement->PoseDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				TransformElement->PoseDirtyState.Current.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				TransformElement->PoseDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				TransformElement->PoseDirtyState.Current.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];

				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
				{
					ControlElement->OffsetStorage.Initial.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetStorage.Current.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetStorage.Initial.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetStorage.Current.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->OffsetDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->OffsetDirtyState.Current.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->OffsetDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->OffsetDirtyState.Current.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeStorage.Initial.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeStorage.Current.Local.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeStorage.Initial.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeStorage.Current.Global.StorageIndex = TransformIndices[UsedTransformIndex++];
					ControlElement->ShapeDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeDirtyState.Current.Local.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
					ControlElement->ShapeDirtyState.Current.Global.StorageIndex = DirtyStateIndices[UsedDirtyStateIndex++];
				}
			}
			else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Target))
			{
				CurveElement->StorageIndex = CurveIndices[UsedCurveIndex++];
			}
			Target->LinkStorage(ElementTransforms.GetStorage(), ElementDirtyStates.GetStorage(), ElementCurves.GetStorage());

			Target->CopyFrom(Source);

			CopyAllMetadataFromElement(Target, Source);
		}

		(void)SortElementStorage();

		PreviousHierarchyNameMap.Append(InHierarchy->PreviousHierarchyNameMap);
		PreviousHierarchyParentMap.Append(InHierarchy->PreviousHierarchyParentMap);

		// make sure the curves are marked as unset after copying
		UnsetCurveValues();

		// Components
		ElementComponents.Reset();
		ElementComponents.Reserve(InHierarchy->ElementComponents.Num());
		ComponentIndexLookup.Reset();
		ComponentIndexLookup.Reserve(InHierarchy->ElementComponents.Num());

		for(int32 SourceComponentIndex = 0; SourceComponentIndex < InHierarchy->ElementComponents.Num(); SourceComponentIndex++)
		{
			if(!InHierarchy->ElementComponents[SourceComponentIndex].IsValid())
			{
				continue;
			}
			const int32 TargetComponentIndex = ElementComponents.Add(InHierarchy->ElementComponents[SourceComponentIndex]);
			FRigBaseComponent* TargetComponent = ElementComponents[TargetComponentIndex].GetMutablePtr<FRigBaseComponent>();
			TargetComponent->IndexInHierarchy = TargetComponentIndex;

			TargetComponent->Element = Find(TargetComponent->GetElementKey());
			if (ensure(TargetComponent->Element))
			{
				TargetComponent->IndexInElement = TargetComponent->Element->ComponentIndices.Add(TargetComponent->IndexInHierarchy);
				ComponentIndexLookup.Add(TargetComponent->GetKey(), TargetComponentIndex);
			}
			else
			{
				TargetComponent->IndexInElement = -1;
				UE_LOGF(
					LogControlRig,
					Warning,
					"%ls: Can't find an owning element for component '%ls.%ls'.",
					*GetPathName(),
					*TargetComponent->GetElementKey().Name.ToString(),
					*TargetComponent->GetKey().Name.ToString()
				);
			}
		}

		// copy the topology version from the hierarchy.
		// for this we'll include the hash of the hierarchy to make sure we
		// are deterministic for different hierarchies.
		TopologyVersion = HashCombine(InHierarchy->GetTopologyVersion(), InHierarchy->GetTopologyHash());

		// Increment the topology version to invalidate our cached children.
		IncrementTopologyVersion();

		ParentWeightVersion += InHierarchy->GetParentWeightVersion();

		// Keep incrementing the metadata version so that the UI can refresh.
		MetadataVersion += InHierarchy->GetMetadataVersion();
		MetadataTagVersion += InHierarchy->GetMetadataTagVersion();
	}

	if (MetadataChangedDelegate.IsBound())
	{
		MetadataChangedDelegate.Broadcast(FRigElementKey(ERigElementType::All), NAME_None);
	}

	EnsureCacheValidity();

	bIsCopyingHierarchy = false;
	Notify(ERigHierarchyNotification::HierarchyCopied, {});

	if(!PreviousSelection.IsEmpty())
	{
		if(URigHierarchyController* Controller = GetController())
		{
			Controller->SetHierarchySelection(PreviousSelection, false);
		}
	}
}

uint32 URigHierarchy::GetNameHash() const
{
	UE::TScopeLock Lock(ElementsLock);
	
	uint32 Hash = GetTypeHash(GetTopologyVersion());
	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		const FRigBaseElement* Element = Elements[ElementIndex];
		Hash = HashCombine(Hash, GetTypeHash(Element->GetFName()));
	}
	return Hash;	
}

uint32 URigHierarchy::GetTopologyHash(bool bIncludeTopologyVersion, bool bIncludeTransientControls) const
{
	UE::TScopeLock Lock(ElementsLock);
	
	uint32 Hash = bIncludeTopologyVersion ? TopologyVersion : 0;

	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		const FRigBaseElement* Element = Elements[ElementIndex];
		
		// skip transient controls
		if(!bIncludeTransientControls)
		{
			if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
			{
				if(ControlElement->Settings.bIsTransientControl)
				{
					continue;
				}
			}
		}
		
		Hash = HashCombine(Hash, GetTypeHash(Element->GetKey()));

		if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(Element))
		{
			if(SingleParentElement->ParentElement)
			{
				Hash = HashCombine(Hash, GetTypeHash(SingleParentElement->ParentElement->GetKey()));
			}
		}
		if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(Element))
		{
			for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
			{
				Hash = HashCombine(Hash, GetTypeHash(ParentConstraint.ParentElement->GetKey()));
			}
		}
		if(const FRigBoneElement* BoneElement = Cast<FRigBoneElement>(Element))
		{
			Hash = HashCombine(Hash, GetTypeHash(BoneElement->BoneType));
		}
		if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			Hash = HashCombine(Hash, GetTypeHash(ControlElement->Settings));
		}
		if(const FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Element))
		{
			Hash = HashCombine(Hash, GetTypeHash(ConnectorElement->Settings));
		}
	}

	for(int32 ComponentIndex = 0; ComponentIndex < ElementComponents.Num(); ComponentIndex++)
	{
		if(const FRigBaseComponent* Component = GetComponent(ComponentIndex))
		{
			Hash = HashCombine(Hash, GetTypeHash(Component->GetScriptStruct()));
			Hash = HashCombine(Hash, GetTypeHash(Component->GetIndexInHierarchy()));
			Hash = HashCombine(Hash, GetTypeHash(Component->GetIndexInElement()));
			if(Component->GetElement())
			{
				Hash = HashCombine(Hash, GetTypeHash(Component->GetElement()->GetIndex()));
			}
		}
	}

	return Hash;
}

#if WITH_EDITOR

bool URigHierarchy::HasOnlyUniqueShortNames(ERigElementType InElementType) const
{
	if(!NonUniqueShortNamesCache.IsValid(TopologyVersion))
	{
		TSet<FRigElementKey> NonUniqueShortNames;
		int32 MaxNum = 0;
		for(int32 FlatIndex = 0; FlatIndex < ElementsPerType.Num(); FlatIndex++)
		{
			MaxNum = FMath::Max(MaxNum, ElementsPerType[FlatIndex].Num());
		}

		TSet<FName> UniqueNames;
		UniqueNames.Reserve(MaxNum);

		for(int32 FlatIndex = 0; FlatIndex < ElementsPerType.Num(); FlatIndex++)
		{
			UniqueNames.Reset();
			
			for(const FRigBaseElement* Element : ElementsPerType[FlatIndex])
			{
				FName Name = Element->GetFName();
				const FRigHierarchyModulePath Path(Name);
				if(Path.IsValid())
				{
					Name = Path.GetElementFName();
				}
				if(UniqueNames.Contains(Name))
				{
					NonUniqueShortNames.Add({Name, Element->GetType()});
				}
				UniqueNames.Add(Name);
			}
		}

		NonUniqueShortNamesCache.Set(NonUniqueShortNames, TopologyVersion);
	}

	// if we don't have any non-unique short names - exit early
	if(NonUniqueShortNamesCache.Get().IsEmpty())
	{
		return true;
	}

	// if there's any non-unique short names and we queried for all element types - exit early
	if(InElementType == ERigElementType::All)
	{
		return false;
	}

	// if we find any non-unique short name for the given element type...
	const uint8 InElementTypeAsUInt8 = static_cast<uint8>(InElementType);
	for(const FRigElementKey& Key : NonUniqueShortNamesCache.Get())
	{
		if ((InElementTypeAsUInt8 & static_cast<uint8>(Key.Type)) != 0)
		{
			return false;
		}
	}

	return true;
}

bool URigHierarchy::HasUniqueShortName(ERigElementType InElementType, const FRigName& InName) const
{
	FName Name = InName.GetFName();
	if(Name.IsNone())
	{
		return true;
	}
	
	if(HasOnlyUniqueShortNames(ERigElementType::All))
	{
		return true;
	}

	const FRigHierarchyModulePath Path(InName.ToString());
	if(Path.IsValid())
	{
		Name = Path.GetElementFName(); 
	}

	return !NonUniqueShortNamesCache.Get().Contains({Name, InElementType});
}

bool URigHierarchy::HasUniqueShortName(ERigElementType InElementType, const FString& InName) const
{
	if(InName.IsEmpty())
	{
		return true;
	}
	return HasUniqueShortName(InElementType, FRigName(InName));
}

bool URigHierarchy::HasUniqueShortName(const FRigBaseElement* InElement) const
{
	return HasUniqueShortName(InElement->GetType(), FRigName(InElement->GetFName()));
}

void URigHierarchy::RegisterListeningHierarchy(URigHierarchy* InHierarchy)
{
	if (InHierarchy)
	{
		bool bFoundListener = false;
		for(int32 ListenerIndex = ListeningHierarchies.Num() - 1; ListenerIndex >= 0; ListenerIndex--)
		{
			const URigHierarchy::FRigHierarchyListener& Listener = ListeningHierarchies[ListenerIndex];
			if(Listener.Hierarchy.IsValid())
			{
				if(Listener.Hierarchy.Get() == InHierarchy)
				{
					bFoundListener = true;
					break;
				}
			}
		}

		if(!bFoundListener)
		{
			URigHierarchy::FRigHierarchyListener Listener;
			Listener.Hierarchy = InHierarchy; 
			ListeningHierarchies.Add(Listener);
		}
	}
}

void URigHierarchy::UnregisterListeningHierarchy(URigHierarchy* InHierarchy)
{
	if (InHierarchy)
	{
		for(int32 ListenerIndex = ListeningHierarchies.Num() - 1; ListenerIndex >= 0; ListenerIndex--)
		{
			const URigHierarchy::FRigHierarchyListener& Listener = ListeningHierarchies[ListenerIndex];
			if(Listener.Hierarchy.IsValid())
			{
				if(Listener.Hierarchy.Get() == InHierarchy)
				{
					ListeningHierarchies.RemoveAt(ListenerIndex);
				}
			}
		}
	}
}

void URigHierarchy::ClearListeningHierarchy()
{
	ListeningHierarchies.Reset();
}

void URigHierarchy::CleanupListeningHierarchies()
{
	ListeningHierarchies = ListeningHierarchies.FilterByPredicate([](const URigHierarchy::FRigHierarchyListener& Listener)
		{
			return Listener.Hierarchy.IsValid() && 
					!Listener.Hierarchy->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed | RF_MirroredGarbage);
		});
}
#endif

void URigHierarchy::CopyPose(URigHierarchy* InHierarchy, bool bCurrent, bool bInitial, bool bWeights, bool bMatchPoseInGlobalIfNeeded)
{
	check(InHierarchy);
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	// if we need to copy the weights but the hierarchies are topologically
	// different we need to reset the topology. this is expensive and should
	// only happen during construction of the hierarchy itself.
	if(bWeights && (GetTopologyVersion() != InHierarchy->GetTopologyVersion()))
	{
		CopyHierarchy(InHierarchy);
	}

	const bool bPerformTopologyCheck = GetTopologyVersion() != InHierarchy->GetTopologyVersion();
	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		FRigBaseElement* Element = Elements[ElementIndex];
		if(FRigBaseElement* OtherElement = InHierarchy->Find(Element->GetKey()))
		{
			Element->CopyPose(OtherElement, bCurrent, bInitial, bWeights);
			IncrementPoseVersion(Element->Index);

			// if the topologies don't match and we are supposed to match
			// elements in global space...
			if(bMatchPoseInGlobalIfNeeded && bPerformTopologyCheck)
			{
				FRigMultiParentElement* MultiParentElementA = Cast<FRigMultiParentElement>(Element);
				FRigMultiParentElement* MultiParentElementB = Cast<FRigMultiParentElement>(OtherElement);
				if(MultiParentElementA && MultiParentElementB)
				{
					if(MultiParentElementA->ParentConstraints.Num() != MultiParentElementB->ParentConstraints.Num())
					{
						FRigControlElement* ControlElementA = Cast<FRigControlElement>(Element);
						FRigControlElement* ControlElementB = Cast<FRigControlElement>(OtherElement);
						if(ControlElementA && ControlElementB)
						{
							if(bCurrent)
							{
								ControlElementA->GetOffsetTransform().Set(ERigTransformType::CurrentGlobal, InHierarchy->GetControlOffsetTransform(ControlElementB, ERigTransformType::CurrentGlobal));
								ControlElementA->GetOffsetDirtyState().MarkClean(ERigTransformType::CurrentGlobal);
								ControlElementA->GetOffsetDirtyState().MarkDirty(ERigTransformType::CurrentLocal);
								ControlElementA->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
								ControlElementA->GetShapeDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
								IncrementPoseVersion(ControlElementA->Index);
							}
							if(bInitial)
							{
								ControlElementA->GetOffsetTransform().Set(ERigTransformType::InitialGlobal, InHierarchy->GetControlOffsetTransform(ControlElementB, ERigTransformType::InitialGlobal));
								ControlElementA->GetOffsetDirtyState().MarkClean(ERigTransformType::InitialGlobal);
								ControlElementA->GetOffsetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
								ControlElementA->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
								ControlElementA->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
								IncrementPoseVersion(ControlElementA->Index);
							}
						}
						else
						{
							if(bCurrent)
							{
								MultiParentElementA->GetTransform().Set(ERigTransformType::CurrentGlobal, InHierarchy->GetTransform(MultiParentElementB, ERigTransformType::CurrentGlobal));
								MultiParentElementA->GetDirtyState().MarkClean(ERigTransformType::CurrentGlobal);
								MultiParentElementA->GetDirtyState().MarkDirty(ERigTransformType::CurrentLocal);
								IncrementPoseVersion(MultiParentElementA->Index);
							}
							if(bInitial)
							{
								MultiParentElementA->GetTransform().Set(ERigTransformType::InitialGlobal, InHierarchy->GetTransform(MultiParentElementB, ERigTransformType::InitialGlobal));
								MultiParentElementA->GetDirtyState().MarkClean(ERigTransformType::InitialGlobal);
								MultiParentElementA->GetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
								IncrementPoseVersion(MultiParentElementA->Index);
							}
						}
					}
				}
			}
		}
	}

	EnsureCacheValidity();
}

void URigHierarchy::UpdateReferences(const FRigVMExecuteContext* InContext)
{
	check(InContext);
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	constexpr int32 RefIndex = ElementToFlatIndex<FRigReferenceElement>(); 
	for (FRigBaseElement* Element : ElementsPerType.IsValidIndex(RefIndex) ? ElementsPerType[RefIndex] : Elements)
	{
		if(FRigReferenceElement* Reference = Cast<FRigReferenceElement>(Element))
		{
			const FTransform InitialWorldTransform = Reference->GetReferenceWorldTransform(InContext, true);
			const FTransform CurrentWorldTransform = Reference->GetReferenceWorldTransform(InContext, false);

			const FTransform InitialGlobalTransform = InitialWorldTransform.GetRelativeTransform(InContext->GetToWorldSpaceTransform());
			const FTransform CurrentGlobalTransform = CurrentWorldTransform.GetRelativeTransform(InContext->GetToWorldSpaceTransform());

			const FTransform InitialParentTransform = GetParentTransform(Reference, ERigTransformType::InitialGlobal); 
			const FTransform CurrentParentTransform = GetParentTransform(Reference, ERigTransformType::CurrentGlobal);

			const FTransform InitialLocalTransform = InitialGlobalTransform.GetRelativeTransform(InitialParentTransform);
			const FTransform CurrentLocalTransform = CurrentGlobalTransform.GetRelativeTransform(CurrentParentTransform);

			SetTransform(Reference, InitialLocalTransform, ERigTransformType::InitialLocal, true, false);
			SetTransform(Reference, CurrentLocalTransform, ERigTransformType::CurrentLocal, true, false);
		}
	}
}

void URigHierarchy::ResetPoseToInitial(ERigElementType InTypeFilter)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	bool bPerformFiltering = InTypeFilter != ERigElementType::All;
	
	UE::TScopeLock Lock(ElementsLock);
	const TGuardValue<bool> DisableRecordingOfCurveChanges(bRecordCurveChanges, false);
	
	// if we are resetting the pose on some elements, we need to check if
	// any of affected elements has any children that would not be affected
	// by resetting the pose. if all children are affected we can use the
	// fast path.
	if(bPerformFiltering)
	{
		const int32 Hash = HashCombine(GetTopologyVersion(), (int32)InTypeFilter);
		if(Hash != ResetPoseHash)
		{
			ResetPoseIsFilteredOut.Reset();
			ElementsToRetainLocalTransform.Reset();
			ResetPoseHash = Hash;

			// let's look at all elements and mark all parent of unaffected children
			ResetPoseIsFilteredOut.AddZeroed(Elements.Num());

			Traverse([this, InTypeFilter](FRigBaseElement* InElement, bool& bContinue)
			{
				bContinue = true;
				ResetPoseIsFilteredOut[InElement->GetIndex()] = !InElement->IsTypeOf(InTypeFilter);

				// make sure to distribute the filtering options from
				// the parent to the children of the part of the tree
				const FRigBaseElementParentArray Parents = GetParents(InElement);
				for(const FRigBaseElement* Parent : Parents)
				{
					if(!ResetPoseIsFilteredOut[Parent->GetIndex()])
					{
						if(InElement->IsA<FRigNullElement>() || InElement->IsA<FRigControlElement>())
						{
							ElementsToRetainLocalTransform.Add(InElement->GetIndex());
						}
						else
						{
							ResetPoseIsFilteredOut[InElement->GetIndex()] = false;
						}
					}
				}
			});
		}

		// if the per element state is empty
		// it means that the filter doesn't affect 
		if(ResetPoseIsFilteredOut.IsEmpty())
		{
			bPerformFiltering = false;
		}
	}

	if(bPerformFiltering)
	{
		for(const int32 ElementIndex : ElementsToRetainLocalTransform)
		{
			if(FRigTransformElement* TransformElement = Get<FRigTransformElement>(ElementIndex))
			{
				// compute the local value if necessary
				GetTransform(TransformElement, ERigTransformType::CurrentLocal);

				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
				{
					// compute the local offset if necessary
					GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentLocal);
					GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
				}

				PropagateDirtyFlags(TransformElement, false, true, true, false);
			}
		}
		
		for(const int32 ElementIndex : ElementsToRetainLocalTransform)
		{
			if(FRigTransformElement* TransformElement = Get<FRigTransformElement>(ElementIndex))
			{
				if(TransformElement->GetDirtyState().IsDirty(ERigTransformType::CurrentGlobal))
				{
					continue;
				}
				
				TransformElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);

				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
				{
					ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
					ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
				}

				PropagateDirtyFlags(TransformElement, false, true, false, true);
			}
		}
	}

	for(int32 ElementIndex=0; ElementIndex<Elements.Num(); ElementIndex++)
	{
		if(!ResetPoseIsFilteredOut.IsEmpty() && bPerformFiltering)
		{
			if(ResetPoseIsFilteredOut[ElementIndex])
			{
				continue;
			}
		}

		// reset the weights to the initial values as well
		if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(Elements[ElementIndex]))
		{
			for(FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
			{
				ParentConstraint.Weight = ParentConstraint.InitialWeight;
			}
		}

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[ElementIndex]))
		{
			ControlElement->GetOffsetTransform().Current = ControlElement->GetOffsetTransform().Initial;
			ControlElement->GetOffsetDirtyState().Current = ControlElement->GetOffsetDirtyState().Initial;
			ControlElement->GetShapeTransform().Current = ControlElement->GetShapeTransform().Initial;
			ControlElement->GetShapeDirtyState().Current = ControlElement->GetShapeDirtyState().Initial;
			ControlElement->PreferredEulerAngles.Current = ControlElement->PreferredEulerAngles.Initial;
		}

		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Elements[ElementIndex]))
		{
			TransformElement->GetTransform().Current = TransformElement->GetTransform().Initial;
			TransformElement->GetDirtyState().Current = TransformElement->GetDirtyState().Initial;
		}
	}
	
	EnsureCacheValidity();
}

void URigHierarchy::ResetCurveValues()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	const TGuardValue<bool> DisableRecordingOfCurveChanges(bRecordCurveChanges, false);

	TArray<FRigBaseElement*> Curves = GetCurvesFast();
	for(FRigBaseElement* Element : Curves)
	{
		if(FRigCurveElement* CurveElement = CastChecked<FRigCurveElement>(Element))
		{
			SetCurveValue(CurveElement, 0.f);
		}
	}
}

void URigHierarchy::UnsetCurveValues(bool bSetupUndo)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	TArray<FRigBaseElement*> Curves = GetCurvesFast();
	for(FRigBaseElement* Element : Curves)
	{
		if(FRigCurveElement* CurveElement = CastChecked<FRigCurveElement>(Element))
		{
			UnsetCurveValue(CurveElement, bSetupUndo);
		}
	}
	ResetChangedCurveIndices();
}

const TArray<int32>& URigHierarchy::GetChangedCurveIndices() const
{
	return ChangedCurveIndices;
}

void URigHierarchy::ResetChangedCurveIndices()
{
	ChangedCurveIndices.Reset();
}

int32 URigHierarchy::Num(ERigElementType InElementType) const
{
	const int32 FlatIndex = RigElementTypeToFlatIndex(InElementType);
	if(ElementsPerType.IsValidIndex(FlatIndex))
	{
		return ElementsPerType[FlatIndex].Num();
	}
	return 0;
}

bool URigHierarchy::IsProcedural(const FRigElementKey& InKey) const
{
	return IsProcedural(Find(InKey));
}

bool URigHierarchy::IsProcedural(const FRigBaseElement* InElement) const
{
	if(InElement == nullptr)
	{
		return false;
	}
	return InElement->IsProcedural();
}

bool URigHierarchy::IsProcedural(const FRigComponentKey& InKey) const
{
	return IsProcedural(FindComponent(InKey));
}

bool URigHierarchy::IsProcedural(const FRigBaseComponent* InComponent) const
{
	if(InComponent == nullptr)
	{
		return false;
	}
	return InComponent->IsProcedural();
}

bool URigHierarchy::IsProcedural(const FRigHierarchyKey& InKey) const
{
	if(InKey.IsElement())
	{
		return IsProcedural(InKey.GetElement());
	}
	if(InKey.IsComponent())
	{
		return IsProcedural(InKey.GetComponent());
	}
	return false;
}

int32 URigHierarchy::GetIndex(const FRigElementKey& InKey, bool bFollowRedirector) const
{
	if(ElementKeyRedirector && bFollowRedirector)
	{
		if(FRigElementKeyRedirector::FCachedKeyArray* CachedRigElements = ElementKeyRedirector->Find(InKey))
		{
			if(CachedRigElements->Num() == 1)
			{
				if((*CachedRigElements)[0].UpdateCache(this))
				{
					return (*CachedRigElements)[0].GetIndex();
				}
			}
			return INDEX_NONE;
		}
	}
	
	if(const int32* Index = ElementIndexLookup.Find(InKey))
	{
		return *Index;
	}

	const FRigElementKey PatchedKey = PatchElementKeyInLookup(InKey);
	if(PatchedKey != InKey)
	{
#if WITH_EDITOR
		if(!IsLoading() && !bSuspendNameSpaceKeyWarnings)
		{
			const int32 NumReceivedNameSpaceBasedKeys = ReceivedNameSpaceBasedKeys.Num(); 
			if(ReceivedNameSpaceBasedKeys.AddUnique(InKey) == NumReceivedNameSpaceBasedKeys)
			{
				if(!IsRunningCookCommandlet() && !IsRunningCookOnTheFly())
				{
					UE_LOGF(
						LogControlRig,
						Warning,
						"%ls: Element '%ls' has been accessed using a namespace based key ('%ls'). Please consider updating your code.",
						*GetPathName(),
						*PatchedKey.ToString(),
						*InKey.ToString()
					);
				}
			}
		}
#endif
		return GetIndex(PatchedKey);
	}
	return INDEX_NONE;
}

int32 URigHierarchy::GetSpawnIndex(const FRigHierarchyKey& InKey) const
{
	if(InKey.IsComponent())
	{
		if(const FRigBaseComponent* Component = FindComponent(InKey.GetComponent()))
		{
			return Component->GetSpawnIndex();
		}
	}
	else if(const FRigBaseElement* Element = Find(InKey.GetElement()))
	{
		return Element->GetSpawnIndex();
	}
	return INDEX_NONE;
}

const FRigBaseComponent* URigHierarchy::FindComponent(const FRigComponentKey& InKey) const
{
	const int32 ComponentIndex = GetComponentIndex(InKey);
	if(ComponentIndex != INDEX_NONE)
	{
		return GetComponent(ComponentIndex);
	}
	return nullptr;
}

FRigBaseComponent* URigHierarchy::FindComponent(const FRigComponentKey& InKey)
{
	const int32 ComponentIndex = GetComponentIndex(InKey);
	if(ComponentIndex != INDEX_NONE)
	{
		return GetComponent(ComponentIndex);
	}
	return nullptr;
}

TArray<const FRigBaseComponent*> URigHierarchy::GetComponents(const UScriptStruct* InComponentStruct) const
{
	check(InComponentStruct);
	check(InComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()));
	
	TArray<const FRigBaseComponent*> Result;
	for(const FInstancedStruct& ComponentInstancedStruct : ElementComponents)
	{
		if(ComponentInstancedStruct.GetScriptStruct() == InComponentStruct)
		{
			Result.Add(reinterpret_cast<const FRigBaseComponent*>(ComponentInstancedStruct.GetMemory()));
		}
	}
	return Result;
}

int32 URigHierarchy::GetComponentIndex(const FRigComponentKey& InComponentKey, bool bFollowRedirector) const
{
	if(bFollowRedirector)
	{
		if(ElementKeyRedirector == nullptr)
		{
			bFollowRedirector = false;
		}
	}

	if(!bFollowRedirector)
	{
		if(!ComponentIndexLookup.Contains(InComponentKey))
		{
			bFollowRedirector = true;
		}
	}
	
	if(bFollowRedirector)
	{
		const int32 ElementIndex = GetIndex(InComponentKey.ElementKey);
		if(ElementIndex != INDEX_NONE)
		{
			const FRigElementKey& RedirectedKey = Elements[ElementIndex]->GetKey(); 
			if(RedirectedKey != InComponentKey.ElementKey)
			{
				const FRigComponentKey RedirectedComponentKey(RedirectedKey, InComponentKey.Name);
				return GetComponentIndex(RedirectedComponentKey, false);
			}
		}
	}
	
	if(const int32* IndexPtr = ComponentIndexLookup.Find(InComponentKey))
	{
		check(ElementComponents.IsValidIndex(*IndexPtr));
		check(ElementComponents[*IndexPtr].IsValid());
		return (*IndexPtr);
	}
	return INDEX_NONE;
}

const FRigBaseComponent* URigHierarchy::GetComponent(int32 InIndex) const
{
	if(ElementComponents.IsValidIndex(InIndex))
	{
		if(ElementComponents[InIndex].IsValid())
		{
			return ElementComponents[InIndex].GetPtr<FRigBaseComponent>();
		}
	}
	return nullptr;
}

FRigBaseComponent* URigHierarchy::GetComponent(int32 InIndex)
{
	if(ElementComponents.IsValidIndex(InIndex))
	{
		if(ElementComponents[InIndex].IsValid())
		{
			return ElementComponents[InIndex].GetMutablePtr<FRigBaseComponent>();
		}
	}
	return nullptr;
}

int32 URigHierarchy::NumComponents() const
{
	return Algo::CountIf(ElementComponents, [](const FInstancedStruct& InComponentInstancedStruct)
	{
		return InComponentInstancedStruct.IsValid();
	});
}

int32 URigHierarchy::NumComponents(const UScriptStruct* InComponentStruct) const
{
	check(InComponentStruct);
	check(InComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()));

	return Algo::CountIf(ElementComponents, [InComponentStruct](const FInstancedStruct& InComponentInstancedStruct)
	{
		return InComponentInstancedStruct.GetScriptStruct() == InComponentStruct;
	});
}

int32 URigHierarchy::NumComponents(FRigElementKey InElement) const
{
	if(const FRigBaseElement* Element = Find(InElement))
	{
		return Element->NumComponents();
	}
	return 0;
}

TArray<FRigComponentKey> URigHierarchy::GetAllComponentKeys() const
{
	TArray<FRigComponentKey> Keys;
	ComponentIndexLookup.GenerateKeyArray(Keys);
	return Keys;
}

TArray<FRigComponentKey> URigHierarchy::GetComponentKeys(FRigElementKey InElement) const
{
	if(const FRigBaseElement* Element = Find(InElement))
	{
		return Element->GetComponentKeys();
	}
	return {};
}

FRigComponentKey URigHierarchy::GetComponentKey(FRigElementKey InElement, int32 InComponentIndex) const
{
	if(const FRigBaseElement* Element = Find(InElement))
	{
		if(const FRigBaseComponent* Component = Element->GetComponent(InComponentIndex))
		{
			return Component->GetKey();
		}
	}
	return FRigComponentKey();
}

FName URigHierarchy::GetComponentName(FRigElementKey InElement, int32 InComponentIndex) const
{
	if(const FRigBaseElement* Element = Find(InElement))
	{
		if(const FRigBaseComponent* Component = Element->GetComponent(InComponentIndex))
		{
			return Component->GetFName();
		}
	}
	return NAME_None;
}

UScriptStruct* URigHierarchy::GetComponentType(FRigElementKey InElement, int32 InComponentIndex) const
{
	if(const FRigBaseElement* Element = Find(InElement))
	{
		if(const FRigBaseComponent* Component = Element->GetComponent(InComponentIndex))
		{
			return Component->GetScriptStruct();
		}
	}
	return nullptr;
}

FString URigHierarchy::GetComponentContent(FRigElementKey InElement, int32 InComponentIndex) const
{
	if(const FRigBaseElement* Element = Find(InElement))
	{
		if(const FRigBaseComponent* Component = Element->GetComponent(InComponentIndex))
		{
			return Component->GetContentAsText();
		}
	}
	return FString();
}

bool URigHierarchy::CanAddComponent(FRigElementKey InElementKey, const UScriptStruct* InComponentStruct, FString* OutFailureReason) const
{
	check(InComponentStruct);

	if(!InElementKey.IsValid())
	{
		if(OutFailureReason)
		{
			*OutFailureReason = FString::Printf(TEXT("InElementKey %s is not valid."), *InElementKey.ToString());
		}
		return false;
	}

	if(!InComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
	{
		if(OutFailureReason)
		{
			*OutFailureReason = FString::Printf(TEXT("The provided structure '%s' is not a component."), *InComponentStruct->GetName());
		}
		return false;
	}

	const FRigBaseElement* Element = Find(InElementKey);
	if(Element == nullptr)
	{
		if(OutFailureReason)
		{
			*OutFailureReason = FString::Printf(TEXT("The element %s cannot be found."), *InElementKey.ToString());
		}
		return false;
	}

	const FStructOnScope StructOnScope(InComponentStruct);
	if(const FRigBaseComponent* StructMemory = reinterpret_cast<const FRigBaseComponent*>(StructOnScope.GetStructMemory()))
	{
		if(StructMemory->IsSingleton())
		{
			if(Element)
			{
				for(int32 Index = 0; Index < Element->NumComponents(); Index++)
				{
					if(const FRigBaseComponent* ExistingComponent = Element->GetComponent(Index))
					{
						if(ExistingComponent->GetScriptStruct() == InComponentStruct)
						{
							if(OutFailureReason)
							{
								*OutFailureReason = FString::Printf(TEXT("A component of type '%s' already exists on element '%s'."), *InComponentStruct->GetName(), *InElementKey.ToString());
							}
							return false;
						}
					}
				}
			}
		}
		if(!StructMemory->CanBeAddedTo(InElementKey, this, OutFailureReason))
		{
			return false;
		}
	}
	return true;
}

bool URigHierarchy::CanAddComponent(FRigElementKey InElementKey, const FRigBaseComponent* InComponent, FString* OutFailureReason) const
{
	if(!CanAddComponent(InElementKey, InComponent->GetScriptStruct(), OutFailureReason))
	{
		return false;
	}
	if(InComponent->GetElementKey() == InElementKey)
	{
		if(OutFailureReason)
		{
			*OutFailureReason = TEXT("Component is already under target element.");
		}
		return false;
	}
	return true;
}

int32 URigHierarchy::GetNextSpawnIndex() const
{
	return Num() + NumComponents();
}

TArray<FRigControlElement*> URigHierarchy::GetTransientControls() const
{
	TArray<FRigControlElement*> TransientControls;
	
	constexpr int32 ControlsFlat = ElementToFlatIndex<FRigControlElement>();
	for (FRigBaseElement* Element: ElementsPerType.IsValidIndex(ControlsFlat) ? ElementsPerType[ControlsFlat] : Elements)
	{
		FRigControlElement* ControlElement = Cast<FRigControlElement>(Element);
		if (ControlElement && ControlElement->Settings.bIsTransientControl)
		{
			TransientControls.Add(ControlElement);
		}
	}
		
	return TransientControls;
}

TArray<FRigSocketState> URigHierarchy::GetSocketStates() const
{
	const TArray<FRigElementKey> Keys = GetSocketKeys(true);
	TArray<FRigSocketState> States;
	States.Reserve(Keys.Num());
	for(const FRigElementKey& Key : Keys)
	{
		const FRigSocketElement* Socket = FindChecked<FRigSocketElement>(Key);
		if(!Socket->IsProcedural())
		{
			States.Add(Socket->GetSocketState(this));
		}
	}
	return States;
}

TArray<FRigElementKey> URigHierarchy::RestoreSocketsFromStates(TArray<FRigSocketState> InStates, bool bSetupUndoRedo)
{
	TArray<FRigElementKey> Keys;
	for(const FRigSocketState& State : InStates)
	{
		FRigElementKey Key(State.Name, ERigElementType::Socket);

		if(FRigSocketElement* Socket = Find<FRigSocketElement>(Key))
		{
			(void)GetController()->SetParent(Key, State.Parent);
			Socket->SetColor(State.Color, this);
			Socket->SetDescription(State.Description, this);
			SetInitialLocalTransform(Key, State.InitialLocalTransform);
			SetLocalTransform(Key, State.InitialLocalTransform);
		}
		else
		{
			Key = GetController()->AddSocket(State.Name, State.Parent, State.InitialLocalTransform, false, State.Color, State.Description, bSetupUndoRedo, false);
		}

		Keys.Add(Key);
	}
	return Keys;
}

TArray<FRigConnectorState> URigHierarchy::GetConnectorStates() const
{
	const TArray<FRigElementKey> Keys = GetConnectorKeys(true);
	TArray<FRigConnectorState> States;
	States.Reserve(Keys.Num());
	for(const FRigElementKey& Key : Keys)
	{
		const FRigConnectorElement* Connector = FindChecked<FRigConnectorElement>(Key);
		if(!Connector->IsProcedural())
		{
			States.Add(Connector->GetConnectorState(this));
		}
	}
	return States;
}

TArray<FRigElementKey> URigHierarchy::RestoreConnectorsFromStates(TArray<FRigConnectorState> InStates, bool bSetupUndoRedo)
{
	TArray<FRigElementKey> Keys;
	for(const FRigConnectorState& State : InStates)
	{
		FRigElementKey Key(State.Name, ERigElementType::Connector);

		if(const FRigConnectorElement* Connector = Find<FRigConnectorElement>(Key))
		{
			SetConnectorSettings(Key, State.Settings, bSetupUndoRedo, false, false);
		}
		else
		{
			Key = GetController()->AddConnector(State.Name, State.Settings, bSetupUndoRedo, false);
		}

		Keys.Add(Key);
	}
	return Keys;
}

TArray<FName> URigHierarchy::GetMetadataNames(FRigElementKey InItem) const
{
	TArray<FName> Names;
	if (const FRigBaseElement* Element = Find(InItem))
	{
		if (ElementMetadata.IsValidIndex(Element->MetadataStorageIndex))
		{
			ElementMetadata[Element->MetadataStorageIndex].MetadataMap.GetKeys(Names);
		}
	}
	return Names;
}

ERigMetadataType URigHierarchy::GetMetadataType(FRigElementKey InItem, FName InMetadataName) const
{
	if (const FRigBaseElement* Element = Find(InItem))
	{
		if (Element->MetadataStorageIndex != INDEX_NONE)
		{
			if (const FRigBaseMetadata* const* MetadataPtrPtr = ElementMetadata[Element->MetadataStorageIndex].MetadataMap.Find(InMetadataName))
			{
				return (*MetadataPtrPtr)->GetType();
			}
		}
	}
	
	return ERigMetadataType::Invalid;
}

bool URigHierarchy::RemoveMetadata(FRigElementKey InItem, FName InMetadataName)
{
	return RemoveMetadataForElement(Find(InItem), InMetadataName);
}

bool URigHierarchy::RemoveAllMetadata(FRigElementKey InItem)
{
	return RemoveAllMetadataForElement(Find(InItem));
}

FName URigHierarchy::GetModulePathFName(FRigElementKey InItem) const
{
	return GetModuleFName(InItem);
}

FString URigHierarchy::GetModulePath(FRigElementKey InItem) const
{
	return GetModuleName(InItem);
}

FName URigHierarchy::GetModuleFName(FRigElementKey InItem) const
{
	if(!InItem.IsValid())
	{
		return NAME_None;
	}
	
	const FName Result = GetNameMetadata(InItem, ModuleMetadataName, NAME_None);
	if(!Result.IsNone())
	{
		return Result;
	}

	// fall back on the name of the item
	const FRigHierarchyModulePath ModulePath = InItem.Name.ToString();
	FString ModuleName;
	if(ModulePath.Split(&ModuleName, nullptr))
	{
		return *ModuleName;
	}

	return NAME_None;
}

FString URigHierarchy::GetModuleName(FRigElementKey InItem) const
{
	const FName ModulePathName = GetModuleFName(InItem);
	if(!ModulePathName.IsNone())
	{
		return ModulePathName.ToString();
	}
	return FString();
}

FString URigHierarchy::GetModulePrefix(FRigElementKey InItem) const
{
	return GetModuleName(InItem) + FRigHierarchyModulePath::ModuleNameSuffix;
}

FName URigHierarchy::GetNameSpaceFName(FRigElementKey InItem) const
{
	const FString NameSpaceString = GetNameSpace(InItem);
	if(NameSpaceString.IsEmpty())
	{
		return NAME_None;
	}
	return *NameSpaceString;
}

FString URigHierarchy::GetNameSpace(FRigElementKey InItem) const
{
	if(!InItem.IsValid())
	{
		return FString();
	}
	
	// fall back on the name of the item
	const FString NameString = GetModuleName(InItem);
	return NameString + FRigHierarchyModulePath::NamespaceSeparator_Deprecated;
}

TArray<const FRigBaseElement*> URigHierarchy::GetSelectedElements(ERigElementType InTypeFilter) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	TArray<const FRigBaseElement*> Selection;

	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		TArray<const FRigBaseElement*> SelectedElements = HierarchyForSelection->GetSelectedElements(InTypeFilter);
		for(const FRigBaseElement* SelectedElement : SelectedElements)
		{
			if(const FRigBaseElement* Element = Find(SelectedElement->GetKey()))
			{
				Selection.Add(Element);
			}
		}
		return Selection;
	}

	for (const FRigHierarchyKey& SelectedKey : OrderedSelection)
	{
		if(SelectedKey.IsElement())
		{
			if(SelectedKey.GetElement().IsTypeOf(InTypeFilter))
			{
				if(const FRigBaseElement* Element = FindChecked(SelectedKey.GetElement()))
				{
					ensure(Element->IsSelected());
					Selection.Add(Element);
				}
			}
		}
	}
	return Selection;
}

TArray<const FRigBaseComponent*> URigHierarchy::GetSelectedComponents() const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	TArray<const FRigBaseComponent*> Selection;

	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		TArray<const FRigBaseComponent*> SelectedComponents = HierarchyForSelection->GetSelectedComponents();
		for(const FRigBaseComponent* SelectedComponent : SelectedComponents)
		{
			if(const FRigBaseComponent* Component = FindComponent(SelectedComponent->GetKey()))
			{
				Selection.Add(Component);
			}
		}
		return Selection;
	}

	for (const FRigHierarchyKey& SelectedKey : OrderedSelection)
	{
		if(SelectedKey.IsComponent())
		{
			if(const FRigBaseComponent* Component = FindComponent(SelectedKey.GetComponent()))
			{
				ensure(Component->IsSelected());
				Selection.Add(Component);
			}
		}
	}
	return Selection;
}

TArray<FRigElementKey> URigHierarchy::GetSelectedKeys(ERigElementType InTypeFilter) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		return HierarchyForSelection->GetSelectedKeys(InTypeFilter);
	}

	TArray<FRigElementKey> Selection;
	for (const FRigHierarchyKey& SelectedKey : OrderedSelection)
	{
		if(SelectedKey.IsElement())
		{
			if(SelectedKey.GetElement().IsTypeOf(InTypeFilter))
			{
				Selection.Add(SelectedKey.GetElement());
			}
		}
	}
	
	return Selection;
}

const TArray<FRigHierarchyKey>& URigHierarchy::GetSelectedHierarchyKeys() const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		return HierarchyForSelection->GetSelectedHierarchyKeys();
	}

	return OrderedSelection;
}

bool URigHierarchy::HasAnythingSelectedByPredicate(TFunctionRef<bool(const FRigElementKey&)> InPredicate) const
{
	if (const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		return HierarchyForSelection->HasAnythingSelectedByPredicate(InPredicate);
	}

	return OrderedSelection.ContainsByPredicate([InPredicate](const FRigHierarchyKey& SelectedKey)
	{
		if(SelectedKey.IsElement())
		{
			return InPredicate(SelectedKey.GetElement());
		}
		return false;
	});
}

TArray<FRigElementKey> URigHierarchy::GetSelectedKeysByPredicate(TFunctionRef<bool(const FRigElementKey&)> InPredicate) const
{
	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		return HierarchyForSelection->GetSelectedKeysByPredicate(InPredicate);
	}

	TArray<FRigElementKey> Result;
	for(const FRigHierarchyKey& Key : OrderedSelection)
	{
		if(Key.IsElement())
		{
			if(InPredicate(Key.GetElement()))
			{
				Result.Add(Key.GetElement());
			}
		}
	}
	return Result;
}

FString URigHierarchy::JoinNameSpace_Deprecated(const FString& InLeft, const FString& InRight)
{
	if(InLeft.EndsWith(FRigHierarchyModulePath::NamespaceSeparator_Deprecated))
	{
		return InLeft + InRight;
	}
	return InLeft + FRigHierarchyModulePath::NamespaceSeparator_Deprecated + InRight;
}

FRigName URigHierarchy::JoinNameSpace_Deprecated(const FRigName& InLeft, const FRigName& InRight)
{
	return FRigName(JoinNameSpace_Deprecated(InLeft.ToString(), InRight.ToString()));
}

TPair<FString, FString> URigHierarchy::SplitNameSpace_Deprecated(const FString& InNameSpacedPath, bool bFromEnd)
{
	TPair<FString, FString> Result;
	(void)SplitNameSpace_Deprecated(InNameSpacedPath, &Result.Key, &Result.Value, bFromEnd);
	return Result;
}

TPair<FRigName, FRigName> URigHierarchy::SplitNameSpace_Deprecated(const FRigName& InNameSpacedPath, bool bFromEnd)
{
	const TPair<FString, FString> Result = SplitNameSpace_Deprecated(InNameSpacedPath.GetName(), bFromEnd);
	return {FRigName(Result.Key), FRigName(Result.Value)};
}

bool URigHierarchy::SplitNameSpace_Deprecated(const FString& InNameSpacedPath, FString* OutNameSpace, FString* OutName, bool bFromEnd)
{
	return InNameSpacedPath.Split(FRigHierarchyModulePath::NamespaceSeparator_Deprecated, OutNameSpace, OutName, ESearchCase::CaseSensitive, bFromEnd ? ESearchDir::FromEnd : ESearchDir::FromStart);
}

bool URigHierarchy::SplitNameSpace_Deprecated(const FRigName& InNameSpacedPath, FRigName* OutNameSpace, FRigName* OutName, bool bFromEnd)
{
	FString NameSpace, Name;
	if(SplitNameSpace_Deprecated(InNameSpacedPath.GetName(), &NameSpace, &Name, bFromEnd))
	{
		if(OutNameSpace)
		{
			OutNameSpace->SetName(NameSpace);
		}
		if(OutName)
		{
			OutName->SetName(Name);
		}
		return true;
	}
	return false;
}

int32 URigHierarchy::SetMaxNameLength(int32 InMaxNameLength)
{
	MaxNameLength = InMaxNameLength + UE_RIGHIERARCHY_NUMERIC_SUFFIX_LEN;

	// ensure the new setting suffices current element names
	for(const FRigBaseElement* Element : Elements)
	{
		MaxNameLength = FMath::Max<int32>(MaxNameLength, Element->GetName().Len());
	}
	
	// ensure the new setting suffices current component names
	for(FInstancedStruct& InstancedStruct : ElementComponents)
	{
		if(!InstancedStruct.IsValid())
		{
			continue;
		}

		FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
		MaxNameLength = FMath::Max<int32>(MaxNameLength, Component->GetName().Len());
	}

	return MaxNameLength;
}

void URigHierarchy::SanitizeName(FRigName& InOutName, bool bAllowNameSpaces, int32 InMaxNameLength)
{
	// Sanitize the name
	FString SanitizedNameString = InOutName.GetName();
	bool bChangedSomething = false;
	for (int32 i = 0; i < SanitizedNameString.Len(); ++i)
	{
		TCHAR& C = SanitizedNameString[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||					 // Any letter
			(C == '_') || (C == '-') || (C == '.') || (C == '|') ||	 // _  - .  | anytime
			(FChar::IsDigit(C)) ||									 // 0-9 anytime
			((i > 0) && (C== ' '));									 // Space after the first character to support virtual bones

		if (!bGoodChar)
		{
			if(bAllowNameSpaces && (C == FRigHierarchyModulePath::NamespaceSeparatorChar_Deprecated || C == FRigHierarchyModulePath::ModuleNameSuffixChar))
			{
				continue;
			}
			
			C = '_';
			bChangedSomething = true;
		}
	}

	if (SanitizedNameString.Len() > InMaxNameLength)
	{
		SanitizedNameString.LeftChopInline(SanitizedNameString.Len() - InMaxNameLength);
		bChangedSomething = true;
	}

	if(bChangedSomething)
	{
		InOutName.SetName(SanitizedNameString);
	}
}

FRigName URigHierarchy::GetSanitizedName(const FRigName& InName, bool bAllowNameSpaces, int32 InMaxNameLength)
{
	FRigName Name = InName;
	SanitizeName(Name, bAllowNameSpaces, InMaxNameLength);
	return Name;
}

bool URigHierarchy::IsNameAvailable(const FRigName& InPotentialNewName, ERigElementType InType, FString* OutErrorMessage) const
{
	// check for fixed keywords
	const FRigElementKey PotentialKey(InPotentialNewName.GetFName(), InType);
	if(PotentialKey == URigHierarchy::GetDefaultParentKey())
	{
		return false;
	}

	if (GetIndex(PotentialKey) != INDEX_NONE)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("Name already used.");
		}
		return false;
	}

	const FRigName UnsanitizedName = InPotentialNewName;
	if (UnsanitizedName.Len() > GetMaxNameLength())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("Name too long.");
		}
		return false;
	}

	if (UnsanitizedName.IsNone())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("None is not a valid name.");
		}
		return false;
	}

	bool bAllowNameSpaces = bAllowNameSpaceWhenSanitizingName;

	// try to find a control rig this belongs to
	const UControlRig* ControlRig = Cast<UControlRig>(GetOuter());
	if(ControlRig == nullptr)
	{
		if(const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			if(const UClass* Class = Blueprint->GeneratedClass)
			{
				ControlRig = Cast<UControlRig>(Class->GetDefaultObject());
			}
		}
	}

	// allow namespaces on default control rigs (non-module and non-modular)
	if(ControlRig)
	{
		if(!ControlRig->IsRigModule() &&
			!ControlRig->GetClass()->IsChildOf(UModularRig::StaticClass()))
		{
			bAllowNameSpaces = true;
		}
	}
	else
	{
		bAllowNameSpaces = true;
	}

	FRigName SanitizedName = UnsanitizedName;
	SanitizeName(SanitizedName, bAllowNameSpaces, GetMaxNameLength());

	if (SanitizedName != UnsanitizedName)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("Name contains invalid characters.");
		}
		return false;
	}

	return true;
}

bool URigHierarchy::IsDisplayNameAvailable(const FRigElementKey& InParentElement,
	const FRigName& InPotentialNewDisplayName, FString* OutErrorMessage) const
{
	if(InParentElement.IsValid())
	{
		const TArray<FRigElementKey> ChildKeys = GetChildren(InParentElement);
		if(ChildKeys.ContainsByPredicate([&InPotentialNewDisplayName, this](const FRigElementKey& InChildKey) -> bool
		{
			if(const FRigBaseElement* BaseElement = Find(InChildKey))
			{
				if(BaseElement->GetDisplayName() == InPotentialNewDisplayName.GetFName())
				{
					return true;
				}
			}
			return false;
		}))
		{
			if (OutErrorMessage)
			{
				*OutErrorMessage = TEXT("Name already used.");
			}
			return false;
		}
	}

	const FRigName UnsanitizedName = InPotentialNewDisplayName;
	if (UnsanitizedName.Len() > GetMaxNameLength())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("Name too long.");
		}
		return false;
	}

	if (UnsanitizedName.IsNone())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("None is not a valid name.");
		}
		return false;
	}

	FRigName SanitizedName = UnsanitizedName;
	SanitizeName(SanitizedName, true, GetMaxNameLength());

	if (SanitizedName != UnsanitizedName)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("Name contains invalid characters.");
		}
		return false;
	}

	return true;
}

bool URigHierarchy::IsComponentNameAvailable(const FRigElementKey& InElementKey, const FRigName& InPotentialNewName, FString* OutErrorMessage) const
{
	const int32 Count = NumComponents(InElementKey);
	for(int32 Index = 0; Index < Count; Index++)
	{
		if(InPotentialNewName.GetFName() == GetComponentName(InElementKey, Index))
		{
			return false;
		}
	}
	return true;
}

FRigName URigHierarchy::GetSafeNewName(const FRigName& InPotentialNewName, ERigElementType InType, bool bAllowNameSpace) const
{
	FRigName SanitizedName = InPotentialNewName;
	SanitizeName(SanitizedName, bAllowNameSpace, GetMaxNameLength());

	bAllowNameSpaceWhenSanitizingName = bAllowNameSpace;
	if(ExecuteContext)
	{
		const FControlRigExecuteContext& CRContext = ExecuteContext->GetPublicData<FControlRigExecuteContext>();
		if(CRContext.IsRigModule())
		{
			FString LastSegmentName;
			const FRigHierarchyModulePath ModulePath = SanitizedName.ToString();
			if(ModulePath.Split(nullptr, &LastSegmentName))
			{
				SanitizedName = LastSegmentName;
			}
			SanitizedName = CRContext.GetRigModulePrefix() + SanitizedName.GetName();
			bAllowNameSpaceWhenSanitizingName = true;
		}
	}

	if (SanitizedName.Len() > GetMaxNameLength() - UE_RIGHIERARCHY_NUMERIC_SUFFIX_LEN)
	{
		SanitizedName.SetName(SanitizedName.GetName().LeftChop(SanitizedName.Len() - (GetMaxNameLength() - UE_RIGHIERARCHY_NUMERIC_SUFFIX_LEN)));
	}

	const FName UniqueName = FRigName(GetUniqueName(*SanitizedName.GetName(), [this, InType](const FName& InName) -> bool
	{
		return IsNameAvailable(InName, InType);
	}));

	bAllowNameSpaceWhenSanitizingName = false;
	return FRigName(UniqueName);
}

FRigName URigHierarchy::GetSafeNewDisplayName(const FRigElementKey& InParentElement, const FRigName& InPotentialNewDisplayName) const
{
	if(InPotentialNewDisplayName.IsNone())
	{
		return FRigName();
	}

	static const int32 ControlElementIndex = RigElementTypeToFlatIndex(ERigElementType::Control); 

	FRigName SanitizedName = InPotentialNewDisplayName;
	SanitizeName(SanitizedName, true, GetMaxNameLength());

	// this can be nullptr - and that's ok.
	const FRigBaseElement* ParentElement = Find(InParentElement);

	TArray<FRigElementKey> KeysToCheck;

	// two paths: if the children are up2date, rely on the previously cached children.
	if(ChildElementCacheTopologyVersion == TopologyVersion)
	{
		if(InParentElement.IsValid())
		{
			const TConstArrayView<FRigBaseElement*> Children = GetChildren(ParentElement);
			for(const FRigBaseElement* ChildElement : Children)
			{
				if(ChildElement->IsA<FRigControlElement>())
				{
					KeysToCheck.Add(ChildElement->GetKey());
				}
			}
		}
		else
		{
			// get all of the root control elements
			for(const FRigBaseElement* Element : ElementsPerType[ControlElementIndex])
			{
				if(GetNumberOfParents(Element) == 0)
				{
					KeysToCheck.Add(Element->GetKey());
				}
			}
		}
	}
	else
	{
		// if we enter here we need to find the children for us. first we'll visit all elements
		// and will check for a name collision there. if there is a collision we can rely on the code
		// path based on the KeysToCheck
		bool bHasNameCollision = false;
		KeysToCheck.Reserve(10);
		
		for(const FRigBaseElement* Element : ElementsPerType[ControlElementIndex])
		{
			if(GetFirstParent(Element) == ParentElement)
			{
				if(CastChecked<FRigControlElement>(Element)->Settings.DisplayName == SanitizedName.GetFName())
				{
					bHasNameCollision = true;
				}
				KeysToCheck.Add(Element->GetKey());
			}
		}

		// early exit - we haven't hit any name collision anywhere
		if(!bHasNameCollision)
		{
			return SanitizedName;
		}
	}

	if (SanitizedName.Len() > GetMaxNameLength() - 4)
	{
		SanitizedName.SetName(SanitizedName.GetName().LeftChop(SanitizedName.Len() - (GetMaxNameLength() - 4)));
	}

	TArray<FString> DisplayNames;
	Algo::Transform(KeysToCheck, DisplayNames, [this](const FRigElementKey& InKey) -> FString
	{
		if(const FRigControlElement* ControlElement = Find<FRigControlElement>(InKey))
		{
			return ControlElement->GetDisplayName().ToString();
		}
		return FString();
	});

	const FName UniqueName = FRigName(GetUniqueName(*SanitizedName.GetName(), [this, &DisplayNames](const FName& InName) -> bool
	{
		return !DisplayNames.Contains(InName);
	}));

	return UniqueName;
}

FRigName URigHierarchy::GetSafeNewComponentName(const FRigElementKey& InElementKey, const FRigName& InPotentialNewName) const
{
	if(InPotentialNewName.IsNone())
	{
		return FRigName();
	}

	FRigName SanitizedName = InPotentialNewName;
	SanitizeName(SanitizedName, true, GetMaxNameLength());

	if (SanitizedName.Len() > GetMaxNameLength() - 4)
	{
		SanitizedName.SetName(SanitizedName.GetName().LeftChop(SanitizedName.Len() - (GetMaxNameLength() - 4)));
	}

	const FName UniqueName = FRigName(GetUniqueName(*SanitizedName.GetName(), [this, InElementKey](const FName& InName) -> bool
	{
		return IsComponentNameAvailable(InElementKey, InName);
	}));

	return FRigName(UniqueName);
}

FText URigHierarchy::GetDisplayNameForUI(const FRigBaseElement* InElement, EElementNameDisplayMode InNameMode) const
{
	check(InElement);

#if WITH_EDITOR
	
	const FName& OriginalDisplayName = InElement->GetDisplayName();
	FName DisplayName = OriginalDisplayName;
	
	if(CVarControlRigEnableOverrides.GetValueOnAnyThread())
	{
		if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(InElement))
		{
			if(UControlRig* ControlRig = GetTypedOuter<UControlRig>())
			{
				static const FString DisplayNamePath = TEXT("Settings->DisplayName");
				for(const UControlRigOverrideAsset* OverrideAsset : ControlRig->OverrideAssets)
				{
					if(OverrideAsset)
					{
						if(const FControlRigOverrideValue* Override = OverrideAsset->Overrides.Find(DisplayNamePath, ControlElement->GetFName()))
						{
							ensureMsgf(Override->GetLeafProperty()->IsA<FNameProperty>(), TEXT("Encountered an override property that's not a FName."));
							if(const FName* DisplayNameFromOverride = Override->GetData<FName>())
							{
								if(!DisplayNameFromOverride->IsNone())
								{
									DisplayName = *DisplayNameFromOverride;
								}
							}
						}
					}
				}
			}
		}
	}
	
	FString DisplayNameString = DisplayName.ToString();
	
	const FRigHierarchyModulePath ModulePath(DisplayNameString);
	(void)ModulePath.Split(nullptr, &DisplayNameString);

	if(InNameMode == EElementNameDisplayMode::AssetDefault)
	{
		InNameMode = EElementNameDisplayMode::Auto;

		if(const UControlRig* ControlRig = Cast<UControlRig>(GetOuter()))
		{
			InNameMode = ControlRig->HierarchySettings.ElementNameDisplayMode;
		}
	}

	bool bIncludeModuleName;
	switch(InNameMode)
	{
		case EElementNameDisplayMode::Auto:
		{
			bIncludeModuleName = !HasUniqueShortName(InElement);
			break;
		}
		case EElementNameDisplayMode::ForceShort:
		{
			bIncludeModuleName = false;
			break;
		}
		case EElementNameDisplayMode::ForceLong:
		default:
		{
			bIncludeModuleName = true;
			break;
		}
	}
	if(bIncludeModuleName)
	{
		const FName ModuleName = GetNameMetadata(InElement->Key, ModuleMetadataName, NAME_None);
		if(!ModuleName.IsNone())
		{
			static const FText ModulePathDisplayFormat = NSLOCTEXT("URigHierarchy", "ModulePathDisplayFormat", "{0} / {1}");
			return FText::Format(ModulePathDisplayFormat, FText::FromName(ModuleName), FText::FromString(DisplayNameString));
		}
	}

	return FText::FromString(*DisplayNameString);
#else

	return FText::FromName(InElement->GetFName());

#endif
}

FText URigHierarchy::GetDisplayNameForUI(const FRigElementKey& InKey, EElementNameDisplayMode InNameMode) const
{
	if(const FRigBaseElement* Element = Find(InKey))
	{
		return GetDisplayNameForUI(Element, InNameMode);
	}
	return FText();
}

int32 URigHierarchy::GetPoseVersion(const FRigElementKey& InKey) const
{
	const FRigTransformElement* TransformElement = Find<FRigTransformElement>(InKey);
	return GetPoseVersion(TransformElement);
}

int32 URigHierarchy::GetPoseVersion(const FRigTransformElement* InTransformElement) const
{
	return InTransformElement && InTransformElement->Index > INDEX_NONE ? GetPoseVersion(InTransformElement->Index) : INDEX_NONE;
}

FEdGraphPinType URigHierarchy::GetControlPinType(FRigControlElement* InControlElement) const
{
	check(InControlElement);
	return GetControlPinType(InControlElement->Settings.ControlType);
}

FEdGraphPinType URigHierarchy::GetControlPinType(ERigControlType ControlType)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	// local copy of UEdGraphSchema_K2::PC_ ... static members
	static const FName PC_Boolean(TEXT("bool"));
	static const FName PC_Float(TEXT("float"));
	static const FName PC_Int(TEXT("int"));
	static const FName PC_Struct(TEXT("struct"));
	static const FName PC_Real(TEXT("real"));

	FEdGraphPinType PinType;

	switch(ControlType)
	{
		case ERigControlType::Bool:
		{
			PinType.PinCategory = PC_Boolean;
			break;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			PinType.PinCategory = PC_Real;
			PinType.PinSubCategory = PC_Float;
			break;
		}
		case ERigControlType::Integer:
		{
			PinType.PinCategory = PC_Int;
			break;
		}
		case ERigControlType::Vector2D:
		{
			PinType.PinCategory = PC_Struct;
			PinType.PinSubCategoryObject = TBaseStructure<FVector2D>::Get();
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			PinType.PinCategory = PC_Struct;
			PinType.PinSubCategoryObject = TBaseStructure<FVector>::Get();
			break;
		}
		case ERigControlType::Rotator:
		{
			PinType.PinCategory = PC_Struct;
			PinType.PinSubCategoryObject = TBaseStructure<FRotator>::Get();
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{
			PinType.PinCategory = PC_Struct;
			PinType.PinSubCategoryObject = TBaseStructure<FTransform>::Get();
			break;
		}
	}

	return PinType;
}

FString URigHierarchy::GetControlPinDefaultValue(FRigControlElement* InControlElement, bool bForEdGraph, ERigControlValueType InValueType) const
{
	check(InControlElement);
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	FRigControlValue Value = GetControlValue(InControlElement, InValueType);
	switch(InControlElement->Settings.ControlType)
	{
		case ERigControlType::Bool:
		{
			return Value.ToString<bool>();
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			return Value.ToString<float>();
		}
		case ERigControlType::Integer:
		{
			return Value.ToString<int32>();
		}
		case ERigControlType::Vector2D:
		{
			const FVector3f Vector = Value.Get<FVector3f>();
			const FVector2D Vector2D(Vector.X, Vector.Y);

			if(bForEdGraph)
			{
				return Vector2D.ToString();
			}

			FString Result;
			TBaseStructure<FVector2D>::Get()->ExportText(Result, &Vector2D, nullptr, nullptr, PPF_None, nullptr);
			return Result;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			if(bForEdGraph)
			{
				// NOTE: We can not use ToString() here since the FDefaultValueHelper::IsStringValidVector used in
				// EdGraphSchema_K2 expects a string with format '#,#,#', while FVector:ToString is returning the value
				// with format 'X=# Y=# Z=#'				
				const FVector Vector(Value.Get<FVector3f>());
				return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f"), Vector.X, Vector.Y, Vector.Z);
			}
			return Value.ToString<FVector>();
		}
		case ERigControlType::Rotator:
		{
				if(bForEdGraph)
				{
					// NOTE: se explanations above for Position/Scale
					const FRotator Rotator = FRotator::MakeFromEuler((FVector)Value.GetRef<FVector3f>());
					return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f"), Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
				}
				return Value.ToString<FRotator>();
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{
			const FTransform Transform = Value.GetAsTransform(
				InControlElement->Settings.ControlType,
				InControlElement->Settings.PrimaryAxis);
				
			if(bForEdGraph)
			{
				return Transform.ToString();
			}

			FString Result;
			TBaseStructure<FTransform>::Get()->ExportText(Result, &Transform, nullptr, nullptr, PPF_None, nullptr);
			return Result;
		}
	}
	return FString();
}

TArray<FRigElementKey> URigHierarchy::GetChildren(FRigElementKey InKey, bool bRecursive) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	if (bRecursive)
	{
		return ConvertElementsToKeys(GetChildren(Find(InKey), true));
	}
	else
	{
		return ConvertElementsToKeys(GetChildren(Find(InKey)));
	}
}

FRigBaseElementChildrenArray URigHierarchy::GetActiveChildren(const FRigBaseElement* InElement, bool bRecursive) const
{
	TArray<FRigBaseElement*> Children;

	TArray<const FRigBaseElement*> ToProcess;
	ToProcess.Push(InElement);

	while (!ToProcess.IsEmpty())
	{
		const FRigBaseElement* CurrentParent = ToProcess.Pop();
		TArray<FRigBaseElement*> CurrentChildren;
		if (CurrentParent)
		{
			CurrentChildren = GetChildren(CurrentParent);
			if (CurrentParent->GetKey() == URigHierarchy::GetWorldSpaceReferenceKey())
			{
				CurrentChildren.Append(GetFilteredElements<FRigBaseElement>([this](const FRigBaseElement* Element)
				{
					return !Element->GetKey().IsTypeOf(ERigElementType::Reference) && GetActiveParent(Element) == nullptr;
				}));
			}
		}
		else
		{
			CurrentChildren = GetFilteredElements<FRigBaseElement>([this](const FRigBaseElement* Element)
			{
				return !Element->GetKey().IsTypeOf(ERigElementType::Reference) && GetActiveParent(Element) == nullptr;
			});
		}
		const FRigElementKey ParentKey = CurrentParent ? CurrentParent->GetKey() : URigHierarchy::GetWorldSpaceReferenceKey();
		for (const FRigBaseElement* Child : CurrentChildren)
		{
			const FRigBaseElement* ThisParent = GetActiveParent(Child);
			const FRigElementKey ThisParentKey = ThisParent ? ThisParent->GetKey() : URigHierarchy::GetWorldSpaceReferenceKey();
			if (ThisParentKey == ParentKey)
			{
				Children.Add(const_cast<FRigBaseElement*>(Child));
				if (bRecursive)
				{
					ToProcess.Push(Child);
				}
			}
		}
	}
	
	return FRigBaseElementChildrenArray(Children);
}

TArray<int32> URigHierarchy::GetChildren(int32 InIndex, bool bRecursive) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	if (!ensure(Elements.IsValidIndex(InIndex)))
	{
		return {};
	}
	
	TArray<int32> ChildIndices;
	ConvertElementsToIndices(GetChildren(Elements[InIndex]), ChildIndices);

	if (bRecursive)
	{
		// Go along the children array and add all children. Once we stop adding children, the traversal index
		// will reach the end and we're done.
		for(int32 TraversalIndex = 0; TraversalIndex != ChildIndices.Num(); TraversalIndex++)
		{
			ConvertElementsToIndices(GetChildren(Elements[ChildIndices[TraversalIndex]]), ChildIndices);
		}
	}
	return ChildIndices;
}

TConstArrayView<FRigBaseElement*> URigHierarchy::GetChildren(const FRigBaseElement* InElement) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InElement)
	{
		EnsureCachedChildrenAreCurrent();

		if (InElement->ChildCacheIndex != INDEX_NONE)
		{
			const FChildElementOffsetAndCount& OffsetAndCount = ChildElementOffsetAndCountCache[InElement->ChildCacheIndex];
			return TConstArrayView<FRigBaseElement*>(&ChildElementCache[OffsetAndCount.Offset], OffsetAndCount.Count);
		}
	}
	return {};
}

TArrayView<FRigBaseElement*> URigHierarchy::GetChildren(const FRigBaseElement* InElement)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InElement)
	{
		EnsureCachedChildrenAreCurrent();

		if (InElement->ChildCacheIndex != INDEX_NONE)
		{
			const FChildElementOffsetAndCount& OffsetAndCount = ChildElementOffsetAndCountCache[InElement->ChildCacheIndex];
			return TArrayView<FRigBaseElement*>(&ChildElementCache[OffsetAndCount.Offset], OffsetAndCount.Count);
		}
	}
	return {};
}



FRigBaseElementChildrenArray URigHierarchy::GetChildren(const FRigBaseElement* InElement, bool bRecursive) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	FRigBaseElementChildrenArray Children;

	Children.Append(GetChildren(InElement));

	if (bRecursive)
	{
		// Go along the children array and add all children. Once we stop adding children, the traversal index
		// will reach the end and we're done.
		for(int32 TraversalIndex = 0; TraversalIndex != Children.Num(); TraversalIndex++)
		{
			Children.Append(GetChildren(Children[TraversalIndex]));
		}
	}
	
	return Children;
}

TArray<FRigElementKey> URigHierarchy::GetParents(FRigElementKey InKey, bool bRecursive) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	const FRigBaseElementParentArray& Parents = GetParents(Find(InKey), bRecursive);
	TArray<FRigElementKey> Keys;
	for(const FRigBaseElement* Parent : Parents)
	{
		Keys.Add(Parent->Key);
	}
	return Keys;
}

TArray<int32> URigHierarchy::GetParents(int32 InIndex, bool bRecursive) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	const FRigBaseElementParentArray& Parents = GetParents(Get(InIndex), bRecursive);
	TArray<int32> Indices;
	for(const FRigBaseElement* Parent : Parents)
	{
		Indices.Add(Parent->Index);
	}
	return Indices;
}

FRigBaseElementParentArray URigHierarchy::GetParents(const FRigBaseElement* InElement, bool bRecursive) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	FRigBaseElementParentArray Parents;

	if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InElement))
	{
		if(SingleParentElement->ParentElement)
		{
			Parents.Add(SingleParentElement->ParentElement);
		}
	}
	else if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		Parents.Reserve(MultiParentElement->ParentConstraints.Num());
		for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
		{
			Parents.Add(ParentConstraint.ParentElement);
		}
	}

	if(bRecursive)
	{
		const int32 CurrentNumberParents = Parents.Num();
		for(int32 ParentIndex = 0;ParentIndex < CurrentNumberParents; ParentIndex++)
		{
			const FRigBaseElementParentArray GrandParents = GetParents(Parents[ParentIndex], bRecursive);
			for (FRigBaseElement* GrandParent : GrandParents)
			{
				Parents.AddUnique(GrandParent);
			}
		}
	}

	return Parents;
}

FRigElementKey URigHierarchy::GetDefaultParent(FRigElementKey InKey) const
{
	if (DefaultParentCacheTopologyVersion != GetTopologyVersion())
	{
		DefaultParentPerElement.Reset();
		DefaultParentCacheTopologyVersion = GetTopologyVersion();
	}
	
	FRigElementKey DefaultParent;
	if(const FRigElementKey* DefaultParentPtr = DefaultParentPerElement.Find(InKey))
	{
		DefaultParent = *DefaultParentPtr;
	}
	else
	{
		DefaultParent = GetFirstParent(InKey);
		DefaultParentPerElement.Add(InKey, DefaultParent);
	}
	return DefaultParent;
}

FRigElementKey URigHierarchy::GetFirstParent(FRigElementKey InKey) const
{
	if(FRigBaseElement* FirstParent = GetFirstParent(Find(InKey)))
	{
		return FirstParent->Key;
	}
	return FRigElementKey();
}

int32 URigHierarchy::GetFirstParent(int32 InIndex) const
{
	if(FRigBaseElement* FirstParent = GetFirstParent(Get(InIndex)))
	{
		return FirstParent->Index;
	}
	return INDEX_NONE;
}

FRigBaseElement* URigHierarchy::GetFirstParent(const FRigBaseElement* InElement) const
{
	if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InElement))
	{
		return SingleParentElement->ParentElement;
	}
	else if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		if(MultiParentElement->ParentConstraints.Num() > 0)
		{
			return MultiParentElement->ParentConstraints[0].ParentElement;
		}
	}
	
	return nullptr;
}

int32 URigHierarchy::GetNumberOfParents(FRigElementKey InKey) const
{
	return GetNumberOfParents(Find(InKey));
}

int32 URigHierarchy::GetNumberOfParents(int32 InIndex) const
{
	return GetNumberOfParents(Get(InIndex));
}

int32 URigHierarchy::GetNumberOfParents(const FRigBaseElement* InElement) const
{
	if(InElement == nullptr)
	{
		return 0;
	}

	if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InElement))
	{
		return SingleParentElement->ParentElement == nullptr ? 0 : 1;
	}
	else if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		return MultiParentElement->ParentConstraints.Num();
	}

	return 0;
}

FRigElementWeight URigHierarchy::GetParentWeight(FRigElementKey InChild, FRigElementKey InParent, bool bInitial) const
{
	return GetParentWeight(Find(InChild), Find(InParent), bInitial);
}

FRigElementWeight URigHierarchy::GetParentWeight(const FRigBaseElement* InChild, const FRigBaseElement* InParent, bool bInitial) const
{
	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		if(const int32* ParentIndexPtr = MultiParentElement->IndexLookup.Find(InParent->GetKey()))
		{
			return GetParentWeight(InChild, *ParentIndexPtr, bInitial);
		}
	}
	return FRigElementWeight(FLT_MAX);
}

FRigElementWeight URigHierarchy::GetParentWeight(const FRigBaseElement* InChild, int32 InParentIndex, bool bInitial) const
{
	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		if(MultiParentElement->ParentConstraints.IsValidIndex(InParentIndex))
		{
			if(bInitial)
			{
				return MultiParentElement->ParentConstraints[InParentIndex].InitialWeight;
			}
			else
			{
				return MultiParentElement->ParentConstraints[InParentIndex].Weight;
			}
		}
	}
	return FRigElementWeight(FLT_MAX);
}

TArray<FRigElementWeight> URigHierarchy::GetParentWeightArray(FRigElementKey InChild, bool bInitial) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	return GetParentWeightArray(Find(InChild), bInitial);
}

TArray<FRigElementWeight> URigHierarchy::GetParentWeightArray(const FRigBaseElement* InChild, bool bInitial) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	TArray<FRigElementWeight> Weights;
	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		for(int32 ParentIndex = 0; ParentIndex < MultiParentElement->ParentConstraints.Num(); ParentIndex++)
		{
			if(bInitial)
			{
				Weights.Add(MultiParentElement->ParentConstraints[ParentIndex].InitialWeight);
			}
			else
			{
				Weights.Add(MultiParentElement->ParentConstraints[ParentIndex].Weight);
			}
		}
	}
	return Weights;
}

FRigElementKey URigHierarchy::GetActiveParent(const FRigElementKey& InKey, bool bReferenceKey, bool bInitial) const
{
	if(FRigBaseElement* Parent = GetActiveParent(Find(InKey), bInitial))
	{
		if (bReferenceKey && Parent->GetKey() == GetDefaultParent(InKey))
		{
			return URigHierarchy::GetDefaultParentKey();
		}
		return Parent->Key;
	}

	if (bReferenceKey)
	{
		return URigHierarchy::GetWorldSpaceReferenceKey();
	}

	return FRigElementKey();
}

int32 URigHierarchy::GetActiveParent(int32 InIndex, bool bInitial) const
{
	if(FRigBaseElement* Parent = GetActiveParent(Get(InIndex), bInitial))
	{
		return Parent->Index;
	}
	return INDEX_NONE;
}

FRigBaseElement* URigHierarchy::GetActiveParent(const FRigBaseElement* InElement, bool bInitial) const
{
	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		const TArray<FRigElementWeight> ParentWeights = GetParentWeightArray(InElement, bInitial);
		if (ParentWeights.Num() > 0)
		{
			const FRigBaseElementParentArray ParentKeys = GetParents(InElement);
			check(ParentKeys.Num() == ParentWeights.Num());
			for (int32 ParentIndex = 0; ParentIndex < ParentKeys.Num(); ParentIndex++)
			{
				if (ParentWeights[ParentIndex].IsAlmostZero())
				{
					continue;
				}
				if (Elements.IsValidIndex(ParentKeys[ParentIndex]->GetIndex()))
				{
					return Elements[ParentKeys[ParentIndex]->GetIndex()];
				}
			}
		}
		return nullptr;
	}

	return GetFirstParent(InElement);
}

FRigBaseElementParentArray URigHierarchy::GetActiveParents(const FRigBaseElement* InElement, bool bInitial) const
{
	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		TArray<FRigBaseElement*> Parents;
		const TArray<FRigElementWeight> ParentWeights = GetParentWeightArray(InElement, bInitial);
		if (ParentWeights.Num() > 0)
		{
			const FRigBaseElementParentArray ParentKeys = GetParents(InElement);
			check(ParentKeys.Num() == ParentWeights.Num());
			for (int32 ParentIndex = 0; ParentIndex < ParentKeys.Num(); ParentIndex++)
			{
				if (ParentWeights[ParentIndex].IsAlmostZero())
				{
					continue;
				}
				if (Elements.IsValidIndex(ParentKeys[ParentIndex]->GetIndex()))
				{
					Parents.Add(Elements[ParentKeys[ParentIndex]->GetIndex()]);
				}
			}
		}
		return FRigBaseElementParentArray(Parents);
	}

	if (FRigBaseElement* Parent = GetFirstParent(InElement))
	{
		return {Parent};
	}
	
	return {};
}

FName URigHierarchy::GetDisplayLabelForParent(const FRigElementKey& InChildKey, const FRigElementKey& InParentKey) const
{
	if(InParentKey == GetDefaultParentKey())
	{
		return DefaultParentKeyLabel.Resolve();
	}
	if(InParentKey == GetWorldSpaceReferenceKey())
	{
		return WorldSpaceKeyLabel.Resolve();
	}

	if(!InChildKey.IsValid() || !InParentKey.IsValid())
	{
		return NAME_None;
	}

	if(const FRigControlElement* ControlElement = Find<FRigControlElement>(InChildKey))
	{
		if(const FRigElementKeyWithLabel* AvailableSpace = ControlElement->Settings.Customization.AvailableSpaces.FindByKey(InParentKey))
		{
			if(!AvailableSpace->Label.IsNone())
			{
				return AvailableSpace->Label;
			}
		}
	}
	if(const FRigMultiParentElement* MultiParentElement = Find<FRigMultiParentElement>(InChildKey))
	{
		if(const int32* ParentIndexPtr = MultiParentElement->IndexLookup.Find(InParentKey))
		{
			if(GetDefaultParent(InChildKey) == InParentKey)
			{
				return DefaultParentKeyLabel.Resolve();
			}
			
			const FName& Label = MultiParentElement->ParentConstraints[*ParentIndexPtr].DisplayLabel;
			if(!Label.IsNone())
			{
				return Label;
			}
		}
	}

	return InParentKey.Name;
}

bool URigHierarchy::SetParentWeight(FRigElementKey InChild, FRigElementKey InParent, FRigElementWeight InWeight, bool bInitial, bool bAffectChildren)
{
	return SetParentWeight(Find(InChild), Find(InParent), InWeight, bInitial, bAffectChildren);
}

bool URigHierarchy::SetParentWeight(FRigBaseElement* InChild, const FRigBaseElement* InParent, FRigElementWeight InWeight, bool bInitial, bool bAffectChildren)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		if(const int32* ParentIndexPtr = MultiParentElement->IndexLookup.Find(InParent->GetKey()))
		{
			return SetParentWeight(InChild, *ParentIndexPtr, InWeight, bInitial, bAffectChildren);
		}
	}
	return false;
}

bool URigHierarchy::SetParentWeight(FRigBaseElement* InChild, int32 InParentIndex, FRigElementWeight InWeight, bool bInitial, bool bAffectChildren)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	using namespace ERigTransformType;

	if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		if(MultiParentElement->ParentConstraints.IsValidIndex(InParentIndex))
		{
			if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
			{
				// animation channels cannot change their parent weights, 
				// they are not 3d things - so multi parenting doesn't matter for transforms.
				if(ControlElement->IsAnimationChannel())
				{
					return false;
				}
			}
			
			InWeight.Location = FMath::Max(InWeight.Location, 0.f);
			InWeight.Rotation = FMath::Max(InWeight.Rotation, 0.f);
			InWeight.Scale = FMath::Max(InWeight.Scale, 0.f);

			FRigElementWeight& TargetWeight = bInitial?
				MultiParentElement->ParentConstraints[InParentIndex].InitialWeight :
				MultiParentElement->ParentConstraints[InParentIndex].Weight;

			if(FMath::IsNearlyZero(InWeight.Location - TargetWeight.Location) &&
				FMath::IsNearlyZero(InWeight.Rotation - TargetWeight.Rotation) &&
				FMath::IsNearlyZero(InWeight.Scale - TargetWeight.Scale))
			{
				return false;
			}

			// if we are setting a weight affecting the hierarchy
			// we need to guard against cycles
			if (!bInitial && !InWeight.IsAlmostZero())
			{
				if (!CanSwitchToParent(
					InChild->GetKey(),
					MultiParentElement->ParentConstraints[InParentIndex].ParentElement->GetKey()))
				{
					return false;
				}
			}
			
			const ERigTransformType::Type LocalType = bInitial ? InitialLocal : CurrentLocal;
			const ERigTransformType::Type GlobalType = SwapLocalAndGlobal(LocalType);

			if(bAffectChildren)
			{
				GetParentTransform(MultiParentElement, LocalType);
				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
				{
					GetControlOffsetTransform(ControlElement, LocalType);
				}
				GetTransform(MultiParentElement, LocalType);
				MultiParentElement->GetDirtyState().MarkDirty(GlobalType);
			}
			else
			{
				GetParentTransform(MultiParentElement, GlobalType);
				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
				{
					GetControlOffsetTransform(ControlElement, GlobalType);
				}
				GetTransform(MultiParentElement, GlobalType);
				MultiParentElement->GetDirtyState().MarkDirty(LocalType);
			}

			TargetWeight = InWeight;

			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
			{
				ControlElement->GetOffsetDirtyState().MarkDirty(GlobalType);
			}

			PropagateDirtyFlags(MultiParentElement, ERigTransformType::IsInitial(LocalType), bAffectChildren);
			EnsureCacheValidity();
			IncrementParentWeightVersion();
			
#if WITH_EDITOR
			if (!bPropagatingChange)
			{
				TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);

				ForEachListeningHierarchy([this, LocalType, InChild, InParentIndex, InWeight, bInitial, bAffectChildren](const FRigHierarchyListener& Listener)
				{
					if(!bForcePropagation && !Listener.ShouldReactToChange(LocalType))
					{
						return;
					}

					if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
					{
						if(FRigBaseElement* ListeningElement = ListeningHierarchy->Find(InChild->GetKey()))
						{
							ListeningHierarchy->SetParentWeight(ListeningElement, InParentIndex, InWeight, bInitial, bAffectChildren);
						}
					}
				});
			}
#endif

			Notify(ERigHierarchyNotification::ParentWeightsChanged, MultiParentElement);
			return true;
		}
	}
	return false;
}

bool URigHierarchy::SetParentWeightArray(FRigElementKey InChild, TArray<FRigElementWeight> InWeights, bool bInitial,
	bool bAffectChildren)
{
	return SetParentWeightArray(Find(InChild), InWeights, bInitial, bAffectChildren);
}

bool URigHierarchy::SetParentWeightArray(FRigBaseElement* InChild, const TArray<FRigElementWeight>& InWeights,
	bool bInitial, bool bAffectChildren)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InWeights.Num() == 0)
	{
		return false;
	}
	
	TArrayView<const FRigElementWeight> View(InWeights.GetData(), InWeights.Num());
	return SetParentWeightArray(InChild, View, bInitial, bAffectChildren);
}

bool URigHierarchy::SetParentWeightArray(FRigBaseElement* InChild,  const TArrayView<const FRigElementWeight>& InWeights,
	bool bInitial, bool bAffectChildren)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	using namespace ERigTransformType;

	if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
		{
			// animation channels cannot change their parent weights, 
			// they are not 3d things - so multi parenting doesn't matter for transforms.
			if(ControlElement->IsAnimationChannel())
			{
				return false;
			}
		}

		if(MultiParentElement->ParentConstraints.Num() == InWeights.Num())
		{
			TArray<FRigElementWeight> InputWeights;
			InputWeights.Reserve(InWeights.Num());

			bool bFoundDifference = false;
			for(int32 WeightIndex=0; WeightIndex < InWeights.Num(); WeightIndex++)
			{
				FRigElementWeight InputWeight = InWeights[WeightIndex];
				InputWeight.Location = FMath::Max(InputWeight.Location, 0.f);
				InputWeight.Rotation = FMath::Max(InputWeight.Rotation, 0.f);
				InputWeight.Scale = FMath::Max(InputWeight.Scale, 0.f);
				InputWeights.Add(InputWeight);

				FRigElementWeight& TargetWeight = bInitial?
					MultiParentElement->ParentConstraints[WeightIndex].InitialWeight :
					MultiParentElement->ParentConstraints[WeightIndex].Weight;

				if(!FMath::IsNearlyZero(InputWeight.Location - TargetWeight.Location) ||
					!FMath::IsNearlyZero(InputWeight.Rotation - TargetWeight.Rotation) ||
					!FMath::IsNearlyZero(InputWeight.Scale - TargetWeight.Scale))
				{
					bFoundDifference = true;
				}
			}

			if(!bFoundDifference)
			{
				return false;
			}
			
			const ERigTransformType::Type LocalType = bInitial ? InitialLocal : CurrentLocal;
			const ERigTransformType::Type GlobalType = SwapLocalAndGlobal(LocalType);

			if(bAffectChildren)
			{
				GetTransform(MultiParentElement, LocalType);
				MultiParentElement->GetDirtyState().MarkDirty(GlobalType);
			}
			else
			{
				GetTransform(MultiParentElement, GlobalType);
				MultiParentElement->GetDirtyState().MarkDirty(LocalType);
			}

			// Propagate before switching parent to make sure all children have a clean state
			PropagateDirtyFlags(MultiParentElement, bInitial, bAffectChildren, true, false);

			// first set all the zero weights
			for(int32 WeightIndex=0; WeightIndex < InWeights.Num(); WeightIndex++)
			{
				if (!InWeights[WeightIndex].IsAlmostZero())
				{
					continue;
				}
				if(bInitial)
				{
					MultiParentElement->ParentConstraints[WeightIndex].InitialWeight = InputWeights[WeightIndex];
				}
				else
				{
					MultiParentElement->ParentConstraints[WeightIndex].Weight = InputWeights[WeightIndex];
				}
			}

			IncrementParentWeightVersion();

			// now set all the non-zero weights
			for(int32 WeightIndex=0; WeightIndex < InWeights.Num(); WeightIndex++)
			{
				if (InWeights[WeightIndex].IsAlmostZero())
				{
					continue;
				}

				if (!bInitial)
				{
					if (!CanSwitchToParent(
						InChild->GetKey(),
						MultiParentElement->ParentConstraints[WeightIndex].ParentElement->GetKey()))
					{
						return false;
					}
				}

				if(bInitial)
				{
					MultiParentElement->ParentConstraints[WeightIndex].InitialWeight = InputWeights[WeightIndex];
				}
				else
				{
					MultiParentElement->ParentConstraints[WeightIndex].Weight = InputWeights[WeightIndex];
				}
			}

			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
			{
				// Resolve offset/shape local before invalidating the global: CopyPose can leave
				// a control with Local dirty + Global clean (deferred local-from-global resolve);
				// marking Global dirty without cleaning Local first would put both flags dirty
				// and trip MarkDirty's ensure.
				GetControlOffsetTransform(ControlElement, LocalType);
				ControlElement->GetOffsetDirtyState().MarkDirty(GlobalType);
				GetControlShapeTransform(ControlElement, LocalType);
				ControlElement->GetShapeDirtyState().MarkDirty(GlobalType);
			}

			PropagateDirtyFlags(MultiParentElement, ERigTransformType::IsInitial(LocalType), bAffectChildren);
			EnsureCacheValidity();
			IncrementParentWeightVersion();
			
#if WITH_EDITOR
			if (!bPropagatingChange)
			{
				TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
				ForEachListeningHierarchy([this, LocalType, InChild, InWeights, bInitial, bAffectChildren](const FRigHierarchyListener& Listener)
     			{
					if(!bForcePropagation && !Listener.ShouldReactToChange(LocalType))
					{
						return;
					}

					if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
					{
						if(FRigBaseElement* ListeningElement = ListeningHierarchy->Find(InChild->GetKey()))
						{
							ListeningHierarchy->SetParentWeightArray(ListeningElement, InWeights, bInitial, bAffectChildren);
						}
					}
				});
			}
#endif

			Notify(ERigHierarchyNotification::ParentWeightsChanged, MultiParentElement);

			return true;
		}
	}
	return false;
}

bool URigHierarchy::CanCauseCycle(FRigElementKey InChild, FRigElementKey InParent, const IRigDependenciesProvider& InDependencyProvider,
	FString* OutFailureReason) const
{
	InParent = PreprocessParentElementKeyForSpaceSwitching(InChild, InParent);

	const FRigBaseElement* Child = Find(InChild);
	if(Child == nullptr)
	{
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Child Element %s cannot be found."), *InChild.ToString());
		}
		return false;
	}

	const FRigBaseElement* Parent = Find(InParent);
	if(Parent == nullptr)
	{
		// if we don't specify anything and the element is parented directly to the world,
		// performing this switch means unparenting it from world (since there is no default parent)
		if(!InParent.IsValid() && GetFirstParent(InChild) == GetWorldSpaceReferenceKey())
		{
			return false;
		}
		
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Parent Element %s cannot be found."), *InParent.ToString());
		}
		return false;
	}

	const FRigTransformElement* TransformChild = Cast<FRigTransformElement>(Child);
	if(TransformChild == nullptr)
	{
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Child Element %s is not a transform element."), *InChild.ToString());
		}
		return false;
	}

	const FRigTransformElement* TransformParent = Cast<FRigTransformElement>(Parent);
	if(TransformParent == nullptr)
	{
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Parent Element %s is not a transform element"), *InParent.ToString());
		}
		return false;
	}

	return CanCauseCycle(TransformChild, TransformParent, InDependencyProvider, OutFailureReason);
}

bool URigHierarchy::CanCauseCycle(const FRigTransformElement* InChild, const FRigTransformElement* InParent,
	const IRigDependenciesProvider& InDependencyProvider, FString* OutFailureReason) const
{
	if (!InChild || !InParent)
	{
		return false;
	}
	
	// see if this is already parented to the target parent
	if(GetParents(InChild).Contains(InParent))
	{
		return false;
	}

	constexpr bool bConsiderInactiveParents = true;
	FRigHierarchyDependencyChain DependencyChain;
	if(IsParentedTo(const_cast<FRigTransformElement*>(InParent), const_cast<FRigTransformElement*>(InChild), InDependencyProvider, &DependencyChain, bConsiderInactiveParents))
	{
#if WITH_EDITOR
		if (!DependencyChain.IsEmpty() && InDependencyProvider.IsInteractiveDialogEnabled() && DismissDependencyDelegate.IsBound())
		{
			// for dependencies which are not just pure elements - we should ask the user to potentially dismiss it 
			if (DependencyChain.ContainsByPredicate([](const TPair<FRigHierarchyRecord,FRigHierarchyRecord>& Dependency) -> bool
			{
				return !Dependency.Key.IsElement(); 
			}))
			{
				if (CVarControlRigHierarchyTreatAvailableSpaceSafe.GetValueOnAnyThread())
				{
					// if this is a defined space for something - allow it even if there's a cycle.
					// we assume that spaces defined by riggers are safe to use.
					if (const FRigControlElement* Control = Cast<FRigControlElement>(InChild))
					{
						if (Control->Settings.Customization.AvailableSpaces.ContainsByPredicate([InParent](const FRigElementKeyWithLabel& Space) -> bool
						{
							return Space.Key == InParent->GetKey();
						}))
						{
							return false;
						}
					}
				}			
				if (CVarControlRigHierarchyAllowToIgnoreCycles.GetValueOnAnyThread() && IsInGameThread())
				{
					// if the user decides to dismiss this dependency - we'll ignore the cycle and will still allow the action.
					if (DismissDependencyDelegate.Execute(this, InChild->GetKey(), InParent->GetKey(), DependencyChain))
					{
						return false;
					}
				}
			}
		}

		if(OutFailureReason)
		{
			if (DependencyChain.IsEmpty())
			{
				OutFailureReason->Appendf(TEXT("Relating '%s' to '%s' would cause a cycle."), *InChild->GetKey().ToString(), *InParent->GetKey().ToString());
			}
			else
			{
				const FString Message = GetMessageFromDependencyChain(DependencyChain);
				OutFailureReason->Appendf(TEXT("Relating '%s' to '%s' would cause a cycle.\n\n%s"), *InChild->GetKey().ToString(), *InParent->GetKey().ToString(), *Message);
			}
		}
#endif
		return true;
	}

	return false;
}

bool URigHierarchy::CanSwitchToParent(FRigElementKey InChild, FRigElementKey InParent, const IRigDependenciesProvider& InDependencyProvider, FString* OutFailureReason) const
{
	InParent = PreprocessParentElementKeyForSpaceSwitching(InChild, InParent);
	
	const FRigBaseElement* Child = Find(InChild);
	if(Child == nullptr)
	{
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Child Element %s cannot be found."), *InChild.ToString());
		}
		return false;
	}

	const FRigBaseElement* Parent = Find(InParent);
	if(Parent == nullptr)
	{
		// if we don't specify anything and the element is parented directly to the world,
		// performing this switch means unparenting it from world (since there is no default parent)
		if(!InParent.IsValid() && GetFirstParent(InChild) == GetWorldSpaceReferenceKey())
		{
			return true;
		}
		
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Parent Element %s cannot be found."), *InParent.ToString());
		}
		return false;
	}

	const FRigMultiParentElement* MultiParentChild = Cast<FRigMultiParentElement>(Child);
	if(MultiParentChild == nullptr)
	{
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Child Element %s does not allow space switching (it's not a multi parent element)."), *InChild.ToString());
		}
	}

	const FRigTransformElement* TransformParent = Cast<FRigTransformElement>(Parent);
	if(TransformParent == nullptr)
	{
		if(OutFailureReason)
		{
			OutFailureReason->Appendf(TEXT("Parent Element %s is not a transform element"), *InParent.ToString());
		}
	}

	return !CanCauseCycle(MultiParentChild, TransformParent, InDependencyProvider, OutFailureReason);
}

bool URigHierarchy::SwitchToParent(FRigElementKey InChild, FRigElementKey InParent, bool bInitial, bool bAffectChildren, const IRigDependenciesProvider& InDependencyProvider, FString* OutFailureReason)
{
	InParent = PreprocessParentElementKeyForSpaceSwitching(InChild, InParent);
	return SwitchToParent(Find(InChild), Find(InParent), bInitial, bAffectChildren, InDependencyProvider, OutFailureReason);
}

bool URigHierarchy::SwitchToParent(FRigBaseElement* InChild, FRigBaseElement* InParent, bool bInitial,
	bool bAffectChildren, const IRigDependenciesProvider& InDependencyProvider, FString* OutFailureReason)
{
	FRigHierarchyEnableControllerBracket EnableController(this, true);

	// Exit early if switching to the same parent
	if (InChild)
	{
		const FRigElementKey ChildKey = InChild->GetKey();
		const FRigElementKey ParentKey = InParent ? InParent->GetKey() : GetDefaultParent(ChildKey);
		const FRigElementKey ActiveParentKey = GetActiveParent(ChildKey, true, bInitial);
		if(ActiveParentKey == ParentKey ||
			(ActiveParentKey == URigHierarchy::GetDefaultParentKey() && GetDefaultParent(ChildKey) == ParentKey))
		{
			return true;
		}
	}

	if(InChild && InParent)
	{
		if(!CanSwitchToParent(InChild->GetKey(), InParent->GetKey(), InDependencyProvider, OutFailureReason))
		{
			return false;
		}
	}

	if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		int32 ParentIndex = INDEX_NONE;
		if(InParent)
		{
			if(const int32* ParentIndexPtr = MultiParentElement->IndexLookup.Find(InParent->GetKey()))
			{
				ParentIndex = *ParentIndexPtr;
			}
			else
			{
				if(URigHierarchyController* Controller = GetController(true))
				{
					if(Controller->AddParent(InChild, InParent, 0.f, true, false))
					{
						ParentIndex = MultiParentElement->IndexLookup.FindChecked(InParent->GetKey());
					}
				}
			}
		}
		return SwitchToParent(InChild, ParentIndex, bInitial, bAffectChildren);
	}
	return false;
}

bool URigHierarchy::SwitchToParent(FRigBaseElement* InChild, int32 InParentIndex, bool bInitial, bool bAffectChildren)
{
	TArray<FRigElementWeight> Weights = GetParentWeightArray(InChild, bInitial);
	FMemory::Memzero(Weights.GetData(), Weights.GetAllocatedSize());
	if(Weights.IsValidIndex(InParentIndex))
	{
		Weights[InParentIndex] = 1.f;
	}
	return SetParentWeightArray(InChild, Weights, bInitial, bAffectChildren);
}

bool URigHierarchy::SwitchToDefaultParent(FRigElementKey InChild, bool bInitial, bool bAffectChildren)
{
	return SwitchToParent(InChild, GetDefaultParentKey(), bInitial, bAffectChildren);
}

bool URigHierarchy::SwitchToDefaultParent(FRigBaseElement* InChild, bool bInitial, bool bAffectChildren)
{
	// we assume that the first stored parent is the default parent
	check(InChild);
	return SwitchToParent(InChild->GetKey(), GetDefaultParentKey(), bInitial, bAffectChildren);
}

bool URigHierarchy::SwitchToWorldSpace(FRigElementKey InChild, bool bInitial, bool bAffectChildren)
{
	return SwitchToParent(InChild, GetWorldSpaceReferenceKey(), bInitial, bAffectChildren);
}

bool URigHierarchy::SwitchToWorldSpace(FRigBaseElement* InChild, bool bInitial, bool bAffectChildren)
{
	check(InChild);
	return SwitchToParent(InChild->GetKey(), GetWorldSpaceReferenceKey(), bInitial, bAffectChildren);
}

FRigElementKey URigHierarchy::GetOrAddWorldSpaceReference()
{
	FRigHierarchyEnableControllerBracket EnableController(this, true);

	const FRigElementKey WorldSpaceReferenceKey = GetWorldSpaceReferenceKey();

	FRigBaseElement* Parent = Find(WorldSpaceReferenceKey);
	if(Parent)
	{
		return Parent->GetKey();
	}

	if(URigHierarchyController* Controller = GetController(true))
	{
		return Controller->AddReference(
			WorldSpaceReferenceKey.Name,
			FRigElementKey(),
			FRigReferenceGetWorldTransformDelegate::CreateUObject(this, &URigHierarchy::GetWorldTransformForReference),
			false);
	}

	return FRigElementKey();
}

FRigElementKey URigHierarchy::GetDefaultParentKey()
{
	static const FName DefaultParentName = TEXT("DefaultParent");
	return FRigElementKey(DefaultParentName, ERigElementType::Reference); 
}

FRigElementKey URigHierarchy::GetWorldSpaceReferenceKey()
{
	static const FName WorldSpaceReferenceName = TEXT("WorldSpace");
	return FRigElementKey(WorldSpaceReferenceName, ERigElementType::Reference); 
}

TArray<FRigElementKey> URigHierarchy::GetAnimationChannels(FRigElementKey InKey, bool bOnlyDirectChildren) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	return ConvertElementsToKeys(GetAnimationChannels(Find<FRigControlElement>(InKey), bOnlyDirectChildren));
}

TArray<int32> URigHierarchy::GetAnimationChannels(int32 InIndex, bool bOnlyDirectChildren) const
{
	return ConvertElementsToIndices(GetAnimationChannels(Get<FRigControlElement>(InIndex), bOnlyDirectChildren));
}

TArray<FRigControlElement*> URigHierarchy::GetAnimationChannels(const FRigControlElement* InElement, bool bOnlyDirectChildren) const
{
	if(InElement == nullptr)
	{
		return {};
	}

	const TConstArrayView<FRigBaseElement*> AllChildren = GetChildren(InElement);
	const TArray<FRigBaseElement*> FilteredChildren = AllChildren.FilterByPredicate([](const FRigBaseElement* Element) -> bool
	{
		if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			return ControlElement->IsAnimationChannel();
		}
		return false;
	});

	TArray<FRigControlElement*> AnimationChannels = ConvertElements<FRigControlElement>(FilteredChildren);
	if(!bOnlyDirectChildren)
	{
		AnimationChannels.Append(GetFilteredElements<FRigControlElement>(
			[InElement](const FRigControlElement* PotentialAnimationChannel) -> bool
			{
				if(PotentialAnimationChannel->IsAnimationChannel())
				{
					if(PotentialAnimationChannel->Settings.Customization.AvailableSpaces.FindByKey(InElement->Key))
					{
						return true;
					}
				}
				return false;
			},
		true /* traverse */
		));
	}
	
	return AnimationChannels;
}

TArray<FRigElementKey> URigHierarchy::GetAllKeys(bool bTraverse, ERigElementType InElementType) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"))

	return GetKeysByPredicate([InElementType](const FRigBaseElement& InElement)
	{
		return InElement.IsTypeOf(InElementType);
	}, bTraverse);
}

TArray<FRigElementKey> URigHierarchy::GetKeysByPredicate(
	TFunctionRef<bool(const FRigBaseElement&)> InPredicateFunc,
	bool bTraverse
	) const
{
	auto ElementTraverser = [&](TFunctionRef<void(const FRigBaseElement&)> InProcessFunc)
	{
		if(bTraverse)
		{
			// TBitArray reserves 4, we'll do 16 so we can remember at least 512 elements before
			// we need to hit the heap.
			TBitArray<TInlineAllocator<16>> ElementVisited(false, Elements.Num());

			const TArray<FRigBaseElement*> RootElements = GetRootElements();
			for (FRigBaseElement* Element : RootElements)
			{
				Traverse(Element, true, [&ElementVisited, InProcessFunc, InPredicateFunc](const FRigBaseElement* InElement, bool& bContinue)
				{
					bContinue = !ElementVisited[InElement->GetIndex()];

					if(bContinue)
					{
						if(InPredicateFunc(*InElement))
						{
							InProcessFunc(*InElement);
						}
						ElementVisited[InElement->GetIndex()] = true;
					}
				});
			}
		}
		else
		{
			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
			{
				const FRigBaseElement* Element = Elements[ElementIndex];
				if(InPredicateFunc(*Element))
				{
					InProcessFunc(*Element);
				}
			}
		}
	};
	
	// First count up how many elements we matched and only reserve that amount. There's very little overhead
	// since we're just running over the same data, so it should still be hot when we do the second pass.
	int32 NbElements = 0;
	ElementTraverser([&NbElements](const FRigBaseElement&) { NbElements++; });
	
	TArray<FRigElementKey> Keys;
	Keys.Reserve(NbElements);
	ElementTraverser([&Keys](const FRigBaseElement& InElement) { Keys.Add(InElement.GetKey()); });

	return Keys;
}

void URigHierarchy::Traverse(FRigBaseElement* InElement, bool bTowardsChildren,
                             TFunction<void(FRigBaseElement*, bool&)> PerElementFunction, bool bIgnoreInactiveRelations) const
{
	bool bContinue = true;
	PerElementFunction(InElement, bContinue);

	if(bContinue)
	{
		if(bTowardsChildren)
		{
			TArray<FRigBaseElement*> Children;
			if (bIgnoreInactiveRelations)
			{
				Children = GetActiveChildren(InElement);
			}
			else
			{
				Children = GetChildren(InElement);
			}
			
			for (FRigBaseElement* Child : Children)
			{
				Traverse(Child, true, PerElementFunction, bIgnoreInactiveRelations);
			}
		}
		else
		{
			FRigBaseElementParentArray Parents = GetParents(InElement);
			if (bIgnoreInactiveRelations)
			{
				Parents = GetActiveParents(InElement);
				
			}
			else
			{
				Parents = GetParents(InElement);
			}
			for (FRigBaseElement* Parent : Parents)
			{
				Traverse(Parent, false, PerElementFunction, bIgnoreInactiveRelations);
			}
		}
	}
}

void URigHierarchy::Traverse(TFunction<void(FRigBaseElement*, bool& /* continue */)> PerElementFunction, bool bTowardsChildren, bool bIgnoreInactiveRelations) const
{
	if(bTowardsChildren)
	{
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			FRigBaseElement* Element = Elements[ElementIndex];
			if(GetNumberOfParents(Element) == 0)
			{
				Traverse(Element, bTowardsChildren, PerElementFunction, bIgnoreInactiveRelations);
			}
        }
	}
	else
	{
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			FRigBaseElement* Element = Elements[ElementIndex];
			if(GetChildren(Element).Num() == 0)
			{
				Traverse(Element, bTowardsChildren, PerElementFunction, bIgnoreInactiveRelations);
			}
		}
	}
}

const FRigElementKey& URigHierarchy::GetResolvedTarget(const FRigElementKey& InConnectorKey) const
{
	if (InConnectorKey.Type == ERigElementType::Connector)
	{
		if(ElementKeyRedirector)
		{
			if(const FRigElementKeyRedirector::FCachedKeyArray* Targets = ElementKeyRedirector->Find(InConnectorKey))
			{
				if(!Targets->IsEmpty())
				{
					return (*Targets)[0].GetKey();
				}
			}
		}
	}
	return InConnectorKey;
}

TArray<FRigElementKey> URigHierarchy::GetResolvedTargets(const FRigElementKey& InConnectorKey) const
{
	if (InConnectorKey.Type == ERigElementType::Connector)
	{
		if(ElementKeyRedirector)
		{
			if(const FRigElementKeyRedirector::FCachedKeyArray* Targets = ElementKeyRedirector->Find(InConnectorKey))
			{
				if(!Targets->IsEmpty())
				{
					TArray<FRigElementKey> TargetKeys;
					TargetKeys.Reserve(Targets->Num());
					for(const FCachedRigElement& Cache : *Targets)
					{
						TargetKeys.Add(Cache.GetKey());
					}
					return TargetKeys;
				}
			}
		}
	}
	return {InConnectorKey};
}

bool URigHierarchy::Undo()
{
#if WITH_EDITOR
	
	if(TransformUndoStack.IsEmpty())
	{
		return false;
	}

	const FRigTransformStackEntry Entry = TransformUndoStack.Pop();
	ApplyTransformFromStack(Entry, true);
	UndoRedoEvent.Broadcast(this, Entry.Key, Entry.TransformType, Entry.OldTransform, true);
	TransformRedoStack.Push(Entry);
	TransformStackIndex = TransformUndoStack.Num();
	return true;
	
#else
	
	return false;
	
#endif
}

bool URigHierarchy::Redo()
{
#if WITH_EDITOR

	if(TransformRedoStack.IsEmpty())
	{
		return false;
	}

	const FRigTransformStackEntry Entry = TransformRedoStack.Pop();
	ApplyTransformFromStack(Entry, false);
	UndoRedoEvent.Broadcast(this, Entry.Key, Entry.TransformType, Entry.NewTransform, false);
	TransformUndoStack.Push(Entry);
	TransformStackIndex = TransformUndoStack.Num();
	return true;
	
#else
	
	return false;
	
#endif
}

bool URigHierarchy::SetTransformStackIndex(int32 InTransformStackIndex)
{
#if WITH_EDITOR

	while(TransformUndoStack.Num() > InTransformStackIndex)
	{
		if(TransformUndoStack.Num() == 0)
		{
			return false;
		}

		if(!Undo())
		{
			return false;
		}
	}
	
	while(TransformUndoStack.Num() < InTransformStackIndex)
	{
		if(TransformRedoStack.Num() == 0)
		{
			return false;
		}

		if(!Redo())
		{
			return false;
		}
	}

	return InTransformStackIndex == TransformStackIndex;

#else
	
	return false;
	
#endif
}

#if WITH_EDITOR

void URigHierarchy::PreEditUndo()
{
	Super::PreEditUndo();

	SelectedKeysBeforeUndo = GetSelectedHierarchyKeys();
}

void URigHierarchy::PostEditUndo()
{
	Super::PostEditUndo();

	const int32 DesiredStackIndex = TransformStackIndex;
	TransformStackIndex = TransformUndoStack.Num();
	if (DesiredStackIndex != TransformStackIndex)
	{
		SetTransformStackIndex(DesiredStackIndex);
	}

	if(URigHierarchyController* Controller = GetController(false))
	{
		Controller->SetHierarchy(this);
	}

	NotifyPostUndoSelectionChanges();
}

#endif

void URigHierarchy::SendEvent(const FRigEventContext& InEvent, bool bAsynchronous)
{
	if(EventDelegate.IsBound())
	{
		TWeakObjectPtr<URigHierarchy> WeakThis = this;
		FRigEventDelegate& Delegate = EventDelegate;

		if (bAsynchronous)
		{

			ExecuteOnGameThread(UE_SOURCE_LOCATION,[WeakThis, Delegate, InEvent]()
			{
                Delegate.Broadcast(WeakThis.Get(), InEvent);
            });
		}
		else
		{
			Delegate.Broadcast(this, InEvent);
		}
	}

}

void URigHierarchy::SendAutoKeyEvent(FRigElementKey InElement, float InOffsetInSeconds, bool bAsynchronous)
{
	FRigEventContext Context;
	Context.Event = ERigEvent::RequestAutoKey;
	Context.Key = InElement;
	Context.LocalTime = InOffsetInSeconds;
	if(UControlRig* Rig = Cast<UControlRig>(GetOuter()))
	{
		Context.LocalTime += Rig->AbsoluteTime;
	}
	SendEvent(Context, bAsynchronous);
}

bool URigHierarchy::IsControllerAvailable() const
{
	return bIsControllerAvailable;
}

URigHierarchyController* URigHierarchy::GetController(bool bCreateIfNeeded)
{
	if(!IsControllerAvailable())
	{
		return nullptr;
	}
	if(HierarchyController)
	{
		return HierarchyController;
	}
	else if(bCreateIfNeeded)
	{
		 {
			 FGCScopeGuard Guard;
			 HierarchyController = NewObject<URigHierarchyController>(this, TEXT("HierarchyController"), RF_Transient);
			 // In case we create this object from async loading thread
			 HierarchyController->ClearInternalFlags(EInternalObjectFlags::Async);

			 HierarchyController->SetHierarchy(this);
			 return HierarchyController;
		 }
	}
	return nullptr;
}

UModularRigRuleManager* URigHierarchy::GetRuleManager(bool bCreateIfNeeded)
{
	if(RuleManager)
	{
		return RuleManager;
	}
	else if(bCreateIfNeeded)
	{
		{
			FGCScopeGuard Guard;
			RuleManager = NewObject<UModularRigRuleManager>(this, TEXT("RuleManager"), RF_Transient);
			// In case we create this object from async loading thread
			RuleManager->ClearInternalFlags(EInternalObjectFlags::Async);

			RuleManager->SetHierarchy(this);
			return RuleManager;
		}
	}
	return nullptr;
}

void URigHierarchy::IncrementTopologyVersion()
{
	TopologyVersion++;
	KeyCollectionCache.Reset();
}

void URigHierarchy::IncrementParentWeightVersion()
{
	ParentWeightVersion++;
}

FRigPose URigHierarchy::GetPose(
	bool bInitial,
	ERigElementType InElementType,
	const FRigElementKeyCollection& InItems,
	bool bIncludeTransientControls
) const
{
	return GetPose(bInitial, InElementType, TArrayView<const FRigElementKey>(InItems.Keys.GetData(), InItems.Num()), bIncludeTransientControls);
}

FRigPose URigHierarchy::GetPose(bool bInitial, ERigElementType InElementType,
	const TArrayView<const FRigElementKey>& InItems, bool bIncludeTransientControls) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	FRigPose Pose;
	Pose.HierarchyTopologyVersion = GetTopologyVersion();
	Pose.PoseHash = Pose.HierarchyTopologyVersion;

	const uint8 InElementTypeAsUInt8 = static_cast<uint8>(InElementType);
	
	int32 ExpectedNumElements = 0;
	for(int32 ElementTypeIndex = 0; ElementTypeIndex < ElementsPerType.Num(); ElementTypeIndex++)
	{
		const ERigElementType ElementType = FlatIndexToRigElementType(ElementTypeIndex);  
		if ((InElementTypeAsUInt8 & static_cast<uint8>(ElementType)) != 0)
		{
			ExpectedNumElements += ElementsPerType[ElementTypeIndex].Num();
		}
	}
	if(ExpectedNumElements == 0)
	{
		ExpectedNumElements = Elements.Num();
	}
	Pose.Elements.Reserve(ExpectedNumElements);

	const uint32 TopologyVersionHash = GetTopologyVersionHash();

	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		FRigBaseElement* Element = Elements[ElementIndex];

		// filter by type
		if (((uint8)InElementType & (uint8)Element->GetType()) == 0)
		{
			continue;
		}

		// filter by optional collection
		if(InItems.Num() > 0)
		{
			if(!InItems.Contains(Element->GetKey()))
			{
				continue;
			}
		}
		
		FRigPoseElement PoseElement;
		PoseElement.Index.Set(Element, TopologyVersionHash);
		
		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
		{
			PoseElement.LocalTransform = GetTransform(TransformElement, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal);
			PoseElement.GlobalTransform = GetTransform(TransformElement, bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal);
			PoseElement.ActiveParent = GetActiveParent(Element->GetKey());

			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
			{
				if (bUsePreferredEulerAngles)
				{
					PoseElement.PreferredEulerAngle = GetControlPreferredEulerAngles(ControlElement,
					   GetControlPreferredEulerRotationOrder(ControlElement), bInitial);
				}

				if(!bIncludeTransientControls && ControlElement->Settings.bIsTransientControl)
				{
					continue;
				}
			}
		}
		else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Element))
		{
			PoseElement.CurveValue = GetCurveValue(CurveElement);
		}
		else
		{
			continue;
		}
		Pose.Elements.Add(PoseElement);
		Pose.PoseHash = HashCombine(Pose.PoseHash, GetTypeHash(PoseElement.Index.GetKey()));
	}
	return Pose;
}

void URigHierarchy::SetPose(
	const FRigPose& InPose,
	ERigTransformType::Type InTransformType,
	ERigElementType InElementType,
	const FRigElementKeyCollection& InItems,
	float InWeight
)
{
	SetPose(InPose, InTransformType, InElementType, TArrayView<const FRigElementKey>(InItems.Keys.GetData(), InItems.Num()), InWeight);
}

void URigHierarchy::SetPose(const FRigPose& InPose, ERigTransformType::Type InTransformType,
	ERigElementType InElementType, const TArrayView<const FRigElementKey>& InItems, float InWeight)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	const float U = FMath::Clamp(InWeight, 0.f, 1.f);
	if(U < SMALL_NUMBER)
	{
		return;
	}

	const bool bBlend = U < 1.f - SMALL_NUMBER;
	const bool bLocal = IsLocal(InTransformType);
	static constexpr bool bAffectChildren = true;

	for(const FRigPoseElement& PoseElement : InPose)
	{
		FCachedRigElement Index = PoseElement.Index;

		// filter by type
		if (((uint8)InElementType & (uint8)Index.GetKey().Type) == 0)
		{
			continue;
		}

		// filter by optional collection
		if(InItems.Num() > 0)
		{
			if(!InItems.Contains(Index.GetKey()))
			{
				continue;
			}
		}

		if(Index.UpdateCache(this))
		{
			FRigBaseElement* Element = Get(Index.GetIndex());
			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
			{
				// only nulls and controls can switch parent (cf. FRigUnit_SwitchParent)
				const bool bCanSwitch = TransformElement->IsA<FRigMultiParentElement>() && PoseElement.ActiveParent.IsValid();
					
				const FTransform& PoseTransform = bLocal ? PoseElement.LocalTransform : PoseElement.GlobalTransform;
				if (bBlend)
				{
					const FTransform PreviousTransform = GetTransform(TransformElement, InTransformType);
					const FTransform TransformToSet = FControlRigMathLibrary::LerpTransform(PreviousTransform, PoseTransform, U);
					if (bCanSwitch)
					{
						SwitchToParent(Element->GetKey(), PoseElement.ActiveParent);
					}
					SetTransform(TransformElement, TransformToSet, InTransformType, bAffectChildren);
				}
				else
				{
					if (bCanSwitch)
					{
						SwitchToParent(Element->GetKey(), PoseElement.ActiveParent);
					}
					SetTransform(TransformElement, PoseTransform, InTransformType, bAffectChildren);
				}
			}
			else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Element))
			{
				SetCurveValue(CurveElement, PoseElement.CurveValue);
			}
		}
	}
}

void URigHierarchy::LinkPoseAdapter(TSharedPtr<FRigHierarchyPoseAdapter> InPoseAdapter)
{
	if(PoseAdapter)
	{
		PoseAdapter->PreUnlinked(this);
		PoseAdapter->bLinked = false;
		PoseAdapter->WeakHierarchy.Reset();;
		PoseAdapter->LastTopologyVersion = INDEX_NONE;
		PoseAdapter.Reset();
	}

	if(InPoseAdapter)
	{
		PoseAdapter = InPoseAdapter;
		PoseAdapter->PostLinked(this);
		PoseAdapter->bLinked = true;
		PoseAdapter->WeakHierarchy = this;
		PoseAdapter->LastTopologyVersion = GetTopologyVersion();

	}
}

void URigHierarchy::Notify(ERigHierarchyNotification InNotifType, const FRigNotificationSubject& InSubject)
{
	if(bSuspendNotifications)
	{
		return;
	}

	if (!IsValidChecked(this))
	{
		return;
	}

	// if we are running a VM right now
	{
		UE::TScopeLock Lock(ExecuteContextLock);
		if(ExecuteContext != nullptr)
		{
			QueueNotification(InNotifType, InSubject);
			return;
		}
	}
	

	if(QueuedNotifications.IsEmpty())
	{
		ModifiedEvent.Broadcast(InNotifType, this, InSubject);
		if(ModifiedEventDynamic.IsBound())
		{
			FRigElementKey Key;
			if(InSubject.Element)
			{
				Key = InSubject.Element->GetKey();
			}
			ModifiedEventDynamic.Broadcast(InNotifType, this, Key);
		}
	}
	else
	{
		QueueNotification(InNotifType, InSubject);
		SendQueuedNotifications();
	}

	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;

#if WITH_EDITOR

	// certain events needs to be forwarded to the listening hierarchies.
	// this mainly has to do with topological change within the hierarchy.
	switch (InNotifType)
	{
		case ERigHierarchyNotification::ElementAdded:
		case ERigHierarchyNotification::ElementRemoved:
		case ERigHierarchyNotification::ElementRenamed:
		case ERigHierarchyNotification::ParentChanged:
		case ERigHierarchyNotification::ParentWeightsChanged:
		{
			if (ensure(InElement != nullptr))
			{
				ForEachListeningHierarchy([this, InNotifType, InElement](const FRigHierarchyListener& Listener)
				{
					if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
					{
						if(const FRigBaseElement* ListeningElement = ListeningHierarchy->Find( InElement->GetKey()))
						{
							ListeningHierarchy->Notify(InNotifType, ListeningElement);
						}
					}
				});
			}
			break;
		}
		case ERigHierarchyNotification::ComponentAdded:
		case ERigHierarchyNotification::ComponentRemoved:
		case ERigHierarchyNotification::ComponentRenamed:
		case ERigHierarchyNotification::ComponentReparented:
		{
			if (ensure(InComponent != nullptr))
			{
				const FRigBaseElement* Element = InComponent->GetElement();
				ForEachListeningHierarchy([this, InNotifType, InComponent, Element](const FRigHierarchyListener& Listener)
				{
					if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
					{			
						if(const FRigBaseComponent* ListeningComponent = ListeningHierarchy->FindComponent(InComponent->GetKey()))
						{
							ListeningHierarchy->Notify(InNotifType, ListeningComponent);
						}
					}
				});
			}
			break;
		}
		default:
		{
			break;
		}
	}

#endif
}

void URigHierarchy::QueueNotification(ERigHierarchyNotification InNotification, const FRigNotificationSubject& InSubject)
{
	FQueuedNotification Entry;
	Entry.Type = InNotification;
	if(InSubject.Element)
	{
		Entry.Key = InSubject.Element->GetKey();
		Entry.ComponentName = NAME_None;
	}
	else if(InSubject.Component)
	{
		Entry.Key = InSubject.Component->GetElementKey();
		Entry.ComponentName = InSubject.Component->GetFName();
	}
	QueuedNotifications.Enqueue(Entry);
}

void URigHierarchy::SendQueuedNotifications()
{
	if(bSuspendNotifications)
	{
		QueuedNotifications.Empty();
		return;
	}

	if(QueuedNotifications.IsEmpty())
	{
		return;
	}

	{
		UE::TScopeLock Lock(ExecuteContextLock);
    	if(ExecuteContext != nullptr)
    	{
    		return;
    	}
	}
	
	// enable access to the controller during this method
	FRigHierarchyEnableControllerBracket EnableController(this, true);

	// we'll collect all notifications and will clean them up
	// to guard against notification storms.
	TArray<FQueuedNotification> AllNotifications;
	FQueuedNotification EntryFromQueue;
	while(QueuedNotifications.Dequeue(EntryFromQueue))
	{
		AllNotifications.Add(EntryFromQueue);
	}
	QueuedNotifications.Empty();

	// now we'll filter the notifications. we'll go through them in
	// reverse and will skip any aggregates (like change color multiple times on the same thing)
	// as well as collapse pairs such as select and deselect
	TArray<FQueuedNotification> FilteredNotifications;
	TArray<FQueuedNotification> UniqueNotifications;
	for(int32 Index = AllNotifications.Num() - 1; Index >= 0; Index--)
	{
		const FQueuedNotification& Entry = AllNotifications[Index];

		bool bSkipNotification = false;
		switch(Entry.Type)
		{
			case ERigHierarchyNotification::HierarchyReset:
			case ERigHierarchyNotification::ElementRemoved:
			case ERigHierarchyNotification::ElementRenamed:
			case ERigHierarchyNotification::ComponentRemoved:
			case ERigHierarchyNotification::ComponentRenamed:
			case ERigHierarchyNotification::ComponentReparented:
			{
				// we don't allow these to happen during the run of a VM
				if(const URigHierarchyController* Controller = GetController())
				{
					static constexpr TCHAR InvalidNotificationReceivedMessage[] =
						TEXT("Found invalid queued notification %s - %s. Skipping notification.");
					const FString NotificationText = StaticEnum<ERigHierarchyNotification>()->GetNameStringByValue((int64)Entry.Type);
					
					Controller->ReportErrorf(InvalidNotificationReceivedMessage, *NotificationText, *Entry.Key.ToString());	
				}
				bSkipNotification = true;
				break;
			}
			case ERigHierarchyNotification::ControlSettingChanged:
			case ERigHierarchyNotification::ControlVisibilityChanged:
			case ERigHierarchyNotification::ControlDrivenListChanged:
			case ERigHierarchyNotification::ControlShapeTransformChanged:
			case ERigHierarchyNotification::ParentChanged:
			case ERigHierarchyNotification::ParentWeightsChanged:
			{
				// these notifications are aggregates - they don't need to happen
				// more than once during an update
				bSkipNotification = UniqueNotifications.Contains(Entry);
				break;
			}
			case ERigHierarchyNotification::ElementSelected:
			case ERigHierarchyNotification::ElementDeselected:
			{
				FQueuedNotification OppositeEntry;
				OppositeEntry.Type = (Entry.Type == ERigHierarchyNotification::ElementSelected) ?
					ERigHierarchyNotification::ElementDeselected :
					ERigHierarchyNotification::ElementSelected;
				OppositeEntry.Key = Entry.Key;

				// we don't need to add this if we already performed the selection or
				// deselection of the same item before
				bSkipNotification = UniqueNotifications.Contains(Entry) ||
					UniqueNotifications.Contains(OppositeEntry);
				break;
			}
			case ERigHierarchyNotification::Max:
			{
				bSkipNotification = true;
				break;
			}
			case ERigHierarchyNotification::ElementAdded:
			case ERigHierarchyNotification::ComponentAdded:
			case ERigHierarchyNotification::ComponentContentChanged:
			case ERigHierarchyNotification::InteractionBracketOpened:
			case ERigHierarchyNotification::InteractionBracketClosed:
			default:
			{
				break;
			}
		}

		UniqueNotifications.AddUnique(Entry);
		if(!bSkipNotification)
		{
			FilteredNotifications.Add(Entry);
		}

		// if we ever hit a reset then we don't need to deal with
		// any previous notifications.
		if(Entry.Type == ERigHierarchyNotification::HierarchyReset)
		{
			break;
		}
	}

	if(FilteredNotifications.IsEmpty())
	{
		return;
	}

	ModifiedEvent.Broadcast(ERigHierarchyNotification::InteractionBracketOpened, this, {});
	if(ModifiedEventDynamic.IsBound())
	{
		ModifiedEventDynamic.Broadcast(ERigHierarchyNotification::InteractionBracketOpened, this, FRigElementKey());
	}

	// finally send all of the notifications
	// (they have been added in the reverse order to the array)
	for(int32 Index = FilteredNotifications.Num() - 1; Index >= 0; Index--)
	{
		const FQueuedNotification& Entry = FilteredNotifications[Index];
		FRigNotificationSubject Subject;
		Subject.Component = FindComponent({Entry.Key, Entry.ComponentName});
		if(Subject.Component == nullptr)
		{
			Subject.Element = Find(Entry.Key);
		}

		ModifiedEvent.Broadcast(Entry.Type, this, Subject);

		// the dynamic multicast for now only supports element notifs
		if(Subject.Element && ModifiedEventDynamic.IsBound())
		{
			ModifiedEventDynamic.Broadcast(Entry.Type, this, Subject.Element->GetKey());
		}
	}

	ModifiedEvent.Broadcast(ERigHierarchyNotification::InteractionBracketClosed, this, {});
	if(ModifiedEventDynamic.IsBound())
	{
		ModifiedEventDynamic.Broadcast(ERigHierarchyNotification::InteractionBracketClosed, this, FRigElementKey());
	}
}

FTransform URigHierarchy::GetTransform(FRigTransformElement* InTransformElement,
	const ERigTransformType::Type InTransformType) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InTransformElement == nullptr)
	{
		return FTransform::Identity;
	}

#if WITH_EDITOR
	{
		if(bRecordInstructionsAtRuntime)
		{
			UE::TScopeLock Lock(ExecuteContextLock);
			if (ExecuteContext != nullptr)
			{
				const FControlRigExecuteContext& PublicData = ExecuteContext->GetPublicData<FControlRigExecuteContext>();
				const FName VMName = PublicData.GetRigModuleInstance() ? PublicData.GetRigModuleInstance()->Name : NAME_None;

				const FInstructionRecord Record(PublicData.GetInstructionIndex(),
					ExecuteContext->GetSlice().GetIndex(),
					InTransformElement->GetIndex(),
					VMName);

				PublicData.AddReadInstructionRecord(Record);
			}			
		}
	}
	
	TGuardValue<bool> RecordTransformsPerInstructionGuard(bRecordInstructionsAtRuntime, false);
	
#endif
	
	if(InTransformElement->GetDirtyState().IsDirty(InTransformType))
	{
		const ERigTransformType::Type OpposedType = SwapLocalAndGlobal(InTransformType);
		const ERigTransformType::Type GlobalType = MakeGlobal(InTransformType);
		ensure(!InTransformElement->GetDirtyState().IsDirty(OpposedType));

		FTransform ParentTransform;
		if(IsLocal(InTransformType))
		{
			// if we have a zero scale provided - and the parent also contains a zero scale,
			// we'll keep the local translation and scale since otherwise we'll loose the values.
			// we cannot compute the local from the global if the scale is 0 - since the local scale
			// may be anything - any translation or scale multiplied with the parent's zero scale is zero. 
			auto CompensateZeroScale = [this, InTransformElement, InTransformType](FTransform& Transform)
			{
				const FVector Scale = Transform.GetScale3D();
				if(FMath::IsNearlyZero(Scale.X) || FMath::IsNearlyZero(Scale.Y) || FMath::IsNearlyZero(Scale.Z))
				{
					const FTransform ParentTransform =
						GetParentTransform(InTransformElement, ERigTransformType::SwapLocalAndGlobal(InTransformType));
					const FVector ParentScale = ParentTransform.GetScale3D();
					if(FMath::IsNearlyZero(ParentScale.X) || FMath::IsNearlyZero(ParentScale.Y) || FMath::IsNearlyZero(ParentScale.Z))
					{
						const FTransform& InputTransform = InTransformElement->GetTransform().Get(InTransformType); 
						Transform.SetTranslation(InputTransform.GetTranslation());
						Transform.SetScale3D(InputTransform.GetScale3D());
					}
				}
			};
			
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InTransformElement))
			{
				FTransform NewTransform = ComputeLocalControlValue(ControlElement, ControlElement->GetTransform().Get(OpposedType), GlobalType);
				CompensateZeroScale(NewTransform);
				ControlElement->GetTransform().Set(InTransformType, NewTransform);
				ControlElement->GetDirtyState().MarkClean(InTransformType);
				/** from mikez we do not want geting a pose to set these preferred angles
				switch(ControlElement->Settings.ControlType)
				{
					case ERigControlType::Rotator:
					case ERigControlType::EulerTransform:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					{
						ControlElement->PreferredEulerAngles.SetRotator(NewTransform.Rotator(), IsInitial(InTransformType), true);
						break;
					}
					default:
					{
						break;
					}
					
				}*/
			}
			else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InTransformElement))
			{
				// this is done for nulls and any element that can have more than one parent which 
				// is not a control
				const FTransform& GlobalTransform = MultiParentElement->GetTransform().Get(GlobalType);
				FTransform LocalTransform = InverseSolveParentConstraints(
					GlobalTransform, 
					MultiParentElement->ParentConstraints, GlobalType, FTransform::Identity);
				CompensateZeroScale(LocalTransform);
				MultiParentElement->GetTransform().Set(InTransformType, LocalTransform);
				MultiParentElement->GetDirtyState().MarkClean(InTransformType);
			}
			else
			{
				ParentTransform = GetParentTransform(InTransformElement, GlobalType);

				FTransform NewTransform = InTransformElement->GetTransform().Get(OpposedType).GetRelativeTransform(ParentTransform);
				NewTransform.NormalizeRotation();
				CompensateZeroScale(NewTransform);
				InTransformElement->GetTransform().Set(InTransformType, NewTransform);
				InTransformElement->GetDirtyState().MarkClean(InTransformType);
			}
		}
		else
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InTransformElement))
			{
				// using GetControlOffsetTransform to check dirty flag before accessing the transform
				// note: no need to do the same for Pose.Local because there is already an ensure:
				// "ensure(!InTransformElement->Pose.IsDirty(OpposedType));" above
				const FTransform NewTransform = SolveParentConstraints(
					ControlElement->ParentConstraints, InTransformType,
					GetControlOffsetTransform(ControlElement, OpposedType), true,
					ControlElement->GetTransform().Get(OpposedType), true);
				ControlElement->GetTransform().Set(InTransformType, NewTransform);
				ControlElement->GetDirtyState().MarkClean(InTransformType);
			}
			else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InTransformElement))
			{
				// this is done for nulls and any element that can have more than one parent which 
				// is not a control
				const FTransform NewTransform = SolveParentConstraints(
					MultiParentElement->ParentConstraints, InTransformType,
					FTransform::Identity, false,
					MultiParentElement->GetTransform().Get(OpposedType), true);
				MultiParentElement->GetTransform().Set(InTransformType, NewTransform);
				MultiParentElement->GetDirtyState().MarkClean(InTransformType);
			}
			else
			{
				ParentTransform = GetParentTransform(InTransformElement, GlobalType);

				FTransform NewTransform = InTransformElement->GetTransform().Get(OpposedType) * ParentTransform;
				NewTransform.NormalizeRotation();
				InTransformElement->GetTransform().Set(InTransformType, NewTransform);
				InTransformElement->GetDirtyState().MarkClean(InTransformType);
			}
		}

		EnsureCacheValidity();
	}
	return InTransformElement->GetTransform().Get(InTransformType);
}

void URigHierarchy::SetTransform(FRigTransformElement* InTransformElement, const FTransform& InTransform, const ERigTransformType::Type InTransformType, bool bAffectChildren, bool bSetupUndo, bool bForce, bool bPrintPythonCommands)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InTransformElement == nullptr)
	{
		return;
	}

	if(IsGlobal(InTransformType))
	{
		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InTransformElement))
		{
			FTransform LocalTransform = ComputeLocalControlValue(ControlElement, InTransform, InTransformType);
			ControlElement->Settings.ApplyLimits(LocalTransform);
			SetTransform(ControlElement, LocalTransform, MakeLocal(InTransformType), bAffectChildren, false, false, bPrintPythonCommands);
			return;
		}
	}

#if WITH_EDITOR

	// lock execute context scope
	{
		if(bRecordInstructionsAtRuntime)
		{
			UE::TScopeLock Lock(ExecuteContextLock);
			if (ExecuteContext)
			{
				const FControlRigExecuteContext& PublicData = ExecuteContext->GetPublicData<FControlRigExecuteContext>();
				const FRigVMSlice& Slice = ExecuteContext->GetSlice();
				const FName VMName = PublicData.GetRigModuleInstance() ? PublicData.GetRigModuleInstance()->Name : NAME_None;
				const FInstructionRecord Record(PublicData.GetInstructionIndex(), Slice.GetIndex(), InTransformElement->GetIndex(), VMName);
				PublicData.AddWrittenInstructionRecord(Record);

				// if we are setting a control / null parent after a child here - let's let the user know
				if(InTransformElement->IsA<FRigControlElement>() || InTransformElement->IsA<FRigNullElement>())
				{
					if(const UWorld* World = GetWorld())
					{
						// only fire these notes if we are inside the asset editor
						if(World->WorldType == EWorldType::EditorPreview)
						{
							for (FRigBaseElement* Child : GetChildren(InTransformElement))
							{
								const bool bChildFound = PublicData.GetInstructionRecords().WrittenRecords.ContainsByPredicate([Child](const FInstructionRecord& Record) -> bool
								{
									return Record.ElementIndex == Child->GetIndex();
								});

								if(bChildFound)
								{
									const FControlRigExecuteContext& CRContext = ExecuteContext->GetPublicData<FControlRigExecuteContext>();
									if(CRContext.GetLog())
									{
										static constexpr TCHAR MessageFormat[] = TEXT("Setting transform of parent (%s) after setting child (%s).\nThis may lead to unexpected results.");
										const FString& Message = FString::Printf(
											MessageFormat,
											*InTransformElement->GetName(),
											*Child->GetName());
										CRContext.Report(
											EMessageSeverity::Info,
											ExecuteContext->GetPublicData<>().GetFunctionName(),
											ExecuteContext->GetPublicData<>().GetInstructionIndex(),
											Message);
									}
								}
							}
						}
					}
				}
			}
		}
	}
	
	TGuardValue<bool> RecordTransformsPerInstructionGuard(bRecordInstructionsAtRuntime, false);
	
#endif

	if(!InTransformElement->GetDirtyState().IsDirty(InTransformType))
	{
		const FTransform PreviousTransform = InTransformElement->GetTransform().Get(InTransformType);
		if(!bForce && FRigComputedTransform::Equals(PreviousTransform, InTransform))
		{
			return;
		}
	}

	const FTransform PreviousTransform = GetTransform(InTransformElement, InTransformType);
	PropagateDirtyFlags(InTransformElement, ERigTransformType::IsInitial(InTransformType), bAffectChildren);

	const ERigTransformType::Type OpposedType = SwapLocalAndGlobal(InTransformType);
	InTransformElement->GetTransform().Set(InTransformType, InTransform);
	InTransformElement->GetDirtyState().MarkClean(InTransformType);
	InTransformElement->GetDirtyState().MarkDirty(OpposedType);
	IncrementPoseVersion(InTransformElement->Index);

	if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InTransformElement))
	{
		ControlElement->GetShapeDirtyState().MarkDirty(MakeGlobal(InTransformType));

		if(bUsePreferredEulerAngles && ERigTransformType::IsLocal(InTransformType))
		{
			const bool bInitial = ERigTransformType::IsInitial(InTransformType);
			const FVector Angle = GetControlAnglesFromQuat(ControlElement, InTransform.GetRotation(), true);
			ControlElement->PreferredEulerAngles.SetAngles(Angle, bInitial, ControlElement->PreferredEulerAngles.RotationOrder, true);
		}
	}

	EnsureCacheValidity();
	
#if WITH_EDITOR
	if(bSetupUndo || IsTracingChanges())
	{
		PushTransformToStack(
			InTransformElement->GetKey(),
			ERigTransformStackEntryType::TransformPose,
			InTransformType,
			PreviousTransform,
			InTransformElement->GetTransform().Get(InTransformType),
			bAffectChildren,
			bSetupUndo);
	}

	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);

		ForEachListeningHierarchy([this, InTransformElement, InTransform, InTransformType, bAffectChildren, bForce](const FRigHierarchyListener& Listener)
		{
			if(!bForcePropagation && !Listener.ShouldReactToChange(InTransformType))
			{
				return;
			}

			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{			
				if(FRigTransformElement* ListeningElement = Cast<FRigTransformElement>(ListeningHierarchy->Find(InTransformElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undos
					ListeningHierarchy->SetTransform(ListeningElement, InTransform, InTransformType, bAffectChildren, false, bForce);
				}
			}
		});
	}

	if (bPrintPythonCommands)
	{
		FString BlueprintName;
		if (UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			BlueprintName = Blueprint->GetFName().ToString();
		}
		else if (UControlRig* Rig = Cast<UControlRig>(GetOuter()))
		{
			if (UBlueprint* BlueprintCR = Cast<UBlueprint>(Rig->GetClass()->ClassGeneratedBy))
			{
				BlueprintName = BlueprintCR->GetFName().ToString();
			}
		}
		if (!BlueprintName.IsEmpty())
		{
			FString MethodName;
			switch (InTransformType)
			{
				case ERigTransformType::InitialLocal: 
				case ERigTransformType::CurrentLocal:
				{
					MethodName = TEXT("set_local_transform");
					break;
				}
				case ERigTransformType::InitialGlobal: 
				case ERigTransformType::CurrentGlobal:
				{
					MethodName = TEXT("set_global_transform");
					break;
				}
			}

			RigVMPythonUtils::Print(BlueprintName,
				FString::Printf(TEXT("hierarchy.%s(%s, %s, %s, %s)"),
				*MethodName,
				*InTransformElement->GetKey().ToPythonString(),
				*RigVMPythonUtils::TransformToPythonString(InTransform),
				(InTransformType == ERigTransformType::InitialGlobal || InTransformType == ERigTransformType::InitialLocal) ? TEXT("True") : TEXT("False"),
				(bAffectChildren) ? TEXT("True") : TEXT("False")));
		}
	}
#endif
}

FTransform URigHierarchy::GetControlOffsetTransform(FRigControlElement* InControlElement,
	const ERigTransformType::Type InTransformType) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InControlElement == nullptr)
	{
		return FTransform::Identity;
	}

#if WITH_EDITOR

	// lock execute context scope
	{
		if(bRecordInstructionsAtRuntime) 
		{
			UE::TScopeLock Lock(ExecuteContextLock);
			if (ExecuteContext)
			{
				const FControlRigExecuteContext& PublicData = ExecuteContext->GetPublicData<FControlRigExecuteContext>();
				const FName VMName = PublicData.GetRigModuleInstance() ? PublicData.GetRigModuleInstance()->Name : NAME_None;

				const FInstructionRecord Record(PublicData.GetInstructionIndex(),
					ExecuteContext->GetSlice().GetIndex(),
					InControlElement->GetIndex(),
					VMName);

				PublicData.AddReadInstructionRecord(Record);
			}
		}
	}
	
	TGuardValue<bool> RecordTransformsPerInstructionGuard(bRecordInstructionsAtRuntime, false);
	
#endif

	if(InControlElement->GetOffsetDirtyState().IsDirty(InTransformType))
	{
		const ERigTransformType::Type OpposedType = SwapLocalAndGlobal(InTransformType);
		const ERigTransformType::Type GlobalType = MakeGlobal(InTransformType);
		ensure(!InControlElement->GetOffsetDirtyState().IsDirty(OpposedType));

		if(IsLocal(InTransformType))
		{
			const FTransform& GlobalTransform = InControlElement->GetOffsetTransform().Get(GlobalType);
			const FTransform LocalTransform = InverseSolveParentConstraints(
				GlobalTransform, 
				InControlElement->ParentConstraints, GlobalType, FTransform::Identity);
			InControlElement->GetOffsetTransform().Set(InTransformType, LocalTransform);
			InControlElement->GetOffsetDirtyState().MarkClean(InTransformType);

			if(bEnableCacheValidityCheck)
			{
				const FTransform ComputedTransform = SolveParentConstraints(
					InControlElement->ParentConstraints, MakeGlobal(InTransformType),
					LocalTransform, true,
					FTransform::Identity, false);

				const TArray<FString>& TransformTypeStrings = GetTransformTypeStrings();

				checkf(FRigComputedTransform::Equals(GlobalTransform, ComputedTransform),
					TEXT("Element '%s' Offset %s Cached vs Computed doesn't match. ('%s' <-> '%s')"),
					*InControlElement->GetName(),
					*TransformTypeStrings[(int32)InTransformType],
					*GlobalTransform.ToString(), *ComputedTransform.ToString());
			}
		}
		else
		{
			const FTransform& LocalTransform = InControlElement->GetOffsetTransform().Get(OpposedType); 
			const FTransform GlobalTransform = SolveParentConstraints(
				InControlElement->ParentConstraints, InTransformType,
				LocalTransform, true,
				FTransform::Identity, false);
			InControlElement->GetOffsetTransform().Set(InTransformType, GlobalTransform);
			InControlElement->GetOffsetDirtyState().MarkClean(InTransformType);

			if(bEnableCacheValidityCheck)
			{
				const FTransform ComputedTransform = InverseSolveParentConstraints(
					GlobalTransform, 
					InControlElement->ParentConstraints, GlobalType, FTransform::Identity);

				const TArray<FString>& TransformTypeStrings = GetTransformTypeStrings();

				checkf(FRigComputedTransform::Equals(LocalTransform, ComputedTransform),
					TEXT("Element '%s' Offset %s Cached vs Computed doesn't match. ('%s' <-> '%s')"),
					*InControlElement->GetName(),
					*TransformTypeStrings[(int32)InTransformType],
					*LocalTransform.ToString(), *ComputedTransform.ToString());
			}
		}

		EnsureCacheValidity();
	}
	return InControlElement->GetOffsetTransform().Get(InTransformType);
}

void URigHierarchy::SetControlOffsetTransform(FRigControlElement* InControlElement, const FTransform& InTransform,
                                              const ERigTransformType::Type InTransformType, bool bAffectChildren, bool bSetupUndo, bool bForce, bool bPrintPythonCommands)
{
	if(InControlElement == nullptr)
	{
		return;
	}

#if WITH_EDITOR

	// lock execute context scope
	{
		if(bRecordInstructionsAtRuntime)
		{
			UE::TScopeLock Lock(ExecuteContextLock);
			if (ExecuteContext)
			{
				const FControlRigExecuteContext& PublicData = ExecuteContext->GetPublicData<FControlRigExecuteContext>();
				const FName VMName = PublicData.GetRigModuleInstance() ? PublicData.GetRigModuleInstance()->Name : NAME_None;
				const FInstructionRecord Record(PublicData.GetInstructionIndex(), ExecuteContext->GetSlice().GetIndex(), InControlElement->GetIndex(), VMName);
				PublicData.AddWrittenInstructionRecord(Record);
			}
		}
	}
	
	TGuardValue<bool> RecordTransformsPerInstructionGuard(bRecordInstructionsAtRuntime, false);
	
#endif

	if(!InControlElement->GetOffsetDirtyState().IsDirty(InTransformType))
	{
		const FTransform PreviousTransform = InControlElement->GetOffsetTransform().Get(InTransformType);
		if(!bForce && FRigComputedTransform::Equals(PreviousTransform, InTransform))
		{
			return;
		}
	}
	
	const FTransform PreviousTransform = GetControlOffsetTransform(InControlElement, InTransformType);
	PropagateDirtyFlags(InControlElement, ERigTransformType::IsInitial(InTransformType), bAffectChildren);

	GetTransform(InControlElement, MakeLocal(InTransformType));
	InControlElement->GetDirtyState().MarkDirty(MakeGlobal(InTransformType));

	const ERigTransformType::Type OpposedType = SwapLocalAndGlobal(InTransformType);
	InControlElement->GetOffsetTransform().Set(InTransformType, InTransform);
	InControlElement->GetOffsetDirtyState().MarkClean(InTransformType);
	InControlElement->GetOffsetDirtyState().MarkDirty(OpposedType);
	InControlElement->GetShapeDirtyState().MarkDirty(MakeGlobal(InTransformType));

	EnsureCacheValidity();

	if (ERigTransformType::IsInitial(InTransformType))
	{
		// control's offset transform is considered a special type of transform
		// whenever its initial value is changed, we want to make sure the current is kept in sync
		// such that the viewport can reflect this change
		SetControlOffsetTransform(InControlElement, InTransform, ERigTransformType::MakeCurrent(InTransformType), bAffectChildren, false, bForce);
	}
	

#if WITH_EDITOR
	if(bSetupUndo || IsTracingChanges())
	{
		PushTransformToStack(
            InControlElement->GetKey(),
            ERigTransformStackEntryType::ControlOffset,
            InTransformType,
            PreviousTransform,
            InControlElement->GetOffsetTransform().Get(InTransformType),
            bAffectChildren,
            bSetupUndo);
	}

	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InControlElement, InTransform, InTransformType, bAffectChildren, bForce](const FRigHierarchyListener& Listener)
		{
			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{	
				if(FRigControlElement* ListeningElement = Cast<FRigControlElement>(ListeningHierarchy->Find(InControlElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undos
					ListeningHierarchy->SetControlOffsetTransform(ListeningElement, InTransform, InTransformType, bAffectChildren, false, bForce);
				}
			}
		});
	}

	if (bPrintPythonCommands)
	{
		FString BlueprintName;
		if (UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			BlueprintName = Blueprint->GetFName().ToString();
		}
		else if (UControlRig* Rig = Cast<UControlRig>(GetOuter()))
		{
			if (UBlueprint* BlueprintCR = Cast<UBlueprint>(Rig->GetClass()->ClassGeneratedBy))
			{
				BlueprintName = BlueprintCR->GetFName().ToString();
			}
		}
		if (!BlueprintName.IsEmpty())
		{
			RigVMPythonUtils::Print(BlueprintName,
				FString::Printf(TEXT("hierarchy.set_control_offset_transform(%s, %s, %s, %s)"),
				*InControlElement->GetKey().ToPythonString(),
				*RigVMPythonUtils::TransformToPythonString(InTransform),
				(ERigTransformType::IsInitial(InTransformType)) ? TEXT("True") : TEXT("False"),
				(bAffectChildren) ? TEXT("True") : TEXT("False")));
		}
	}
#endif
}

FTransform URigHierarchy::GetControlShapeTransform(FRigControlElement* InControlElement,
	const ERigTransformType::Type InTransformType) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InControlElement == nullptr)
	{
		return FTransform::Identity;
	}
	
	if(InControlElement->GetShapeDirtyState().IsDirty(InTransformType))
	{
		const ERigTransformType::Type OpposedType = SwapLocalAndGlobal(InTransformType);
		const ERigTransformType::Type GlobalType = MakeGlobal(InTransformType);
		ensure(!InControlElement->GetShapeDirtyState().IsDirty(OpposedType));

		const FTransform ParentTransform = GetTransform(InControlElement, GlobalType);
		if(IsLocal(InTransformType))
		{
			FTransform LocalTransform = InControlElement->GetShapeTransform().Get(OpposedType).GetRelativeTransform(ParentTransform);
			LocalTransform.NormalizeRotation();
			InControlElement->GetShapeTransform().Set(InTransformType, LocalTransform);
			InControlElement->GetShapeDirtyState().MarkClean(InTransformType);
		}
		else
		{
			FTransform GlobalTransform = InControlElement->GetShapeTransform().Get(OpposedType) * ParentTransform;
			GlobalTransform.NormalizeRotation();
			InControlElement->GetShapeTransform().Set(InTransformType, GlobalTransform);
			InControlElement->GetShapeDirtyState().MarkClean(InTransformType);
		}

		EnsureCacheValidity();
	}
	return InControlElement->GetShapeTransform().Get(InTransformType);
}

void URigHierarchy::SetControlShapeTransform(FRigControlElement* InControlElement, const FTransform& InTransform,
	const ERigTransformType::Type InTransformType, bool bSetupUndo, bool bForce, bool bPrintPythonCommands)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InControlElement == nullptr)
	{
		return;
	}

	if(!InControlElement->GetShapeDirtyState().IsDirty(InTransformType))
	{
		const FTransform PreviousTransform = InControlElement->GetShapeTransform().Get(InTransformType);
		if(!bForce && FRigComputedTransform::Equals(PreviousTransform, InTransform))
		{
			return;
		}
	}

	const FTransform PreviousTransform = GetControlShapeTransform(InControlElement, InTransformType);
	const ERigTransformType::Type OpposedType = SwapLocalAndGlobal(InTransformType);
	InControlElement->GetShapeTransform().Set(InTransformType, InTransform);
	InControlElement->GetShapeDirtyState().MarkClean(InTransformType);
	InControlElement->GetShapeDirtyState().MarkDirty(OpposedType);

	if (IsInitial(InTransformType))
	{
		// control's shape transform, similar to offset transform, is considered a special type of transform
		// whenever its initial value is changed, we want to make sure the current is kept in sync
		// such that the viewport can reflect this change
		SetControlShapeTransform(InControlElement, InTransform, ERigTransformType::MakeCurrent(InTransformType), false, bForce);
	}
	
	EnsureCacheValidity();
	
#if WITH_EDITOR
	if(bSetupUndo || IsTracingChanges())
	{
		PushTransformToStack(
            InControlElement->GetKey(),
            ERigTransformStackEntryType::ControlShape,
            InTransformType,
            PreviousTransform,
            InControlElement->GetShapeTransform().Get(InTransformType),
            false,
            bSetupUndo);
	}
#endif

	if(IsLocal(InTransformType))
	{
		Notify(ERigHierarchyNotification::ControlShapeTransformChanged, InControlElement);
	}

#if WITH_EDITOR
	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InControlElement, InTransform, InTransformType, bForce](const FRigHierarchyListener& Listener)
		{
			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{	
				if(FRigControlElement* ListeningElement = Cast<FRigControlElement>(ListeningHierarchy->Find(InControlElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undos
					ListeningHierarchy->SetControlShapeTransform(ListeningElement, InTransform, InTransformType, false, bForce);
				}
			}
		});
	}

	if (bPrintPythonCommands)
	{
		FString BlueprintName;
		if (UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			BlueprintName = Blueprint->GetFName().ToString();
		}
		else if (UControlRig* Rig = Cast<UControlRig>(GetOuter()))
		{
			if (UBlueprint* BlueprintCR = Cast<UBlueprint>(Rig->GetClass()->ClassGeneratedBy))
			{
				BlueprintName = BlueprintCR->GetFName().ToString();
			}
		}
		if (!BlueprintName.IsEmpty())
		{
			RigVMPythonUtils::Print(BlueprintName,
				FString::Printf(TEXT("hierarchy.set_control_shape_transform(%s, %s, %s)"),
				*InControlElement->GetKey().ToPythonString(),
				*RigVMPythonUtils::TransformToPythonString(InTransform),
				ERigTransformType::IsInitial(InTransformType) ? TEXT("True") : TEXT("False")));
		}
	
	}
#endif
}

void URigHierarchy::SetControlSettings(FRigControlElement* InControlElement, FRigControlSettings InSettings, bool bSetupUndo, bool bForce, bool bPrintPythonCommands)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InControlElement == nullptr)
	{
		return;
	}

	const FRigControlSettings PreviousSettings = InControlElement->Settings;
	if(!bForce && PreviousSettings == InSettings)
	{
		return;
	}

	if(bSetupUndo && !HasAnyFlags(RF_Transient))
	{
		Modify();
	}

	InControlElement->Settings = InSettings;
	Notify(ERigHierarchyNotification::ControlSettingChanged, InControlElement);
	
#if WITH_EDITOR
	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InControlElement, InSettings, bForce](const FRigHierarchyListener& Listener)
		{
			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{	
				if(FRigControlElement* ListeningElement = Cast<FRigControlElement>(ListeningHierarchy->Find(InControlElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undos
					ListeningHierarchy->SetControlSettings(ListeningElement, InSettings, false, bForce);
				}
			}
		});
	}

	if (bPrintPythonCommands)
	{
		FString BlueprintName;
		if (UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			BlueprintName = Blueprint->GetFName().ToString();
		}
		else if (UControlRig* Rig = Cast<UControlRig>(GetOuter()))
		{
			if (UBlueprint* BlueprintCR = Cast<UBlueprint>(Rig->GetClass()->ClassGeneratedBy))
			{
				BlueprintName = BlueprintCR->GetFName().ToString();
			}
		}
		if (!BlueprintName.IsEmpty())
		{
			FString ControlNamePythonized = RigVMPythonUtils::PythonizeName(InControlElement->GetName());
			FString SettingsName = FString::Printf(TEXT("control_settings_%s"),
				*ControlNamePythonized);
			TArray<FString> Commands = ControlSettingsToPythonCommands(InControlElement->Settings, SettingsName);

			for (const FString& Command : Commands)
			{
				RigVMPythonUtils::Print(BlueprintName, Command);
			}
			
			RigVMPythonUtils::Print(BlueprintName,
				FString::Printf(TEXT("hierarchy.set_control_settings(%s, %s)"),
				*InControlElement->GetKey().ToPythonString(),
				*SettingsName));
		}
	}
#endif
}

FTransform URigHierarchy::GetParentTransform(FRigBaseElement* InElement, const ERigTransformType::Type InTransformType) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InElement))
	{
		return GetTransform(SingleParentElement->ParentElement, InTransformType);
	}
	else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		const FTransform OutputTransform = SolveParentConstraints(
			MultiParentElement->ParentConstraints,
			InTransformType,
			FTransform::Identity,
			false,
			FTransform::Identity,
			false
		);
		EnsureCacheValidity();
		return OutputTransform;
	}
	return FTransform::Identity;
}

FRigControlValue URigHierarchy::GetControlValue(FRigControlElement* InControlElement, ERigControlValueType InValueType, bool bUsePreferredAngles) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	using namespace ERigTransformType;

	FRigControlValue Value;

	if(InControlElement != nullptr)
	{
		auto GetValueFromPreferredEulerAngles = [this, InControlElement, &Value, InValueType, bUsePreferredAngles]() -> bool
		{
			if (!bUsePreferredAngles)
			{
				return false;
			}
			
			const bool bInitial = InValueType == ERigControlValueType::Initial;
			const ERigTransformType::Type TransformType = bInitial ? InitialLocal : CurrentLocal;
			switch(InControlElement->Settings.ControlType)
			{
				case ERigControlType::Rotator:
				{
					Value = MakeControlValueFromRotator(InControlElement->PreferredEulerAngles.GetRotator(bInitial)); 
					return true;
				}
				case ERigControlType::EulerTransform:
				{
					FEulerTransform EulerTransform(GetTransform(InControlElement, TransformType));
					EulerTransform.Rotation = InControlElement->PreferredEulerAngles.GetRotator(bInitial);
					Value = MakeControlValueFromEulerTransform(EulerTransform); 
					return true;
				}
				default:
				{
					break;
				}
			}
			return false;
		};
		
		switch(InValueType)
		{
			case ERigControlValueType::Current:
			{
				if(GetValueFromPreferredEulerAngles())
				{
					break;
				}
					
				Value.SetFromTransform(
                    GetTransform(InControlElement, CurrentLocal),
                    InControlElement->Settings.ControlType,
                    InControlElement->Settings.PrimaryAxis
                );
				break;
			}
			case ERigControlValueType::Initial:
			{
				if(GetValueFromPreferredEulerAngles())
				{
					break;
				}

				Value.SetFromTransform(
                    GetTransform(InControlElement, InitialLocal),
                    InControlElement->Settings.ControlType,
                    InControlElement->Settings.PrimaryAxis
                );
				break;
			}
			case ERigControlValueType::Minimum:
			{
				return InControlElement->Settings.MinimumValue;
			}
			case ERigControlValueType::Maximum:
			{
				return InControlElement->Settings.MaximumValue;
			}
		}
	}
	return Value;
}

void URigHierarchy::SetPreferredEulerAnglesFromValue(FRigControlElement* InControlElement,
	const FRigControlValue& InValue,
	const ERigControlValueType& InValueType,
	const bool bFixEulerFlips) const
{
	const bool bInitial = InValueType == ERigControlValueType::Initial;

	FRigPreferredEulerAngles& PreferredEulerAngles = InControlElement->PreferredEulerAngles;
	const EEulerRotationOrder RotationOrder = PreferredEulerAngles.RotationOrder;
	
	switch(InControlElement->Settings.ControlType)
	{
	case ERigControlType::Rotator:
		{
			const FVector3f EulerXYZf = InValue.Get<FVector3f>();
			const FVector EulerXYZ(EulerXYZf.X, EulerXYZf.Y, EulerXYZf.Z);
			if (InControlElement->Settings.bUsePreferredRotationOrder)
			{
				const FVector Angle = AnimationCore::ChangeEulerRotationOrder(EulerXYZ, EEulerRotationOrder::XYZ, RotationOrder);
				PreferredEulerAngles.SetAngles(Angle, bInitial, RotationOrder, bFixEulerFlips);
			}
			else
			{
				PreferredEulerAngles.SetRotator(FRotator::MakeFromEuler(EulerXYZ), bInitial, bFixEulerFlips);
			}
			break;
		}
	case ERigControlType::EulerTransform:
		{
			FEulerTransform EulerTransform = GetEulerTransformFromControlValue(InValue);
			FQuat Quat = EulerTransform.GetRotation();
			const FVector Angle = GetControlAnglesFromQuat(InControlElement, Quat, bFixEulerFlips);
			PreferredEulerAngles.SetAngles(Angle, bInitial, RotationOrder, bFixEulerFlips);
			break;
		}
	case ERigControlType::Transform:
		{
			FTransform Transform = GetTransformFromControlValue(InValue);
			FQuat Quat = Transform.GetRotation();
			const FVector Angle = GetControlAnglesFromQuat(InControlElement, Quat, bFixEulerFlips);
			PreferredEulerAngles.SetAngles(Angle, bInitial, RotationOrder, bFixEulerFlips);
			break;
		}
	case ERigControlType::TransformNoScale:
		{
			FTransform Transform = GetTransformNoScaleFromControlValue(InValue);
			FQuat Quat = Transform.GetRotation();
			const FVector Angle = GetControlAnglesFromQuat(InControlElement, Quat, bFixEulerFlips);
			PreferredEulerAngles.SetAngles(Angle, bInitial, RotationOrder, bFixEulerFlips);
			break;
		}
	default:
		{
			break;
		}
	}
};

void URigHierarchy::SetControlValue(FRigControlElement* InControlElement, const FRigControlValue& InValue, ERigControlValueType InValueType, bool bSetupUndo, bool bForce, bool bPrintPythonCommands, bool bFixEulerFlips)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	using namespace ERigTransformType;

	if(InControlElement != nullptr)
	{
		switch(InValueType)
		{
			case ERigControlValueType::Current:
			{
				FRigControlValue Value = InValue;
				InControlElement->Settings.ApplyLimits(Value);

				TGuardValue<bool> DontSetPreferredEulerAngle(bUsePreferredEulerAngles, false);
				SetTransform(
					InControlElement,
					Value.GetAsTransform(
						InControlElement->Settings.ControlType,
						InControlElement->Settings.PrimaryAxis
					),
					CurrentLocal,
					true,
					bSetupUndo,
					bForce,
					bPrintPythonCommands
				);
				if (bFixEulerFlips)
				{
					SetPreferredEulerAnglesFromValue(InControlElement, Value, InValueType, bFixEulerFlips);
				}
				break;
			}
			case ERigControlValueType::Initial:
			{
				FRigControlValue Value = InValue;
				InControlElement->Settings.ApplyLimits(Value);

				TGuardValue<bool> DontSetPreferredEulerAngle(bUsePreferredEulerAngles, false);
				SetTransform(
					InControlElement,
					Value.GetAsTransform(
						InControlElement->Settings.ControlType,
						InControlElement->Settings.PrimaryAxis
					),
					InitialLocal,
					true,
					bSetupUndo,
					bForce,
					bPrintPythonCommands
				);

				if (bFixEulerFlips)
				{
					SetPreferredEulerAnglesFromValue(InControlElement, Value, InValueType, bFixEulerFlips);
				}
				break;
			}
			case ERigControlValueType::Minimum:
			case ERigControlValueType::Maximum:
			{
				if(bSetupUndo)
				{
					Modify();
				}

				if(InValueType == ERigControlValueType::Minimum)
				{
					FRigControlSettings& Settings = InControlElement->Settings;
					Settings.MinimumValue = InValue;

					// Make sure the maximum value respects the new minimum value
					TArray<FRigControlLimitEnabled> NoMaxLimits = Settings.LimitEnabled;
					for (FRigControlLimitEnabled& NoMaxLimit : NoMaxLimits)
					{
						NoMaxLimit.bMaximum = false;
					}
					Settings.MaximumValue.ApplyLimits(NoMaxLimits, Settings.ControlType, Settings.MinimumValue, Settings.MaximumValue);
				}
				else
				{
					FRigControlSettings& Settings = InControlElement->Settings;
					Settings.MaximumValue = InValue;
					
					// Make sure the minimum value respects the new maximum value
					TArray<FRigControlLimitEnabled> NoMinLimits = Settings.LimitEnabled;
					for (FRigControlLimitEnabled& NoMinLimit : NoMinLimits)
					{
						NoMinLimit.bMinimum = false;
					}
					Settings.MinimumValue.ApplyLimits(NoMinLimits, Settings.ControlType, Settings.MinimumValue, Settings.MaximumValue);
				}
				
				Notify(ERigHierarchyNotification::ControlSettingChanged, InControlElement);

#if WITH_EDITOR
				if (!bPropagatingChange)
				{
					TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
					ForEachListeningHierarchy([this, InControlElement, InValue, InValueType, bForce](const FRigHierarchyListener& Listener)
					{
						if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
						{
							if(FRigControlElement* ListeningElement = Cast<FRigControlElement>(ListeningHierarchy->Find(InControlElement->GetKey())))
							{
								ListeningHierarchy->SetControlValue(ListeningElement, InValue, InValueType, false, bForce);
							}
						}
					});
				}

				if (bPrintPythonCommands)
				{
					FString BlueprintName;
					if (UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
					{
						BlueprintName = Blueprint->GetFName().ToString();
					}
					else if (UControlRig* Rig = Cast<UControlRig>(GetOuter()))
					{
						if (UBlueprint* BlueprintCR = Cast<UBlueprint>(Rig->GetClass()->ClassGeneratedBy))
						{
							BlueprintName = BlueprintCR->GetFName().ToString();
						}
					}
					if (!BlueprintName.IsEmpty())
					{
						RigVMPythonUtils::Print(BlueprintName,
							FString::Printf(TEXT("hierarchy.set_control_value(%s, %s, %s)"),
							*InControlElement->GetKey().ToPythonString(),
							*InValue.ToPythonString(InControlElement->Settings.ControlType),
							*RigVMPythonUtils::EnumValueToPythonString<ERigControlValueType>((int64)InValueType)));
					}
				}
#endif
				break;
			}
		}	
	}
}

void URigHierarchy::SetControlVisibility(FRigControlElement* InControlElement, bool bVisibility)
{
	if(InControlElement == nullptr)
	{
		return;
	}

	if(InControlElement->Settings.SetVisible(bVisibility))
	{
		Notify(ERigHierarchyNotification::ControlVisibilityChanged, InControlElement);
	}

#if WITH_EDITOR
	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InControlElement, bVisibility](const FRigHierarchyListener& Listener)
		{
			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{
				if(FRigControlElement* ListeningElement = Cast<FRigControlElement>(ListeningHierarchy->Find(InControlElement->GetKey())))
				{
					ListeningHierarchy->SetControlVisibility(ListeningElement, bVisibility);
				}
			}
		});
	}
#endif
}

void URigHierarchy::SetConnectorSettings(FRigConnectorElement* InConnectorElement, FRigConnectorSettings InSettings,
	bool bSetupUndo, bool bForce, bool bPrintPythonCommands)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InConnectorElement == nullptr)
	{
		return;
	}

	const FRigConnectorSettings PreviousSettings = InConnectorElement->Settings;
	if(!bForce && PreviousSettings == InSettings)
	{
		return;
	}

	if (InSettings.Type == EConnectorType::Primary)
	{
		if (InSettings.bOptional)
		{
			return;
		}
	}

	if(bSetupUndo && !HasAnyFlags(RF_Transient))
	{
		Modify();
	}

	FRigConnectorSettings Settings = InSettings;
	Settings.bIsArray = InConnectorElement->IsPrimary() ? false : InSettings.bIsArray;

	InConnectorElement->Settings = Settings;
	Notify(ERigHierarchyNotification::ConnectorSettingChanged, InConnectorElement);
	
#if WITH_EDITOR
	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InConnectorElement, InSettings, bForce](const FRigHierarchyListener& Listener)
		{
			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{	
				if(FRigConnectorElement* ListeningElement = Cast<FRigConnectorElement>(ListeningHierarchy->Find(InConnectorElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undos
					ListeningHierarchy->SetConnectorSettings(ListeningElement, InSettings, false, bForce);
				}
			}
		});
	}

	if (bPrintPythonCommands)
	{
		FString BlueprintName;
		if (UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			BlueprintName = Blueprint->GetFName().ToString();
		}
		else if (UControlRig* Rig = Cast<UControlRig>(GetOuter()))
		{
			if (UBlueprint* BlueprintCR = Cast<UBlueprint>(Rig->GetClass()->ClassGeneratedBy))
			{
				BlueprintName = BlueprintCR->GetFName().ToString();
			}
		}
		if (!BlueprintName.IsEmpty())
		{
			FString ControlNamePythonized = RigVMPythonUtils::PythonizeName(InConnectorElement->GetName());
			FString SettingsName = FString::Printf(TEXT("connector_settings_%s"),
				*ControlNamePythonized);
			TArray<FString> Commands = ConnectorSettingsToPythonCommands(InConnectorElement->Settings, SettingsName);

			for (const FString& Command : Commands)
			{
				RigVMPythonUtils::Print(BlueprintName, Command);
			}
			
			RigVMPythonUtils::Print(BlueprintName,
				FString::Printf(TEXT("hierarchy.set_connector_settings(%s, %s)"),
				*InConnectorElement->GetKey().ToPythonString(),
				*SettingsName));
		}
	}
#endif
}


float URigHierarchy::GetCurveValue(FRigCurveElement* InCurveElement) const
{
	if(InCurveElement == nullptr)
	{
		return 0.f;
	}
	return InCurveElement->bIsValueSet ? InCurveElement->Get() : 0.f;
}


bool URigHierarchy::IsCurveValueSet(FRigCurveElement* InCurveElement) const
{
	return InCurveElement && InCurveElement->bIsValueSet;
}


void URigHierarchy::SetCurveValue(FRigCurveElement* InCurveElement, float InValue, bool bSetupUndo, bool bForce)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InCurveElement == nullptr)
	{
		return;
	}

	// we need to record the no matter what since this may be in consecutive frames.
	// if the client code desires to set the value we need to remember it as changed,
	// even if the value matches with a previous set. the ChangedCurveIndices array
	// represents a list of values that were changed by the client.
	if(bRecordCurveChanges)
	{
		ChangedCurveIndices.Add(InCurveElement->GetIndex());
	}

	const bool bPreviousIsValueSet = InCurveElement->bIsValueSet; 
	const float PreviousValue = InCurveElement->Get();
	if(!bForce && InCurveElement->bIsValueSet && FMath::IsNearlyZero(PreviousValue - InValue))
	{
		return;
	}

	InCurveElement->Set(InValue, bRecordCurveChanges);

#if WITH_EDITOR
	if(bSetupUndo || IsTracingChanges())
	{
		PushCurveToStack(InCurveElement->GetKey(), PreviousValue, InCurveElement->Get(), bPreviousIsValueSet, true, bSetupUndo);
	}

	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InCurveElement, InValue, bForce](const FRigHierarchyListener& Listener)
		{
			if(!Listener.Hierarchy.IsValid())
			{
				return;
			}

			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{
				if(FRigCurveElement* ListeningElement = Cast<FRigCurveElement>(ListeningHierarchy->Find(InCurveElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undoes
					ListeningHierarchy->SetCurveValue(ListeningElement, InValue, false, bForce);
				}
			}
		});
	}
#endif
}


void URigHierarchy::UnsetCurveValue(FRigCurveElement* InCurveElement, bool bSetupUndo, bool bForce)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(InCurveElement == nullptr)
	{
		return;
	}

	const bool bPreviousIsValueSet = InCurveElement->bIsValueSet; 
	if(!bForce && !InCurveElement->bIsValueSet)
	{
		return;
	}

	InCurveElement->bIsValueSet = false;
	ChangedCurveIndices.Remove(InCurveElement->GetIndex());

#if WITH_EDITOR
	if(bSetupUndo || IsTracingChanges())
	{
		PushCurveToStack(InCurveElement->GetKey(), InCurveElement->Get(), InCurveElement->Get(), bPreviousIsValueSet, false, bSetupUndo);
	}

	if (!bPropagatingChange)
	{
		TGuardValue<bool> bPropagatingChangeGuardValue(bPropagatingChange, true);
			
		ForEachListeningHierarchy([this, InCurveElement, bForce](const FRigHierarchyListener& Listener)
		{
			if(!Listener.Hierarchy.IsValid())
			{
				return;
			}

			if (URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{
				if(FRigCurveElement* ListeningElement = Cast<FRigCurveElement>(ListeningHierarchy->Find(InCurveElement->GetKey())))
				{
					// bSetupUndo = false such that all listening hierarchies performs undo at the same time the root hierarchy undoes
					ListeningHierarchy->UnsetCurveValue(ListeningElement, false, bForce);
				}
			}
		});
	}
#endif
}


FName URigHierarchy::GetPreviousName(const FRigElementKey& InKey) const
{
	return GetPreviousHierarchyName(InKey);
}

FName URigHierarchy::GetPreviousHierarchyName(const FRigHierarchyKey& InKey) const
{
	if(const FRigHierarchyKey* OldKeyPtr = PreviousHierarchyNameMap.Find(InKey))
	{
		return OldKeyPtr->GetFName();
	}
	return NAME_None;
}

FRigElementKey URigHierarchy::GetPreviousParent(const FRigElementKey& InKey) const
{
	return GetPreviousHierarchyParent(InKey).GetElement();
}

FRigHierarchyKey URigHierarchy::GetPreviousHierarchyParent(const FRigHierarchyKey& InKey) const
{
	if(const FRigHierarchyKey* OldParentPtr = PreviousHierarchyParentMap.Find(InKey))
	{
		return *OldParentPtr;
	}
	return FRigHierarchyKey();
}

bool URigHierarchy::IsParentedTo(FRigBaseElement* InChild, FRigBaseElement* InParent, const IRigDependenciesProvider& InDependencyProvider, FRigHierarchyDependencyChain* OutDependencyChain, bool bConsiderInactiveParents) const
{
	FRigHierarchyDependencyChain DependencyChain;

	THierarchyCache<TMap<TTuple<int32, int32>, bool>> TempDependencyCache;
	THierarchyCache<TMap<TTuple<int32, int32>, bool>>* DependencyCachePtr = &TempDependencyCache;
	
	if (OutDependencyChain == nullptr)
	{
		DependencyChain.Reserve(64);
		OutDependencyChain = &DependencyChain;

		// The element dependency cache uses active parents only, so it 
		// cannot be used here if all parents should be considered
		if (!bConsiderInactiveParents)
		{
			DependencyCachePtr = &ElementDependencyCache;
		}
	}
	check(OutDependencyChain != nullptr);

	{
		UE::TScopeLock Lock(IsDependentOnLock);
		if (!IsDependentOn(InChild, InParent, InDependencyProvider, DependencyCachePtr, *OutDependencyChain, bConsiderInactiveParents))
		{
			return false;
		}
	}

	return true;
}

bool URigHierarchy::IsDependentOn(
	FRigBaseElement* InDependent,
	FRigBaseElement* InDependency,
	const IRigDependenciesProvider& InDependencyProvider,
	THierarchyCache<TMap<TTuple<int32, int32>, bool>>* InElementDependencyCache,
	FRigHierarchyDependencyChain& OutDependencyChain,
	bool bConsiderInactiveParents) const
{
	if((InDependent == nullptr) || (InDependency == nullptr))
	{
		return false;
	}

	if(InDependent == InDependency)
	{
		return true;
	}

	const int32 DependentElementIndex = InDependent->GetIndex();
	const int32 DependencyElementIndex = InDependency->GetIndex();
	const TTuple<int32,int32> CacheKey(DependentElementIndex, DependencyElementIndex);

	const FRigHierarchyRecord DependentRecord(FRigHierarchyRecord::EType_RigElement, DependentElementIndex, NAME_None);

	if (OutDependencyChain.ContainsByPredicate([DependentRecord](const TPair<FRigHierarchyRecord, FRigHierarchyRecord>& InPair) -> bool 
	{
		return InPair.Value == DependentRecord;
	}))
	{
		return false;
	}
	
	if (InElementDependencyCache)
	{
		const uint32 ElementDependencyVersion = HashCombine(GetTopologyVersion(), GetParentWeightVersion()); 
		if(!InElementDependencyCache->IsValid(ElementDependencyVersion))
		{
			InElementDependencyCache->Set(TMap<TTuple<int32, int32>, bool>(), ElementDependencyVersion);
		}

		// we'll only update the caches if we are following edges on the actual topology
		if(const bool* bCachedResult = InElementDependencyCache->Get().Find(CacheKey))
		{
			return *bCachedResult;
		}

		// check if the reverse dependency check has been stored before - if the dependency is dependent
		// then we don't need to recurse any further.
		const TTuple<int32,int32> ReverseCacheKey(DependencyElementIndex, DependentElementIndex);
		if(const bool* bReverseCachedResult = InElementDependencyCache->Get().Find(ReverseCacheKey))
		{
			if(*bReverseCachedResult)
			{
				return false;
			}
		}
	}
	
	// collect all possible parents of the dependent
	if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InDependent))
	{
		if (SingleParentElement->ParentElement)
		{
			const FRigHierarchyRecord DependencyRecord(FRigHierarchyRecord::EType_RigElement, SingleParentElement->ParentElement->GetIndex(), NAME_None);
			OutDependencyChain.Emplace(DependencyRecord, DependentRecord);
		
			if(IsDependentOn(SingleParentElement->ParentElement, InDependency, InDependencyProvider, InElementDependencyCache, OutDependencyChain, bConsiderInactiveParents))
			{
				// we'll only update the caches if we are following edges on the actual topology
				if(InElementDependencyCache)
				{
					InElementDependencyCache->Get().FindOrAdd(CacheKey, true);
				}
				return true;
			}

			OutDependencyChain.Pop();
		}
	}
	else if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InDependent))
	{
		for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
		{
			if (!ParentConstraint.ParentElement || (!bConsiderInactiveParents && ParentConstraint.Weight.IsAlmostZero()))
			{
				continue;
			}

			const FRigHierarchyRecord DependencyRecord(FRigHierarchyRecord::EType_RigElement, ParentConstraint.ParentElement->GetIndex(), NAME_None);
			OutDependencyChain.Emplace(DependencyRecord, DependentRecord);

			if(IsDependentOn(ParentConstraint.ParentElement, InDependency, InDependencyProvider, InElementDependencyCache, OutDependencyChain, bConsiderInactiveParents))
			{
				// we'll only update the caches if we are following edges on the actual topology
				if(InElementDependencyCache)
				{
					InElementDependencyCache->Get().FindOrAdd(CacheKey, true);
				}
				return true;
			}

			OutDependencyChain.Pop();
		}
	}

	struct Local
	{
		static bool FoundDependencyInProvider(
			const URigHierarchy* InHierarchy,
			FRigBaseElement* InDependency,
			const IRigDependenciesProvider& InDependencyProvider,
			THierarchyCache<TMap<TTuple<int32, int32>, bool>>* InElementDependencyCache,
			const TTuple<int32,int32>& CacheKey,
			const FRigHierarchyRecord& InDependentRecord,
			FRigHierarchyDependencyChain& OutDependencyChain,
			bool bConsiderInactiveParents)
		{
			const TRigHierarchyDependencyMap& DependencyMap = InDependencyProvider.GetRigHierarchyDependencies();
			
			if(const TSet<FRigHierarchyRecord>* DependencyRecordsPtr = DependencyMap.Find(InDependentRecord))
			{
				const TSet<FRigHierarchyRecord>& DependencyRecords = *DependencyRecordsPtr;
				for(const FRigHierarchyRecord& DependencyRecord : DependencyRecords)
				{
					if (DependencyRecord.IsElement())
					{
						ensure(InHierarchy->Elements.IsValidIndex(DependencyRecord.Index));

						OutDependencyChain.Emplace(DependencyRecord, InDependentRecord);

						if(InHierarchy->IsDependentOn(InHierarchy->Elements[DependencyRecord.Index], InDependency, InDependencyProvider, InElementDependencyCache, OutDependencyChain, bConsiderInactiveParents))
						{
							// we'll only update the caches if we are following edges on the actual topology
							if(InElementDependencyCache)
							{
								InElementDependencyCache->Get().FindOrAdd(CacheKey, true);
							}
							return true;
						}

						OutDependencyChain.Pop();
					}
					else if (DependencyRecord.IsInstruction() || DependencyRecord.IsVariable())
					{
						if (OutDependencyChain.ContainsByPredicate([DependencyRecord](const TPair<FRigHierarchyRecord,FRigHierarchyRecord>& InPair) -> bool
						{
							return InPair.Key == DependencyRecord;
						}))
						{
							return false;
						}

						OutDependencyChain.Emplace(DependencyRecord, InDependentRecord);

						if (FoundDependencyInProvider(InHierarchy, InDependency, InDependencyProvider, InElementDependencyCache, CacheKey, DependencyRecord, OutDependencyChain, bConsiderInactiveParents))
						{
							return true;
						}

						OutDependencyChain.Pop();
					}
				}
			}
			return false;
		}
	};

	// check the optional dependency map
	if (Local::FoundDependencyInProvider(this, InDependency, InDependencyProvider, InElementDependencyCache, CacheKey, DependentRecord, OutDependencyChain, bConsiderInactiveParents))
	{
		// we'll only update the caches if we are following edges on the actual topology
		if(InElementDependencyCache)
		{
			InElementDependencyCache->Get().FindOrAdd(CacheKey, true);
		}
		return true;		
	}

	// we'll only update the caches if we are following edges on the actual topology
	if(InElementDependencyCache)
	{
		InElementDependencyCache->Get().FindOrAdd(CacheKey, false);
	}

	return false;
}

int32 URigHierarchy::GetLocalIndex(const FRigBaseElement* InElement) const
{
	if(InElement == nullptr)
	{
		return INDEX_NONE;
	}
	
	if(const FRigBaseElement* ParentElement = GetFirstParent(InElement))
	{
		TConstArrayView<FRigBaseElement*> Children = GetChildren(ParentElement);
		return Children.Find(const_cast<FRigBaseElement*>(InElement));
	}

	return GetRootElements().Find(const_cast<FRigBaseElement*>(InElement));
}

bool URigHierarchy::IsTracingChanges() const
{
#if WITH_EDITOR
	return (CVarControlRigHierarchyTraceAlways->GetInt() != 0) || (TraceFramesLeft > 0);
#else
	return false;
#endif
}

#if WITH_EDITOR

void URigHierarchy::ResetTransformStack()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	TransformUndoStack.Reset();
	TransformRedoStack.Reset();
	TransformStackIndex = TransformUndoStack.Num();

	if(IsTracingChanges())
	{
		TracePoses.Reset();
		StorePoseForTrace(TEXT("BeginOfFrame"));
	}
}

void URigHierarchy::StorePoseForTrace(const FString& InPrefix)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	check(!InPrefix.IsEmpty());
	
	FName InitialKey = *FString::Printf(TEXT("%s_Initial"), *InPrefix);
	FName CurrentKey = *FString::Printf(TEXT("%s_Current"), *InPrefix);
	TracePoses.FindOrAdd(InitialKey) = GetPose(true);
	TracePoses.FindOrAdd(CurrentKey) = GetPose(false);
}

void URigHierarchy::CheckTraceFormatIfRequired()
{
	if(sRigHierarchyLastTrace != CVarControlRigHierarchyTracePrecision->GetInt())
	{
		sRigHierarchyLastTrace = CVarControlRigHierarchyTracePrecision->GetInt();
	}
}

template <class CharType>
struct TRigHierarchyJsonPrintPolicy
	: public TPrettyJsonPrintPolicy<CharType>
{
	static inline void WriteDouble(  FArchive* Stream, double Value )
	{
		URigHierarchy::CheckTraceFormatIfRequired();
		TJsonPrintPolicy<CharType>::WriteString(Stream, FString::Printf(TEXT("%.*f"), sRigHierarchyLastTrace, Value));
	}
};

void URigHierarchy::DumpTransformStackToFile(FString* OutFilePath)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(IsTracingChanges())
	{
		StorePoseForTrace(TEXT("EndOfFrame"));
	}

	FString PathName = GetPathName();
	PathName.Split(TEXT(":"), nullptr, &PathName);
	PathName.ReplaceCharInline('.', '/');

	FString Suffix;
	if(TraceFramesLeft > 0)
	{
		Suffix = FString::Printf(TEXT("_Trace_%03d"), TraceFramesCaptured);
	}

	FString FileName = FString::Printf(TEXT("%sControlRig/%s%s.json"), *FPaths::ProjectLogDir(), *PathName, *Suffix);
	FString FullFilename = FPlatformFileManager::Get().GetPlatformFile().ConvertToAbsolutePathForExternalAppForWrite(*FileName);

	TSharedPtr<FJsonObject> JsonData = MakeShareable(new FJsonObject);
	JsonData->SetStringField(TEXT("PathName"), GetPathName());

	TSharedRef<FJsonObject> JsonTracedPoses = MakeShareable(new FJsonObject);
	for(const TPair<FName, FRigPose>& Pair : TracePoses)
	{
		TSharedRef<FJsonObject> JsonTracedPose = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(FRigPose::StaticStruct(), &Pair.Value, JsonTracedPose, 0, 0))
		{
			JsonTracedPoses->SetObjectField(Pair.Key.ToString(), JsonTracedPose);
		}
	}
	JsonData->SetObjectField(TEXT("TracedPoses"), JsonTracedPoses);

	TArray<TSharedPtr<FJsonValue>> JsonTransformStack;
	for (const FRigTransformStackEntry& TransformStackEntry : TransformUndoStack)
	{
		TSharedRef<FJsonObject> JsonTransformStackEntry = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(FRigTransformStackEntry::StaticStruct(), &TransformStackEntry, JsonTransformStackEntry, 0, 0))
		{
			JsonTransformStack.Add(MakeShareable(new FJsonValueObject(JsonTransformStackEntry)));
		}
	}
	JsonData->SetArrayField(TEXT("TransformStack"), JsonTransformStack);

	FString JsonText;
	const TSharedRef< TJsonWriter< TCHAR, TRigHierarchyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TRigHierarchyJsonPrintPolicy<TCHAR> >::Create(&JsonText);
	if (FJsonSerializer::Serialize(JsonData.ToSharedRef(), JsonWriter))
	{
		if ( FFileHelper::SaveStringToFile(JsonText, *FullFilename) )
		{
			UE_LOGF(LogControlRig, Display, "Saved hierarchy trace to %ls", *FullFilename);

			if(OutFilePath)
			{
				*OutFilePath = FullFilename;
			}
		}
	}

	TraceFramesLeft = FMath::Max(0, TraceFramesLeft - 1);
	TraceFramesCaptured++;
}

void URigHierarchy::TraceFrames(int32 InNumFramesToTrace)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	TraceFramesLeft = InNumFramesToTrace;
	TraceFramesCaptured = 0;
	ResetTransformStack();
}

#endif

bool URigHierarchy::IsSelected(const FRigBaseElement* InElement) const
{
	if(InElement == nullptr)
	{
		return false;
	}
	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		return HierarchyForSelection->IsSelected(InElement->GetKey());
	}

	const bool bIsSelected = OrderedSelection.Contains(InElement->GetKey());
	ensure(bIsSelected == InElement->IsSelected());
	return bIsSelected;
}

bool URigHierarchy::IsComponentSelected(const FRigBaseComponent* InComponent) const
{
	if(InComponent == nullptr)
	{
		return false;
	}
	if(const URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get())
	{
		return HierarchyForSelection->IsComponentSelected(InComponent->GetKey());
	}

	const bool bIsSelected = OrderedSelection.Contains(InComponent->GetKey());
	ensure(bIsSelected == InComponent->IsSelected());
	return bIsSelected;
}

void URigHierarchy::EnsureCachedChildrenAreCurrent() const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	if(ChildElementCacheTopologyVersion != TopologyVersion)
	{
		const_cast<URigHierarchy*>(this)->UpdateCachedChildren();
	}
}
	
void URigHierarchy::UpdateCachedChildren()
{
	UE::TScopeLock Lock(ElementsLock);
	
	// First we tally up how many children each element has, then we allocate for the total
	// count and do the same loop again.
	TArray<int32> ChildrenCount;
	ChildrenCount.SetNumZeroed(Elements.Num());

	// Bit array that denotes elements that have parents. We'll use this to quickly iterate
	// for the second pass.
	TBitArray<> ElementHasParent(false, Elements.Num());

	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		const FRigBaseElement* Element = Elements[ElementIndex];
		
		if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(Element))
		{
			if(const FRigTransformElement* ParentElement = SingleParentElement->ParentElement)
			{
				ChildrenCount[ParentElement->Index]++;
				ElementHasParent[ElementIndex] = true;
			}
		}
		else if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(Element))
		{
			for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
			{
				if(const FRigTransformElement* ParentElement = ParentConstraint.ParentElement)
				{
					ChildrenCount[ParentElement->Index]++;
					ElementHasParent[ElementIndex] = true;
				}
			}
		}
	}

	// Tally up how many elements have children.
	const int32 NumElementsWithChildren = Algo::CountIf(ChildrenCount, [](int32 InCount) { return InCount > 0; });

	ChildElementOffsetAndCountCache.Reset(NumElementsWithChildren);

	// Tally up how many children there are in total and set the index on each of the elements as we go.
	int32 TotalChildren = 0;
	for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		if (ChildrenCount[ElementIndex])
		{
			Elements[ElementIndex]->ChildCacheIndex = ChildElementOffsetAndCountCache.Num();
			ChildElementOffsetAndCountCache.Add({TotalChildren, ChildrenCount[ElementIndex]});
			TotalChildren += ChildrenCount[ElementIndex];
		}
		else
		{
			// This element has no children, mark it as having no entry in the child table.
			Elements[ElementIndex]->ChildCacheIndex = INDEX_NONE;
		}
	}

	// Now run through all elements that are known to have parents and start filling up the children array.
	ChildElementCache.Reset();
	ChildElementCache.SetNumZeroed(TotalChildren);

	// Recycle this array to indicate where we are with each set of children, as a local offset into each
	// element's children sub-array.
	ChildrenCount.Reset();
	ChildrenCount.SetNumZeroed(Elements.Num());	

	auto SetChildElement = [this, &ChildrenCount](
		const FRigTransformElement* InParentElement,
		FRigBaseElement* InChildElement)
	{
		const int32 ParentElementIndex = InParentElement->Index;
		const int32 CacheOffset = ChildElementOffsetAndCountCache[InParentElement->ChildCacheIndex].Offset;

		ChildElementCache[CacheOffset + ChildrenCount[ParentElementIndex]] = InChildElement;
		ChildrenCount[ParentElementIndex]++;
	};
	
	for (TConstSetBitIterator<> ElementIt(ElementHasParent); ElementIt; ++ElementIt)
	{
		FRigBaseElement* Element = Elements[ElementIt.GetIndex()];
		
		if(const FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(Element))
		{
			if(const FRigTransformElement* ParentElement = SingleParentElement->ParentElement)
			{
				SetChildElement(ParentElement, Element);
			}
		}
		else if(const FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(Element))
		{
			for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
			{
				if(const FRigTransformElement* ParentElement = ParentConstraint.ParentElement)
				{
					SetChildElement(ParentElement, Element);
				}
			}
		}
	}

	// Mark the cache up-to-date.
	ChildElementCacheTopologyVersion = TopologyVersion;
}

	
FRigElementKey URigHierarchy::PreprocessParentElementKeyForSpaceSwitching(const FRigElementKey& InChildKey, const FRigElementKey& InParentKey) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	
	if(InParentKey == GetWorldSpaceReferenceKey())
	{
		return const_cast<URigHierarchy*>(this)->GetOrAddWorldSpaceReference();
	}
	else if(InParentKey == GetDefaultParentKey())
	{
		const FRigElementKey DefaultParent = GetDefaultParent(InChildKey);
		if(DefaultParent == GetWorldSpaceReferenceKey())
		{
			return FRigElementKey();
		}
		else
		{
			return DefaultParent;
		}
	}

	return InParentKey;
}

FRigBaseElement* URigHierarchy::MakeElement(ERigElementType InElementType, int32 InCount, int32* OutStructureSize)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	check(InCount > 0);
	
	FRigBaseElement* Element = nullptr;
	switch(InElementType)
	{
		case ERigElementType::Bone:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigBoneElement);
			}
			Element = NewElement<FRigBoneElement>(InCount);
			break;
		}
		case ERigElementType::Null:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigNullElement);
			}
			Element = NewElement<FRigNullElement>(InCount);
			break;
		}
		case ERigElementType::Control:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigControlElement);
			}
			Element = NewElement<FRigControlElement>(InCount);
			break;
		}
		case ERigElementType::Curve:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigCurveElement);
			}
			Element = NewElement<FRigCurveElement>(InCount);
			break;
		}
		case ERigElementType::Reference:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigReferenceElement);
			}
			Element = NewElement<FRigReferenceElement>(InCount);
			break;
		}
		case ERigElementType::Connector:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigConnectorElement);
			}
			Element = NewElement<FRigConnectorElement>(InCount);
			break;
		}
		case ERigElementType::Socket:
		{
			if(OutStructureSize)
			{
				*OutStructureSize = sizeof(FRigSocketElement);
			}
			Element = NewElement<FRigSocketElement>(InCount);
			break;
		}
		default:
		{
			ensure(false);
		}
	}

	return Element;
}

void URigHierarchy::DestroyElement(FRigBaseElement*& InElement, bool bDestroyComponents, bool bDestroyElementStorage, bool bDestroyMetadata)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	check(InElement != nullptr);

	if(InElement->OwnedInstances == 0)
	{
		return;
	}

	const int32 Count = InElement->OwnedInstances;
	switch(InElement->GetType())
	{
		case ERigElementType::Bone:
		{
			FRigBoneElement* ExistingElements = Cast<FRigBoneElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigBoneElement(); 
			}
			break;
		}
		case ERigElementType::Null:
		{
			FRigNullElement* ExistingElements = Cast<FRigNullElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigNullElement(); 
			}
			break;
		}
		case ERigElementType::Control:
		{
			FRigControlElement* ExistingElements = Cast<FRigControlElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigControlElement(); 
			}
			break;
		}
		case ERigElementType::Curve:
		{
			FRigCurveElement* ExistingElements = Cast<FRigCurveElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigCurveElement(); 
			}
			break;
		}
		case ERigElementType::Reference:
		{
			FRigReferenceElement* ExistingElements = Cast<FRigReferenceElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigReferenceElement(); 
			}
			break;
		}
		case ERigElementType::Connector:
		{
			FRigConnectorElement* ExistingElements = Cast<FRigConnectorElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigConnectorElement(); 
			}
			break;
		}
		case ERigElementType::Socket:
		{
			FRigSocketElement* ExistingElements = Cast<FRigSocketElement>(InElement);
			for(int32 Index=0;Index<Count;Index++)
			{
				TGuardValue<const FRigBaseElement*> DestroyGuard(ElementBeingDestroyed, &ExistingElements[Index]);
				if(bDestroyComponents) DestroyComponents(&ExistingElements[Index]);
				if(bDestroyElementStorage) DeallocateElementStorage(&ExistingElements[Index]);
				if(!bDestroyMetadata) ExistingElements[Index].Owner = nullptr;
				ExistingElements[Index].~FRigSocketElement(); 
			}
			break;
		}
		default:
		{
			ensure(false);
			return;
		}
	}

	FMemory::Free(InElement);
	InElement = nullptr;
	
	ElementTransformRanges.Reset();
}

FRigBaseComponent* URigHierarchy::MakeComponent(const UScriptStruct* InComponentStruct, const FName& InName, FRigBaseElement* InElement)
{
	check(InComponentStruct);
	check(InComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()));
	check(InElement != nullptr);
	check(InElement->GetOwner() == this);
	
	int32 ComponentIndex = INDEX_NONE;
	const int32 SpawnIndex = GetNextSpawnIndex();
	for(int32 ExistingIndex = 0; ExistingIndex < ElementComponents.Num(); ExistingIndex++)
	{
		if(!ElementComponents[ExistingIndex].IsValid())
		{
			ComponentIndex = ExistingIndex;
			ElementComponents[ComponentIndex] = FInstancedStruct(InComponentStruct);
			break;
		}
	}

	if(ComponentIndex == INDEX_NONE)
	{
		ComponentIndex = ElementComponents.Emplace(InComponentStruct);
	}

	const FRigName UniqueName = GetSafeNewComponentName(InElement->GetKey(), InName);
	
	FRigBaseComponent* Component = ElementComponents[ComponentIndex].GetMutablePtr<FRigBaseComponent>();
	Component->IndexInHierarchy = ComponentIndex;

	Component->Key = FRigComponentKey(InElement->GetKey(), UniqueName.GetFName());
	Component->Element = InElement;
	Component->IndexInElement = InElement->ComponentIndices.Add(Component->IndexInHierarchy);
	Component->SpawnIndex = SpawnIndex;

	ComponentIndexLookup.Add(Component->GetKey(), Component->GetIndexInHierarchy());
	
	return Component;
}

void URigHierarchy::DestroyComponent(FRigBaseComponent*& InComponent)
{
	for(int32 ComponentIndex = 0; ComponentIndex < ElementComponents.Num(); ComponentIndex ++)
	{
		FInstancedStruct& InstancedStruct = ElementComponents[ComponentIndex];
		const FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
		if(Component == InComponent)
		{
			if(FRigBaseElement* Element = Find(Component->GetElementKey()))
			{
				Element->ComponentIndices.Remove(ComponentIndex);
				
				for(int32 IndexInElement = 0; IndexInElement < Element->ComponentIndices.Num(); IndexInElement++)
				{
					if(FRigBaseComponent* RemainingComponent = GetComponent(Element->ComponentIndices[IndexInElement]))
					{
						RemainingComponent->IndexInElement = IndexInElement;
					}
				}
			}
			ComponentIndexLookup.Remove(InComponent->GetKey());
			InstancedStruct.Reset();
			InComponent = nullptr;
			break;
		}
	}
}

void URigHierarchy::DestroyComponents(FRigBaseElement* InElement)
{
	for(const int32& ComponentIndex : InElement->ComponentIndices)
	{
		if(const FRigBaseComponent* Component = GetComponent(ComponentIndex))
		{
			ComponentIndexLookup.Remove(Component->GetKey());
		}
		ElementComponents[ComponentIndex].Reset();
	}
	InElement->ComponentIndices.Reset();
}

void URigHierarchy::ShrinkComponentStorage()
{
	// only shrink the memory if any of the components is invalid
	if(!ElementComponents.ContainsByPredicate([](const FInstancedStruct& InstancedStruct) -> bool
	{
		return !InstancedStruct.IsValid();
	}))
	{
		return;
	}

	for(FRigBaseElement* Element : Elements)
	{
		Element->ComponentIndices.Reset();
	}

	ElementComponents.RemoveAll([](const FInstancedStruct& InstancedStruct) -> bool
	{
		return !InstancedStruct.IsValid();
	});

	ComponentIndexLookup.Reset();;

	for(int32 ComponentIndex = 0; ComponentIndex < ElementComponents.Num(); ComponentIndex ++)
	{
		FInstancedStruct& InstancedStruct = ElementComponents[ComponentIndex];
		FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
		Component->IndexInHierarchy = ComponentIndex;
		if(Component->Element)
		{
			Component->Element = Find(Component->GetElementKey());
			Component->Element->ComponentIndices.Add(ComponentIndex);
		}
		ComponentIndexLookup.Add(Component->GetKey(), ComponentIndex);
	}
}

void URigHierarchy::PropagateDirtyFlags(FRigTransformElement* InTransformElement, bool bInitial, bool bAffectChildren, bool bComputeOpposed, bool bMarkDirty) const
{
	if (!bEnableDirtyPropagation)
	{
		return;
	}

	check(InTransformElement);

	const ERigTransformType::Type LocalType = bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal;
	const ERigTransformType::Type GlobalType = bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal;

	const ERigTransformType::Type TypeToCompute = bAffectChildren ? LocalType : GlobalType;
	const ERigTransformType::Type TypeToDirty = SwapLocalAndGlobal(TypeToCompute);

	for (const FRigTransformElement::FElementToDirty& ElementToDirty : InTransformElement->ElementsToDirty)
	{
		if (ElementToDirty.Element->IsElementDirty(TypeToDirty))
		{
			continue;
		}

		if (bComputeOpposed)
		{
			if (FRigControlElement* ControlElement = Cast<FRigControlElement>(ElementToDirty.Element))
			{
				GetControlOffsetTransform(ControlElement, LocalType);
			}
			GetTransform(ElementToDirty.Element, TypeToCompute); // make sure the local / global transform is up 2 date

			if (bAffectChildren)
			{
				PropagateDirtyFlags(ElementToDirty.Element, bInitial, bAffectChildren, true, false);
			}
		}

		if (bMarkDirty)
		{
			ElementToDirty.Element->MarkElementDirty(TypeToDirty);

			if (bAffectChildren)
			{
				PropagateDirtyFlags(ElementToDirty.Element, bInitial, bAffectChildren, false, true);
			}
		}
	}
}

void URigHierarchy::CleanupInvalidCaches()
{
	// create a copy of this hierarchy and pre compute all transforms
	bool bHierarchyWasCreatedDuringCleanup = false;
	if(!IsValid(HierarchyForCacheValidation))
	{
		// Give the validation hierarchy a unique name for debug purposes and to avoid possible memory reuse
		static TAtomic<uint32> HierarchyNameIndex = 0;
		static constexpr TCHAR Format[] = TEXT("CacheValidationHierarchy_%u");
		const FString ValidationHierarchyName = FString::Printf(Format, uint32(++HierarchyNameIndex));
		
		HierarchyForCacheValidation = NewObject<URigHierarchy>(this, *ValidationHierarchyName, RF_Transient);
		HierarchyForCacheValidation->bEnableCacheValidityCheck = false;
		bHierarchyWasCreatedDuringCleanup = true;
	}
	HierarchyForCacheValidation->CopyHierarchy(this);

	struct Local
	{
		static bool NeedsCheck(const FRigLocalAndGlobalDirtyState& InDirtyState)
		{
			return !InDirtyState.Local.Get() && !InDirtyState.Global.Get();
		}
	};

	// mark all elements' initial as dirty where needed
	for(int32 ElementIndex = 0; ElementIndex < HierarchyForCacheValidation->Elements.Num(); ElementIndex++)
	{
		FRigBaseElement* BaseElement = HierarchyForCacheValidation->Elements[ElementIndex];
		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(BaseElement))
		{
			if(Local::NeedsCheck(ControlElement->GetOffsetDirtyState().Initial))
			{
				ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}

			if(Local::NeedsCheck(ControlElement->GetDirtyState().Initial))
			{
				ControlElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}

			if(Local::NeedsCheck(ControlElement->GetShapeDirtyState().Initial))
			{
				ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}
			continue;
		}

		if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(BaseElement))
		{
			if(Local::NeedsCheck(MultiParentElement->GetDirtyState().Initial))
			{
				MultiParentElement->GetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
			}
			continue;
		}

		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(BaseElement))
		{
			if(Local::NeedsCheck(TransformElement->GetDirtyState().Initial))
			{
				TransformElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}
		}
	}

	// recompute 
	HierarchyForCacheValidation->ComputeAllTransforms();

	// we need to check the elements for integrity (global vs local) to be correct.
	for(int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		FRigBaseElement* BaseElement = Elements[ElementIndex];

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(BaseElement))
		{
			FRigControlElement* OtherControlElement = HierarchyForCacheValidation->FindChecked<FRigControlElement>(ControlElement->GetKey());
			
			if(Local::NeedsCheck(ControlElement->GetOffsetDirtyState().Initial))
			{
				const FTransform CachedGlobalTransform = OtherControlElement->GetOffsetTransform().Get(ERigTransformType::InitialGlobal);
				const FTransform ComputedGlobalTransform = HierarchyForCacheValidation->GetControlOffsetTransform(OtherControlElement, ERigTransformType::InitialGlobal);

				if(!FRigComputedTransform::Equals(ComputedGlobalTransform, CachedGlobalTransform, 0.01f))
				{
					ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				}
			}

			if(Local::NeedsCheck(ControlElement->GetDirtyState().Initial))
			{
				const FTransform CachedGlobalTransform = ControlElement->GetTransform().Get(ERigTransformType::InitialGlobal);
				const FTransform ComputedGlobalTransform = HierarchyForCacheValidation->GetTransform(OtherControlElement, ERigTransformType::InitialGlobal);
				
				if(!FRigComputedTransform::Equals(ComputedGlobalTransform, CachedGlobalTransform, 0.01f))
				{
					ControlElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				}
			}

			if(Local::NeedsCheck(ControlElement->GetShapeDirtyState().Initial))
			{
				const FTransform CachedGlobalTransform = ControlElement->GetShapeTransform().Get(ERigTransformType::InitialGlobal);
				const FTransform ComputedGlobalTransform = HierarchyForCacheValidation->GetControlShapeTransform(OtherControlElement, ERigTransformType::InitialGlobal);
				
				if(!FRigComputedTransform::Equals(ComputedGlobalTransform, CachedGlobalTransform, 0.01f))
				{
					ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				}
			}
			continue;
		}

		if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(BaseElement))
		{
			FRigMultiParentElement* OtherMultiParentElement = HierarchyForCacheValidation->FindChecked<FRigMultiParentElement>(MultiParentElement->GetKey());

			if(Local::NeedsCheck(MultiParentElement->GetDirtyState().Initial))
			{
				const FTransform CachedGlobalTransform = MultiParentElement->GetTransform().Get(ERigTransformType::InitialGlobal);
				const FTransform ComputedGlobalTransform = HierarchyForCacheValidation->GetTransform(OtherMultiParentElement, ERigTransformType::InitialGlobal);
				
				if(!FRigComputedTransform::Equals(ComputedGlobalTransform, CachedGlobalTransform, 0.01f))
				{
					// for nulls we perceive the local transform as less relevant
					MultiParentElement->GetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
				}
			}
			continue;
		}

		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(BaseElement))
		{
			FRigTransformElement* OtherTransformElement = HierarchyForCacheValidation->FindChecked<FRigTransformElement>(TransformElement->GetKey());

			if(Local::NeedsCheck(TransformElement->GetDirtyState().Initial))
			{
				const FTransform CachedGlobalTransform = TransformElement->GetTransform().Get(ERigTransformType::InitialGlobal);
				const FTransform ComputedGlobalTransform = HierarchyForCacheValidation->GetTransform(OtherTransformElement, ERigTransformType::InitialGlobal);
				
				if(!FRigComputedTransform::Equals(ComputedGlobalTransform, CachedGlobalTransform, 0.01f))
				{
					TransformElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				}
			}
		}
	}

	ResetPoseToInitial(ERigElementType::All);
	EnsureCacheValidity();

	if(bHierarchyWasCreatedDuringCleanup && HierarchyForCacheValidation)
	{
		HierarchyForCacheValidation->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		HierarchyForCacheValidation->RemoveFromRoot();
		HierarchyForCacheValidation->MarkAsGarbage();
		HierarchyForCacheValidation = nullptr;
	}
}

void URigHierarchy::FMetadataStorage::Reset()
{
	for (TTuple<FName, FRigBaseMetadata*>& Item: MetadataMap)
	{
		FRigBaseMetadata::DestroyMetadata(&Item.Value);
	}
	MetadataMap.Reset();
	LastAccessName = NAME_None;
	LastAccessMetadata = nullptr;
}

void URigHierarchy::FMetadataStorage::Serialize(FArchive& Ar)
{
	static const UEnum* MetadataTypeEnum = StaticEnum<ERigMetadataType>();
	
	if (Ar.IsLoading())
	{
		Reset();
		
		int32 NumEntries = 0;
		Ar << NumEntries;
		MetadataMap.Reserve(NumEntries);
		for (int32 Index = 0; Index < NumEntries; Index++)
		{
			FName ItemName, TypeName;
			Ar << ItemName;
			Ar << TypeName;

			const ERigMetadataType Type = static_cast<ERigMetadataType>(MetadataTypeEnum->GetValueByName(TypeName));
			FRigBaseMetadata* Metadata = FRigBaseMetadata::MakeMetadata(ItemName, Type);
			Metadata->Serialize(Ar);

			MetadataMap.Add(ItemName, Metadata);
		}
	}
	else
	{
		int32 NumEntries = MetadataMap.Num();
		Ar << NumEntries;
		for (const TTuple<FName, FRigBaseMetadata*>& Item: MetadataMap)
		{
			FName ItemName = Item.Key;
			FName TypeName = MetadataTypeEnum->GetNameByValue(static_cast<int64>(Item.Value->GetType()));
			Ar << ItemName;
			Ar << TypeName;
			Item.Value->Serialize(Ar);
		}
	}
}

void URigHierarchy::AllocateDefaultElementStorage(FRigBaseElement* InElement, bool bUpdateAllElements)
{
	if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(InElement))
	{
		// only for the default allocation we want to catch double allocations.
		// so if we are already linked to the default storage, exit early.
		if(ElementTransforms.Contains(TransformElement->PoseStorage.Initial.Local.StorageIndex, TransformElement->PoseStorage.Initial.Local.Storage))
		{
			check(ElementTransforms.Contains(TransformElement->PoseStorage.Current.Local.StorageIndex, TransformElement->PoseStorage.Current.Local.Storage));
			check(ElementTransforms.Contains(TransformElement->PoseStorage.Initial.Global.StorageIndex, TransformElement->PoseStorage.Initial.Global.Storage));
			check(ElementTransforms.Contains(TransformElement->PoseStorage.Current.Global.StorageIndex, TransformElement->PoseStorage.Current.Global.Storage));
			check(ElementDirtyStates.Contains(TransformElement->PoseDirtyState.Initial.Local.StorageIndex, TransformElement->PoseDirtyState.Initial.Local.Storage));
			check(ElementDirtyStates.Contains(TransformElement->PoseDirtyState.Current.Local.StorageIndex, TransformElement->PoseDirtyState.Current.Local.Storage));
			check(ElementDirtyStates.Contains(TransformElement->PoseDirtyState.Initial.Global.StorageIndex, TransformElement->PoseDirtyState.Initial.Global.Storage));
			check(ElementDirtyStates.Contains(TransformElement->PoseDirtyState.Current.Global.StorageIndex, TransformElement->PoseDirtyState.Current.Global.Storage));

			if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
			{
				check(ElementTransforms.Contains(ControlElement->OffsetStorage.Initial.Local.StorageIndex, ControlElement->OffsetStorage.Initial.Local.Storage));
				check(ElementTransforms.Contains(ControlElement->OffsetStorage.Current.Local.StorageIndex, ControlElement->OffsetStorage.Current.Local.Storage));
				check(ElementTransforms.Contains(ControlElement->OffsetStorage.Initial.Global.StorageIndex, ControlElement->OffsetStorage.Initial.Global.Storage));
				check(ElementTransforms.Contains(ControlElement->OffsetStorage.Current.Global.StorageIndex, ControlElement->OffsetStorage.Current.Global.Storage));
				check(ElementDirtyStates.Contains(ControlElement->OffsetDirtyState.Initial.Local.StorageIndex, ControlElement->OffsetDirtyState.Initial.Local.Storage));
				check(ElementDirtyStates.Contains(ControlElement->OffsetDirtyState.Current.Local.StorageIndex, ControlElement->OffsetDirtyState.Current.Local.Storage));
				check(ElementDirtyStates.Contains(ControlElement->OffsetDirtyState.Initial.Global.StorageIndex, ControlElement->OffsetDirtyState.Initial.Global.Storage));
				check(ElementDirtyStates.Contains(ControlElement->OffsetDirtyState.Current.Global.StorageIndex, ControlElement->OffsetDirtyState.Current.Global.Storage));
				check(ElementTransforms.Contains(ControlElement->ShapeStorage.Initial.Local.StorageIndex, ControlElement->ShapeStorage.Initial.Local.Storage));
				check(ElementTransforms.Contains(ControlElement->ShapeStorage.Current.Local.StorageIndex, ControlElement->ShapeStorage.Current.Local.Storage));
				check(ElementTransforms.Contains(ControlElement->ShapeStorage.Initial.Global.StorageIndex, ControlElement->ShapeStorage.Initial.Global.Storage));
				check(ElementTransforms.Contains(ControlElement->ShapeStorage.Current.Global.StorageIndex, ControlElement->ShapeStorage.Current.Global.Storage));
				check(ElementDirtyStates.Contains(ControlElement->ShapeDirtyState.Initial.Local.StorageIndex, ControlElement->ShapeDirtyState.Initial.Local.Storage));
				check(ElementDirtyStates.Contains(ControlElement->ShapeDirtyState.Current.Local.StorageIndex, ControlElement->ShapeDirtyState.Current.Local.Storage));
				check(ElementDirtyStates.Contains(ControlElement->ShapeDirtyState.Initial.Global.StorageIndex, ControlElement->ShapeDirtyState.Initial.Global.Storage));
				check(ElementDirtyStates.Contains(ControlElement->ShapeDirtyState.Current.Global.StorageIndex, ControlElement->ShapeDirtyState.Current.Global.Storage));
			}
			return;
		}
		const TArray<int32, TInlineAllocator<4>> TransformIndices = ElementTransforms.Allocate(TransformElement->GetNumTransforms(), FTransform::Identity);
		check(TransformIndices.Num() >= 4);
		const TArray<int32, TInlineAllocator<4>> DirtyStateIndices = ElementDirtyStates.Allocate(TransformElement->GetNumTransforms(), false);
		check(DirtyStateIndices.Num() >= 4);
		TransformElement->PoseStorage.Initial.Local.StorageIndex = TransformIndices[0];
		TransformElement->PoseStorage.Current.Local.StorageIndex = TransformIndices[1];
		TransformElement->PoseStorage.Initial.Global.StorageIndex = TransformIndices[2];
		TransformElement->PoseStorage.Current.Global.StorageIndex = TransformIndices[3];
		TransformElement->PoseDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[0];
		TransformElement->PoseDirtyState.Current.Local.StorageIndex = DirtyStateIndices[1];
		TransformElement->PoseDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[2];
		TransformElement->PoseDirtyState.Current.Global.StorageIndex = DirtyStateIndices[3];

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
		{
			check(TransformIndices.Num() >= 12);
			check(DirtyStateIndices.Num() >= 12);
			ControlElement->OffsetStorage.Initial.Local.StorageIndex = TransformIndices[4];
			ControlElement->OffsetStorage.Current.Local.StorageIndex = TransformIndices[5];
			ControlElement->OffsetStorage.Initial.Global.StorageIndex = TransformIndices[6];
			ControlElement->OffsetStorage.Current.Global.StorageIndex = TransformIndices[7];
			ControlElement->OffsetDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[4];
			ControlElement->OffsetDirtyState.Current.Local.StorageIndex = DirtyStateIndices[5];
			ControlElement->OffsetDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[6];
			ControlElement->OffsetDirtyState.Current.Global.StorageIndex = DirtyStateIndices[7];
			ControlElement->ShapeStorage.Initial.Local.StorageIndex = TransformIndices[8];
			ControlElement->ShapeStorage.Current.Local.StorageIndex = TransformIndices[9];
			ControlElement->ShapeStorage.Initial.Global.StorageIndex = TransformIndices[10];
			ControlElement->ShapeStorage.Current.Global.StorageIndex = TransformIndices[11];
			ControlElement->ShapeDirtyState.Initial.Local.StorageIndex = DirtyStateIndices[8];
			ControlElement->ShapeDirtyState.Current.Local.StorageIndex = DirtyStateIndices[9];
			ControlElement->ShapeDirtyState.Initial.Global.StorageIndex = DirtyStateIndices[10];
			ControlElement->ShapeDirtyState.Current.Global.StorageIndex = DirtyStateIndices[11];
		}
	}
	else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(InElement))
	{
		if(ElementCurves.Contains(CurveElement->StorageIndex, CurveElement->Storage))
		{
			return;
		}
		
		const TArray<int32, TInlineAllocator<4>> CurveIndices = ElementCurves.Allocate(CurveElement->GetNumCurves(), 0.f);
		check(CurveIndices.Num() >= 1);
		CurveElement->StorageIndex = CurveIndices[0];
	}

	if(bUpdateAllElements)
	{
		UpdateElementStorage();
	}
	
	// the element might be in the creation process and not added to the hierarchy, so UpdateElementStorage might not have linked the storage
	InElement->LinkStorage(ElementTransforms.GetStorage(), ElementDirtyStates.GetStorage(), ElementCurves.GetStorage());

	ElementTransformRanges.Reset();
}

void URigHierarchy::DeallocateElementStorage(FRigBaseElement* InElement)
{
	InElement->UnlinkStorage(ElementTransforms, ElementDirtyStates, ElementCurves);
}

void URigHierarchy::UpdateElementStorage()
{
	for(FRigBaseElement* Element : Elements)
	{
		Element->LinkStorage(ElementTransforms.GetStorage(), ElementDirtyStates.GetStorage(), ElementCurves.GetStorage());
	}
	ElementTransformRanges.Reset();
}

bool URigHierarchy::SortElementStorage()
{
	ElementTransformRanges.Reset();

	if(Elements.IsEmpty())
	{
		return false;
	}

	static constexpr int32 NumTransformTypes = ERigTransformType::NumTransformTypes;

	// figure out which types of elements actually exist on this hierarchy
	static const int32 LastElementType = RigElementTypeToFlatIndex(ERigElementType::Last);
	static const int32 ControlElementType = RigElementTypeToFlatIndex(ERigElementType::Control);

	TArray<int32> PerElementStride, NumElementsPerType, OffsetPerElementType;
	PerElementStride.Reserve(LastElementType+1);
	NumElementsPerType.Reserve(LastElementType+1);
	OffsetPerElementType.AddZeroed(LastElementType+1);

	int32 NumElementTypes = 0;
	int32 TotalNumberOfTransformsPerTransformType = 0;

	for (int32 ElementTypeFlatIndex = 0; ElementTypeFlatIndex <= LastElementType; ElementTypeFlatIndex++)
	{
		const ERigElementType ElementTypeFlag = FlatIndexToRigElementType(ElementTypeFlatIndex);
		const bool bReserveSpace = (((int32)ElementTypeFlag & (int32)ERigElementType::TransformTypes) != 0); // avoid reserving space for non transform types

		NumElementsPerType.Add(bReserveSpace ? ElementsPerType[ElementTypeFlatIndex].Num() : 0);
		PerElementStride.Add(ElementTypeFlatIndex == ControlElementType ? static_cast<int32>(ERigTransformStorageType::NumStorageTypes) : 1);

		if (ElementTypeFlatIndex > 0)
		{
			OffsetPerElementType[ElementTypeFlatIndex] = OffsetPerElementType[ElementTypeFlatIndex - 1] + NumElementsPerType[ElementTypeFlatIndex - 1] * PerElementStride[ElementTypeFlatIndex - 1];
		}

		NumElementTypes += (bReserveSpace && ElementsPerType[ElementTypeFlatIndex].Num() > 0) ? 1 : 0;
		TotalNumberOfTransformsPerTransformType += NumElementsPerType.Last() * PerElementStride.Last();
	}

	const int32 TotalNumberOfTransforms = TotalNumberOfTransformsPerTransformType * NumTransformTypes;

	FMemMark Mark(FMemStack::Get()); // Guard to clean up the stack allocator
	TArray<int32, FAnimStackAllocator> TransformIndexToAllocatedIndex;
	TransformIndexToAllocatedIndex.AddUninitialized(TotalNumberOfTransforms);

	// we sort elements by laying them out first by transform type, then element type, then storage type.
	auto GetElementIndexForSort = [OffsetPerElementType, PerElementStride, TotalNumberOfTransformsPerTransformType](
		const FRigBaseElement* InElement,
		ERigTransformType::Type InTransformType,
		ERigTransformStorageType::Type InStorageType
	) -> int32 {
		const int32 FlatElementIndex = RigElementTypeToFlatIndex(InElement->GetType());
		return
			static_cast<int32>(InTransformType) * TotalNumberOfTransformsPerTransformType +
			(OffsetPerElementType[FlatElementIndex] + InElement->GetSubIndex() * PerElementStride[FlatElementIndex]) +
			static_cast<int32>(InStorageType);
	};

	ForEachTransformElementStorage([&TransformIndexToAllocatedIndex, GetElementIndexForSort](
		FRigTransformElement* InTransformElement,
		ERigTransformType::Type InTransformType,
		ERigTransformStorageType::Type InStorageType,
		FRigComputedTransform* InComputedTransform,
		FRigTransformDirtyState* InDirtyState)
	{
		const int32 Index = GetElementIndexForSort(InTransformElement, InTransformType, InStorageType);
		if(InComputedTransform->GetStorageIndex() == INDEX_NONE)
		{
			TransformIndexToAllocatedIndex[Index] = INDEX_NONE;
		}
		else
		{
			TransformIndexToAllocatedIndex[Index] = 0;
		}
	});

	// accumulate the allocated indices
	int32 NumAllocatedTransforms = 0;
	for(int32 Index = 0; Index < TransformIndexToAllocatedIndex.Num(); Index++)
	{
		if(TransformIndexToAllocatedIndex[Index] != INDEX_NONE)
		{
			TransformIndexToAllocatedIndex[Index] = NumAllocatedTransforms++;
		}
	}

	TArray<TTuple<int32, int32>> SortedRanges;
	SortedRanges.Reserve(NumTransformTypes);
	for(int32 TransformType = 0; TransformType < NumTransformTypes; TransformType++)
	{
		SortedRanges.Emplace(INT32_MAX, INDEX_NONE);
	}

	bool bRequiresSort = false;
	ForEachTransformElementStorage([
		this,
		GetElementIndexForSort,
		TransformIndexToAllocatedIndex,
		&SortedRanges,
		&bRequiresSort
	](
		const FRigTransformElement* InTransformElement,
		ERigTransformType::Type InTransformType,
		ERigTransformStorageType::Type InStorageType,
		const FRigComputedTransform* InComputedTransform,
		const FRigTransformDirtyState* InDirtyState
	) {
		
		if(InComputedTransform->GetStorageIndex() == INDEX_NONE)
		{
			return;
		}
		
		const int32 Index = GetElementIndexForSort(InTransformElement, InTransformType, InStorageType);
		const int32 AllocatedIndex = TransformIndexToAllocatedIndex[Index];

		TTuple<int32, int32>& Range = SortedRanges[static_cast<int32>(InTransformType)]; 
		Range.Get<0>() = FMath::Min(Range.Get<0>(), AllocatedIndex); 
		Range.Get<1>() = FMath::Max(Range.Get<1>(), AllocatedIndex); 

		bRequiresSort = bRequiresSort || (InComputedTransform->GetStorageIndex() != AllocatedIndex && InDirtyState->GetStorageIndex() != AllocatedIndex);
	});

	if(!bRequiresSort)
	{
		ElementTransformRanges = MoveTemp(SortedRanges);
		return false;
	}

	FRigReusableElementStorage<FTransform> SortedElementTransforms;
	FRigReusableElementStorage<bool> SortedElementDirtyStates;
	SortedElementTransforms.AddUninitialized(NumAllocatedTransforms);
	SortedElementDirtyStates.AddUninitialized(NumAllocatedTransforms);

	ForEachTransformElementStorage([
		this,
		&SortedElementTransforms,
		&SortedElementDirtyStates,
		GetElementIndexForSort,
		TransformIndexToAllocatedIndex
	](
		const FRigTransformElement* InTransformElement,
		ERigTransformType::Type InTransformType,
		ERigTransformStorageType::Type InStorageType,
		FRigComputedTransform* InOutComputedTransform,
		FRigTransformDirtyState* InOutDirtyState
	) {
		if(InOutComputedTransform->GetStorageIndex() == INDEX_NONE)
		{
			return;
		}

		const int32 Index = GetElementIndexForSort(InTransformElement, InTransformType, InStorageType);
		const int32 AllocatedIndex = TransformIndexToAllocatedIndex[Index];

		SortedElementTransforms[AllocatedIndex] = InOutComputedTransform->Get();
		SortedElementDirtyStates[AllocatedIndex] = InOutDirtyState->Get();

		InOutComputedTransform->StorageIndex = AllocatedIndex;
		InOutComputedTransform->Storage = &SortedElementTransforms[AllocatedIndex];
		
		InOutDirtyState->StorageIndex = AllocatedIndex;
		InOutDirtyState->Storage = &SortedElementDirtyStates[AllocatedIndex];
	});

	ElementTransforms.FreeList.Empty();
	ElementTransforms.Storage = MoveTemp(SortedElementTransforms.Storage);
	ElementDirtyStates.FreeList.Empty();
	ElementDirtyStates.Storage = MoveTemp(SortedElementDirtyStates.Storage);
	ElementTransformRanges = MoveTemp(SortedRanges);
	return true;
}

bool URigHierarchy::ShrinkElementStorage()
{
	const TMap<int32, int32> TransformLookup = ElementTransforms.Shrink();
	const TMap<int32, int32> DirtyStateLookup = ElementDirtyStates.Shrink();
	const TMap<int32, int32> CurveLookup = ElementCurves.Shrink();

	if(!TransformLookup.IsEmpty() || !DirtyStateLookup.IsEmpty())
	{
		ForEachTransformElementStorage([&TransformLookup, &DirtyStateLookup](
			FRigTransformElement* InTransformElement,
			ERigTransformType::Type InTransformType,
			ERigTransformStorageType::Type InStorageType,
			FRigComputedTransform* InComputedTransform,
			FRigTransformDirtyState* InDirtyState)
		{
			if(const int32* NewTransformIndex = TransformLookup.Find(InComputedTransform->StorageIndex))
			{
				InComputedTransform->StorageIndex = *NewTransformIndex;
			}
			if(const int32* NewDirtyStateIndex = DirtyStateLookup.Find(InDirtyState->StorageIndex))
			{
				InDirtyState->StorageIndex = *NewDirtyStateIndex;
			}
		});
	}

	if(!CurveLookup.IsEmpty())
	{
		static const int32 CurveElementIndex = RigElementTypeToFlatIndex(ERigElementType::Curve); 
		for(FRigBaseElement* Element : ElementsPerType[CurveElementIndex])
		{
			FRigCurveElement* CurveElement = CastChecked<FRigCurveElement>(Element);
			if(const int32* NewIndex = CurveLookup.Find(CurveElement->StorageIndex))
			{
				CurveElement->StorageIndex = *NewIndex;
			}
		}
	}

	if(!TransformLookup.IsEmpty() || !DirtyStateLookup.IsEmpty() || !CurveLookup.IsEmpty())
	{
		UpdateElementStorage();
		(void)SortElementStorage();
		return true;
	}
	
	return false;
}

void URigHierarchy::ForEachTransformElementStorage(
	TFunction<void(FRigTransformElement*, ERigTransformType::Type, ERigTransformStorageType::Type, FRigComputedTransform*, FRigTransformDirtyState*)>
	InCallback)
{
	check(InCallback);
	
	for(FRigBaseElement* Element : Elements)
	{
		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
		{
			InCallback(TransformElement, ERigTransformType::InitialLocal, ERigTransformStorageType::Pose, &TransformElement->PoseStorage.Initial.Local, &TransformElement->PoseDirtyState.Initial.Local);
			InCallback(TransformElement, ERigTransformType::InitialGlobal, ERigTransformStorageType::Pose, &TransformElement->PoseStorage.Initial.Global, &TransformElement->PoseDirtyState.Initial.Global);
			InCallback(TransformElement, ERigTransformType::CurrentLocal, ERigTransformStorageType::Pose, &TransformElement->PoseStorage.Current.Local, &TransformElement->PoseDirtyState.Current.Local);
			InCallback(TransformElement, ERigTransformType::CurrentGlobal, ERigTransformStorageType::Pose, &TransformElement->PoseStorage.Current.Global, &TransformElement->PoseDirtyState.Current.Global);

			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
			{
				InCallback(ControlElement, ERigTransformType::InitialLocal, ERigTransformStorageType::Offset, &ControlElement->OffsetStorage.Initial.Local, &ControlElement->OffsetDirtyState.Initial.Local);
				InCallback(ControlElement, ERigTransformType::InitialGlobal, ERigTransformStorageType::Offset, &ControlElement->OffsetStorage.Initial.Global, &ControlElement->OffsetDirtyState.Initial.Global);
				InCallback(ControlElement, ERigTransformType::CurrentLocal, ERigTransformStorageType::Offset, &ControlElement->OffsetStorage.Current.Local, &ControlElement->OffsetDirtyState.Current.Local);
				InCallback(ControlElement, ERigTransformType::CurrentGlobal, ERigTransformStorageType::Offset, &ControlElement->OffsetStorage.Current.Global, &ControlElement->OffsetDirtyState.Current.Global);
				InCallback(ControlElement, ERigTransformType::InitialLocal, ERigTransformStorageType::Shape, &ControlElement->ShapeStorage.Initial.Local, &ControlElement->ShapeDirtyState.Initial.Local);
				InCallback(ControlElement, ERigTransformType::InitialGlobal, ERigTransformStorageType::Shape, &ControlElement->ShapeStorage.Initial.Global, &ControlElement->ShapeDirtyState.Initial.Global);
				InCallback(ControlElement, ERigTransformType::CurrentLocal, ERigTransformStorageType::Shape, &ControlElement->ShapeStorage.Current.Local, &ControlElement->ShapeDirtyState.Current.Local);
				InCallback(ControlElement, ERigTransformType::CurrentGlobal, ERigTransformStorageType::Shape, &ControlElement->ShapeStorage.Current.Global, &ControlElement->ShapeDirtyState.Current.Global);
			}
		}
	}
}

TTuple<FRigComputedTransform*, FRigTransformDirtyState*> URigHierarchy::GetElementTransformStorage(const FRigElementKeyAndIndex& InKey,
	ERigTransformType::Type InTransformType, ERigTransformStorageType::Type InStorageType)
{
	FRigTransformElement* TransformElement = Get<FRigTransformElement>(InKey);
	if(TransformElement && TransformElement->GetKey() == InKey.Key)
	{
		FRigCurrentAndInitialTransform& Transform = TransformElement->PoseStorage;
		FRigCurrentAndInitialDirtyState& DirtyState = TransformElement->PoseDirtyState;
		if(InStorageType == ERigTransformStorageType::Offset || InStorageType == ERigTransformStorageType::Shape)
		{
			FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement);
			if(ControlElement == nullptr)
			{
				return {nullptr, nullptr};
			}
			if(InStorageType == ERigTransformStorageType::Offset)
			{
				Transform = ControlElement->OffsetStorage;
				DirtyState = ControlElement->OffsetDirtyState;
			}
			else
			{
				Transform = ControlElement->ShapeStorage;
				DirtyState = ControlElement->ShapeDirtyState;
			}
		}
		
		switch(InTransformType)
		{
			case ERigTransformType::InitialLocal:
			{
				return {&Transform.Initial.Local, &DirtyState.Initial.Local};
			}
			case ERigTransformType::CurrentLocal:
			{
				return {&Transform.Current.Local, &DirtyState.Current.Local};
			}
			case ERigTransformType::InitialGlobal:
			{
				return {&Transform.Initial.Global, &DirtyState.Initial.Global};
			}
			case ERigTransformType::CurrentGlobal:
			{
				return {&Transform.Current.Global, &DirtyState.Current.Global};
			}
			default:
			{
				break;
			}
		}
	}
	return {nullptr, nullptr};
}

TOptional<TTuple<int32, int32>> URigHierarchy::GetElementStorageRange(ERigTransformType::Type InTransformType) const
{
	const int32 Index = static_cast<int32>(InTransformType);
	if(ElementTransformRanges.IsValidIndex(Index))
	{
		return ElementTransformRanges[Index];
	}
	return TOptional<TTuple<int32, int32>>();
}

void URigHierarchy::PropagateMetadata(const FRigElementKey& InKey, const FName& InName, bool bNotify)
{
	if(const FRigBaseElement* Element = Find(InKey))
	{
		PropagateMetadata(Element, InName, bNotify);
	}
}

void URigHierarchy::PropagateMetadata(const FRigBaseElement* InElement, const FName& InName, bool bNotify)
{
#if WITH_EDITOR
	check(InElement);

	if (!ElementMetadata.IsValidIndex(InElement->MetadataStorageIndex))
	{
		return;
	}

	FMetadataStorage& Storage = ElementMetadata[InElement->MetadataStorageIndex];
	FRigBaseMetadata** MetadataPtrPtr = Storage.MetadataMap.Find(InName);
	if (!MetadataPtrPtr)
	{
		return;
	}

	FRigBaseMetadata* MetadataPtr = *MetadataPtrPtr;
	if(MetadataPtr == nullptr)
	{
		return;
	}

	ForEachListeningHierarchy([InElement, MetadataPtr, bNotify](const FRigHierarchyListener& Listener)
	{
		if(URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
		{
			if(FRigBaseElement* Element = ListeningHierarchy->Find(InElement->GetKey()))
			{
				if (FRigBaseMetadata* Metadata = ListeningHierarchy->GetMetadataForElement(Element, MetadataPtr->GetName(), MetadataPtr->GetType(), bNotify))
				{
					Metadata->SetValueData(MetadataPtr->GetValueData(), MetadataPtr->GetValueSize());
					ListeningHierarchy->PropagateMetadata(Element, Metadata->GetName(), bNotify);
				}
			}
		}
	});
#endif
}

TMap<FRigElementKey, URigHierarchy::FMetadataStorage> URigHierarchy::CopyMetadata() const
{
	TMap<FRigElementKey, FMetadataStorage> ElementMetadataToSave;
	for (const FRigBaseElement* Element: Elements)
	{
		if (ElementMetadata.IsValidIndex(Element->MetadataStorageIndex))
		{
			const FMetadataStorage& SourceStorage = ElementMetadata[Element->MetadataStorageIndex];
			FMetadataStorage& TargetStorage = ElementMetadataToSave.FindOrAdd(Element->Key);
			for (const TPair<FName, FRigBaseMetadata*>& Pair : SourceStorage.MetadataMap)
			{
				FRigBaseMetadata* NewValue = FRigBaseMetadata::MakeMetadata(Pair.Key, Pair.Value->GetType());
				Pair.Value->GetValueProperty()->CopyCompleteValue(NewValue->GetValueData(), Pair.Value->GetValueData());
				TargetStorage.MetadataMap.Add(Pair.Key, NewValue);
			}
		}
	}
	return ElementMetadataToSave;
}

bool URigHierarchy::SetMetadata(const TMap<FRigElementKey, URigHierarchy::FMetadataStorage>& InMetadata)
{
	bool bResult = true;
	{
		TGuardValue<FRigHierarchyMetadataChangedDelegate> MetadataChangedDelegateGuard(MetadataChangedDelegate, FRigHierarchyMetadataChangedDelegate());
		RemoveAllMetadata();
	
		for (const TPair<FRigElementKey, FMetadataStorage>& Entry: InMetadata)
		{
			if (FRigBaseElement* Element = Find(Entry.Key))
			{
				for (const TTuple<FName, FRigBaseMetadata*>& MetadataElement : Entry.Value.MetadataMap)
				{
					if (FRigBaseMetadata* TargetMetadata = GetMetadataForElement(Element, MetadataElement.Key, MetadataElement.Value->Type, false))
					{
						TargetMetadata->GetValueProperty()->CopyCompleteValue(TargetMetadata->GetValueData(), MetadataElement.Value->GetValueData());
					}
				}
			}
			else
			{
				bResult = false;
			}
		}
	}

	OnMetadataChanged(FRigElementKey(ERigElementType::All), NAME_None);

	return bResult;
}

void URigHierarchy::OnMetadataChanged(const FRigElementKey& InKey, const FName& InName)
{
	MetadataVersion++;

	if (!bSuspendMetadataNotifications)
	{
		if(MetadataChangedDelegate.IsBound())
		{
			MetadataChangedDelegate.Broadcast(InKey, InName);
		}
	}
}

void URigHierarchy::OnMetadataTagChanged(const FRigElementKey& InKey, const FName& InTag, bool bAdded)
{
	MetadataTagVersion++;

	if (!bSuspendMetadataNotifications)
	{
		if(MetadataTagChangedDelegate.IsBound())
		{
			MetadataTagChangedDelegate.Broadcast(InKey, InTag, bAdded);
		}
	}
}

FRigBaseMetadata* URigHierarchy::GetMetadataForElement(FRigBaseElement* InElement, const FName& InName, ERigMetadataType InType, bool bInNotify)
{
	if (!ElementMetadata.IsValidIndex(InElement->MetadataStorageIndex))
	{
		InElement->MetadataStorageIndex = ElementMetadata.Add(FMetadataStorage());
	}
	
	FMetadataStorage& Storage = ElementMetadata[InElement->MetadataStorageIndex];

	// If repeatedly accessing the same element, store it here for faster access to avoid map lookups.
	if (Storage.LastAccessMetadata && Storage.LastAccessName == InName && Storage.LastAccessMetadata->GetType() == InType)
	{
		return Storage.LastAccessMetadata;
	}

	FRigBaseMetadata* Metadata;
	if (FRigBaseMetadata** MetadataPtrPtr = Storage.MetadataMap.Find(InName))
	{
		if ((*MetadataPtrPtr)->GetType() == InType)
		{
			Metadata = *MetadataPtrPtr;
		}
		else
		{
			// The type changed, replace the existing metadata with a new one of the correct type.
			FRigBaseMetadata::DestroyMetadata(MetadataPtrPtr);
			Metadata = *MetadataPtrPtr = FRigBaseMetadata::MakeMetadata(InName, InType);
			
			if (bInNotify)
			{
				OnMetadataChanged(InElement->Key, InName);
			}
		}
	}
	else
	{
		// No metadata with that name existed on the element, create one from scratch.
		Metadata = FRigBaseMetadata::MakeMetadata(InName, InType);
		Storage.MetadataMap.Add(InName, Metadata);

		if (bInNotify)
		{
			OnMetadataChanged(InElement->Key, InName);
		}
	}
	
	Storage.LastAccessName = InName;
	Storage.LastAccessMetadata = Metadata;
	return Metadata;
}


FRigBaseMetadata* URigHierarchy::FindMetadataForElement(const FRigBaseElement* InElement, const FName& InName, ERigMetadataType InType)
{
	if (!ElementMetadata.IsValidIndex(InElement->MetadataStorageIndex))
	{
		return nullptr;
	}

	FMetadataStorage& Storage = ElementMetadata[InElement->MetadataStorageIndex];
	if (InName == Storage.LastAccessName && (InType == ERigMetadataType::Invalid || (Storage.LastAccessMetadata && Storage.LastAccessMetadata->GetType() == InType)))
	{
		return Storage.LastAccessMetadata;
	}
	
	FRigBaseMetadata** MetadataPtrPtr = Storage.MetadataMap.Find(InName);
	if (!MetadataPtrPtr)
	{
		Storage.LastAccessName = NAME_None;
		Storage.LastAccessMetadata = nullptr;
		return nullptr;
	}

	if (InType != ERigMetadataType::Invalid && (*MetadataPtrPtr)->GetType() != InType)
	{
		Storage.LastAccessName = NAME_None;
		Storage.LastAccessMetadata = nullptr;
		return nullptr;
	}

	Storage.LastAccessName = InName;
	Storage.LastAccessMetadata = *MetadataPtrPtr;

	return *MetadataPtrPtr;
}

const FRigBaseMetadata* URigHierarchy::FindMetadataForElement(const FRigBaseElement* InElement, const FName& InName, ERigMetadataType InType) const
{
	return const_cast<URigHierarchy*>(this)->FindMetadataForElement(InElement, InName, InType);
}

bool URigHierarchy::HasMetadata(const FRigBaseElement* InElement) const
{
	return ElementMetadata.IsValidIndex(InElement->MetadataStorageIndex) && !ElementMetadata[InElement->MetadataStorageIndex].MetadataMap.IsEmpty();
}

bool URigHierarchy::RemoveMetadataForElement(FRigBaseElement* InElement, const FName& InName)
{
	if (!ElementMetadata.IsValidIndex(InElement->MetadataStorageIndex))
	{
		return false;
	}

	FMetadataStorage& Storage = ElementMetadata[InElement->MetadataStorageIndex];
	FRigBaseMetadata** MetadataPtrPtr = Storage.MetadataMap.Find(InName);
	if (!MetadataPtrPtr)
	{
		return false;
	}
	
	FRigBaseMetadata::DestroyMetadata(MetadataPtrPtr);
	Storage.MetadataMap.Remove(InName);

	// If the storage is now empty, remove the element's storage, so we're not lugging it around
	// unnecessarily. Add the storage slot to the freelist so that the next element to add a new
	// metadata storage can just recycle that.
	if (Storage.MetadataMap.IsEmpty())
	{
		ElementMetadata.Deallocate(InElement->MetadataStorageIndex, nullptr);
		InElement->MetadataStorageIndex = INDEX_NONE;
	}
	else if (Storage.LastAccessName == InName)
	{
		Storage.LastAccessMetadata = nullptr;
	}

	if(ElementBeingDestroyed != InElement)
	{
		OnMetadataChanged(InElement->Key, InName);
	}
	return true;
}


bool URigHierarchy::RemoveAllMetadataForElement(FRigBaseElement* InElement)
{
	if (!ElementMetadata.IsValidIndex(InElement->MetadataStorageIndex))
	{
		return false;
	}

	
	FMetadataStorage& Storage = ElementMetadata[InElement->MetadataStorageIndex];
	TArray<FName> Names;
	Storage.MetadataMap.GetKeys(Names);
	
	// Clear the storage for the next user.
	Storage.Reset();
	
	ElementMetadata.Deallocate(InElement->MetadataStorageIndex, nullptr);
	InElement->MetadataStorageIndex = INDEX_NONE;

	if(ElementBeingDestroyed != InElement)
	{
		for (FName Name: Names)
		{
			OnMetadataChanged(InElement->Key, Name);
		}
	}
	
	return true; 
}

bool URigHierarchy::RemoveAllMetadata()
{
	bool bSuccess = true;

	{
		TGuardValue<FRigHierarchyMetadataChangedDelegate> MetadataChangedDelegateGuard(MetadataChangedDelegate, FRigHierarchyMetadataChangedDelegate());
		ForEach([this, &bSuccess](FRigBaseElement* Element)
		{
			bSuccess &= RemoveAllMetadataForElement(Element);
			return true;
		});
	}

	OnMetadataChanged(FRigElementKey(ERigElementType::All), NAME_None);
	
	return bSuccess;
}


void URigHierarchy::CopyAllMetadataFromElement(FRigBaseElement* InTargetElement, const FRigBaseElement* InSourceElement)
{
	if (!ensure(InSourceElement->Owner))
	{
		return;
	}

	if (!InSourceElement->Owner->ElementMetadata.IsValidIndex(InSourceElement->MetadataStorageIndex))
	{
		return;
	}
	
	const FMetadataStorage& SourceStorage = InSourceElement->Owner->ElementMetadata[InSourceElement->MetadataStorageIndex];
	
	for (const TTuple<FName, FRigBaseMetadata*>& SourceItem: SourceStorage.MetadataMap)
	{
		const FRigBaseMetadata* SourceMetadata = SourceItem.Value; 	

		constexpr bool bNotify = false;
		FRigBaseMetadata* TargetMetadata = GetMetadataForElement(InTargetElement, SourceItem.Key, SourceMetadata->GetType(), bNotify);
		TargetMetadata->SetValueData(SourceMetadata->GetValueData(), SourceMetadata->GetValueSize());
	}
}


void URigHierarchy::EnsureCacheValidityImpl()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	if(!bEnableCacheValidityCheck)
	{
		return;
	}
	TGuardValue<bool> Guard(bEnableCacheValidityCheck, false);

	static const TArray<FString>& TransformTypeStrings = GetTransformTypeStrings();

	// make sure that elements which are marked as dirty don't have fully cached children
	ForEach<FRigTransformElement>([](FRigTransformElement* TransformElement)
    {
		for(int32 TransformTypeIndex = 0; TransformTypeIndex < (int32) ERigTransformType::NumTransformTypes; TransformTypeIndex++)
		{
			const ERigTransformType::Type GlobalType = (ERigTransformType::Type)TransformTypeIndex;
			const ERigTransformType::Type LocalType = ERigTransformType::SwapLocalAndGlobal(GlobalType);
			const FString& TransformTypeString = TransformTypeStrings[TransformTypeIndex];

			if(ERigTransformType::IsLocal(GlobalType))
			{
				continue;
			}

			if(!TransformElement->GetDirtyState().IsDirty(GlobalType))
			{
				continue;
			}

			for(const FRigTransformElement::FElementToDirty& ElementToDirty : TransformElement->ElementsToDirty)
			{
				if(FRigMultiParentElement* MultiParentElementToDirty = Cast<FRigMultiParentElement>(ElementToDirty.Element))
				{
                    if(FRigControlElement* ControlElementToDirty = Cast<FRigControlElement>(ElementToDirty.Element))
                    {
                        if(ControlElementToDirty->GetOffsetDirtyState().IsDirty(GlobalType))
                        {
                            checkf(ControlElementToDirty->GetDirtyState().IsDirty(GlobalType) ||
                                    ControlElementToDirty->GetDirtyState().IsDirty(LocalType),
                                    TEXT("Control '%s' %s Offset Cache is dirty, but the Pose is not."),
									*ControlElementToDirty->GetKey().ToString(),
									*TransformTypeString);
						}

                        if(ControlElementToDirty->GetDirtyState().IsDirty(GlobalType))
                        {
                            checkf(ControlElementToDirty->GetShapeDirtyState().IsDirty(GlobalType) ||
                                    ControlElementToDirty->GetShapeDirtyState().IsDirty(LocalType),
                                    TEXT("Control '%s' %s Pose Cache is dirty, but the Shape is not."),
									*ControlElementToDirty->GetKey().ToString(),
									*TransformTypeString);
						}
                    }
                    else
                    {
                        checkf(MultiParentElementToDirty->GetDirtyState().IsDirty(GlobalType) ||
                                MultiParentElementToDirty->GetDirtyState().IsDirty(LocalType),
                                TEXT("MultiParent '%s' %s Parent Cache is dirty, but the Pose is not."),
								*MultiParentElementToDirty->GetKey().ToString(),
								*TransformTypeString);
                    }
				}
				else
				{
					checkf(ElementToDirty.Element->GetDirtyState().IsDirty(GlobalType) ||
						ElementToDirty.Element->GetDirtyState().IsDirty(LocalType),
						TEXT("SingleParent '%s' %s Pose is not dirty in Local or Global"),
						*ElementToDirty.Element->GetKey().ToString(),
						*TransformTypeString);
				}
			}
		}
		
        return true;
    });

	// store our own pose in a transient hierarchy used for cache validation
	if(HierarchyForCacheValidation == nullptr)
	{
		HierarchyForCacheValidation = NewObject<URigHierarchy>(this, NAME_None, RF_Transient);
		HierarchyForCacheValidation->bEnableCacheValidityCheck = false;
	}
	if(HierarchyForCacheValidation->GetTopologyVersion() != GetTopologyVersion())
	{
		HierarchyForCacheValidation->CopyHierarchy(this);
	}
	HierarchyForCacheValidation->CopyPose(this, true, true, true);

	// traverse the copied hierarchy and compare cached vs computed values
	URigHierarchy* HierarchyForLambda = HierarchyForCacheValidation;
	HierarchyForLambda->Traverse([HierarchyForLambda](FRigBaseElement* Element, bool& bContinue)
	{
		bContinue = true;

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			for(int32 TransformTypeIndex = 0; TransformTypeIndex < (int32) ERigTransformType::NumTransformTypes; TransformTypeIndex++)
			{
				const ERigTransformType::Type TransformType = (ERigTransformType::Type)TransformTypeIndex;
				const ERigTransformType::Type OpposedType = ERigTransformType::SwapLocalAndGlobal(TransformType);
				const FString& TransformTypeString = TransformTypeStrings[TransformTypeIndex];

				if(!ControlElement->GetOffsetDirtyState().IsDirty(TransformType) && !ControlElement->GetOffsetDirtyState().IsDirty(OpposedType))
				{
					const FTransform CachedTransform = HierarchyForLambda->GetControlOffsetTransform(ControlElement, TransformType);
					ControlElement->GetOffsetDirtyState().MarkDirty(TransformType);
					const FTransform ComputedTransform = HierarchyForLambda->GetControlOffsetTransform(ControlElement, TransformType);
					checkf(FRigComputedTransform::Equals(CachedTransform, ComputedTransform),
						TEXT("Element '%s' Offset %s Cached vs Computed doesn't match. ('%s' <-> '%s')"),
						*Element->GetName(),
						*TransformTypeString,
						*CachedTransform.ToString(),
						*ComputedTransform.ToString());
				}
			}
		}

		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
		{
			for(int32 TransformTypeIndex = 0; TransformTypeIndex < (int32) ERigTransformType::NumTransformTypes; TransformTypeIndex++)
			{
				const ERigTransformType::Type TransformType = (ERigTransformType::Type)TransformTypeIndex;
				const ERigTransformType::Type OpposedType = ERigTransformType::SwapLocalAndGlobal(TransformType);
				const FString& TransformTypeString = TransformTypeStrings[TransformTypeIndex];

				if(!TransformElement->GetDirtyState().IsDirty(TransformType) && !TransformElement->GetDirtyState().IsDirty(OpposedType))
				{
					const FTransform CachedTransform = HierarchyForLambda->GetTransform(TransformElement, TransformType);
					TransformElement->GetDirtyState().MarkDirty(TransformType);
					const FTransform ComputedTransform = HierarchyForLambda->GetTransform(TransformElement, TransformType);
					checkf(FRigComputedTransform::Equals(CachedTransform, ComputedTransform),
						TEXT("Element '%s' Pose %s Cached vs Computed doesn't match. ('%s' <-> '%s')"),
						*Element->GetName(),
						*TransformTypeString,
						*CachedTransform.ToString(), *ComputedTransform.ToString());
				}
			}
		}

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			for(int32 TransformTypeIndex = 0; TransformTypeIndex < (int32) ERigTransformType::NumTransformTypes; TransformTypeIndex++)
			{
				const ERigTransformType::Type TransformType = (ERigTransformType::Type)TransformTypeIndex;
				const ERigTransformType::Type OpposedType = ERigTransformType::SwapLocalAndGlobal(TransformType);
				const FString& TransformTypeString = TransformTypeStrings[TransformTypeIndex];

				if(!ControlElement->GetShapeDirtyState().IsDirty(TransformType) && !ControlElement->GetShapeDirtyState().IsDirty(OpposedType))
				{
					const FTransform CachedTransform = HierarchyForLambda->GetControlShapeTransform(ControlElement, TransformType);
					ControlElement->GetShapeDirtyState().MarkDirty(TransformType);
					const FTransform ComputedTransform = HierarchyForLambda->GetControlShapeTransform(ControlElement, TransformType);
					checkf(FRigComputedTransform::Equals(CachedTransform, ComputedTransform),
						TEXT("Element '%s' Shape %s Cached vs Computed doesn't match. ('%s' <-> '%s')"),
						*Element->GetName(),
						*TransformTypeString,
						*CachedTransform.ToString(), *ComputedTransform.ToString());
				}
			}
		}
	});
}

FName URigHierarchy::GetUniqueName(const FName& InName, TFunction<bool(const FName&)> IsNameAvailablePredicate)
{
	static constexpr int32 InitialSuffix = 3; // offset since FName uses 0 as the indicator that there's no number suffix, plus we want to start with index 2. 
	int32 NameSuffix = InitialSuffix; 
	FName Name = InName;

	while (!IsNameAvailablePredicate(Name))
	{
		if(NameSuffix == InitialSuffix)
		{
			Name = FName(Name, NameSuffix);
		}
		else
		{
			Name.SetNumber(NameSuffix);
		}
		NameSuffix++;
	}
	return Name;
}

void URigHierarchy::UpdateVisibilityOnProxyControls()
{
	URigHierarchy* HierarchyForSelection = HierarchyForSelectionPtr.Get();
	if(HierarchyForSelection == nullptr)
	{
		HierarchyForSelection = this;
	}

	const UWorld* World = GetWorld();
	if (!World || World->IsPreviewWorld())
	{
		return;
	}

	constexpr int32 ControlsFlat = ElementToFlatIndex<FRigControlElement>();
	const TArray<FRigBaseElement*>& Controls = ElementsPerType.IsValidIndex(ControlsFlat) ? ElementsPerType[ControlsFlat] : Elements;
	constexpr bool bForceVisibility = true;
	
	// hide proxies based on selection if there's nothing selected
	if (HierarchyForSelection->OrderedSelection.IsEmpty())
	{
		constexpr bool bNotVisible = false;
		for (FRigBaseElement* Element: Controls)
		{
			if (FRigControlElement* Control = Cast<FRigControlElement>(Element))
			{
				FRigControlSettings& Settings = Control->Settings;
				
				if (Settings.AnimationType == ERigControlAnimationType::ProxyControl &&
					Settings.ShapeVisibility == ERigControlVisibility::BasedOnSelection)
				{
					if (Settings.SetVisible(bNotVisible, bForceVisibility))
					{
						Notify(ERigHierarchyNotification::ControlVisibilityChanged, Control);
					}
				}
			}
		}
		return;
	}

	// otherwise, create a local map of visible things, starting with the selection
	TSet<FRigHierarchyKey> VisibleElements(HierarchyForSelection->OrderedSelection);

	// if the control is visible - we should consider the
	// driven controls visible as well - so that other proxies
	// assigned to the same driven control also show up
	TSet<FRigControlElement*> ProxiesBasedOnSelection;
	for (FRigBaseElement* Element: Controls)
	{
		if (FRigControlElement* Control = Cast<FRigControlElement>(Element))
		{
			FRigControlSettings& Settings = Control->Settings;
			
			if (Settings.AnimationType == ERigControlAnimationType::ProxyControl)
			{
				if (Settings.ShapeVisibility == ERigControlVisibility::BasedOnSelection)
				{
					ProxiesBasedOnSelection.Add(Control);
				}
				
				if (VisibleElements.Contains(Control->GetKey()))
				{
					for(const FRigElementKey& DrivenControlKey : Settings.DrivenControls)
					{
						VisibleElements.Add(DrivenControlKey);
					}
				}
			}
		}
	}

	for (FRigControlElement* Proxy : ProxiesBasedOnSelection)
	{
		// a proxy control should be visible if itself or a driven control is selected / visible
		bool bVisible = VisibleElements.Contains(Proxy->GetKey());
		if (!bVisible)
		{
			if(Proxy->Settings.DrivenControls.FindByPredicate([VisibleElements](const FRigElementKey& AffectedControl) -> bool
			{
				return VisibleElements.Contains(AffectedControl);
			}) != nullptr)
			{
				bVisible = true;
			}
		}

		if(bVisible)
		{
			VisibleElements.Add(Proxy->GetKey());
		}

		if(Proxy->Settings.SetVisible(bVisible, bForceVisibility))
		{
			Notify(ERigHierarchyNotification::ControlVisibilityChanged, Proxy);
		}
	}
}

const TArray<FString>& URigHierarchy::GetTransformTypeStrings()
{
	static TArray<FString> TransformTypeStrings;
	if(TransformTypeStrings.IsEmpty())
	{
		for(int32 TransformTypeIndex = 0; TransformTypeIndex < (int32) ERigTransformType::NumTransformTypes; TransformTypeIndex++)
		{
			TransformTypeStrings.Add(StaticEnum<ERigTransformType::Type>()->GetDisplayNameTextByValue((int64)TransformTypeIndex).ToString());
		}
	}
	return TransformTypeStrings;
}

void URigHierarchy::PushTransformToStack(const FRigElementKey& InKey, ERigTransformStackEntryType InEntryType,
                                         ERigTransformType::Type InTransformType, const FTransform& InOldTransform, const FTransform& InNewTransform,
                                         bool bAffectChildren, bool bModify)
{
#if WITH_EDITOR

	if(GIsTransacting)
	{
		return;
	}

	static const FText TransformPoseTitle = NSLOCTEXT("RigHierarchy", "Set Pose Transform", "Set Pose Transform");
	static const FText ControlOffsetTitle = NSLOCTEXT("RigHierarchy", "Set Control Offset", "Set Control Offset");
	static const FText ControlShapeTitle = NSLOCTEXT("RigHierarchy", "Set Control Shape", "Set Control Shape");
	static const FText CurveValueTitle = NSLOCTEXT("RigHierarchy", "Set Curve Value", "Set Curve Value");
	
	FText Title;
	switch(InEntryType)
	{
		case ERigTransformStackEntryType::TransformPose:
		{
			Title = TransformPoseTitle;
			break;
		}
		case ERigTransformStackEntryType::ControlOffset:
		{
			Title = TransformPoseTitle;
			break;
		}
		case ERigTransformStackEntryType::ControlShape:
		{
			Title = TransformPoseTitle;
			break;
		}
		case ERigTransformStackEntryType::CurveValue:
		{
			Title = TransformPoseTitle;
			break;
		}
	}

	TGuardValue<bool> TransactingGuard(bTransactingForTransformChange, true);

	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bModify)
	{
		TransactionPtr = MakeShareable(new FScopedTransaction(Title));
	}

	if(bIsInteracting)
	{
		bool bCanMerge = LastInteractedKey == InKey;

		FRigTransformStackEntry LastEntry;
		if(!TransformUndoStack.IsEmpty())
		{
			LastEntry = TransformUndoStack.Last();
		}

		if(bCanMerge && LastEntry.Key == InKey && LastEntry.EntryType == InEntryType && LastEntry.bAffectChildren == bAffectChildren)
		{
			// merge the entries on the stack
			TransformUndoStack.Last() = 
                FRigTransformStackEntry(InKey, InEntryType, InTransformType, LastEntry.OldTransform, InNewTransform, bAffectChildren);
		}
		else
		{
			Modify();

			TransformUndoStack.Add(
                FRigTransformStackEntry(InKey, InEntryType, InTransformType, InOldTransform, InNewTransform, bAffectChildren));
			TransformStackIndex = TransformUndoStack.Num();
		}

		TransformRedoStack.Reset();
		LastInteractedKey = InKey;
		return;
	}

	if(bModify)
	{
		Modify();
	}

	TArray<FString> Callstack;
	if(IsTracingChanges() && (CVarControlRigHierarchyTraceCallstack->GetInt() != 0))
	{
		FString JoinedCallStack;
		RigHierarchyCaptureCallStack(JoinedCallStack, 1);
		JoinedCallStack.ReplaceInline(TEXT("\r"), TEXT(""));

		FString Left, Right;
		do
		{
			if(!JoinedCallStack.Split(TEXT("\n"), &Left, &Right))
			{
				Left = JoinedCallStack;
				Right.Empty();
			}

			Left.TrimStartAndEndInline();
			if(Left.StartsWith(TEXT("0x")))
			{
				Left.Split(TEXT(" "), nullptr, &Left);
			}
			Callstack.Add(Left);
			JoinedCallStack = Right;
		}
		while(!JoinedCallStack.IsEmpty());
	}

	TransformUndoStack.Add(
		FRigTransformStackEntry(InKey, InEntryType, InTransformType, InOldTransform, InNewTransform, bAffectChildren, Callstack));
	TransformStackIndex = TransformUndoStack.Num();

	TransformRedoStack.Reset();
	
#endif
}

void URigHierarchy::PushCurveToStack(const FRigElementKey& InKey, float InOldCurveValue, float InNewCurveValue, bool bInOldIsCurveValueSet, bool bInNewIsCurveValueSet, bool bModify)
{
#if WITH_EDITOR

	FTransform OldTransform = FTransform::Identity;
	FTransform NewTransform = FTransform::Identity;

	OldTransform.SetTranslation(FVector(InOldCurveValue, bInOldIsCurveValueSet ? 1.f : 0.f, 0.f));
	NewTransform.SetTranslation(FVector(InNewCurveValue, bInNewIsCurveValueSet ? 1.f : 0.f, 0.f));

	PushTransformToStack(InKey, ERigTransformStackEntryType::CurveValue, ERigTransformType::CurrentLocal, OldTransform, NewTransform, false, bModify);

#endif
}

bool URigHierarchy::ApplyTransformFromStack(const FRigTransformStackEntry& InEntry, bool bUndo)
{
#if WITH_EDITOR

	bool bApplyInitialForCurrent = false;
	FRigBaseElement* Element = Find(InEntry.Key);
	if(Element == nullptr)
	{
		// this might be a transient control which had been removed.
		if(InEntry.Key.Type == ERigElementType::Control)
		{
			const FRigElementKey TargetKey = UControlRig::GetElementKeyFromTransientControl(InEntry.Key);
			Element = Find(TargetKey);
			bApplyInitialForCurrent = Element != nullptr;
		}

		if(Element == nullptr)
		{
			return false;
		}
	}

	const FTransform& Transform = bUndo ? InEntry.OldTransform : InEntry.NewTransform;
	
	switch(InEntry.EntryType)
	{
		case ERigTransformStackEntryType::TransformPose:
		{
			SetTransform(Cast<FRigTransformElement>(Element), Transform, InEntry.TransformType, InEntry.bAffectChildren, false);

			if(ERigTransformType::IsCurrent(InEntry.TransformType) && bApplyInitialForCurrent)
			{
				SetTransform(Cast<FRigTransformElement>(Element), Transform, ERigTransformType::MakeInitial(InEntry.TransformType), InEntry.bAffectChildren, false);
			}
			break;
		}
		case ERigTransformStackEntryType::ControlOffset:
		{
			SetControlOffsetTransform(Cast<FRigControlElement>(Element), Transform, InEntry.TransformType, InEntry.bAffectChildren, false); 
			break;
		}
		case ERigTransformStackEntryType::ControlShape:
		{
			SetControlShapeTransform(Cast<FRigControlElement>(Element), Transform, InEntry.TransformType, false); 
			break;
		}
		case ERigTransformStackEntryType::CurveValue:
		{
			const float CurveValue = Transform.GetTranslation().X;
			SetCurveValue(Cast<FRigCurveElement>(Element), CurveValue, false);
			break;
		}
	}

	return true;
#else
	return false;
#endif
}

void URigHierarchy::ComputeAllTransforms()
{
	for(int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
	{
		for(int32 TransformTypeIndex = 0; TransformTypeIndex < (int32) ERigTransformType::NumTransformTypes; TransformTypeIndex++)
		{
			const ERigTransformType::Type TransformType = (ERigTransformType::Type)TransformTypeIndex; 
			if(FRigControlElement* ControlElement = Get<FRigControlElement>(ElementIndex))
			{
				GetControlOffsetTransform(ControlElement, TransformType);
			}
			if(FRigTransformElement* TransformElement = Get<FRigTransformElement>(ElementIndex))
			{
				GetTransform(TransformElement, TransformType);
			}
			if(FRigControlElement* ControlElement = Get<FRigControlElement>(ElementIndex))
			{
				GetControlShapeTransform(ControlElement, TransformType);
			}
		}
	}
}

#if WITH_EDITOR
void URigHierarchy::NotifyPostUndoSelectionChanges()
{
	for (const FRigHierarchyKey& PreviouslySelectedKey : SelectedKeysBeforeUndo)
	{
		FRigBaseElement* Element = Find<FRigBaseElement>(PreviouslySelectedKey.GetElement());
		if (!Element) // Was element removed by the transaction?
		{
			continue;
		}

		if (!Element->IsSelected())
		{
			Notify(ERigHierarchyNotification::ElementDeselected, FRigNotificationSubject(Element));
		}
	}

	for (const FRigBaseElement* Element : GetSelectedElements(ERigElementType::All))
	{
		if (!SelectedKeysBeforeUndo.Contains(Element->Key))
		{
			Notify(ERigHierarchyNotification::ElementSelected, FRigNotificationSubject(Element));
		}
	}

	SelectedKeysBeforeUndo.Reset();
}

FString URigHierarchy::GetMessageFromDependencyChain(const FRigHierarchyDependencyChain& InDependencyChain) const
{
	TArray<FString> Messages;

	for (const TPair<FRigHierarchyRecord,FRigHierarchyRecord>& DependencyPair : InDependencyChain)
	{
		auto GetLabelForRecord = [this](const FRigHierarchyRecord& InRecord) -> FString
		{
			FString Label;
				
			switch (InRecord.Type)
			{
				case FRigHierarchyRecord::EType_RigElement:
				case FRigHierarchyRecord::EType_Metadata:
				{
					Label = GetKey(InRecord.Index).ToString();
					if (InRecord.IsMetadata())
					{
						Label += FString::Printf(TEXT(" Metadata(%s)"), *InRecord.Name.ToString());
					}
					return Label;
				}
				case FRigHierarchyRecord::EType_Variable:
				case FRigHierarchyRecord::EType_Instruction:
				{
					if (InRecord.IsVariable())
					{
						Label = TEXT("Variable");
					}
					else
					{
						Label = TEXT("Node");
					}

					UControlRig* ControlRig = Cast<UControlRig>(GetOuter());
					if (!ControlRig)
					{
						break;
					}

					UModularRig* ModularRig = Cast<UModularRig>(ControlRig);

					const URigVM* VM = nullptr;
					if (InRecord.Name.IsNone() || !ModularRig)
					{
						VM = ControlRig->GetVM();
					}
					else
					{
						const FRigModuleInstance* RigModule = ModularRig->FindModule(InRecord.Name);
						if (!RigModule)
						{
							break;
						}
						UControlRig* ModuleRig = RigModule->GetRig();
						if (!ModuleRig)
						{
							break;
						}
						VM = ModuleRig->GetVM();
					}

					check(VM);

					if (InRecord.IsVariable())
					{
						const TArray<FRigVMExternalVariableDef>& ExternalVariableDefs = VM->GetExternalVariableDefs();
						if (ExternalVariableDefs.IsValidIndex(InRecord.Index))
						{
							Label += FString::Printf(TEXT("(%s)"), *ExternalVariableDefs[InRecord.Index].GetName().ToString());
						}
					}
					else
					{
						if (!VM->GetByteCode().GetInstructions().IsValidIndex(InRecord.Index))
						{
							break;
						}

						const FRigVMInstruction Instruction = VM->GetByteCode().GetInstructions()[InRecord.Index];
						if (Instruction.OpCode != ERigVMOpCode::Execute)
						{
							break;
						}

						URigVMNode* Node = Cast<URigVMNode>(VM->GetByteCode().GetSubjectForInstruction(InRecord.Index));
						if (Node == nullptr)
						{
							break;
						}

						Label += FString::Printf(TEXT("(Instruction %03d, %s)"), InRecord.Index, *Node->GetNodeTitle());
					}

					break;
				}
				default:
				{
					Label = TEXT("Unknown");
					break;
				}
			}

			return Label;
		};

		const FString TargetLabel = GetLabelForRecord(DependencyPair.Value);
		const FString SourceLabel = GetLabelForRecord(DependencyPair.Key);
		Messages.Add(FString::Printf(TEXT("'%s' depends on '%s'."), *TargetLabel, *SourceLabel));
	}

	return FString::Join(Messages, TEXT("\n"));
}
#endif

bool URigHierarchy::IsAnimatable(const FRigElementKey& InKey) const
{
	if(const FRigControlElement* ControlElement = Find<FRigControlElement>(InKey))
	{
		return IsAnimatable(ControlElement);
	}
	return false;
}

bool URigHierarchy::IsAnimatable(const FRigControlElement* InControlElement) const
{
	if(InControlElement)
	{
		if(!InControlElement->Settings.IsAnimatable())
		{
			return false;
		}

		// animation channels are dependent on the control they are under.
		if(InControlElement->IsAnimationChannel())
		{
			if(const FRigControlElement* ParentControlElement = Cast<FRigControlElement>(GetFirstParent(InControlElement)))
			{
				return IsAnimatable(ParentControlElement);
			}
		}
		
		return true;
	}
	return false;
}

bool URigHierarchy::ShouldBeGrouped(const FRigElementKey& InKey) const
{
	if(const FRigControlElement* ControlElement = Find<FRigControlElement>(InKey))
	{
		return ShouldBeGrouped(ControlElement);
	}
	return false;
}

bool URigHierarchy::ShouldBeGrouped(const FRigControlElement* InControlElement) const
{
	if(InControlElement)
	{
		if(!InControlElement->Settings.ShouldBeGrouped())
		{
			return false;
		}

		if(!GetChildren(InControlElement).IsEmpty())
		{
			return false;
		}

		if(const FRigControlElement* ParentControlElement = Cast<FRigControlElement>(GetFirstParent(InControlElement)))
		{
			return ParentControlElement->Settings.AnimationType == ERigControlAnimationType::AnimationControl;
		}
	}
	return false;
}

FTransform URigHierarchy::GetWorldTransformForReference(const FRigVMExecuteContext* InContext, const FRigElementKey& InKey, bool bInitial)
{
	if(const USceneComponent* OuterSceneComponent = GetTypedOuter<USceneComponent>())
	{
		return OuterSceneComponent->GetComponentToWorld().Inverse();
	}
	return FTransform::Identity;
}

FTransform URigHierarchy::ComputeLocalControlValue(FRigControlElement* ControlElement,
	const FTransform& InGlobalTransform, ERigTransformType::Type InTransformType) const
{
	check(ERigTransformType::IsGlobal(InTransformType));

	const FTransform OffsetTransform =
		GetControlOffsetTransform(ControlElement, ERigTransformType::MakeLocal(InTransformType));

	FTransform Result = InverseSolveParentConstraints(
		InGlobalTransform,
		ControlElement->ParentConstraints,
		InTransformType,
		OffsetTransform);

	return Result;
}

FTransform URigHierarchy::SolveParentConstraints(
	const FRigElementParentConstraintArray& InConstraints,
	const ERigTransformType::Type InTransformType,
	const FTransform& InLocalOffsetTransform,
	bool bApplyLocalOffsetTransform,
	const FTransform& InLocalPoseTransform,
	bool bApplyLocalPoseTransform) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	FTransform Result = FTransform::Identity;
	const bool bInitial = IsInitial(InTransformType);

	// collect all of the weights
	FConstraintIndex FirstConstraint;
	FConstraintIndex SecondConstraint;
	FConstraintIndex NumConstraintsAffecting(0);
	FRigElementWeight TotalWeight(0.f);
	ComputeParentConstraintIndices(InConstraints, InTransformType, FirstConstraint, SecondConstraint, NumConstraintsAffecting, TotalWeight);

	// performance improvement for case of a single parent
	if(NumConstraintsAffecting.Location == 1 &&
		NumConstraintsAffecting.Rotation == 1 &&
		NumConstraintsAffecting.Scale == 1 &&
		FirstConstraint.Location == FirstConstraint.Rotation &&
		FirstConstraint.Location == FirstConstraint.Scale)
	{
		return LazilyComputeParentConstraint(InConstraints, FirstConstraint.Location, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);
	}

	if(NumConstraintsAffecting.Location == 0 ||
		NumConstraintsAffecting.Rotation == 0 ||
		NumConstraintsAffecting.Scale == 0)
	{
		if(bApplyLocalOffsetTransform)
		{
			Result = InLocalOffsetTransform;
		}
		
		if(bApplyLocalPoseTransform)
		{
			Result = InLocalPoseTransform * Result;
		}

		if(NumConstraintsAffecting.Location == 0 &&
			NumConstraintsAffecting.Rotation == 0 &&
			NumConstraintsAffecting.Scale == 0)
		{
			Result.NormalizeRotation();
			return Result;
		}
	}

	if(NumConstraintsAffecting.Location == 1)
	{
		check(FirstConstraint.Location != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraint = InConstraints[FirstConstraint.Location];
		const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Location, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

		check(Weight.AffectsLocation());
		Result.SetLocation(Transform.GetLocation());
	}
	else if(NumConstraintsAffecting.Location == 2)
	{
		check(FirstConstraint.Location != INDEX_NONE);
		check(SecondConstraint.Location != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraintA = InConstraints[FirstConstraint.Location];
		const FRigElementParentConstraint& ParentConstraintB = InConstraints[SecondConstraint.Location];

		const FRigElementWeight& WeightA = ParentConstraintA.GetWeight(bInitial); 
		const FRigElementWeight& WeightB = ParentConstraintB.GetWeight(bInitial);
		check(WeightA.AffectsLocation());
		check(WeightB.AffectsLocation());
		const float Weight = GetWeightForLerp(WeightA.Location, WeightB.Location);

		const FTransform TransformA = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Location, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);
		const FTransform TransformB = LazilyComputeParentConstraint(InConstraints, SecondConstraint.Location, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

		const FVector ParentLocationA = TransformA.GetLocation();
		const FVector ParentLocationB = TransformB.GetLocation();
		Result.SetLocation(FMath::Lerp<FVector>(ParentLocationA, ParentLocationB, Weight));
	}
	else if(NumConstraintsAffecting.Location > 2)
	{
		check(TotalWeight.Location > SMALL_NUMBER);
		
		FVector Location = FVector::ZeroVector;
		
		for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
		{
			const FRigElementParentConstraint& ParentConstraint = InConstraints[ConstraintIndex];
			const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
			if(!Weight.AffectsLocation())
			{
				continue;
			}

			const FTransform Transform = LazilyComputeParentConstraint(InConstraints, ConstraintIndex, InTransformType,
				InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

			IntegrateParentConstraintVector(Location, Transform, Weight.Location / TotalWeight.Location, true);
		}

		Result.SetLocation(Location);
	}

	if(NumConstraintsAffecting.Rotation == 1)
	{
		check(FirstConstraint.Rotation != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraint = InConstraints[FirstConstraint.Rotation];
		const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Rotation, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);
		check(Weight.AffectsRotation());
		Result.SetRotation(Transform.GetRotation());
	}
	else if(NumConstraintsAffecting.Rotation == 2)
	{
		check(FirstConstraint.Rotation != INDEX_NONE);
		check(SecondConstraint.Rotation != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraintA = InConstraints[FirstConstraint.Rotation];
		const FRigElementParentConstraint& ParentConstraintB = InConstraints[SecondConstraint.Rotation];

		const FRigElementWeight& WeightA = ParentConstraintA.GetWeight(bInitial); 
		const FRigElementWeight& WeightB = ParentConstraintB.GetWeight(bInitial);
		check(WeightA.AffectsRotation());
		check(WeightB.AffectsRotation());
		const float Weight = GetWeightForLerp(WeightA.Rotation, WeightB.Rotation);

		const FTransform TransformA = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Rotation, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);
		const FTransform TransformB = LazilyComputeParentConstraint(InConstraints, SecondConstraint.Rotation, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

		const FQuat ParentRotationA = TransformA.GetRotation();
		const FQuat ParentRotationB = TransformB.GetRotation();
		Result.SetRotation(FQuat::Slerp(ParentRotationA, ParentRotationB, Weight));
	}
	else if(NumConstraintsAffecting.Rotation > 2)
	{
		check(TotalWeight.Rotation > SMALL_NUMBER);
		
		int NumMixedRotations = 0;
		FQuat FirstRotation = FQuat::Identity;
		FQuat MixedRotation = FQuat(0.f, 0.f, 0.f, 0.f);

		for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
		{
			const FRigElementParentConstraint& ParentConstraint = InConstraints[ConstraintIndex];
			const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
			if(!Weight.AffectsRotation())
			{
				continue;
			}

			const FTransform Transform = LazilyComputeParentConstraint(InConstraints, ConstraintIndex, InTransformType,
				InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

			IntegrateParentConstraintQuat(
				NumMixedRotations,
				FirstRotation,
				MixedRotation,
				Transform,
				Weight.Rotation / TotalWeight.Rotation);
		}

		Result.SetRotation(MixedRotation.GetNormalized());
	}

	if(NumConstraintsAffecting.Scale == 1)
	{
		check(FirstConstraint.Scale != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraint = InConstraints[FirstConstraint.Scale];
		const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
		
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Scale, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

		check(Weight.AffectsScale());
		Result.SetScale3D(Transform.GetScale3D());
	}
	else if(NumConstraintsAffecting.Scale == 2)
	{
		check(FirstConstraint.Scale != INDEX_NONE);
		check(SecondConstraint.Scale != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraintA = InConstraints[FirstConstraint.Scale];
		const FRigElementParentConstraint& ParentConstraintB = InConstraints[SecondConstraint.Scale];

		const FRigElementWeight& WeightA = ParentConstraintA.GetWeight(bInitial); 
		const FRigElementWeight& WeightB = ParentConstraintB.GetWeight(bInitial);
		check(WeightA.AffectsScale());
		check(WeightB.AffectsScale());
		const float Weight = GetWeightForLerp(WeightA.Scale, WeightB.Scale);

		const FTransform TransformA = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Scale, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);
		const FTransform TransformB = LazilyComputeParentConstraint(InConstraints, SecondConstraint.Scale, InTransformType,
			InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

		const FVector ParentScaleA = TransformA.GetScale3D();
		const FVector ParentScaleB = TransformB.GetScale3D();
		Result.SetScale3D(FMath::Lerp<FVector>(ParentScaleA, ParentScaleB, Weight));
	}
	else if(NumConstraintsAffecting.Scale > 2)
	{
		check(TotalWeight.Scale > SMALL_NUMBER);
		
		FVector Scale = FVector::ZeroVector;
		
		for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
		{
			const FRigElementParentConstraint& ParentConstraint = InConstraints[ConstraintIndex];
			const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
			if(!Weight.AffectsScale())
			{
				continue;
			}

			const FTransform Transform = LazilyComputeParentConstraint(InConstraints, ConstraintIndex, InTransformType,
				InLocalOffsetTransform, bApplyLocalOffsetTransform, InLocalPoseTransform, bApplyLocalPoseTransform);

			IntegrateParentConstraintVector(Scale, Transform, Weight.Scale / TotalWeight.Scale, false);
		}

		Result.SetScale3D(Scale);
	}

	Result.NormalizeRotation();
	return Result;
}

FTransform URigHierarchy::InverseSolveParentConstraints(
	const FTransform& InGlobalTransform,
	const FRigElementParentConstraintArray& InConstraints,
	const ERigTransformType::Type InTransformType,
	const FTransform& InLocalOffsetTransform) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));

	FTransform Result = FTransform::Identity;

	// this function is doing roughly the following 
	// ResultLocalTransform = InGlobalTransform.GetRelative(ParentGlobal)
	// InTransformType is only used to determine Initial vs Current
	const bool bInitial = IsInitial(InTransformType);
	check(ERigTransformType::IsGlobal(InTransformType));

	// collect all of the weights
	FConstraintIndex FirstConstraint;
	FConstraintIndex SecondConstraint;
	FConstraintIndex NumConstraintsAffecting(0);
	FRigElementWeight TotalWeight(0.f);
	ComputeParentConstraintIndices(InConstraints, InTransformType, FirstConstraint, SecondConstraint, NumConstraintsAffecting, TotalWeight);

	// performance improvement for case of a single parent
	if(NumConstraintsAffecting.Location == 1 &&
		NumConstraintsAffecting.Rotation == 1 &&
		NumConstraintsAffecting.Scale == 1 &&
		FirstConstraint.Location == FirstConstraint.Rotation &&
		FirstConstraint.Location == FirstConstraint.Scale)
	{
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Location, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);

		return InGlobalTransform.GetRelativeTransform(Transform);
	}

	if(NumConstraintsAffecting.Location == 0 ||
		NumConstraintsAffecting.Rotation == 0 ||
		NumConstraintsAffecting.Scale == 0)
	{
		Result = InGlobalTransform.GetRelativeTransform(InLocalOffsetTransform);
		
		if(NumConstraintsAffecting.Location == 0 &&
			NumConstraintsAffecting.Rotation == 0 &&
			NumConstraintsAffecting.Scale == 0)
		{
			Result.NormalizeRotation();
			return Result;
		}
	}

	if(NumConstraintsAffecting.Location == 1)
	{
		check(FirstConstraint.Location != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraint = InConstraints[FirstConstraint.Location];
		const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Location, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);

		check(Weight.AffectsLocation());
		Result.SetLocation(InGlobalTransform.GetRelativeTransform(Transform).GetLocation());
	}
	else if(NumConstraintsAffecting.Location == 2)
	{
		check(FirstConstraint.Location != INDEX_NONE);
		check(SecondConstraint.Location != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraintA = InConstraints[FirstConstraint.Location];
		const FRigElementParentConstraint& ParentConstraintB = InConstraints[SecondConstraint.Location];

		const FRigElementWeight& WeightA = ParentConstraintA.GetWeight(bInitial); 
		const FRigElementWeight& WeightB = ParentConstraintB.GetWeight(bInitial);
		check(WeightA.AffectsLocation());
		check(WeightB.AffectsLocation());
		const float Weight = GetWeightForLerp(WeightA.Location, WeightB.Location);

		const FTransform TransformA = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Location, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);
		const FTransform TransformB = LazilyComputeParentConstraint(InConstraints, SecondConstraint.Location, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);

		const FTransform MixedTransform = FControlRigMathLibrary::LerpTransform(TransformA, TransformB, Weight);
		Result.SetLocation(InGlobalTransform.GetRelativeTransform(MixedTransform).GetLocation());
	}
	else if(NumConstraintsAffecting.Location > 2)
	{
		check(TotalWeight.Location > SMALL_NUMBER);
		
		FVector Location = FVector::ZeroVector;
		int NumMixedRotations = 0;
		FQuat FirstRotation = FQuat::Identity;
		FQuat MixedRotation = FQuat(0.f, 0.f, 0.f, 0.f);
		FVector Scale = FVector::ZeroVector;

		for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
		{
			const FRigElementParentConstraint& ParentConstraint = InConstraints[ConstraintIndex];
			const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
			if(!Weight.AffectsLocation())
			{
				continue;
			}

			const FTransform Transform = LazilyComputeParentConstraint(InConstraints, ConstraintIndex, InTransformType,
				InLocalOffsetTransform, true, FTransform::Identity, false);

			const float NormalizedWeight = Weight.Location / TotalWeight.Location;
			IntegrateParentConstraintVector(Location, Transform, NormalizedWeight, true);
			IntegrateParentConstraintQuat(NumMixedRotations, FirstRotation, MixedRotation, Transform, NormalizedWeight);
			IntegrateParentConstraintVector(Scale, Transform, NormalizedWeight, false);
		}

		FTransform ParentTransform(MixedRotation.GetNormalized(), Location, Scale);
		Result.SetLocation(InGlobalTransform.GetRelativeTransform(ParentTransform).GetLocation());
	}

	if(NumConstraintsAffecting.Rotation == 1)
	{
		check(FirstConstraint.Rotation != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraint = InConstraints[FirstConstraint.Rotation];
		const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Rotation, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);
		check(Weight.AffectsRotation());
		Result.SetRotation(InGlobalTransform.GetRelativeTransform(Transform).GetRotation());
	}
	else if(NumConstraintsAffecting.Rotation == 2)
	{
		check(FirstConstraint.Rotation != INDEX_NONE);
		check(SecondConstraint.Rotation != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraintA = InConstraints[FirstConstraint.Rotation];
		const FRigElementParentConstraint& ParentConstraintB = InConstraints[SecondConstraint.Rotation];

		const FRigElementWeight& WeightA = ParentConstraintA.GetWeight(bInitial); 
		const FRigElementWeight& WeightB = ParentConstraintB.GetWeight(bInitial);
		check(WeightA.AffectsRotation());
		check(WeightB.AffectsRotation());
		const float Weight = GetWeightForLerp(WeightA.Rotation, WeightB.Rotation);

		const FTransform TransformA = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Rotation, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);
		const FTransform TransformB = LazilyComputeParentConstraint(InConstraints, SecondConstraint.Rotation, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);

		const FTransform MixedTransform = FControlRigMathLibrary::LerpTransform(TransformA, TransformB, Weight);
		Result.SetRotation(InGlobalTransform.GetRelativeTransform(MixedTransform).GetRotation());
	}
	else if(NumConstraintsAffecting.Rotation > 2)
	{
		check(TotalWeight.Rotation > SMALL_NUMBER);
		
		FVector Location = FVector::ZeroVector;
		int NumMixedRotations = 0;
		FQuat FirstRotation = FQuat::Identity;
		FQuat MixedRotation = FQuat(0.f, 0.f, 0.f, 0.f);
		FVector Scale = FVector::ZeroVector;

		for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
		{
			const FRigElementParentConstraint& ParentConstraint = InConstraints[ConstraintIndex];
			const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
			if(!Weight.AffectsRotation())
			{
				continue;
			}

			const FTransform Transform = LazilyComputeParentConstraint(InConstraints, ConstraintIndex, InTransformType,
				InLocalOffsetTransform, true, FTransform::Identity, false);

			const float NormalizedWeight = Weight.Rotation / TotalWeight.Rotation;
			IntegrateParentConstraintVector(Location, Transform, NormalizedWeight, true);
			IntegrateParentConstraintQuat(NumMixedRotations, FirstRotation, MixedRotation, Transform, NormalizedWeight);
			IntegrateParentConstraintVector(Scale, Transform, NormalizedWeight, false);
		}

		FTransform ParentTransform(MixedRotation.GetNormalized(), Location, Scale);
		Result.SetRotation(InGlobalTransform.GetRelativeTransform(ParentTransform).GetRotation());
	}

	if(NumConstraintsAffecting.Scale == 1)
	{
		check(FirstConstraint.Scale != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraint = InConstraints[FirstConstraint.Scale];
		const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
		
		const FTransform Transform = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Scale, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);

		check(Weight.AffectsScale());
		Result.SetScale3D(InGlobalTransform.GetRelativeTransform(Transform).GetScale3D());
	}
	else if(NumConstraintsAffecting.Scale == 2)
	{
		check(FirstConstraint.Scale != INDEX_NONE);
		check(SecondConstraint.Scale != INDEX_NONE);

		const FRigElementParentConstraint& ParentConstraintA = InConstraints[FirstConstraint.Scale];
		const FRigElementParentConstraint& ParentConstraintB = InConstraints[SecondConstraint.Scale];

		const FRigElementWeight& WeightA = ParentConstraintA.GetWeight(bInitial); 
		const FRigElementWeight& WeightB = ParentConstraintB.GetWeight(bInitial);
		check(WeightA.AffectsScale());
		check(WeightB.AffectsScale());
		const float Weight = GetWeightForLerp(WeightA.Scale, WeightB.Scale);

		const FTransform TransformA = LazilyComputeParentConstraint(InConstraints, FirstConstraint.Scale, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);
		const FTransform TransformB = LazilyComputeParentConstraint(InConstraints, SecondConstraint.Scale, InTransformType,
			InLocalOffsetTransform, true, FTransform::Identity, false);

		const FTransform MixedTransform = FControlRigMathLibrary::LerpTransform(TransformA, TransformB, Weight);
		Result.SetScale3D(InGlobalTransform.GetRelativeTransform(MixedTransform).GetScale3D());
	}
	else if(NumConstraintsAffecting.Scale > 2)
	{
		check(TotalWeight.Scale > SMALL_NUMBER);
		
		FVector Location = FVector::ZeroVector;
		int NumMixedRotations = 0;
		FQuat FirstRotation = FQuat::Identity;
		FQuat MixedRotation = FQuat(0.f, 0.f, 0.f, 0.f);
		FVector Scale = FVector::ZeroVector;
		
		for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
		{
			const FRigElementParentConstraint& ParentConstraint = InConstraints[ConstraintIndex];
			const FRigElementWeight& Weight = ParentConstraint.GetWeight(bInitial);
			if(!Weight.AffectsScale())
			{
				continue;
			}

			const FTransform Transform = LazilyComputeParentConstraint(InConstraints, ConstraintIndex, InTransformType,
				InLocalOffsetTransform, true, FTransform::Identity, false);

			const float NormalizedWeight = Weight.Scale / TotalWeight.Scale;
			IntegrateParentConstraintVector(Location, Transform, NormalizedWeight, true);
			IntegrateParentConstraintQuat(NumMixedRotations, FirstRotation, MixedRotation, Transform, NormalizedWeight);
			IntegrateParentConstraintVector(Scale, Transform, NormalizedWeight, false);
		}

		FTransform ParentTransform(MixedRotation.GetNormalized(), Location, Scale);
		Result.SetScale3D(InGlobalTransform.GetRelativeTransform(ParentTransform).GetScale3D());
	}

	Result.NormalizeRotation();
	return Result;
}

FTransform URigHierarchy::LazilyComputeParentConstraint(
	const FRigElementParentConstraintArray& InConstraints,
	int32 InIndex,
	const ERigTransformType::Type InTransformType,
	const FTransform& InLocalOffsetTransform,
	bool bApplyLocalOffsetTransform,
	const FTransform& InLocalPoseTransform,
	bool bApplyLocalPoseTransform) const
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	const FRigElementParentConstraint& Constraint = InConstraints[InIndex];
	if(Constraint.bCacheIsDirty)
	{
		FTransform Transform = GetTransform(Constraint.ParentElement, InTransformType);
		if(bApplyLocalOffsetTransform)
		{
			Transform = InLocalOffsetTransform * Transform;
		}
		if(bApplyLocalPoseTransform)
		{
			Transform = InLocalPoseTransform * Transform;
		}

		Transform.NormalizeRotation();
		Constraint.Cache = Transform;
		Constraint.bCacheIsDirty = false;
	}
	return Constraint.Cache;
}

void URigHierarchy::ComputeParentConstraintIndices(
	const FRigElementParentConstraintArray& InConstraints,
	ERigTransformType::Type InTransformType,
	FConstraintIndex& OutFirstConstraint,
	FConstraintIndex& OutSecondConstraint,
	FConstraintIndex& OutNumConstraintsAffecting,
	FRigElementWeight& OutTotalWeight)
{
	const bool bInitial = IsInitial(InTransformType);
	
	// find all of the weights affecting this output
	for(int32 ConstraintIndex = 0; ConstraintIndex < InConstraints.Num(); ConstraintIndex++)
	{
		// this is not relying on the cache whatsoever. we might as well remove it.
		InConstraints[ConstraintIndex].bCacheIsDirty = true;
		
		const FRigElementWeight& Weight = InConstraints[ConstraintIndex].GetWeight(bInitial);
		if(Weight.AffectsLocation())
		{
			OutNumConstraintsAffecting.Location++;
			OutTotalWeight.Location += Weight.Location;

			if(OutFirstConstraint.Location == INDEX_NONE)
			{
				OutFirstConstraint.Location = ConstraintIndex;
			}
			else if(OutSecondConstraint.Location == INDEX_NONE)
			{
				OutSecondConstraint.Location = ConstraintIndex;
			}
		}
		if(Weight.AffectsRotation())
		{
			OutNumConstraintsAffecting.Rotation++;
			OutTotalWeight.Rotation += Weight.Rotation;

			if(OutFirstConstraint.Rotation == INDEX_NONE)
			{
				OutFirstConstraint.Rotation = ConstraintIndex;
			}
			else if(OutSecondConstraint.Rotation == INDEX_NONE)
			{
				OutSecondConstraint.Rotation = ConstraintIndex;
			}
		}
		if(Weight.AffectsScale())
		{
			OutNumConstraintsAffecting.Scale++;
			OutTotalWeight.Scale += Weight.Scale;

			if(OutFirstConstraint.Scale == INDEX_NONE)
			{
				OutFirstConstraint.Scale = ConstraintIndex;
			}
			else if(OutSecondConstraint.Scale == INDEX_NONE)
			{
				OutSecondConstraint.Scale = ConstraintIndex;
			}
		}
	}
}
void URigHierarchy::IntegrateParentConstraintVector(
	FVector& OutVector,
	const FTransform& InTransform,
	float InWeight,
	bool bIsLocation)
{
	if(bIsLocation)
	{
		OutVector += InTransform.GetLocation() * InWeight;
	}
	else
	{
		OutVector += InTransform.GetScale3D() * InWeight;
	}
}

void URigHierarchy::IntegrateParentConstraintQuat(
	int32& OutNumMixedRotations,
	FQuat& OutFirstRotation,
	FQuat& OutMixedRotation,
	const FTransform& InTransform,
	float InWeight)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/ControlRig"));
	FQuat ParentRotation = InTransform.GetRotation().GetNormalized();

	if(OutNumMixedRotations == 0)
	{
		OutFirstRotation = ParentRotation; 
	}
	else if ((ParentRotation | OutFirstRotation) <= 0.f)
	{
		InWeight = -InWeight;
	}

	OutMixedRotation.X += InWeight * ParentRotation.X;
	OutMixedRotation.Y += InWeight * ParentRotation.Y;
	OutMixedRotation.Z += InWeight * ParentRotation.Z;
	OutMixedRotation.W += InWeight * ParentRotation.W;
	OutNumMixedRotations++;
}

#if WITH_EDITOR
TArray<FString> URigHierarchy::ControlSettingsToPythonCommands(const FRigControlSettings& Settings, const FString& NameSettings)
{
	TArray<FString> Commands;
	Commands.Add(FString::Printf(TEXT("%s = unreal.RigControlSettings()"),
			*NameSettings));

	ERigControlType ControlType = Settings.ControlType;
	switch(ControlType)
	{
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		{
			ControlType = ERigControlType::EulerTransform;
			break;
		}
		default:
		{
			break;
		}
	}

	const FString AnimationTypeStr = RigVMPythonUtils::EnumValueToPythonString<ERigControlAnimationType>((int64)Settings.AnimationType);
	const FString ControlTypeStr = RigVMPythonUtils::EnumValueToPythonString<ERigControlType>((int64)ControlType);

	static const TCHAR* TrueText = TEXT("True");
	static const TCHAR* FalseText = TEXT("False");

	TArray<FString> LimitEnabledParts;
	for(const FRigControlLimitEnabled& LimitEnabled : Settings.LimitEnabled)
	{
		LimitEnabledParts.Add(FString::Printf(TEXT("unreal.RigControlLimitEnabled(%s, %s)"),
						   LimitEnabled.bMinimum ? TrueText : FalseText,
						   LimitEnabled.bMaximum ? TrueText : FalseText));
	}
	
	const FString LimitEnabledStr = FString::Join(LimitEnabledParts, TEXT(", "));
	
	Commands.Add(FString::Printf(TEXT("%s.animation_type = %s"),
									*NameSettings,
									*AnimationTypeStr));
	Commands.Add(FString::Printf(TEXT("%s.control_type = %s"),
									*NameSettings,
									*ControlTypeStr));
	Commands.Add(FString::Printf(TEXT("%s.display_name = '%s'"),
		*NameSettings,
		*Settings.DisplayName.ToString()));
	Commands.Add(FString::Printf(TEXT("%s.draw_limits = %s"),
		*NameSettings,
		Settings.bDrawLimits ? TrueText : FalseText));
	Commands.Add(FString::Printf(TEXT("%s.shape_color = %s"),
		*NameSettings,
		*RigVMPythonUtils::LinearColorToPythonString(Settings.ShapeColor)));
	Commands.Add(FString::Printf(TEXT("%s.shape_name = '%s'"),
		*NameSettings,
		*Settings.ShapeName.ToString()));
	Commands.Add(FString::Printf(TEXT("%s.shape_visible = %s"),
		*NameSettings,
		Settings.bShapeVisible ? TrueText : FalseText));
	Commands.Add(FString::Printf(TEXT("%s.is_transient_control = %s"),
		*NameSettings,
		Settings.bIsTransientControl ? TrueText : FalseText));
	Commands.Add(FString::Printf(TEXT("%s.limit_enabled = [%s]"),
		*NameSettings,
		*LimitEnabledStr));
	Commands.Add(FString::Printf(TEXT("%s.minimum_value = %s"),
		*NameSettings,
		*Settings.MinimumValue.ToPythonString(Settings.ControlType)));
	Commands.Add(FString::Printf(TEXT("%s.maximum_value = %s"),
		*NameSettings,
		*Settings.MaximumValue.ToPythonString(Settings.ControlType)));
	Commands.Add(FString::Printf(TEXT("%s.primary_axis = %s"),
		*NameSettings,
		*RigVMPythonUtils::EnumValueToPythonString<ERigControlAxis>((int64)Settings.PrimaryAxis)));

	return Commands;
}

TArray<FString> URigHierarchy::ConnectorSettingsToPythonCommands(const FRigConnectorSettings& Settings, const FString& NameSettings)
{
	TArray<FString> Commands;
	Commands.Add(FString::Printf(TEXT("%s = unreal.RigConnectorSettings()"),
			*NameSettings));

	// no content values just yet - we are skipping the ResolvedItem here since
	// we don't assume it is going to be resolved initially.

	return Commands;
}

#endif

FRigElementKey URigHierarchy::PatchElementKeyInLookup(const FRigElementKey& InKey, const TMap<FRigHierarchyModulePath, FName>* InModulePathToName) const
{
	if(!InKey.IsValid())
	{
		return InKey;
	}
	if(const int32* Index = ElementIndexLookup.Find(InKey))
	{
		return Get(*Index)->GetKey();
	}
	
	const FString NameString = InKey.Name.ToString();
	if(!NameString.Contains(FRigHierarchyModulePath::NamespaceSeparator_Deprecated))
	{
		return InKey;
	}

	const TMap<FRigHierarchyModulePath, FName>* ModulePathToName = InModulePathToName;
	if(ModulePathToName == nullptr)
	{
		const UModularRig* ModularRig = Cast<UModularRig>(GetOuter());
		if(ModularRig == nullptr)
		{
			return InKey;
		}
		ModulePathToName = &ModularRig->GetModularRigModel().PreviousModulePaths;
	}
	
	const FRigElementKey PatchedKey = InKey.ConvertToModuleNameFormat(ModulePathToName);
	if(PatchedKey == InKey)
	{
		return InKey;
	}

	if(const int32* PatchedIndex = ElementIndexLookup.Find(PatchedKey))
	{
		const int32 Index = *PatchedIndex;
		const_cast<URigHierarchy*>(this)->ElementIndexLookup.Add(InKey, Index);
	}

	return PatchedKey;
}

void URigHierarchy::PatchElementMetadata(const TMap<FRigHierarchyModulePath, FName>& InModulePathToName)
{
	bool bPerformedChange = false;
	for(int32 Index = 0; Index < ElementMetadata.Num(); Index++)
	{
		FMetadataStorage& MetadataStorage = ElementMetadata[Index];
		if(MetadataStorage.MetadataMap.IsEmpty())
		{
			continue;
		}

		bool bContainsNamespace = false;
		for(const TPair<FName, FRigBaseMetadata*>& Pair : MetadataStorage.MetadataMap)
		{
			if(FRigHierarchyModulePath(Pair.Key).UsesNameSpaceFormat())
			{
				bContainsNamespace = true;
				break;
			}
		}
		
		if(bContainsNamespace)
		{
			TMap<FName, FRigBaseMetadata*> OldMetadataMap;
			Swap(OldMetadataMap, MetadataStorage.MetadataMap);

			for(const TPair<FName, FRigBaseMetadata*>& Pair : OldMetadataMap)
			{
				FRigHierarchyModulePath ModulePath(Pair.Key);
				if(ModulePath.ConvertToModuleNameFormatInline(&InModulePathToName))
				{
					MetadataStorage.MetadataMap.Add(*ModulePath.GetPath(), Pair.Value);
					bPerformedChange = true;
					continue;
				}
				MetadataStorage.MetadataMap.Add(Pair.Key, Pair.Value);
			}
		}

		for(const TPair<FName, FRigBaseMetadata*>& Pair : MetadataStorage.MetadataMap)
		{
			if(Pair.Key == ModuleMetadataName && Pair.Value->GetType() == ERigMetadataType::Name)
			{
				FRigNameMetadata* NameMetadata = static_cast<FRigNameMetadata*>(Pair.Value);
				const FName OldValueName = NameMetadata->GetValue();
				if(const FName* ModuleName = InModulePathToName.Find((OldValueName.ToString())))
				{
					NameMetadata->SetValue(*ModuleName);
					bPerformedChange = true;
				}
			}
			else if(Pair.Value->GetType() == ERigMetadataType::RigElementKey)
			{
				FRigElementKeyMetadata* KeyMetadata = static_cast<FRigElementKeyMetadata*>(Pair.Value);
				FRigElementKey ValueKey = KeyMetadata->GetValue();
				if(ValueKey.ConvertToModuleNameFormatInline(&InModulePathToName))
				{
					KeyMetadata->SetValue(ValueKey);
					bPerformedChange = true;
				}
			}
			else if(Pair.Value->GetType() == ERigMetadataType::RigElementKeyArray)
			{
				FRigElementKeyArrayMetadata* KeyArrayMetadata = static_cast<FRigElementKeyArrayMetadata*>(Pair.Value);
				TArray<FRigElementKey> ValueKeys = KeyArrayMetadata->GetValue();
				for(FRigElementKey& ValueKey : ValueKeys)
				{
					if(ValueKey.ConvertToModuleNameFormatInline(&InModulePathToName))
					{
						bPerformedChange = true;
					}
				}
				KeyArrayMetadata->SetValue(ValueKeys);
			}
		}
	}

	if(bPerformedChange)
	{
		MetadataVersion++;
	}
}

void URigHierarchy::PatchModularRigComponentKeys(const TMap<FRigHierarchyModulePath, FName>& InModulePathToName)
{
	TMap<FRigHierarchyKey,FRigHierarchyKey> PatchedKeys;
	for(FRigBaseElement* Element : Elements)
	{
		FRigElementKey PatchedKey = Element->GetKey();
		if(PatchedKey.ConvertToModuleNameFormatInline(&InModulePathToName))
		{
			PatchedKeys.Add(Element->GetKey(), PatchedKey);
			continue;
		}

		// resolve the element key in reverse
		const FRigHierarchyModulePath ModulePath = Element->GetName();
		if(ModulePath.UsesModuleNameFormat())
		{
			FString ModuleName,ElementName;
			if(ModulePath.Split(&ModuleName, &ElementName))
			{
				if(const FRigHierarchyModulePath* OldModulePath = InModulePathToName.FindKey(*ModuleName))
				{
					const FString PathBasedName = JoinNameSpace_Deprecated(OldModulePath->GetPath(), ElementName);
					PatchedKeys.Add(FRigElementKey(*PathBasedName, Element->GetType()), Element->GetKey());
				}
			}
		}
	}

	for(FInstancedStruct& InstancedStruct : ElementComponents)
	{
		if(!InstancedStruct.IsValid())
		{
			continue;
		}

		FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
		const FRigElementKey PatchedKey = PatchElementKeyInLookup(Component->GetElementKey(), &InModulePathToName);
		if(PatchedKey != Component->GetElementKey())
		{
			Modify();
			
			const FRigComponentKey OldKey = Component->Key;
			Component->Key.ElementKey = PatchedKey;
			const FRigComponentKey& NewKey = Component->Key;
			PatchedKeys.Add(OldKey, NewKey);
			ComponentIndexLookup.Remove(OldKey);
			ComponentIndexLookup.Add(NewKey, Component->IndexInHierarchy);
		}
	}

	for(FInstancedStruct& InstancedStruct : ElementComponents)
	{
		if(!InstancedStruct.IsValid())
		{
			continue;
		}

		FRigBaseComponent* Component = InstancedStruct.GetMutablePtr<FRigBaseComponent>();
		for(const TPair<FRigHierarchyKey,FRigHierarchyKey>& Pair : PatchedKeys)
		{
			Component->OnRigHierarchyKeyChanged(Pair.Key, Pair.Value);
		}
	}
}

void URigHierarchy::PatchZeroWeightAnimationChannels()
{
	ForEach([](FRigBaseElement* Element)
	{
		if (FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			if (ControlElement->IsAnimationChannel() && !ControlElement->ParentConstraints.IsEmpty())
			{
				ControlElement->ParentConstraints[0].InitialWeight = 1.f;
				ControlElement->ParentConstraints[0].Weight = 1.f;
			}
		}
		return true;
	});
}

void URigHierarchy::SetControlPreferredEulerAngles(FRigControlElement* InControlElement, const FTransform& InTransform, bool bIsInitial)
{
	if (InControlElement)
	{
		const FEulerTransform EulerTransform(InTransform);
		const FVector EulerXYZAngle(EulerTransform.Rotation.Roll, EulerTransform.Rotation.Pitch, EulerTransform.Rotation.Yaw);

		// switch from XYZ to the actual rotation order as FTransform doesn't have any notion of order
		const EEulerRotationOrder RotationOrder = InControlElement->Settings.PreferredRotationOrder;
		const FVector EulerAngle = GetUsePreferredRotationOrder(InControlElement) ?
			AnimationCore::ChangeEulerRotationOrder(EulerXYZAngle, EEulerRotationOrder::XYZ, RotationOrder) : EulerXYZAngle;
			
		switch(InControlElement->Settings.ControlType)
		{
		case ERigControlType::Transform:
		case ERigControlType::Rotator:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
			SetControlSpecifiedEulerAngle(InControlElement, EulerAngle, bIsInitial);
			break;
		default:
			break;
		}
	}
}

FRigHierarchyRedirectorGuard::FRigHierarchyRedirectorGuard(UControlRig* InControlRig)
: Guard(InControlRig->GetHierarchy()->ElementKeyRedirector, &InControlRig->GetElementKeyRedirector())
{
}

FRigHierarchyMemoryWriter::FRigHierarchyMemoryWriter(TArray<uint8>& InOutBuffer, TArray<FName>& InOutNames, bool bIsPersistent)
: FMemoryWriter(InOutBuffer, bIsPersistent)
, Names(InOutNames)
{
}

FArchive& FRigHierarchyMemoryWriter::operator<<(FName& Value)
{
	if(const int32* ExistingIndexPtr = NameToIndex.Find(Value))
	{
		int32 ExistingIndex = *ExistingIndexPtr;
		*this << ExistingIndex;
	}
	else
	{
		int32 NewIndex = Names.Add(Value);
		NameToIndex.Add(Value, NewIndex);
		*this << NewIndex;
	}
	return *this;
}

FArchive& FRigHierarchyMemoryWriter::operator<<(FText& Value)
{
	FString ValueString = Value.ToString();
	*this << ValueString;
	return *this;
}

FRigHierarchyMemoryReader::FRigHierarchyMemoryReader(TArray<uint8>& InOutBuffer, TArray<FName>& InOutNames, bool bIsPersistent)
: FMemoryReader(InOutBuffer, bIsPersistent)
, Names(InOutNames)
{
}

FArchive& FRigHierarchyMemoryReader::operator<<(FName& Value)
{
	int32 NameIndex = INDEX_NONE;
	*this << NameIndex;
	check(Names.IsValidIndex(NameIndex));
	Value = Names[NameIndex];
	return *this;
}

FArchive& FRigHierarchyMemoryReader::operator<<(FText& Value)
{
	FString ValueString;
	*this << ValueString;
	Value = FText::FromString(ValueString);
	return *this;
}
