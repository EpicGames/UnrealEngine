// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyCache.h"
#include "Rigs/RigHierarchy.h"
#include "ModularRigModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyCache)

////////////////////////////////////////////////////////////////////////////////
// FCachedRigElement
////////////////////////////////////////////////////////////////////////////////

const FRigElementKey& FCachedRigElement::GetResolvedKey() const
{
	if(Element)
	{
		return Element->GetKey();
	}
	static FRigElementKey InvalidKey;
	return InvalidKey;
}

bool FCachedRigElement::UpdateCache(const URigHierarchy* InHierarchy)
{
	if(InHierarchy)
	{
		if(!IsValid() || InHierarchy->GetTopologyVersionHash() != ContainerVersion || Element != InHierarchy->Get(Index))
		{
			return UpdateCache(GetKey(), InHierarchy);
		}
		return IsValid();
	}
	return false;
}

bool FCachedRigElement::UpdateCache(const FRigElementKey& InKey, const URigHierarchy* InHierarchy)
{
	if(InHierarchy)
	{
		if(!IsValid() || !IsIdentical(InKey, InHierarchy) || Element != InHierarchy->Get(Index))
		{
			// have to create a copy since Reset below
			// potentially resets the InKey as well.
			const FRigElementKey KeyToResolve = InKey;

			// first try to resolve with the known index.
			// this happens a lot - where the topology version has
			// increased - but the known item is still valid
			if(InHierarchy->IsValidIndex(Index))
			{
				if(const FRigBaseElement* PreviousElement = InHierarchy->Get(Index))
				{
					if(PreviousElement->GetKey() == KeyToResolve)
					{
						Key = KeyToResolve;
						Element = PreviousElement;
						ContainerVersion = InHierarchy->GetTopologyVersionHash();
						return IsValid();
					}
				}
			}

			int32 Idx = InHierarchy->GetIndex(KeyToResolve);
			if(Idx != INDEX_NONE)
			{
				Key = KeyToResolve;
				Index = (uint16)Idx;
				Element = InHierarchy->Get(Index);
			}
			else
			{
				Invalidate();
				Key = KeyToResolve;
			}

			ContainerVersion = InHierarchy->GetTopologyVersionHash();
		}
		return IsValid();
	}
	return false;
}

bool FCachedRigElement::IsIdentical(const FRigElementKey& InKey, const URigHierarchy* InHierarchy)
{
	return InKey == Key && InHierarchy->GetTopologyVersionHash() == ContainerVersion;
}

void FCachedRigElement::Set(const FRigBaseElement* InElement, int32 InTopologyHashVersion)
{
	check(InElement);
	Element = InElement;
	Key = Element->GetKey();
	Index = Element->GetIndex();
	if(InTopologyHashVersion == INDEX_NONE)
	{
		if(const URigHierarchy* Hierarchy = Element->GetOwner())
		{
			ContainerVersion = Hierarchy->GetTopologyVersionHash();
		}
	}
	else
	{
		ContainerVersion = InTopologyHashVersion;
	}
}

////////////////////////////////////////////////////////////////////////////////
// FCachedRigComponent
////////////////////////////////////////////////////////////////////////////////

const FRigElementKey& FCachedRigComponent::GetResolvedElementKey() const
{
	if(Component)
	{
		if(const FRigBaseElement* Element = Component->GetElement())
		{
			return Element->GetKey();
		}
	}
	static FRigElementKey InvalidKey;
	return InvalidKey;
}

bool FCachedRigComponent::UpdateCache(const URigHierarchy* InHierarchy)
{
	if(!IsValid() || InHierarchy->GetTopologyVersionHash() != CachedElement.ContainerVersion || Component != InHierarchy->GetComponent(Index))
	{
		return UpdateCache(GetElementKey(), GetName(), InHierarchy);
	}
	return IsValid();
}

bool FCachedRigComponent::UpdateCache(const FRigElementKey& InKey, const FName& InName, const URigHierarchy* InHierarchy)
{
	if(InHierarchy)
	{
		if(!IsValid() || !IsIdentical(InKey, InName, InHierarchy) || Component != InHierarchy->GetComponent(Index))
		{
			if(CachedElement.UpdateCache(InKey, InHierarchy))
			{
				Component = InHierarchy->FindComponent({InKey, InName});
				if(Component)
				{
					Index = Component->GetIndexInHierarchy();
				}
				else
				{
					Invalidate();
				}
			}
		}
		return IsValid();
	}
	return false;
}

bool FCachedRigComponent::UpdateCache(const FRigComponentKey& InKey, const URigHierarchy* InHierarchy)
{
	return UpdateCache(InKey.ElementKey, InKey.Name, InHierarchy);
}

bool FCachedRigComponent::IsIdentical(const FRigElementKey& InKey, const FName& InName, const URigHierarchy* InHierarchy)
{
	return InName == Name && CachedElement.IsIdentical(InKey, InHierarchy);
}

////////////////////////////////////////////////////////////////////////////////
// FRigElementKeyRedirector
////////////////////////////////////////////////////////////////////////////////

FRigElementKeyRedirector::FRigElementKeyRedirector(const FKeyMap& InMap, const URigHierarchy* InHierarchy)
{
	check(InHierarchy);
	InternalKeyToExternalKey.Reserve(InMap.Num());
	ExternalKeys.Reserve(InMap.Num());

	Hash = 0;
	for(const FKeyPair& Pair : InMap)
	{
		check(Pair.Key.IsValid());
		Add(Pair.Key, Pair.Value, InHierarchy);
	}
}

FRigElementKeyRedirector::FRigElementKeyRedirector(const TMap<FRigElementKey, FRigElementKeyCollection>& InMap, const URigHierarchy* InHierarchy)
{
	check(InHierarchy);
	InternalKeyToExternalKey.Reserve(InMap.Num());
	ExternalKeys.Reserve(InMap.Num());

	Hash = 0;
	for(const TPair<FRigElementKey, FRigElementKeyCollection>& Pair : InMap)
	{
		check(Pair.Key.IsValid());
		Add(Pair.Key, Convert(Pair.Value.Keys), InHierarchy);
	}
}

FRigElementKeyRedirector::FRigElementKeyRedirector(const FRigElementKeyRedirector& InOther, const URigHierarchy* InHierarchy)
{
	check(InHierarchy);
	InternalKeyToExternalKey.Reserve(InOther.InternalKeyToExternalKey.Num());
	ExternalKeys.Reserve(InOther.ExternalKeys.Num());

	Hash = 0;
	for(const FCachedKeyPair& Pair : InOther.InternalKeyToExternalKey)
	{
		check(Pair.Key.IsValid());
		Add(Pair.Key, Convert(Pair.Value), InHierarchy);
	}
}

FRigElementKeyRedirector::FRigElementKeyRedirector(const FModularRigConnections& InOther, const URigHierarchy* InHierarchy)
{
	check(InHierarchy);
	InternalKeyToExternalKey.Reserve(InOther.Num());
	ExternalKeys.Reserve(InOther.Num());

	Hash = 0;
	for(const FModularRigSingleConnection& Connection : InOther)
	{
		check(Connection.Connector.IsValid());
		check(IsValid(Connection.Targets));
		Add(Connection.Connector, Convert(Connection.Targets), InHierarchy);
	}
}

const FRigElementKey* FRigElementKeyRedirector::FindReverse(const FRigElementKey& InKey) const
{
	for(const FCachedKeyPair& Pair : InternalKeyToExternalKey)
	{
		for(const FCachedRigElement& Target : Pair.Value)
		{
			if(Target.GetKey() == InKey)
			{
				return &Pair.Key;
			}
		}
	}
	return nullptr;
}

void FRigElementKeyRedirector::Add(const FRigElementKey& InSource, const FKeyArray& InTargets, const URigHierarchy* InHierarchy)
{
	if(!InSource.IsValid() || InTargets.IsEmpty() || InTargets.Contains(InSource))
	{
		return;
	}

	if(!InTargets.IsEmpty())
	{
		const FCachedKeyArray Cache = Convert(InTargets, InHierarchy, true, true);
		if(!Cache.IsEmpty())
		{
			InternalKeyToExternalKey.Add(InSource, Cache);
		}
	}
	
	ExternalKeys.Add(InSource, InTargets);
	Hash = HashCombine(Hash, HashCombine(GetTypeHash(InSource), GetTypeHash(InTargets)));
}

FRigElementKeyRedirector::FKeyArray FRigElementKeyRedirector::Convert(const FCachedKeyArray& InCachedKeys)
{
	FKeyArray Result;
	Result.Reserve(InCachedKeys.Num());
	for(int32 Index = 0; Index < InCachedKeys.Num(); Index++)
	{
		Result.Add(InCachedKeys[Index].GetKey());
	}
	return Result;
}

FRigElementKeyRedirector::FKeyArray FRigElementKeyRedirector::Convert(const TArray<FRigElementKey>& InKeys)
{
	FKeyArray Result;
	Result.Append(InKeys);
	return Result;
}

FRigElementKeyRedirector::FCachedKeyArray FRigElementKeyRedirector::Convert(const FKeyArray& InKeys, const URigHierarchy* InHierarchy,
	bool bForceStoreKey, bool bOnlyValidItems)
{
	FCachedKeyArray Result;
	Result.Reserve(InKeys.Num());
	for(const FRigElementKey& Key : InKeys)
	{
		if(!bOnlyValidItems || Key.IsValid())
		{
			Result.Emplace(Key, InHierarchy, bForceStoreKey);
		}
	}
	return Result;
}