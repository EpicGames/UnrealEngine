// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyComponents.h"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyElements.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/AnimObjectVersion.h"
#include "ControlRigObjectVersion.h"
#include "HAL/UnrealMemory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyComponents)

////////////////////////////////////////////////////////////////////////////////
// FRigComponentState
////////////////////////////////////////////////////////////////////////////////

bool FRigComponentState::IsValid() const
{
	return ComponentStruct != nullptr && !Data.IsEmpty();
}

const UScriptStruct* FRigComponentState::GetComponentStruct() const
{
	return ComponentStruct;
}

bool FRigComponentState::operator==(const FRigComponentState& InOther) const
{
	if(IsValid() != InOther.IsValid())
	{
		return false;
	}

	if(IsValid())
	{
		if(GetComponentStruct() != InOther.GetComponentStruct())
		{
			return false;
		}

		if(Data.Num() != InOther.Data.Num())
		{
			return false;
		}

		if(FMemory::Memcmp(Data.GetData(), InOther.Data.GetData(), Data.Num()) != 0)
		{
			return false;
		}
	}
	
	return true;
}

////////////////////////////////////////////////////////////////////////////////
// FRigBaseComponent
////////////////////////////////////////////////////////////////////////////////

FRigBaseComponent::~FRigBaseComponent()
{
}

FName FRigBaseComponent::GetDefaultComponentName() const
{
#if WITH_EDITORONLY_DATA
	return *GetScriptStruct()->GetDisplayNameText().ToString();
#else
	return TEXT("Component");
#endif
}

const FSlateIcon& FRigBaseComponent::GetIconForUI() const
{
	// todo
	static const FSlateIcon ComponentIcon = FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.Tree.RigidBody"));
	return ComponentIcon;
}

FSlateColor FRigBaseComponent::GetColorForUI() const
{
	return FSlateColor::UseForeground();
}

TArray<UScriptStruct*> FRigBaseComponent::GetAllComponentScriptStructs(bool bSorted)
{
	TArray<UScriptStruct*> RigComponentStructs;
	for (TObjectIterator<UScriptStruct> ScriptStructIt; ScriptStructIt; ++ScriptStructIt)
	{
		UScriptStruct* ScriptStruct = *ScriptStructIt;
		if(ScriptStruct != StaticStruct() && ScriptStruct->IsChildOf(StaticStruct()))
		{
			if(ScriptStruct->GetStructCPPName().Contains(TEXT("Base")))
			{
				continue;
			}
			RigComponentStructs.Add(ScriptStruct);
		}
	}
	if(bSorted)
	{
		Algo::Sort(RigComponentStructs, [](const UScriptStruct* A, const UScriptStruct* B) -> bool
		{
			return A->GetFName().LexicalLess(B->GetFName());
		});
	}
	return RigComponentStructs;
}

bool FRigBaseComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	if(Ar.IsLoading())
	{
		Load(Ar);
	}
	else if(Ar.IsSaving())
	{
		Save(Ar);
	}
	return true;
}

void FRigBaseComponent::Save(FArchive& Ar)
{
	Ar << Key;
}

void FRigBaseComponent::Load(FArchive& Ar)
{
	Ar << Key;
	CachedNameString.Reset();
}

FRigComponentState FRigBaseComponent::GetState() const
{
	FRigComponentState State;
	State.ComponentStruct = GetScriptStruct();
	FMemoryWriter Writer(State.Data);
	const_cast<FRigBaseComponent*>(this)->Save(Writer);
	State.Versions = Writer.GetCustomVersions();
	return State;
}

bool FRigBaseComponent::SetState(const FRigComponentState& InState)
{
	if(InState.IsValid() && InState.GetComponentStruct() == GetScriptStruct())
	{
		FMemoryReader Reader(InState.Data);
		Reader.SetCustomVersions(InState.Versions);
		Load(Reader);
		return true;
	}
	return false;
}

FString FRigBaseComponent::GetContentAsText() const
{
	FString Content;
	if(const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		ScriptStruct->ExportText(Content, this, nullptr, nullptr, PPF_ExternalEditor, nullptr);
	}
	return Content;
}
