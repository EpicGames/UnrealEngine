// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyContainer.h"
#include "ControlRig.h"
#include "HelperUtil.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyContainer)

////////////////////////////////////////////////////////////////////////////////
// FRigHierarchyContainer
////////////////////////////////////////////////////////////////////////////////

FRigHierarchyContainer::FRigHierarchyContainer()
{
}

FRigHierarchyContainer::FRigHierarchyContainer(const FRigHierarchyContainer& InOther)
{
	BoneHierarchy = InOther.BoneHierarchy;
	SpaceHierarchy = InOther.SpaceHierarchy;
	ControlHierarchy = InOther.ControlHierarchy;
	CurveContainer = InOther.CurveContainer;
}

FRigHierarchyContainer& FRigHierarchyContainer::operator= (const FRigHierarchyContainer &InOther)
{
	BoneHierarchy = InOther.BoneHierarchy;
	SpaceHierarchy = InOther.SpaceHierarchy;
	ControlHierarchy = InOther.ControlHierarchy;
	CurveContainer = InOther.CurveContainer;

	return *this;
}

class FRigHierarchyContainerImportErrorContext : public FOutputDevice
{
public:

	int32 NumErrors;

	FRigHierarchyContainerImportErrorContext()
        : FOutputDevice()
        , NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		UE_LOGF(LogControlRig, Error, "Error Importing To Hierarchy: %ls", V);
		NumErrors++;
	}
};

TArray<FRigElementKey> FRigHierarchyContainer::ImportFromText(const FRigHierarchyCopyPasteContent& InData)
{
	TArray<FRigElementKey> PastedKeys;

	if (InData.Contents.Num() == 0 ||
		(InData.Types.Num() != InData.Contents.Num()) ||
		(InData.LocalTransforms.Num() != InData.Contents.Num()) ||
		(InData.GlobalTransforms.Num() != InData.Contents.Num()))
	{
		return PastedKeys;
	}

	TMap<FRigElementKey, FRigElementKey> ElementMap;
	for (const FRigBone& Element : BoneHierarchy)
	{
		ElementMap.Add(Element.GetElementKey(), Element.GetElementKey());
	}
	for (const FRigControl& Element : ControlHierarchy)
	{
		ElementMap.Add(Element.GetElementKey(), Element.GetElementKey());
	}
	for (const FRigSpace& Element : SpaceHierarchy)
	{
		ElementMap.Add(Element.GetElementKey(), Element.GetElementKey());
	}
	for (const FRigCurve& Element : CurveContainer)
	{
		ElementMap.Add(Element.GetElementKey(), Element.GetElementKey());
	}

	FRigHierarchyContainerImportErrorContext ErrorPipe;
	TArray<TSharedPtr<FRigElement>> Elements;
	for (int32 Index = 0; Index < InData.Types.Num(); Index++)
	{
		ErrorPipe.NumErrors = 0;
		
		TSharedPtr<FRigElement> NewElement;
		switch (InData.Types[Index])
		{
			case ERigElementType::Bone:
			{
				NewElement = MakeShared<FRigBone>();
				FRigBone::StaticStruct()->ImportText(*InData.Contents[Index], NewElement.Get(), nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigBone::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Control:
			{
				NewElement = MakeShared<FRigControl>();
				FRigControl::StaticStruct()->ImportText(*InData.Contents[Index], NewElement.Get(), nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigControl::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Space:
			{
				NewElement = MakeShared<FRigSpace>();
				FRigSpace::StaticStruct()->ImportText(*InData.Contents[Index], NewElement.Get(), nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigSpace::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Curve:
			{
				NewElement = MakeShared<FRigCurve>();
				FRigCurve::StaticStruct()->ImportText(*InData.Contents[Index], NewElement.Get(), nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigCurve::StaticStruct()->GetName(), true);
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		if (ErrorPipe.NumErrors > 0)
		{
			return PastedKeys;
		}

		Elements.Add(NewElement);
	}

	for (int32 Index = 0; Index < InData.Types.Num(); Index++)
	{
		switch (InData.Types[Index])
		{
			case ERigElementType::Bone:
			{
				const FRigBone& Element = *static_cast<FRigBone*>(Elements[Index].Get());

				FName ParentName = NAME_None;
				if (const FRigElementKey* ParentKey = ElementMap.Find(Element.GetParentElementKey()))
				{
					ParentName = ParentKey->Name;
				}

				FRigBone& NewElement = BoneHierarchy.Add(Element.Name, ParentName, ERigBoneType::User, Element.InitialTransform, Element.LocalTransform, InData.GlobalTransforms[Index]);
				ElementMap.FindOrAdd(Element.GetElementKey()) = NewElement.GetElementKey();
				PastedKeys.Add(NewElement.GetElementKey());
				break;
			}
			case ERigElementType::Control:
			{
				const FRigControl& Element = *static_cast<FRigControl*>(Elements[Index].Get());

				FName ParentName = NAME_None;
				if (const FRigElementKey* ParentKey = ElementMap.Find(Element.GetParentElementKey()))
				{
					ParentName = ParentKey->Name;
				}

				FName SpaceName = NAME_None;
				if (const FRigElementKey* SpaceKey = ElementMap.Find(Element.GetSpaceElementKey()))
				{
					SpaceName = SpaceKey->Name;
				}

				FRigControl& NewElement = ControlHierarchy.Add(Element.Name, Element.ControlType, ParentName, SpaceName, Element.OffsetTransform, Element.InitialValue, Element.GizmoName, Element.GizmoTransform, Element.GizmoColor);

				// copy additional members
				NewElement.DisplayName = Element.DisplayName;
				NewElement.bAnimatable = Element.bAnimatable;
				NewElement.PrimaryAxis = Element.PrimaryAxis;
				NewElement.bLimitTranslation = Element.bLimitTranslation;
				NewElement.bLimitRotation = Element.bLimitRotation;
				NewElement.bLimitScale= Element.bLimitScale;
				NewElement.MinimumValue = Element.MinimumValue;
				NewElement.MaximumValue = Element.MaximumValue;
				NewElement.bDrawLimits = Element.bDrawLimits;
				NewElement.bGizmoEnabled = Element.bGizmoEnabled;
				NewElement.bGizmoVisible = Element.bGizmoVisible;
				NewElement.ControlEnum = Element.ControlEnum;

				ElementMap.FindOrAdd(Element.GetElementKey()) = NewElement.GetElementKey();
				PastedKeys.Add(NewElement.GetElementKey());

				break;
			}
			case ERigElementType::Space:
			{
				const FRigSpace& Element = *static_cast<FRigSpace*>(Elements[Index].Get());

				FName ParentName = NAME_None;
				if (const FRigElementKey* ParentKey = ElementMap.Find(Element.GetParentElementKey()))
				{
					ParentName = ParentKey->Name;
				}

				FRigSpace& NewElement = SpaceHierarchy.Add(Element.Name, Element.SpaceType, ParentName, Element.InitialTransform);
				ElementMap.FindOrAdd(Element.GetElementKey()) = NewElement.GetElementKey();
				PastedKeys.Add(NewElement.GetElementKey());
				break;
			}
			case ERigElementType::Curve:
			{
				const FRigCurve& Element = *static_cast<FRigCurve*>(Elements[Index].Get());
				FRigCurve& NewElement = CurveContainer.Add(Element.Name, Element.Value);
				ElementMap.FindOrAdd(Element.GetElementKey()) = NewElement.GetElementKey();
				PastedKeys.Add(NewElement.GetElementKey());
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}
	}

	return PastedKeys;
}

