// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyController.h"
#include "ControlRig.h"
#include "AnimationCoreLibrary.h"
#include "UObject/Package.h"
#include "ModularRig.h"
#include "HelperUtil.h"
#include "Engine/SkeletalMesh.h"
#include "Math/Axis.h"
#include "Misc/ScopeLock.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "RigVMPythonUtils.h"
#endif

#include "Algo/Count.h"
#include "Commands/ElementSelectionCommand.h"
#include "Engine/SkeletalMeshSocket.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyController)

////////////////////////////////////////////////////////////////////////////////
// FRigHierarchyImportErrorContext
////////////////////////////////////////////////////////////////////////////////

class FRigHierarchyImportErrorContext : public FOutputDevice
{
public:

	int32 NumErrors;

	FRigHierarchyImportErrorContext()
		: FOutputDevice()
		, NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		UE_LOGF(LogControlRig, Error, "Error Importing To Hierarchy: %ls", V);
		NumErrors++;
	}
};

////////////////////////////////////////////////////////////////////////////////
// URigHierarchyController
////////////////////////////////////////////////////////////////////////////////

URigHierarchyController::~URigHierarchyController()
{
}

void URigHierarchyController::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
    {
		URigHierarchy* OuterHierarchy = Cast<URigHierarchy>(GetOuter());
		SetHierarchy(OuterHierarchy);
    }
}

URigHierarchy* URigHierarchyController::GetHierarchy() const
{
	return Cast<URigHierarchy>(GetOuter());
}

void URigHierarchyController::SetHierarchy(URigHierarchy* InHierarchy)
{
	// since we changed the controller to be a property of the hierarchy,
	// controlling a different hierarchy is no longer allowed
	if (ensure(InHierarchy == GetOuter()))
	{
		InHierarchy->OnModified().RemoveAll(this);
		InHierarchy->OnModified().AddUObject(this, &URigHierarchyController::HandleHierarchyModified);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Invalid API Usage, Called URigHierarchyController::SetHierarchy(...) with a Hierarchy that is not the outer of the controller");
	}
}

bool URigHierarchyController::SelectElement(FRigElementKey InKey, bool bSelect, bool bClearSelection, bool bSetupUndo)
{
	return SelectHierarchyKey(InKey, bSelect, bClearSelection, bSetupUndo);
}

bool URigHierarchyController::SelectComponent(FRigComponentKey InKey, bool bSelect, bool bClearSelection, bool bSetupUndo)
{
	return SelectHierarchyKey(InKey, bSelect, bClearSelection, bSetupUndo);
}

bool URigHierarchyController::SelectHierarchyKey(FRigHierarchyKey InKey, bool bSelect, bool bClearSelection, bool bSetupUndo)
{
	using namespace UE::ControlRig;
	
	if(!IsValid())
	{
		return false;
	}

	if(bClearSelection)
	{
		TArray<FRigHierarchyKey> KeysToSelect;
		KeysToSelect.Add(InKey);
		return SetHierarchySelection(KeysToSelect);
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	if(URigHierarchy* HierarchyForSelection = Hierarchy->HierarchyForSelectionPtr.Get())
	{
		if(URigHierarchyController* ControllerForSelection = HierarchyForSelection->GetController())
		{
			return ControllerForSelection->SelectHierarchyKey(InKey, bSelect, bClearSelection, bSetupUndo);
		}
	}

	FRigHierarchyKey Key = InKey;
	if(Hierarchy->ElementKeyRedirector && Key.IsElement())
	{
		if(const FRigElementKeyRedirector::FCachedKeyArray* Cache = Hierarchy->ElementKeyRedirector->Find(Key.GetElement()))
		{
			if(!Cache->IsEmpty())
			{
				FRigElementKeyRedirector::FKeyArray Keys;
				for(const FCachedRigElement& CachedRigElement : *Cache)
				{
					if(const_cast<FCachedRigElement*>(&CachedRigElement)->UpdateCache(Hierarchy))
					{
						Keys.Add(CachedRigElement.GetKey());
					}
				}

				if(Keys.Num() == 1)
				{
					Key = Keys[0];
				}
				else
				{
					for(const FRigElementKey& RedirectedKey : Keys)
					{
						if(!SelectElement(RedirectedKey, bSelect, false))
						{
							return false;
						}
					}
					return true;
				}
			}
		}
	}

	if(Key.IsElement())
	{
		FRigBaseElement* Element = Hierarchy->Find(Key.GetElement());
		if(Element == nullptr)
		{
			return false;
		}

		const bool bSelectionState = Hierarchy->OrderedSelection.Contains(Element->GetKey());
		ensure(bSelectionState == Element->bSelected);
		if(Element->bSelected == bSelect)
		{
			return false;
		}

		// If recording into transaction buffer, modify the hierarchy so undoing selection works correctly.
		if (bSetupUndo && GUndo)
		{
			FControlRigCommandChange::StoreUndo(
				Hierarchy, MakeUnique<FElementSelectionCommand>(FElementSelectionData(Key, bSelect))
				);
		}
		
		Element->bSelected = bSelect;

		if(bSelect)
		{
			Hierarchy->OrderedSelection.Add(Element->GetKey());
		}
		else
		{
			Hierarchy->OrderedSelection.Remove(Element->GetKey());
		}

		if(Element->bSelected)
		{
			Notify(ERigHierarchyNotification::ElementSelected, Element);
		}
		else
		{
			Notify(ERigHierarchyNotification::ElementDeselected, Element);
		}

		Hierarchy->UpdateVisibilityOnProxyControls();
	}
	else if(Key.IsComponent())
	{
		FRigBaseComponent* Component = Hierarchy->FindComponent(Key.GetComponent());
		if(Component == nullptr)
		{
			return false;
		}

		const bool bSelectionState = Hierarchy->OrderedSelection.Contains(Component->GetKey());
		ensure(bSelectionState == Component->bSelected);
		if(Component->bSelected == bSelect)
		{
			return false;
		}

		// If recording into transaction buffer, modify the hierarchy so undoing selection works correctly.
		if (bSetupUndo && GUndo)
		{
			FControlRigCommandChange::StoreUndo(
				Hierarchy, MakeUnique<FElementSelectionCommand>(FElementSelectionData(Key, bSelect))
				);
		}
		
		Component->bSelected = bSelect;
	
		if(bSelect)
		{
			Hierarchy->OrderedSelection.Add(Component->GetKey());
		}
		else
		{
			Hierarchy->OrderedSelection.Remove(Component->GetKey());
		}

		if(Component->bSelected)
		{
			Notify(ERigHierarchyNotification::ComponentSelected, Component);
		}
		else
		{
			Notify(ERigHierarchyNotification::ComponentDeselected, Component);
		}

		Hierarchy->UpdateVisibilityOnProxyControls();
	}

	return true;
}

bool URigHierarchyController::SetSelection(const TArray<FRigElementKey>& InKeys, bool bPrintPythonCommand, bool bSetupUndo)
{
	TArray<FRigHierarchyKey> Keys;
	Keys.Reserve(InKeys.Num());
	for(const FRigElementKey& Key : InKeys)
	{
		Keys.Emplace(Key);
	}
	return SetHierarchySelection(Keys, bPrintPythonCommand, bSetupUndo);
}

bool URigHierarchyController::SetComponentSelection(const TArray<FRigComponentKey>& InKeys, bool bPrintPythonCommand)
{
	TArray<FRigHierarchyKey> Keys;
	Keys.Reserve(InKeys.Num());
	for(const FRigComponentKey& Key : InKeys)
	{
		Keys.Emplace(Key);
	}
	return SetHierarchySelection(Keys);
}

bool URigHierarchyController::SetHierarchySelection(const TArray<FRigHierarchyKey>& InKeys, bool bPrintPythonCommand, bool bSetupUndo)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	if(URigHierarchy* HierarchyForSelection = Hierarchy->HierarchyForSelectionPtr.Get())
	{
		if(URigHierarchyController* ControllerForSelection = HierarchyForSelection->GetController())
		{
			return ControllerForSelection->SetHierarchySelection(InKeys, false, bSetupUndo);
		}
	}

	bool bResult = true;
	
	const bool bShouldUpdateSelection = !(InKeys.IsEmpty() && Hierarchy->OrderedSelection.IsEmpty());
	if (bShouldUpdateSelection)
	{
		// disable python printing here as we only want to print a single command instead of one per selected item
		const TGuardValue<bool> Guard(bSuspendPythonPrinting, true);
		constexpr bool bDoNotClearSelection = false;
		
		const TArray<FRigHierarchyKey> PreviousSelection(Hierarchy->OrderedSelection);

		// unselect all
		if (InKeys.IsEmpty())
		{
			constexpr bool bUnSelect = false;
			for (const FRigHierarchyKey& KeyToDeselect : PreviousSelection)
			{
				if (!SelectHierarchyKey(KeyToDeselect, bUnSelect, bDoNotClearSelection, bSetupUndo))
				{
					bResult = false;
				}
			}			
		}
		else
		{
			// update selection
			{
				constexpr bool bUnSelect = false;
				for (const FRigHierarchyKey& KeyToDeselect : PreviousSelection)
				{
					if (!InKeys.Contains(KeyToDeselect))
					{
						if (!SelectHierarchyKey(KeyToDeselect, bUnSelect, bDoNotClearSelection, bSetupUndo))
						{
							bResult = false;
						}
					}
				}
			}
			
			{
				constexpr bool bSelect = true;
				for(const FRigHierarchyKey& KeyToSelect : InKeys)
				{
					if(!PreviousSelection.Contains(KeyToSelect))
					{
						if (!SelectHierarchyKey(KeyToSelect, bSelect, bDoNotClearSelection, bSetupUndo))
						{
							bResult = false;
						}
					}
				}
			}
		}
	}

#if WITH_EDITOR
	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			const int32 NumElements = Algo::CountIf(InKeys, [](const FRigHierarchyKey& Key) { return Key.IsElement(); });
			const int32 NumComponents = Algo::CountIf(InKeys, [](const FRigHierarchyKey& Key) { return Key.IsComponent(); });
			if(NumComponents == 0)
			{
				const FString Selection = FString::JoinBy( InKeys, TEXT(", "), [](const FRigHierarchyKey& Key)
				{
					return Key.GetElement().ToPythonString();
				});
				
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("hierarchy_controller.set_selection([%s])"),
					*Selection ) );
			}
			else if(NumElements == 0)
			{
				const FString Selection = FString::JoinBy( InKeys, TEXT(", "), [](const FRigHierarchyKey& Key)
				{
					return Key.GetComponent().ToPythonString();
				});
				
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("hierarchy_controller.set_component_selection([%s])"),
					*Selection ) );
			}
		}
	}
#endif

	return bResult;
}

FRigElementKey URigHierarchyController::AddBone(FName InName, FRigElementKey InParent, FTransform InTransform,
                                                bool bTransformInGlobal, ERigBoneType InBoneType, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Bone", "Add Bone"));
        Hierarchy->Modify();
	}
#endif

	FRigBoneElement* NewElement = MakeElement<FRigBoneElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(Hierarchy->bEnableCacheValidityCheck, false);
		NewElement->Key.Type = ERigElementType::Bone;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);
		NewElement->BoneType = InBoneType;

		constexpr bool bMaintainGlobalTransform = true;
		FRigBaseElement* Parent = Hierarchy->Get(Hierarchy->GetIndex(InParent));
		AddTransformElementInternal(NewElement, Parent, InTransform, bTransformInGlobal, bMaintainGlobalTransform, InName);
	}

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			TArray<FString> Commands = GetAddBonePythonCommands(NewElement);
			for (const FString& Command : Commands)
			{			
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();
		
	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddNull(FName InName, FRigElementKey InParent, FTransform InTransform,
                                                bool bTransformInGlobal, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Null", "Add Null"));
		Hierarchy->Modify();
	}
#endif

	FRigNullElement* NewElement = MakeElement<FRigNullElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(Hierarchy->bEnableCacheValidityCheck, false);		
		NewElement->Key.Type = ERigElementType::Null;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);

		constexpr bool bDoNotMaintainGlobalTransform = false;
		FRigBaseElement* Parent = Hierarchy->Get(Hierarchy->GetIndex(InParent));
		AddTransformElementInternal(NewElement, Parent, InTransform, bTransformInGlobal, bDoNotMaintainGlobalTransform, InName);
	}

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			TArray<FString> Commands = GetAddNullPythonCommands(NewElement);
			for (const FString& Command : Commands)
			{
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddControl(
	FName InName,
	FRigElementKey InParent,
	FRigControlSettings InSettings,
	FRigControlValue InValue,
	FTransform InOffsetTransform,
	FTransform InShapeTransform,
	bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Control", "Add Control"));
		Hierarchy->Modify();
	}
#endif

	FRigControlElement* NewElement = MakeElement<FRigControlElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(Hierarchy->bEnableCacheValidityCheck, false);		
		NewElement->Key.Type = ERigElementType::Control;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);
		NewElement->Settings = InSettings;
		if(NewElement->Settings.LimitEnabled.IsEmpty())
		{
			NewElement->Settings.SetupLimitArrayForType();
		}

		if(!NewElement->Settings.DisplayName.IsNone())
		{
			// avoid self name collision
			FName DesiredDisplayName = NAME_None;
			Swap(DesiredDisplayName, NewElement->Settings.DisplayName);
			
			NewElement->Settings.DisplayName = Hierarchy->GetSafeNewDisplayName(InParent, DesiredDisplayName); 
		}
		else if(Hierarchy->HasExecuteContext())
		{
			const FControlRigExecuteContext& CRContext = Hierarchy->ExecuteContext->GetPublicData<FControlRigExecuteContext>();
			if(!CRContext.GetRigModulePrefix().IsEmpty())
			{
				// avoid self name collision
				NewElement->Settings.DisplayName = NAME_None;
				NewElement->Settings.DisplayName = Hierarchy->GetSafeNewDisplayName(InParent, NewElement->Key.Name);
			}
		}
		
		AddElement(NewElement, Hierarchy->Get(Hierarchy->GetIndex(InParent)), false, InName);
		
		NewElement->GetOffsetTransform().Set(ERigTransformType::InitialLocal, InOffsetTransform);  
		NewElement->GetOffsetDirtyState().MarkClean(ERigTransformType::InitialLocal);
		NewElement->GetShapeTransform().Set(ERigTransformType::InitialLocal, InShapeTransform);
		NewElement->GetShapeDirtyState().MarkClean(ERigTransformType::InitialLocal);
		Hierarchy->SetControlValue(NewElement, InValue, ERigControlValueType::Initial, false);
		const FTransform LocalTransform = Hierarchy->GetTransform(NewElement, ERigTransformType::InitialLocal);
		static constexpr bool bInitial = true;
		Hierarchy->SetControlPreferredEulerAngles(NewElement, LocalTransform, bInitial);

		NewElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
		NewElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
		NewElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
		NewElement->GetOffsetTransform().Current = NewElement->GetOffsetTransform().Initial;
		NewElement->GetOffsetDirtyState().Current = NewElement->GetOffsetDirtyState().Initial; 
		NewElement->GetTransform().Current = NewElement->GetTransform().Initial;
		NewElement->GetDirtyState().Current = NewElement->GetDirtyState().Initial;
		NewElement->PreferredEulerAngles.Current = NewElement->PreferredEulerAngles.Initial;
		NewElement->GetShapeTransform().Current = NewElement->GetShapeTransform().Initial;
		NewElement->GetShapeDirtyState().Current = NewElement->GetShapeDirtyState().Initial;
	}

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			TArray<FString> Commands = GetAddControlPythonCommands(NewElement);
			for (const FString& Command : Commands)
			{
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddAnimationChannel(FName InName, FRigElementKey InParentControl,
	FRigControlSettings InSettings, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	if(const FRigControlElement* ParentControl = Hierarchy->Find<FRigControlElement>(InParentControl))
	{
		InSettings.AnimationType = ERigControlAnimationType::AnimationChannel;
		InSettings.bGroupWithParentControl = true;

		return AddControl(InName, ParentControl->GetKey(), InSettings, InSettings.GetIdentityValue(),
			FTransform::Identity, FTransform::Identity, bSetupUndo, bPrintPythonCommand);
	}

	return FRigElementKey();
}

FRigElementKey URigHierarchyController::AddCurve(FName InName, float InValue, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Curve", "Add Curve"));
		Hierarchy->Modify();
	}
#endif

	FRigCurveElement* NewElement = MakeElement<FRigCurveElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(Hierarchy->bEnableCacheValidityCheck, false);		
		NewElement->Key.Type = ERigElementType::Curve;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);
		AddElement(NewElement, nullptr, false, InName);
		NewElement->Set(InValue);
		NewElement->bIsValueSet = false;
	}

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			TArray<FString> Commands = GetAddCurvePythonCommands(NewElement);
			for (const FString& Command : Commands)
			{
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddReference(FName InName, FRigElementKey InParent,
	FRigReferenceGetWorldTransformDelegate InDelegate, bool bSetupUndo)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Reference", "Add Reference"));
		Hierarchy->Modify();
	}
#endif

	FRigReferenceElement* NewElement = MakeElement<FRigReferenceElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(Hierarchy->bEnableCacheValidityCheck, false);		
		NewElement->Key.Type = ERigElementType::Reference;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);
		NewElement->GetWorldTransformDelegate = InDelegate;
		AddElement(NewElement, Hierarchy->Get(Hierarchy->GetIndex(InParent)), true, InName);

		Hierarchy->SetTransform(NewElement, FTransform::Identity, ERigTransformType::InitialLocal, true, false);
		NewElement->GetTransform().Current = NewElement->GetTransform().Initial;
		NewElement->GetDirtyState().Current = NewElement->GetDirtyState().Initial;
	}

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	Hierarchy->EnsureCacheValidity();

	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddConnector(FName InName, FRigConnectorSettings InSettings, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	// only allow to add one primary connector
	if(InSettings.Type == EConnectorType::Primary)
	{
		InSettings.bIsArray = false;
		
		const TArray<FRigConnectorElement*>& Connectors = Hierarchy->GetConnectors();
		for(const FRigConnectorElement* Connector : Connectors)
		{
			if(Connector->IsPrimary())
			{
				if(Hierarchy->HasExecuteContext())
				{
					const FControlRigExecuteContext& CRContext = Hierarchy->ExecuteContext->GetPublicData<FControlRigExecuteContext>();
					const FString ModulePrefix = CRContext.GetRigModulePrefix();
					if(!ModulePrefix.IsEmpty())
					{
						const FString ConnectorModulePrefix = Hierarchy->GetModulePrefix(Connector->GetKey());
						if(!ConnectorModulePrefix.IsEmpty() && ConnectorModulePrefix.Equals(ModulePrefix, ESearchCase::IgnoreCase))
						{
							static constexpr TCHAR Format[] = TEXT("Cannot add connector '%s' - there already is a primary connector.");
							ReportAndNotifyErrorf(Format, *InName.ToString());
							return FRigElementKey();
						}
					}
				}
			}
		}

		if(InSettings.bOptional)
		{
			static constexpr TCHAR Format[] = TEXT("Cannot add connector '%s' - primary connectors cannot be optional.");
			ReportAndNotifyErrorf(Format, *InName.ToString());
			return FRigElementKey();
		}
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Connector", "Add Connector"));
		Hierarchy->Modify();
	}
#endif

	FRigConnectorElement* NewElement = MakeElement<FRigConnectorElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(Hierarchy->bEnableCacheValidityCheck, false);
		NewElement->Key.Type = ERigElementType::Connector;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);
		NewElement->Settings = InSettings;
		AddElement(NewElement, nullptr, true, InName);
	}

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			TArray<FString> Commands = GetAddConnectorPythonCommands(NewElement);
			for (const FString& Command : Commands)
			{			
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();
		
	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddSocket(FName InName, FRigElementKey InParent, FTransform InTransform,
	bool bTransformInGlobal, const FLinearColor& InColor, const FString& InDescription, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* CurrentHierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Socket", "Add Socket"));
		CurrentHierarchy->Modify();
	}
#endif

	FRigSocketElement* NewElement = MakeElement<FRigSocketElement>();
	{
		TGuardValue<bool> DisableCacheValidityChecks(CurrentHierarchy->bEnableCacheValidityCheck, false);
		NewElement->Key.Type = ERigElementType::Socket;
		NewElement->Key.Name = GetSafeNewName(InName, NewElement->Key.Type);
		
		constexpr bool bMaintainGlobalTransform = true;
		FRigBaseElement* Parent = CurrentHierarchy->Get(CurrentHierarchy->GetIndex(InParent));
		AddTransformElementInternal(NewElement, Parent, InTransform, bTransformInGlobal, bMaintainGlobalTransform, InName);

		NewElement->SetColor(InColor, CurrentHierarchy);
		NewElement->SetDescription(InDescription, CurrentHierarchy);
		CurrentHierarchy->SetRigElementKeyMetadata(NewElement->Key, FRigSocketElement::DesiredParentMetaName, InParent);
	}

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			TArray<FString> Commands = GetAddSocketPythonCommands(NewElement);
			for (const FString& Command : Commands)
			{			
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	CurrentHierarchy->EnsureCacheValidity();
		
	return NewElement->Key;
}

FRigElementKey URigHierarchyController::AddDefaultRootSocket()
{
	FRigElementKey SocketKey;
	if(const URigHierarchy* CurrentHierarchy = GetHierarchy())
	{
		static const FRigElementKey RootSocketKey(TEXT("Root"), ERigElementType::Socket);
		if(CurrentHierarchy->Contains(RootSocketKey))
		{
			return RootSocketKey;
		}

		CurrentHierarchy->ForEach<FRigBoneElement>([this, CurrentHierarchy, &SocketKey](const FRigBoneElement* Bone) -> bool
		{
			// find first root bone
			if(CurrentHierarchy->GetNumberOfParents(Bone) == 0)
			{
				SocketKey = AddSocket(RootSocketKey.Name, Bone->GetKey(), FTransform::Identity);

				// stop
				return false;
			}

			// continue with the search
			return true;
		});
	}
	return SocketKey; 
}

FRigControlSettings URigHierarchyController::GetControlSettings(FRigElementKey InKey) const
{
	if(!IsValid())
	{
		return FRigControlSettings();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InKey);
	if(ControlElement == nullptr)
	{
		return FRigControlSettings();
	}

	return ControlElement->Settings;
}

bool URigHierarchyController::SetControlSettings(FRigElementKey InKey, FRigControlSettings InSettings, bool bSetupUndo) const
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InKey);
	if(ControlElement == nullptr)
	{
		return false;
	}

	#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "SetControlSettings", "Set Control Settings"));
		Hierarchy->Modify();
	}
#endif

	ControlElement->Settings = InSettings;
	if(ControlElement->Settings.LimitEnabled.IsEmpty())
	{
		ControlElement->Settings.SetupLimitArrayForType(false, false, false);
	}

	FRigControlValue InitialValue = Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Initial);
	FRigControlValue CurrentValue = Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Current);

	ControlElement->Settings.ApplyLimits(InitialValue);
	ControlElement->Settings.ApplyLimits(CurrentValue);

	Hierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);

	Hierarchy->SetControlValue(ControlElement, InitialValue, ERigControlValueType::Initial, bSetupUndo);
	Hierarchy->SetControlValue(ControlElement, CurrentValue, ERigControlValueType::Current, bSetupUndo);

#if WITH_EDITOR
    TransactionPtr.Reset();
#endif

	Hierarchy->EnsureCacheValidity();
	
	return true;
}

FRigComponentKey URigHierarchyController::AddComponent(UScriptStruct* InComponentStruct, FName InName, FRigElementKey InElement, FString InContent, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return FRigComponentKey();
	}
	
	if(InComponentStruct == nullptr)
	{
		ReportError(TEXT("The passed component struct is nullptr."));
		return FRigComponentKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);

	FRigBaseElement* Element = Hierarchy->Find(InElement);
	if(Element == nullptr)
	{
		ReportErrorf(TEXT("The element '%s' could not be found."), *InElement.ToString());
		return FRigComponentKey();
	}

	FString FailureReason;
	if(!Hierarchy->CanAddComponent(InElement, InComponentStruct, &FailureReason))
	{
		ReportError(FailureReason);
		return FRigComponentKey();
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Component", "Add Component"));
		Hierarchy->Modify();
	}
#endif

	if(InName.IsNone())
	{
		const FStructOnScope StructOnScope(InComponentStruct);
		if(const FRigBaseComponent* StructMemory = reinterpret_cast<const FRigBaseComponent*>(StructOnScope.GetStructMemory()))
		{
			InName = StructMemory->GetDefaultComponentName();
			check(!InName.IsNone());
		}
	}

	FRigBaseComponent* NewComponent = Hierarchy->MakeComponent(InComponentStruct, InName, Element);
	NewComponent->CreatedAtInstructionIndex = CurrentInstructionIndex;
	if(!InContent.IsEmpty())
	{
		(void)SetComponentContent(NewComponent->GetKey(), InContent, false, false);
	}
	Hierarchy->IncrementTopologyVersion();
	Notify(ERigHierarchyNotification::ComponentAdded, NewComponent);

	// allow the component to react to it being spawned
	NewComponent->OnAddedToHierarchy(Hierarchy, this);

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			TArray<FString> Commands = GetAddComponentPythonCommands(NewComponent);
			for (const FString& Command : Commands)
			{			
				RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
					FString::Printf(TEXT("%s"), *Command));
			}
		}
	}
#endif

	return NewComponent->GetKey();
}

bool URigHierarchyController::RemoveComponent(FRigComponentKey InComponent, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}
	
	if(!InComponent.IsValid())
	{
		ReportError(TEXT("The passed component key is invalid."));
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);
	
	FRigBaseComponent* Component = Hierarchy->FindComponent(InComponent);
	if(Component == nullptr)
	{
		ReportErrorf(TEXT("The component '%s' cannot be found."), *InComponent.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Remove Component", "Remove Component"));
		Hierarchy->Modify();
	}
#endif

	Notify(ERigHierarchyNotification::ComponentRemoved, Component);
	Hierarchy->DestroyComponent(Component);
	Hierarchy->IncrementTopologyVersion();

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.remove_component(%s"), *InComponent.ToPythonString()));
		}
	}
#endif

	return true;
}

FRigComponentKey URigHierarchyController::RenameComponent(FRigComponentKey InComponent, FName InName, bool bSetupUndo, bool bPrintPythonCommand,
	bool bClearSelection)
{
	if(!IsValid())
	{
		return FRigComponentKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseComponent* Component = Hierarchy->FindComponent(InComponent);
	if(Component == nullptr)
	{
		ReportWarningf(TEXT("Cannot Rename Component: '%s' not found."), *InComponent.ToString());
		return FRigComponentKey();
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Rename Component", "Rename Component"));
		Hierarchy->Modify();
	}
#endif

	const bool bRenamed = RenameComponent(Component, InName, bClearSelection, bSetupUndo);

#if WITH_EDITOR
	if(!bRenamed && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bRenamed && bClearSelection)
	{
		ClearSelection();
	}

	if (bRenamed && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(), 
				FString::Printf(TEXT("hierarchy_controller.rename_component(%s, '%s')"),
				*InComponent.ToPythonString(),
				*InName.ToString()));
		}
	}
#endif

	return bRenamed ? Component->GetKey() : FRigComponentKey();
}

FRigComponentKey URigHierarchyController::ReparentComponent(FRigComponentKey InComponentKey, FRigElementKey InParentElementKey, bool bSetupUndo,
	bool bPrintPythonCommand, bool bClearSelection)
{
	if(!IsValid())
	{
		return FRigComponentKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseComponent* Component = Hierarchy->FindComponent(InComponentKey);
	if(Component == nullptr)
	{
		ReportWarningf(TEXT("Cannot Reparent Component: '%s' not found."), *InComponentKey.ToString());
		return FRigComponentKey();
	}

	// it's ok if this is nullptr
	FRigBaseElement* ParentElement = Hierarchy->Find(InParentElementKey);

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Reparent Component", "Reparent Component"));
		Hierarchy->Modify();
	}
#endif

	const bool bReparented = ReparentComponent(Component, ParentElement, bClearSelection, bSetupUndo);

#if WITH_EDITOR
	if(!bReparented && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bReparented && bClearSelection)
	{
		ClearSelection();
	}

	if (bReparented && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(), 
				FString::Printf(TEXT("hierarchy_controller.reparent_component(%s, '%s')"),
				*InComponentKey.ToPythonString(),
				*InParentElementKey.ToPythonString()));
		}
	}
#endif

	return bReparented ? Component->GetKey() : FRigComponentKey();
}

bool URigHierarchyController::SetComponentContent(FRigComponentKey InComponent, const FString& InContent, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}
	
	if(!InComponent.IsValid())
	{
		ReportError(TEXT("The passed component key is valid."));
		return false;
	}

	if(InContent.IsEmpty())
	{
		ReportError(TEXT("The passed content is empty."));
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);
	
	FRigBaseComponent* Component = Hierarchy->FindComponent(InComponent);
	if(Component == nullptr)
	{
		ReportErrorf(TEXT("The component '%s' cannot be found."), *InComponent.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Set Component Content", "Set Component Content"));
		Hierarchy->Modify();
	}
#endif

	// create a dummy component first to import onto
	FStructOnScope ScopedStruct(Component->GetScriptStruct());
	FRigHierarchyImportErrorContext ErrorPipe;
	Component->GetScriptStruct()->ImportText(*InContent, ScopedStruct.GetStructMemory(), nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, Component->GetScriptStruct()->GetName(), true);
	if (ErrorPipe.NumErrors > 0)
	{
		return false;;
	}

	// now import onto the actual component
	ErrorPipe.NumErrors = 0;
	Component->GetScriptStruct()->ImportText(*InContent, Component, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, Component->GetScriptStruct()->GetName(), true);

	Notify(ERigHierarchyNotification::ComponentContentChanged, Component);

#if WITH_EDITOR
	TransactionPtr.Reset();

	if (bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.set_component_content( %s, '%s'"), *InComponent.ToPythonString(), *InContent));
		}
	}
#endif

	return true;
}

bool URigHierarchyController::SetComponentState(FRigComponentKey InComponent, const FRigComponentState& InState, bool bSetupUndo)
{
	if(!IsValid())
	{
		return false;
	}
	
	if(!InComponent.IsValid())
	{
		ReportError(TEXT("The passed component key is valid."));
		return false;
	}

	if(!InState.IsValid())
	{
		ReportError(TEXT("The passed content is not valid."));
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);
	
	FRigBaseComponent* Component = Hierarchy->FindComponent(InComponent);
	if(Component == nullptr)
	{
		ReportErrorf(TEXT("The component '%s' cannot be found."), *InComponent.ToString());
		return false;
	}

	if(InState.GetComponentStruct() != Component->GetScriptStruct())
	{
		ReportErrorf(TEXT("The passed content(%s) does not match the component(%s)."), *InState.GetComponentStruct()->GetName(), *Component->GetScriptStruct()->GetName());
		return false;
	}

	const FRigComponentState CurrentState = Component->GetState();
	if(CurrentState == InState)
	{
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Set Component State", "Set Component State"));
		Hierarchy->Modify();
	}
#endif

	if(!Component->SetState(InState))
	{
#if WITH_EDITOR
		if(TransactionPtr)
		{
			TransactionPtr->Cancel();
		}
#endif
		return false;
	}
	Notify(ERigHierarchyNotification::ComponentContentChanged, Component);

#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	return true;
}

TArray<FRigElementKey> URigHierarchyController::ImportBones(
	USkeletalMesh* InSkeletalMesh,
	const FName& InNameSpace,
	bool bReplaceExistingBones,
	bool bRemoveObsoleteBones,
	bool bSelectBones,
	bool bSetupUndo)
{
	TArray<FRigElementKey> RigElementKeys;

	if (InSkeletalMesh != nullptr)
	{
		const FReferenceSkeleton& RefSkeleton = InSkeletalMesh->GetRefSkeleton();
		const TArray<FMeshBoneInfo>& RefBoneInfos = RefSkeleton.GetRefBoneInfo();
		const TArray<FTransform>& RefBonePoses = RefSkeleton.GetRefBonePose();

		USkeleton* MeshSkeleton = InSkeletalMesh->GetSkeleton();
	
		TArray<FMeshBoneInfo> BoneInfos;
		BoneInfos.Reserve(RefBoneInfos.Num());
		TArray<FTransform> BoneTransforms;
		BoneTransforms.Reserve(RefBonePoses.Num());

		const int32 NumSkeletonBones = RefBoneInfos.Num();
		for (int32 i=0; i<NumSkeletonBones; ++i)
		{
			const FMeshBoneInfo& MeshBoneInfo = RefBoneInfos[i];
			const int32 SkeletonBoneIndex = RefSkeleton.FindBoneIndex(MeshBoneInfo.Name);
			if (MeshSkeleton->GetMeshBoneIndexFromSkeletonBoneIndex(InSkeletalMesh, SkeletonBoneIndex) == INDEX_NONE)
			{
				// if bone index is none, the bone does not exist in the mesh and it has been excluded
				// add an empty bone info to keep the same indexes
				BoneInfos.Add(FMeshBoneInfo());
				BoneTransforms.Add(FTransform::Identity);
			}
			else
			{
				const FTransform& BoneTransform = RefBonePoses[i];
				BoneInfos.Add(MeshBoneInfo);
				BoneTransforms.Add(BoneTransform);
			}
		}

		RigElementKeys = ImportBones(BoneInfos, BoneTransforms, InNameSpace, bReplaceExistingBones, bRemoveObsoleteBones, bSelectBones, bSetupUndo);
	}
	return RigElementKeys;
}

TArray<FRigElementKey> URigHierarchyController::ImportBones(const FReferenceSkeleton& InSkeleton,
	const FName& InNameSpace, bool bReplaceExistingBones, bool bRemoveObsoleteBones, bool bSelectBones,
	bool bSetupUndo)
{
	const TArray<FMeshBoneInfo>& BoneInfos = InSkeleton.GetRefBoneInfo();
	const TArray<FTransform>& BoneTransforms = InSkeleton.GetRefBonePose();

	return ImportBones(BoneInfos, BoneTransforms, InNameSpace, bReplaceExistingBones, bRemoveObsoleteBones, bSelectBones, bSetupUndo);
}

TArray<FRigElementKey> URigHierarchyController::ImportBones(
	const TArray<FMeshBoneInfo> & BoneInfos,
	const TArray<FTransform>& BoneTransforms,
	const FName& InNameSpace,
	bool bReplaceExistingBones,
	bool bRemoveObsoleteBones,
	bool bSelectBones,
	bool bSetupUndo)
{
	const TGuardValue<bool> CycleCheckGuard(bSuspendParentCycleCheck, true);

	TArray<FRigElementKey> AddedBones;

	if(!IsValid())
	{
		return AddedBones;
	}
	
	TArray<FRigElementKey> BonesToSelect;
	TMap<FName, FName> BoneNameMap;

	URigHierarchy* Hierarchy = GetHierarchy();

	Hierarchy->ResetPoseToInitial();

	struct Local
	{
		static FName DetermineBoneName(const FName& InBoneName, const FName& InLocalNameSpace)
		{
			if (InLocalNameSpace == NAME_None || InBoneName == NAME_None)
			{
				return InBoneName;
			}
			return *FString::Printf(TEXT("%s_%s"), *InLocalNameSpace.ToString(), *InBoneName.ToString());
		}
	};

	if (bReplaceExistingBones)
	{
		TArray<FRigBoneElement*> AllBones = GetHierarchy()->GetElementsOfType<FRigBoneElement>(true);
		for(FRigBoneElement* BoneElement : AllBones)
		{
			BoneNameMap.Add(BoneElement->GetFName(), BoneElement->GetFName());
		}

		for (int32 Index = 0; Index < BoneInfos.Num(); ++Index)
		{
			const FName& BoneName = BoneInfos[Index].Name;
			if (BoneName == NAME_None)
			{
				continue;
			}
			const FRigElementKey ExistingBoneKey(BoneName, ERigElementType::Bone);
			const int32 ExistingBoneIndex = Hierarchy->GetIndex(ExistingBoneKey);
			
			const FName DesiredBoneName = Local::DetermineBoneName(BoneName, InNameSpace);
			const int32 ParentBoneIndex = BoneInfos[Index].ParentIndex;
			FName ParentName = (ParentBoneIndex != INDEX_NONE && BoneInfos.IsValidIndex(ParentBoneIndex)) ? BoneInfos[ParentBoneIndex].Name : NAME_None;
			ParentName = Local::DetermineBoneName(ParentName, InNameSpace);

			const FName* MappedParentNamePtr = BoneNameMap.Find(ParentName);
			if (MappedParentNamePtr)
			{
				ParentName = *MappedParentNamePtr;
			}

			const FRigElementKey ParentKey(ParentName, ERigElementType::Bone);

			// if this bone already exists
			if (ExistingBoneIndex != INDEX_NONE)
			{
				const int32 ParentIndex = Hierarchy->GetIndex(ParentKey);
				
				// check it's parent
				if (ParentIndex != INDEX_NONE)
				{
					SetParent(ExistingBoneKey, ParentKey, bSetupUndo);
				}

				Hierarchy->SetInitialLocalTransform(ExistingBoneIndex, BoneTransforms[Index], true, bSetupUndo);
				Hierarchy->SetLocalTransform(ExistingBoneIndex, BoneTransforms[Index], true, bSetupUndo);

				BonesToSelect.Add(ExistingBoneKey);
			}
			else
			{
				const FRigElementKey AddedBoneKey = AddBone(DesiredBoneName, ParentKey, BoneTransforms[Index], false, ERigBoneType::Imported, bSetupUndo);
				BoneNameMap.Add(DesiredBoneName, AddedBoneKey.Name);
				AddedBones.Add(AddedBoneKey);
				BonesToSelect.Add(AddedBoneKey);
			}
		}

	}
	else // import all as new
	{
		for (int32 Index = 0; Index < BoneInfos.Num(); ++Index)
		{
			const FName& BoneName = BoneInfos[Index].Name;
			if (BoneName == NAME_None)
			{
				continue;
			}
			FName DesiredBoneName = Local::DetermineBoneName(BoneName, InNameSpace);
			FName ParentName = (BoneInfos[Index].ParentIndex != INDEX_NONE) ? BoneInfos[BoneInfos[Index].ParentIndex].Name : NAME_None;
			ParentName = Local::DetermineBoneName(ParentName, InNameSpace);

			const FName* MappedParentNamePtr = BoneNameMap.Find(ParentName);
			if (MappedParentNamePtr)
			{
				ParentName = *MappedParentNamePtr;
			}

			const FRigElementKey ParentKey(ParentName, ERigElementType::Bone);
			const FRigElementKey AddedBoneKey = AddBone(DesiredBoneName, ParentKey, BoneTransforms[Index], false, ERigBoneType::Imported, bSetupUndo);
			BoneNameMap.Add(DesiredBoneName, AddedBoneKey.Name);
			AddedBones.Add(AddedBoneKey);
			BonesToSelect.Add(AddedBoneKey);
		}
	}

	if (bReplaceExistingBones && bRemoveObsoleteBones)
	{
		TMap<FName, int32> BoneNameToIndexInSkeleton;
		for (const FMeshBoneInfo& BoneInfo : BoneInfos)
		{
			const FName& BoneName = BoneInfo.Name;
			if (BoneName == NAME_None)
			{
				continue;
			}
			FName DesiredBoneName = Local::DetermineBoneName(BoneName, InNameSpace);
			BoneNameToIndexInSkeleton.Add(DesiredBoneName, BoneNameToIndexInSkeleton.Num());
		}
		
		TArray<FRigElementKey> BonesToDelete;
		TArray<FRigBoneElement*> AllBones = GetHierarchy()->GetElementsOfType<FRigBoneElement>(true);
		for(FRigBoneElement* BoneElement : AllBones)
        {
            if (!BoneNameToIndexInSkeleton.Contains(BoneElement->GetFName()))
			{
				if (BoneElement->BoneType == ERigBoneType::Imported)
				{
					BonesToDelete.Add(BoneElement->GetKey());
				}
			}
		}

		for (const FRigElementKey& BoneToDelete : BonesToDelete)
		{
			TArray<FRigElementKey> Children = Hierarchy->GetChildren(BoneToDelete);
			Algo::Reverse(Children);
			
			for (const FRigElementKey& Child : Children)
			{
				if(BonesToDelete.Contains(Child))
				{
					continue;
				}
				RemoveAllParents(Child, true, bSetupUndo);
			}
		}

		for (const FRigElementKey& BoneToDelete : BonesToDelete)
		{
			RemoveElement(BoneToDelete);
			BonesToSelect.Remove(BoneToDelete);
		}

		// update the sub index to match the bone index in the skeleton
		for (int32 Index = 0; Index < BoneInfos.Num(); ++Index)
		{
			const FName& BoneName = BoneInfos[Index].Name;
			if (BoneName == NAME_None)
			{
				continue;
			}
			FName DesiredBoneName = Local::DetermineBoneName(BoneName, InNameSpace);
			const FRigElementKey Key(DesiredBoneName, ERigElementType::Bone);
			if(FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(Key))
			{
				BoneElement->SubIndex = Index;
			}
		}		
	}

	if (bSelectBones)
	{
		SetSelection(BonesToSelect);
	}

	Hierarchy->EnsureCacheValidity();

	return AddedBones;
}

TArray<FRigElementKey> URigHierarchyController::ImportBones(USkeleton* InSkeleton, FName InNameSpace,
                                                            bool bReplaceExistingBones, bool bRemoveObsoleteBones,
                                                            bool bSelectBones, bool bSetupUndo,
                                                            bool bPrintPythonCommand)
{
	FReferenceSkeleton EmptySkeleton;
	FReferenceSkeleton& RefSkeleton = EmptySkeleton; 
	if (InSkeleton != nullptr)
	{
		RefSkeleton = InSkeleton->GetReferenceSkeleton();
	}

	const TArray<FRigElementKey> BoneKeys = ImportBones(RefSkeleton, InNameSpace, bReplaceExistingBones, bRemoveObsoleteBones,
	                   bSelectBones, bSetupUndo);

#if WITH_EDITOR
	if (!BoneKeys.IsEmpty() && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (UObject* Asset = GetHierarchy()->GetOuter())
		{
			RigVMPythonUtils::Print(Asset->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.import_bones_from_asset('%s', '%s', %s, %s, %s)"),
				(InSkeleton != nullptr) ? *InSkeleton->GetPathName() : TEXT(""),
				*InNameSpace.ToString(),
				(bReplaceExistingBones) ? TEXT("True") : TEXT("False"),
				(bRemoveObsoleteBones) ? TEXT("True") : TEXT("False"),
				(bSelectBones) ? TEXT("True") : TEXT("False")));
		}
	}
#endif
	return BoneKeys;
}

TArray<FRigElementKey> URigHierarchyController::ImportBonesFromSkeletalMesh(
	USkeletalMesh* InSkeletalMesh,
	const FName& InNameSpace,
	bool bReplaceExistingBones,
	bool bRemoveObsoleteBones,
	bool bSelectBones,
	bool bSetupUndo,
	bool bPrintPythonCommand)
{
	const TArray<FRigElementKey> BoneKeys = ImportBones(InSkeletalMesh, InNameSpace, bReplaceExistingBones, bRemoveObsoleteBones, bSelectBones, bSetupUndo);

#if WITH_EDITOR
	if (!BoneKeys.IsEmpty() && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.import_bones_from_asset('%s', '%s', %s, %s, %s)"),
					(InSkeletalMesh != nullptr) ? *InSkeletalMesh->GetPathName() : TEXT(""),
					*InNameSpace.ToString(),
					(bReplaceExistingBones) ? TEXT("True") : TEXT("False"),
					(bRemoveObsoleteBones) ? TEXT("True") : TEXT("False"),
					(bSelectBones) ? TEXT("True") : TEXT("False")));
		}
	}
#endif
	return BoneKeys;
}

TArray<FRigElementKey> URigHierarchyController::ImportSocketsFromSkeletalMesh(USkeletalMesh* InSkeletalMesh, const FName& InNameSpace,
	bool bReplaceExistingSockets, bool bRemoveObsoleteSockets, bool bSelectSockets, bool bSetupUndo, bool bPrintPythonCommand)
{
	const TGuardValue<bool> CycleCheckGuard(bSuspendParentCycleCheck, true);
	
	TArray<FRigElementKey> SocketKeys;
	
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		for(int32 SocketIndex = 0; SocketIndex < InSkeletalMesh->NumSockets(); SocketIndex++)
		{
			if(const USkeletalMeshSocket* Socket = InSkeletalMesh->GetSocketByIndex(SocketIndex))
			{
				const FRigElementKey ParentKey(URigHierarchy::GetSanitizedName(Socket->BoneName).GetFName(), ERigElementType::Bone);
				if(Hierarchy->Contains(ParentKey))
				{
					FRigElementKey SocketKey(URigHierarchy::GetSanitizedName(Socket->SocketName).GetFName(), ERigElementType::Null);
					const FTransform SocketTransform = Socket->GetSocketLocalTransform();

					if(bReplaceExistingSockets && Hierarchy->Contains(SocketKey))
					{
						// set the parent - doesn't do anything if the parent is already correct
						SetParent(SocketKey, ParentKey, false, bSetupUndo, false);

						// update the transforms
						Hierarchy->SetLocalTransform(SocketKey, SocketTransform, true, true, bSetupUndo, false);
						Hierarchy->SetLocalTransform(SocketKey, SocketTransform, false, true, bSetupUndo, false);
					}
					else
					{
						SocketKey = AddNull(SocketKey.Name, ParentKey, SocketTransform, false, bSetupUndo, false);
					}

					Hierarchy->SetTag(SocketKey, TEXT("MeshSocket"));

					SocketKeys.Add(SocketKey);
				}
			}
		}

		if(bRemoveObsoleteSockets)
		{
			const TArray<FRigElementKey> NullKeys = Hierarchy->GetNullKeys();
			for(const FRigElementKey& NullKey : NullKeys)
			{
				if(Hierarchy->HasTag(NullKey, TEXT("MeshSocket")))
				{
					if(InSkeletalMesh->FindSocket(NullKey.Name) == nullptr)
					{
						RemoveElement(NullKey, bSetupUndo, false);
					}
				}
			}
		}

		if(bSelectSockets)
		{
			SetSelection(SocketKeys, false);
		}
	}
	return SocketKeys;
}

#if WITH_EDITOR

TArray<FRigElementKey> URigHierarchyController::ImportBonesFromAsset(FString InAssetPath, FName InNameSpace,
	bool bReplaceExistingBones, bool bRemoveObsoleteBones, bool bSelectBones, bool bSetupUndo)
{
	if(USkeleton* Skeleton = GetSkeletonFromAssetPath(InAssetPath))
	{
		return ImportBones(Skeleton, InNameSpace, bReplaceExistingBones, bRemoveObsoleteBones, bSelectBones, bSetupUndo);
	}
	return TArray<FRigElementKey>();
}

TArray<FRigElementKey> URigHierarchyController::ImportCurvesFromAsset(FString InAssetPath, FName InNameSpace,
                                                                      bool bSelectCurves, bool bSetupUndo)
{
	if(USkeletalMesh* SkeletalMesh = GetSkeletalMeshFromAssetPath(InAssetPath))
	{
		return ImportCurvesFromSkeletalMesh(SkeletalMesh, InNameSpace, bSelectCurves, bSetupUndo);
	}
	if(USkeleton* Skeleton = GetSkeletonFromAssetPath(InAssetPath))
	{
		return ImportCurves(Skeleton, InNameSpace, bSelectCurves, bSetupUndo);
	}
	return TArray<FRigElementKey>();
}

#endif

TArray<FRigElementKey> URigHierarchyController::ImportPreviewSkeletalMesh(USkeletalMesh* InSkeletalMesh, bool bReplaceExistingBones,
	bool bRemoveObsoleteBones, bool bSelectBones, bool bSetupUndo)
{
	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);
	
	// find the instruction index for the construction event
	int32 InstructionIndex = INDEX_NONE;
	if(UControlRig* ControlRig = Cast<UControlRig>(Hierarchy->GetOuter()))
	{
		if(URigVM* VM = ControlRig->GetVM())
		{
			const int32 EntryIndex = VM->GetByteCode().FindEntryIndex(FRigUnit_PrepareForExecution::EventName);
			if(EntryIndex != INDEX_NONE)
			{
				InstructionIndex = VM->GetByteCode().GetEntry(EntryIndex).InstructionIndex;
			}
		}
	}

	// import the bones for the preview hierarchy
	// use the ref skeleton so we'll only see the bones that are actually part of the mesh
	const TArray<FRigElementKey> Bones = ImportBones(InSkeletalMesh->GetRefSkeleton(), NAME_None, bReplaceExistingBones, bRemoveObsoleteBones, bSelectBones, bSetupUndo);
	for(const FRigElementKey& Bone : Bones)
	{
		if(FRigBaseElement* Element = Hierarchy->Find(Bone))
		{
			Element->CreatedAtInstructionIndex = InstructionIndex;
		}
	}

	// import the bones for the preview hierarchy
	// use the ref skeleton so we'll only see the bones that are actually part of the mesh
	const TArray<FRigElementKey> MeshSockets = ImportSocketsFromSkeletalMesh(InSkeletalMesh, NAME_None, bReplaceExistingBones, bRemoveObsoleteBones, bSelectBones, bSetupUndo);
	for(const FRigElementKey& MeshSocket : MeshSockets)
	{
		if(FRigBaseElement* Element = Hierarchy->Find(MeshSocket))
		{
			Element->CreatedAtInstructionIndex = InstructionIndex;
		}
	}

	// import the curves for the preview hierarchy
	// use the ref skeleton so we'll only see the curves that are actually part of the mesh
	const TArray<FRigElementKey> Curves = ImportCurvesFromSkeletalMesh(InSkeletalMesh, NAME_None, bSelectBones, bSetupUndo);
	for(const FRigElementKey& Curve : Curves)
	{
		if(FRigBaseElement* Element = Hierarchy->Find(Curve))
		{
			Element->CreatedAtInstructionIndex = InstructionIndex;
		}
	}

	// create a null to store controls under
	static const FRigElementKey ControlParentKey(TEXT("Controls"), ERigElementType::Null);
	if(!Hierarchy->Contains(ControlParentKey))
	{
		const FRigElementKey Null = AddNull(ControlParentKey.Name, FRigElementKey(), FTransform::Identity, true, false, false);
		if(FRigBaseElement* Element = Hierarchy->Find(Null))
		{
			Element->CreatedAtInstructionIndex = InstructionIndex;
		}
	}

	return Bones;
}

#if WITH_EDITOR

USkeletalMesh* URigHierarchyController::GetSkeletalMeshFromAssetPath(const FString& InAssetPath)
{
	UObject* AssetObject = StaticLoadObject(UObject::StaticClass(), NULL, *InAssetPath, NULL, LOAD_None, NULL);
	if(AssetObject == nullptr)
	{
		return nullptr;
	}

	if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(AssetObject))
	{
		return SkeletalMesh;
	}

	return nullptr;
}

USkeleton* URigHierarchyController::GetSkeletonFromAssetPath(const FString& InAssetPath)
{
	UObject* AssetObject = StaticLoadObject(UObject::StaticClass(), NULL, *InAssetPath, NULL, LOAD_None, NULL);
	if(AssetObject == nullptr)
	{
		return nullptr;
    }

	if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(AssetObject))
	{
		return SkeletalMesh->GetSkeleton();
	}

	if(USkeleton* Skeleton = Cast<USkeleton>(AssetObject))
	{
		return Skeleton;
	}

	return nullptr;
}

#endif

void URigHierarchyController::UpdateComponentsOnHierarchyKeyChange(const TArray<TPair<FRigHierarchyKey, FRigHierarchyKey>>& InKeyMap, bool bSetupUndoRedo)
{
	URigHierarchy* Hierarchy = GetHierarchy();

	// let all components know that their content may have changed
	const int32 NumComponents = Hierarchy->NumComponents();
	for(int32 ComponentIndex = 0; ComponentIndex < NumComponents; ComponentIndex++)
	{
		const FRigBaseComponent* CurrentComponent = Hierarchy->GetComponent(ComponentIndex);
		if(CurrentComponent->IsProcedural())
		{
			continue;
		}

		const FRigComponentState OldState = CurrentComponent->GetState();
		FStructOnScope StructOnScope(CurrentComponent->GetScriptStruct());
		FRigBaseComponent* TempComponent = reinterpret_cast<FRigBaseComponent*>(StructOnScope.GetStructMemory());
		TempComponent->SetState(OldState);

		for(const TPair<FRigHierarchyKey, FRigHierarchyKey>& Pair : InKeyMap)
		{
			TempComponent->OnRigHierarchyKeyChanged(Pair.Key, Pair.Value);
		}

		const FRigComponentState NewState = TempComponent->GetState();
		if(NewState != OldState)
		{
			SetComponentState(CurrentComponent->GetKey(), NewState, bSetupUndoRedo);
		}
	}
}

TArray<FRigElementKey> URigHierarchyController::ImportCurves(UAnimCurveMetaData* InAnimCurvesMetadata, FName InNameSpace, bool bSetupUndo)
{
	TArray<FRigElementKey> Keys;
	if (InAnimCurvesMetadata == nullptr)
	{
		return Keys;
	}

	if(!IsValid())
	{
		return Keys;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

	InAnimCurvesMetadata->ForEachCurveMetaData([this, Hierarchy, InNameSpace, &Keys, bSetupUndo](const FName& InCurveName, const FCurveMetaData& InMetaData)
	{
		FName Name = InCurveName;
		if (!InNameSpace.IsNone())
		{
			Name = *FString::Printf(TEXT("%s::%s"), *InNameSpace.ToString(), *InCurveName.ToString());
		}

		const FRigElementKey ExpectedKey(Name, ERigElementType::Curve);
		if(Hierarchy->Contains(ExpectedKey))
		{
			Keys.Add(ExpectedKey);
			return;
		}
		
		const FRigElementKey CurveKey = AddCurve(Name, 0.f, bSetupUndo);
		Keys.Add(FRigElementKey(Name, ERigElementType::Curve));
	});

	return Keys;
}

TArray<FRigElementKey> URigHierarchyController::ImportCurves(USkeleton* InSkeleton, FName InNameSpace,
                                                             bool bSelectCurves, bool bSetupUndo, bool bPrintPythonCommand)
{
	TArray<FRigElementKey> Keys;
	if (InSkeleton == nullptr)
	{
		return Keys;
	}

	if(!IsValid())
	{
		return Keys;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Import Curves", "Import Curves"));
		Hierarchy->Modify();
	}
#endif

	Keys.Append(ImportCurves(InSkeleton->GetAssetUserData<UAnimCurveMetaData>(), InNameSpace, bSetupUndo));

	if(bSelectCurves)
	{
		SetSelection(Keys);
	}
	
#if WITH_EDITOR
	if (!Keys.IsEmpty() && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.import_curves_from_asset('%s', '%s', %s)"),
				*InSkeleton->GetPathName(),
				*InNameSpace.ToString(),
				(bSelectCurves) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	return Keys;
}

TArray<FRigElementKey> URigHierarchyController::ImportCurvesFromSkeletalMesh(USkeletalMesh* InSkeletalMesh, FName InNameSpace,
	bool bSelectCurves, bool bSetupUndo, bool bPrintPythonCommand)
{
	TArray<FRigElementKey> Keys;
	if(InSkeletalMesh == nullptr)
	{
		return Keys;
	}

	if(!IsValid())
	{
		return Keys;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Import Curves", "Import Curves"));
		Hierarchy->Modify();
	}
#endif

	Keys.Append(ImportCurves(InSkeletalMesh->GetSkeleton(), InNameSpace, false, bSetupUndo, false));
	Keys.Append(ImportCurves(InSkeletalMesh->GetAssetUserData<UAnimCurveMetaData>(), InNameSpace, bSetupUndo));
	
	if(bSelectCurves)
	{
		SetSelection(Keys);
	}
	
#if WITH_EDITOR
	if (!Keys.IsEmpty() && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.import_curves_from_asset('%s', '%s', %s)"),
				*InSkeletalMesh->GetPathName(),
				*InNameSpace.ToString(),
				(bSelectCurves) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	return Keys;
}

FString URigHierarchyController::ExportSelectionToText() const
{
	if(!IsValid())
	{
		return FString();
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	return ExportToText(Hierarchy->GetSelectedKeys());
}

FString URigHierarchyController::ExportToText(TArray<FRigElementKey> InKeys) const
{
	if(!IsValid() || InKeys.IsEmpty())
	{
		return FString();
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	Hierarchy->ComputeAllTransforms();

	// sort the keys by traversal order
	TArray<FRigElementKey> Keys = Hierarchy->SortKeys(InKeys);

	FRigHierarchyCopyPasteContent Data;
	for (const FRigElementKey& Key : Keys)
	{
		FRigBaseElement* Element = Hierarchy->Find(Key);
		if(Element == nullptr)
		{
			continue;
		}

		FRigHierarchyCopyPasteContentPerElement PerElementData;
		PerElementData.Key = Key;
		const TArray<FRigElementKey> ParentKeys = Hierarchy->GetParents(Key);
		PerElementData.Parents.Reserve(ParentKeys.Num());
		for(const FRigElementKey& ParentKey : ParentKeys)
		{
			PerElementData.Parents.Emplace(ParentKey, Hierarchy->GetDisplayLabelForParent(Key, ParentKey));
		}

		if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(Element))
		{
			ensure(PerElementData.Parents.Num() == MultiParentElement->ParentConstraints.Num());

			for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
			{
				PerElementData.ParentWeights.Add(ParentConstraint.Weight);
			}
		}
		else
		{
			PerElementData.ParentWeights.SetNumZeroed(PerElementData.Parents.Num());
			if(PerElementData.ParentWeights.Num() > 0)
			{
				PerElementData.ParentWeights[0] = 1.f;
			}
		}

		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
		{
			PerElementData.Poses.Add(Hierarchy->GetTransform(TransformElement, ERigTransformType::InitialLocal));
			PerElementData.Poses.Add(Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentLocal));
			PerElementData.Poses.Add(Hierarchy->GetTransform(TransformElement, ERigTransformType::InitialGlobal));
			PerElementData.Poses.Add(Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentGlobal));
			PerElementData.DirtyStates.Add(TransformElement->GetDirtyState().GetDirtyFlag(ERigTransformType::InitialLocal));
			PerElementData.DirtyStates.Add(TransformElement->GetDirtyState().GetDirtyFlag(ERigTransformType::CurrentLocal));
			PerElementData.DirtyStates.Add(TransformElement->GetDirtyState().GetDirtyFlag(ERigTransformType::InitialGlobal));
			PerElementData.DirtyStates.Add(TransformElement->GetDirtyState().GetDirtyFlag(ERigTransformType::CurrentGlobal));
		}
		
		if (FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			PerElementData.Poses.Add(Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialLocal));
			PerElementData.Poses.Add(Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentLocal));
			PerElementData.Poses.Add(Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialGlobal));
			PerElementData.Poses.Add(Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentGlobal));
			
			PerElementData.Poses.Add(Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal));
			PerElementData.Poses.Add(Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal));
			PerElementData.Poses.Add(Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialGlobal));
			PerElementData.Poses.Add(Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentGlobal));
		}

		switch (Key.Type)
		{
			case ERigElementType::Bone:
			{
				FRigBoneElement DefaultElement;
				FRigBoneElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			case ERigElementType::Control:
			{
				FRigControlElement DefaultElement;
				FRigControlElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			case ERigElementType::Null:
			{
				FRigNullElement DefaultElement;
				FRigNullElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			case ERigElementType::Curve:
			{
				FRigCurveElement DefaultElement;
				FRigCurveElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			case ERigElementType::Reference:
			{
				FRigReferenceElement DefaultElement;
				FRigReferenceElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			case ERigElementType::Connector:
			{
				FRigConnectorElement DefaultElement;
				FRigConnectorElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			case ERigElementType::Socket:
			{
				FRigSocketElement DefaultElement;
				FRigSocketElement::StaticStruct()->ExportText(PerElementData.Content, Element, &DefaultElement, nullptr, PPF_None, nullptr);
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		Data.Elements.Add(PerElementData);
	}

	FString ExportedText;
	FRigHierarchyCopyPasteContent DefaultContent;
	FRigHierarchyCopyPasteContent::StaticStruct()->ExportText(ExportedText, &Data, &DefaultContent, nullptr, PPF_None, nullptr);
	return ExportedText;
}

TArray<FRigElementKey> URigHierarchyController::ImportFromText(FString InContent, bool bReplaceExistingElements, bool bSelectNewElements, bool bSetupUndo, bool bPrintPythonCommands)
{
	return ImportFromText(InContent, ERigElementType::All, bReplaceExistingElements, bSelectNewElements, bSetupUndo, bPrintPythonCommands);
}

TArray<FRigElementKey> URigHierarchyController::ImportFromText(FString InContent, ERigElementType InAllowedTypes, bool bReplaceExistingElements, bool bSelectNewElements, bool bSetupUndo, bool bPrintPythonCommands)
{
	const TGuardValue<bool> CycleCheckGuard(bSuspendParentCycleCheck, true);

	TArray<FRigElementKey> PastedKeys;
	if(!IsValid())
	{
		return PastedKeys;
	}

	FRigHierarchyCopyPasteContent Data;
	FRigHierarchyImportErrorContext ErrorPipe;
	FRigHierarchyCopyPasteContent::StaticStruct()->ImportText(*InContent, &Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigHierarchyCopyPasteContent::StaticStruct()->GetName(), true);
	if (ErrorPipe.NumErrors > 0)
	{
		return PastedKeys;
	}

	if (Data.Elements.Num() == 0)
	{
		// check if this is a copy & paste buffer from pre-5.0
		if(Data.Contents.Num() > 0)
		{
			const int32 OriginalNumElements = Data.Elements.Num();
			for (int32 i=0; i<Data.Types.Num(); )
			{
				if (((uint8)InAllowedTypes & (uint8)Data.Types[i]) == 0)
				{
					Data.Contents.RemoveAt(i);
					Data.Types.RemoveAt(i);
					Data.LocalTransforms.RemoveAt(i);
					Data.GlobalTransforms.RemoveAt(i);
					continue;
				}
				++i;
			}
			if (OriginalNumElements > Data.Types.Num())
			{
				ReportAndNotifyErrorf(TEXT("Some elements were not allowed to be pasted."));
			}
			FRigHierarchyContainer OldHierarchy;
			if(OldHierarchy.ImportFromText(Data).Num() > 0)
			{
				return ImportFromHierarchyContainer(OldHierarchy, true);
			}
		}
		
		return PastedKeys;
	}

	const int32 OriginalNumElements = Data.Elements.Num();
	Data.Elements = Data.Elements.FilterByPredicate([InAllowedTypes](const FRigHierarchyCopyPasteContentPerElement& Element)
	{
		return ((uint8)InAllowedTypes & (uint8)Element.Key.Type) != 0;
	});
	if (OriginalNumElements > Data.Elements.Num())
	{
		ReportAndNotifyErrorf(TEXT("Some elements were not allowed to be pasted."));
	}

	URigHierarchy* Hierarchy = GetHierarchy();

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Elements", "Add Elements"));
		Hierarchy->Modify();
	}
#endif

	TMap<FRigElementKey, FRigElementKey> KeyMap;
	for(FRigBaseElement* Element : *Hierarchy)
	{
		KeyMap.Add(Element->GetKey(), Element->GetKey());
	}
	TArray<FRigElementKey> PreviouslyExistingKeys;

	FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

	for(const FRigHierarchyCopyPasteContentPerElement& PerElementData : Data.Elements)
	{
		ErrorPipe.NumErrors = 0;

		FRigBaseElement* NewElement = nullptr;

		switch (PerElementData.Key.Type)
		{
			case ERigElementType::Bone:
			{
				NewElement = MakeElement<FRigBoneElement>(true, true);
				FRigBoneElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigBoneElement::StaticStruct()->GetName(), true);
				CastChecked<FRigBoneElement>(NewElement)->BoneType = ERigBoneType::User;
				break;
			}
			case ERigElementType::Null:
			{
				NewElement = MakeElement<FRigNullElement>(true, true);
				FRigNullElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigNullElement::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Control:
			{
				NewElement = MakeElement<FRigControlElement>(true, true);
				FRigControlElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigControlElement::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Curve:
			{
				NewElement = MakeElement<FRigCurveElement>(true, true);
				FRigCurveElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigCurveElement::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Reference:
			{
				NewElement = MakeElement<FRigReferenceElement>(true, true);
				FRigReferenceElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigReferenceElement::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Connector:
			{
				NewElement = MakeElement<FRigConnectorElement>(true, true);
				FRigConnectorElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigConnectorElement::StaticStruct()->GetName(), true);
				break;
			}
			case ERigElementType::Socket:
			{
				NewElement = MakeElement<FRigSocketElement>(true, true);
				FRigSocketElement::StaticStruct()->ImportText(*PerElementData.Content, NewElement, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigSocketElement::StaticStruct()->GetName(), true);
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		if (ErrorPipe.NumErrors > 0)
		{
#if WITH_EDITOR
			if (bSetupUndo)
			{
				TransactionPtr->Cancel();
			}
#endif
			Hierarchy->DestroyElement(NewElement);
			return PastedKeys;
		}

		check(NewElement);
		NewElement->Key = PerElementData.Key;

		if(bReplaceExistingElements)
		{
			if(FRigBaseElement* ExistingElement = Hierarchy->Find(NewElement->GetKey()))
			{
				// as we have created a new element in the same hierarchy, we have to update the storage link of the existing element
				ExistingElement->LinkStorage(Hierarchy->ElementTransforms.GetStorage(), Hierarchy->ElementDirtyStates.GetStorage(), Hierarchy->ElementCurves.GetStorage());

				ExistingElement->CopyPose(NewElement, true, true, false);

				if(FRigControlElement* ControlElement = Cast<FRigControlElement>(ExistingElement))
				{
					Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
					Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal);
					ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
					ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				}
				
				TArray<FRigElementKey> CurrentParents = Hierarchy->GetParents(NewElement->GetKey());

				bool bUpdateParents = CurrentParents.Num() != PerElementData.Parents.Num();
				if(!bUpdateParents)
				{
					for(const FRigElementKey& CurrentParent : CurrentParents)
					{
						if(!PerElementData.Parents.Contains(CurrentParent))
						{
							bUpdateParents = true;
							break;
						}
					}
				}

				if(bUpdateParents)
				{
					RemoveAllParents(ExistingElement->GetKey(), true, bSetupUndo);

					for(const FRigElementKeyWithLabel& NewParent : PerElementData.Parents)
					{
						AddParent(ExistingElement->GetKey(), NewParent.Key, 0.f, true, NewParent.Label, bSetupUndo);
					}
				}
				
				for(int32 ParentIndex = 0; ParentIndex < PerElementData.ParentWeights.Num(); ParentIndex++)
				{
					Hierarchy->SetParentWeight(ExistingElement, ParentIndex, PerElementData.ParentWeights[ParentIndex], true, true);
					Hierarchy->SetParentWeight(ExistingElement, ParentIndex, PerElementData.ParentWeights[ParentIndex], false, true);
				}

				PastedKeys.Add(ExistingElement->GetKey());
				PreviouslyExistingKeys.Add(ExistingElement->GetKey());

				Hierarchy->DestroyElement(NewElement);
				continue;
			}
		}

		const FName DesiredName = NewElement->Key.Name;
		NewElement->Key.Name = GetSafeNewName(DesiredName, NewElement->Key.Type);
		AddElement(NewElement, nullptr, true, DesiredName);

		KeyMap.FindOrAdd(PerElementData.Key) = NewElement->Key;
	}

	Hierarchy->UpdateElementStorage();
	
	for(const FRigHierarchyCopyPasteContentPerElement& PerElementData : Data.Elements)
	{
		if(PreviouslyExistingKeys.Contains(PerElementData.Key))
		{
			continue;
		}
		
		FRigElementKey MappedKey = KeyMap.FindChecked(PerElementData.Key);
		FRigBaseElement* NewElement = Hierarchy->FindChecked(MappedKey);

		for(const FRigElementKeyWithLabel& OriginalParent : PerElementData.Parents)
		{
			FRigElementKey Parent = OriginalParent.Key;
			if(const FRigElementKey* RemappedParent = KeyMap.Find(Parent))
			{
				Parent = *RemappedParent;
			}

			AddParent(NewElement->GetKey(), Parent, 0.f, true, OriginalParent.Label, bSetupUndo);
		}
		
		for(int32 ParentIndex = 0; ParentIndex < PerElementData.ParentWeights.Num(); ParentIndex++)
		{
			Hierarchy->SetParentWeight(NewElement, ParentIndex, PerElementData.ParentWeights[ParentIndex], true, true);
			Hierarchy->SetParentWeight(NewElement, ParentIndex, PerElementData.ParentWeights[ParentIndex], false, true);
		}

		PastedKeys.AddUnique(NewElement->GetKey());
	}

	for(const FRigHierarchyCopyPasteContentPerElement& PerElementData : Data.Elements)
	{
		FRigElementKey MappedKey = KeyMap.FindChecked(PerElementData.Key);
		FRigBaseElement* Element = Hierarchy->FindChecked(MappedKey);

		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
		{
			if(PerElementData.Poses.Num() >= 2)
			{
				Hierarchy->SetTransform(TransformElement, PerElementData.Poses[0], ERigTransformType::InitialLocal, true, true);
				Hierarchy->SetTransform(TransformElement, PerElementData.Poses[1], ERigTransformType::CurrentLocal, true, true);
			}
		}
		
		if (FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			if (PerElementData.Poses.Num() >= 6)
			{
				Hierarchy->SetControlOffsetTransform(ControlElement, PerElementData.Poses[4], ERigTransformType::InitialLocal, true, true);
				Hierarchy->SetControlOffsetTransform(ControlElement, PerElementData.Poses[5], ERigTransformType::CurrentLocal, true, true);
			}
			
			if (PerElementData.Poses.Num() >= 10)
			{
				Hierarchy->SetControlShapeTransform(ControlElement, PerElementData.Poses[8], ERigTransformType::InitialLocal, true, true);
				Hierarchy->SetControlShapeTransform(ControlElement, PerElementData.Poses[9], ERigTransformType::CurrentLocal, true, true);
			}
		}
	}
	
#if WITH_EDITOR
	TransactionPtr.Reset();
#endif

	if(bSelectNewElements)
	{
		SetSelection(PastedKeys);
	}

#if WITH_EDITOR
	if (bPrintPythonCommands && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			FString PythonContent = InContent.Replace(TEXT("\\\""), TEXT("\\\\\""));
		
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.import_from_text('%s', %s, %s)"),
				*PythonContent,
				(bReplaceExistingElements) ? TEXT("True") : TEXT("False"),
				(bSelectNewElements) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	return PastedKeys;
}

TArray<FRigElementKey> URigHierarchyController::ImportFromHierarchyContainer(const FRigHierarchyContainer& InContainer, bool bIsCopyAndPaste)
{
	const TGuardValue<bool> CycleCheckGuard(bSuspendParentCycleCheck, true);
	
	URigHierarchy* Hierarchy = GetHierarchy();

	TMap<FRigElementKey, FRigElementKey> KeyMap;;
	FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

	for(const FRigBone& Bone : InContainer.BoneHierarchy)
	{
		const FRigElementKey OriginalParentKey = Bone.GetParentElementKey(true);
		const FRigElementKey* ParentKey = nullptr;
		if(OriginalParentKey.IsValid())
		{
			ParentKey = KeyMap.Find(OriginalParentKey);
		}
		if(ParentKey == nullptr)
		{
			ParentKey = &OriginalParentKey;
		}

		const FRigElementKey Key = AddBone(Bone.Name, *ParentKey, Bone.InitialTransform, true, bIsCopyAndPaste ? ERigBoneType::User : Bone.Type, false);
		KeyMap.Add(Bone.GetElementKey(), Key);
	}
	for(const FRigSpace& Space : InContainer.SpaceHierarchy)
	{
		const FRigElementKey Key = AddNull(Space.Name, FRigElementKey(), Space.InitialTransform, false, false);
		KeyMap.Add(Space.GetElementKey(), Key);
	}
	for(const FRigControl& Control : InContainer.ControlHierarchy)
	{
		FRigControlSettings Settings;
		Settings.ControlType = Control.ControlType;
		Settings.DisplayName = Control.DisplayName;
		Settings.PrimaryAxis = Control.PrimaryAxis;
		Settings.bIsCurve = Control.bIsCurve;
		Settings.SetAnimationTypeFromDeprecatedData(Control.bAnimatable, Control.bGizmoEnabled);
		Settings.SetupLimitArrayForType(Control.bLimitTranslation, Control.bLimitRotation, Control.bLimitScale);
		Settings.bDrawLimits = Control.bDrawLimits;
		Settings.MinimumValue = Control.MinimumValue;
		Settings.MaximumValue = Control.MaximumValue;
		Settings.bShapeVisible = Control.bGizmoVisible;
		Settings.ShapeName = Control.GizmoName;
		Settings.ShapeColor = Control.GizmoColor;
		Settings.ControlEnum = Control.ControlEnum;
		Settings.bGroupWithParentControl = Settings.IsAnimatable() && (
			Settings.ControlType == ERigControlType::Bool ||
			Settings.ControlType == ERigControlType::Float ||
			Settings.ControlType == ERigControlType::ScaleFloat ||
			Settings.ControlType == ERigControlType::Integer ||
			Settings.ControlType == ERigControlType::Vector2D
		);

		if(Settings.ShapeName == FRigControl().GizmoName)
		{
			Settings.ShapeName = FControlRigShapeDefinition().ShapeName; 
		}

		FRigControlValue InitialValue = Control.InitialValue;

#if WITH_EDITORONLY_DATA
		if(!InitialValue.IsValid())
		{
			InitialValue.SetFromTransform(InitialValue.Storage_DEPRECATED, Settings.ControlType, Settings.PrimaryAxis);
		}
#endif
		
		const FRigElementKey Key = AddControl(
			Control.Name,
			FRigElementKey(),
			Settings,
			InitialValue,
			Control.OffsetTransform,
			Control.GizmoTransform,
			false);

		KeyMap.Add(Control.GetElementKey(), Key);
	}
	
	for(const FRigCurve& Curve : InContainer.CurveContainer)
	{
		const FRigElementKey Key = AddCurve(Curve.Name, Curve.Value, false);
		KeyMap.Add(Curve.GetElementKey(), Key);
	}

	for(const FRigSpace& Space : InContainer.SpaceHierarchy)
	{
		const FRigElementKey SpaceKey = KeyMap.FindChecked(Space.GetElementKey());
		const FRigElementKey OriginalParentKey = Space.GetParentElementKey();
		if(OriginalParentKey.IsValid())
		{
			FRigElementKey ParentKey;
			if(const FRigElementKey* ParentKeyPtr = KeyMap.Find(OriginalParentKey))
			{
				ParentKey = *ParentKeyPtr;
			}
			SetParent(SpaceKey, ParentKey, false, false);
		}
	}

	for(const FRigControl& Control : InContainer.ControlHierarchy)
	{
		const FRigElementKey ControlKey = KeyMap.FindChecked(Control.GetElementKey());
		FRigElementKey OriginalParentKey = Control.GetParentElementKey();
		const FRigElementKey SpaceKey = Control.GetSpaceElementKey();
		OriginalParentKey = SpaceKey.IsValid() ? SpaceKey : OriginalParentKey;
		if(OriginalParentKey.IsValid())
		{
			FRigElementKey ParentKey;
			if(const FRigElementKey* ParentKeyPtr = KeyMap.Find(OriginalParentKey))
			{
				ParentKey = *ParentKeyPtr;
			}
			SetParent(ControlKey, ParentKey, false, false);
		}
	}

#if WITH_EDITOR
	if(!IsRunningCommandlet()) // don't show warnings like this if we are cooking
	{
		for(const TPair<FRigElementKey, FRigElementKey>& Pair: KeyMap)
		{
			if(Pair.Key != Pair.Value)
			{
				check(Pair.Key.Type == Pair.Value.Type);
				const FText TypeLabel = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Pair.Key.Type);
				ReportWarningf(TEXT("%s '%s' was renamed to '%s' during load (fixing invalid name)."), *TypeLabel.ToString(), *Pair.Key.Name.ToString(), *Pair.Value.Name.ToString());
			}
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();

	TArray<FRigElementKey> AddedKeys;
	KeyMap.GenerateValueArray(AddedKeys);
	return AddedKeys;
}

#if WITH_EDITOR
TArray<FString> URigHierarchyController::GeneratePythonCommands()
{
	URigHierarchy* Hierarchy = GetHierarchy();

	TArray<FString> Commands;
	Hierarchy->Traverse([&](FRigBaseElement* Element, bool& bContinue)
	{
		Commands.Append(GetAddElementPythonCommands(Element));
		
		bContinue = true;
		return;
	});

	return Commands;
}

TArray<FString> URigHierarchyController::GetAddElementPythonCommands(FRigBaseElement* Element) const
{
	if(FRigBoneElement* BoneElement = Cast<FRigBoneElement>(Element))
	{
		return GetAddBonePythonCommands(BoneElement);		
	}
	else if(FRigNullElement* NullElement = Cast<FRigNullElement>(Element))
	{
		return GetAddNullPythonCommands(NullElement);
	}
	else if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
	{
		return GetAddControlPythonCommands(ControlElement);
	}
	else if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Element))
	{
		return GetAddCurvePythonCommands(CurveElement);
	}
	else if(FRigReferenceElement* ReferenceElement = Cast<FRigReferenceElement>(Element))
	{
		ensure(false);
	}
	else if(FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Element))
	{
		return GetAddConnectorPythonCommands(ConnectorElement);
	}
	else if(FRigSocketElement* SocketElement = Cast<FRigSocketElement>(Element))
	{
		return GetAddSocketPythonCommands(SocketElement);
	}
	return TArray<FString>();
}

TArray<FString> URigHierarchyController::GetAddBonePythonCommands(FRigBoneElement* Bone) const
{
	TArray<FString> Commands;
	if (!Bone)
	{
		return Commands;
	}
	
	FString TransformStr = RigVMPythonUtils::TransformToPythonString(Bone->GetTransform().Initial.Local.Get());
	FString ParentKeyStr = "''";
	if (Bone->ParentElement)
	{
		ParentKeyStr = Bone->ParentElement->GetKey().ToPythonString();
	}
	
	// AddBone(FName InName, FRigElementKey InParent, FTransform InTransform, bool bTransformInGlobal = true, ERigBoneType InBoneType = ERigBoneType::User, bool bSetupUndo = false);
	Commands.Add(FString::Printf(TEXT("hierarchy_controller.add_bone('%s', %s, %s, False, %s)"),
		*Bone->GetName(),
		*ParentKeyStr,
		*TransformStr,
		*RigVMPythonUtils::EnumValueToPythonString<ERigBoneType>((int64)Bone->BoneType)
	));

	return Commands;
}

TArray<FString> URigHierarchyController::GetAddNullPythonCommands(FRigNullElement* Null) const
{
	FString TransformStr = RigVMPythonUtils::TransformToPythonString(Null->GetTransform().Initial.Local.Get());

	FString ParentKeyStr = "''";
	if (Null->ParentConstraints.Num() > 0)
	{
		ParentKeyStr = Null->ParentConstraints[0].ParentElement->GetKey().ToPythonString();		
	}
		
	// AddNull(FName InName, FRigElementKey InParent, FTransform InTransform, bool bTransformInGlobal = true, bool bSetupUndo = false);
	return {FString::Printf(TEXT("hierarchy_controller.add_null('%s', %s, %s, False)"),
		*Null->GetName(),
		*ParentKeyStr,
		*TransformStr)};
}

TArray<FString> URigHierarchyController::GetAddControlPythonCommands(FRigControlElement* Control) const
{
	TArray<FString> Commands;
	FString TransformStr = RigVMPythonUtils::TransformToPythonString(Control->GetTransform().Initial.Local.Get());

	FString ParentKeyStr = "''";
	if (Control->ParentConstraints.Num() > 0)
	{
		ParentKeyStr = Control->ParentConstraints[0].ParentElement->GetKey().ToPythonString();
	}

	FRigControlSettings& Settings = Control->Settings;
	FString SettingsStr;
	{
		FString ControlNamePythonized = RigVMPythonUtils::PythonizeName(Control->GetName());
		SettingsStr = FString::Printf(TEXT("control_settings_%s"),
			*ControlNamePythonized);
			
		Commands.Append(URigHierarchy::ControlSettingsToPythonCommands(Settings, SettingsStr));	
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigControlValue Value = Hierarchy->GetControlValue(Control->GetKey(), ERigControlValueType::Initial);
	FString ValueStr = Value.ToPythonString(Settings.ControlType);
	
	// AddControl(FName InName, FRigElementKey InParent, FRigControlSettings InSettings, FRigControlValue InValue, bool bSetupUndo = true);
	Commands.Add(FString::Printf(TEXT("hierarchy_controller.add_control('%s', %s, %s, %s)"),
		*Control->GetName(),
		*ParentKeyStr,
		*SettingsStr,
		*ValueStr));

	Commands.Append(GetSetControlShapeTransformPythonCommands(Control, Control->GetShapeTransform().Initial.Local.Get(), true));
	Commands.Append(GetSetControlValuePythonCommands(Control, Settings.MinimumValue, ERigControlValueType::Minimum));
	Commands.Append(GetSetControlValuePythonCommands(Control, Settings.MaximumValue, ERigControlValueType::Maximum));
	Commands.Append(GetSetControlOffsetTransformPythonCommands(Control, Control->GetOffsetTransform().Initial.Local.Get(), true, true));
	Commands.Append(GetSetControlValuePythonCommands(Control, Value, ERigControlValueType::Current));

	return Commands;
}

TArray<FString> URigHierarchyController::GetAddCurvePythonCommands(FRigCurveElement* Curve) const
{
	URigHierarchy* Hierarchy = GetHierarchy();

	// FRigElementKey AddCurve(FName InName, float InValue = 0.f, bool bSetupUndo = true);
	return {FString::Printf(TEXT("hierarchy_controller.add_curve('%s', %f)"),
		*Curve->GetName(),
		Hierarchy->GetCurveValue(Curve))};
}

TArray<FString> URigHierarchyController::GetAddConnectorPythonCommands(FRigConnectorElement* Connector) const
{
	TArray<FString> Commands;

	FRigConnectorSettings& Settings = Connector->Settings;
	FString SettingsStr;
	{
		FString ConnectorNamePythonized = RigVMPythonUtils::PythonizeName(Connector->GetName());
		SettingsStr = FString::Printf(TEXT("connector_settings_%s"),
			*ConnectorNamePythonized);
			
		Commands.Append(URigHierarchy::ConnectorSettingsToPythonCommands(Settings, SettingsStr));	
	}

	// AddConnector(FName InName, FRigConnectorSettings InSettings = FRigConnectorSettings(), bool bSetupUndo = false);
	Commands.Add(FString::Printf(TEXT("hierarchy_controller.add_connector('%s', %s)"),
		*Connector->GetName(),
		*SettingsStr
	));

	return Commands;
}

TArray<FString> URigHierarchyController::GetAddSocketPythonCommands(FRigSocketElement* Socket) const
{
	TArray<FString> Commands;
	FString TransformStr = RigVMPythonUtils::TransformToPythonString(Socket->GetTransform().Initial.Local.Get());

	FString ParentKeyStr = "''";
	if (Socket->ParentElement)
	{
		ParentKeyStr = Socket->ParentElement->GetKey().ToPythonString();
	}

	const URigHierarchy* CurrentHierarchy = GetHierarchy();

	// AddSocket(FName InName, FRigElementKey InParent, FTransform InTransform, bool bTransformInGlobal = true, FLinearColor Color, FString Description, bool bSetupUndo = false);
	Commands.Add(FString::Printf(TEXT("hierarchy_controller.add_socket('%s', %s, %s, False, %s, '%s')"),
		*Socket->GetName(),
		*ParentKeyStr,
		*TransformStr,
		*RigVMPythonUtils::LinearColorToPythonString(Socket->GetColor(CurrentHierarchy)),
		*Socket->GetDescription(CurrentHierarchy)
	));

	return Commands;
}

TArray<FString> URigHierarchyController::GetSetControlValuePythonCommands(const FRigControlElement* Control, const FRigControlValue& Value,
                                                                          const ERigControlValueType& Type) const
{
	return {FString::Printf(TEXT("hierarchy.set_control_value(%s, %s, %s)"),
		*Control->GetKey().ToPythonString(),
		*Value.ToPythonString(Control->Settings.ControlType),
		*RigVMPythonUtils::EnumValueToPythonString<ERigControlValueType>((int64)Type))};
}

TArray<FString> URigHierarchyController::GetSetControlOffsetTransformPythonCommands(const FRigControlElement* Control,
                                                                                    const FTransform& Offset, bool bInitial, bool bAffectChildren) const
{
	//SetControlOffsetTransform(FRigElementKey InKey, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false)
	return {FString::Printf(TEXT("hierarchy.set_control_offset_transform(%s, %s, %s, %s)"),
		*Control->GetKey().ToPythonString(),
		*RigVMPythonUtils::TransformToPythonString(Offset),
		bInitial ? TEXT("True") : TEXT("False"),
		bAffectChildren ? TEXT("True") : TEXT("False"))};
}

TArray<FString> URigHierarchyController::GetSetControlShapeTransformPythonCommands(const FRigControlElement* Control,
	const FTransform& InTransform, bool bInitial) const
{
	//SetControlShapeTransform(FRigElementKey InKey, FTransform InTransform, bool bInitial = false, bool bSetupUndo = false)
	return {FString::Printf(TEXT("hierarchy.set_control_shape_transform(%s, %s, %s)"),
		*Control->GetKey().ToPythonString(),
		*RigVMPythonUtils::TransformToPythonString(InTransform),
		bInitial ? TEXT("True") : TEXT("False"))};
}

TArray<FString> URigHierarchyController::GetAddComponentPythonCommands(const FRigBaseComponent* Component) const
{
	TArray<FString> Commands;

	const FString ElementKeyStr = Component->GetElement()->GetKey().ToPythonString();

	FString ContentStr;
	Component->GetScriptStruct()->ExportText(ContentStr, Component, Component, nullptr, PPF_None, nullptr);
	
	// AddComponent(UScriptStruct* InComponentStruct, FName InName, FRigElementKey InElement, FString InContent);
	Commands.Add(FString::Printf(TEXT("hierarchy_controller.add_component(unreal.%s, '%s', %s, '%s')"),
		*Component->GetScriptStruct()->GetName(),
		*Component->GetName(),
		*ElementKeyStr,
		*ContentStr
	));

	return Commands;
}
#endif

void URigHierarchyController::Notify(ERigHierarchyNotification InNotifType, const FRigNotificationSubject& InSubject)
{
	if(!IsValid())
	{
		return;
	}
	if(bSuspendAllNotifications)
	{
		return;
	}
	if(bSuspendSelectionNotifications)
	{
		if(InNotifType == ERigHierarchyNotification::ElementSelected ||
			InNotifType == ERigHierarchyNotification::ElementDeselected)
		{
			return;
		}
	}	
	GetHierarchy()->Notify(InNotifType, InSubject);
}

void URigHierarchyController::HandleHierarchyModified(ERigHierarchyNotification InNotifType, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject) const
{
	if(bSuspendAllNotifications)
	{
		return;
	}
	ensure(IsValid());
	ensure(InHierarchy == GetHierarchy());
	ModifiedEvent.Broadcast(InNotifType, InHierarchy, InSubject);
}

bool URigHierarchyController::IsValid() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if(IsThisNotNull(this, "URigHierarchyController::IsValid") && IsValidChecked(this))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return ::IsValid(GetHierarchy());
	}
	return false;
}

FName URigHierarchyController::GetSafeNewName(const FName& InDesiredName, ERigElementType InElementType, bool bAllowNameSpace) const
{
	FRigName Name(InDesiredName);

	// remove potential namespaces from it
	if(!bAllowNameSpace)
	{
		const FRigHierarchyModulePath ModulePath(InDesiredName);
		if(ModulePath.IsValid())
		{
			Name.SetFName(ModulePath.GetElementFName());
		}
	}
	
	return GetHierarchy()->GetSafeNewName(Name, InElementType, bAllowNameSpace).GetFName();
}

int32 URigHierarchyController::AddElement(FRigBaseElement* InElementToAdd, FRigBaseElement* InFirstParent, bool bMaintainGlobalTransform, const FName& InDesiredName)
{
	ensure(IsValid());

	URigHierarchy* Hierarchy = GetHierarchy();
	UE::TScopeLock Lock(Hierarchy->ElementsLock);

	InElementToAdd->CachedNameString.Reset();
	InElementToAdd->SubIndex = Hierarchy->Num(InElementToAdd->Key.Type);
	InElementToAdd->SpawnIndex = Hierarchy->GetNextSpawnIndex();
	InElementToAdd->Index = Hierarchy->Elements.Add(InElementToAdd);
	Hierarchy->ElementsPerType[URigHierarchy::RigElementTypeToFlatIndex(InElementToAdd->GetKey().Type)].Add(InElementToAdd);
	Hierarchy->ElementIndexLookup.Add(InElementToAdd->Key, InElementToAdd->Index);
	Hierarchy->AllocateDefaultElementStorage(InElementToAdd, true);
	Hierarchy->IncrementTopologyVersion();

	FRigName DesiredName = InDesiredName;
	URigHierarchy::SanitizeName(DesiredName,true, Hierarchy->GetMaxNameLength());

	const FRigHierarchyModulePath ModulePath(InDesiredName.ToString());
	if(ModulePath.IsValid())
	{
		DesiredName = ModulePath.GetElementFName();
	}

	if(!InDesiredName.IsNone() &&
		!InElementToAdd->GetFName().IsEqual(DesiredName.GetFName(), ENameCase::CaseSensitive))
	{
		Hierarchy->SetNameMetadata(InElementToAdd->Key, URigHierarchy::DesiredNameMetadataName, DesiredName.GetFName());
		Hierarchy->SetRigElementKeyMetadata(InElementToAdd->Key, URigHierarchy::DesiredKeyMetadataName, FRigElementKey(DesiredName.GetFName(), InElementToAdd->Key.Type));
	}
	
	if(Hierarchy->HasExecuteContext())
	{
		const FControlRigExecuteContext& CRContext = Hierarchy->ExecuteContext->GetPublicData<FControlRigExecuteContext>();

		if(!CRContext.GetRigModulePrefix().IsEmpty())
		{
			if(InElementToAdd->GetName().StartsWith(CRContext.GetRigModulePrefix(), ESearchCase::IgnoreCase))
			{
				Hierarchy->SetNameMetadata(InElementToAdd->Key, URigHierarchy::ModuleMetadataName, *CRContext.GetRigModulePrefix().LeftChop(1));

				if(Hierarchy->ElementKeyRedirector)
				{
					Hierarchy->ElementKeyRedirector->Add(FRigElementKey(DesiredName.GetFName(), InElementToAdd->Key.Type), {InElementToAdd->Key}, Hierarchy);
				}
			}
		}
	}
	
	{
		const TGuardValue<bool> NotificationGuard(bSuspendAllNotifications, true);
		const TGuardValue<bool> CycleCheckGuard(bSuspendParentCycleCheck, true);
		SetParent(InElementToAdd, InFirstParent, bMaintainGlobalTransform);
	}

	if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InElementToAdd))
	{
		Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
		Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal);
		ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
		ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
	}

	// only notify once at the end
	Notify(ERigHierarchyNotification::ElementAdded, InElementToAdd);

	return InElementToAdd->Index;
}

bool URigHierarchyController::RemoveElement(FRigElementKey InElement, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Element = Hierarchy->Find(InElement);
	if(Element == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Element: '%s' not found."), *InElement.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Remove Element", "Remove Element"));
		Hierarchy->Modify();
	}
#endif

	const bool bRemoved = RemoveElement(Element);
	
#if WITH_EDITOR
	if(!bRemoved && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bRemoved && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.remove_element(%s)"),
				*InElement.ToPythonString()));
		}
	}
#endif

	return bRemoved;
}

bool URigHierarchyController::RemoveElement(FRigBaseElement* InElement)
{
	if(InElement == nullptr)
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();
	UE::TScopeLock Lock(Hierarchy->ElementsLock);

	// make sure this element is part of this hierarchy
	ensure(Hierarchy->FindChecked(InElement->Key) == InElement);

	// OwnedInstances may be 0 (batch member) or >1 (batch head) for elements allocated in
	// bulk by CopyHierarchy. Logical removal is allowed for all; memory is reclaimed later:
	//   == 1 : destroyed immediately after removal.
	//   == 0 : freed when the batch head is destroyed.
	//   >  1 : tracked in OrphanedBatchHeads, freed during Reset_Impl.

	// deselect if needed
	if(InElement->IsSelected())
	{
		SelectElement(InElement->GetKey(), false);
	}

	// if this is a transform element - make sure to allow dependents to store their global transforms
	if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(InElement))
	{
		FRigTransformElement::FElementsToDirtyArray PreviousElementsToDirty = TransformElement->ElementsToDirty; 
		for(const FRigTransformElement::FElementToDirty& ElementToDirty : PreviousElementsToDirty)
		{
			if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(ElementToDirty.Element))
			{
				if(SingleParentElement->ParentElement == InElement)
				{
					RemoveParent(SingleParentElement, InElement, true);
				}
			}
			else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(ElementToDirty.Element))
			{
				for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
				{
					if(ParentConstraint.ParentElement == InElement)
					{
						RemoveParent(MultiParentElement, InElement, true);
						break;
					}
				}
			}
		}
	}

	const int32 NumElementsRemoved = Hierarchy->Elements.Remove(InElement);
	ensure(NumElementsRemoved == 1);

	const int32 NumTypeElementsRemoved = Hierarchy->ElementsPerType[URigHierarchy::RigElementTypeToFlatIndex(InElement->GetKey().Type)].Remove(InElement);
	ensure(NumTypeElementsRemoved == 1);

	const int32 NumLookupsRemoved = Hierarchy->ElementIndexLookup.Remove(InElement->Key);
	ensure(NumLookupsRemoved == 1);
	for(TPair<FRigElementKey, int32>& Pair : Hierarchy->ElementIndexLookup)
	{
		if(Pair.Value > InElement->Index)
		{
			Pair.Value--;
		}
	}

	// update the indices of all other elements
	for (FRigBaseElement* RemainingElement : Hierarchy->Elements)
	{
		if(RemainingElement->Index > InElement->Index)
		{
			RemainingElement->Index--;
		}
	}

	if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InElement))
	{
		RemoveElementToDirty(SingleParentElement->ParentElement, InElement);
	}
	else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InElement))
	{
		for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
		{
			RemoveElementToDirty(ParentConstraint.ParentElement, InElement);
		}
	}

	if(InElement->SubIndex != INDEX_NONE)
	{
		for(FRigBaseElement* Element : Hierarchy->Elements)
		{
			if(Element->SubIndex > InElement->SubIndex && Element->GetType() == InElement->GetType())
			{
				Element->SubIndex--;
			}
		}
	}

	for(FRigBaseElement* Element : Hierarchy->Elements)
	{
		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			const int32 ExistingSpaceIndex = ControlElement->Settings.Customization.AvailableSpaces.IndexOfByKey(InElement->GetKey());
			if(ExistingSpaceIndex != INDEX_NONE)
			{
				ControlElement->Settings.Customization.AvailableSpaces.RemoveAt(ExistingSpaceIndex);
			}
			ControlElement->Settings.Customization.RemovedSpaces.Remove(InElement->GetKey());
			ControlElement->Settings.DrivenControls.Remove(InElement->GetKey());
		}
	}

	Hierarchy->DeallocateElementStorage(InElement);
	Hierarchy->IncrementTopologyVersion();

	Notify(ERigHierarchyNotification::ElementRemoved, InElement);
	if(Hierarchy->Num() == 0)
	{
		Notify(ERigHierarchyNotification::HierarchyReset, {});
	}

	if (InElement->OwnedInstances == 1)
	{
		// Individually-allocated element — destroy and free immediately.
		Hierarchy->DestroyElement(InElement);
	}
	else if (InElement->OwnedInstances > 1)
	{
		// Batch head removed from Elements. Track it so Reset_Impl can free the block
		// once all sibling batch members are also torn down.
		Hierarchy->OrphanedBatchHeads.Add(InElement);
	}
	// OwnedInstances == 0: non-owning batch member. Memory is owned by the batch head
	// (still in Elements or in OrphanedBatchHeads) and will be freed via its DestroyElement.

	Hierarchy->EnsureCacheValidity();

	return NumElementsRemoved == 1;
}

void URigHierarchyController::AddTransformElementInternal(
	FRigTransformElement* InNewTransformElement, FRigBaseElement* InParent,
	const FTransform& InInitialTransform, const bool bTransformInGlobal,
    const bool bMaintainGlobalTransform, const FName InName)
{
	URigHierarchy* Hierarchy = GetHierarchy();

	AddElement(InNewTransformElement, InParent, bMaintainGlobalTransform, InName);

	constexpr bool bAffectChildren = true, bNoUndo = false;
	if (bTransformInGlobal)
	{
		Hierarchy->SetTransform(InNewTransformElement, InInitialTransform, ERigTransformType::InitialGlobal, bAffectChildren, bNoUndo);
		Hierarchy->SetTransform(InNewTransformElement, InInitialTransform, ERigTransformType::CurrentGlobal, bAffectChildren, bNoUndo);
	}
	else
	{
		Hierarchy->SetTransform(InNewTransformElement, InInitialTransform, ERigTransformType::InitialLocal, bAffectChildren, bNoUndo);
		Hierarchy->SetTransform(InNewTransformElement, InInitialTransform, ERigTransformType::CurrentLocal, bAffectChildren, bNoUndo);
	}

	InNewTransformElement->GetTransform().Current = InNewTransformElement->GetTransform().Initial;
	InNewTransformElement->GetDirtyState().Current = InNewTransformElement->GetDirtyState().Initial;
}

FRigElementKey URigHierarchyController::RenameElement(FRigElementKey InElement, FName InName, bool bSetupUndo, bool bPrintPythonCommand, bool bClearSelection)
{
	if(!IsValid())
	{
		return FRigElementKey();
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Element = Hierarchy->Find(InElement);
	if(Element == nullptr)
	{
		ReportWarningf(TEXT("Cannot Rename Element: '%s' not found."), *InElement.ToString());
		return FRigElementKey();
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Rename Element", "Rename Element"));
		Hierarchy->Modify();
	}
#endif

	const bool bRenamed = RenameElement(Element, InName, bClearSelection, bSetupUndo);

#if WITH_EDITOR
	if(!bRenamed && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bRenamed && bClearSelection)
	{
		ClearSelection();
	}

	if (bRenamed && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(), 
				FString::Printf(TEXT("hierarchy_controller.rename_element(%s, '%s')"),
				*InElement.ToPythonString(),
				*InName.ToString()));
		}
	}
#endif

	return bRenamed ? Element->GetKey() : FRigElementKey();
}

bool URigHierarchyController::ReorderElement(FRigElementKey InElement, int32 InIndex, bool bSetupUndo,
	bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Element = Hierarchy->Find(InElement);
	if(Element == nullptr)
	{
		ReportWarningf(TEXT("Cannot Reorder Element: '%s' not found."), *InElement.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Reorder Element", "Reorder Element"));
		Hierarchy->Modify();
	}
#endif

	const bool bReordered = ReorderElement(Element, InIndex);

#if WITH_EDITOR
	if(!bReordered && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bReordered && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(), 
				FString::Printf(TEXT("hierarchy_controller.reorder_element(%s, %d)"),
				*InElement.ToPythonString(),
				InIndex));
		}
	}
#endif

	return bReordered;
}

FName URigHierarchyController::SetDisplayName(FRigElementKey InControl, FName InDisplayName, bool bRenameElement, bool bSetupUndo,
                                              bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return NAME_None;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InControl);
	if(ControlElement == nullptr)
	{
		ReportWarningf(TEXT("Cannot Rename Control: '%s' not found."), *InControl.ToString());
		return NAME_None;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Set Display Name on Control", "Set Display Name on Control"));
		Hierarchy->Modify();
	}
#endif

	const FName NewDisplayName = SetDisplayName(ControlElement, InDisplayName, bRenameElement);
	const bool bDisplayNameChanged = !NewDisplayName.IsNone();

#if WITH_EDITOR
	if(!bDisplayNameChanged && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bDisplayNameChanged && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(), 
				FString::Printf(TEXT("hierarchy_controller.set_display_name(%s, '%s', %s)"),
				*InControl.ToPythonString(),
				*InDisplayName.ToString(),
				(bRenameElement) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	return NewDisplayName;
}

bool URigHierarchyController::RenameElement(FRigBaseElement* InElement, const FName &InName, bool bClearSelection, bool bSetupUndoRedo)
{
	if(InElement == nullptr)
	{
		return false;
	}

	if (InElement->GetFName().IsEqual(InName, ENameCase::CaseSensitive))
	{
		return false;
	}

	const FRigElementKey OldKey = InElement->GetKey();
	TArray<FRigComponentKey> OldComponentKeys = InElement->GetComponentKeys();

	URigHierarchy* Hierarchy = GetHierarchy();

	// deselect the key that no longer exists
	// no need to trigger a reselect since we always clear selection after rename
	const bool bWasSelected = Hierarchy->IsSelected(InElement); 
	if (bWasSelected)
	{
		DeselectElement(OldKey);
	}

	{
		// create a temp copy of the map and remove the current item's key
		TMap<FRigElementKey, int32> TemporaryMap = Hierarchy->ElementIndexLookup;
		TemporaryMap.Remove(OldKey);
   
		TGuardValue<TMap<FRigElementKey, int32>> MapGuard(Hierarchy->ElementIndexLookup, TemporaryMap);
		InElement->Key.Name = GetSafeNewName(InName, InElement->GetType());
		InElement->CachedNameString.Reset();
	}
	
	const FRigElementKey NewKey = InElement->GetKey();

	Hierarchy->ElementIndexLookup.Remove(OldKey);
	Hierarchy->ElementIndexLookup.Add(NewKey, InElement->Index);

	for(const FRigComponentKey& ComponentKey : OldComponentKeys)
	{
		Hierarchy->ComponentIndexLookup.Remove(ComponentKey);
	}

	TArray<TPair<FRigHierarchyKey, FRigHierarchyKey>> ChangedHierarchyKeys;
	ChangedHierarchyKeys.Reserve(InElement->NumComponents() + 1);
	ChangedHierarchyKeys.Emplace(OldKey, NewKey);
	
	for(int32 ComponentIndex = 0; ComponentIndex < InElement->NumComponents(); ComponentIndex++)
	{
		FRigBaseComponent* Component = InElement->GetComponent(ComponentIndex);
		if(ensure(Component))
		{
			const FRigComponentKey OldComponentKey = Component->GetKey();
			Component->Key.ElementKey = NewKey;
			Hierarchy->ComponentIndexLookup.Add(Component->GetKey(), Component->GetIndexInHierarchy());
			ChangedHierarchyKeys.Emplace(OldComponentKey, Component->GetKey());
		}
	}

	// update all multi parent elements' index lookups
	for (FRigBaseElement* Element : Hierarchy->Elements)
	{
		if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(Element))
		{
			const int32* ExistingIndexPtr = MultiParentElement->IndexLookup.Find(OldKey);
			if(ExistingIndexPtr)
			{
				const int32 ExistingIndex = *ExistingIndexPtr;
				MultiParentElement->IndexLookup.Remove(OldKey);
				MultiParentElement->IndexLookup.Add(NewKey, ExistingIndex);
			}
		}

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
		{
			for(FRigElementKeyWithLabel& Favorite : ControlElement->Settings.Customization.AvailableSpaces)
			{
				if(Favorite.Key == OldKey)
				{
					Favorite.Key.Name = NewKey.Name;
				}
			}

			for(FRigElementKey& DrivenControl : ControlElement->Settings.DrivenControls)
			{
				if(DrivenControl == OldKey)
				{
					DrivenControl.Name = NewKey.Name;
				}
			}
		}
	}

	Hierarchy->PreviousHierarchyNameMap.FindOrAdd(NewKey) = OldKey;
	Hierarchy->IncrementTopologyVersion();

	UpdateComponentsOnHierarchyKeyChange(ChangedHierarchyKeys, bSetupUndoRedo);

	Notify(ERigHierarchyNotification::ElementRenamed, InElement);

	if (!bClearSelection && bWasSelected)
	{
		SelectElement(InElement->GetKey(), true);
	}

	return true;
}

bool URigHierarchyController::RenameComponent(FRigBaseComponent* InComponent, const FName& InName, bool bClearSelection, bool bSetupUndoRedo)
{
	if(InComponent == nullptr)
	{
		return false;
	}

	if (InComponent->GetFName().IsEqual(InName, ENameCase::CaseSensitive))
	{
		return false;
	}

	const FRigComponentKey OldKey = InComponent->GetKey();

	URigHierarchy* Hierarchy = GetHierarchy();

	// deselect the key that no longer exists
	// no need to trigger a reselect since we always clear selection after rename
	const bool bWasSelected = Hierarchy->IsComponentSelected(InComponent); 
	if (bWasSelected)
	{
		DeselectComponent(OldKey);
	}

	{
		// create a temp copy of the map and remove the current item's key
		TMap<FRigComponentKey, int32> TemporaryMap = Hierarchy->ComponentIndexLookup;
		TemporaryMap.Remove(OldKey);
   
		TGuardValue<TMap<FRigComponentKey, int32>> MapGuard(Hierarchy->ComponentIndexLookup, TemporaryMap);
		InComponent->Key.Name = Hierarchy->GetSafeNewComponentName(OldKey.ElementKey, InName);
		InComponent->CachedNameString.Reset();
	}
	
	const FRigComponentKey NewKey = InComponent->GetKey();

	Hierarchy->ComponentIndexLookup.Remove(OldKey);
	Hierarchy->ComponentIndexLookup.Add(NewKey, InComponent->IndexInHierarchy);
	Hierarchy->PreviousHierarchyNameMap.FindOrAdd(NewKey) = OldKey;

	Hierarchy->IncrementTopologyVersion();

	UpdateComponentsOnHierarchyKeyChange({{OldKey, NewKey}}, bSetupUndoRedo);

	Notify(ERigHierarchyNotification::ComponentRenamed, InComponent);

	if (!bClearSelection && bWasSelected)
	{
		SelectComponent(InComponent->GetKey(), true);
	}

	return true;
}

bool URigHierarchyController::ReparentComponent(FRigBaseComponent* InComponent, FRigBaseElement* InParentElement, bool bClearSelection, bool bSetupUndoRedo)
{
	if(InComponent == nullptr)
	{
		return false;
	}

	if(InComponent->Element == InParentElement)
	{
		return false;
	}

	const FRigElementKey NewParentKey = InParentElement->GetKey();

	URigHierarchy* Hierarchy = GetHierarchy();
	if(!Hierarchy->CanAddComponent(NewParentKey, InComponent))
	{
		return false;
	}

	const FRigComponentKey OldKey = InComponent->GetKey();

	// deselect the key that no longer exists
	// no need to trigger a reselect since we always clear selection after rename
	const bool bWasSelected = Hierarchy->IsComponentSelected(InComponent); 
	if (bWasSelected)
	{
		DeselectComponent(OldKey);
	}

	if(FRigBaseElement* OldParentElement = Hierarchy->Find(OldKey.ElementKey))
	{
		OldParentElement->ComponentIndices.Remove(InComponent->IndexInHierarchy);
	}

	InComponent->Key.Name = Hierarchy->GetSafeNewComponentName(NewParentKey, InComponent->Key.Name);
	InComponent->Key.ElementKey = NewParentKey;
	InComponent->CachedNameString.Reset();

	check(InParentElement);
	check(!InParentElement->ComponentIndices.Contains(InComponent->IndexInHierarchy));
	InParentElement->ComponentIndices.Add(InComponent->IndexInHierarchy);
	InComponent->Element = InParentElement;

	for(int32 IndexInElement = 0; IndexInElement < InParentElement->ComponentIndices.Num(); IndexInElement++)
	{
		if(FRigBaseComponent* RemainingComponent = Hierarchy->GetComponent(InParentElement->ComponentIndices[IndexInElement]))
		{
			RemainingComponent->IndexInElement = IndexInElement;
		}
	}
	
	const FRigComponentKey NewKey = InComponent->GetKey();

	Hierarchy->ComponentIndexLookup.Remove(OldKey);
	Hierarchy->ComponentIndexLookup.Add(NewKey, InComponent->IndexInHierarchy);
	Hierarchy->PreviousHierarchyParentMap.Add(NewKey, FRigHierarchyKey(OldKey.ElementKey, true));

	Hierarchy->IncrementTopologyVersion();

	UpdateComponentsOnHierarchyKeyChange({{OldKey, NewKey}}, bSetupUndoRedo);

	Notify(ERigHierarchyNotification::ComponentReparented, InComponent);

	if (!bClearSelection && bWasSelected)
	{
		SelectComponent(InComponent->GetKey(), true);
	}

	return true;
}

bool URigHierarchyController::ReorderElement(FRigBaseElement* InElement, int32 InIndex)
{
	if(InElement == nullptr)
	{
		return false;
	}

	InIndex = FMath::Max<int32>(InIndex, 0);

	URigHierarchy* Hierarchy = GetHierarchy();

	TArray<FRigBaseElement*> LocalElements;
	if(const FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(InElement))
	{
		LocalElements.Append(Hierarchy->GetChildren(ParentElement));
	}
	else
	{
		const TArray<FRigBaseElement*> RootElements = Hierarchy->GetRootElements();
		LocalElements.Append(RootElements);
	}

	const int32 CurrentIndex = LocalElements.Find(InElement);
	if(CurrentIndex == INDEX_NONE || CurrentIndex == InIndex)
	{
		return false;
	}

	Hierarchy->IncrementTopologyVersion();

	TArray<int32> GlobalIndices;
	GlobalIndices.Reserve(LocalElements.Num());
	for(const FRigBaseElement* Element : LocalElements)
	{
		GlobalIndices.Add(Element->GetIndex());
	}

	LocalElements.RemoveAt(CurrentIndex);
	if(InIndex >= LocalElements.Num())
	{
		LocalElements.Add(InElement);
	}
	else
	{
		LocalElements.Insert(InElement, InIndex);
	}

	InIndex = FMath::Min<int32>(InIndex, LocalElements.Num() - 1);
	const int32 LowerBound = FMath::Min<int32>(InIndex, CurrentIndex);
	const int32 UpperBound = FMath::Max<int32>(InIndex, CurrentIndex);
	for(int32 LocalIndex = LowerBound; LocalIndex <= UpperBound; LocalIndex++)
	{
		const int32 GlobalIndex = GlobalIndices[LocalIndex];
		FRigBaseElement* Element = LocalElements[LocalIndex];
		Hierarchy->Elements[GlobalIndex] = Element;
		Element->Index = GlobalIndex;
		Hierarchy->ElementIndexLookup.FindOrAdd(Element->Key) = GlobalIndex;
	}

	Notify(ERigHierarchyNotification::ElementReordered, InElement);

	return true;
}

FName URigHierarchyController::SetDisplayName(FRigControlElement* InControlElement, const FName& InDisplayName, bool bRenameElement)
{
	if(InControlElement == nullptr)
	{
		return NAME_None;
	}

	if (InControlElement->Settings.DisplayName.IsEqual(InDisplayName, ENameCase::CaseSensitive))
	{
		return NAME_None;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigElementKey ParentElementKey;
	if(const FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(InControlElement))
	{
		ParentElementKey = ParentElement->GetKey();
	}

	// avoid self name collision
	InControlElement->Settings.DisplayName = NAME_None;
	const FName DisplayName = Hierarchy->GetSafeNewDisplayName(ParentElementKey, InDisplayName);
	InControlElement->Settings.DisplayName = DisplayName;

	Hierarchy->IncrementTopologyVersion();
	Notify(ERigHierarchyNotification::ControlSettingChanged, InControlElement);

	if(bRenameElement)
	{
		RenameElement(InControlElement, InControlElement->Settings.DisplayName, false);
	}
#if WITH_EDITOR
	else
	{
		// if we are merely setting the display name - we want to update all listening hierarchies
		for(URigHierarchy::FRigHierarchyListener& Listener : Hierarchy->ListeningHierarchies)
		{
			if(URigHierarchy* ListeningHierarchy = Listener.Hierarchy.Get())
			{
				if(URigHierarchyController* ListeningController = ListeningHierarchy->GetController())
				{
					const TGuardValue<bool> Guard(ListeningController->bSuspendAllNotifications, true);
					ListeningController->SetDisplayName(InControlElement->GetKey(), InDisplayName, bRenameElement, false, false);
				}
			}
		}
	}
#endif
	return InControlElement->Settings.DisplayName;
}

bool URigHierarchyController::AddParent(FRigElementKey InChild, FRigElementKey InParent, float InWeight, bool bMaintainGlobalTransform, FName InDisplayLabel, bool bSetupUndo)
{
	if(!IsValid())
	{
		return false;
	}

	if(InParent.Type == ERigElementType::Socket)
	{
		ReportWarningf(TEXT("Cannot parent Child '%s' under a Socket parent."), *InChild.ToString());
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Child = Hierarchy->Find(InChild);
	if(Child == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Parent, Child '%s' not found."), *InChild.ToString());
		return false;
	}

	FRigBaseElement* Parent = Hierarchy->Find(InParent);
	if(Parent == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Parent, Parent '%s' not found."), *InParent.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Add Parent", "Add Parent"));
		Hierarchy->Modify();
	}
#endif

	const bool bAdded = AddParent(Child, Parent, InWeight, bMaintainGlobalTransform, false, InDisplayLabel);

#if WITH_EDITOR
	if(!bAdded && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();
#endif
	
	return bAdded;
}

bool URigHierarchyController::AddParent(FRigBaseElement* InChild, FRigBaseElement* InParent, float InWeight, bool bMaintainGlobalTransform, bool bRemoveAllParents, const FName& InDisplayLabel)
{
	if(InChild == nullptr || InParent == nullptr)
	{
		return false;
	}

	// single parent children can't be parented multiple times
	if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InChild))
	{
		if(SingleParentElement->ParentElement == InParent)
		{
			return false;
		}
		bRemoveAllParents = true;
	}

	else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		for(const FRigElementParentConstraint& ParentConstraint : MultiParentElement->ParentConstraints)
		{
			if(ParentConstraint.ParentElement == InParent)
			{
				return false;
			}
		}
	}

	// we can only parent things to controls which are not animation channels (animation channels are not 3D things)
	if(FRigControlElement* ParentControlElement = Cast<FRigControlElement>(InParent))
	{
		if(ParentControlElement->IsAnimationChannel())
		{
			return false;
		}
	}

	// we can only reparent animation channels - we cannot add a parent to them
	if(FRigControlElement* ChildControlElement = Cast<FRigControlElement>(InChild))
	{
		if(ChildControlElement->IsAnimationChannel())
		{
			bMaintainGlobalTransform = false;
			InWeight = 1.f;
			bRemoveAllParents = true;
		}

		if(ChildControlElement->Settings.bRestrictSpaceSwitching)
		{
			if(ChildControlElement->Settings.Customization.AvailableSpaces.FindByKey(InParent->GetKey()) != nullptr)
			{
				return false;
			}
		}
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	if (!(bSuspendParentCycleCheck && Hierarchy->GetChildren(InChild).IsEmpty()))
	{
		FRigHierarchyDependencyChain* DummyDependencyChain = nullptr;
		constexpr bool bConsiderInactiveParents = true;
		if(Hierarchy->IsParentedTo(InParent, InChild, FEmptyRigDependenciesProvider(), DummyDependencyChain, bConsiderInactiveParents))
		{
			ReportErrorf(TEXT("Cannot parent '%s' to '%s' - would cause a cycle."), *InChild->Key.ToString(), *InParent->Key.ToString());
			return false;
		}
	}

	Hierarchy->EnsureCacheValidity();

	if(bRemoveAllParents)
	{
		RemoveAllParents(InChild, bMaintainGlobalTransform);		
	}

	if(InWeight > SMALL_NUMBER || bRemoveAllParents)
	{
		if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(InChild))
		{
			if(bMaintainGlobalTransform)
			{
				Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentGlobal);
				Hierarchy->GetTransform(TransformElement, ERigTransformType::InitialGlobal);
				TransformElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentLocal);
				TransformElement->GetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
			}
			else
			{
				Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentLocal);
				Hierarchy->GetTransform(TransformElement, ERigTransformType::InitialLocal);
				TransformElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
				TransformElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}
		}

		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InChild))
		{
			Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
			Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal);
		}
	}

	FRigElementParentConstraint Constraint;
	Constraint.ParentElement = Cast<FRigTransformElement>(InParent);
	if(Constraint.ParentElement == nullptr)
	{
		return false;
	}
	Constraint.InitialWeight = InWeight;
	Constraint.Weight = InWeight;
	Constraint.DisplayLabel = InDisplayLabel;

	if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InChild))
	{
		AddElementToDirty(Constraint.ParentElement, SingleParentElement);
		SingleParentElement->ParentElement = Constraint.ParentElement;

		if(Cast<FRigSocketElement>(SingleParentElement))
		{
			Hierarchy->SetRigElementKeyMetadata(SingleParentElement->GetKey(), FRigSocketElement::DesiredParentMetaName, Constraint.ParentElement->GetKey());
			Hierarchy->Notify(ERigHierarchyNotification::SocketDesiredParentChanged, SingleParentElement);
		}

		Hierarchy->IncrementTopologyVersion();

		if(!bMaintainGlobalTransform)
		{
			Hierarchy->PropagateDirtyFlags(SingleParentElement, true, true);
			Hierarchy->PropagateDirtyFlags(SingleParentElement, false, true);
		}

		Notify(ERigHierarchyNotification::ParentChanged, SingleParentElement);
		
		Hierarchy->EnsureCacheValidity();
		
		return true;
	}
	else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		if(FRigControlElement* ControlElement = Cast<FRigControlElement>(InChild))
		{
			if(!ControlElement->Settings.DisplayName.IsNone())
			{
				// avoid self name collision
				FName DesiredDisplayName = NAME_None;
				Swap(DesiredDisplayName, ControlElement->Settings.DisplayName);
				
				ControlElement->Settings.DisplayName =
					Hierarchy->GetSafeNewDisplayName(
						InParent->GetKey(),
						DesiredDisplayName);
			}
		}
		
		AddElementToDirty(Constraint.ParentElement, MultiParentElement);

		const int32 ParentIndex = MultiParentElement->ParentConstraints.Add(Constraint);
		MultiParentElement->IndexLookup.Add(Constraint.ParentElement->GetKey(), ParentIndex);

		if(InWeight > SMALL_NUMBER)
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
			{
				ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);  
				ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);  
				ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}
		}

		Hierarchy->IncrementTopologyVersion();

		if(InWeight > SMALL_NUMBER)
		{
			if(!bMaintainGlobalTransform)
			{
				Hierarchy->PropagateDirtyFlags(MultiParentElement, true, true);
				Hierarchy->PropagateDirtyFlags(MultiParentElement, false, true);
			}
		}

		if(FRigControlElement* ChildControlElement = Cast<FRigControlElement>(InChild))
		{
			const FTransform LocalTransform = Hierarchy->GetTransform(ChildControlElement, ERigTransformType::InitialLocal);
			Hierarchy->SetControlPreferredEulerAngles(ChildControlElement, LocalTransform, true);
			ChildControlElement->PreferredEulerAngles.Current = ChildControlElement->PreferredEulerAngles.Initial;
		}

		Notify(ERigHierarchyNotification::ParentChanged, MultiParentElement);

		Hierarchy->EnsureCacheValidity();
		
		return true;
	}
	
	return false;
}

bool URigHierarchyController::RemoveParent(FRigElementKey InChild, FRigElementKey InParent, bool bMaintainGlobalTransform, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Child = Hierarchy->Find(InChild);
	if(Child == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Parent, Child '%s' not found."), *InChild.ToString());
		return false;
	}

	FRigBaseElement* Parent = Hierarchy->Find(InParent);
	if(Parent == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Parent, Parent '%s' not found."), *InParent.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Remove Parent", "Remove Parent"));
		Hierarchy->Modify();
	}
#endif

	const bool bRemoved = RemoveParent(Child, Parent, bMaintainGlobalTransform);

#if WITH_EDITOR
	if(!bRemoved && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bRemoved && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.remove_parent(%s, %s, %s)"),
				*InChild.ToPythonString(),
				*InParent.ToPythonString(),
				(bMaintainGlobalTransform) ? TEXT("True") : TEXT("False")));
		}
	}
#endif
	
	return bRemoved;
}

bool URigHierarchyController::RemoveParent(FRigBaseElement* InChild, FRigBaseElement* InParent, bool bMaintainGlobalTransform)
{
	if(InChild == nullptr || InParent == nullptr)
	{
		return false;
	}

	FRigTransformElement* ParentTransformElement = Cast<FRigTransformElement>(InParent);
	if(ParentTransformElement == nullptr)
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	// single parent children can't be parented multiple times
	if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InChild))
	{
		if(SingleParentElement->ParentElement == ParentTransformElement)
		{
			if(bMaintainGlobalTransform)
			{
				Hierarchy->GetTransform(SingleParentElement, ERigTransformType::CurrentGlobal);
				Hierarchy->GetTransform(SingleParentElement, ERigTransformType::InitialGlobal);
				SingleParentElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentLocal);
				SingleParentElement->GetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
			}
			else
			{
				Hierarchy->GetTransform(SingleParentElement, ERigTransformType::CurrentLocal);
				Hierarchy->GetTransform(SingleParentElement, ERigTransformType::InitialLocal);
				SingleParentElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
				SingleParentElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}

			const FRigElementKey PreviousParentKey = SingleParentElement->ParentElement->GetKey();
			Hierarchy->PreviousHierarchyParentMap.FindOrAdd(SingleParentElement->GetKey()) = PreviousParentKey;
			
			// remove the previous parent
			SingleParentElement->ParentElement = nullptr;

			if(Cast<FRigSocketElement>(SingleParentElement))
			{
				Hierarchy->RemoveMetadata(SingleParentElement->GetKey(), FRigSocketElement::DesiredParentMetaName);
				Hierarchy->Notify(ERigHierarchyNotification::SocketDesiredParentChanged, SingleParentElement);
			}

			RemoveElementToDirty(InParent, SingleParentElement); 
			Hierarchy->IncrementTopologyVersion();

			if(!bMaintainGlobalTransform)
			{
				Hierarchy->PropagateDirtyFlags(SingleParentElement, true, true);
				Hierarchy->PropagateDirtyFlags(SingleParentElement, false, true);
			}

			Notify(ERigHierarchyNotification::ParentChanged, SingleParentElement);

			Hierarchy->EnsureCacheValidity();
			
			return true;
		}
	}

	// single parent children can't be parented multiple times
	else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		int32 ParentIndex = INDEX_NONE;
		for(int32 ConstraintIndex = 0; ConstraintIndex < MultiParentElement->ParentConstraints.Num(); ConstraintIndex++)
		{
			if(MultiParentElement->ParentConstraints[ConstraintIndex].ParentElement == ParentTransformElement)
			{
				ParentIndex = ConstraintIndex;
				break;
			}
		}
				
		if(MultiParentElement->ParentConstraints.IsValidIndex(ParentIndex))
		{
			if(bMaintainGlobalTransform)
			{
				Hierarchy->GetTransform(MultiParentElement, ERigTransformType::CurrentGlobal);
				Hierarchy->GetTransform(MultiParentElement, ERigTransformType::InitialGlobal);
				MultiParentElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentLocal);
				MultiParentElement->GetDirtyState().MarkDirty(ERigTransformType::InitialLocal);
			}
			else
			{
				Hierarchy->GetTransform(MultiParentElement, ERigTransformType::CurrentLocal);
				Hierarchy->GetTransform(MultiParentElement, ERigTransformType::InitialLocal);
				MultiParentElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
				MultiParentElement->GetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}

			// remove the previous parent
			RemoveElementToDirty(InParent, MultiParentElement); 

			const FRigElementKey PreviousParentKey = MultiParentElement->ParentConstraints[ParentIndex].ParentElement->GetKey();
			Hierarchy->PreviousHierarchyParentMap.FindOrAdd(MultiParentElement->GetKey()) = PreviousParentKey;

			MultiParentElement->ParentConstraints.RemoveAt(ParentIndex);
			MultiParentElement->IndexLookup.Remove(ParentTransformElement->GetKey());
			for(TPair<FRigElementKey, int32>& Pair : MultiParentElement->IndexLookup)
			{
				if(Pair.Value > ParentIndex)
				{
					Pair.Value--;
				}
			}

			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(MultiParentElement))
			{
				ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);  
				ControlElement->GetOffsetDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
				ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);  
				ControlElement->GetShapeDirtyState().MarkDirty(ERigTransformType::InitialGlobal);
			}

			Hierarchy->IncrementTopologyVersion();

			if(!bMaintainGlobalTransform)
			{
				Hierarchy->PropagateDirtyFlags(MultiParentElement, true, true);
				Hierarchy->PropagateDirtyFlags(MultiParentElement, false, true);
			}

			Notify(ERigHierarchyNotification::ParentChanged, MultiParentElement);

			Hierarchy->EnsureCacheValidity();
			
			return true;
		}
	}

	return false;
}

bool URigHierarchyController::RemoveAllParents(FRigElementKey InChild, bool bMaintainGlobalTransform, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Child = Hierarchy->Find(InChild);
	if(Child == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove All Parents, Child '%s' not found."), *InChild.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Remove Parent", "Remove Parent"));
		Hierarchy->Modify();
	}
#endif

	const bool bRemoved = RemoveAllParents(Child, bMaintainGlobalTransform);

#if WITH_EDITOR
	if(!bRemoved && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();
	
	if (bRemoved && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.remove_all_parents(%s, %s)"),
				*InChild.ToPythonString(),
				(bMaintainGlobalTransform) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	return bRemoved;
}

bool URigHierarchyController::RemoveAllParents(FRigBaseElement* InChild, bool bMaintainGlobalTransform)
{
	if(FRigSingleParentElement* SingleParentElement = Cast<FRigSingleParentElement>(InChild))
	{
		return RemoveParent(SingleParentElement, SingleParentElement->ParentElement, bMaintainGlobalTransform);
	}
	else if(FRigMultiParentElement* MultiParentElement = Cast<FRigMultiParentElement>(InChild))
	{
		bool bSuccess = true;

		FRigElementParentConstraintArray ParentConstraints = MultiParentElement->ParentConstraints;
		for(const FRigElementParentConstraint& ParentConstraint : ParentConstraints)
		{
			if(!RemoveParent(MultiParentElement, ParentConstraint.ParentElement, bMaintainGlobalTransform))
			{
				bSuccess = false;
			}
		}

		return bSuccess;
	}
	return false;
}

bool URigHierarchyController::SetParent(FRigElementKey InChild, FRigElementKey InParent, bool bMaintainGlobalTransform, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* Child = Hierarchy->Find(InChild);
	if(Child == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Parent, Child '%s' not found."), *InChild.ToString());
		return false;
	}

	FRigBaseElement* Parent = Hierarchy->Find(InParent);
	if(Parent == nullptr)
	{
		if(InChild.Type == ERigElementType::Socket)
		{
			Hierarchy->SetRigElementKeyMetadata(InChild, FRigSocketElement::DesiredParentMetaName, InParent);
			Hierarchy->Notify(ERigHierarchyNotification::SocketDesiredParentChanged, Child);
			return true;
		}
		ReportWarningf(TEXT("Cannot Set Parent, Parent '%s' not found."), *InParent.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "Set Parent", "Set Parent"));
		Hierarchy->Modify();
	}
#endif

	const bool bParentSet = SetParent(Child, Parent, bMaintainGlobalTransform);

#if WITH_EDITOR
	if(!bParentSet && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bParentSet && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.set_parent(%s, %s, %s)"),
				*InChild.ToPythonString(),
				*InParent.ToPythonString(),
				(bMaintainGlobalTransform) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	return bParentSet;
}

bool URigHierarchyController::AddAvailableSpace(FRigElementKey InControl, FRigElementKey InSpace, FName InDisplayLabel, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* ControlBase = Hierarchy->Find(InControl);
	if(ControlBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Available Space, Control '%s' not found."), *InControl.ToString());
		return false;
	}

	FRigControlElement* Control = Cast<FRigControlElement>(ControlBase);
	if(Control == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Available Space, '%s' is not a Control."), *InControl.ToString());
		return false;
	}

	const FRigBaseElement* SpaceBase = Hierarchy->Find(InSpace);
	if(SpaceBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Available Space, Space '%s' not found."), *InSpace.ToString());
		return false;
	}

	const FRigTransformElement* Space = Cast<FRigTransformElement>(SpaceBase);
	if(Space == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Available Space, '%s' is not a Transform."), *InSpace.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "AddAvailableSpace", "Add Available Space"));
		Hierarchy->Modify();
	}
#endif

	const bool bSuccess = AddAvailableSpace(Control, Space, InDisplayLabel);

#if WITH_EDITOR
	if(!bSuccess && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bSuccess && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.add_available_space(%s, %s, '%s')"),
				*InControl.ToPythonString(),
				*InSpace.ToPythonString(),
				*InDisplayLabel.ToString()));
		}
	}
#endif

	return bSuccess;
}

bool URigHierarchyController::RemoveAvailableSpace(FRigElementKey InControl, FRigElementKey InSpace, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* ControlBase = Hierarchy->Find(InControl);
	if(ControlBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Available Space, Control '%s' not found."), *InControl.ToString());
		return false;
	}

	FRigControlElement* Control = Cast<FRigControlElement>(ControlBase);
	if(Control == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Available Space, '%s' is not a Control."), *InControl.ToString());
		return false;
	}

	const FRigBaseElement* SpaceBase = Hierarchy->Find(InSpace);
	if(SpaceBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Available Space, Space '%s' not found."), *InSpace.ToString());
		return false;
	}

	const FRigTransformElement* Space = Cast<FRigTransformElement>(SpaceBase);
	if(Space == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Available Space, '%s' is not a Transform."), *InSpace.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "RemoveAvailableSpace", "Remove Available Space"));
		Hierarchy->Modify();
	}
#endif

	const bool bSuccess = RemoveAvailableSpace(Control, Space);

#if WITH_EDITOR
	if(!bSuccess && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bSuccess && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.remove_available_space(%s, %s)"),
				*InControl.ToPythonString(),
				*InSpace.ToPythonString()));
		}
	}
#endif

	return bSuccess;
}

bool URigHierarchyController::SetAvailableSpaceIndex(FRigElementKey InControl, FRigElementKey InSpace, int32 InIndex, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* ControlBase = Hierarchy->Find(InControl);
	if(ControlBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Index, Control '%s' not found."), *InControl.ToString());
		return false;
	}

	FRigControlElement* Control = Cast<FRigControlElement>(ControlBase);
	if(Control == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Index, '%s' is not a Control."), *InControl.ToString());
		return false;
	}

	const FRigBaseElement* SpaceBase = Hierarchy->Find(InSpace);
	if(SpaceBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Index, Space '%s' not found."), *InSpace.ToString());
		return false;
	}

	const FRigTransformElement* Space = Cast<FRigTransformElement>(SpaceBase);
	if(Space == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Index, '%s' is not a Transform."), *InSpace.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "SetAvailableSpaceIndex", "Reorder Available Space"));
		Hierarchy->Modify();
	}
#endif

	const bool bSuccess = SetAvailableSpaceIndex(Control, Space, InIndex);

#if WITH_EDITOR
	if(!bSuccess && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bSuccess && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.set_available_space_index(%s, %s)"),
				*InControl.ToPythonString(),
				*InSpace.ToPythonString()));
		}
	}
#endif

	return bSuccess;
}

bool URigHierarchyController::SetAvailableSpaceLabel(FRigElementKey InControl, FRigElementKey InSpace, FName InDisplayLabel, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* ControlBase = Hierarchy->Find(InControl);
	if(ControlBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Label, Control '%s' not found."), *InControl.ToString());
		return false;
	}

	FRigControlElement* Control = Cast<FRigControlElement>(ControlBase);
	if(Control == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Label, '%s' is not a Control."), *InControl.ToString());
		return false;
	}

	const FRigBaseElement* SpaceBase = Hierarchy->Find(InSpace);
	if(SpaceBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Label, Space '%s' not found."), *InSpace.ToString());
		return false;
	}

	const FRigTransformElement* Space = Cast<FRigTransformElement>(SpaceBase);
	if(Space == nullptr)
	{
		ReportWarningf(TEXT("Cannot Set Available Space Label, '%s' is not a Transform."), *InSpace.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "SetAvailableSpaceLabel", "Set Available Space Label"));
		Hierarchy->Modify();
	}
#endif

	const bool bSuccess = SetAvailableSpaceLabel(Control, Space, InDisplayLabel);

#if WITH_EDITOR
	if(!bSuccess && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bSuccess && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.set_available_space_label(%s, %s, '%s')"),
				*InControl.ToPythonString(),
				*InSpace.ToPythonString(),
				*InDisplayLabel.ToString()));
		}
	}
#endif

	return bSuccess;
}

bool URigHierarchyController::AddChannelHost(FRigElementKey InChannel, FRigElementKey InHost, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* ChannelBase = Hierarchy->Find(InChannel);
	if(ChannelBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, Channel '%s' not found."), *InChannel.ToString());
		return false;
	}

	FRigControlElement* Channel = Cast<FRigControlElement>(ChannelBase);
	if(Channel == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, '%s' is not a Control."), *InChannel.ToString());
		return false;
	}

	if(!Channel->IsAnimationChannel())
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, '%s' is not an animation channel."), *InChannel.ToString());
		return false;
	}

	const FRigBaseElement* HostBase = Hierarchy->Find(InHost);
	if(HostBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, Host '%s' not found."), *InHost.ToString());
		return false;
	}

	const FRigControlElement* Host = Cast<FRigControlElement>(HostBase);
	if(Host == nullptr)
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, '%s' is not a Control."), *InHost.ToString());
		return false;
	}

	if(Host->IsAnimationChannel())
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, '%s' is also an animation channel."), *InHost.ToString());
		return false;
	}

	// the default parent cannot be added to the channel host list
	if(Hierarchy->GetParents(InChannel).Contains(InHost))
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, '%s' is the parent of channel '%s'."), *InHost.ToString(), *InChannel.ToString());
		return false;
	}

	if(Channel->Settings.Customization.AvailableSpaces.FindByKey(Host->GetKey()))
	{
		ReportWarningf(TEXT("Cannot Add Channel Host, '%s' is already a host for channel '%s'."), *InHost.ToString(), *InChannel.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "AddChannelHost", "Add Channel Host"));
		Hierarchy->Modify();
	}
#endif

	const bool bSuccess = AddAvailableSpace(Channel, Host);

#if WITH_EDITOR
	if(!bSuccess && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bSuccess && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.add_channel_host(%s, %s)"),
				*InChannel.ToPythonString(),
				*InHost.ToPythonString()));
		}
	}
#endif

	return bSuccess;
}

bool URigHierarchyController::RemoveChannelHost(FRigElementKey InChannel, FRigElementKey InHost, bool bSetupUndo, bool bPrintPythonCommand)
{
	if(!IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = GetHierarchy();

	FRigBaseElement* ChannelBase = Hierarchy->Find(InChannel);
	if(ChannelBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, Channel '%s' not found."), *InChannel.ToString());
		return false;
	}

	FRigControlElement* Channel = Cast<FRigControlElement>(ChannelBase);
	if(Channel == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, '%s' is not a Control."), *InChannel.ToString());
		return false;
	}

	if(!Channel->IsAnimationChannel())
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, '%s' is not an animation channel."), *InChannel.ToString());
		return false;
	}

	const FRigBaseElement* HostBase = Hierarchy->Find(InHost);
	if(HostBase == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, Host '%s' not found."), *InHost.ToString());
		return false;
	}

	const FRigControlElement* Host = Cast<FRigControlElement>(HostBase);
	if(Host == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, '%s' is not a Control."), *InHost.ToString());
		return false;
	}

	if(Host->IsAnimationChannel())
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, '%s' is also an animation channel."), *InHost.ToString());
		return false;
	}

	if(Channel->Settings.Customization.AvailableSpaces.FindByKey(Host->GetKey()) == nullptr)
	{
		ReportWarningf(TEXT("Cannot Remove Channel Host, '%s' is not a host for channel '%s'."), *InHost.ToString(), *InChannel.ToString());
		return false;
	}

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> TransactionPtr;
	if(bSetupUndo)
	{
		TransactionPtr = MakeShared<FScopedTransaction>(NSLOCTEXT("RigHierarchyController", "RemoveChannelHost", "Remove Channel Host"));
		Hierarchy->Modify();
	}
#endif

	const bool bSuccess = RemoveAvailableSpace(Channel, Host);

#if WITH_EDITOR
	if(!bSuccess && TransactionPtr.IsValid())
	{
		TransactionPtr->Cancel();
	}
	TransactionPtr.Reset();

	if (bSuccess && bPrintPythonCommand && !bSuspendPythonPrinting)
	{
		if (const UBlueprint* Blueprint = GetTypedOuter<UBlueprint>())
		{
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.remove_channel_host(%s, %s)"),
				*InChannel.ToPythonString(),
				*InHost.ToPythonString()));
		}
	}
#endif

	return bSuccess;
}

TArray<FRigElementKey> URigHierarchyController::DuplicateElements(TArray<FRigElementKey> InKeys, bool bSelectNewElements, bool bSetupUndo, bool bPrintPythonCommands)
{
	const FString Content = ExportToText(InKeys);
	TArray<FRigElementKey> Result = ImportFromText(Content, false, bSelectNewElements, bSetupUndo);

#if WITH_EDITOR
	if (!Result.IsEmpty() && bPrintPythonCommands && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			FString ArrayStr = TEXT("[");
			for (auto It = InKeys.CreateConstIterator(); It; ++It)
			{
				ArrayStr += It->ToPythonString();
				if (It.GetIndex() < InKeys.Num() - 1)
				{
					ArrayStr += TEXT(", ");
				}
			}
			ArrayStr += TEXT("]");		
		
			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.duplicate_elements(%s, %s)"),
				*ArrayStr,
				(bSelectNewElements) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	GetHierarchy()->EnsureCacheValidity();
	
	return Result;
}

TArray<FRigElementKey> URigHierarchyController::MirrorElements(TArray<FRigElementKey> InKeys, FRigVMMirrorSettings InSettings, bool bSelectNewElements, bool bSetupUndo, bool bPrintPythonCommands)
{
	URigHierarchy* Hierarchy = GetHierarchy();
	FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

	TArray<FRigElementKey> OriginalKeys = Hierarchy->SortKeys(InKeys);
	TArray<FRigElementKey> DuplicatedKeys = DuplicateElements(OriginalKeys, bSelectNewElements, bSetupUndo);

	if (DuplicatedKeys.Num() != OriginalKeys.Num())
	{
		return DuplicatedKeys;
	}

	for (int32 Index = 0; Index < OriginalKeys.Num(); Index++)
	{
		if (DuplicatedKeys[Index].Type != OriginalKeys[Index].Type)
		{
			return DuplicatedKeys;
		}
	}

	// mirror the transforms
	for (int32 Index = 0; Index < OriginalKeys.Num(); Index++)
	{
		FTransform GlobalTransform = Hierarchy->GetGlobalTransform(OriginalKeys[Index]);
		FTransform InitialTransform = Hierarchy->GetInitialGlobalTransform(OriginalKeys[Index]);

		// also mirror the offset, limits and shape transform
		if (OriginalKeys[Index].Type == ERigElementType::Control)
		{
			if(FRigControlElement* DuplicatedControlElement = Hierarchy->Find<FRigControlElement>(DuplicatedKeys[Index]))
			{
				TGuardValue<TArray<FRigControlLimitEnabled>> DisableLimits(DuplicatedControlElement->Settings.LimitEnabled, TArray<FRigControlLimitEnabled>());

				// mirror offset
				FTransform OriginalGlobalOffsetTransform = Hierarchy->GetGlobalControlOffsetTransform(OriginalKeys[Index]);
				FTransform ParentTransform = Hierarchy->GetParentTransform(DuplicatedKeys[Index]);
				FTransform OffsetTransform = InSettings.MirrorTransform(OriginalGlobalOffsetTransform).GetRelativeTransform(ParentTransform);
				Hierarchy->SetControlOffsetTransform(DuplicatedKeys[Index], OffsetTransform, true, false, true);
				Hierarchy->SetControlOffsetTransform(DuplicatedKeys[Index], OffsetTransform, false, false, true);

				// mirror limits
				FTransform DuplicatedGlobalOffsetTransform = Hierarchy->GetGlobalControlOffsetTransform(DuplicatedKeys[Index]);

				for (ERigControlValueType ValueType = ERigControlValueType::Minimum;
                    ValueType <= ERigControlValueType::Maximum;
                    ValueType = (ERigControlValueType)(uint8(ValueType) + 1)
                    )
				{
					const FRigControlValue LimitValue = Hierarchy->GetControlValue(DuplicatedKeys[Index], ValueType);
					const FTransform LocalLimitTransform = LimitValue.GetAsTransform(DuplicatedControlElement->Settings.ControlType, DuplicatedControlElement->Settings.PrimaryAxis);
					FTransform GlobalLimitTransform = LocalLimitTransform * OriginalGlobalOffsetTransform;
					FTransform DuplicatedLimitTransform = InSettings.MirrorTransform(GlobalLimitTransform).GetRelativeTransform(DuplicatedGlobalOffsetTransform);
					FRigControlValue DuplicatedValue;
					DuplicatedValue.SetFromTransform(DuplicatedLimitTransform, DuplicatedControlElement->Settings.ControlType, DuplicatedControlElement->Settings.PrimaryAxis);
					Hierarchy->SetControlValue(DuplicatedControlElement, DuplicatedValue, ValueType, false);
				}

				// we need to do this here to make sure that the limits don't apply ( the tguardvalue is still active within here )
				Hierarchy->SetGlobalTransform(DuplicatedKeys[Index], InSettings.MirrorTransform(GlobalTransform), true, false, true);
				Hierarchy->SetGlobalTransform(DuplicatedKeys[Index], InSettings.MirrorTransform(GlobalTransform), false, false, true);

				// mirror shape transform
				FTransform GlobalShapeTransform = Hierarchy->GetControlShapeTransform(DuplicatedControlElement, ERigTransformType::InitialLocal) * OriginalGlobalOffsetTransform;
				Hierarchy->SetControlShapeTransform(DuplicatedControlElement, InSettings.MirrorTransform(GlobalShapeTransform).GetRelativeTransform(DuplicatedGlobalOffsetTransform), ERigTransformType::InitialLocal, true);
				Hierarchy->SetControlShapeTransform(DuplicatedControlElement, InSettings.MirrorTransform(GlobalShapeTransform).GetRelativeTransform(DuplicatedGlobalOffsetTransform), ERigTransformType::CurrentLocal, true);
			}
		}
		else
		{
			Hierarchy->SetGlobalTransform(DuplicatedKeys[Index], InSettings.MirrorTransform(GlobalTransform), true, false, true);
			Hierarchy->SetGlobalTransform(DuplicatedKeys[Index], InSettings.MirrorTransform(GlobalTransform), false, false, true);
		}
	}

	// correct the names
	if (!InSettings.SearchString.IsEmpty() && !InSettings.ReplaceString.IsEmpty())
	{
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);
		
		for (int32 Index = 0; Index < DuplicatedKeys.Num(); Index++)
		{
			FName OldName = OriginalKeys[Index].Name;
			FString OldNameStr = OldName.ToString();
			FString NewNameStr = OldNameStr.Replace(*InSettings.SearchString, *InSettings.ReplaceString, ESearchCase::CaseSensitive);
			if (NewNameStr != OldNameStr)
			{
				Controller->RenameElement(DuplicatedKeys[Index], *NewNameStr, true);
			}
		}
	}

#if WITH_EDITOR
	if (!DuplicatedKeys.IsEmpty() && bPrintPythonCommands && !bSuspendPythonPrinting)
	{
		UBlueprint* Blueprint = GetTypedOuter<UBlueprint>();
		if (Blueprint)
		{
			FString ArrayStr = TEXT("[");
			for (auto It = InKeys.CreateConstIterator(); It; ++It)
			{
				ArrayStr += It->ToPythonString();
				if (It.GetIndex() < InKeys.Num() - 1)
				{
					ArrayStr += TEXT(", ");
				}
			}
			ArrayStr += TEXT("]");

			RigVMPythonUtils::Print(Blueprint->GetFName().ToString(),
				FString::Printf(TEXT("hierarchy_controller.mirror_elements(%s, unreal.RigMirrorSettings(%s, %s, '%s', '%s'), %s)"),
				*ArrayStr,
				*RigVMPythonUtils::EnumValueToPythonString<EAxis::Type>((int64)InSettings.MirrorAxis.GetValue()),
				*RigVMPythonUtils::EnumValueToPythonString<EAxis::Type>((int64)InSettings.AxisToFlip.GetValue()),
				*InSettings.SearchString,
				*InSettings.ReplaceString,
				(bSelectNewElements) ? TEXT("True") : TEXT("False")));
		}
	}
#endif

	Hierarchy->EnsureCacheValidity();
	
	return DuplicatedKeys;
}

bool URigHierarchyController::SetParent(FRigBaseElement* InChild, FRigBaseElement* InParent, bool bMaintainGlobalTransform)
{
	if(InChild == nullptr || InParent == nullptr)
	{
		return false;
	}
	return AddParent(InChild, InParent, 1.f, bMaintainGlobalTransform, true);
}

bool URigHierarchyController::AddAvailableSpace(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement, const FName& InDisplayLabel)
{
	if(InControlElement == nullptr || InSpaceElement == nullptr)
	{
		return false;
	}

	// we cannot use animation channels as spaces / channel hosts
	if(const FRigControlElement* SpaceControlElement = Cast<FRigControlElement>(InSpaceElement))
	{
		if(SpaceControlElement->IsAnimationChannel())
		{
			return false;
		}
	}

	// in case of animation channels - we can only relate them to controls
	if(InControlElement->IsAnimationChannel())
	{
		if(!InSpaceElement->IsA<FRigControlElement>())
		{
			return false;
		}
	}

	// the default parent cannot be added to the available spaces list
	if(GetHierarchy()->GetParents(InControlElement).Contains(InSpaceElement))
	{
		return false;
	}

	FRigControlSettings Settings = InControlElement->Settings;
	TArray<FRigElementKeyWithLabel>& AvailableSpaces = Settings.Customization.AvailableSpaces;
	if(AvailableSpaces.FindByKey(InSpaceElement->GetKey()))
	{
		return false;
	}

	AvailableSpaces.Emplace(InSpaceElement->GetKey(), InDisplayLabel);

	GetHierarchy()->SetControlSettings(InControlElement, Settings, false, false, false);
	return true;
}

bool URigHierarchyController::RemoveAvailableSpace(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement)
{
	if(InControlElement == nullptr || InSpaceElement == nullptr)
	{
		return false;
	}

	FRigControlSettings Settings = InControlElement->Settings;
	TArray<FRigElementKeyWithLabel>& AvailableSpaces =Settings.Customization.AvailableSpaces;
	const int32 ExistingSpaceIndex = AvailableSpaces.IndexOfByKey(InSpaceElement->GetKey());
	if(ExistingSpaceIndex == INDEX_NONE)
	{
		return false;
	}
	AvailableSpaces.RemoveAt(ExistingSpaceIndex);
	
	GetHierarchy()->SetControlSettings(InControlElement, Settings, false, false, false);
	return true;
}

bool URigHierarchyController::SetAvailableSpaceIndex(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement, int32 InIndex)
{
	if(InControlElement == nullptr || InSpaceElement == nullptr)
	{
		return false;
	}

	FRigControlSettings Settings = InControlElement->Settings;
	TArray<FRigElementKeyWithLabel>& AvailableSpaces = Settings.Customization.AvailableSpaces;

	bool bAddedAvailableSpace = false;
	if(AvailableSpaces.FindByKey(InSpaceElement->GetKey()) == nullptr)
	{
		bAddedAvailableSpace = AddAvailableSpace(InControlElement, InSpaceElement);
		if(!bAddedAvailableSpace)
		{
			return false;
		}
		Settings = InControlElement->Settings;
		AvailableSpaces = Settings.Customization.AvailableSpaces;
	}

	const int32 ExistingSpaceIndex = AvailableSpaces.IndexOfByKey(InSpaceElement->GetKey());
	if(ExistingSpaceIndex == InIndex)
	{
		return bAddedAvailableSpace;
	}

	const FRigElementKeyWithLabel AvailableSpaceToMove = AvailableSpaces[ExistingSpaceIndex];
	AvailableSpaces.RemoveAt(ExistingSpaceIndex);
	
	InIndex = FMath::Max(InIndex, 0);
	if(InIndex >= AvailableSpaces.Num())
	{
		AvailableSpaces.Add(AvailableSpaceToMove);
	}
	else
	{
		AvailableSpaces.Insert(AvailableSpaceToMove, InIndex);
	}

	GetHierarchy()->SetControlSettings(InControlElement, Settings, false, false, false);
	return true;
}

bool URigHierarchyController::SetAvailableSpaceLabel(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement, const FName& InDisplayLabel)
{
	if(InControlElement == nullptr || InSpaceElement == nullptr)
	{
		return false;
	}

	FRigControlSettings Settings = InControlElement->Settings;
	TArray<FRigElementKeyWithLabel>& AvailableSpaces = Settings.Customization.AvailableSpaces;

	// first let's see if this is an available space that's registered in the control's settings
	if(FRigElementKeyWithLabel* AvailableSpace = AvailableSpaces.FindByKey(InSpaceElement->GetKey()))
	{
		if(AvailableSpace->Label.IsEqual(InDisplayLabel))
		{
			return false;
		}
		AvailableSpace->Label = InDisplayLabel;
		GetHierarchy()->SetControlSettings(InControlElement, Settings, false, false, false);
		return true;
	}

	// now we'll need to take a look at the parent constraints of the control
	if(const int32* ParentConstraintIndexPtr = InControlElement->IndexLookup.Find(InSpaceElement->GetKey()))
	{
		if(GetHierarchy()->GetDefaultParent(InControlElement->GetKey()) == InSpaceElement->GetKey())
		{
			return false;
		}
		
		FRigElementParentConstraint& ParentConstraint = InControlElement->ParentConstraints[*ParentConstraintIndexPtr];
		if(ParentConstraint.DisplayLabel.IsEqual(InDisplayLabel))
		{
			return false;
		}
		ParentConstraint.DisplayLabel = InDisplayLabel;

		Notify(ERigHierarchyNotification::ControlSettingChanged, InControlElement);
		return true;
	}

	return false;
}

void URigHierarchyController::AddElementToDirty(FRigBaseElement* InParent, FRigBaseElement* InElementToAdd, int32 InHierarchyDistance) const
{
	if(InParent == nullptr)
	{
		return;
	} 

	FRigTransformElement* ElementToAdd = Cast<FRigTransformElement>(InElementToAdd);
	if(ElementToAdd == nullptr)
	{
		return;
	}

	if(FRigTransformElement* TransformParent = Cast<FRigTransformElement>(InParent))
	{
		const FRigTransformElement::FElementToDirty ElementToDirty(ElementToAdd, InHierarchyDistance);
		TransformParent->ElementsToDirty.AddUnique(ElementToDirty);
	}
}

void URigHierarchyController::RemoveElementToDirty(FRigBaseElement* InParent, FRigBaseElement* InElementToRemove) const
{
	if(InParent == nullptr)
	{
		return;
	}

	FRigTransformElement* ElementToRemove = Cast<FRigTransformElement>(InElementToRemove);
	if(ElementToRemove == nullptr)
	{
		return;
	}

	if(FRigTransformElement* TransformParent = Cast<FRigTransformElement>(InParent))
	{
		TransformParent->ElementsToDirty.Remove(ElementToRemove);
	}
}

void URigHierarchyController::ReportWarning(const FString& InMessage) const
{
	if(!bReportWarningsAndErrors)
	{
		return;
	}

	if(LogFunction)
	{
		LogFunction(EMessageSeverity::Warning, InMessage);
		return;
	}

	FString Message = InMessage;
	if (const URigHierarchy* Hierarchy = GetHierarchy())
	{
		if (const UPackage* Package = Cast<UPackage>(Hierarchy->GetOutermost()))
		{
			Message = FString::Printf(TEXT("%s : %s"), *Package->GetPathName(), *InMessage);
		}
	}

	FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *Message, *FString());
}

void URigHierarchyController::ReportError(const FString& InMessage) const
{
	if(!bReportWarningsAndErrors)
	{
		return;
	}

	if(LogFunction)
	{
		LogFunction(EMessageSeverity::Error, InMessage);
		return;
	}

	FString Message = InMessage;
	if (const URigHierarchy* Hierarchy = GetHierarchy())
	{
		if (const UPackage* Package = Cast<UPackage>(Hierarchy->GetOutermost()))
		{
			Message = FString::Printf(TEXT("%s : %s"), *Package->GetPathName(), *InMessage);
		}
	}

	FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, *Message, *FString());
}

void URigHierarchyController::ReportAndNotifyError(const FString& InMessage) const
{
	if (!bReportWarningsAndErrors)
	{
		return;
	}

	ReportError(InMessage);

#if WITH_EDITOR
	FNotificationInfo Info(FText::FromString(InMessage));
	Info.bUseSuccessFailIcons = true;
	Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
	Info.bFireAndForget = true;
	Info.bUseThrobber = true;
	// longer message needs more time to read
	Info.FadeOutDuration = FMath::Clamp(0.1f * InMessage.Len(), 5.0f, 20.0f);
	Info.ExpireDuration = Info.FadeOutDuration;
	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	if (NotificationPtr)
	{
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
	}
#endif
}
