// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyDefines.h"
#include "Rigs/RigHierarchy.h"
#include "Misc/WildcardString.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyDefines)

#if WITH_EDITOR
#include "RigVMPythonUtils.h"
#endif

FRigHierarchySerializationSettings::FRigHierarchySerializationSettings(const FArchive& InArchive)
{
	const bool bIsDuplicating = (InArchive.GetPortFlags() & PPF_Duplicate) != 0;
	bIsSerializingToPackage =
		!bIsDuplicating &&
		InArchive.IsPersistent() &&
		!InArchive.IsObjectReferenceCollector() &&
		!InArchive.ShouldSkipBulkData() &&
		!InArchive.IsTransacting();

	ControlRigVersion = (FControlRigObjectVersion::Type)InArchive.CustomVer(FControlRigObjectVersion::GUID);
}

void FRigHierarchySerializationSettings::Save(FArchive& InArchive)
{
	int32 ControlRigVersionInt = (int32)ControlRigVersion;
	int32 SerializationPhaseInt = (int32)SerializationPhase;

	InArchive << ControlRigVersionInt;
	InArchive << bIsSerializingToPackage;
	InArchive << bUseCompressedArchive;
	InArchive << bStoreCompactTransforms;
	InArchive << bSerializeLocalTransform;
	InArchive << bSerializeGlobalTransform;
	InArchive << bSerializeInitialTransform;
	InArchive << bSerializeCurrentTransform;
	InArchive << SerializationPhaseInt;
}

void FRigHierarchySerializationSettings::Load(FArchive& InArchive)
{
	int32 ControlRigVersionInt = INDEX_NONE;
	int32 SerializationPhaseInt = INDEX_NONE;

	InArchive << ControlRigVersionInt;
	InArchive << bIsSerializingToPackage;
	InArchive << bUseCompressedArchive;
	InArchive << bStoreCompactTransforms;
	InArchive << bSerializeLocalTransform;
	InArchive << bSerializeGlobalTransform;
	InArchive << bSerializeInitialTransform;
	InArchive << bSerializeCurrentTransform;
	InArchive << SerializationPhaseInt;

	ControlRigVersion = (FControlRigObjectVersion::Type)ControlRigVersionInt;
	SerializationPhase = (ESerializationPhase)SerializationPhaseInt;
}

////////////////////////////////////////////////////////////////////////////////
// FRigControlLimitEnabled
////////////////////////////////////////////////////////////////////////////////

void FRigControlLimitEnabled::Serialize(FArchive& Ar)
{
	Ar << bMinimum;
	Ar << bMaximum;
}

bool FRigControlLimitEnabled::GetForValueType(ERigControlValueType InValueType) const
{
	if(InValueType == ERigControlValueType::Minimum)
	{
		return bMinimum;
	}
	return bMaximum;
}

void FRigControlLimitEnabled::SetForValueType(ERigControlValueType InValueType, bool InValue)
{
	if(InValueType == ERigControlValueType::Minimum)
	{
		bMinimum = InValue;
	}
	else
	{
		bMaximum = InValue;
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigHierarchyModulePath
////////////////////////////////////////////////////////////////////////////////

bool FRigHierarchyModulePath::IsValid() const
{
	return !IsEmpty() && (UsesNameSpaceFormat() || UsesModuleNameFormat());
}

bool FRigHierarchyModulePath::UsesNameSpaceFormat() const
{
	const int32 Len = ModulePath.Len();
	if(Len >= 3)
	{
		int32 SeparatorIndex = INDEX_NONE;
		if(ModulePath.FindLastChar(NamespaceSeparatorChar_Deprecated, SeparatorIndex))
		{
			return SeparatorIndex > 0 && SeparatorIndex < Len - 1;
		}
	};
	return false;
}

bool FRigHierarchyModulePath::UsesModuleNameFormat() const
{
	const int32 Len = ModulePath.Len();
	if(Len >= 3)
	{
		int32 SeparatorIndex = INDEX_NONE;
		if(ModulePath.FindLastChar(ModuleNameSuffixChar, SeparatorIndex))
		{
			return SeparatorIndex > 0 && SeparatorIndex < Len - 1;
		}
	};
	return false;
}

FName FRigHierarchyModulePath::GetPathFName() const
{
	if(IsEmpty())
	{
		return NAME_None;
	}
	return FName(*ModulePath);
}

FStringView FRigHierarchyModulePath::GetModuleName() const
{
	const int32 Len = ModulePath.Len();
	if(Len >= 3)
	{
		int32 SeparatorIndex = INDEX_NONE;
		if(ModulePath.FindLastChar(ModuleNameSuffixChar, SeparatorIndex))
		{
			return FStringView(ModulePath).Left(SeparatorIndex);
		}
		if(ModulePath.FindLastChar(NamespaceSeparatorChar_Deprecated, SeparatorIndex))
		{
			return FStringView(ModulePath).Left(SeparatorIndex);
		}
	};
	static const FString EmptyString;
	return EmptyString;
}

const FString& FRigHierarchyModulePath::GetModuleNameString() const
{
	if(!CachedModuleNameString.IsSet())
	{
		CachedModuleNameString = FString(GetModuleName());
	}
	return CachedModuleNameString.GetValue();
}

const FName& FRigHierarchyModulePath::GetModuleFName() const
{
	if(!CachedModuleFName.IsSet())
	{
		const FString& ModuleNameString = GetModuleNameString();
		if(ModuleNameString.IsEmpty())
		{
			CachedModuleFName = NAME_None;
		}
		else
		{
			CachedModuleFName = *ModuleNameString;
		}
	}
	return CachedModuleFName.GetValue();
}

FStringView FRigHierarchyModulePath::GetModulePrefix() const
{
	const int32 Len = ModulePath.Len();
	if(Len >= 3)
	{
		int32 SeparatorIndex = INDEX_NONE;
		if(ModulePath.FindLastChar(ModuleNameSuffixChar, SeparatorIndex))
		{
			return FStringView(ModulePath).Left(SeparatorIndex + 1);
		}
		if(ModulePath.FindLastChar(NamespaceSeparatorChar_Deprecated, SeparatorIndex))
		{
			return FStringView(ModulePath).Left(SeparatorIndex + 1);
		}
	};
	static const FString EmptyString;
	return EmptyString;
}

FString FRigHierarchyModulePath::GetModulePrefixString() const
{
	return FString(GetModulePrefix());
}

FStringView FRigHierarchyModulePath::GetElementName() const
{
	const int32 Len = ModulePath.Len();
	if(Len >= 3)
	{
		int32 SeparatorIndex = INDEX_NONE;
		if(ModulePath.FindLastChar(ModuleNameSuffixChar, SeparatorIndex))
		{
			return FStringView(ModulePath).Mid(SeparatorIndex+1);
		}
		if(ModulePath.FindLastChar(NamespaceSeparatorChar_Deprecated, SeparatorIndex))
		{
			return FStringView(ModulePath).Mid(SeparatorIndex+1);
		}
	};
	static const FString EmptyString;
	return EmptyString;
}

const FString& FRigHierarchyModulePath::GetElementNameString() const
{
	if(!CachedElementNameString.IsSet())
	{
		CachedElementNameString = FString(GetElementName());
	}
	return CachedElementNameString.GetValue();
}

const FName& FRigHierarchyModulePath::GetElementFName() const
{
	if(!CachedElementFName.IsSet())
	{
		const FString& ElementNameString = GetElementNameString();
		if(ElementNameString.IsEmpty())
		{
			CachedElementFName = NAME_None;
		}
		else
		{
			CachedElementFName = *ElementNameString;
		}
	}
	return CachedElementFName.GetValue();
}

FRigHierarchyModulePath FRigHierarchyModulePath::Join(const FString& InModuleName, const FString& InElementName)
{
	if(InModuleName.IsEmpty() || InElementName.IsEmpty())
	{
		return FRigHierarchyModulePath(); 
	}
	if(InModuleName.EndsWith(ModuleNameSuffix))
	{
		return InModuleName + InElementName;
	}
	FRigHierarchyModulePath Result(InModuleName + ModuleNameSuffix + InElementName);
	Result.CachedModuleNameString = InModuleName;
	Result.CachedElementNameString = InElementName;
	return Result;
}

FRigHierarchyModulePath FRigHierarchyModulePath::Join(const FName& InModuleFName, const FName& InElementFName)
{
	if(InModuleFName.IsNone() || InElementFName.IsNone())
	{
		return FRigHierarchyModulePath(); 
	}
	FRigHierarchyModulePath Result = Join(InModuleFName.ToString(), InElementFName.ToString());
	Result.CachedModuleFName = InModuleFName;
	Result.CachedElementFName = InElementFName;
	return Result;
}

bool FRigHierarchyModulePath::Split(FStringView* OutModuleName, FStringView* OutElementName) const
{
	if(CachedModuleNameString.IsSet() && CachedElementNameString.IsSet())
	{
		if(OutModuleName)
		{
			*OutModuleName = CachedModuleNameString.GetValue();
		}
		if(OutElementName)
		{
			*OutElementName = CachedElementNameString.GetValue();
		}
		return true;
	}
	
	const int32 Len = ModulePath.Len();
	if(Len >= 3)
	{
		int32 SeparatorIndex = INDEX_NONE;
		if(ModulePath.FindLastChar(ModuleNameSuffixChar, SeparatorIndex))
		{
			if(OutModuleName)
			{
				*OutModuleName = FStringView(ModulePath).Left(SeparatorIndex); 
			}
			if(OutElementName)
			{
				*OutElementName = FStringView(ModulePath).Mid(SeparatorIndex+1); 
			}
			return true;
		}
		if(ModulePath.FindLastChar(NamespaceSeparatorChar_Deprecated, SeparatorIndex))
		{
			if(OutModuleName)
			{
				*OutModuleName = FStringView(ModulePath).Left(SeparatorIndex); 
			}
			if(OutElementName)
			{
				*OutElementName = FStringView(ModulePath).Mid(SeparatorIndex+1); 
			}
			return true;
		}
	};
	return false;
}

bool FRigHierarchyModulePath::Split(FString* OutModuleName, FString* OutElementName) const
{
	if(CachedModuleNameString.IsSet() && CachedElementNameString.IsSet())
	{
		if(OutModuleName)
		{
			*OutModuleName = CachedModuleNameString.GetValue();
		}
		if(OutElementName)
		{
			*OutElementName = CachedElementNameString.GetValue();
		}
		return true;
	}

	FStringView ModuleName, ElementName;
	if(Split(&ModuleName, &ElementName))
	{
		if(OutModuleName)
		{
			*OutModuleName = ModuleName;
		}
		if(OutElementName)
		{
			*OutElementName = ElementName;
		}
		return true;
	}
	return false;
}

FRigHierarchyModulePath FRigHierarchyModulePath::ConvertToModuleNameFormat(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName) const
{
	FRigHierarchyModulePath Result = *this;
	(void)Result.ConvertToModuleNameFormatInline(InModulePathToModuleName);
	return Result;
}

bool FRigHierarchyModulePath::ConvertToModuleNameFormatInline(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName)
{
	if(UsesNameSpaceFormat())
	{
		FString Left, Right;
		if(Split(&Left, &Right))
		{
			if(InModulePathToModuleName)
			{
				if(const FName* RemappedModuleName = InModulePathToModuleName->Find(FString(Left)))
				{
					const FString NameString = RemappedModuleName->ToString();
					*this = Join(NameString, Right);
					return true;
				}
			}

			const FRigHierarchyModulePath OldModulePath = Left;
			if(OldModulePath.UsesNameSpaceFormat())
			{
				*this = Join(FString(OldModulePath.GetElementName()), Right);
			}
			else
			{
				*this = Join(Left, Right);
			}
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////
// FRigElementKey
////////////////////////////////////////////////////////////////////////////////

void FRigElementKey::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		// remove due to FPIEFixupSerializer hitting this checkNoEntry();
	}
}

void FRigElementKey::Save(FArchive& Ar)
{
	static const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();

	FName TypeName = ElementTypeEnum->GetNameByValue((int64)Type);
	Ar << TypeName;
	Ar << Name;
}

void FRigElementKey::Load(FArchive& Ar)
{
	static const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();

	FName TypeName;
	Ar << TypeName;

	const int64 TypeValue = ElementTypeEnum->GetValueByName(TypeName);
	Type = (ERigElementType)TypeValue;

	Ar << Name;
}

FString FRigElementKey::ToPythonString() const
{
#if WITH_EDITOR
	return FString::Printf(TEXT("unreal.RigElementKey(type=%s, name='%s')"),
		*RigVMPythonUtils::EnumValueToPythonString<ERigElementType>((int64)Type),
		*Name.ToString());
#else
	return FString();
#endif
}

FRigElementKey FRigElementKey::ConvertToModuleNameFormat(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName) const
{
	FRigElementKey PatchedKey = *this;
	(void)PatchedKey.ConvertToModuleNameFormatInline(InModulePathToModuleName);
	return PatchedKey;
}

bool FRigElementKey::ConvertToModuleNameFormatInline(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName)
{
	FRigHierarchyModulePath ModulePath = Name.ToString();
	if(ModulePath.ConvertToModuleNameFormatInline(InModulePathToModuleName))
	{
		Name = *ModulePath.GetPath();
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////
// FRigComponentKey
////////////////////////////////////////////////////////////////////////////////

void FRigComponentKey::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		// remove due to FPIEFixupSerializer hitting this checkNoEntry();
	}
}

void FRigComponentKey::Save(FArchive& Ar)
{
	Ar << Name;
	Ar << ElementKey;
}

void FRigComponentKey::Load(FArchive& Ar)
{
	Ar << Name;
	Ar << ElementKey;
}

bool FRigComponentKey::IsValid() const
{
	return Name.IsValid() && Name != NAME_None && ElementKey.IsValid();
}

FString FRigComponentKey::ToPythonString() const
{
#if WITH_EDITOR
	return FString::Printf(TEXT("unreal.RigComponentKey(element_key=%s, name='%s')"),
		*ElementKey.ToPythonString(),
		*Name.ToString());
#else
	return FString();
#endif
}

FRigComponentKey FRigComponentKey::ConvertToModuleNameFormat(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName) const
{
	FRigComponentKey Result = *this;
	(void)Result.ConvertToModuleNameFormatInline(InModulePathToModuleName);
	return Result;
}

bool FRigComponentKey::ConvertToModuleNameFormatInline(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName)
{
	return ElementKey.ConvertToModuleNameFormatInline(InModulePathToModuleName);
}

////////////////////////////////////////////////////////////////////////////////
// FRigHierarchyKey
////////////////////////////////////////////////////////////////////////////////

const FName& FRigHierarchyKey::GetFName() const
{
	if(IsElement())
	{
		return Element.GetValue().Name;
	}
	if(IsComponent())
	{
		return Component.GetValue().Name;
	}
	static const FName InvalidName(NAME_None);
	return InvalidName;;
}

void FRigHierarchyKey::Serialize(FArchive& Ar)
{
	if(Ar.IsLoading())
	{
		bool bIsElement = false;
		Ar << bIsElement;
		if(bIsElement)
		{
			FRigElementKey Key;
			Ar << Key;
			Element = Key;
		}
		else
		{
			Element.Reset();
		}

		bool bIsComponent = false;
		Ar << bIsComponent;
		if(bIsComponent)
		{
			FRigComponentKey Key;
			Ar << Key;
			Component = Key;
		}
		else
		{
			Component.Reset();
		}
	}
	else if(Ar.IsSaving())
	{
		bool bIsElement = IsElement();
		Ar << bIsElement;
		if(bIsElement)
		{
			FRigElementKey Key = GetElement();
			Ar << Key;
		}

		bool bIsComponent = IsComponent();
		Ar << bIsComponent;
		if(bIsComponent)
		{
			FRigComponentKey Key = GetComponent();
			Ar << Key;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigElementKeyAndIndex
////////////////////////////////////////////////////////////////////////////////

FRigElementKeyAndIndex::FRigElementKeyAndIndex(const FRigBaseElement* InElement)
: Key(InElement->Key)
, Index(InElement->Index)
{
}

////////////////////////////////////////////////////////////////////////////////
// FRigElementKeyCollection
////////////////////////////////////////////////////////////////////////////////

FRigElementKeyCollection FRigElementKeyCollection::MakeFromChildren(
	URigHierarchy* InHierarchy,
	const FRigElementKey& InParentKey,
	bool bRecursive,
	bool bIncludeParent,
	uint8 InElementTypes)
{
	check(InHierarchy);

	FRigElementKeyCollection Collection;

	int32 Index = InHierarchy->GetIndex(InParentKey);
	if (Index == INDEX_NONE)
	{
		return Collection;
	}

	if (bIncludeParent)
	{
		Collection.AddUnique(InParentKey);
	}

	TArray<FRigElementKey> ParentKeys;
	ParentKeys.Add(InParentKey);

	bool bAddBones = (InElementTypes & (uint8)ERigElementType::Bone) == (uint8)ERigElementType::Bone;
	bool bAddControls = (InElementTypes & (uint8)ERigElementType::Control) == (uint8)ERigElementType::Control;
	bool bAddNulls = (InElementTypes & (uint8)ERigElementType::Null) == (uint8)ERigElementType::Null;
	bool bAddCurves = (InElementTypes & (uint8)ERigElementType::Curve) == (uint8)ERigElementType::Curve;

	for (int32 ParentIndex = 0; ParentIndex < ParentKeys.Num(); ParentIndex++)
	{
		const FRigElementKey ParentKey = ParentKeys[ParentIndex];
		TArray<FRigElementKey> Children = InHierarchy->GetChildren(ParentKey);
		for(const FRigElementKey& Child : Children)
		{
			if((InElementTypes & (uint8)Child.Type) == (uint8)Child.Type)
			{
				const int32 PreviousSize = Collection.Num();
				if(PreviousSize == Collection.AddUnique(Child))
				{
					if(bRecursive)
					{
						ParentKeys.Add(Child);
					}
				}
			}
		}
	}

	return Collection;
}

FRigElementKeyCollection FRigElementKeyCollection::MakeFromName(
	URigHierarchy* InHierarchy,
	const FName& InPartialName,
	uint8 InElementTypes
)
{
	if (InPartialName.IsNone())
	{
		return MakeFromCompleteHierarchy(InHierarchy, InElementTypes);
	}

	check(InHierarchy);

	constexpr bool bTraverse = true;

	const FString PartialNameString = InPartialName.ToString();
	const FWildcardString WildcardString(PartialNameString);
	if(WildcardString.ContainsWildcards())
	{
		return InHierarchy->GetKeysByPredicate([WildcardString, InElementTypes](const FRigBaseElement& InElement) -> bool
		{
			return InElement.IsTypeOf(static_cast<ERigElementType>(InElementTypes)) &&
				   WildcardString.IsMatch(InElement.GetName());
		}, bTraverse);
	}
	
	return InHierarchy->GetKeysByPredicate([PartialNameString, InElementTypes](const FRigBaseElement& InElement) -> bool
	{
		return InElement.IsTypeOf(static_cast<ERigElementType>(InElementTypes)) &&
			   InElement.GetName().Contains(PartialNameString);
	}, bTraverse);
}

FRigElementKeyCollection FRigElementKeyCollection::MakeFromChain(
	URigHierarchy* InHierarchy,
	const FRigElementKey& InFirstItem,
	const FRigElementKey& InLastItem,
	bool bReverse
)
{
	check(InHierarchy);

	FRigElementKeyCollection Collection;

	int32 FirstIndex = InHierarchy->GetIndex(InFirstItem);
	int32 LastIndex = InHierarchy->GetIndex(InLastItem);

	if (FirstIndex == INDEX_NONE || LastIndex == INDEX_NONE)
	{
		return Collection;
	}

	FRigElementKey LastKey = InLastItem;
	while (LastKey.IsValid() && LastKey != InFirstItem)
	{
		Collection.Keys.Add(LastKey);
		LastKey = InHierarchy->GetFirstParent(LastKey);
	}

	if (LastKey != InFirstItem)
	{
		Collection.Reset();
	}
	else
	{
		Collection.AddUnique(InFirstItem);
	}

	if (!bReverse)
	{
		Algo::Reverse(Collection.Keys);
	}

	return Collection;
}

FRigElementKeyCollection FRigElementKeyCollection::MakeFromCompleteHierarchy(
	URigHierarchy* InHierarchy,
	uint8 InElementTypes
)
{
	check(InHierarchy);

	FRigElementKeyCollection Collection(InHierarchy->GetAllKeys(true));
	return Collection.FilterByType(InElementTypes);
}

FRigElementKeyCollection FRigElementKeyCollection::MakeUnion(const FRigElementKeyCollection& A, const FRigElementKeyCollection& B, bool bAllowDuplicates)
{
	FRigElementKeyCollection Collection;
	for (const FRigElementKey& Key : A)
	{
		Collection.Add(Key);
	}
	for (const FRigElementKey& Key : B)
	{
		if(bAllowDuplicates)
		{
			Collection.Add(Key);
		}
		else
		{
			Collection.AddUnique(Key);
		}
	}
	return Collection;
}

FRigElementKeyCollection FRigElementKeyCollection::MakeIntersection(const FRigElementKeyCollection& A, const FRigElementKeyCollection& B)
{
	FRigElementKeyCollection Collection;
	for (const FRigElementKey& Key : A)
	{
		if (B.Contains(Key))
		{
			Collection.Add(Key);
		}
	}
	return Collection;
}

FRigElementKeyCollection FRigElementKeyCollection::MakeDifference(const FRigElementKeyCollection& A, const FRigElementKeyCollection& B)
{
	FRigElementKeyCollection Collection;
	for (const FRigElementKey& Key : A)
	{
		if (!B.Contains(Key))
		{
			Collection.Add(Key);
		}
	}
	return Collection;
}

FRigElementKeyCollection FRigElementKeyCollection::MakeReversed(const FRigElementKeyCollection& InCollection)
{
	FRigElementKeyCollection Reversed = InCollection;
	Algo::Reverse(Reversed.Keys);
	return Reversed;
}

FRigElementKeyCollection FRigElementKeyCollection::FilterByType(uint8 InElementTypes) const
{
	FRigElementKeyCollection Collection;
	for (const FRigElementKey& Key : *this)
	{
		if ((InElementTypes & (uint8)Key.Type) == (uint8)Key.Type)
		{
			Collection.Add(Key);
		}
	}
	return Collection;
}

FRigElementKeyCollection FRigElementKeyCollection::FilterByName(const FName& InPartialName) const
{
	FString SearchToken = InPartialName.ToString();

	FRigElementKeyCollection Collection;
	for (const FRigElementKey& Key : *this)
	{
		if (Key.Name == InPartialName)
		{
			Collection.Add(Key);
		}
		else if (Key.Name.ToString().Contains(SearchToken, ESearchCase::CaseSensitive, ESearchDir::FromStart))
		{
			Collection.Add(Key);
		}
	}
	return Collection;
}

FArchive& operator<<(FArchive& Ar, FRigControlValue& Value)
{
	Ar <<  Value.FloatStorage.Float00;
	Ar <<  Value.FloatStorage.Float01;
	Ar <<  Value.FloatStorage.Float02;
	Ar <<  Value.FloatStorage.Float03;
	Ar <<  Value.FloatStorage.Float10;
	Ar <<  Value.FloatStorage.Float11;
	Ar <<  Value.FloatStorage.Float12;
	Ar <<  Value.FloatStorage.Float13;
	Ar <<  Value.FloatStorage.Float20;
	Ar <<  Value.FloatStorage.Float21;
	Ar <<  Value.FloatStorage.Float22;
	Ar <<  Value.FloatStorage.Float23;
	Ar <<  Value.FloatStorage.Float30;
	Ar <<  Value.FloatStorage.Float31;
	Ar <<  Value.FloatStorage.Float32;
	Ar <<  Value.FloatStorage.Float33;
	Ar <<  Value.FloatStorage.Float00_2;
	Ar <<  Value.FloatStorage.Float01_2;
	Ar <<  Value.FloatStorage.Float02_2;
	Ar <<  Value.FloatStorage.Float03_2;
	Ar <<  Value.FloatStorage.Float10_2;
	Ar <<  Value.FloatStorage.Float11_2;
	Ar <<  Value.FloatStorage.Float12_2;
	Ar <<  Value.FloatStorage.Float13_2;
	Ar <<  Value.FloatStorage.Float20_2;
	Ar <<  Value.FloatStorage.Float21_2;
	Ar <<  Value.FloatStorage.Float22_2;
	Ar <<  Value.FloatStorage.Float23_2;
	Ar <<  Value.FloatStorage.Float30_2;
	Ar <<  Value.FloatStorage.Float31_2;
	Ar <<  Value.FloatStorage.Float32_2;
	Ar <<  Value.FloatStorage.Float33_2;

	return Ar;
}

////////////////////////////////////////////////////////////////////////////////
// FRigElementResolveResult
////////////////////////////////////////////////////////////////////////////////

bool FRigElementResolveResult::IsValid() const
{
	return State == ERigElementResolveState::PossibleTarget ||
		State == ERigElementResolveState::DefaultTarget;
}

void FRigElementResolveResult::SetInvalidTarget(const FText& InMessage)
{
	State = ERigElementResolveState::InvalidTarget;
	Message = InMessage;
}

void FRigElementResolveResult::SetPossibleTarget(const FText& InMessage)
{
	State = ERigElementResolveState::PossibleTarget;
	Message = InMessage;
}

void FRigElementResolveResult::SetDefaultTarget(const FText& InMessage)
{
	State = ERigElementResolveState::DefaultTarget;
	Message = InMessage;
}

////////////////////////////////////////////////////////////////////////////////
// FModularRigResolveResult
////////////////////////////////////////////////////////////////////////////////

bool FModularRigResolveResult::IsValid() const
{
	return State == EModularRigResolveState::Success && !Matches.IsEmpty();
}

bool FModularRigResolveResult::ContainsMatch(const FRigElementKey& InKey, FString* OutErrorMessage) const
{
	if(Matches.ContainsByPredicate([InKey](const FRigElementResolveResult& InMatch) -> bool
	{
		return InMatch.GetKey() == InKey;
	}))
	{
		return true;
	}

	if(OutErrorMessage)
	{
		if(const FRigElementResolveResult* Mismatch = Excluded.FindByPredicate([InKey](const FRigElementResolveResult& InMatch) -> bool
		{
			return InMatch.GetKey() == InKey;
		}))
		{
			*OutErrorMessage = Mismatch->GetMessage().ToString();
		}
	}
	
	return false;
}

const FRigElementResolveResult* FModularRigResolveResult::FindMatch(const FRigElementKey& InKey) const
{
	return Matches.FindByPredicate([InKey](const FRigElementResolveResult& InMatch) -> bool
	{
		return InMatch.GetKey() == InKey;
	});
}

const FRigElementResolveResult* FModularRigResolveResult::GetDefaultMatch() const
{
	return Matches.FindByPredicate([](const FRigElementResolveResult& Match)
	{
		return Match.GetState() == ERigElementResolveState::DefaultTarget;
	});
}
