// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyElements.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigControlHierarchy.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "ControlRigObjectVersion.h"
#include "ControlRigGizmoLibrary.h"
#include "AnimationCoreLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyElements)

////////////////////////////////////////////////////////////////////////////////
// FRigBaseElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigBaseElement::ElementTypeIndex = BaseElement;

FRigBaseElement::~FRigBaseElement()
{
	if (Owner)
	{
		Owner->RemoveAllMetadataForElement(this);
	}
}

void FRigBaseElement::Serialize(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		Load(Ar, InSettings);
	}
	else
	{
		Save(Ar, InSettings);
	}
}

void FRigBaseElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		Ar << Key;
	}
}

void FRigBaseElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	checkf(Owner != nullptr, TEXT("Loading should not happen on a rig element without an owner"));
	
	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		FRigElementKey LoadedKey;
	
		Ar << LoadedKey;

		ensure(LoadedKey.Type == Key.Type);
		Key = LoadedKey;

		ChildCacheIndex = INDEX_NONE;
		CachedNameString.Reset();

		if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::HierarchyElementMetadata &&
			Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RigHierarchyStoresElementMetadata)
		{
			static const UEnum* MetadataTypeEnum = StaticEnum<ERigMetadataType>();

			int32 MetadataNum = 0;
			Ar << MetadataNum;

			for(int32 MetadataIndex = 0; MetadataIndex < MetadataNum; MetadataIndex++)
			{
				FName MetadataName;
				FName MetadataTypeName;
				Ar << MetadataName;
				Ar << MetadataTypeName;

				const ERigMetadataType MetadataType = static_cast<ERigMetadataType>(MetadataTypeEnum->GetValueByName(MetadataTypeName));

				FRigBaseMetadata* Md = Owner->GetMetadataForElement(this, MetadataName, MetadataType, false);
				Md->Serialize(Ar);
			}
		}
	}
}

FRigBaseMetadata* FRigBaseElement::GetMetadata(const FName& InName, ERigMetadataType InType)
{
	if (!Owner)
	{
		return nullptr;
	}
	return Owner->FindMetadataForElement(this, InName, InType);
}


const FRigBaseMetadata* FRigBaseElement::GetMetadata(const FName& InName, ERigMetadataType InType) const
{
	if (Owner == nullptr)
	{
		return nullptr;
	}
	return Owner->FindMetadataForElement(this, InName, InType);
}


bool FRigBaseElement::SetMetadata(const FName& InName, ERigMetadataType InType, const void* InData, int32 InSize)
{
	if (Owner)
	{
		constexpr bool bNotify = true;
		if (FRigBaseMetadata* Metadata = Owner->GetMetadataForElement(this, InName, InType, bNotify))
		{
			Metadata->SetValueData(InData, InSize);
			return true;
		}
	}
	return false;
}

FRigBaseMetadata* FRigBaseElement::SetupValidMetadata(const FName& InName, ERigMetadataType InType)
{
	if (Owner == nullptr)
	{
		return nullptr;
	}
	constexpr bool bNotify = true;
	return Owner->GetMetadataForElement(this, InName, InType, bNotify);
}


bool FRigBaseElement::RemoveMetadata(const FName& InName)
{
	if (Owner == nullptr)
	{
		return false;
	}
	return Owner->RemoveMetadataForElement(this, InName);
}

bool FRigBaseElement::RemoveAllMetadata()
{
	if (Owner == nullptr)
	{
		return false;
	}
	return Owner->RemoveAllMetadataForElement(this);
}

void FRigBaseElement::NotifyMetadataTagChanged(const FName& InTag, bool bAdded)
{
	if (Owner)
	{
		Owner->OnMetadataTagChanged(Key, InTag, bAdded);
	}
}

int32 FRigBaseElement::NumComponents() const
{
	return ComponentIndices.Num();
}

const FRigBaseComponent* FRigBaseElement::GetComponent(int32 InIndex) const
{
	return const_cast<FRigBaseElement*>(this)->GetComponent(InIndex);
}

FRigBaseComponent* FRigBaseElement::GetComponent(int32 InIndex)
{
	if(ComponentIndices.IsValidIndex(InIndex))
	{
		if(URigHierarchy* Hierarchy = GetOwner())
		{
			return Hierarchy->GetComponent(ComponentIndices[InIndex]);
		}
	}
	return nullptr;
}

const FRigBaseComponent* FRigBaseElement::FindComponent(const FName& InName) const
{
	return const_cast<FRigBaseElement*>(this)->FindComponent(InName);
}

FRigBaseComponent* FRigBaseElement::FindComponent(const FName& InName)
{
	if(!ComponentIndices.IsEmpty())
	{
		if(URigHierarchy* Hierarchy = GetOwner())
		{
			for(int32 ComponentIndex : ComponentIndices)
			{
				if(FRigBaseComponent* Component = Hierarchy->GetComponent(ComponentIndex))
				{
					if(Component->GetName() == InName)
					{
						return Component;
					}
				}
			}
		}
	}
	return nullptr;
}

const FRigBaseComponent* FRigBaseElement::GetFirstComponent(const UScriptStruct* InComponentStruct) const
{
	return const_cast<FRigBaseElement*>(this)->GetFirstComponent(InComponentStruct);
}

FRigBaseComponent* FRigBaseElement::GetFirstComponent(const UScriptStruct* InComponentStruct)
{
	if(!ComponentIndices.IsEmpty())
	{
		if(URigHierarchy* Hierarchy = GetOwner())
		{
			for(const int32& ComponentIndex : ComponentIndices)
			{
				if(FRigBaseComponent* Component = Hierarchy->GetComponent(ComponentIndex))
				{
					if(Component->IsA(InComponentStruct))
					{
						return Component;
					}
				}
			}
		}
	}
	return nullptr;
}

TArray<FRigComponentKey> FRigBaseElement::GetComponentKeys() const
{
	TArray<FRigComponentKey> Keys;
	Keys.Reserve(NumComponents());
	for(int32 ComponentIndex = 0; ComponentIndex < NumComponents(); ComponentIndex++)
	{
		const FRigBaseComponent* Component = GetComponent(ComponentIndex);
		if(ensure(Component))
		{
			Keys.Add(Component->GetKey());
		}
	}
	return Keys;
}


void FRigBaseElement::InitializeFrom(const FRigBaseElement* InOther)
{
	Key = InOther->Key;
	Index = InOther->Index;
	SubIndex = InOther->SubIndex;
	CreatedAtInstructionIndex = InOther->CreatedAtInstructionIndex;
	SpawnIndex = InOther->SpawnIndex;
	bSelected = false;
}


void FRigBaseElement::CopyFrom(const FRigBaseElement* InOther)
{
}

////////////////////////////////////////////////////////////////////////////////
// FRigCompactTransform
////////////////////////////////////////////////////////////////////////////////

FRigCompactTransform::FRigCompactTransform(FTransform& InTransform)
: Transform(InTransform)
{
}

void FRigCompactTransform::Serialize(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, ERepresentation* OutRepresentation)
{
	if(Ar.IsLoading())
	{
		Load(Ar, InSettings, OutRepresentation);
	}
	else if(Ar.IsSaving())
	{
		Save(Ar, InSettings, OutRepresentation);
	}
}

void FRigCompactTransform::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, ERepresentation* OutRepresentation)
{
	uint8 State = Float_Zero_Identity_One;
	if(!InSettings.bStoreCompactTransforms)
	{
		State = Double_Complete;
		if(OutRepresentation)
		{
			*OutRepresentation = (ERepresentation)State;
		}
		Ar << State;
		Ar << Transform;
		return;
	}

	FVector3f Position(Transform.GetLocation());
	if(!Position.IsNearlyZero(UE_SMALL_NUMBER))
	{
		State += 6;
	}
	FQuat4f Rotation(Transform.GetRotation());
	if(!Rotation.IsIdentity(UE_SMALL_NUMBER))
	{
		State += 3;
	}
	FVector3f Scale(Transform.GetScale3D());
	float ScaleX = Scale.X;

	if(!Scale.IsUniform(UE_SMALL_NUMBER))
	{
		State += 2;
	}
	else if(!FMath::IsNearlyEqual(Scale.X, 1.f, UE_SMALL_NUMBER))
	{
		State += 1;
	}

	if(OutRepresentation)
	{
		*OutRepresentation = (ERepresentation)State;
	}

	Ar << State;
	switch(State)
	{
		case Float_Zero_Identity_One:
		{
			// don't store anything
			break;
		}
		case Float_Zero_Identity_Uniform:
		{
			Ar << ScaleX;
			break;
		}
		case Float_Zero_Identity_NonUniform:
		{
			Ar << Scale;
			break;
		}
		case Float_Zero_Quat_One:
		{
			Ar << Rotation;
			break;
		}
		case Float_Zero_Quat_Uniform:
		{
			Ar << Rotation;
			Ar << ScaleX;
			break;
		}
		case Float_Zero_Quat_NonUniform:
		{
			Ar << Rotation;
			Ar << Scale;
			break;
		}
		case Float_Position_Identity_One:
		{
			Ar << Position;
			break;
		}
		case Float_Position_Identity_Uniform:
		{
			Ar << Position;
			Ar << ScaleX;
			break;
		}
		case Float_Position_Identity_NonUniform:
		{
			Ar << Position;
			Ar << Scale;
			break;
		}
		case Float_Position_Quat_One:
		{
			Ar << Position;
			Ar << Rotation;
			break;
		}
		case Float_Position_Quat_Uniform:
		{
			Ar << Position;
			Ar << Rotation;
			Ar << ScaleX;
			break;
		}
		case Float_Position_Quat_NonUniform:
		{
			Ar << Position;
			Ar << Rotation;
			Ar << Scale;
			break;
		}
		case Double_Complete:
		default:
		{
			Ar << Transform;
			break;
		}
	}
}

void FRigCompactTransform::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, ERepresentation* OutRepresentation)
{
	if(InSettings.ControlRigVersion < FControlRigObjectVersion::RigHierarchyCompactTransformSerialization)
	{
		Ar << Transform;
		return;
	}

	uint8 State = Float_Zero_Identity_One;
	Ar << State;

	if(OutRepresentation)
	{
		*OutRepresentation = (ERepresentation)State;
	}

	if(State == Double_Complete)
	{
		Ar << Transform;
		return;
	}
	
	Transform = FTransform::Identity;
	FVector3f Position = FVector3f::ZeroVector;
	FQuat4f Rotation = FQuat4f::Identity;
	FVector3f Scale = FVector3f::OneVector;
	float ScaleX = 1.f;

	switch(State)
	{
		case Float_Zero_Identity_One:
		{
			// don't load anything
			break;
		}
		case Float_Zero_Identity_Uniform:
		{
			Ar << ScaleX;
			Transform.SetScale3D({ScaleX, ScaleX, ScaleX});
			break;
		}
		case Float_Zero_Identity_NonUniform:
		{
			Ar << Scale;
			Transform.SetScale3D(FVector(Scale));
			break;
		}
		case Float_Zero_Quat_One:
		{
			Ar << Rotation;
			Transform.SetRotation(FQuat(Rotation));
			break;
		}
		case Float_Zero_Quat_Uniform:
		{
			Ar << Rotation;
			Ar << ScaleX;
			Transform.SetRotation(FQuat(Rotation));
			Transform.SetScale3D({ScaleX, ScaleX, ScaleX});
			break;
		}
		case Float_Zero_Quat_NonUniform:
		{
			Ar << Rotation;
			Ar << Scale;
			Transform.SetRotation(FQuat(Rotation));
			Transform.SetScale3D(FVector(Scale));
			break;
		}
		case Float_Position_Identity_One:
		{
			Ar << Position;
			Transform.SetLocation(FVector(Position));
			break;
		}
		case Float_Position_Identity_Uniform:
		{
			Ar << Position;
			Ar << ScaleX;
			Transform.SetLocation(FVector(Position));
			Transform.SetScale3D({ScaleX, ScaleX, ScaleX});
			break;
		}
		case Float_Position_Identity_NonUniform:
		{
			Ar << Position;
			Ar << Scale;
			Transform.SetLocation(FVector(Position));
			Transform.SetScale3D(FVector(Scale));
			break;
		}
		case Float_Position_Quat_One:
		{
			Ar << Position;
			Ar << Rotation;
			Transform.SetLocation(FVector(Position));
			Transform.SetRotation(FQuat(Rotation));
			break;
		}
		case Float_Position_Quat_Uniform:
		{
			Ar << Position;
			Ar << Rotation;
			Ar << ScaleX;
			Transform.SetLocation(FVector(Position));
			Transform.SetRotation(FQuat(Rotation));
			Transform.SetScale3D({ScaleX, ScaleX, ScaleX});
			break;
		}
		case Float_Position_Quat_NonUniform:
		{
			Ar << Position;
			Ar << Rotation;
			Ar << Scale;
			Transform.SetLocation(FVector(Position));
			Transform.SetRotation(FQuat(Rotation));
			Transform.SetScale3D(FVector(Scale));
			break;
		}
		case Double_Complete:
		default:
		{
			Ar << Transform;
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigTransformDirtyState
////////////////////////////////////////////////////////////////////////////////

const bool& FRigTransformDirtyState::Get() const
{
	if(Storage)
	{
		return *Storage;
	}
	ensure(false);
	static constexpr bool bDefaultDirtyFlag = false;
	return bDefaultDirtyFlag;
}

bool& FRigTransformDirtyState::Get()
{
	if(Storage)
	{
		return *Storage;
	}
	ensure(false);
	static bool bDefaultDirtyFlag = false;
	return bDefaultDirtyFlag;
}

bool FRigTransformDirtyState::Set(bool InDirty)
{
	if(Storage)
	{
		if(*Storage != InDirty)
		{
			*Storage = InDirty;
			return true;
		}
	}
	return false;
}

FRigTransformDirtyState& FRigTransformDirtyState::operator=(const FRigTransformDirtyState& InOther)
{
	if(Storage)
	{
		*Storage = InOther.Get();
	}
	return *this;
}

void FRigTransformDirtyState::LinkStorage(const TArrayView<bool>& InStorage)
{
	if(InStorage.IsValidIndex(StorageIndex))
	{
		Storage = InStorage.GetData() + StorageIndex;
	}
}

void FRigTransformDirtyState::UnlinkStorage(FRigReusableElementStorage<bool>& InStorage)
{
	InStorage.Deallocate(StorageIndex, &Storage);
}

////////////////////////////////////////////////////////////////////////////////
// FRigLocalAndGlobalDirtyState
////////////////////////////////////////////////////////////////////////////////

FRigLocalAndGlobalDirtyState& FRigLocalAndGlobalDirtyState::operator=(const FRigLocalAndGlobalDirtyState& InOther)
{
	Local = InOther.Local;
	Global = InOther.Global;
	return *this;
}

void FRigLocalAndGlobalDirtyState::LinkStorage(const TArrayView<bool>& InStorage)
{
	Local.LinkStorage(InStorage);
	Global.LinkStorage(InStorage);
}

void FRigLocalAndGlobalDirtyState::UnlinkStorage(FRigReusableElementStorage<bool>& InStorage)
{
	Local.UnlinkStorage(InStorage);
	Global.UnlinkStorage(InStorage);
}

////////////////////////////////////////////////////////////////////////////////
// FRigCurrentAndInitialDirtyState
////////////////////////////////////////////////////////////////////////////////

FRigCurrentAndInitialDirtyState& FRigCurrentAndInitialDirtyState::operator=(const FRigCurrentAndInitialDirtyState& InOther)
{
	Current = InOther.Current;
	Initial = InOther.Initial;
	return *this;
}

void FRigCurrentAndInitialDirtyState::LinkStorage(const TArrayView<bool>& InStorage)
{
	Current.LinkStorage(InStorage);
	Initial.LinkStorage(InStorage);
}

void FRigCurrentAndInitialDirtyState::UnlinkStorage(FRigReusableElementStorage<bool>& InStorage)
{
	Current.UnlinkStorage(InStorage);
	Initial.UnlinkStorage(InStorage);
}

////////////////////////////////////////////////////////////////////////////////
// FRigComputedTransform
////////////////////////////////////////////////////////////////////////////////

void FRigComputedTransform::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, const FRigTransformDirtyState& InDirtyState)
{
	FTransform Transform = Get();
	bool bDirty = InDirtyState.Get();

	Ar << bDirty;
	if(!bDirty)
	{
		FRigCompactTransform(Transform).Save(Ar, InSettings);
	}
}

void FRigComputedTransform::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, FRigTransformDirtyState& InDirtyState)
{
	FTransform Transform = FTransform::Identity;
	bool bDirty = false;

	if(InSettings.ControlRigVersion < FControlRigObjectVersion::RigHierarchyCompactTransformSerialization)
	{
		Ar << Transform;
		Ar << bDirty;
	}
	else
	{
		Ar << bDirty;
		if(!bDirty)
		{
			FRigCompactTransform(Transform).Load(Ar, InSettings);
		}
	}

	Set(Transform);
	(void)InDirtyState.Set(bDirty);
}

const FTransform& FRigComputedTransform::Get() const
{
	if(Storage)
	{
		return *Storage;
	}
	ensure(false);
	static const FTransform DefaultTransform = FTransform::Identity;
	return DefaultTransform;
}

FRigComputedTransform& FRigComputedTransform::operator=(const FRigComputedTransform& InOther)
{
	if(Storage)
	{
		*Storage = InOther.Get();
	}
	return *this;
}

void FRigComputedTransform::LinkStorage(const TArrayView<FTransform>& InStorage)
{
	if(InStorage.IsValidIndex(StorageIndex))
	{
		Storage = InStorage.GetData() + StorageIndex;
	}
}

void FRigComputedTransform::UnlinkStorage(FRigReusableElementStorage<FTransform>& InStorage)
{
	InStorage.Deallocate(StorageIndex, &Storage);
}

////////////////////////////////////////////////////////////////////////////////
// FRigLocalAndGlobalTransform
////////////////////////////////////////////////////////////////////////////////

void FRigLocalAndGlobalTransform::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, const FRigLocalAndGlobalDirtyState& InDirtyState)
{
	if(InSettings.bSerializeLocalTransform)
	{
		if(!InSettings.bSerializeGlobalTransform)
		{
			check(!InDirtyState.Local.Get());
		}
		Local.Save(Ar, InSettings, InDirtyState.Local);
	}
	if(InSettings.bSerializeGlobalTransform)
	{
		if(!InSettings.bSerializeLocalTransform)
		{
			check(!InDirtyState.Global.Get());
		}
		Global.Save(Ar, InSettings, InDirtyState.Global);
	}
}

void FRigLocalAndGlobalTransform::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, FRigLocalAndGlobalDirtyState& OutDirtyState)
{
	if(InSettings.bSerializeLocalTransform)
	{
		Local.Load(Ar, InSettings, OutDirtyState.Local);
		if(!InSettings.bSerializeGlobalTransform)
		{
			check(!OutDirtyState.Local.Get());
			OutDirtyState.Global.Set(true);
		}
	}
	if(InSettings.bSerializeGlobalTransform)
	{
		Global.Load(Ar, InSettings, OutDirtyState.Global);
		if(!InSettings.bSerializeLocalTransform)
		{
			check(!OutDirtyState.Global.Get());
			OutDirtyState.Local.Set(true);
		}
	}
}

FRigLocalAndGlobalTransform& FRigLocalAndGlobalTransform::operator=(const FRigLocalAndGlobalTransform& InOther)
{
	Local = InOther.Local;
	Global = InOther.Global;
	return *this;
}

void FRigLocalAndGlobalTransform::LinkStorage(const TArrayView<FTransform>& InStorage)
{
	Local.LinkStorage(InStorage);
	Global.LinkStorage(InStorage);
}

void FRigLocalAndGlobalTransform::UnlinkStorage(FRigReusableElementStorage<FTransform>& InStorage)
{
	Local.UnlinkStorage(InStorage);
	Global.UnlinkStorage(InStorage);
}

////////////////////////////////////////////////////////////////////////////////
// FRigCurrentAndInitialTransform
////////////////////////////////////////////////////////////////////////////////

void FRigCurrentAndInitialTransform::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, const FRigCurrentAndInitialDirtyState& InDirtyState)
{
	if(InSettings.bSerializeCurrentTransform)
	{
		Current.Save(Ar, InSettings, InDirtyState.Current);
	}
	if(InSettings.bSerializeInitialTransform)
	{
		Initial.Save(Ar, InSettings, InDirtyState.Initial);
	}
}

void FRigCurrentAndInitialTransform::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, FRigCurrentAndInitialDirtyState& OutDirtyState)
{
	if(InSettings.bSerializeCurrentTransform)
	{
		Current.Load(Ar, InSettings, OutDirtyState.Current);
	}
	if(InSettings.bSerializeInitialTransform)
	{
		Initial.Load(Ar, InSettings, OutDirtyState.Initial);
		if(!InSettings.bSerializeCurrentTransform)
		{
			Current = Initial;
			OutDirtyState.Current = OutDirtyState.Initial;
		}
	}
}

FRigCurrentAndInitialTransform& FRigCurrentAndInitialTransform::operator=(const FRigCurrentAndInitialTransform& InOther)
{
	Current = InOther.Current;
	Initial = InOther.Initial;
	return *this;
}

void FRigCurrentAndInitialTransform::LinkStorage(const TArrayView<FTransform>& InStorage)
{
	Current.LinkStorage(InStorage);
	Initial.LinkStorage(InStorage);
}

void FRigCurrentAndInitialTransform::UnlinkStorage(FRigReusableElementStorage<FTransform>& InStorage)
{
	Current.UnlinkStorage(InStorage);
	Initial.UnlinkStorage(InStorage);
}

////////////////////////////////////////////////////////////////////////////////
// FRigPreferredEulerAngles
////////////////////////////////////////////////////////////////////////////////

void FRigPreferredEulerAngles::Save(FArchive& Ar)
{
	static const UEnum* RotationOrderEnum = StaticEnum<EEulerRotationOrder>();
	FName RotationOrderName = RotationOrderEnum->GetNameByValue((int64)RotationOrder);
	Ar << RotationOrderName;
	Ar << Current;
	Ar << Initial;
}

void FRigPreferredEulerAngles::Load(FArchive& Ar)
{
	static const UEnum* RotationOrderEnum = StaticEnum<EEulerRotationOrder>();
	FName RotationOrderName;
	Ar << RotationOrderName;
	RotationOrder = (EEulerRotationOrder)RotationOrderEnum->GetValueByName(RotationOrderName);
	Ar << Current;
	Ar << Initial;
}

void FRigPreferredEulerAngles::Reset()
{
	RotationOrder = DefaultRotationOrder;
	Initial = Current = FVector::ZeroVector;
}

FRotator FRigPreferredEulerAngles::GetRotator(bool bInitial) const
{
	return FRotator::MakeFromEuler(GetAngles(bInitial, RotationOrder));
}

FRotator FRigPreferredEulerAngles::SetRotator(const FRotator& InValue, bool bInitial, bool bFixEulerFlips)
{
	SetAngles(InValue.Euler(), bInitial, RotationOrder, bFixEulerFlips);
	return InValue;
}

FVector FRigPreferredEulerAngles::GetAngles(bool bInitial, EEulerRotationOrder InRotationOrder) const
{
	if(RotationOrder == InRotationOrder)
	{
		return Get(bInitial);
	}
	return AnimationCore::ChangeEulerRotationOrder(Get(bInitial), RotationOrder, InRotationOrder);
}

void FRigPreferredEulerAngles::SetAngles(const FVector& InValue, bool bInitial, EEulerRotationOrder InRotationOrder, bool bFixEulerFlips)
{
	FVector Value = InValue;
	if(RotationOrder != InRotationOrder)
	{
		Value = AnimationCore::ChangeEulerRotationOrder(Value, InRotationOrder, RotationOrder);
	}

	if(bFixEulerFlips)
	{
		const FRotator CurrentRotator = FRotator::MakeFromEuler(GetAngles(bInitial, RotationOrder));
		const FRotator InRotator = FRotator::MakeFromEuler(Value);

		//Find Diff of the rotation from current and just add that instead of setting so we can go over/under -180
		FRotator CurrentWinding;
		FRotator CurrentRotRemainder;
		CurrentRotator.GetWindingAndRemainder(CurrentWinding, CurrentRotRemainder);

		FRotator DeltaRot = InRotator - CurrentRotRemainder;
		DeltaRot.Normalize();
		const FRotator FixedValue = CurrentRotator + DeltaRot;

		Get(bInitial) = FixedValue.Euler();
		return;

	}
	
	Get(bInitial) = Value;
}

void FRigPreferredEulerAngles::SetRotationOrder(EEulerRotationOrder InRotationOrder)
{
	if(RotationOrder != InRotationOrder)
	{
		const EEulerRotationOrder PreviousRotationOrder = RotationOrder;
		const FVector PreviousAnglesCurrent = GetAngles(false, RotationOrder);
		const FVector PreviousAnglesInitial = GetAngles(true, RotationOrder);
		RotationOrder = InRotationOrder;
		SetAngles(PreviousAnglesCurrent, false, PreviousRotationOrder);
		SetAngles(PreviousAnglesInitial, true, PreviousRotationOrder);
	}
}

FRotator FRigPreferredEulerAngles::GetRotatorFromQuat(const FQuat& InQuat) const
{
	FVector Vector = AnimationCore::EulerFromQuat(InQuat, RotationOrder, true);
	return FRotator::MakeFromEuler(Vector);
}

FQuat FRigPreferredEulerAngles::GetQuatFromRotator(const FRotator& InRotator) const
{
	FVector Vector = InRotator.Euler();
	return AnimationCore::QuatFromEuler(Vector, RotationOrder, true);
}


////////////////////////////////////////////////////////////////////////////////
// FRigElementHandle
////////////////////////////////////////////////////////////////////////////////

FRigElementHandle::FRigElementHandle(URigHierarchy* InHierarchy, const FRigElementKey& InKey)
: Hierarchy(InHierarchy)
, Key(InKey)
{
}

FRigElementHandle::FRigElementHandle(URigHierarchy* InHierarchy, const FRigBaseElement* InElement)
: Hierarchy(InHierarchy)
, Key(InElement->GetKey())
{
}

const FRigBaseElement* FRigElementHandle::Get() const
{
	if(Hierarchy.IsValid())
	{
		return Hierarchy->Find(Key);
	}
	return nullptr;
}

FRigBaseElement* FRigElementHandle::Get()
{
	if(Hierarchy.IsValid())
	{
		return Hierarchy->Find(Key);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
// FRigComponentHandle
////////////////////////////////////////////////////////////////////////////////

FRigComponentHandle::FRigComponentHandle(URigHierarchy* InHierarchy, const FRigComponentKey& InKey)
: Hierarchy(InHierarchy)
, Key(InKey)
{
}

FRigComponentHandle::FRigComponentHandle(URigHierarchy* InHierarchy, const FRigBaseComponent* InComponent)
: Hierarchy(InHierarchy)
, Key(InComponent->GetKey())
{
}

const FRigBaseComponent* FRigComponentHandle::Get() const
{
	if(const URigHierarchy* CurrentHierarchy = Hierarchy.Get())
	{
		return CurrentHierarchy->FindComponent(Key);
	}
	return nullptr;
}

FRigBaseComponent* FRigComponentHandle::Get()
{
	if(URigHierarchy* CurrentHierarchy = Hierarchy.Get())
	{
		return CurrentHierarchy->FindComponent(Key);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
// FRigTransformElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigTransformElement::ElementTypeIndex = TransformElement;

void FRigTransformElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		GetTransform().Save(Ar, InSettings, GetDirtyState());
	}
}

void FRigTransformElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		GetTransform().Load(Ar, InSettings, GetDirtyState());
	}
}

void FRigTransformElement::CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights)
{
	Super::CopyPose(InOther, bCurrent, bInitial, bWeights);

	if(FRigTransformElement* Other = Cast<FRigTransformElement>(InOther))
	{
		if(bCurrent)
		{
			GetTransform().Current = Other->GetTransform().Current;
			GetDirtyState().Current = Other->GetDirtyState().Current; 
		}
		if(bInitial)
		{
			GetTransform().Initial = Other->GetTransform().Initial;
			GetDirtyState().Initial = Other->GetDirtyState().Initial; 
		}
	}
}

const FRigCurrentAndInitialTransform& FRigTransformElement::GetTransform() const
{
	return PoseStorage;
}

FRigCurrentAndInitialTransform& FRigTransformElement::GetTransform()
{
	return PoseStorage;
}

const FRigCurrentAndInitialDirtyState& FRigTransformElement::GetDirtyState() const
{
	return PoseDirtyState;
}

FRigCurrentAndInitialDirtyState& FRigTransformElement::GetDirtyState()
{
	return PoseDirtyState;
}

bool FRigTransformElement::IsElementDirty(ERigTransformType::Type InTransformType) const
{
	return PoseDirtyState.IsDirty(InTransformType);
}

void FRigTransformElement::MarkElementDirty(ERigTransformType::Type InTransformType)
{
	PoseDirtyState.MarkDirty(InTransformType);
}

void FRigTransformElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	const FRigTransformElement* SourceTransform = CastChecked<FRigTransformElement>(InOther);
	GetTransform() = SourceTransform->GetTransform();
	GetDirtyState() = SourceTransform->GetDirtyState();

	ElementsToDirty.Reset();
	ElementsToDirty.Reserve(SourceTransform->ElementsToDirty.Num());
	
	for(int32 ElementToDirtyIndex = 0; ElementToDirtyIndex < SourceTransform->ElementsToDirty.Num(); ElementToDirtyIndex++)
	{
		const FElementToDirty& Source = SourceTransform->ElementsToDirty[ElementToDirtyIndex];
		FRigTransformElement* TargetTransform = CastChecked<FRigTransformElement>(Owner->Get(Source.Element->Index));
		const FElementToDirty Target(TargetTransform, Source.HierarchyDistance);
		ElementsToDirty.Add(Target);
		check(ElementsToDirty[ElementToDirtyIndex].Element->GetKey() == Source.Element->GetKey());
	}
}

void FRigTransformElement::LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates,
	const TArrayView<float>& InCurves)
{
	FRigBaseElement::LinkStorage(InTransforms, InDirtyStates, InCurves);
	PoseStorage.LinkStorage(InTransforms);
	PoseDirtyState.LinkStorage(InDirtyStates);
}

void FRigTransformElement::UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms,
	FRigReusableElementStorage<bool>& InDirtyStates, FRigReusableElementStorage<float>& InCurves)
{
	FRigBaseElement::UnlinkStorage(InTransforms, InDirtyStates, InCurves);
	PoseStorage.UnlinkStorage(InTransforms);
	PoseDirtyState.UnlinkStorage(InDirtyStates);
}

////////////////////////////////////////////////////////////////////////////////
// FRigSingleParentElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigSingleParentElement::ElementTypeIndex = SingleParentElement;

void FRigSingleParentElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::InterElementData)
	{
		FRigElementKey ParentKey;
		if(ParentElement)
		{
			ParentKey = ParentElement->GetKey();
		}
		Ar << ParentKey;
	}
}

void FRigSingleParentElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::InterElementData)
	{
		FRigElementKey ParentKey;
		Ar << ParentKey;

		if(ParentKey.IsValid())
		{
			ParentElement = Owner->FindChecked<FRigTransformElement>(ParentKey);
		}
	}
}

void FRigSingleParentElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);

	const FRigSingleParentElement* Source = CastChecked<FRigSingleParentElement>(InOther); 
	if(Source->ParentElement)
	{
		ParentElement = CastChecked<FRigTransformElement>(Owner->Get(Source->ParentElement->GetIndex()));
		check(ParentElement->GetKey() == Source->ParentElement->GetKey());
	}
	else
	{
		ParentElement = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigMultiParentElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigMultiParentElement::ElementTypeIndex = MultiParentElement;

void FRigMultiParentElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		int32 NumParents = ParentConstraints.Num();
		Ar << NumParents;
	}
	else if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::InterElementData)
	{
		for(int32 ParentIndex = 0; ParentIndex < ParentConstraints.Num(); ParentIndex++)
		{
			FRigElementKey ParentKey;
			if(ParentConstraints[ParentIndex].ParentElement)
			{
				ParentKey = ParentConstraints[ParentIndex].ParentElement->GetKey();
			}

			Ar << ParentKey;
			Ar << ParentConstraints[ParentIndex].InitialWeight;
			Ar << ParentConstraints[ParentIndex].Weight;
			Ar << ParentConstraints[ParentIndex].DisplayLabel;
		}
	}
}

void FRigMultiParentElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RemovedMultiParentParentCache)
		{
			FRigCurrentAndInitialTransform Parent;
			FRigCurrentAndInitialDirtyState DirtyState;
			Parent.Load(Ar, InSettings, DirtyState);
		}

		int32 NumParents = 0;
		Ar << NumParents;

		ParentConstraints.SetNum(NumParents);
	}
	else if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::InterElementData)
	{
		for(int32 ParentIndex = 0; ParentIndex < ParentConstraints.Num(); ParentIndex++)
		{
			FRigElementKey ParentKey;
			Ar << ParentKey;
			ensure(ParentKey.IsValid());

			ParentConstraints[ParentIndex].ParentElement = Owner->FindChecked<FRigTransformElement>(ParentKey);
			ParentConstraints[ParentIndex].bCacheIsDirty = true;

			if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyMultiParentConstraints)
			{
				Ar << ParentConstraints[ParentIndex].InitialWeight;
				Ar << ParentConstraints[ParentIndex].Weight;
			}
			else
			{
				float InitialWeight = 0.f;
				Ar << InitialWeight;
				ParentConstraints[ParentIndex].InitialWeight = FRigElementWeight(InitialWeight);

				float Weight = 0.f;
				Ar << Weight;
				ParentConstraints[ParentIndex].Weight = FRigElementWeight(Weight);
			}
			
			if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RigHierarchyParentContraintWithLabel)
			{
				ParentConstraints[ParentIndex].DisplayLabel = NAME_None;
			}
			else
			{
				Ar << ParentConstraints[ParentIndex].DisplayLabel;
			}

			IndexLookup.Add(ParentKey, ParentIndex);
		}
	}
}

void FRigMultiParentElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	const FRigMultiParentElement* Source = CastChecked<FRigMultiParentElement>(InOther);
	ParentConstraints.Reset();
	ParentConstraints.Reserve(Source->ParentConstraints.Num());
	IndexLookup.Reset();
	IndexLookup.Reserve(Source->IndexLookup.Num());

	for(int32 ParentIndex = 0; ParentIndex < Source->ParentConstraints.Num(); ParentIndex++)
	{
		FRigElementParentConstraint ParentConstraint = Source->ParentConstraints[ParentIndex];
		const FRigTransformElement* SourceParentElement = ParentConstraint.ParentElement;
		ParentConstraint.ParentElement = CastChecked<FRigTransformElement>(Owner->Get(SourceParentElement->GetIndex()));
		ParentConstraints.Add(ParentConstraint);
		check(ParentConstraints[ParentIndex].ParentElement->GetKey() == SourceParentElement->GetKey());
		IndexLookup.Add(ParentConstraint.ParentElement->GetKey(), ParentIndex);
	}
}

void FRigMultiParentElement::CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights)
{
	Super::CopyPose(InOther, bCurrent, bInitial, bWeights);

	if(bWeights)
	{
		FRigMultiParentElement* Source = Cast<FRigMultiParentElement>(InOther);
		if(ensure(Source))
		{
			// Find the map between constraint indices
			TMap<int32, int32> ConstraintIndexToSourceConstraintIndex;
			for(int32 ConstraintIndex = 0; ConstraintIndex < ParentConstraints.Num(); ConstraintIndex++)
			{
				const FRigElementParentConstraint& ParentConstraint = ParentConstraints[ConstraintIndex];
				int32 SourceConstraintIndex = Source->ParentConstraints.IndexOfByPredicate([ParentConstraint](const FRigElementParentConstraint& Constraint)
				{
					return Constraint.ParentElement->GetKey() == ParentConstraint.ParentElement->GetKey();
				});
				if (SourceConstraintIndex != INDEX_NONE)
				{
					ConstraintIndexToSourceConstraintIndex.Add(ConstraintIndex, SourceConstraintIndex);
				}
			}
			
			for(int32 ParentIndex = 0; ParentIndex < ParentConstraints.Num(); ParentIndex++)
			{
				if (int32* SourceConstraintIndex = ConstraintIndexToSourceConstraintIndex.Find(ParentIndex))
				{
					ParentConstraints[ParentIndex].CopyPose(Source->ParentConstraints[*SourceConstraintIndex], bCurrent, bInitial);
				}
				else
				{
					// Otherwise, reset the weights to 0
					if (bCurrent)
					{
						ParentConstraints[ParentIndex].Weight = 0.f;
					}
					if (bInitial)
					{
						ParentConstraints[ParentIndex].InitialWeight = 0.f;
					}
				}
			}
		}
	}
}

bool FRigMultiParentElement::IsElementDirty(ERigTransformType::Type InTransformType) const
{
	return PoseDirtyState.IsDirty(ERigTransformType::MakeGlobal(InTransformType));
}

void FRigMultiParentElement::MarkElementDirty(ERigTransformType::Type InTransformType)
{
	PoseDirtyState.MarkDirty(ERigTransformType::MakeGlobal(InTransformType));
}


////////////////////////////////////////////////////////////////////////////////
// FRigBoneElement
////////////////////////////////////////////////////////////////////////////////
#if !UE_DETECT_DELEGATES_RACE_CONDITIONS // race detector increases mem footprint but is compiled out from test/shipping builds
static_assert(sizeof(FRigBoneElement) <= 736, "FRigBoneElement was optimized to fit into 736 bytes bin of MallocBinned3");
#endif

const FRigBaseElement::EElementIndex FRigBoneElement::ElementTypeIndex = BoneElement;

void FRigBoneElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		static const UEnum* BoneTypeEnum = StaticEnum<ERigBoneType>();
		FName TypeName = BoneTypeEnum->GetNameByValue((int64)BoneType);
		Ar << TypeName;
	}
}

void FRigBoneElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		static const UEnum* BoneTypeEnum = StaticEnum<ERigBoneType>();
		FName TypeName;
		Ar << TypeName;
		BoneType = (ERigBoneType)BoneTypeEnum->GetValueByName(TypeName);
	}
}

void FRigBoneElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	const FRigBoneElement* Source = CastChecked<FRigBoneElement>(InOther);
	BoneType = Source->BoneType;
}

////////////////////////////////////////////////////////////////////////////////
// FRigNullElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigNullElement::ElementTypeIndex = NullElement;

////////////////////////////////////////////////////////////////////////////////
// FRigControlSettings
////////////////////////////////////////////////////////////////////////////////

FRigControlSettings::FRigControlSettings()
: AnimationType(ERigControlAnimationType::AnimationControl)
, ControlType(ERigControlType::EulerTransform)
, DisplayName(NAME_None)
, PrimaryAxis(ERigControlAxis::X)
, bIsCurve(false)
, LimitEnabled()
, bDrawLimits(true)
, MinimumValue()
, MaximumValue()
, bShapeVisible(true)
, ShapeVisibility(ERigControlVisibility::UserDefined)
, ShapeName(NAME_None)
, ShapeColor(FLinearColor::Red)
, bIsTransientControl(false)
, ControlEnum(nullptr)
, Customization()
, bGroupWithParentControl(false)
, bRestrictSpaceSwitching(false) 
, PreferredRotationOrder(FRigPreferredEulerAngles::DefaultRotationOrder)
, bUsePreferredRotationOrder(false)
, ShapeTransform(FTransform::Identity)
{
	// rely on the default provided by the shape definition
	ShapeName = FControlRigShapeDefinition().ShapeName; 
}

void FRigControlSettings::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	static const UEnum* AnimationTypeEnum = StaticEnum<ERigControlAnimationType>();
	static const UEnum* ControlTypeEnum = StaticEnum<ERigControlType>();
	static const UEnum* ShapeVisibilityEnum = StaticEnum<ERigControlVisibility>();
	static const UEnum* ControlAxisEnum = StaticEnum<ERigControlAxis>();

	FName AnimationTypeName = AnimationTypeEnum->GetNameByValue((int64)AnimationType);
	FName ControlTypeName = ControlTypeEnum->GetNameByValue((int64)ControlType);
	FName ShapeVisibilityName = ShapeVisibilityEnum->GetNameByValue((int64)ShapeVisibility);
	FName PrimaryAxisName = ControlAxisEnum->GetNameByValue((int64)PrimaryAxis);

	FString ControlEnumPathName;
	if(ControlEnum)
	{
		ControlEnumPathName = ControlEnum->GetPathName();
		if (Ar.IsObjectReferenceCollector())
		{
			FSoftObjectPath DeclareControlEnumToCooker(ControlEnumPathName);
			Ar << DeclareControlEnumToCooker;
		}
	}

	Ar << AnimationTypeName;
	Ar << ControlTypeName;
	Ar << DisplayName;
	Ar << PrimaryAxisName;
	Ar << bIsCurve;
	Ar << LimitEnabled;
	Ar << bDrawLimits;
	Ar << MinimumValue;
	Ar << MaximumValue;
	Ar << bShapeVisible;
	Ar << ShapeVisibilityName;
	Ar << ShapeName;
	Ar << ShapeColor;
	Ar << bIsTransientControl;
	Ar << ControlEnumPathName;
	Ar << Customization.AvailableSpaces;
	Ar << DrivenControls;
	Ar << bGroupWithParentControl;
	Ar << bRestrictSpaceSwitching;
	Ar << FilteredChannels;
	Ar << PreferredRotationOrder;
	Ar << bUsePreferredRotationOrder;

}

void FRigControlSettings::Load(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	static const UEnum* AnimationTypeEnum = StaticEnum<ERigControlAnimationType>();
	static const UEnum* ControlTypeEnum = StaticEnum<ERigControlType>();
	static const UEnum* ShapeVisibilityEnum = StaticEnum<ERigControlVisibility>();
	static const UEnum* ControlAxisEnum = StaticEnum<ERigControlAxis>();

	FName AnimationTypeName, ControlTypeName, ShapeVisibilityName, PrimaryAxisName;
	FString ControlEnumPathName;

	bool bLimitTranslation_DEPRECATED = false;
	bool bLimitRotation_DEPRECATED = false;
	bool bLimitScale_DEPRECATED = false;
	bool bAnimatableDeprecated = false;
	bool bShapeEnabledDeprecated = false;

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::ControlAnimationType)
	{
		Ar << AnimationTypeName;
	}
	Ar << ControlTypeName;
	Ar << DisplayName;
	Ar << PrimaryAxisName;
	Ar << bIsCurve;
	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::ControlAnimationType)
	{
		Ar << bAnimatableDeprecated;
	}
	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::PerChannelLimits)
	{
		Ar << bLimitTranslation_DEPRECATED;
		Ar << bLimitRotation_DEPRECATED;
		Ar << bLimitScale_DEPRECATED;
	}
	else
	{
		Ar << LimitEnabled;
	}
	Ar << bDrawLimits;

	FTransform MinimumTransform, MaximumTransform;
	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::StorageMinMaxValuesAsFloatStorage)
	{
		Ar << MinimumValue;
		Ar << MaximumValue;
	}
	else
	{
		Ar << MinimumTransform;
		Ar << MaximumTransform;
	}

	ControlType = (ERigControlType)ControlTypeEnum->GetValueByName(ControlTypeName);
	
	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::ControlAnimationType)
	{
		Ar << bShapeEnabledDeprecated;
		SetAnimationTypeFromDeprecatedData(bAnimatableDeprecated, bShapeEnabledDeprecated);
		AnimationTypeName = AnimationTypeEnum->GetNameByValue((int64)AnimationType);
	}
	
	Ar << bShapeVisible;
	
	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::ControlAnimationType)
	{
		ShapeVisibilityName = ShapeVisibilityEnum->GetNameByValue((int64)ERigControlVisibility::UserDefined);
	}
	else
	{
		Ar << ShapeVisibilityName;
	}
	Ar << ShapeName;

	if(Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RenameGizmoToShape)
	{
		if(ShapeName == FRigControl().GizmoName)
		{
			ShapeName = FControlRigShapeDefinition().ShapeName; 
		}
	}
	
	Ar << ShapeColor;
	Ar << bIsTransientControl;
	Ar << ControlEnumPathName;

	AnimationType = (ERigControlAnimationType)AnimationTypeEnum->GetValueByName(AnimationTypeName);
	PrimaryAxis = (ERigControlAxis)ControlAxisEnum->GetValueByName(PrimaryAxisName);
	ShapeVisibility = (ERigControlVisibility)ShapeVisibilityEnum->GetValueByName(ShapeVisibilityName);

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::StorageMinMaxValuesAsFloatStorage)
	{
		MinimumValue.SetFromTransform(MinimumTransform, ControlType, PrimaryAxis);
		MaximumValue.SetFromTransform(MaximumTransform, ControlType, PrimaryAxis);
	}

	ControlEnum = nullptr;
	if(!ControlEnumPathName.IsEmpty())
	{
		if (IsInGameThread() && !IsAsyncLoading())
		{
			ControlEnum = LoadObject<UEnum>(nullptr, *ControlEnumPathName);
		}
		else
		{			
			ControlEnum = FindObject<UEnum>(nullptr, *ControlEnumPathName);
		}
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyParentContraintWithLabel)
	{
		Ar << Customization.AvailableSpaces;
	}
	else if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyControlSpaceFavorites)
	{
		TArray<FRigElementKey> AvailableSpaceKeys;
		Ar << AvailableSpaceKeys;
		Customization.AvailableSpaces.Reset();
		Customization.AvailableSpaces.Reserve(AvailableSpaceKeys.Num());
		for(const FRigElementKey& AvailableSpaceKey : AvailableSpaceKeys)
		{
			Customization.AvailableSpaces.Emplace(AvailableSpaceKey, NAME_None);
		}
	}
	else
	{
		Customization.AvailableSpaces.Reset();
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::ControlAnimationType)
	{
		Ar << DrivenControls;
	}
	else
	{
		DrivenControls.Reset();
	}

	PreviouslyDrivenControls.Reset();

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::PerChannelLimits)
	{
		SetupLimitArrayForType(bLimitTranslation_DEPRECATED, bLimitRotation_DEPRECATED, bLimitScale_DEPRECATED);
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::ControlAnimationType)
	{
		Ar << bGroupWithParentControl;
	}
	else
	{
		bGroupWithParentControl = IsAnimatable() && (
			ControlType == ERigControlType::Bool ||
			ControlType == ERigControlType::Float ||
			ControlType == ERigControlType::ScaleFloat ||
			ControlType == ERigControlType::Integer ||
			ControlType == ERigControlType::Vector2D
		);
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RestrictSpaceSwitchingForControls)
	{
		Ar << bRestrictSpaceSwitching;
	}
	else
	{
		bRestrictSpaceSwitching = false;
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::ControlTransformChannelFiltering)
	{
		Ar << FilteredChannels;
	}
	else
	{
		FilteredChannels.Reset();
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyControlPreferredRotationOrder)
	{
		Ar << PreferredRotationOrder;
	}
	else
	{
		PreferredRotationOrder = FRigPreferredEulerAngles::DefaultRotationOrder;
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyControlPreferredRotationOrderFlag)
	{
		Ar << bUsePreferredRotationOrder;
	}
	else
	{
		bUsePreferredRotationOrder = false;
	}

	ShapeTransform = FTransform::Identity;
}

uint32 GetTypeHash(const FRigControlSettings& Settings)
{
	uint32 Hash = GetTypeHash(Settings.ControlType);
	Hash = HashCombine(Hash, GetTypeHash(Settings.AnimationType));
	Hash = HashCombine(Hash, GetTypeHash(Settings.DisplayName));
	Hash = HashCombine(Hash, GetTypeHash(Settings.PrimaryAxis));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bIsCurve));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bDrawLimits));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bShapeVisible));
	Hash = HashCombine(Hash, GetTypeHash(Settings.ShapeVisibility));
	Hash = HashCombine(Hash, GetTypeHash(Settings.ShapeName));
	Hash = HashCombine(Hash, GetTypeHash(Settings.ShapeColor));
	Hash = HashCombine(Hash, GetTypeHash(Settings.ControlEnum));
	Hash = HashCombine(Hash, GetTypeHash(Settings.DrivenControls));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bGroupWithParentControl));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bRestrictSpaceSwitching));
	Hash = HashCombine(Hash, GetTypeHash(Settings.FilteredChannels.Num()));
	for(const ERigControlTransformChannel& Channel : Settings.FilteredChannels)
	{
		Hash = HashCombine(Hash, GetTypeHash(Channel));
	}
	Hash = HashCombine(Hash, GetTypeHash(Settings.PreferredRotationOrder));
	return Hash;
}

bool FRigControlSettings::operator==(const FRigControlSettings& InOther) const
{
	if(AnimationType != InOther.AnimationType)
	{
		return false;
	}
	if(ControlType != InOther.ControlType)
	{
		return false;
	}
	if(DisplayName != InOther.DisplayName)
	{
		return false;
	}
	if(PrimaryAxis != InOther.PrimaryAxis)
	{
		return false;
	}
	if(bIsCurve != InOther.bIsCurve)
	{
		return false;
	}
	if(LimitEnabled != InOther.LimitEnabled)
	{
		return false;
	}
	if(bDrawLimits != InOther.bDrawLimits)
	{
		return false;
	}
	if(bShapeVisible != InOther.bShapeVisible)
	{
		return false;
	}
	if(ShapeVisibility != InOther.ShapeVisibility)
	{
		return false;
	}
	if(ShapeName != InOther.ShapeName)
	{
		return false;
	}
	if(bIsTransientControl != InOther.bIsTransientControl)
	{
		return false;
	}
	if(ControlEnum != InOther.ControlEnum)
	{
		return false;
	}
	if(!ShapeColor.Equals(InOther.ShapeColor, 0.001f))
	{
		return false;
	}
	if(Customization.AvailableSpaces != InOther.Customization.AvailableSpaces)
	{
		return false;
	}
	if(DrivenControls != InOther.DrivenControls)
	{
		return false;
	}
	if(bGroupWithParentControl != InOther.bGroupWithParentControl)
	{
		return false;
	}
	if(bRestrictSpaceSwitching != InOther.bRestrictSpaceSwitching)
	{
		return false;
	}
	if(FilteredChannels != InOther.FilteredChannels)
	{
		return false;
	}
	if(PreferredRotationOrder != InOther.PreferredRotationOrder)
	{
		return false;
	}
	if (bUsePreferredRotationOrder != InOther.bUsePreferredRotationOrder)
	{
		return false;
	}


	const FTransform MinimumTransform = MinimumValue.GetAsTransform(ControlType, PrimaryAxis);
	const FTransform OtherMinimumTransform = InOther.MinimumValue.GetAsTransform(ControlType, PrimaryAxis);
	if(!MinimumTransform.Equals(OtherMinimumTransform, 0.001))
	{
		return false;
	}

	const FTransform MaximumTransform = MaximumValue.GetAsTransform(ControlType, PrimaryAxis);
	const FTransform OtherMaximumTransform = InOther.MaximumValue.GetAsTransform(ControlType, PrimaryAxis);
	if(!MaximumTransform.Equals(OtherMaximumTransform, 0.001))
	{
		return false;
	}

	return true;
}

void FRigControlSettings::SetupLimitArrayForType(bool bLimitTranslation, bool bLimitRotation, bool bLimitScale)
{
	switch(ControlType)
	{
		case ERigControlType::Integer:
		case ERigControlType::Float:
		{
			LimitEnabled.SetNum(1);
			LimitEnabled[0].Set(bLimitTranslation);
			break;
		}
		case ERigControlType::ScaleFloat:
		{
			LimitEnabled.SetNum(1);
			LimitEnabled[0].Set(bLimitScale);
			break;
		}
		case ERigControlType::Vector2D:
		{
			LimitEnabled.SetNum(2);
			LimitEnabled[0] = LimitEnabled[1].Set(bLimitTranslation);
			break;
		}
		case ERigControlType::Position:
		{
			LimitEnabled.SetNum(3);
			LimitEnabled[0] = LimitEnabled[1] = LimitEnabled[2].Set(bLimitTranslation);
			break;
		}
		case ERigControlType::Scale:
		{
			LimitEnabled.SetNum(3);
			LimitEnabled[0] = LimitEnabled[1] = LimitEnabled[2].Set(bLimitScale);
			break;
		}
		case ERigControlType::Rotator:
		{
			LimitEnabled.SetNum(3);
			LimitEnabled[0] = LimitEnabled[1] = LimitEnabled[2].Set(bLimitRotation);
			break;
		}
		case ERigControlType::TransformNoScale:
		{
			LimitEnabled.SetNum(6);
			LimitEnabled[0] = LimitEnabled[1] = LimitEnabled[2].Set(bLimitTranslation);
			LimitEnabled[3] = LimitEnabled[4] = LimitEnabled[5].Set(bLimitRotation);
			break;
		}
		case ERigControlType::EulerTransform:
		case ERigControlType::Transform:
		{
			LimitEnabled.SetNum(9);
			LimitEnabled[0] = LimitEnabled[1] = LimitEnabled[2].Set(bLimitTranslation);
			LimitEnabled[3] = LimitEnabled[4] = LimitEnabled[5].Set(bLimitRotation);
			LimitEnabled[6] = LimitEnabled[7] = LimitEnabled[8].Set(bLimitScale);
			break;
		}
		case ERigControlType::Bool:
		default:
		{
			LimitEnabled.Reset();
			break;
		}
	}
}

const FRigCurrentAndInitialTransform& FRigControlElement::GetOffsetTransform() const
{
	return OffsetStorage;
}

FRigCurrentAndInitialTransform& FRigControlElement::GetOffsetTransform()
{
	return OffsetStorage;
}

const FRigCurrentAndInitialDirtyState& FRigControlElement::GetOffsetDirtyState() const
{
	return OffsetDirtyState;
}

FRigCurrentAndInitialDirtyState& FRigControlElement::GetOffsetDirtyState()
{
	return OffsetDirtyState;
}

const FRigCurrentAndInitialTransform& FRigControlElement::GetShapeTransform() const
{
	return ShapeStorage;
}

FRigCurrentAndInitialTransform& FRigControlElement::GetShapeTransform()
{
	return ShapeStorage;
}

const FRigCurrentAndInitialDirtyState& FRigControlElement::GetShapeDirtyState() const
{
	return ShapeDirtyState;
}

FRigCurrentAndInitialDirtyState& FRigControlElement::GetShapeDirtyState()
{
	return ShapeDirtyState;
}

////////////////////////////////////////////////////////////////////////////////
// FRigControlElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigControlElement::ElementTypeIndex = ControlElement;

void FRigControlElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		Settings.Save(Ar);
		GetOffsetTransform().Save(Ar, InSettings, GetOffsetDirtyState());
		GetShapeTransform().Save(Ar, InSettings, GetShapeDirtyState());
		PreferredEulerAngles.Save(Ar);
	}
}

void FRigControlElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		Settings.Load(Ar);
		GetOffsetTransform().Load(Ar, InSettings, GetOffsetDirtyState());
		GetShapeTransform().Load(Ar, InSettings, GetShapeDirtyState());

		if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::PreferredEulerAnglesForControls)
		{
			PreferredEulerAngles.Load(Ar);
		}
		else
		{
			PreferredEulerAngles.Reset();
		}
		PreferredEulerAngles.SetRotationOrder(Settings.PreferredRotationOrder);
	}
}

void FRigControlElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	const FRigControlElement* Source = CastChecked<FRigControlElement>(InOther);
	Settings = Source->Settings;
	GetOffsetTransform() = Source->GetOffsetTransform();
	GetOffsetDirtyState() = Source->GetOffsetDirtyState();
	GetShapeTransform() = Source->GetShapeTransform();
	GetShapeDirtyState() = Source->GetShapeDirtyState();
	PreferredEulerAngles = Source->PreferredEulerAngles;
}

void FRigControlElement::CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights)
{
	Super::CopyPose(InOther, bCurrent, bInitial, bWeights);
	
	if(FRigControlElement* Other = Cast<FRigControlElement>(InOther))
	{
		if(bCurrent)
		{
			GetOffsetTransform().Current = Other->GetOffsetTransform().Current;
			GetOffsetDirtyState().Current = Other->GetOffsetDirtyState().Current;
			GetShapeTransform().Current = Other->GetShapeTransform().Current;
			GetShapeDirtyState().Current = Other->GetShapeDirtyState().Current;
			PreferredEulerAngles.SetAngles(Other->PreferredEulerAngles.GetAngles(false), false);
		}
		if(bInitial)
		{
			GetOffsetTransform().Initial = Other->GetOffsetTransform().Initial;
			GetOffsetDirtyState().Initial = Other->GetOffsetDirtyState().Initial;
			GetShapeTransform().Initial = Other->GetShapeTransform().Initial;
			GetShapeDirtyState().Initial = Other->GetShapeDirtyState().Initial;
			PreferredEulerAngles.SetAngles(Other->PreferredEulerAngles.GetAngles(true), true);
		}
	}
}

bool FRigControlElement::IsElementDirty(ERigTransformType::Type InTransformType) const
{
	if (IsAnimationChannel())
	{
		// animation channels never dirty their local value
		InTransformType = ERigTransformType::MakeGlobal(InTransformType);
	}

	if (ERigTransformType::IsGlobal(InTransformType))
	{
		return (GetOffsetDirtyState().IsDirty(InTransformType)
			&& GetDirtyState().IsDirty(InTransformType)
			&& GetShapeDirtyState().IsDirty(InTransformType));
	}

	return PoseDirtyState.IsDirty(InTransformType);
}

void FRigControlElement::MarkElementDirty(ERigTransformType::Type InTransformType)
{
	const ERigTransformType::Type GlobalType = ERigTransformType::MakeGlobal(InTransformType);

	if (IsAnimationChannel())
	{
		// animation channels never dirty their local value
		InTransformType = GlobalType;
	}

	PoseDirtyState.MarkDirty(InTransformType);
	GetOffsetDirtyState().MarkDirty(GlobalType);
	GetShapeDirtyState().MarkDirty(GlobalType);
}

void FRigControlElement::LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates,
	const TArrayView<float>& InCurves)
{
	FRigMultiParentElement::LinkStorage(InTransforms, InDirtyStates, InCurves);
	OffsetStorage.LinkStorage(InTransforms);
	ShapeStorage.LinkStorage(InTransforms);
	OffsetDirtyState.LinkStorage(InDirtyStates);
	ShapeDirtyState.LinkStorage(InDirtyStates);
}

void FRigControlElement::UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms, FRigReusableElementStorage<bool>& InDirtyStates,
	FRigReusableElementStorage<float>& InCurves)
{
	FRigMultiParentElement::UnlinkStorage(InTransforms, InDirtyStates, InCurves);
	OffsetStorage.UnlinkStorage(InTransforms);
	ShapeStorage.UnlinkStorage(InTransforms);
	OffsetDirtyState.UnlinkStorage(InDirtyStates);
	ShapeDirtyState.UnlinkStorage(InDirtyStates);
}

////////////////////////////////////////////////////////////////////////////////
// FRigCurveElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigCurveElement::ElementTypeIndex = CurveElement;

void FRigCurveElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		float Value = Get();
		Ar << bIsValueSet;
		Ar << Value;
	}
}

void FRigCurveElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::CurveElementValueStateFlag)
		{
			Ar << bIsValueSet;
		}
		else
		{
			bIsValueSet = true;
		}

		float Value = 0.f;
		Ar << Value;

		Set(Value, bIsValueSet);
	}
}

void FRigCurveElement::CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights)
{
	Super::CopyPose(InOther, bCurrent, bInitial, bWeights);
	
	if(const FRigCurveElement* Other = Cast<FRigCurveElement>(InOther))
	{
		Set(Other->Get());
		bIsValueSet = Other->bIsValueSet;
	}
}

const float& FRigCurveElement::Get() const
{
	if(Storage)
	{
		return *Storage;
	}
	ensure(false);
	static constexpr float DefaultCurve = 0.f;
	return DefaultCurve;
}

void FRigCurveElement::Set(const float& InValue, bool InValueIsSet)
{
	if(Storage)
	{
		*Storage = InValue;
		bIsValueSet = InValueIsSet;
	}
}

void FRigCurveElement::LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates,
	const TArrayView<float>& InCurves)
{
	FRigBaseElement::LinkStorage(InTransforms, InDirtyStates, InCurves);
	if(InCurves.IsValidIndex(StorageIndex))
	{
		Storage = InCurves.GetData() + StorageIndex;
	}
}

void FRigCurveElement::UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms, FRigReusableElementStorage<bool>& InDirtyStates,
	FRigReusableElementStorage<float>& InCurves)
{
	FRigBaseElement::UnlinkStorage(InTransforms, InDirtyStates, InCurves);
	InCurves.Deallocate(StorageIndex, &Storage);
}

void FRigCurveElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	if(const FRigCurveElement* Other = CastChecked<FRigCurveElement>(InOther))
	{
		Set(Other->Get());
		bIsValueSet = Other->bIsValueSet;
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigReferenceElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigReferenceElement::ElementTypeIndex = ReferenceElement;

void FRigReferenceElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);
}

void FRigReferenceElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);
}

void FRigReferenceElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	const FRigReferenceElement* Source = CastChecked<FRigReferenceElement>(InOther);
	GetWorldTransformDelegate = Source->GetWorldTransformDelegate;
}

FTransform FRigReferenceElement::GetReferenceWorldTransform(const FRigVMExecuteContext* InContext, bool bInitial) const
{
	if(GetWorldTransformDelegate.IsBound())
	{
		return GetWorldTransformDelegate.Execute(InContext, GetKey(), bInitial);
	}
	return FTransform::Identity;
}

void FRigReferenceElement::CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights)
{
	Super::CopyPose(InOther, bCurrent, bInitial, bWeights);
	
	if(FRigReferenceElement* Other = Cast<FRigReferenceElement>(InOther))
	{
		if(Other->GetWorldTransformDelegate.IsBound())
		{
			GetWorldTransformDelegate = Other->GetWorldTransformDelegate;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
// FRigConnectorSettings
////////////////////////////////////////////////////////////////////////////////

FRigConnectorSettings::FRigConnectorSettings()
	: Type(EConnectorType::Primary)
	, bOptional(false)
	, bIsArray(false)
	, bPostConstruction(false)
{
}

FRigConnectorSettings FRigConnectorSettings::DefaultSettings()
{
	FRigConnectorSettings Settings;
	Settings.AddRule(FRigTypeConnectionRule(ERigElementType::Socket));
	return Settings;
}

void FRigConnectorSettings::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	Ar << Description;
	Ar << Type;
	Ar << bOptional;
	Ar << bIsArray;
	Ar << bPostConstruction;

	int32 NumRules = Rules.Num();
	Ar << NumRules;
	for(int32 Index = 0; Index < NumRules; Index++)
	{

		Rules[Index].Save(Ar);
	}
}

void FRigConnectorSettings::Load(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);

	Ar << Description;

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::ConnectorsWithType)
	{
		Ar << Type;
		Ar << bOptional;
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyArrayConnectors)
	{
		Ar << bIsArray;
	}
	else
	{
		bIsArray = false;
	}

	if (Ar.CustomVer(FControlRigObjectVersion::GUID) >= FControlRigObjectVersion::RigHierarchyPostConstructionConnectors)
	{
		Ar << bPostConstruction;
	}
	else
	{
		bPostConstruction = false;
	}

	int32 NumRules = 0;
	Ar << NumRules;
	Rules.SetNumZeroed(NumRules);
	for(int32 Index = 0; Index < NumRules; Index++)
	{
		Rules[Index].Load(Ar);
	}
}

bool FRigConnectorSettings::operator==(const FRigConnectorSettings& InOther) const
{
	if(!Description.Equals(InOther.Description, ESearchCase::CaseSensitive))
	{
		return false;
	}
	if(Type != InOther.Type)
	{
		return false;
	}
	if(bOptional != InOther.bOptional)
	{
		return false;
	}
	if(bIsArray != InOther.bIsArray)
	{
		return false;
	}
	if(bPostConstruction != InOther.bPostConstruction)
	{
		return false;
	}
	if(Rules.Num() != InOther.Rules.Num())
	{
		return false;
	}
	for(int32 Index = 0; Index < Rules.Num(); Index++)
	{
		if(Rules[Index] != InOther.Rules[Index])
		{
			return false;
		}
	}
	return true;
}

uint32 FRigConnectorSettings::GetRulesHash() const
{
	uint32 Hash = GetTypeHash(Rules.Num());
	for(const FRigConnectionRuleStash& Rule : Rules)
	{
		Hash = HashCombine(Hash, GetTypeHash(Rule));
	}
	return Hash;
}

uint32 GetTypeHash(const FRigConnectorSettings& Settings)
{
	uint32 Hash = HashCombine(GetTypeHash(Settings.Type), Settings.GetRulesHash());
	Hash = HashCombine(Hash, GetTypeHash(Settings.bOptional));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bIsArray));
	Hash = HashCombine(Hash, GetTypeHash(Settings.bPostConstruction));
	return Hash;
}

////////////////////////////////////////////////////////////////////////////////
// FRigConnectorElement
////////////////////////////////////////////////////////////////////////////////

const FRigBaseElement::EElementIndex FRigConnectorElement::ElementTypeIndex = ConnectorElement;

void FRigConnectorElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		Settings.Save(Ar);
	}
}

void FRigConnectorElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);

	if(InSettings.SerializationPhase == FRigHierarchySerializationSettings::StaticData)
	{
		Settings.Load(Ar);
	}
}

FRigConnectorState FRigConnectorElement::GetConnectorState(const URigHierarchy* InHierarchy) const
{
	FRigConnectorState State;
	State.Name = Key.Name;
	State.ResolvedTarget = InHierarchy->GetResolvedTarget(Key);
	State.Settings = Settings;
	return State;
}

void FRigConnectorElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
	
	const FRigConnectorElement* Source = CastChecked<FRigConnectorElement>(InOther);
	Settings = Source->Settings;
}

////////////////////////////////////////////////////////////////////////////////
// FRigSocketElement
////////////////////////////////////////////////////////////////////////////////

FRigSocketState::FRigSocketState()
: Name(NAME_None)
, InitialLocalTransform(FTransform::Identity)
, Color(FRigSocketElement::SocketDefaultColor)
{
}

const FRigBaseElement::EElementIndex FRigSocketElement::ElementTypeIndex = SocketElement;
const FName FRigSocketElement::ColorMetaName = TEXT("SocketColor");
const FName FRigSocketElement::DescriptionMetaName = TEXT("SocketDescription");
const FName FRigSocketElement::DesiredParentMetaName = TEXT("SocketDesiredParent");
const FLinearColor FRigSocketElement::SocketDefaultColor = FLinearColor::White;

void FRigSocketElement::Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Save(Ar, InSettings);
}

void FRigSocketElement::Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings)
{
	Super::Load(Ar, InSettings);
}

FRigSocketState FRigSocketElement::GetSocketState(const URigHierarchy* InHierarchy) const
{
	FRigSocketState State;
	State.Name = GetFName();
	State.Parent = InHierarchy->GetRigElementKeyMetadata(GetKey(), DesiredParentMetaName, FRigElementKey());
	if(!State.Parent.IsValid())
	{
		State.Parent = InHierarchy->GetFirstParent(GetKey());
	}
	State.InitialLocalTransform = InHierarchy->GetInitialLocalTransform(GetIndex());
	State.Color = GetColor(InHierarchy);
	State.Description = GetDescription(InHierarchy);
	return State;
}

FLinearColor FRigSocketElement::GetColor(const URigHierarchy* InHierarchy) const
{
	return InHierarchy->GetLinearColorMetadata(GetKey(), ColorMetaName, SocketDefaultColor);
}

void FRigSocketElement::SetColor(const FLinearColor& InColor, URigHierarchy* InHierarchy, bool bNotify)
{
	if(InHierarchy->GetLinearColorMetadata(GetKey(), ColorMetaName, SocketDefaultColor).Equals(InColor))
	{
		return;
	}
	InHierarchy->SetLinearColorMetadata(GetKey(), ColorMetaName, InColor);
	InHierarchy->PropagateMetadata(GetKey(), ColorMetaName, bNotify);
	if(bNotify)
	{
		InHierarchy->Notify(ERigHierarchyNotification::SocketColorChanged, this);
	}
}

FString FRigSocketElement::GetDescription(const URigHierarchy* InHierarchy) const
{
	const FName Description = InHierarchy->GetNameMetadata(GetKey(), DescriptionMetaName, NAME_None);
	if(Description.IsNone())
	{
		return FString();
	}
	return Description.ToString();
}

void FRigSocketElement::SetDescription(const FString& InDescription, URigHierarchy* InHierarchy, bool bNotify)
{
	const FName Description = InDescription.IsEmpty() ? FName(NAME_None) : *InDescription;
	if(InHierarchy->GetNameMetadata(GetKey(), DescriptionMetaName, NAME_None).IsEqual(Description, ENameCase::CaseSensitive))
	{
		return;
	}
	InHierarchy->SetNameMetadata(GetKey(), DescriptionMetaName, *InDescription);
	InHierarchy->PropagateMetadata(this, DescriptionMetaName, bNotify);
	if(bNotify)
	{
		InHierarchy->Notify(ERigHierarchyNotification::SocketDescriptionChanged, this);
	}
}

void FRigSocketElement::CopyFrom(const FRigBaseElement* InOther)
{
	Super::CopyFrom(InOther);
}
