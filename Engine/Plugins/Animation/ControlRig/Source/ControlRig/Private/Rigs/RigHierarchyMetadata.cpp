// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyMetadata.h"

#include "ControlRigObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyMetadata)

////////////////////////////////////////////////////////////////////////////////
// FRigBaseMetadata
////////////////////////////////////////////////////////////////////////////////

UScriptStruct* FRigBaseMetadata::GetMetadataStruct() const
{
	return GetMetadataStruct(GetType());
}

UScriptStruct* FRigBaseMetadata::GetMetadataStruct(const ERigMetadataType& InType)
{
	switch(InType)
	{
		case ERigMetadataType::Bool:
		{
			return FRigBoolMetadata::StaticStruct();
		}
		case ERigMetadataType::BoolArray:
		{
			return FRigBoolArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Float:
		{
			return FRigFloatMetadata::StaticStruct();
		}
		case ERigMetadataType::FloatArray:
		{
			return FRigFloatArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Int32:
		{
			return FRigInt32Metadata::StaticStruct();
		}
		case ERigMetadataType::Int32Array:
		{
			return FRigInt32ArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Name:
		{
			return FRigNameMetadata::StaticStruct();
		}
		case ERigMetadataType::NameArray:
		{
			return FRigNameArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Vector:
		{
			return FRigVectorMetadata::StaticStruct();
		}
		case ERigMetadataType::VectorArray:
		{
			return FRigVectorArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Rotator:
		{
			return FRigRotatorMetadata::StaticStruct();
		}
		case ERigMetadataType::RotatorArray:
		{
			return FRigRotatorArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Quat:
		{
			return FRigQuatMetadata::StaticStruct();
		}
		case ERigMetadataType::QuatArray:
		{
			return FRigQuatArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::Transform:
		{
			return FRigTransformMetadata::StaticStruct();
		}
		case ERigMetadataType::TransformArray:
		{
			return FRigTransformArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::LinearColor:
		{
			return FRigLinearColorMetadata::StaticStruct();
		}
		case ERigMetadataType::LinearColorArray:
		{
			return FRigLinearColorArrayMetadata::StaticStruct();
		}
		case ERigMetadataType::RigElementKey:
		{
			return FRigElementKeyMetadata::StaticStruct();
		}
		case ERigMetadataType::RigElementKeyArray:
		{
			return FRigElementKeyArrayMetadata::StaticStruct();
		}
		default:
		{
			break;
		}
	}
	return StaticStruct();
}

FRigBaseMetadata* FRigBaseMetadata::MakeMetadata(const FName& InName, ERigMetadataType InType)
{
	check(InType != ERigMetadataType::Invalid);
	
	const UScriptStruct* Struct = GetMetadataStruct(InType);
	check(Struct);
	
	FRigBaseMetadata* Md = static_cast<FRigBaseMetadata*>(FMemory::Malloc(Struct->GetStructureSize()));
	Struct->InitializeStruct(Md, 1);

	Md->Name = InName;
	Md->Type = InType;
	return Md;
}

void FRigBaseMetadata::DestroyMetadata(FRigBaseMetadata** Metadata)
{
	check(Metadata);
	FRigBaseMetadata* Md = *Metadata;
	check(Md);
	if(const UScriptStruct* Struct = Md->GetMetadataStruct())
	{
		Struct->DestroyStruct(Md, 1);
	}
	FMemory::Free(Md);
	Md = nullptr;
}

void FRigBaseMetadata::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FControlRigObjectVersion::GUID);
	Ar << Name;
	Ar << Type;
}
