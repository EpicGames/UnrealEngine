// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyPose.h"
#include "Rigs/RigHierarchy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigHierarchyPose)


FRigPoseElement::FRigPoseElement(): Index()
                                  , GlobalTransform(FTransform::Identity)
                                  , LocalTransform(FTransform::Identity)
                                  , PreferredEulerAngle(FVector::ZeroVector)
                                  , ActiveParent(URigHierarchy::GetDefaultParentKey())
                                  , CurveValue(0.f)
{
}
