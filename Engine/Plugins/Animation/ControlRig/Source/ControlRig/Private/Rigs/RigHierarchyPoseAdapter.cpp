// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigHierarchyPoseAdapter.h"
#include "Rigs/RigHierarchy.h"

URigHierarchy* FRigHierarchyPoseAdapter::GetHierarchy() const
{
	if(WeakHierarchy.IsValid())
	{
		return WeakHierarchy.Get();
	}
	return nullptr;
}

void FRigHierarchyPoseAdapter::PostLinked(URigHierarchy* InHierarchy)
{
	WeakHierarchy = InHierarchy;
	LastTopologyVersion = InHierarchy->GetTopologyVersion();
}

void FRigHierarchyPoseAdapter::PreUnlinked(URigHierarchy* InHierarchy)
{
	LastTopologyVersion = UINT32_MAX;
}

void FRigHierarchyPoseAdapter::PostUnlinked(URigHierarchy* InHierarchy)
{
	WeakHierarchy.Reset();
}

bool FRigHierarchyPoseAdapter::IsLinkedTo(const URigHierarchy* InHierarchy) const
{
	check(InHierarchy);
	return IsLinked() && (GetHierarchy() == InHierarchy);
}

bool FRigHierarchyPoseAdapter::IsUpdateToDate(const URigHierarchy* InHierarchy) const
{
	check(InHierarchy);
	if(!IsLinkedTo(InHierarchy))
	{
		return false;
	}
	return LastTopologyVersion == InHierarchy->GetTopologyVersion();
}

TTuple<FRigComputedTransform*, FRigTransformDirtyState*> FRigHierarchyPoseAdapter::GetElementTransformStorage(
	const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType, ERigTransformStorageType::Type InStorageType) const
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		return Hierarchy->GetElementTransformStorage(InKeyAndIndex, InTransformType, InStorageType);
	}
	return {nullptr, nullptr};
}

bool FRigHierarchyPoseAdapter::RelinkTransformStorage(const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType,
                                                      ERigTransformStorageType::Type InStorageType, FTransform* InTransformStorage, bool* InDirtyFlagStorage)
{
	TArray<TTuple<FRigElementKeyAndIndex, ERigTransformType::Type, ERigTransformStorageType::Type, FTransform*, bool*>> Data =
		{{InKeyAndIndex, InTransformType, InStorageType, InTransformStorage, InDirtyFlagStorage}};
	return RelinkTransformStorage(Data);
}

bool FRigHierarchyPoseAdapter::RestoreTransformStorage(const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType,
	ERigTransformStorageType::Type InStorageType, bool bUpdateElementStorage)
{
	TArray<TTuple<FRigElementKeyAndIndex, ERigTransformType::Type, ERigTransformStorageType::Type>> Data =
		{{InKeyAndIndex, InTransformType, InStorageType}};
	return RestoreTransformStorage(Data, bUpdateElementStorage);
}

bool FRigHierarchyPoseAdapter::RelinkTransformStorage(
	const TArrayView<TTuple<FRigElementKeyAndIndex, ERigTransformType::Type, ERigTransformStorageType::Type, FTransform*, bool*>>& InData)
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		TArray<int32> TransformIndicesToDeallocate;
		TArray<int32> DirtyStateIndicesToDeallocate;
		TransformIndicesToDeallocate.Reserve(InData.Num());
		DirtyStateIndicesToDeallocate.Reserve(InData.Num());
		
		bool bPerformedChange = false;
		for(const TTuple<FRigElementKeyAndIndex, ERigTransformType::Type, ERigTransformStorageType::Type, FTransform*, bool*>& Tuple : InData)
		{
			auto CurrentStorage = Hierarchy->GetElementTransformStorage(Tuple.Get<0>(), Tuple.Get<1>(), Tuple.Get<2>());

			if(FTransform* NewTransformStorage = Tuple.Get<3>())
			{
				const FTransform PreviousTransform = CurrentStorage.Get<0>()->Get();
				if(Hierarchy->ElementTransforms.Contains(CurrentStorage.Get<0>()))
				{
					TransformIndicesToDeallocate.Add(CurrentStorage.Get<0>()->GetStorageIndex());
				}
				CurrentStorage.Get<0>()->StorageIndex = INDEX_NONE;
				CurrentStorage.Get<0>()->Storage = NewTransformStorage;
				CurrentStorage.Get<0>()->Set(PreviousTransform);
				bPerformedChange = true;
			}
			if(bool* NewDirtyStateStorage = Tuple.Get<4>())
			{
				const bool bPreviousState = CurrentStorage.Get<1>()->Get();
				if(Hierarchy->ElementDirtyStates.Contains(CurrentStorage.Get<1>()))
				{
					DirtyStateIndicesToDeallocate.Add(CurrentStorage.Get<1>()->GetStorageIndex());
				}
				CurrentStorage.Get<1>()->StorageIndex = INDEX_NONE;
				CurrentStorage.Get<1>()->Storage = NewDirtyStateStorage;
				CurrentStorage.Get<1>()->Set(bPreviousState);
				bPerformedChange = true;
			}
		}

		Hierarchy->ElementTransforms.Deallocate(TransformIndicesToDeallocate);
		Hierarchy->ElementDirtyStates.Deallocate(DirtyStateIndicesToDeallocate);
		return bPerformedChange;
	}
	return false;
}

bool FRigHierarchyPoseAdapter::RestoreTransformStorage(
	const TArrayView<TTuple<FRigElementKeyAndIndex, ERigTransformType::Type, ERigTransformStorageType::Type>>& InData, bool bUpdateElementStorage)
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		TArray<TTuple<FRigComputedTransform*, FRigTransformDirtyState*>> StoragePerElement;
		StoragePerElement.Reserve(InData.Num());
		for(const TTuple<FRigElementKeyAndIndex, ERigTransformType::Type, ERigTransformStorageType::Type>& Tuple : InData)
		{
			const TTuple<FRigComputedTransform*, FRigTransformDirtyState*> CurrentStorage =
				Hierarchy->GetElementTransformStorage(Tuple.Get<0>(), Tuple.Get<1>(), Tuple.Get<2>());

			if(CurrentStorage.Get<0>() == nullptr || CurrentStorage.Get<1>() == nullptr)
			{
				continue;
			}
			if(Hierarchy->ElementTransforms.Contains(CurrentStorage.Get<0>()) ||
				Hierarchy->ElementDirtyStates.Contains(CurrentStorage.Get<1>()))
			{
				continue;
			}
			StoragePerElement.Add(CurrentStorage);
		}

		if(StoragePerElement.IsEmpty())
		{
			return false;
		}

		const TArray<int32, TInlineAllocator<4>> NewTransformIndices = Hierarchy->ElementTransforms.Allocate(StoragePerElement.Num(), FTransform::Identity);
		const TArray<int32, TInlineAllocator<4>> NewDirtyStateIndices = Hierarchy->ElementDirtyStates.Allocate(StoragePerElement.Num(), false);
		check(StoragePerElement.Num() == NewTransformIndices.Num());
		check(StoragePerElement.Num() == NewDirtyStateIndices.Num());
		for(int32 Index = 0; Index < StoragePerElement.Num(); Index++)
		{
			FRigComputedTransform* ComputedTransform = StoragePerElement[Index].Get<0>();
			FRigTransformDirtyState* DirtyState = StoragePerElement[Index].Get<1>();

			const FTransform PreviousTransform = ComputedTransform->Get();
			const bool bPreviousState = DirtyState->Get();

			ComputedTransform->StorageIndex = NewTransformIndices[Index];
			ComputedTransform->Storage = &Hierarchy->ElementTransforms[ComputedTransform->StorageIndex];

			DirtyState->StorageIndex = NewDirtyStateIndices[Index];
			DirtyState->Storage = &Hierarchy->ElementDirtyStates[DirtyState->StorageIndex];
			
			ComputedTransform->Set(PreviousTransform);
			DirtyState->Set(bPreviousState);
		}
		
		if(bUpdateElementStorage)
		{
			(void)UpdateHierarchyStorage();
			(void)SortHierarchyStorage();
		}
		return true;
	}
	return false;
}

bool FRigHierarchyPoseAdapter::RelinkCurveStorage(const FRigElementKeyAndIndex& InKeyAndIndex, float* InCurveStorage)
{
	TArray<TTuple<FRigElementKeyAndIndex, float*>> Data = {{InKeyAndIndex, InCurveStorage}};
	return RelinkCurveStorage(Data);
}

bool FRigHierarchyPoseAdapter::RestoreCurveStorage(const FRigElementKeyAndIndex& InKeyAndIndex, bool bUpdateElementStorage)
{
	TArray<FRigElementKeyAndIndex> Data = {InKeyAndIndex};
	return RestoreCurveStorage(Data, bUpdateElementStorage);
}

bool FRigHierarchyPoseAdapter::RelinkCurveStorage(const TArrayView<TTuple<FRigElementKeyAndIndex, float*>>& InData)
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		TArray<int32> CurveIndicesToDeallocate;
		CurveIndicesToDeallocate.Reserve(InData.Num());
		
		bool bPerformedChange = false;
		for(const TTuple<FRigElementKeyAndIndex, float*>& Tuple : InData)
		{
			FRigCurveElement* CurveElement = Hierarchy->Get<FRigCurveElement>(Tuple.Get<0>());
			const float PreviousValue = CurveElement->Get();

			if(float* NewCurveStorage = Tuple.Get<1>())
			{
				if(Hierarchy->ElementCurves.Contains(CurveElement))
				{
					CurveIndicesToDeallocate.Add(CurveElement->GetStorageIndex());
				}
				CurveElement->StorageIndex = INDEX_NONE;
				CurveElement->Storage = NewCurveStorage;
				bPerformedChange = true;
			}

			CurveElement->Set(PreviousValue, CurveElement->bIsValueSet);
		}

		Hierarchy->ElementCurves.Deallocate(CurveIndicesToDeallocate);
		return bPerformedChange;
	}
	return false;
}

bool FRigHierarchyPoseAdapter::RestoreCurveStorage(const TArrayView<FRigElementKeyAndIndex>& InData, bool bUpdateElementStorage)
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		TArray<FRigCurveElement*> Curves;
		Curves.Reserve(InData.Num());
		for(const FRigElementKeyAndIndex& KeyAndIndex : InData)
		{
			if(FRigCurveElement* CurveElement = Hierarchy->Get<FRigCurveElement>(KeyAndIndex))
			{
				if(!Hierarchy->ElementCurves.Contains(CurveElement))
				{
					Curves.Add(CurveElement);
				}
			}
		}

		if(Curves.IsEmpty())
		{
			return false;
		}

		const TArray<int32, TInlineAllocator<4>> NewCurveIndices = Hierarchy->ElementCurves.Allocate(Curves.Num(), 0.f);
		check(Curves.Num() == NewCurveIndices.Num());
		for(int32 Index = 0; Index < Curves.Num(); Index++)
		{
			FRigCurveElement* CurveElement = Curves[Index];
			const float PreviousValue = CurveElement->Get();
			CurveElement->StorageIndex = NewCurveIndices[Index];
			CurveElement->Storage = &Hierarchy->ElementCurves[CurveElement->StorageIndex];
			CurveElement->Set(PreviousValue, CurveElement->bIsValueSet);
		}

		if(bUpdateElementStorage)
		{
			(void)UpdateHierarchyStorage();
		}
		return true;
	}
	return false;
}

bool FRigHierarchyPoseAdapter::SortHierarchyStorage()
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		return Hierarchy->SortElementStorage();
	}
	return false;
}

bool FRigHierarchyPoseAdapter::ShrinkHierarchyStorage()
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		return Hierarchy->ShrinkElementStorage();
	}
	return false;
}

bool FRigHierarchyPoseAdapter::UpdateHierarchyStorage()
{
	if(URigHierarchy* Hierarchy = GetHierarchy())
	{
		Hierarchy->UpdateElementStorage();
		return true;
	}
	return false;
}
