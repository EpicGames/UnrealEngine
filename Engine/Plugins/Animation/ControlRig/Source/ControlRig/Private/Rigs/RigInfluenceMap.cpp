// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigInfluenceMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigInfluenceMap)

////////////////////////////////////////////////////////////////////////////////
// RigInfluenceEntry
////////////////////////////////////////////////////////////////////////////////

void FRigInfluenceEntry::OnKeyRemoved(const FRigElementKey& InKey)
{
	AffectedList.Remove(InKey);
}

void FRigInfluenceEntry::OnKeyRenamed(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey)
{
	for (FRigElementKey& AffectedKey : AffectedList)
	{
		if (AffectedKey == InOldKey)
		{
			AffectedKey = InNewKey;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
// RigInfluenceMap
////////////////////////////////////////////////////////////////////////////////

FRigInfluenceEntry& FRigInfluenceMap::FindOrAdd(const FRigElementKey& InKey)
{
	int32 Index = GetIndex(InKey);
	if(Index == INDEX_NONE)
	{
		FRigInfluenceEntry NewEntry;
		NewEntry.Source = InKey;
		Index = Entries.Add(NewEntry);
		KeyToIndex.Add(InKey, Index);
	}
	return Entries[Index];
}

const FRigInfluenceEntry* FRigInfluenceMap::Find(const FRigElementKey& InKey) const
{
	int32 Index = GetIndex(InKey);
	if(Index == INDEX_NONE)
	{
		return nullptr;
	}
	return &Entries[Index];
}

void FRigInfluenceMap::Remove(const FRigElementKey& InKey)
{
	int32 Index = GetIndex(InKey);
	if(Index == INDEX_NONE)
	{
		return;
	}

	Entries.RemoveAt(Index);
	for(TPair<FRigElementKey, int32>& Pair : KeyToIndex)
	{
		if(Pair.Value > Index)
		{
			Pair.Value--;
		}
	}
}

FRigInfluenceMap FRigInfluenceMap::Inverse() const
{
	FRigInfluenceMap Inverted;
	Inverted.EventName = EventName;

	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		const FRigInfluenceEntry& Entry = Entries[EntryIndex];
		for (int32 AffectedIndex = 0; AffectedIndex < Entry.Num(); AffectedIndex++)
		{
			Inverted.FindOrAdd(Entry[AffectedIndex]).AddUnique(Entry.Source);
		}
	}
	return Inverted;
}

bool FRigInfluenceMap::Merge(const FRigInfluenceMap& Other)
{
	return Merge(Other, false);
}

FRigInfluenceEntryModifier FRigInfluenceMap::GetEntryModifier(const FRigElementKey& InKey) const
{
	FRigInfluenceEntryModifier Modifier;
	if (const FRigInfluenceEntry* EntryPtr = Find(InKey))
	{
		const FRigInfluenceEntry& Entry = *EntryPtr;
		for (const FRigElementKey& Affected : Entry)
		{
			Modifier.AffectedList.Add(Affected);
		}
	}
	return Modifier;
}

void FRigInfluenceMap::SetEntryModifier(const FRigElementKey& InKey, const FRigInfluenceEntryModifier& InModifier)
{
	FRigInfluenceEntry& Entry = FindOrAdd(InKey);
	Entry.AffectedList.Reset();

	for (const FRigElementKey& AffectedKey : InModifier)
	{
		Entry.AddUnique(AffectedKey);
	}
}

void FRigInfluenceMap::OnKeyRemoved(const FRigElementKey& InKey)
{
	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		FRigInfluenceEntry& Entry = Entries[EntryIndex];
		if (Entry.Source != InKey)
		{
			Entry.OnKeyRemoved(InKey);
		}
	}
	Remove(InKey);
}

void FRigInfluenceMap::OnKeyRenamed(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey)
{
	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		FRigInfluenceEntry& Entry = Entries[EntryIndex];
		if (Entry.Source != InOldKey)
		{
			Entry.OnKeyRenamed(InOldKey, InNewKey);
		}
	}

	int32 IndexToRename = GetIndex(InOldKey);
	if (IndexToRename != INDEX_NONE)
	{
		Entries[IndexToRename].Source = InNewKey;
		KeyToIndex.Remove(InOldKey);
		KeyToIndex.Add(InNewKey, IndexToRename);
	}
}

bool FRigInfluenceMap::Merge(const FRigInfluenceMap& Other, bool bIgnoreEventName)
{
	if(!bIgnoreEventName && (Other.EventName != EventName))
	{
		return false;
	}

	FRigInfluenceMap Temp = *this;
	for(const FRigInfluenceEntry& OtherEntry : Other)
	{
		FRigInfluenceEntry& Entry = Temp.FindOrAdd(OtherEntry.Source);
		if(!Entry.Merge(OtherEntry))
		{
			return false;
		}
	}

	*this = Temp;
	return true;
}

////////////////////////////////////////////////////////////////////////////////
// FRigInfluenceMapPerEvent
////////////////////////////////////////////////////////////////////////////////

FRigInfluenceMap& FRigInfluenceMapPerEvent::FindOrAdd(const FName& InEventName)
{
	int32 Index = GetIndex(InEventName);
	if(Index == INDEX_NONE)
	{
		FRigInfluenceMap NewMap;
		NewMap.EventName = InEventName;
		Index = Maps.Add(NewMap);
		EventToIndex.Add(InEventName, Index);
	}
	return Maps[Index];
}

const FRigInfluenceMap* FRigInfluenceMapPerEvent::Find(const FName& InEventName) const
{
	int32 Index = GetIndex(InEventName);
	if(Index == INDEX_NONE)
	{
		return nullptr;
	}
	return &Maps[Index];
}

void FRigInfluenceMapPerEvent::Remove(const FName& InEventName)
{
	int32 Index = GetIndex(InEventName);
	if(Index == INDEX_NONE)
	{
		return;
	}

	Maps.RemoveAt(Index);
	for(TPair<FName, int32>& Pair : EventToIndex)
	{
		if(Pair.Value > Index)
		{
			Pair.Value--;
		}
	}
}

FRigInfluenceMapPerEvent FRigInfluenceMapPerEvent::Inverse() const
{
	FRigInfluenceMapPerEvent Inverted;
	for (int32 MapIndex = 0; MapIndex < Maps.Num(); MapIndex++)
	{
		const FRigInfluenceMap& Map = Maps[MapIndex];
		FRigInfluenceMap& InvertedMap = Inverted.FindOrAdd(Map.GetEventName());
		InvertedMap = Map.Inverse();
	}
	return Inverted;
}

bool FRigInfluenceMapPerEvent::Merge(const FRigInfluenceMapPerEvent& Other)
{
	FRigInfluenceMapPerEvent Temp = *this;
	for(const FRigInfluenceMap& OtherMap : Other)
	{
		FRigInfluenceMap& Map = Temp.FindOrAdd(OtherMap.EventName);
		if(!Map.Merge(OtherMap))
		{
			return false;
		}
	}

	*this = Temp;
	return true;
}

FRigInfluenceEntryModifier FRigInfluenceMapPerEvent::GetEntryModifier(const FName& InEventName, const FRigElementKey& InKey) const
{
	if (const FRigInfluenceMap* MapPtr = Find(InEventName))
	{
		return MapPtr->GetEntryModifier(InKey);
	}
	return FRigInfluenceEntryModifier();
}

void FRigInfluenceMapPerEvent::SetEntryModifier(const FName& InEventName, const FRigElementKey& InKey, const FRigInfluenceEntryModifier& InModifier)
{
	FindOrAdd(InEventName).SetEntryModifier(InKey, InModifier);
}

void FRigInfluenceMapPerEvent::OnKeyRemoved(const FRigElementKey& InKey)
{
	for (FRigInfluenceMap& Map : Maps)
	{
		Map.OnKeyRemoved(InKey);
	}
}

void FRigInfluenceMapPerEvent::OnKeyRenamed(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey)
{
	for (FRigInfluenceMap& Map : Maps)
	{
		Map.OnKeyRenamed(InOldKey, InNewKey);
	}
}
