// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigModuleDefines.h"

#include "ModularRigModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigModuleDefines)

bool FRigModuleConnector::operator==(const FRigModuleConnector& Other) const
{
	return Name == Other.Name && GetTypeHash(Settings) == GetTypeHash(Other.Settings);
}

FModuleReferenceData::FModuleReferenceData(const FRigModuleReference* InModule)
{
	if (InModule)
	{
		ModulePath = InModule->GetModulePath();
		if (InModule->ControlRigAssetReference.IsValid())
		{
			ReferencedModuleAsset = InModule->ControlRigAssetReference;
		}
	}
}

void FModuleReferenceData::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (ReferencedModule_DEPRECATED.IsValid())
		{
			ReferencedModuleAsset.Set(TSoftClassPtr<UControlRig>(ReferencedModule_DEPRECATED));
			ReferencedModule_DEPRECATED.Reset();
		}
	}
}
