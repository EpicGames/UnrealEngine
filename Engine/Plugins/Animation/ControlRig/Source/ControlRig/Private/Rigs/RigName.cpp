// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigName.h"

const FName FRigName::EmptyName(NAME_None);
const FString FRigName::EmptyString;
