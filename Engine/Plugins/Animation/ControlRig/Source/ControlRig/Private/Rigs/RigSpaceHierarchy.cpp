// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rigs/RigSpaceHierarchy.h"
#include "Rigs/RigHierarchyContainer.h"
#include "HelperUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigSpaceHierarchy)

////////////////////////////////////////////////////////////////////////////////
// FRigSpaceHierarchy
////////////////////////////////////////////////////////////////////////////////

FRigSpaceHierarchy::FRigSpaceHierarchy()
{
}

FRigSpace& FRigSpaceHierarchy::Add(const FName& InNewName, ERigSpaceType InSpaceType, const FName& InParentName, const FTransform& InTransform)
{
	FRigSpace NewSpace;
	NewSpace.Name = InNewName;
	NewSpace.ParentIndex = INDEX_NONE; // we no longer support indices 
	NewSpace.SpaceType = InSpaceType;
	NewSpace.ParentName = InParentName;
	NewSpace.InitialTransform = InTransform;
	NewSpace.LocalTransform = InTransform;
	const int32 Index = Spaces.Add(NewSpace);
	return Spaces[Index];
}

