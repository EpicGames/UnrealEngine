// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UControlRigLayerInstance.cpp: Layer AnimInstance that support single Source Anim Instance and multiple control rigs
	The source AnimInstance can be any AnimBlueprint 
=============================================================================*/ 

#include "Sequencer/ControlRigLayerInstance.h"
#include "Sequencer/ControlRigLayerInstanceProxy.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigLayerInstance)

/////////////////////////////////////////////////////
// UControlRigLayerInstance
/////////////////////////////////////////////////////

const FName UControlRigLayerInstance::SequencerPoseName(TEXT("Sequencer_Pose_Name"));

UControlRigLayerInstance::UControlRigLayerInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = false;
}

FAnimInstanceProxy* UControlRigLayerInstance::CreateAnimInstanceProxy()
{
	return new FControlRigLayerInstanceProxy(this);
}

void UControlRigLayerInstance::NativeUpdateAnimation(float DeltaSeconds)
{
	Super::NativeUpdateAnimation(DeltaSeconds);
	
	if (UAnimInstance* SourceAnimInstance = GetSourceAnimInstance())
	{
		if (UControlRig* ControlRig = GetFirstAvailableControlRig())
		{
			if (ControlRig->IsAdditive())
			{
				SourceAnimInstance->UpdateAnimation(DeltaSeconds, true, EUpdateAnimationFlag::Default);
			}
		}
	}
}

void UControlRigLayerInstance::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies)
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().UpdateAnimTrack(InAnimSequence, SequenceId, InPosition, Weight, bFireNotifies);
}

void UControlRigLayerInstance::UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies)
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().UpdateAnimTrack(InAnimSequence, SequenceId, InFromPosition, InToPosition, Weight, bFireNotifies);
}

void UControlRigLayerInstance::ConstructNodes()
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().ConstructNodes();
}

void UControlRigLayerInstance::ResetNodes()
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().ResetNodes();
}

void UControlRigLayerInstance::ResetPose()
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().ResetPose();
}

UControlRig* UControlRigLayerInstance::GetFirstAvailableControlRig() const
{
	return GetProxyOnGameThread<FControlRigLayerInstanceProxy>().GetFirstAvailableControlRig();
}

/** Anim Instance Source info - created externally and used here */
void UControlRigLayerInstance::SetSourceAnimInstance(UAnimInstance* SourceAnimInstance)
{
	USkeletalMeshComponent* MeshComponent = GetOwningComponent();
	ensure (MeshComponent->GetAnimInstance() != SourceAnimInstance);

	if (SourceAnimInstance)
	{
		// Add the current animation instance as a linked instance
		FLinkedInstancesAdapter::AddLinkedInstance(MeshComponent, SourceAnimInstance);

		// Direct the control rig instance to the current animation instance to evaluate as its source (input pose)
		GetProxyOnGameThread<FControlRigLayerInstanceProxy>().SetSourceAnimInstance(SourceAnimInstance, UAnimInstance::GetProxyOnGameThreadStatic<FAnimInstanceProxy>(SourceAnimInstance));
	}
	else
	{
		UAnimInstance* CurrentSourceAnimInstance = GetProxyOnGameThread<FControlRigLayerInstanceProxy>().GetSourceAnimInstance();		
		// Remove the original instances from the linked instances as it should be reinstated as the main anim instance
		FLinkedInstancesAdapter::RemoveLinkedInstance(MeshComponent, CurrentSourceAnimInstance);

		// Null out the animation instance used to as the input source for the control rig instance
		GetProxyOnGameThread<FControlRigLayerInstanceProxy>().SetSourceAnimInstance(nullptr, nullptr);
	}
}

/** ControlRig related support */
void UControlRigLayerInstance::AddControlRigTrack(int32 ControlRigID, UControlRig* InControlRig)
{
	if (InControlRig->IsAdditive())
	{
		if (UAnimInstance* SourceAnimInstance = GetSourceAnimInstance())
		{
			SourceAnimInstance->bUseMultiThreadedAnimationUpdate = false;
		}
	}
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().AddControlRigTrack(ControlRigID, InControlRig);
}

bool UControlRigLayerInstance::HasControlRigTrack(int32 ControlRigID)
{
	return GetProxyOnGameThread<FControlRigLayerInstanceProxy>().HasControlRigTrack(ControlRigID);
}

void UControlRigLayerInstance::UpdateControlRigTrack(int32 ControlRigID, float Weight, const FControlRigIOSettings& InputSettings, bool bExecute)
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().UpdateControlRigTrack(ControlRigID, Weight, InputSettings, bExecute);
}

void UControlRigLayerInstance::RemoveControlRigTrack(int32 ControlRigID)
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().RemoveControlRigTrack(ControlRigID);
	if (UAnimInstance* SourceAnimInstance = GetSourceAnimInstance())
	{
		if (UControlRig* ControlRig = GetFirstAvailableControlRig())
		{
			SourceAnimInstance->bUseMultiThreadedAnimationUpdate = !ControlRig->IsAdditive();
		}
		else
		{
			SourceAnimInstance->bUseMultiThreadedAnimationUpdate = true;
		}
	}
}

void UControlRigLayerInstance::ResetControlRigTracks()
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().ResetControlRigTracks();
}

/** Sequencer AnimInstance Interface */
void UControlRigLayerInstance::AddAnimation(int32 SequenceId, UAnimSequenceBase* InAnimSequence)
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().AddAnimation(SequenceId, InAnimSequence);
}

void UControlRigLayerInstance::RemoveAnimation(int32 SequenceId)
{
	GetProxyOnGameThread<FControlRigLayerInstanceProxy>().RemoveAnimation(SequenceId);
}

void UControlRigLayerInstance::SavePose()
{
	if (USkeletalMeshComponent* SkeletalMeshComponent = GetSkelMeshComponent())
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() && SkeletalMeshComponent->GetComponentSpaceTransforms().Num() > 0)
		{
			SavePoseSnapshot(UControlRigLayerInstance::SequencerPoseName);
		}
	}
}

UAnimInstance* UControlRigLayerInstance::GetSourceAnimInstance()  
{	
	return GetProxyOnGameThread<FControlRigLayerInstanceProxy>().GetSourceAnimInstance();
}

#if WITH_EDITOR
void UControlRigLayerInstance::HandleObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	Super::HandleObjectsReinstanced(OldToNewInstanceMap);

	static IConsoleVariable* UseLegacyAnimInstanceReinstancingBehavior = IConsoleManager::Get().FindConsoleVariable(TEXT("bp.UseLegacyAnimInstanceReinstancingBehavior"));
	if(UseLegacyAnimInstanceReinstancingBehavior == nullptr || !UseLegacyAnimInstanceReinstancingBehavior->GetBool())
	{
		// Forward to control rig nodes
		FControlRigLayerInstanceProxy& Proxy = GetProxyOnGameThread<FControlRigLayerInstanceProxy>();
		for(TSharedPtr<FAnimNode_ControlRig_ExternalSource>& ControlRigNode : Proxy.ControlRigNodes)
		{
			if(ControlRigNode.IsValid())
			{
				ControlRigNode->HandleObjectsReinstanced(OldToNewInstanceMap);
			}
		}
	}
}
#endif

