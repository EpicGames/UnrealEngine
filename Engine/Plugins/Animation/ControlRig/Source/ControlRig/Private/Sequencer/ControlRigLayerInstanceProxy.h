// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstanceProxy.h"
#include "AnimNode_ControlRig_ExternalSource.h"
#include "ControlRigLayerInstanceProxy.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class UAnimSequencerInstance;

/** Custom internal Input Pose node that handles any AnimInstance */
USTRUCT()
struct FAnimNode_ControlRigInputPose : public FAnimNode_Base
{
	GENERATED_BODY()

	FAnimNode_ControlRigInputPose()
		: InputProxy(nullptr)
		, InputAnimInstance(nullptr)
	{
	}

	/** Input pose, optionally linked dynamically to another graph */
	UPROPERTY()
	FPoseLink InputPose;

	// FAnimNode_Base interface
	virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	/** Called by linked instance nodes to dynamically link this to an outer graph */
	void Link(UAnimInstance* InInputInstance, FAnimInstanceProxy* InInputProxy);

	/** Called by linked instance nodes to dynamically unlink this to an outer graph */
	void Unlink();

private:
	/** The proxy to use when getting inputs, set when dynamically linked */
	FAnimInstanceProxy* InputProxy;
	UAnimInstance*		InputAnimInstance;
};

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FControlRigLayerInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

public:
	FControlRigLayerInstanceProxy()
		: CurrentRoot(nullptr)
		, CurrentSourceAnimInstance(nullptr)
	{
	}

	FControlRigLayerInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
		, CurrentRoot(nullptr)
		, CurrentSourceAnimInstance(nullptr)
	{
	}

	UE_API virtual ~FControlRigLayerInstanceProxy();

	// FAnimInstanceProxy interface
	UE_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	UE_API virtual bool Evaluate(FPoseContext& Output) override;
	UE_API virtual void CacheBones() override;
	UE_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	UE_API virtual void PreEvaluateAnimation(UAnimInstance* InAnimInstance) override;

	/** Anim Instance Source info - created externally and used here */
	UE_API void SetSourceAnimInstance(UAnimInstance* SourceAnimInstance, FAnimInstanceProxy* SourceAnimInputProxy);
	UAnimInstance* GetSourceAnimInstance() const { return CurrentSourceAnimInstance; }

	/** ControlRig related support */
	UE_API void AddControlRigTrack(int32 ControlRigID, UControlRig* InControlRig);
	UE_API void UpdateControlRigTrack(int32 ControlRigID, float Weight, const FControlRigIOSettings& InputSettings, bool bExecute);
	UE_API void RemoveControlRigTrack(int32 ControlRigID);
	UE_API bool HasControlRigTrack(int32 ControlRigID);
	UE_API void ResetControlRigTracks();

	/** Sequencer AnimInstance Interface */
	UE_API void AddAnimation(int32 SequenceId, UAnimSequenceBase* InAnimSequence);
	UE_API void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies);
	UE_API void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies);
	UE_API void RemoveAnimation(int32 SequenceId);

	/** Reset all nodes in this instance */
	UE_API virtual void ResetNodes();
	/** Reset the pose in this instance*/
	UE_API virtual void ResetPose();
	/** Construct and link the base part of the blend tree */
	UE_API virtual void ConstructNodes();

	/** return first available control rig from the node it has */
	UE_API UControlRig* GetFirstAvailableControlRig() const;

	UE_API virtual void AddReferencedObjects(UAnimInstance* InAnimInstance, FReferenceCollector& Collector) override;

	// this doesn't work because this instance continuously change root
	// this will invalidate the evaluation
// 	virtual FAnimNode_Base* GetCustomRootNode() 

	friend struct FAnimNode_ControlRigInputPose;
	friend class UControlRigLayerInstance;
protected:
	/** Sort Control Rig node*/
	UE_API void SortControlRigNodes();

	/** Find ControlRig node of the */
	UE_API FAnimNode_ControlRig_ExternalSource* FindControlRigNode(int32 ControlRigID) const;

	/** Input pose anim node */
	FAnimNode_ControlRigInputPose InputPose;

	/** Cuyrrent Root node - this changes whenever track changes */
	FAnimNode_Base* CurrentRoot;

	/** ControlRig Nodes */
	TArray<TSharedPtr<FAnimNode_ControlRig_ExternalSource>> ControlRigNodes;

	/** mapping from sequencer index to internal player index */
	TMap<int32, FAnimNode_ControlRig_ExternalSource*> SequencerToControlRigNodeMap;

	/** Source Anim Instance */
	TObjectPtr<UAnimInstance> CurrentSourceAnimInstance;

	/** getter for Sequencer AnimInstance. It will return null if it's using AnimBP */
	UE_API UAnimSequencerInstance* GetSequencerAnimInstance();

	static UE_API void InitializeCustomProxy(FAnimInstanceProxy* InputProxy, UAnimInstance* InAnimInstance);
	static UE_API void GatherCustomProxyDebugData(FAnimInstanceProxy* InputProxy, FNodeDebugData& DebugData);
	static UE_API void CacheBonesCustomProxy(FAnimInstanceProxy* InputProxy);
	static UE_API void UpdateCustomProxy(FAnimInstanceProxy* InputProxy, const FAnimationUpdateContext& Context);
	static UE_API void EvaluateCustomProxy(FAnimInstanceProxy* InputProxy, FPoseContext& Output);
	/** reset internal Counters of given animinstance proxy */
	static UE_API void ResetCounter(FAnimInstanceProxy* InAnimInstanceProxy);
};

#undef UE_API
