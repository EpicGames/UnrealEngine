// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/ControlRigSequenceObjectReference.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSequenceObjectReference)

FControlRigSequenceObjectReference FControlRigSequenceObjectReference::Create(UControlRig* InControlRig)
{
	check(InControlRig);

	FControlRigSequenceObjectReference NewReference;
	NewReference.ControlRigAssetReference = InControlRig->GetAssetReference();

	return NewReference;
}

bool FControlRigSequenceObjectReferenceMap::HasBinding(const FGuid& ObjectId) const
{
	return BindingIds.Contains(ObjectId);
}

void FControlRigSequenceObjectReferenceMap::RemoveBinding(const FGuid& ObjectId)
{
	int32 Index = BindingIds.IndexOfByKey(ObjectId);
	if (Index != INDEX_NONE)
	{
		BindingIds.RemoveAtSwap(Index, EAllowShrinking::No);
		References.RemoveAtSwap(Index, EAllowShrinking::No);
	}
}

void FControlRigSequenceObjectReferenceMap::CreateBinding(const FGuid& ObjectId, const FControlRigSequenceObjectReference& ObjectReference)
{
	int32 ExistingIndex = BindingIds.IndexOfByKey(ObjectId);
	if (ExistingIndex == INDEX_NONE)
	{
		ExistingIndex = BindingIds.Num();

		BindingIds.Add(ObjectId);
		References.AddDefaulted();
	}

	References[ExistingIndex].Array.AddUnique(ObjectReference);
}

