// Copyright Epic Games, Inc. All Rights Reserved.
#include "Sequencer/ControlRigSequencerHelpers.h"
#include "ControlRig.h"
#include "Rigs/RigHierarchyElements.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"

TPair<const FChannelMapInfo*, int32> FControlRigSequencerHelpers::GetInfoAndNumFloatChannels(
	const UControlRig* InControlRig,
	const FName& InControlName,
	const UMovieSceneControlRigParameterSection* InSection)
{
	const FRigControlElement* ControlElement = InControlRig ? InControlRig->FindControl(InControlName) : nullptr;
	auto GetNumFloatChannels = [](const ERigControlType& InControlType)
	{
		switch (InControlType)
		{
		case ERigControlType::Position:
		case ERigControlType::Scale:
		case ERigControlType::Rotator:
			return 3;
		case ERigControlType::TransformNoScale:
			return 6;
		case ERigControlType::Transform:
		case ERigControlType::EulerTransform:
			return 9;
		case ERigControlType::Vector2D:
			return 2;
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
			return 1;
		default:
			break;
		}
		return 0;
	};

	const int32 NumFloatChannels = ControlElement ? GetNumFloatChannels(ControlElement->Settings.ControlType) : 0;
	const FChannelMapInfo* ChannelInfo = InSection ? InSection->ControlChannelMap.Find(InControlName) : nullptr;

	return { ChannelInfo, NumFloatChannels };
}


TArrayView<FMovieSceneFloatChannel*>  FControlRigSequencerHelpers::GetFloatChannels(const UControlRig* InControlRig,
	const FName& InControlName, const UMovieSceneSection* InSection)
{
	// no floats for transform sections
	static const TArrayView<FMovieSceneFloatChannel*> EmptyChannelsView;

	const FChannelMapInfo* ChannelInfo = nullptr;
	int32 NumChannels = 0;
	const UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (CRSection == nullptr)
	{
		return EmptyChannelsView;
	}

	Tie(ChannelInfo, NumChannels) = FControlRigSequencerHelpers::GetInfoAndNumFloatChannels(InControlRig, InControlName, CRSection);

	if (ChannelInfo == nullptr || NumChannels == 0)
	{
		return EmptyChannelsView;
	}

	// return a sub view that just represents the control's channels
	const TArrayView<FMovieSceneFloatChannel*> FloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	const int32 ChannelStartIndex = ChannelInfo->ChannelIndex;
	return FloatChannels.Slice(ChannelStartIndex, NumChannels);
}


TArrayView<FMovieSceneBoolChannel*> FControlRigSequencerHelpers::GetBoolChannels(const UControlRig* InControlRig,
	const FName& InControlName, const UMovieSceneSection* InSection)
{
	static const TArrayView<FMovieSceneBoolChannel*> EmptyChannelsView;
	const UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (CRSection == nullptr)
	{
		return EmptyChannelsView;
	}
	const FRigControlElement* ControlElement = InControlRig ? InControlRig->FindControl(InControlName) : nullptr;
	if (ControlElement == nullptr)
	{
		return EmptyChannelsView;
	}
	if (ControlElement->Settings.ControlType == ERigControlType::Bool)
	{
		int32 NumChannels = 1;
		const FChannelMapInfo* ChannelInfo = CRSection ? CRSection->ControlChannelMap.Find(InControlName) : nullptr;
		if (ChannelInfo == nullptr)
		{
			return EmptyChannelsView;
		}
		const TArrayView<FMovieSceneBoolChannel*> Channels = CRSection->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
		const int32 ChannelStartIndex = ChannelInfo->ChannelIndex;
		return Channels.Slice(ChannelStartIndex, NumChannels);

	}
	return EmptyChannelsView;
}



TArrayView<FMovieSceneByteChannel*> FControlRigSequencerHelpers::GetByteChannels(const UControlRig* InControlRig,
	const FName& InControlName, const UMovieSceneSection* InSection)
{
	static const TArrayView<FMovieSceneByteChannel*> EmptyChannelsView;
	const UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (CRSection == nullptr)
	{
		return EmptyChannelsView;
	}
	const FRigControlElement* ControlElement = InControlRig ? InControlRig->FindControl(InControlName) : nullptr;
	if (ControlElement == nullptr)
	{
		return EmptyChannelsView;
	}
	if (ControlElement->Settings.ControlType == ERigControlType::Integer && ControlElement->Settings.ControlEnum)
	{
		int32 NumChannels = 1;
		const FChannelMapInfo* ChannelInfo = CRSection ? CRSection->ControlChannelMap.Find(InControlName) : nullptr;
		if (ChannelInfo == nullptr || ChannelInfo->ChannelTypeName != FMovieSceneByteChannel::StaticStruct()->GetFName())
		{
			return EmptyChannelsView;
		}
		const TArrayView<FMovieSceneByteChannel*> Channels = CRSection->GetChannelProxy().GetChannels<FMovieSceneByteChannel>();
		const int32 ChannelStartIndex = ChannelInfo->ChannelIndex;
		return Channels.Slice(ChannelStartIndex, NumChannels);

	}
	return EmptyChannelsView;
}

TArrayView<FMovieSceneIntegerChannel*> FControlRigSequencerHelpers::GetIntegerChannels(const UControlRig* InControlRig,
	const FName& InControlName, const UMovieSceneSection* InSection)
{
	static const TArrayView<FMovieSceneIntegerChannel*> EmptyChannelsView;
	const UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (CRSection == nullptr)
	{
		return EmptyChannelsView;
	}
	const FRigControlElement* ControlElement = InControlRig ? InControlRig->FindControl(InControlName) : nullptr;
	if (ControlElement == nullptr)
	{
		return EmptyChannelsView;
	}
	if (ControlElement->Settings.ControlType == ERigControlType::Integer && ControlElement->Settings.ControlEnum == nullptr)
	{
		int32 NumChannels = 1;
		const FChannelMapInfo* ChannelInfo = CRSection ? CRSection->ControlChannelMap.Find(InControlName) : nullptr;
		if (ChannelInfo == nullptr || ChannelInfo->ChannelTypeName != FMovieSceneIntegerChannel::StaticStruct()->GetFName())
		{
			return EmptyChannelsView;
		}
		const TArrayView<FMovieSceneIntegerChannel*> Channels = CRSection->GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
		const int32 ChannelStartIndex = ChannelInfo->ChannelIndex;
		return Channels.Slice(ChannelStartIndex, NumChannels);

	}
	return EmptyChannelsView;
}

UMovieSceneControlRigParameterTrack* FControlRigSequencerHelpers::FindControlRigTrack(UMovieSceneSequence* MovieSceneSequence, const UControlRig* ControlRig)
{
	if (const UMovieScene* MovieScene = (ControlRig && MovieSceneSequence) ? MovieSceneSequence->GetMovieScene() : nullptr)
	{
		for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
		{
			const TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid());
			for (UMovieSceneTrack* Track : Tracks)
			{
				UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
				if (ControlRigTrack && ControlRigTrack->GetControlRig() == ControlRig)
				{
					return ControlRigTrack;
				}
			}
		}
	}
	
	return nullptr;
}
