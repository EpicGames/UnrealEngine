// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigComponentTypes.h"
#include "Sequencer/MovieSceneControlRigParameterBuffer.h"
#include "Sequencer/MovieSceneControlRigSpaceChannel.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

namespace UE::MovieScene
{


static TUniquePtr<FControlRigComponentTypes> GControlRigComponentTypes;

static bool GControlRigComponentTypesDestroyed = false;


FControlRigComponentTypes* FControlRigComponentTypes::Get()
{
	if (!GControlRigComponentTypes.IsValid())
	{
		check(!GControlRigComponentTypesDestroyed);
		GControlRigComponentTypes.Reset(new FControlRigComponentTypes);
	}
	return GControlRigComponentTypes.Get();
}

void FControlRigComponentTypes::Destroy()
{
	GControlRigComponentTypes.Reset();
	GControlRigComponentTypesDestroyed = true;
}

FControlRigComponentTypes::FControlRigComponentTypes()
{
	using namespace UE::MovieScene;

	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	ComponentRegistry->NewComponentType(&ControlRigSource, TEXT("Control Rig Source"), EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->Factories.DuplicateChildComponent(ControlRigSource);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ControlRigSource, BuiltInComponents->EvalSeconds);

	ComponentRegistry->NewComponentType(&BaseControlRigEvalData, TEXT("Base Control Rig Eval Data"));

	ComponentRegistry->NewComponentType(&SpaceChannel, TEXT("Space Channel"));
	ComponentRegistry->NewComponentType(&SpaceResult, TEXT("Space Result"));
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(SpaceChannel, BuiltInComponents->EvalTime);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(SpaceChannel, SpaceResult);

	Tags.BaseControlRig = ComponentRegistry->NewTag(TEXT("Base Control Rig"), EComponentTypeFlags::CopyToChildren);

	Tags.Space = ComponentRegistry->NewTag(TEXT("Control Rig Space"), EComponentTypeFlags::CopyToChildren);

	Tags.ControlRigParameter = ComponentRegistry->NewTag(TEXT("Control Rig Parameter"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);

	ComponentRegistry->NewComponentType(&AccumulatedControlEntryIndex, TEXT("Accumulated Control Entry Index"));

	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(Tags.BaseControlRig, BaseControlRigEvalData);
}


} // namespace UE::MovieScene