// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigParameterBuffer.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigSpaceChannel.h"
#include "Sequencer/MovieSceneControlRigSystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"

#include "Algo/IndexOf.h"


namespace UE::MovieScene
{

/** This function is a temporary stand-in in the absence of a proper ZipSort algorithm in the core library,
 *     which would remove the need to copy everything into a temporary array */
template<typename T>
void SortParametersByName(TArrayView<FName> Names, TArrayView<T> Parameters)
{
	check(Names.Num() == Parameters.Num());

	static constexpr int32 FixedSize = 128;
	const int32 Num = Names.Num();
	if (Num <= 1)
	{
		return;
	}

	using PairType = TPair<FName, T>;

	TArray<PairType, TFixedAllocator<FixedSize>> TempFixedSize;
	TArray<PairType> TempHeap;

	TArrayView<PairType> ZippedArray;
	if (Num <= FixedSize)
	{
		TempFixedSize.SetNumUninitialized(Num);
		ZippedArray = TempFixedSize;
	}
	else
	{
		TempHeap.SetNumUninitialized(Num);
		ZippedArray = TempHeap;
	}

	for (int32 Index = 0; Index < Num; ++Index)
	{
		new (&ZippedArray[Index]) PairType(MakeTuple(MoveTemp(Names[Index]), MoveTemp(Parameters[Index])));
	}

	// Sort by name
	Algo::SortBy(ZippedArray, [](const PairType& In) { return In.Key; }, FNameFastLess());

	for (int32 Index = 0; Index < Num; ++Index)
	{
		Names[Index] = MoveTemp(ZippedArray[Index].Key);
		Parameters[Index] = MoveTemp(ZippedArray[Index].Value);
	}
}

FControlRigParameterValueHeader::FControlRigParameterValueHeader(EControlRigControlType InType, EControlRigParameterBufferIndexStability IndexStability)
	: Data(nullptr)
	, Capacity(0)
	, NumElements(0)
	, bStableIndices(IndexStability == EControlRigParameterBufferIndexStability::Stable)
	, Type(InType)
{
	static_assert(std::is_trivially_copyable_v<FMovieSceneControlRigSpaceBaseKey>, "Parameter value types must be trivially copyable to be used inside a parameter buffer");
	static_assert(std::is_trivially_copyable_v<bool>,                              "Parameter value types must be trivially copyable to be used inside a parameter buffer");
	static_assert(std::is_trivially_copyable_v<uint8>,                             "Parameter value types must be trivially copyable to be used inside a parameter buffer");
	static_assert(std::is_trivially_copyable_v<int32>,                             "Parameter value types must be trivially copyable to be used inside a parameter buffer");
	static_assert(std::is_trivially_copyable_v<float>,                             "Parameter value types must be trivially copyable to be used inside a parameter buffer");
	static_assert(std::is_trivially_copyable_v<FVector3f>,                         "Parameter value types must be trivially copyable to be used inside a parameter buffer");
	static_assert(std::is_trivially_copyable_v<FEulerTransform>,                   "Parameter value types must be trivially copyable to be used inside a parameter buffer");

	switch (InType)
	{
		case EControlRigControlType::Space:
			Alignment = alignof(FMovieSceneControlRigSpaceBaseKey);
			break;
		case EControlRigControlType::Parameter_Bool:
			Alignment = alignof(bool);
			break;
		case EControlRigControlType::Parameter_Enum:
			Alignment = alignof(uint8);
			break;
		case EControlRigControlType::Parameter_Integer:
			Alignment = alignof(int32);
			break;
		case EControlRigControlType::Parameter_Scalar:
			Alignment = alignof(float);
			break;
		case EControlRigControlType::Parameter_Vector:
			Alignment = alignof(FVector3f);
			break;
		case EControlRigControlType::Parameter_Transform:
			Alignment = alignof(FEulerTransform);
			break;
		default:
			check(false);
			break;
	}
}

FControlRigParameterValueHeader::FControlRigParameterValueHeader(FControlRigParameterValueHeader&& RHS)
	: Data(RHS.Data)
	, Capacity(RHS.Capacity)
	, NumElements(RHS.NumElements)
	, Alignment(RHS.Alignment)
	, bStableIndices(RHS.bStableIndices)
	, Type(RHS.Type)
{
	RHS.Data            = nullptr;
	RHS.Capacity        = 0;
	RHS.NumElements     = 0;
	RHS.Alignment       = 0;
	RHS.bStableIndices  = 0;
}

FControlRigParameterValueHeader& FControlRigParameterValueHeader::operator=(FControlRigParameterValueHeader&& RHS)
{
	if (Data)
	{
		FMemory::Free(Data);
	}

	Data            = RHS.Data;
	Capacity        = RHS.Capacity;
	NumElements     = RHS.NumElements;
	Alignment       = RHS.Alignment;
	bStableIndices  = RHS.bStableIndices;
	Type            = RHS.Type;

	RHS.Data            = nullptr;
	RHS.Capacity        = 0;
	RHS.NumElements     = 0;
	RHS.Alignment       = 0;
	RHS.bStableIndices  = 0;

	return *this;
}

FControlRigParameterValueHeader::FControlRigParameterValueHeader(const FControlRigParameterValueHeader& RHS)
	: Data(nullptr)
	, Capacity(0)
	, NumElements(0)
	, Alignment(RHS.Alignment)
	, bStableIndices(RHS.bStableIndices)
	, Type(RHS.Type)
{
	Reserve(RHS.Capacity);
	NumElements = RHS.NumElements;

	if (NumElements > 0)
	{
		size_t AlignmentOffset = Alignment != alignof(FName) ? Alignment : 0;
		size_t SizeInBytes     = Capacity*sizeof(FName) + AlignmentOffset + GetParameterSize()*Capacity;

		FMemory::Memcpy(Data, RHS.Data, SizeInBytes);
	}
}

FControlRigParameterValueHeader& FControlRigParameterValueHeader::operator=(const FControlRigParameterValueHeader& RHS)
{
	if (Data)
	{
		FMemory::Free(Data);
		Data            = nullptr;
		Capacity        = 0;
		NumElements     = 0;
		Alignment       = 0;
		bStableIndices  = 0;
	}

	Alignment       = RHS.Alignment;
	bStableIndices  = RHS.bStableIndices;
	Type            = RHS.Type;

	Reserve(RHS.Capacity);
	NumElements = RHS.NumElements;

	if (NumElements > 0)
	{
		size_t AlignmentOffset = Alignment != alignof(FName) ? Alignment : 0;
		size_t SizeInBytes     = Capacity*sizeof(FName) + AlignmentOffset + GetParameterSize()*Capacity;

		FMemory::Memcpy(Data, RHS.Data, SizeInBytes);
	}
	return *this;
}


FControlRigParameterValueHeader::~FControlRigParameterValueHeader()
{
	if (Data)
	{
		FMemory::Free(Data);
	}
}

void FControlRigParameterValueHeader::Reset()
{
	NumElements = 0;
}

EControlRigControlType FControlRigParameterValueHeader::GetType() const
{
	return Type;
}

int32 FControlRigParameterValueHeader::Num() const
{
	return NumElements;
}


TArrayView<const FName> FControlRigParameterValueHeader::GetNames() const
{
	if (NumElements == 0)
	{
		return TArrayView<const FName>();
	}

	const FName* Names = reinterpret_cast<const FName*>(Data);
	return TArrayView<const FName>(Names, NumElements);
}

uint8* FControlRigParameterValueHeader::GetParameterBuffer() const
{
	return Data ? GetParameterBuffer(Data, Capacity, Alignment) : nullptr;
}

void* FControlRigParameterValueHeader::GetParameter(int32 Index) const
{
	check(Index < NumElements);
	return GetParameterBuffer() + Index*GetParameterSize();
}

int32 FControlRigParameterValueHeader::Add_GetIndex(FName InName)
{
	int32 Index = INDEX_NONE;
	if (bStableIndices)
	{
		// If we need stable indices then we should only ever grow
		Index = NumElements;
		InsertDefaulted(InName, Index);
	}
	else
	{
		TArrayView<const FName> Names = GetNames();
		Index = Algo::LowerBound(Names, InName, FNameFastLess());

		if (Index >= NumElements || Names[Index] != InName)
		{
			InsertDefaulted(InName, Index);
		}
	}

	return Index;
}

void* FControlRigParameterValueHeader::Add_GetPtr(FName InName)
{
	const int32 Index = Add_GetIndex(InName);
	return GetParameterBuffer() + GetParameterSize()*Index;
}

void FControlRigParameterValueHeader::Add(FName InName, const void* Value)
{
	void* DestBuffer = Add_GetPtr(InName);
	FMemory::Memcpy(DestBuffer, Value, GetParameterSize());
}

void FControlRigParameterValueHeader::Remove(FName InName)
{
	const int32 Index = Find(InName);
	if (Index != INDEX_NONE)
	{
		RemoveAt(Index);
	}
}
bool FControlRigParameterValueHeader::Contains(FName InName) const
{
	return Find(InName) != INDEX_NONE;
}

int32 FControlRigParameterValueHeader::Find(FName InName) const
{
	TArrayView<const FName> Names = GetNames();
	if (bStableIndices)
	{
		return Algo::IndexOf(Names, InName);
	}
	else
	{
		return Algo::BinarySearch(Names, InName, FNameFastLess());
	}
}

void FControlRigParameterValueHeader::OptimizeForLookup()
{
	if (bStableIndices == true)
	{
		bStableIndices = false;
		Resort();
	}
}


void FControlRigParameterValueHeader::Resort()
{
	check(bStableIndices == false);

	FControlRigParameterValueHeader* NonConstThis = const_cast<FControlRigParameterValueHeader*>(this);

	TArrayView<FName> Names = NonConstThis->GetMutableNames();
	switch (Type)
	{
	case EControlRigControlType::Space:
		{
			TArrayView<FMovieSceneControlRigSpaceBaseKey> Parameters = NonConstThis->GetParameters<FMovieSceneControlRigSpaceBaseKey>();
			SortParametersByName(Names, Parameters);
		}
		break;
	case EControlRigControlType::Parameter_Bool:
		{
			TArrayView<bool> Parameters = NonConstThis->GetParameters<bool>();
			SortParametersByName(Names, Parameters);
		}
		break;
	case EControlRigControlType::Parameter_Enum:
		{
			TArrayView<uint8> Parameters = NonConstThis->GetParameters<uint8>();
			SortParametersByName(Names, Parameters);
		}
		break;
	case EControlRigControlType::Parameter_Integer:
		{
			TArrayView<int32> Parameters = NonConstThis->GetParameters<int32>();
			SortParametersByName(Names, Parameters);
		}
		break;
	case EControlRigControlType::Parameter_Scalar:
		{
			TArrayView<float> Parameters = NonConstThis->GetParameters<float>();
			SortParametersByName(Names, Parameters);
		}
		break;
	case EControlRigControlType::Parameter_Vector:
		{
			TArrayView<FVector3f> Parameters = NonConstThis->GetParameters<FVector3f>();
			SortParametersByName(Names, Parameters);
		}
		break;
	case EControlRigControlType::Parameter_Transform:
		{
			TArrayView<FEulerTransform> Parameters = NonConstThis->GetParameters<FEulerTransform>();
			SortParametersByName(Names, Parameters);
		}
		break;
	}
}

TArrayView<FName> FControlRigParameterValueHeader::GetMutableNames()
{
	if (NumElements == 0)
	{
		return TArrayView<FName>();
	}

	FName* Names = reinterpret_cast<FName*>(Data);
	return TArrayView<FName>(Names, NumElements);
}

void FControlRigParameterValueHeader::RemoveAt(int32 Index)
{
	check(Index >= 0 && Index < NumElements);

	FName* Names  = reinterpret_cast<FName*>(Data);
	uint8* Values = GetParameterBuffer(Data, Capacity, Alignment);

	// Move the tail first
	const int32 TailNum = int32(NumElements) - Index - 1;
	if (TailNum > 0)
	{
		size_t ParameterSize = GetParameterSize();
		FMemory::Memmove(Names + Index, Names+(Index+1), TailNum*sizeof(FName));
		FMemory::Memmove(Values + Index*ParameterSize, Values + (Index+1)*ParameterSize, TailNum*ParameterSize);
	}

	--NumElements;
}

void FControlRigParameterValueHeader::InsertDefaulted(FName Name, int32 Index)
{
	Reserve(NumElements+1);
	check(Capacity >= NumElements+1);

	++NumElements;

	FName* Names  = reinterpret_cast<FName*>(Data);
	uint8* Values = GetParameterBuffer(Data, Capacity, Alignment);

	// Move the tail first
	const int32 TailNum = int32(NumElements) - Index - 1;
	if (TailNum > 0)
	{
		size_t ParameterSize = GetParameterSize();
		FMemory::Memmove(Names + (Index+1), Names + Index, TailNum*sizeof(FName));
		FMemory::Memmove(Values + (Index+1)*ParameterSize, Values + Index*ParameterSize, TailNum*ParameterSize);
	}

	// Emplace the new name at the correct location
	new (Names + Index) FName(MoveTemp(Name));

	switch (Type)
	{
	case EControlRigControlType::Space:
		new (Values + sizeof(FMovieSceneControlRigSpaceBaseKey)*Index) FMovieSceneControlRigSpaceBaseKey();
		break;
	case EControlRigControlType::Parameter_Bool:
		new (Values + sizeof(bool)*Index) bool(false);
		break;
	case EControlRigControlType::Parameter_Enum:
		new (Values + sizeof(uint8)*Index) uint8(0);
		break;
	case EControlRigControlType::Parameter_Integer:
		new (Values + sizeof(int32)*Index) int32(0);
		break;
	case EControlRigControlType::Parameter_Scalar:
		new (Values + sizeof(float)*Index) float(0.f);
		break;
	case EControlRigControlType::Parameter_Vector:
		new (Values + sizeof(FVector3f)*Index) FVector3f(0.f, 0.f, 0.f);
		break;
	case EControlRigControlType::Parameter_Transform:
		new (Values + sizeof(FEulerTransform)*Index) FEulerTransform(FEulerTransform::Identity);
		break;
	default:
		check(false);
		break;
	}
}

uint8* FControlRigParameterValueHeader::GetParameterBuffer(uint8* BasePtr, uint16 InCapacity, uint8 InAlignment)
{
	return Align(BasePtr + InCapacity*sizeof(FName), InAlignment);
}

size_t FControlRigParameterValueHeader::GetParameterSize() const
{
	switch (Type)
	{
	case EControlRigControlType::Space:
		return sizeof(FMovieSceneControlRigSpaceBaseKey);
	case EControlRigControlType::Parameter_Bool:
		return sizeof(bool);
	case EControlRigControlType::Parameter_Enum:
		return sizeof(uint8);
	case EControlRigControlType::Parameter_Integer:
		return sizeof(int32);
	case EControlRigControlType::Parameter_Scalar:
		return sizeof(float);
	case EControlRigControlType::Parameter_Vector:
		return sizeof(FVector3f);
	case EControlRigControlType::Parameter_Transform:
		return sizeof(FEulerTransform);
	default:
		check(false);
		return 0;
	}
}

void FControlRigParameterValueHeader::Reserve(size_t NewCapacity)
{
	const size_t Size = GetParameterSize();

	// Allocate capacity in blocks of 8
	NewCapacity = Align(NewCapacity, 8);

	size_t AlignmentOffset = Alignment != alignof(FName) ? Alignment : 0;

	uint8* OldData = Data;
	if (NewCapacity > Capacity)
	{
		const int32 RequiredCapacityBytes = NewCapacity*sizeof(FName) + AlignmentOffset + Size*NewCapacity;
		Data = (uint8*)FMemory::Malloc(RequiredCapacityBytes, alignof(FName));

		if (NumElements > 0)
		{
			// Copy the names over
			FMemory::Memcpy(Data, OldData, sizeof(FName)*NumElements);

			// Copy the values over
			uint8* OldValuePtr = GetParameterBuffer(OldData, Capacity, Alignment);
			uint8* NewValuePtr = GetParameterBuffer(Data, NewCapacity, Alignment);

			FMemory::Memcpy(NewValuePtr, OldValuePtr, Size*NumElements);
			FMemory::Free(OldData);
		}

		Capacity = NewCapacity;
	}
}


FControlRigParameterValues::FControlRigParameterValues(EControlRigParameterBufferIndexStability IndexStability)
	: Headers{
		FControlRigParameterValueHeader(EControlRigControlType::Space, IndexStability),
		FControlRigParameterValueHeader(EControlRigControlType::Parameter_Bool, IndexStability),
		FControlRigParameterValueHeader(EControlRigControlType::Parameter_Enum, IndexStability),
		FControlRigParameterValueHeader(EControlRigControlType::Parameter_Integer, IndexStability),
		FControlRigParameterValueHeader(EControlRigControlType::Parameter_Scalar, IndexStability),
		FControlRigParameterValueHeader(EControlRigControlType::Parameter_Vector, IndexStability),
		FControlRigParameterValueHeader(EControlRigControlType::Parameter_Transform, IndexStability),
	}
{}

FControlRigParameterValueHeader& FControlRigParameterValues::GetHeader(EControlRigControlType InType)
{
	return Headers[(uint8)InType];
}

const FControlRigParameterValueHeader& FControlRigParameterValues::GetHeader(EControlRigControlType InType) const
{
	return Headers[(uint8)InType];
}

FControlRigParameterValues::~FControlRigParameterValues()
{
}

void FControlRigParameterValues::OptimizeForLookup()
{
	for (FControlRigParameterValueHeader& Header : Headers)
	{
		Header.OptimizeForLookup();
	}
}

void FControlRigParameterValues::Reset()
{
	for (FControlRigParameterValueHeader& Header : Headers)
	{
		Header.Reset();
	}
}

bool FControlRigParameterValues::InitializeParameters(UControlRig* Rig, FPreAnimatedControlRigParameterStorage& Storage)
{
	check(Rig);

	// Make space for the headers
	int32 TotalNum = 0;


	FObjectKey RigKey(Rig);
	for (FControlRigParameterValueHeader& Header : Headers)
	{
		for (FName ParameterName : Header.GetNames())
		{
			FPreAnimatedStorageIndex StorageIndex = Storage.FindStorageIndex(MakeTuple(RigKey, ParameterName));
			if (StorageIndex && !Storage.IsStorageRequirementSatisfied(StorageIndex, EPreAnimatedStorageRequirement::Transient))
			{
				Storage.AssignPreAnimatedValue(StorageIndex, EPreAnimatedStorageRequirement::Transient, Storage.Traits.CachePreAnimatedValue(Rig, ParameterName));
			}
		}

		TotalNum += Header.Num();
	}
	return TotalNum != 0;
}

void FControlRigParameterValues::ApplyAndRemove(UControlRig* Rig, FName InName)
{
	for (FControlRigParameterValueHeader& Header : Headers)
	{
		Header.ApplyAndRemove(Rig, InName);
	}
}

void FControlRigParameterValues::AddCurrentValue(UControlRig* Rig, FRigControlElement* ControlElement)
{
	const FName ControlName = ControlElement->GetFName();

	auto GetSpaceValue = [ControlElement]()
	{
		//@helge how can we get the correct current space here? this is for restoring it.
		//for now just using parent space
		//FRigElementKey DefaultParent = RigHierarchy->GetFirstParent(ControlElement->GetKey());
		FMovieSceneControlRigSpaceBaseKey SpaceValue;
		SpaceValue.ControlRigElement = ControlElement->GetKey();
		SpaceValue.SpaceType = EMovieSceneControlRigSpaceType::Parent;

		return SpaceValue;
	};
	
	switch (ControlElement->Settings.ControlType)
	{
	case ERigControlType::Bool:
	{
		const bool Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<bool>();
		Add(ControlName, Val);
		break;
	}

	case ERigControlType::Float:
	case ERigControlType::ScaleFloat:
	{
		const float Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
		Add(ControlName, Val);
		break;
	}

	case ERigControlType::Integer:
	{
		if (ControlElement->Settings.ControlEnum != nullptr)
		{
			const uint8 Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<uint8>();
			Add(ControlName, Val);
		}
		else
		{
			const int32 Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int32>();
			Add(ControlName, Val);
		}
		break;
	}

	case ERigControlType::Vector2D:
	{
		const FVector3f Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
		Add(ControlName, Val);
		break;
	}

	case ERigControlType::Position:
	case ERigControlType::Scale:
	case ERigControlType::Rotator:
	{
		const FMovieSceneControlRigSpaceBaseKey SpaceValue = GetSpaceValue();
		Add(ControlName, SpaceValue);

		FVector3f Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
		if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
		{
			FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
			Val = FVector3f(Vector.X, Vector.Y, Vector.Z);
		}
		Add(ControlName, Val);
		//mz todo specify rotator special so we can do quat interps
		break;
	}

	case ERigControlType::Transform:
	{
		const FMovieSceneControlRigSpaceBaseKey SpaceValue = GetSpaceValue();
		Add(ControlName, SpaceValue);

		const FTransform Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>().ToTransform();
		FEulerTransform EulerTransform(Val);
		FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
		EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
		Add(ControlName, EulerTransform);
		break;
	}
	case ERigControlType::TransformNoScale:
	{
		const FTransformNoScale NoScale =
			Rig
			->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
		FEulerTransform EulerTransform(NoScale.ToFTransform());
		FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
		EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
		Add(ControlName, EulerTransform);
		break;
	}
	case ERigControlType::EulerTransform:
	{	
		const FMovieSceneControlRigSpaceBaseKey SpaceValue = GetSpaceValue();
		Add(ControlName, SpaceValue);
			
		FEulerTransform EulerTransform =
			Rig
			->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
		FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
		EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
		Add(ControlName, EulerTransform);
		break;
	}

	} // end switch()
}

void FControlRigParameterValues::CopyFrom(const FControlRigParameterValues& Other, FName ControlName)
{
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(Headers); ++Index)
	{
		const FControlRigParameterValueHeader& OtherHeader = Other.Headers[Index];
		FControlRigParameterValueHeader&       ThisHeader  = this->Headers[Index];

		const int32 EntryIndex = OtherHeader.Find(ControlName);
		if (EntryIndex != INDEX_NONE)
		{
			ThisHeader.Add(ControlName, OtherHeader.GetParameter(EntryIndex));
		}
	}
}

void FControlRigParameterValues::ApplyTo(UControlRig* Rig) const
{
	for (const FControlRigParameterValueHeader& Header : Headers)
	{
		Header.Apply(Rig);
	}
}

void FControlRigParameterValues::PopulateFrom(UControlRig* Rig)
{
	for (FRigControlElement* ControlElement : Rig->AvailableControls())
	{
		AddCurrentValue(Rig, ControlElement);
	}

	for (FControlRigParameterValueHeader& Header : Headers)
	{
		Header.Resort();
	}
}

FControlRigValueView FControlRigParameterValues::Find(FName Name) const
{
	for (const FControlRigParameterValueHeader& Header : Headers)
	{
		const int32 Index = Header.Find(Name);
		if (Index != INDEX_NONE)
		{
			return FControlRigValueView(Header.GetParameter(Index), Header.GetType());
		}
	}
	return FControlRigValueView();
}

FControlRigValueView FControlRigParameterValues::Find(FName Name, EControlRigControlType ControlRigType) const
{
	if (ControlRigType < EControlRigControlType::Num)
	{
		const FControlRigParameterValueHeader& Header = GetHeader(ControlRigType);
		const int32 Index = Header.Find(Name);
		if (Index != INDEX_NONE)
		{
			return FControlRigValueView(Header.GetParameter(Index), Header.GetType());
		}
	}
	return FControlRigValueView();
}
	
FControlRigValueView FControlRigParameterValues::FindParameter(FName Name) const
{
	TArrayView<const FControlRigParameterValueHeader> ParameterHeaders(Headers);
	ParameterHeaders.RightChopInline(1);

	for (const FControlRigParameterValueHeader& Header : ParameterHeaders)
	{
		const int32 Index = Header.Find(Name);
		if (Index != INDEX_NONE)
		{
			return FControlRigValueView(Header.GetParameter(Index), Header.GetType());
		}
	}
	return FControlRigValueView();
}

FControlRigParameterBuffer::FControlRigParameterBuffer(UControlRig* InRig, EControlRigParameterBufferIndexStability IndexStability)
	: WeakControlRig(InRig)
	, Values(IndexStability)
{
}

void FAccumulatedControlRigValues::InitializeParameters(FPreAnimatedControlRigParameterStorage& Storage)
{
	for (FEntry& Entry : ValuesArray)
	{
		if (UControlRig* Rig = Entry.WeakControlRig.Get())
		{
			Entry.Values.InitializeParameters(Rig, Storage);
		}
	}
}

void FAccumulatedControlRigValues::Compact()
{
	for (int32 Index = NextValidIndex; Index < ValuesArray.Num(); ++Index)
	{
		ParameterValuesByTrack.Remove(ValuesArray[Index].Track.Get());
	}
	ValuesArray.RemoveAtSwap(NextValidIndex, ValuesArray.Num()-NextValidIndex, EAllowShrinking::Yes);
}

void FAccumulatedControlRigValues::MarkAsActive(int32 Index)
{
	FEntry& Entry = ValuesArray[Index];
	if (Entry.bIsActive)
	{
		return;
	}

	check(Index >= NextValidIndex);

	Entry.bIsActive = true;

	// If it's already at the head just increment
	if (Index == NextValidIndex)
	{
		++NextValidIndex;
		return;
	}

	// Move it to the head
	ParameterValuesByTrack.Add(ValuesArray[NextValidIndex].Track, Index);
	ParameterValuesByTrack.Add(ValuesArray[Index].Track, NextValidIndex);
	Swap(ValuesArray[NextValidIndex], ValuesArray[Index]);
	++NextValidIndex;
}

int32 FAccumulatedControlRigValues::InitializeRig(UMovieSceneControlRigParameterTrack* Track, UControlRig* Rig)
{
	const int32* Existing = ParameterValuesByTrack.Find(Track);
	if (Existing)
	{
		MarkAsActive(*Existing);
		ValuesArray[*Existing].WeakControlRig = Rig;
		return *Existing;
	}

	const int32 Index = ValuesArray.Num();
	ValuesArray.Emplace(Track, Rig);

	ParameterValuesByTrack.Add(Track, Index);
	MarkAsActive(Index);
	return Index;
}

FAccumulatedControlRigValues::FEntry::FEntry(UMovieSceneControlRigParameterTrack* InTrack, UControlRig* Rig)
	: FControlRigParameterBuffer(Rig, EControlRigParameterBufferIndexStability::Stable)
	, Track(InTrack)

{}

UControlRig* FAccumulatedControlRigValues::FindControlRig(FAccumulatedControlEntryIndex Entry) const
{
	return Entry.IsValid()
		? ValuesArray[Entry.EntryIndex].WeakControlRig.Get()
		: nullptr;
}

UControlRig* FAccumulatedControlRigValues::FindControlRig(UMovieSceneControlRigParameterTrack* Track) const
{
	const int32* EntryIndex = ParameterValuesByTrack.Find(Track);

	return EntryIndex
		? ValuesArray[*EntryIndex].WeakControlRig.Get()
		: nullptr;
}

const FControlRigParameterBuffer* FAccumulatedControlRigValues::FindParameterBuffer(UMovieSceneControlRigParameterTrack* Track) const
{
	const int32* EntryIndex = ParameterValuesByTrack.Find(Track);

	return EntryIndex
		? &ValuesArray[*EntryIndex]
		: nullptr;
}

bool FAccumulatedControlRigValues::DoesEntryExistForTrack(UMovieSceneControlRigParameterTrack* Track)
{
	return ParameterValuesByTrack.Find(Track) != nullptr;
}

FAccumulatedControlEntryIndex FAccumulatedControlRigValues::AllocateEntryIndex(UMovieSceneControlRigParameterTrack* Track, FName Name, EControlRigControlType ControlType)
{
	const int32* Existing = ParameterValuesByTrack.Find(Track);
	if (!Existing)
	{
		return FAccumulatedControlEntryIndex();
	}

	FEntry& Entry = ValuesArray[*Existing];

	const int32 AccumulatorIndex = Entry.Values.GetHeader(ControlType).Add_GetIndex(Name);
	return FAccumulatedControlEntryIndex{ uint16(*Existing), uint16(AccumulatorIndex), ControlType };
}

void FAccumulatedControlRigValues::PrimeForInstantiation()
{
	NextValidIndex = 0;
	for (FAccumulatedControlRigValues::FEntry& Entry : ValuesArray)
	{
		Entry.bIsActive = false;
		Entry.Values.Reset();
	}
}

void FAccumulatedControlRigValues::Apply() const
{
	for (const FEntry& Accumulator : ValuesArray)
	{
		Accumulator.Apply();
	}
}

void* FAccumulatedControlRigValues::GetData(FAccumulatedControlEntryIndex Entry)
{
	return ValuesArray[Entry.EntryIndex].Values.GetHeader(Entry.ControlType).GetParameterBuffer();
}

void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, FMovieSceneControlRigSpaceBaseKey InValue)
{
	FMovieSceneControlRigSpaceBaseKey* Ptr = static_cast<FMovieSceneControlRigSpaceBaseKey*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}
void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, bool InValue)
{
	bool* Ptr = static_cast<bool*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}
void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, uint8 InValue)
{
	uint8* Ptr = static_cast<uint8*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}
void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, int32 InValue)
{
	int32* Ptr = static_cast<int32*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}
void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, float InValue)
{
	float* Ptr = static_cast<float*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}
void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, FVector3f InValue)
{
	FVector3f* Ptr = static_cast<FVector3f*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}
void FAccumulatedControlRigValues::Store(FAccumulatedControlEntryIndex Entry, FEulerTransform InValue)
{
	FEulerTransform* Ptr = static_cast<FEulerTransform*>(GetData(Entry));
	Ptr[Entry.AccumulatorIndex] = InValue;
}

void FControlRigParameterValueHeader::ApplyAndRemove(UControlRig* Rig, FName InName)
{
	const int32 Index = Find(InName);
	if (Index != INDEX_NONE)
	{
		Apply(Rig, InName, Index, GetParameterBuffer());
		RemoveAt(Index);
	}
}

void FControlRigParameterValueHeader::Apply(UControlRig* Rig, FName Name, int32 Index, const void* ParameterBuffer) const
{
	constexpr bool bSetupUndo = false;
	constexpr bool bNotify    = true;
	FRigControlModifiedContext ModifiedContext(EControlRigSetKey::Never);

	switch (Type)
	{
		case EControlRigControlType::Space:
		{
			const FMovieSceneControlRigSpaceBaseKey* Ptr = static_cast<const FMovieSceneControlRigSpaceBaseKey*>(ParameterBuffer);
			URigHierarchy* RigHierarchy = Rig->GetHierarchy();
			FRigControlElement* RigControl = Rig->FindControl(Name);
			if (!RigHierarchy || !RigControl)
			{
				return;
			}

			const FRigElementKey ControlKey = RigControl->GetKey();

			FMovieSceneControlRigSpaceBaseKey Value = Ptr[Index];
			switch (Value.SpaceType)
			{
			case EMovieSceneControlRigSpaceType::Parent:
				Rig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
				break;
			case EMovieSceneControlRigSpaceType::World:
				Rig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
				break;
			case EMovieSceneControlRigSpaceType::ControlRig:
				Rig->SwitchToParent(ControlKey, Value.ControlRigElement, false, true);
				break;
			}
			break;
		}
		case EControlRigControlType::Parameter_Bool:
		{
			const bool* Ptr = static_cast<const bool*>(ParameterBuffer);
			Rig->SetControlValue<bool>(Name, Ptr[Index], bNotify, ModifiedContext, bSetupUndo);
			break;
		}
		case EControlRigControlType::Parameter_Enum:
		{
			const uint8* Ptr = static_cast<const uint8*>(ParameterBuffer);
			Rig->SetControlValue<int32>(Name, Ptr[Index], bNotify, ModifiedContext, bSetupUndo);
			break;
		}
		case EControlRigControlType::Parameter_Integer:
		{
			const int32* Ptr = static_cast<const int32*>(ParameterBuffer);
			Rig->SetControlValue<int32>(Name, Ptr[Index], bNotify, ModifiedContext, bSetupUndo);
			break;
		}
		case EControlRigControlType::Parameter_Scalar:
		{
			const float* Ptr = static_cast<const float*>(ParameterBuffer);
			Rig->SetControlValue<float>(Name, Ptr[Index], bNotify, ModifiedContext, bSetupUndo);
			break;
		}
		case EControlRigControlType::Parameter_Vector:
		{
			const FVector3f* Ptr = static_cast<const FVector3f*>(ParameterBuffer);

			FRigControlElement* ControlElement = Rig->FindControl(Name);
			URigHierarchy* RigHierarchy = Rig->GetHierarchy();
			if (RigHierarchy && ControlElement && ControlElement->Settings.ControlType == ERigControlType::Rotator)
            {
				const FVector EulerValue(Ptr[Index].X, Ptr[Index].Y, Ptr[Index].Z);
				const FRotator Rotator(RigHierarchy->GetControlQuaternion(ControlElement, EulerValue));
				RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerValue);
				Rig->SetControlValue<FRotator>(Name, Rotator, bNotify, ModifiedContext, bSetupUndo);
			}
			else if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::Vector2D)
			{
				// Vector2D controls are stored as FVector3f (X,Y,0) but the rig expects FVector2D
				Rig->SetControlValue<FVector2D>(Name, FVector2D(Ptr[Index].X, Ptr[Index].Y), bNotify, ModifiedContext, bSetupUndo);
			}
			else
			{
				Rig->SetControlValue<FVector3f>(Name, Ptr[Index], bNotify, ModifiedContext, bSetupUndo);
			}
			break;
		}
		case EControlRigControlType::Parameter_Transform:
		{
			URigHierarchy* RigHierarchy = Rig->GetHierarchy();
			if (!RigHierarchy)
			{
				return;
			}

			const FEulerTransform* Ptr         = static_cast<const FEulerTransform*>(ParameterBuffer);
			FEulerTransform        Transform   = Ptr[Index];

			FRigControlElement* ControlElement = Rig->FindControl(Name);
			if (ControlElement)
			{
				switch (ControlElement->Settings.ControlType)
				{
				case ERigControlType::Transform:
				{
					FVector EulerAngle(Transform.Rotation.Roll, Transform.Rotation.Pitch, Transform.Rotation.Yaw);
					RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
					Rig->SetControlValue<FRigControlValue::FTransform_Float>(Name, Transform.ToFTransform(), bNotify, ModifiedContext, bSetupUndo);
					break;
				}
				case ERigControlType::TransformNoScale:
				{
					FTransformNoScale NoScale = Transform.ToFTransform();
					FVector EulerAngle(Transform.Rotation.Roll, Transform.Rotation.Pitch, Transform.Rotation.Yaw);
					RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
					Rig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(Name, NoScale, bNotify, ModifiedContext, bSetupUndo);
					break;
				}
				case ERigControlType::EulerTransform:
				{
					FVector EulerAngle(Transform.Rotation.Roll, Transform.Rotation.Pitch, Transform.Rotation.Yaw);
					FQuat Quat = RigHierarchy->GetControlQuaternion(ControlElement, EulerAngle);
					RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
					FRotator UERotator(Quat);
					Transform.Rotation = UERotator;
					Rig->SetControlValue<FRigControlValue::FEulerTransform_Float>(Name, Transform, bNotify, ModifiedContext, bSetupUndo);
					RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
					break;
				}
				default:
					break;
				}
			}
			break;
		}
	}
}

void FControlRigParameterValueHeader::Apply(UControlRig* Rig) const
{
	TArrayView<const FName> Names = GetNames();
	const uint8* ParameterBuffer = GetParameterBuffer();

	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		Apply(Rig, Names[Index], Index, ParameterBuffer);
	}
}

void FControlRigParameterBuffer::Populate()
{
	if (UControlRig* Rig = WeakControlRig.Get())
	{
		Values.PopulateFrom(Rig);
	}
}

void FControlRigParameterBuffer::Apply() const
{
	if (UControlRig* Rig = WeakControlRig.Get())
	{
#if STATS || ENABLE_STATNAMEDEVENTS
	#if STATS
		TStatId StatID = FDynamicStats::CreateStatId<STAT_GROUP_TO_FStatGroup(STATGROUP_MovieSceneECS)>(Rig->GetName());
	#else
		// Just use the base UObject stat ID if we only have named events
		TStatId StatID = Rig->GetStatID(true /* bForDeferredUse */);
	#endif

		FScopeCycleCounter Scope(StatID);
#endif

		Values.ApplyTo(Rig);
	}
}

} // namespace UE::MovieScene
