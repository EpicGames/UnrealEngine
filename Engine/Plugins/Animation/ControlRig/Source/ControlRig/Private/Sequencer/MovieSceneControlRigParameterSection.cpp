// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigParameterSection.h"

#include "ConstraintsManager.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "Logging/MessageLog.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "MovieScene.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Rigs/FKControlRig.h"
#include "Animation/AnimSequence.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/ScopedSlowTask.h"
#include "MovieSceneTimeHelpers.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformConstraintUtil.h"
#include "AnimationCoreLibrary.h"
#include "UObject/ObjectSaveContext.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sequencer/MovieSceneControlRigSystem.h"
#include "ControlRigObjectBinding.h"
#include "UObject/UObjectIterator.h"
#include "ControlRigOverride.h"

#if WITH_EDITOR
#include "Misc/MessageDialog.h"
#include "Settings/ControlRigSettings.h"
#include "Misc/TransactionObjectEvent.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneControlRigParameterSection)

namespace UE::MovieScene
{

	/**
	 * Finds an entry within the specified array based on a pointer that might exist within the
	 *     memory address of the entry. Used for finding entries that relate to channels.
	 * 
	 * This function is O(1) in all cases and works using pointer arithmetic.
	 */
	template<typename T>
	const T* FindEntryWithinArrayByPtr(const TArray<T>& Array, const void* Ptr)
	{
		UPTRINT ByteOffset = ((uint8*)Ptr) - ((uint8*)Array.GetData());

		if (ByteOffset >= 0 && ByteOffset < Array.Num()*sizeof(T))
		{
			const int32 Index = ByteOffset / sizeof(T);
			return &Array[Index];
		}
		return nullptr;
	}

	FControlRigChannelMetaData::FControlRigChannelMetaData()
		: Type(EControlRigControlType::Num)
		, IndexWithinControl(INDEX_NONE)
		, EntitySystemID(MAX_uint32)
	{}

	FControlRigChannelMetaData::FControlRigChannelMetaData(EControlRigControlType InType, FName InControlName, int32 InIndexWithinControl, uint32 InEntitySystemID)
		: Type(InType)
		, ControlName(InControlName)
		, IndexWithinControl(InIndexWithinControl)
		, EntitySystemID(InEntitySystemID)
	{
	}

	FControlRigChannelMetaData::operator bool() const
	{
		return Type != EControlRigControlType::Num && IndexWithinControl != INDEX_NONE;
	}

	void InterrogateTrack(UMovieSceneTrack* Track, const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride)
	{
		if (Track)
		{
			UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
			if (ControlRigTrack && ControlRigTrack->GetLegacyEvaluationTrack())
			{
				return ControlRigTrack->GetLegacyEvaluationTrack()->Interrogate(Context, Container, BindingOverride);
			}

			FMovieSceneEvaluationTrack EvalTrack = CastChecked<IMovieSceneTrackTemplateProducer>(Track)->GenerateTrackTemplate(Track);
			return EvalTrack.Interrogate(Context, Container, BindingOverride);
		}
	}
	
} // namespace UE::MovieScene

#if WITH_EDITOR
#include "AnimPose.h"
#endif

#define LOCTEXT_NAMESPACE "MovieSceneControlParameterRigSection"

#if WITH_EDITOR

struct FParameterFloatChannelEditorData
{
	FParameterFloatChannelEditorData(UControlRig *InControlRig, const FName& InName, bool bEnabledOverride, const FText& GroupName, int SortStartIndex)
	{
		ControlRig = InControlRig;
		ParameterName = InName;
		FString NameAsString = InName.ToString();
		{
			MetaData.SetIdentifiers(InName, GroupName, GroupName);
			MetaData.bEnabled = bEnabledOverride;
			MetaData.SortOrder = SortStartIndex++;
			MetaData.bCanCollapseToTrack = true;
		}

		ExternalValues.OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return GetValue(InControlRig, InName,InObject, Bindings); };
		
		ExternalValues.OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight) { GetChannelValueAndWeight(InName, Object, SectionToKey, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight); };
		
	}

	static TOptional<float> GetValue(UControlRig* ControlRig, FName ParameterName, UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement)
			{
				return ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
			}
		}
		return TOptional<float>();
	}
	
	static void GetChannelValueAndWeight(FName ParameterName, UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
		float& OutValue, float& OutWeight)
	{
		OutValue = 0.0f;
		OutWeight = 1.0f;

		UMovieSceneTrack* Track = SectionToKey->GetTypedOuter<UMovieSceneTrack>();

		if (Track)
		{
			FMovieSceneInterrogationData InterrogationData;
			{
				RootTemplate.CopyActuators(InterrogationData.GetAccumulator());
				FMovieSceneContext Context(FMovieSceneEvaluationRange(KeyTime, TickResolution));
				UE::MovieScene::InterrogateTrack(Track, Context, InterrogationData, Object);
			}

			float Val = 0.0f;
			for (const FFloatInterrogationData& InVector : InterrogationData.Iterate<FFloatInterrogationData>(UMovieSceneControlRigParameterSection::GetFloatInterrogationKey()))
			{
				if (InVector.ParameterName == ParameterName)
				{
					Val = InVector.Val;
					break;
				}
			}
			OutValue = Val;
		}
		OutWeight = MovieSceneHelpers::CalculateWeightForBlending(SectionToKey, KeyTime);
	}

	FText							GroupName;
	FMovieSceneChannelMetaData      MetaData;
	TMovieSceneExternalValue<float> ExternalValues;
	FName ParameterName;
	UControlRig *ControlRig;
};

//Set up with all 4 Channels so it can be used by all vector types.
struct FParameterVectorChannelEditorData
{
	FParameterVectorChannelEditorData(UControlRig *InControlRig, const FName& InName, bool bEnabledOverride, const FText& GroupName, int SortStartIndex, int32 NumChannels, ERigControlType ControlType)
	{
		ControlRig = InControlRig;
		ParameterName = InName;

		static constexpr EAxisList::Type Axes[] = { EAxisList::Forward, EAxisList::Left, EAxisList::Up, EAxisList::None };
		static_assert(UE_ARRAY_COUNT(Axes) == UE_ARRAY_COUNT(MetaData));
		
		static constexpr FStringView NameSuffixes[] = { TEXT(".X"), TEXT(".Y"), TEXT(".Z"), TEXT(".W") };
		static_assert(UE_ARRAY_COUNT(NameSuffixes) == UE_ARRAY_COUNT(MetaData));
		
		static constexpr FStringView PositionNameSuffixes[] = { TEXT(".Location.X"), TEXT(".Location.Y"), TEXT(".Location.Z"), nullptr };
		static_assert(UE_ARRAY_COUNT(PositionNameSuffixes) == UE_ARRAY_COUNT(MetaData));
		
		static constexpr FStringView RotationNameSuffixes[] = { TEXT(".Rotation.Roll"), TEXT(".Rotation.Pitch"), TEXT(".Rotation.Yaw"), nullptr };
		static_assert(UE_ARRAY_COUNT(RotationNameSuffixes) == UE_ARRAY_COUNT(MetaData));
		
		static constexpr FStringView ScaleNameSuffixes[] = { TEXT(".Scale.X"), TEXT(".Scale.Y"), TEXT(".Scale.Z"), nullptr };
		static_assert(UE_ARRAY_COUNT(ScaleNameSuffixes) == UE_ARRAY_COUNT(MetaData));
		
		static const FText DisplayTexts[] = { FCommonChannelData::ChannelX, FCommonChannelData::ChannelY, FCommonChannelData::ChannelZ, FCommonChannelData::ChannelW };
		static_assert(UE_ARRAY_COUNT(DisplayTexts) == UE_ARRAY_COUNT(MetaData));
		
		static const FText PositionDisplayTexts[] =
		{
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Location.X", "Location.X"), 
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Location.Y", "Location.Y"), 
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Location.Z", "Location.Z"),
			FText()
		};
		static_assert(UE_ARRAY_COUNT(PositionDisplayTexts) == UE_ARRAY_COUNT(MetaData));
		
		static const FText RotationDisplayTexts[] =
		{
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.X", "Rotation.Roll"),
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.Y", "Rotation.Pitch"),
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.Z", "Rotation.Yaw"),
			FText()
		};
		static_assert(UE_ARRAY_COUNT(RotationDisplayTexts) == UE_ARRAY_COUNT(MetaData));
		
		static const FText ScaleDisplayTexts[] =
		{
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale.X", "Scale.X"),
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale.Y", "Scale.Y"),
			NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale.Z", "Scale.Z"),
			FText()
		};
		static_assert(UE_ARRAY_COUNT(ScaleDisplayTexts) == UE_ARRAY_COUNT(MetaData));
		
		const FString NameAsString = InName.ToString();
		
		check(NumChannels <= UE_ARRAY_COUNT(MetaData));
		check(NumChannels <= UE_ARRAY_COUNT(ExternalValues));
		for (int ChannelIdx = 0; ChannelIdx < NumChannels; ChannelIdx++)
		{
			const EAxisList::Type* Axis = &Axes[ChannelIdx];
			const FStringView* NameSuffix;
			const FText* DisplayText;
			switch (ControlType)
			{
			case ERigControlType::Position:
				NameSuffix = &PositionNameSuffixes[ChannelIdx];
				DisplayText = &PositionDisplayTexts[ChannelIdx];
				break;
			case ERigControlType::Rotator:
				NameSuffix = &RotationNameSuffixes[ChannelIdx];
				DisplayText = &RotationDisplayTexts[ChannelIdx];
				break;
			case ERigControlType::Scale:
				NameSuffix = &ScaleNameSuffixes[ChannelIdx];
				DisplayText = &ScaleDisplayTexts[ChannelIdx];
				break;
			default:
				NameSuffix = &NameSuffixes[ChannelIdx];
				DisplayText = &DisplayTexts[ChannelIdx];
				Axis = nullptr;
				break;
			}

			MetaData[ChannelIdx].SetIdentifiers(FName(NameAsString + *NameSuffix), *DisplayText);
			MetaData[ChannelIdx].IntentName = *DisplayText;
			if (Axis != nullptr && *Axis != EAxisList::None)
			{
				MetaData[ChannelIdx].Color = AxisDisplayInfo::GetAxisColor(*Axis);
			}
			MetaData[ChannelIdx].Group = GroupName;
			MetaData[ChannelIdx].bEnabled = bEnabledOverride;
			MetaData[ChannelIdx].SortOrder = SortStartIndex++;
			MetaData[ChannelIdx].bCanCollapseToTrack = true;
		}

		ExternalValues[0].OnGetExternalValue = [InControlRig, InName, NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelX(InObject, InControlRig, InName, NumChannels); };
		ExternalValues[1].OnGetExternalValue = [InControlRig, InName, NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelY(InObject, InControlRig, InName, NumChannels); };
		ExternalValues[2].OnGetExternalValue = [InControlRig, InName, NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelZ(InObject, InControlRig, InName, NumChannels); };
		ExternalValues[3].OnGetExternalValue = [InControlRig, InName, NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelW(InObject, InControlRig, InName, NumChannels); };

		ExternalValues[0].OnGetCurrentValueAndWeight = [InName, NumChannels](UObject* Object, UMovieSceneSection* SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight) { GetChannelValueAndWeight(InName, NumChannels, 0, Object, SectionToKey, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight); };
		ExternalValues[1].OnGetCurrentValueAndWeight = [InName, NumChannels](UObject* Object, UMovieSceneSection* SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight) { GetChannelValueAndWeight(InName, NumChannels, 1, Object, SectionToKey, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight); };
		ExternalValues[2].OnGetCurrentValueAndWeight = [InName, NumChannels](UObject* Object, UMovieSceneSection* SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight) { GetChannelValueAndWeight(InName, NumChannels, 2, Object, SectionToKey, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight); };
		ExternalValues[3].OnGetCurrentValueAndWeight = [InName, NumChannels](UObject* Object, UMovieSceneSection* SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight) { GetChannelValueAndWeight(InName, NumChannels, 3, Object, SectionToKey, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight); };

	}

	static FVector4 GetPropertyValue(UControlRig* ControlRig, FName ParameterName, UObject& InObject,int32 NumChannels)
	{
		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement)
			{
		
				if (NumChannels == 2)
				{
					const FVector3f Vector = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
					return FVector4(Vector.X, Vector.Y, 0.f, 0.f);
				}
				else if (NumChannels == 3)
				{
					const FVector3f Vector = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
					return FVector4(Vector.X, Vector.Y, Vector.Z, 0.f);
				}
				else
				{
					const FRigControlValue::FTransform_Float Storage = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>();
#if ENABLE_VECTORIZED_TRANSFORM
					return FVector4(Storage.TranslationX, Storage.TranslationY, Storage.TranslationZ, Storage.TranslationW);
#else
					return FVector4(Storage.TranslationX, Storage.TranslationY, Storage.TranslationZ, 0.f);
#endif
				}
			}
		}
		return FVector4();
	}

	static TOptional<float> ExtractChannelX(UObject& InObject, UControlRig* ControlRig, FName ParameterName, int32 NumChannels)
	{
		return GetPropertyValue(ControlRig, ParameterName, InObject, NumChannels).X;
	}
	static TOptional<float> ExtractChannelY(UObject& InObject, UControlRig* ControlRig, FName ParameterName, int32 NumChannels)
	{
		return GetPropertyValue(ControlRig, ParameterName, InObject, NumChannels).Y;
	}
	static TOptional<float> ExtractChannelZ(UObject& InObject, UControlRig* ControlRig, FName ParameterName, int32 NumChannels)
	{
		return GetPropertyValue(ControlRig, ParameterName, InObject, NumChannels).Z;
	}
	static TOptional<float> ExtractChannelW(UObject& InObject, UControlRig* ControlRig, FName ParameterName, int32 NumChannels)
	{
		return GetPropertyValue(ControlRig, ParameterName, InObject, NumChannels).W;
	}

	static void GetChannelValueAndWeight(FName ParameterName, int32 NumChannels, int32 Index, UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
		float& OutValue, float& OutWeight)
	{
		OutValue = 0.0f;
		OutWeight = 1.0f;
		if (Index >= NumChannels)
		{
			return;
		}

		UMovieSceneTrack* Track = SectionToKey->GetTypedOuter<UMovieSceneTrack>();

		if (Track)
		{
			FMovieSceneInterrogationData InterrogationData;
			{
				RootTemplate.CopyActuators(InterrogationData.GetAccumulator());
				FMovieSceneContext Context(FMovieSceneEvaluationRange(KeyTime, TickResolution));
				UE::MovieScene::InterrogateTrack(Track, Context, InterrogationData, Object);
			}

			switch (NumChannels)
			{
			case 2:
			{
				FVector2D Val(0.0f, 0.0f);
				for (const FVector2DInterrogationData& InVector : InterrogationData.Iterate<FVector2DInterrogationData>(UMovieSceneControlRigParameterSection::GetVector2DInterrogationKey()))
				{
					if (InVector.ParameterName == ParameterName)
					{
						Val = InVector.Val;
						break;
					}
				}
				switch (Index)
				{
				case 0:
					OutValue = Val.X;
					break;
				case 1:
					OutValue = Val.Y;
					break;
				}
			}
			break;
			case 3:
			{
				FVector Val(0.0f, 0.0f, 0.0f);
				for (const FVectorInterrogationData& InVector : InterrogationData.Iterate<FVectorInterrogationData>(UMovieSceneControlRigParameterSection::GetVectorInterrogationKey()))
				{
					if (InVector.ParameterName == ParameterName)
					{
						Val = InVector.Val;
						break;
					}
				}
				switch (Index)
				{
				case 0:
					OutValue = Val.X;
					break;
				case 1:
					OutValue = Val.Y;
					break;
				case 2:
					OutValue = Val.Z;
					break;
				}
			}
			break;
			}
		}
		OutWeight = MovieSceneHelpers::CalculateWeightForBlending(SectionToKey, KeyTime);
	}
	FText							GroupName;
	FMovieSceneChannelMetaData      MetaData[4];
	TMovieSceneExternalValue<float> ExternalValues[4];
	FName ParameterName;
	UControlRig *ControlRig;
};

namespace
{
FIntVector4 ReverseSwizzleFunc(const FIntVector4& InSwizzle)
{
	FIntVector4 ReverseSwizzle;
	for (int32 i = 0; i < 4; ++i)
	{
		int32 Index = 0;
		for (int32 j = 0; j < 4; ++j)
		{
			if (InSwizzle[j] == i)
			{
				Index = j;
				break;
			}
		}
		ReverseSwizzle[i] = Index;
	}

	return ReverseSwizzle;
};
} // empty namespace

struct FParameterTransformChannelEditorData
{
	FParameterTransformChannelEditorData(UControlRig *InControlRig, const FName& InName, bool bEnabledOverride, EMovieSceneTransformChannel Mask, 
		const FText& GroupName, int SortStartIndex)
	{
		ControlRig = InControlRig;
		ParameterName = InName;
		static FText LongIntentFormatStr = NSLOCTEXT("MovieSceneControlParameterRigSection", "LongIntentFormatString", "{GroupName}.{IntentName}");

		static const TSet<FName> PropertyMetaDataKeys = { "UIMin", "UIMax", "SliderExponent", "LinearDeltaSensitivity", "Delta", "ClampMin", "ClampMax", "ForceUnits", "WheelStep" };

		const FProperty* RelativeLocationProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName());
		const FProperty* RelativeRotationProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeRotationPropertyName());
		const FProperty* RelativeScale3DProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName());

		//FText LocationGroup = NSLOCTEXT("MovieSceneControlParameterRigSection", "Location", "Location");
		//FText RotationGroup = NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation", "Rotation");
		//FText ScaleGroup = NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale", "Scale");

		FString NameAsString = InName.ToString();
		FString TotalName = NameAsString;
		FText TransformGroup = FText::Format(NSLOCTEXT("MovieSceneControlParameterRigSection", "MovieSceneControlParameterRigSectionGroupName", "{0}"), GroupName);

		FText LocationDisplayName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Location", "Location");
		FText ScaleDisplayName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale", "Scale");

		const EAxisList::Type XAxis = EAxisList::Forward;
		const EAxisList::Type YAxis = EAxisList::Left;
		const EAxisList::Type ZAxis = EAxisList::Up;

		const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
		const FIntVector4 ReverseSwizzle = ReverseSwizzleFunc(Swizzle);
		const int32 TranslationOrderOffset = 0;
		const int32 RotationOrderOffset = TranslationOrderOffset + 3;
		const int32 ScaleOrderOffset = RotationOrderOffset + 3;

		{
			//MetaData[0].SetIdentifiers("Location.X", FCommonChannelData::ChannelX, LocationGroup);
			TotalName += ".Location.X";
			MetaData[0].SetIdentifiers(FName(*TotalName), FText::Join(FText::FromString(TEXT(".")), LocationDisplayName, AxisDisplayInfo::GetAxisDisplayName(XAxis)), TransformGroup);
			MetaData[0].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Location.X", "Location.X");
			MetaData[0].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[0].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationX);
			MetaData[0].Color = AxisDisplayInfo::GetAxisColor(XAxis);
			MetaData[0].SortOrder = SortStartIndex + TranslationOrderOffset + ReverseSwizzle[0];
			MetaData[0].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[0].PropertyMetaData.Add(PropertyMetaDataKey, RelativeLocationProperty->GetMetaData(PropertyMetaDataKey));
			}

			//MetaData[1].SetIdentifiers("Location.Y", FCommonChannelData::ChannelY, LocationGroup);
			TotalName += ".Location.Y";
			MetaData[1].SetIdentifiers(FName(*TotalName), FText::Join(FText::FromString(TEXT(".")), LocationDisplayName, AxisDisplayInfo::GetAxisDisplayName(YAxis)), TransformGroup);
			MetaData[1].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Location.Y", "Location.Y");
			MetaData[1].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[1].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationY);
			MetaData[1].Color = AxisDisplayInfo::GetAxisColor(YAxis);
			MetaData[1].SortOrder = SortStartIndex + TranslationOrderOffset + ReverseSwizzle[1];
			MetaData[1].bCanCollapseToTrack = true;
			MetaData[1].bInvertValue = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[1].PropertyMetaData.Add(PropertyMetaDataKey, RelativeLocationProperty->GetMetaData(PropertyMetaDataKey));
			}

			//MetaData[2].SetIdentifiers("Location.Z", FCommonChannelData::ChannelZ, LocationGroup);
			TotalName += ".Location.Z";
			MetaData[2].SetIdentifiers(FName(*TotalName), FText::Join(FText::FromString(TEXT(".")), LocationDisplayName, AxisDisplayInfo::GetAxisDisplayName(ZAxis)), TransformGroup);
			MetaData[2].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Location.Z", "Location.Z");
			MetaData[2].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[2].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationZ);
			MetaData[2].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
			MetaData[2].SortOrder = SortStartIndex + TranslationOrderOffset + ReverseSwizzle[2];
			MetaData[2].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[2].PropertyMetaData.Add(PropertyMetaDataKey, RelativeLocationProperty->GetMetaData(PropertyMetaDataKey));
			}
		}
		{
			//MetaData[3].SetIdentifiers("Rotation.X", NSLOCTEXT("MovieSceneTransformSection", "RotationX", "Roll"), RotationGroup);
			TotalName += ".Rotation.X";
			MetaData[3].SetIdentifiers(FName(*TotalName), NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.X", "Rotation.Roll"), TransformGroup);
			MetaData[3].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.X", "Rotation.Roll");
			MetaData[3].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[3].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationX);
			MetaData[3].Color = AxisDisplayInfo::GetAxisColor(XAxis);
			MetaData[3].SortOrder = SortStartIndex + RotationOrderOffset + 0;
			MetaData[3].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[3].PropertyMetaData.Add(PropertyMetaDataKey, RelativeRotationProperty->GetMetaData(PropertyMetaDataKey));
			}

			//MetaData[4].SetIdentifiers("Rotation.Y", NSLOCTEXT("MovieSceneTransformSection", "RotationY", "Pitch"), RotationGroup);
			TotalName += ".Rotation.Y";
			MetaData[4].SetIdentifiers(FName(*TotalName), NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.Y", "Rotation.Pitch"), TransformGroup);
			MetaData[4].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.Y", "Rotation.Pitch");
			MetaData[4].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[4].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationY);
			MetaData[4].Color = AxisDisplayInfo::GetAxisColor(YAxis);
			MetaData[4].SortOrder = SortStartIndex + RotationOrderOffset + 1;
			MetaData[4].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[4].PropertyMetaData.Add(PropertyMetaDataKey, RelativeRotationProperty->GetMetaData(PropertyMetaDataKey));
			}

			//MetaData[5].SetIdentifiers("Rotation.Z", NSLOCTEXT("MovieSceneTransformSection", "RotationZ", "Yaw"), RotationGroup);
			TotalName += ".Rotation.Z";
			MetaData[5].SetIdentifiers(FName(*TotalName), NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.Z", "Rotation.Yaw"), TransformGroup);
			MetaData[5].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Rotation.Z", "Rotation.Yaw");
			MetaData[5].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[5].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationZ);
			MetaData[5].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
			MetaData[5].SortOrder = SortStartIndex + RotationOrderOffset + 2;
			MetaData[5].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[5].PropertyMetaData.Add(PropertyMetaDataKey, RelativeRotationProperty->GetMetaData(PropertyMetaDataKey));
			}
		}
		{
			//MetaData[6].SetIdentifiers("Scale.X", FCommonChannelData::ChannelX, ScaleGroup);
			TotalName += ".Scale.X";
			MetaData[6].SetIdentifiers(FName(*TotalName), FText::Join(FText::FromString(TEXT(".")), ScaleDisplayName, AxisDisplayInfo::GetAxisDisplayName(XAxis)), TransformGroup);
			MetaData[6].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale.X", "Scale.X");
			MetaData[6].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[6].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::ScaleX);
			MetaData[6].Color = AxisDisplayInfo::GetAxisColor(XAxis);
			MetaData[6].SortOrder = SortStartIndex + ScaleOrderOffset + ReverseSwizzle[0];
			MetaData[6].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[6].PropertyMetaData.Add(PropertyMetaDataKey, RelativeScale3DProperty->GetMetaData(PropertyMetaDataKey));
			}

			//MetaData[7].SetIdentifiers("Scale.Y", FCommonChannelData::ChannelY, ScaleGroup);
			TotalName += ".Scale.Y";
			MetaData[7].SetIdentifiers(FName(*TotalName), FText::Join(FText::FromString(TEXT(".")), ScaleDisplayName, AxisDisplayInfo::GetAxisDisplayName(YAxis)), TransformGroup);
			MetaData[7].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale.Y", "Scale.Y");
			MetaData[7].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[7].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::ScaleY);
			MetaData[7].Color = AxisDisplayInfo::GetAxisColor(YAxis);
			MetaData[7].SortOrder = SortStartIndex + ScaleOrderOffset + ReverseSwizzle[1];
			MetaData[7].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[7].PropertyMetaData.Add(PropertyMetaDataKey, RelativeScale3DProperty->GetMetaData(PropertyMetaDataKey));
			}

			//MetaData[8].SetIdentifiers("Scale.Z", FCommonChannelData::ChannelZ, ScaleGroup);
			TotalName += ".Scale.Z";
			MetaData[8].SetIdentifiers(FName(*TotalName), FText::Join(FText::FromString(TEXT(".")), ScaleDisplayName, AxisDisplayInfo::GetAxisDisplayName(ZAxis)), TransformGroup);
			MetaData[8].IntentName = NSLOCTEXT("MovieSceneControlParameterRigSection", "Scale.Z", "Scale.Z");
			MetaData[8].LongIntentNameFormat = LongIntentFormatStr;
			TotalName = NameAsString;

			MetaData[8].bEnabled = bEnabledOverride && EnumHasAllFlags(Mask, EMovieSceneTransformChannel::ScaleZ);
			MetaData[8].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
			MetaData[8].SortOrder = SortStartIndex + ScaleOrderOffset + ReverseSwizzle[2];
			MetaData[8].bCanCollapseToTrack = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[8].PropertyMetaData.Add(PropertyMetaDataKey, RelativeScale3DProperty->GetMetaData(PropertyMetaDataKey));
			}
		}
		{
			//MetaData[9].SetIdentifiers("Weight", NSLOCTEXT("MovieSceneTransformSection", "Weight", "Weight"));
			//MetaData[9].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::Weight);
		}

		ExternalValues[0].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FVector> Translation = GetTranslation(InControlRig, InName, InObject, Bindings);
			return Translation.IsSet() ? Translation->X : TOptional<float>();
		};

		ExternalValues[1].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FVector> Translation = GetTranslation(InControlRig, InName, InObject, Bindings);
			return Translation.IsSet() ? Translation->Y : TOptional<float>();
		};
		ExternalValues[2].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FVector> Translation = GetTranslation(InControlRig, InName, InObject, Bindings);
			return Translation.IsSet() ? Translation->Z : TOptional<float>();
		};
		ExternalValues[3].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FRotator> Rotator = GetRotator(InControlRig, InName, InObject, Bindings);
			return Rotator.IsSet() ? Rotator->Roll : TOptional<float>();
		};
		ExternalValues[4].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FRotator> Rotator = GetRotator(InControlRig, InName, InObject, Bindings);
			return Rotator.IsSet() ? Rotator->Pitch : TOptional<float>();
		};
		ExternalValues[5].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FRotator> Rotator = GetRotator(InControlRig, InName, InObject, Bindings);
			return Rotator.IsSet() ? Rotator->Yaw : TOptional<float>();
		};
		ExternalValues[6].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FVector> Scale = GetScale(InControlRig, InName, InObject, Bindings);
			return Scale.IsSet() ? Scale->X : TOptional<float>();
		};
		ExternalValues[7].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FVector> Scale = GetScale(InControlRig, InName, InObject, Bindings);
			return Scale.IsSet() ? Scale->Y : TOptional<float>();
		};
		ExternalValues[8].OnGetExternalValue = [InControlRig, InName](UObject& InObject, FTrackInstancePropertyBindings* Bindings)
		{
			TOptional<FVector> Scale = GetScale(InControlRig, InName, InObject, Bindings);
			return Scale.IsSet() ? Scale->Z : TOptional<float>();
		};

		ExternalValues[0].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 0, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[1].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 1, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[2].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 2, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[3].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 3, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[4].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 4, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[5].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 5, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[6].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 6, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[7].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 7, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};
		ExternalValues[8].OnGetCurrentValueAndWeight = [InName](UObject* Object, UMovieSceneSection*  SectionToKey, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
			float& OutValue, float& OutWeight)
		{
			GetValueAndWeight(InName, Object, SectionToKey, 8, KeyTime, TickResolution, RootTemplate, OutValue, OutWeight);
		};

	}

	static TOptional<FVector> GetTranslation(UControlRig* ControlRig, FName ParameterName, UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement)
			{
				auto GetTranslationFromTransform = [&](const FVector& InTranslation)
				{
					// switch translation to constraint space if needed
					const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, ControlElement->GetFName());
					TOptional<FTransform> ConstraintSpaceTransform = UE::TransformConstraintUtil::GetRelativeTransform(ControlRig->GetWorld(), ControlHash);
					if (ConstraintSpaceTransform)
					{
						return ConstraintSpaceTransform->GetTranslation();
					}
					return InTranslation;
				};
				
				if (ControlElement->Settings.ControlType == ERigControlType::Transform)
				{
					const FRigControlValue::FTransform_Float Transform = 
						ControlRig
						->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>();

					return GetTranslationFromTransform( FVector(Transform.GetTranslation()) );
				}
				else if  (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
				{
					const FRigControlValue::FTransformNoScale_Float Transform = 
						ControlRig
						->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>();

					return GetTranslationFromTransform( FVector(Transform.GetTranslation()) );
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					const FRigControlValue::FEulerTransform_Float Euler = 
						ControlRig
						->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>();

					return GetTranslationFromTransform( FVector(Euler.GetTranslation()) );
				}
				else if(ControlElement->Settings.ControlType == ERigControlType::Position)
				{
					FVector3f Vector = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
					return FVector(Vector.X, Vector.Y, Vector.Z);
				}
			}
		}
		return TOptional<FVector>();
	}

	static TOptional<FRotator> GetRotator(UControlRig* ControlRig, FName ParameterName, UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		if (ControlRig)

		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement)
			{
				if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					// switch rotation to constraint space if needed
					const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, ControlElement->GetFName());
					TOptional<FTransform> ConstraintSpaceTransform = UE::TransformConstraintUtil::GetRelativeTransform(ControlRig->GetWorld(), ControlHash);
					if (ConstraintSpaceTransform)
					{
						return ConstraintSpaceTransform->GetRotation().Rotator();
					}
				}
				
				FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
				FRotator Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
				return Rotation;
			}
		}
		return TOptional<FRotator>();
	}

	static TOptional<FVector> GetScale(UControlRig* ControlRig, FName ParameterName, UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement)
			{
				auto GetScaleFromTransform = [&](const FVector& InScale3D)
				{
					// switch scale to constraint space if needed
					const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, ControlElement->GetFName());
					TOptional<FTransform> ConstraintSpaceTransform = UE::TransformConstraintUtil::GetRelativeTransform(ControlRig->GetWorld(), ControlHash);
					if (ConstraintSpaceTransform)
					{
						return ConstraintSpaceTransform->GetScale3D();
					}
					return InScale3D;
				};
				
				if (ControlElement->Settings.ControlType == ERigControlType::Transform)
				{
					const FRigControlValue::FTransform_Float Transform = 
						ControlRig
						->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>();

					return GetScaleFromTransform( FVector(Transform.GetScale3D()) );
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					const FRigControlValue::FEulerTransform_Float Transform = 
						ControlRig
						->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>();

					return GetScaleFromTransform( FVector(Transform.GetScale3D()) );
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::Scale)
				{
					FVector3f Vector = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
					return FVector(Vector.X, Vector.Y, Vector.Z);
				}
			}
		}
		return TOptional<FVector>();
	}

	static void GetValueAndWeight(FName ParameterName, UObject* Object, UMovieSceneSection*  SectionToKey, int32 Index, FFrameNumber KeyTime, FFrameRate TickResolution, FMovieSceneRootEvaluationTemplateInstance& RootTemplate,
		float& OutValue, float& OutWeight)
	{
		FMovieSceneInterrogationData InterrogationData;
		{
			RootTemplate.CopyActuators(InterrogationData.GetAccumulator());
			FMovieSceneContext Context(FMovieSceneEvaluationRange(KeyTime, TickResolution));
			UE::MovieScene::InterrogateTrack(SectionToKey->GetTypedOuter<UMovieSceneTrack>(), Context, InterrogationData, Object);
		}
		
		FVector CurrentPos = FVector::ZeroVector;
		FRotator CurrentRot = FRotator::ZeroRotator;
		FVector CurrentScale = FVector::ZeroVector;

		for (const FEulerTransformInterrogationData& Transform : InterrogationData.Iterate<FEulerTransformInterrogationData>(UMovieSceneControlRigParameterSection::GetTransformInterrogationKey()))
		{
			if (Transform.ParameterName == ParameterName)
			{
				CurrentPos = Transform.Val.GetLocation();
				CurrentRot = Transform.Val.Rotator();
				CurrentScale = Transform.Val.GetScale3D();
				break;
			}
		}

		switch (Index)
		{
		case 0:
			OutValue = CurrentPos.X;
			break;
		case 1:
			OutValue = CurrentPos.Y;
			break;
		case 2:
			OutValue = CurrentPos.Z;
			break;
		case 3:
			OutValue = CurrentRot.Roll;
			break;
		case 4:
			OutValue = CurrentRot.Pitch;
			break;
		case 5:
			OutValue = CurrentRot.Yaw;
			break;
		case 6:
			OutValue = CurrentScale.X;
			break;
		case 7:
			OutValue = CurrentScale.Y;
			break;
		case 8:
			OutValue = CurrentScale.Z;
			break;

		}
		OutWeight = MovieSceneHelpers::CalculateWeightForBlending(SectionToKey, KeyTime);
	}
		
public:

	FText							GroupName;
	FMovieSceneChannelMetaData      MetaData[9];
	TMovieSceneExternalValue<float> ExternalValues[9];
	FName ParameterName;
	UControlRig *ControlRig;
};

#endif // WITH_EDITOR


UMovieSceneControlRigParameterSection::UMovieSceneControlRigParameterSection() :bDoNotKey(false)
{
	// Section template relies on always restoring state for objects when they are no longer animating. This is how it releases animation control.
	EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	TransformMask = EMovieSceneTransformChannel::AllTransform;

	Weight.SetDefault(1.0f);

#if WITH_EDITOR
	static const FMovieSceneChannelMetaData MetaData("Weight", LOCTEXT("WeightChannelText", "Weight"));
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(Weight, MetaData, TMovieSceneExternalValue<float>());

	// If we are in the process of loading, we can't be certain that we aren't already inside one of these delegate's broadcasts
	if (!IsInAsyncLoadingThread())
	{
		BindEditorDelegates();
	}
#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(Weight);

#endif
}

void UMovieSceneControlRigParameterSection::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	for (FConstraintAndActiveChannel& ConstraintChannel : ConstraintsChannels)
	{
		if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
		{
			if (TransformConstraint->ChildTRSHandle)
			{
				TransformConstraint->ChildTRSHandle->OnBindingIDsUpdated(OldFixedToNewFixedMap, LocalSequenceID, SharedPlaybackState);
			}
			if (TransformConstraint->ParentTRSHandle)
			{
				TransformConstraint->ParentTRSHandle->OnBindingIDsUpdated(OldFixedToNewFixedMap, LocalSequenceID, SharedPlaybackState);
			}
		}
	}
}

void UMovieSceneControlRigParameterSection::GetReferencedBindings(TArray<FGuid>& OutBindings)
{
	for (FConstraintAndActiveChannel& ConstraintChannel : ConstraintsChannels)
	{
		if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(ConstraintChannel.GetConstraint().Get()))
		{
			if (TransformConstraint->ChildTRSHandle && TransformConstraint->ChildTRSHandle->ConstraintBindingID.IsValid())
			{
				OutBindings.Add(TransformConstraint->ChildTRSHandle->ConstraintBindingID.GetGuid());
			}
			if (TransformConstraint->ParentTRSHandle && TransformConstraint->ParentTRSHandle->ConstraintBindingID.IsValid())
			{
				OutBindings.Add(TransformConstraint->ParentTRSHandle->ConstraintBindingID.GetGuid());
			}
		}
	}
}

void UMovieSceneControlRigParameterSection::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
}

FMovieSceneTimeWarpVariant* UMovieSceneControlRigParameterSection::GetTimeWarp()
{
	return &PlayRate;
}

bool UMovieSceneControlRigParameterSection::HasTimeWarp() const
{
	UMovieSceneTimeWarpGetter* Getter = PlayRate.GetType() == EMovieSceneTimeWarpType::Custom
		? PlayRate.AsCustom()
		: nullptr;
	return (Getter != nullptr);

}

//make sure to zero out Scale values if getting to Additive or use the current values if getting set to Override
void UMovieSceneControlRigParameterSection::SetBlendType(EMovieSceneBlendType InBlendType)
{
	if (GetSupportedBlendTypes().Contains(InBlendType))
	{
		Modify();
		BlendType = InBlendType;
		if (ControlRig)
		{
			// Set Defaults based upon Type
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			TArray<FRigControlElement*> Controls = ControlRig->AvailableControls();

			for (FRigControlElement* ControlElement : Controls)
			{
				if (!ControlRig->GetHierarchy()->IsAnimatable(ControlElement))
				{
					continue;
				}
				FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ControlElement->GetFName());
				if (pChannelIndex == nullptr)
				{
					continue;
				}
				int32 ChannelIndex = pChannelIndex->ChannelIndex;

				switch (ControlElement->Settings.ControlType)
				{
				case ERigControlType::Float:
				case ERigControlType::ScaleFloat:
				{
					if (InBlendType == EMovieSceneBlendType::Override)
					{
						float Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
						FloatChannels[ChannelIndex]->SetDefault(Val);
					}
					break;
				}
				case ERigControlType::Vector2D:
				{
					if (InBlendType == EMovieSceneBlendType::Override)
					{
						FVector3f Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
						FloatChannels[ChannelIndex]->SetDefault(Val.X);
						FloatChannels[ChannelIndex + 1]->SetDefault(Val.Y);
					}
					break;
				}
				case ERigControlType::Position:
				case ERigControlType::Rotator:
				{
					if (InBlendType == EMovieSceneBlendType::Override)
					{
						FVector3f Val = (ControlElement->Settings.ControlType == ERigControlType::Rotator)
							? FVector3f(ControlRig->GetHierarchy()->GetControlSpecifiedEulerAngle(ControlElement)) : ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();

						FloatChannels[ChannelIndex]->SetDefault(Val.X);
						FloatChannels[ChannelIndex + 1]->SetDefault(Val.Y);
						FloatChannels[ChannelIndex + 2]->SetDefault(Val.Z);
					}
					break;
				}
				case ERigControlType::Scale:
				{
					if (InBlendType == EMovieSceneBlendType::Absolute)
					{
						FloatChannels[ChannelIndex]->SetDefault(1.0f);
						FloatChannels[ChannelIndex + 1]->SetDefault(1.0f);
						FloatChannels[ChannelIndex + 2]->SetDefault(1.0f);
					}
					else if (InBlendType == EMovieSceneBlendType::Additive)
					{
						FloatChannels[ChannelIndex]->SetDefault(0.0f);
						FloatChannels[ChannelIndex + 1]->SetDefault(0.0f);
						FloatChannels[ChannelIndex + 2]->SetDefault(0.0f);
					}
					else if (InBlendType == EMovieSceneBlendType::Override)
					{
						FVector3f Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
						FloatChannels[ChannelIndex]->SetDefault(Val.X);
						FloatChannels[ChannelIndex + 1]->SetDefault(Val.Y);
						FloatChannels[ChannelIndex + 2]->SetDefault(Val.Z);
					}
				}
				break;
				case ERigControlType::Transform:
				case ERigControlType::EulerTransform:
				case ERigControlType::TransformNoScale:
				{
					FTransform Val = FTransform::Identity;
					if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
					{
						FTransformNoScale NoScale =
							ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
						Val = NoScale;
					}
					else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						FEulerTransform Euler =
							ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
						Val = Euler.ToFTransform();
					}
					else
					{
						Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>().ToTransform();
					}
					if (InBlendType == EMovieSceneBlendType::Override)
					{
						FVector CurrentVector = Val.GetTranslation();
						FloatChannels[ChannelIndex]->SetDefault(CurrentVector.X);
						FloatChannels[ChannelIndex + 1]->SetDefault(CurrentVector.Y);
						FloatChannels[ChannelIndex + 2]->SetDefault(CurrentVector.Z);

						CurrentVector = ControlRig->GetHierarchy()->GetControlSpecifiedEulerAngle(ControlElement);
						FloatChannels[ChannelIndex + 3]->SetDefault(CurrentVector.X);
						FloatChannels[ChannelIndex + 4]->SetDefault(CurrentVector.Y);
						FloatChannels[ChannelIndex + 5]->SetDefault(CurrentVector.Z);
					}
					if (ControlElement->Settings.ControlType != ERigControlType::TransformNoScale)
					{
						if (InBlendType == EMovieSceneBlendType::Absolute)
						{
							FloatChannels[ChannelIndex + 6]->SetDefault(1.0f);
							FloatChannels[ChannelIndex + 7]->SetDefault(1.0f);
							FloatChannels[ChannelIndex + 8]->SetDefault(1.0f);
						}
						else if (InBlendType == EMovieSceneBlendType::Additive)
						{
							FloatChannels[ChannelIndex + 6]->SetDefault(0.0f);
							FloatChannels[ChannelIndex + 7]->SetDefault(0.0f);
							FloatChannels[ChannelIndex + 8]->SetDefault(0.0f);
						}
						else if (InBlendType == EMovieSceneBlendType::Override)
						{
							FVector CurrentVector = Val.GetScale3D();
							FloatChannels[ChannelIndex + 6]->SetDefault(CurrentVector.X);
							FloatChannels[ChannelIndex + 7]->SetDefault(CurrentVector.Y);
							FloatChannels[ChannelIndex + 8]->SetDefault(CurrentVector.Z);
						}
					}
					
				}
				break;
				}
			};
		}
	}
}

void UMovieSceneControlRigParameterSection::BeginDestroy()
{
	Super::BeginDestroy();
#if WITH_EDITOR
	UnbindEditorDelegates();
#endif
}

void UMovieSceneControlRigParameterSection::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Super::Serialize(Ar);
}
#if WITH_EDITOR
void UMovieSceneControlRigParameterSection::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);	
}

void UMovieSceneControlRigParameterSection::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if(TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		const FName OverrideAssetsName = GET_MEMBER_NAME_CHECKED(UMovieSceneControlRigParameterSection, OverrideAssets); 
		if(TransactionEvent.GetChangedProperties().Contains(OverrideAssetsName))
		{
			if(ControlRig)
			{
				ControlRig->UnlinkAllOverrideAssets();
				for(const TSoftObjectPtr<UControlRigOverrideAsset>& OverrideAssetPtr : OverrideAssets)
				{
					if(UControlRigOverrideAsset* OverrideAsset = OverrideAssetPtr.LoadSynchronous())
					{
						ControlRig->LinkOverrideAsset(OverrideAsset);
					}
				}
				ControlRig->RequestConstruction();
			}

			UpdateOverrideAssetDelegates();
		}
	}
}

#endif

void UMovieSceneControlRigParameterSection::HandleOverrideAssetsChanged(UControlRig* InControlRig)
{
#if WITH_EDITORONLY_DATA
	if(bSuspendOverrideAssetSync)
	{
		return;
	}
	if(InControlRig != ControlRig)
	{
		return;
	}
	Modify();
	OverrideAssets.Reset();
	for(int32 Index=0;Index<InControlRig->NumOverrideAssets();Index++)
	{
		OverrideAssets.AddUnique(InControlRig->GetOverrideAsset(Index));
	}
	UpdateOverrideAssetDelegates();
	ReconstructChannelProxy();
#endif
}

void UMovieSceneControlRigParameterSection::UpdateOverrideAssetDelegates()
{
	for (TObjectIterator<UControlRigOverrideAsset> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		It->OnChanged().RemoveAll(this);
	}
#if WITH_EDITORONLY_DATA
	for(const TSoftObjectPtr<UControlRigOverrideAsset>& OverrideAssetPtr : OverrideAssets)
	{
		if(UControlRigOverrideAsset* OverrideAsset = OverrideAssetPtr.Get())
		{
			OverrideAsset->OnChanged().AddUObject(this, &UMovieSceneControlRigParameterSection::HandleOverrideAssetChanged);
		}
	}
#endif
}

void UMovieSceneControlRigParameterSection::HandleOverrideAssetChanged(const UControlRigOverrideAsset* InOverrideAsset)
{
	if(InOverrideAsset)
	{
		static const FString DisplayNameString = TEXT("Settings->DisplayName");
		if(InOverrideAsset->Overrides.ContainsPathForAnySubject(DisplayNameString))
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::PostEditImport()
{
	Super::PostEditImport();
	if (UMovieSceneControlRigParameterTrack* Track = Cast< UMovieSceneControlRigParameterTrack>(GetOuter()))
	{
		SetControlRig(Track->GetControlRig());
	}
	ReconstructChannelProxy();
}

void UMovieSceneControlRigParameterSection::PostLoad()
{
	// When loading older assets, make sure we carry over the Control Rig class it was built with.
	if (ControlRigClass_DEPRECATED)
	{
		ControlRigAssetReference = FControlRigAssetStrongReference(ControlRigClass_DEPRECATED);
	}
	
	Super::PostLoad();
#if WITH_EDITOR
	BindEditorDelegates();
#endif
	PostLoadConstraintChannels();
}

void UMovieSceneControlRigParameterSection::PostLoadConstraintChannels()
{
	if (ControlRig)
	{
		const FSoftObjectPath RigPath(ControlRig);
		auto UpdateHandle = [&RigPath, NewControlRig = ControlRig.Get(), this](const TObjectPtr<UTransformableHandle>& Handle)
		{
			if (UTransformableControlHandle* ControlHandle = ::IsValid(Handle) ? Cast<UTransformableControlHandle>(Handle.Get()) : nullptr)
			{
				const FSoftObjectPath& HandleRigPath = ControlHandle->ControlRig.ToSoftObjectPath();
				if (RigPath.IsSubobject() && HandleRigPath.IsSubobject())
				{
					const FSoftObjectPath HandleRigPathInThisAsset(RigPath.GetAssetPath(), HandleRigPath.GetSubPathUtf8String());					
					if (HandleRigPathInThisAsset == RigPath)
					{
						ControlHandle->UpdateControlRig(NewControlRig);						
					}
				}
			}						
		};

		// update the constraints' handles related to this ControlRig object that live within this section 
		for (FConstraintAndActiveChannel& ConstraintChannel : ConstraintsChannels)
		{
			if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
			{
				UpdateHandle(TransformConstraint->ChildTRSHandle);
				UpdateHandle(TransformConstraint->ParentTRSHandle);
			}
		}

		// update the constraints' handles related to this ControlRig object that live within other constrained sections
		if (const UMovieScene* OuterMovieScene = GetTypedOuter<UMovieScene>())
		{
			static const TSet< TSubclassOf<UMovieSceneTrack> > ConstrainedTracks(
			{
				UMovieSceneControlRigParameterTrack::StaticClass(),
				UMovieScene3DTransformTrack::StaticClass()
			});

			const UObject* OuterTrack = GetOuter();
			auto ProcessSection = [OuterTrack, &UpdateHandle](UMovieSceneSection* InSection)
			{
				if (InSection && InSection->GetOuter() != OuterTrack)
				{
					if (IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(InSection))
					{
						for (FConstraintAndActiveChannel& ConstraintChannel: ConstrainedSection->GetConstraintsChannels())
						{
							if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
							{
								UpdateHandle(TransformConstraint->ParentTRSHandle);
							}
						}
					}
				}
			};

			auto ProcessTrack = [&ProcessSection, OuterTrack](const UMovieSceneTrack* InTrack)
			{
				if (InTrack && InTrack != OuterTrack)
				{
					if (ConstrainedTracks.Contains(InTrack->GetClass()))
					{
						for (UMovieSceneSection* Section : InTrack->GetAllSections())
						{
							ProcessSection(Section);
						}
					}
				}
			};
			
			for (const FMovieSceneBinding& Binding : OuterMovieScene->GetBindings())
			{
				for (const TSubclassOf<UMovieSceneTrack>& TrackClass: ConstrainedTracks)
				{
					for (UMovieSceneTrack* Track: OuterMovieScene->FindTracks(TrackClass, Binding.GetObjectGuid()))
					{
						ProcessTrack(Track);
					}
				}
			}

			for (UMovieSceneTrack* Track: OuterMovieScene->GetTracks())
			{
				ProcessTrack(Track);
			}
		}
	}
}

bool UMovieSceneControlRigParameterSection::HasScalarParameter(FName InParameterName) const
{
	for (const FScalarParameterNameAndCurve& ScalarParameterNameAndCurve : ScalarParameterNamesAndCurves)
	{
		if (ScalarParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasBoolParameter(FName InParameterName) const
{
	for (const FBoolParameterNameAndCurve& BoolParameterNameAndCurve : BoolParameterNamesAndCurves)
	{
		if (BoolParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasEnumParameter(FName InParameterName) const
{
	for (const FEnumParameterNameAndCurve& EnumParameterNameAndCurve : EnumParameterNamesAndCurves)
	{
		if (EnumParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasIntegerParameter(FName InParameterName) const
{
	for (const FIntegerParameterNameAndCurve& IntegerParameterNameAndCurve : IntegerParameterNamesAndCurves)
	{
		if (IntegerParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasVector2DParameter(FName InParameterName) const
{
	for (const FVector2DParameterNameAndCurves& Vector2DParameterNameAndCurve : Vector2DParameterNamesAndCurves)
	{
		if (Vector2DParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasVectorParameter(FName InParameterName) const
{
	for (const FVectorParameterNameAndCurves& VectorParameterNameAndCurve : VectorParameterNamesAndCurves)
	{
		if (VectorParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasColorParameter(FName InParameterName) const
{
	for (const FColorParameterNameAndCurves& ColorParameterNameAndCurve : ColorParameterNamesAndCurves)
	{
		if (ColorParameterNameAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasTransformParameter(FName InParameterName) const
{
	for (const FTransformParameterNameAndCurves& TransformParameterNamesAndCurve : TransformParameterNamesAndCurves)
	{
		if (TransformParameterNamesAndCurve.ParameterName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::HasSpaceChannel(FName InParameterName) const
{
	for (const FSpaceControlNameAndChannel& SpaceChannel : SpaceChannels)
	{
		if (SpaceChannel.ControlName == InParameterName)
		{
			return true;
		}
	}
	return false;
}

FSpaceControlNameAndChannel* UMovieSceneControlRigParameterSection::GetSpaceChannel(FName InParameterName) 
{
	for (FSpaceControlNameAndChannel& SpaceChannel : SpaceChannels)
	{
		if (SpaceChannel.ControlName == InParameterName)
		{
			return &SpaceChannel;
		}
	}
	return nullptr;
}

FName UMovieSceneControlRigParameterSection::FindControlNameFromSpaceChannel(const FMovieSceneControlRigSpaceChannel* InSpaceChannel) const
{
	if (const FSpaceControlNameAndChannel* Space = UE::MovieScene::FindEntryWithinArrayByPtr(SpaceChannels, InSpaceChannel))
	{
		check(InSpaceChannel== &Space->SpaceCurve);
		return Space->ControlName;
	}
	return NAME_None;
}

void UMovieSceneControlRigParameterSection::MaskOutIfThereAreMaskedControls(const FName& InControlName)
{
	if (ControlNameMask.IsEmpty() == false)
	{
		ControlNameMask.Add(InControlName);
	}
}

void UMovieSceneControlRigParameterSection::AddScalarParameter(FName InParameterName, TOptional<float> DefaultValue, bool bReconstructChannel)
{
	FMovieSceneFloatChannel* ExistingChannel = nullptr;
	if (!HasScalarParameter(InParameterName))
	{
		const int32 NewIndex = ScalarParameterNamesAndCurves.Add(FScalarParameterNameAndCurve(InParameterName));
		ExistingChannel = &ScalarParameterNamesAndCurves[NewIndex].ParameterCurve;
		if (DefaultValue.IsSet())
		{
			ExistingChannel->SetDefault(DefaultValue.GetValue());
		}
		else
		{
			ExistingChannel->SetDefault(0.0f);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);
		
		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}

	}
}


void UMovieSceneControlRigParameterSection::AddBoolParameter(FName InParameterName, TOptional<bool> DefaultValue, bool bReconstructChannel)
{
	FMovieSceneBoolChannel* ExistingChannel = nullptr;
	if (!HasBoolParameter(InParameterName))
	{
		const int32 NewIndex = BoolParameterNamesAndCurves.Add(FBoolParameterNameAndCurve(InParameterName));
		ExistingChannel = &BoolParameterNamesAndCurves[NewIndex].ParameterCurve;
		if (DefaultValue.IsSet())
		{
			ExistingChannel->SetDefault(DefaultValue.GetValue());
		}
		else
		{
			ExistingChannel->SetDefault(false);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);

		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}
void UMovieSceneControlRigParameterSection::AddEnumParameter(FName InParameterName, UEnum* Enum,TOptional<uint8> DefaultValue, bool bReconstructChannel)
{
	FMovieSceneByteChannel* ExistingChannel = nullptr;
	if (!HasEnumParameter(InParameterName))
	{
		const int32 NewIndex = EnumParameterNamesAndCurves.Add(FEnumParameterNameAndCurve(InParameterName));
		ExistingChannel = &EnumParameterNamesAndCurves[NewIndex].ParameterCurve;
		if (DefaultValue.IsSet())
		{
			ExistingChannel->SetDefault(DefaultValue.GetValue());
		}
		else
		{
			ExistingChannel->SetDefault(false);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);

		ExistingChannel->SetEnum(Enum);
		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::AddIntegerParameter(FName InParameterName, TOptional<int32> DefaultValue, bool bReconstructChannel)
{
	FMovieSceneIntegerChannel* ExistingChannel = nullptr;
	if (!HasIntegerParameter(InParameterName))
	{
		const int32 NewIndex = IntegerParameterNamesAndCurves.Add(FIntegerParameterNameAndCurve(InParameterName));
		ExistingChannel = &IntegerParameterNamesAndCurves[NewIndex].ParameterCurve;
		if (DefaultValue.IsSet())
		{
			ExistingChannel->SetDefault(DefaultValue.GetValue());
		}
		else
		{
			ExistingChannel->SetDefault(false);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);

		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::AddVector2DParameter(FName InParameterName, TOptional<FVector2D> DefaultValue, bool bReconstructChannel)
{
	FVector2DParameterNameAndCurves* ExistingCurves = nullptr;

	if (!HasVector2DParameter(InParameterName))
	{
		int32 NewIndex = Vector2DParameterNamesAndCurves.Add(FVector2DParameterNameAndCurves(InParameterName));
		ExistingCurves = &Vector2DParameterNamesAndCurves[NewIndex];
		if (DefaultValue.IsSet())
		{
			ExistingCurves->XCurve.SetDefault(DefaultValue.GetValue().X);
			ExistingCurves->YCurve.SetDefault(DefaultValue.GetValue().Y);
		}
		else
		{
			ExistingCurves->XCurve.SetDefault(0.0f);
			ExistingCurves->YCurve.SetDefault(0.0f);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);

		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::AddVectorParameter(FName InParameterName, TOptional<FVector> DefaultValue, bool bReconstructChannel)
{
	FVectorParameterNameAndCurves* ExistingCurves = nullptr;

	if (!HasVectorParameter(InParameterName))
	{
		int32 NewIndex = VectorParameterNamesAndCurves.Add(FVectorParameterNameAndCurves(InParameterName));
		ExistingCurves = &VectorParameterNamesAndCurves[NewIndex];
		if (DefaultValue.IsSet())
		{
			ExistingCurves->XCurve.SetDefault(DefaultValue.GetValue().X);
			ExistingCurves->YCurve.SetDefault(DefaultValue.GetValue().Y);
			ExistingCurves->ZCurve.SetDefault(DefaultValue.GetValue().Z);

		}
		else
		{
			ExistingCurves->XCurve.SetDefault(0.0f);
			ExistingCurves->YCurve.SetDefault(0.0f);
			ExistingCurves->ZCurve.SetDefault(0.0f);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);

		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::AddColorParameter(FName InParameterName, TOptional<FLinearColor> DefaultValue, bool bReconstructChannel)
{
	FColorParameterNameAndCurves* ExistingCurves = nullptr;

	if (!HasColorParameter(InParameterName))
	{
		int32 NewIndex = ColorParameterNamesAndCurves.Add(FColorParameterNameAndCurves(InParameterName));
		ExistingCurves = &ColorParameterNamesAndCurves[NewIndex];
		if (DefaultValue.IsSet())
		{
			ExistingCurves->RedCurve.SetDefault(DefaultValue.GetValue().R);
			ExistingCurves->GreenCurve.SetDefault(DefaultValue.GetValue().G);
			ExistingCurves->BlueCurve.SetDefault(DefaultValue.GetValue().B);
			ExistingCurves->AlphaCurve.SetDefault(DefaultValue.GetValue().A);
		}
		else
		{
			ExistingCurves->RedCurve.SetDefault(0.0f);
			ExistingCurves->GreenCurve.SetDefault(0.0f);
			ExistingCurves->BlueCurve.SetDefault(0.0f);
			ExistingCurves->AlphaCurve.SetDefault(0.0f);
		}

		MaskOutIfThereAreMaskedControls(InParameterName);

		if (bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::AddTransformParameter(FName InParameterName, TOptional<FEulerTransform> DefaultValue, bool bReconstructChannel)
{
	FTransformParameterNameAndCurves* ExistingCurves = nullptr;

	if (!HasTransformParameter(InParameterName))
	{
		int32 NewIndex = TransformParameterNamesAndCurves.Add(FTransformParameterNameAndCurves(InParameterName));
		ExistingCurves = &TransformParameterNamesAndCurves[NewIndex];
		if (DefaultValue.IsSet())
		{
			FEulerTransform& InValue = DefaultValue.GetValue();
			const FVector& Translation = InValue.GetLocation();
			const FRotator& Rotator = InValue.Rotator();
			const FVector& Scale = InValue.GetScale3D();
			ExistingCurves->Translation[0].SetDefault(Translation[0]);
			ExistingCurves->Translation[1].SetDefault(Translation[1]);
			ExistingCurves->Translation[2].SetDefault(Translation[2]);

			ExistingCurves->Rotation[0].SetDefault(Rotator.Roll);
			ExistingCurves->Rotation[1].SetDefault(Rotator.Pitch);
			ExistingCurves->Rotation[2].SetDefault(Rotator.Yaw);

			ExistingCurves->Scale[0].SetDefault(Scale[0]);
			ExistingCurves->Scale[1].SetDefault(Scale[1]);
			ExistingCurves->Scale[2].SetDefault(Scale[2]);

		}
		else if (GetBlendType() == EMovieSceneBlendType::Additive)
		{
			ExistingCurves->Translation[0].SetDefault(0.0f);
			ExistingCurves->Translation[1].SetDefault(0.0f);
			ExistingCurves->Translation[2].SetDefault(0.0f);

			ExistingCurves->Rotation[0].SetDefault(0.0f);
			ExistingCurves->Rotation[1].SetDefault(0.0f);
			ExistingCurves->Rotation[2].SetDefault(0.0f);

			ExistingCurves->Scale[0].SetDefault(0.0f);
			ExistingCurves->Scale[1].SetDefault(0.0f);
			ExistingCurves->Scale[2].SetDefault(0.0f);
		}
		MaskOutIfThereAreMaskedControls(InParameterName);

		if(bReconstructChannel)
		{
			ReconstructChannelProxy();
		}
	}
}

bool UMovieSceneControlRigParameterSection::CopyVectorParameterCurvesToTransform(const FName& InParameterName, ERigControlType InType)
{
	if (HasVectorParameter(InParameterName))
	{
		for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
		{
			if (Vector.ParameterName == InParameterName)
			{
				FTransformParameterNameAndCurves* ExistingCurves = nullptr;
				int32 Index = INDEX_NONE;
				for (int32 IndexCount = 0; IndexCount < TransformParameterNamesAndCurves.Num(); ++ IndexCount)
				{
					FTransformParameterNameAndCurves& TransformParameterNamesAndCurve = TransformParameterNamesAndCurves[IndexCount];
					if (TransformParameterNamesAndCurve.ParameterName == InParameterName)
					{
						Index = IndexCount;
						break;
					}
				}
				if (Index == INDEX_NONE) // no transform so add
				{
					Index = TransformParameterNamesAndCurves.Add(FTransformParameterNameAndCurves(InParameterName));

				}
				if (Index != INDEX_NONE)
				{
					ExistingCurves = &TransformParameterNamesAndCurves[Index];
					if (InType == ERigControlType::Position)
					{
						ExistingCurves->Translation[0] = Vector.XCurve;
						ExistingCurves->Translation[1] = Vector.YCurve;
						ExistingCurves->Translation[2] = Vector.ZCurve;
						return true;
					}
					else if (InType == ERigControlType::Rotator)
					{
						ExistingCurves->Rotation[0] = Vector.XCurve;
						ExistingCurves->Rotation[1] = Vector.YCurve;
						ExistingCurves->Rotation[2] = Vector.ZCurve;
						return true;
					}
					else if (InType == ERigControlType::Scale)
					{
						ExistingCurves->Scale[0] = Vector.XCurve;
						ExistingCurves->Scale[1] = Vector.YCurve;
						ExistingCurves->Scale[2] = Vector.ZCurve;
						return true;
					}
				}
			}
		}
	}
	return false;
}


// only allow creation of space channels onto non-parented Controls
bool UMovieSceneControlRigParameterSection::CanCreateSpaceChannel(FName InControlName) const
{
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InControlName))
	{
		if (ChannelInfo->ParentControlIndex == INDEX_NONE)
		{
			return true;
		}
	}
	return false;
}

void UMovieSceneControlRigParameterSection::AddSpaceChannel(FName InControlName, bool bReconstructChannel)
{
	//only add it if it's the first section since we can't blend them
	if (UMovieSceneControlRigParameterTrack* Track = GetTypedOuter<UMovieSceneControlRigParameterTrack>())
	{
		const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
		if (!Sections.IsEmpty() && Sections[0] == this)
		{
			if (CanCreateSpaceChannel(InControlName) && !HasSpaceChannel(InControlName))
			{
				FSpaceControlNameAndChannel& NameAndChannel = SpaceChannels.Emplace_GetRef(InControlName);
				
				if (const FRigControlElement* Control = ControlRig ? ControlRig->FindControl(InControlName) : nullptr)
				{
					const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
					FRigElementKey CurrentParentKey = Hierarchy->GetActiveParent(Control->GetKey());
					if (CurrentParentKey == URigHierarchy::GetDefaultParentKey())
					{
						CurrentParentKey = Hierarchy->GetDefaultParent(Control->GetKey());
					}

					// set the default channel value to the current parent if it differs from the one initially constructed
					if (CurrentParentKey != Hierarchy->GetFirstParent(Control->GetKey()))
					{
						NameAndChannel.SpaceCurve.SetDefault({EMovieSceneControlRigSpaceType::ControlRig, CurrentParentKey});
					}
				}
				
				if (OnSpaceChannelAdded.IsBound())
				{
					OnSpaceChannelAdded.Broadcast(this, InControlName, &NameAndChannel.SpaceCurve);
				}
			}
			if (bReconstructChannel)
			{
				ReconstructChannelProxy();
			}
		}
	}
}

bool UMovieSceneControlRigParameterSection::HasConstraintChannel(const FGuid& InGuid) const
{
	return ConstraintsChannels.ContainsByPredicate( [InGuid](const FConstraintAndActiveChannel& InChannel)
	{
		return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->ConstraintID == InGuid : false;
	});
}

FConstraintAndActiveChannel* UMovieSceneControlRigParameterSection::GetConstraintChannel(const FGuid& InConstraintID)
{
	const int32 Index = ConstraintsChannels.IndexOfByPredicate([InConstraintID](const FConstraintAndActiveChannel& InChannel)
		{
			return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->ConstraintID == InConstraintID : false;
		});
	return (Index != INDEX_NONE) ? &ConstraintsChannels[Index] : nullptr;	
}

void UMovieSceneControlRigParameterSection::ReplaceConstraint(const FName InConstraintName, UTickableConstraint* InConstraint) 
{
	const int32 Index = ConstraintsChannels.IndexOfByPredicate([InConstraintName](const FConstraintAndActiveChannel& InChannel)
	{
		return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->GetFName() == InConstraintName : false;
	});
	if (Index != INDEX_NONE)
	{
		Modify();
		ConstraintsChannels[Index].SetConstraint(InConstraint);
		ReconstructChannelProxy();
	}
}

void UMovieSceneControlRigParameterSection::OnConstraintsChanged()
{
	ReconstructChannelProxy();
}

void UMovieSceneControlRigParameterSection::AddConstraintChannel(UTickableConstraint* InConstraint)
{
	if (InConstraint && !HasConstraintChannel(InConstraint->ConstraintID))
	{
		Modify();
		
		const int32 NewIndex = ConstraintsChannels.Add(FConstraintAndActiveChannel(InConstraint));

		FMovieSceneConstraintChannel* ExistingChannel = &ConstraintsChannels[NewIndex].ActiveChannel;
		
		//make copy that we can spawn if it doesn't exist
		//the rename changes the outer to this section (from any actor manager)
		InConstraint->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		
		if (OnConstraintChannelAdded.IsBound())
		{
			OnConstraintChannelAdded.Broadcast(this, ExistingChannel);
		}
		//todo got rid of the if(bReconstructChannel) flag since it was always true but it may need to be false from undo, in which case we need to change
		//change this virtual functions signature
		ReconstructChannelProxy();
	}
}

void UMovieSceneControlRigParameterSection::RemoveConstraintChannel(const UTickableConstraint* InConstraint)
{
	if (bDoNotRemoveChannel == true)
	{
		return;
	}
	const int32 Index = ConstraintsChannels.IndexOfByPredicate([InConstraint](const FConstraintAndActiveChannel& InChannel)
	{
		return InChannel.GetConstraint().Get() ? InChannel.GetConstraint().Get() == InConstraint : false;
	});

	if (ConstraintsChannels.IsValidIndex(Index))
	{
		Modify();
		ConstraintsChannels.RemoveAt(Index);
		ReconstructChannelProxy();
	}
}

TArray<FConstraintAndActiveChannel>& UMovieSceneControlRigParameterSection::GetConstraintsChannels() 
{
	return ConstraintsChannels;
}

const TArray<FConstraintAndActiveChannel>& UMovieSceneControlRigParameterSection::GetConstraintsChannels() const
{
	return ConstraintsChannels;
}

const FName& UMovieSceneControlRigParameterSection::FindControlNameFromConstraintChannel(
	const FMovieSceneConstraintChannel* InConstraintChannel) const
{
	const FConstraintAndActiveChannel* Entry = UE::MovieScene::FindEntryWithinArrayByPtr(ConstraintsChannels, InConstraintChannel);
	if (Entry)
	{
		const int32 Index = Entry - ConstraintsChannels.GetData();

		// look for info referencing that constraint index
		for (const TPair<FName, FChannelMapInfo>& It : ControlChannelMap)
		{
			const FChannelMapInfo& Info = It.Value;
			if (Info.ConstraintsIndex.Contains(Index))
			{
				return It.Key;
			}
		}
	}

	static const FName DummyName = NAME_None;
	return DummyName;
}

void UMovieSceneControlRigParameterSection::ForEachParameter(TFunction<void(FBaseParameterNameAndValue*)> InCallback)
{
	ForEachParameter(ScalarParameterNamesAndCurves, InCallback);
	ForEachParameter(BoolParameterNamesAndCurves, InCallback);
	ForEachParameter(EnumParameterNamesAndCurves, InCallback);
	ForEachParameter(IntegerParameterNamesAndCurves, InCallback);
	ForEachParameter(Vector2DParameterNamesAndCurves, InCallback);
	ForEachParameter(VectorParameterNamesAndCurves, InCallback);
	ForEachParameter(ColorParameterNamesAndCurves, InCallback);
	ForEachParameter(TransformParameterNamesAndCurves, InCallback);
}

void UMovieSceneControlRigParameterSection::ForEachParameter(TOptional<ERigControlType> InControlType,
	TFunction<void(FBaseParameterNameAndValue*)> InCallback)
{
	if(!InControlType.IsSet())
	{
		ForEachParameter(InCallback);
		return;
	}
	
	switch (InControlType.GetValue())
	{
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			ForEachParameter(ScalarParameterNamesAndCurves, InCallback);
			break;
		}
		case ERigControlType::Bool:
		{
			ForEachParameter(BoolParameterNamesAndCurves, InCallback);
			break;
		}
		case ERigControlType::Integer:
		{
			ForEachParameter(IntegerParameterNamesAndCurves, InCallback);
			ForEachParameter(EnumParameterNamesAndCurves, InCallback);
			break;
		}
		case ERigControlType::Vector2D:
		{
			ForEachParameter(Vector2DParameterNamesAndCurves, InCallback);
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Rotator:
		case ERigControlType::Scale:
		{
			ForEachParameter(VectorParameterNamesAndCurves, InCallback);
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::EulerTransform:
		case ERigControlType::TransformNoScale:
		{
			ForEachParameter(TransformParameterNamesAndCurves, InCallback);
			break;
		}
		default:
		{
			break;
		}
	}
}

TArray<FSpaceControlNameAndChannel>& UMovieSceneControlRigParameterSection::GetSpaceChannels()
{
	return SpaceChannels;
}
const TArray< FSpaceControlNameAndChannel>& UMovieSceneControlRigParameterSection::GetSpaceChannels() const
{
	return SpaceChannels;
}

uint32 UMovieSceneControlRigParameterSection::ComputeControlsHash(const TArray<FRigControlElement*>& Controls)
{
	uint32 Hash = GetTypeHash(Controls.Num());
	for (const FRigControlElement* Control : Controls)
	{
		if (Control)
		{
			Hash = HashCombine(Hash, GetTypeHash(Control->GetFName()));
			Hash = HashCombine(Hash, GetTypeHash(static_cast<uint8>(Control->Settings.ControlType)));
		}
	}
	return Hash;
}

bool UMovieSceneControlRigParameterSection::IsDifferentThanLastControlsUsedToReconstruct(const TArray<FRigControlElement*>& NewControls) const
{
	return ComputeControlsHash(NewControls) != LastControlsHash;
}

void UMovieSceneControlRigParameterSection::StoreLastControlsUsedToReconstruct(const TArray<FRigControlElement*>& NewControls)
{
	LastControlsHash = ComputeControlsHash(NewControls);
}

void UMovieSceneControlRigParameterSection::ReconstructChannelProxy()
{
	ChannelProxy.Reset();
	BroadcastChanged();
}

//hopefully remove after a bit
void UMovieSceneControlRigParameterSection::HACK_FixMultipleParamsWithSameName()
{
#if WITH_EDITOR
	bool bHasDup = false;
	bHasDup = HACK_CheckForDupParameters(ScalarParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(BoolParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(EnumParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(IntegerParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(Vector2DParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(VectorParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(ColorParameterNamesAndCurves) || bHasDup;
	bHasDup = HACK_CheckForDupParameters(TransformParameterNamesAndCurves) || bHasDup;
	if (bHasDup)
	{

		const EAppReturnType::Type Choice = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("Sequencer", "Duplicate corrupted controls found in level sequence, just keep first set of original Control curves (recommended but may revert to an older version)?"));

		if (Choice == EAppReturnType::Yes)
		{
			bool bRemovedSomething = false;
			bRemovedSomething = HACK_CleanForEachParameter(ScalarParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(BoolParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(EnumParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(IntegerParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(Vector2DParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(VectorParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(ColorParameterNamesAndCurves) || bRemovedSomething;
			bRemovedSomething = HACK_CleanForEachParameter(TransformParameterNamesAndCurves) || bRemovedSomething;

			if (bRemovedSomething)
			{
				UE_LOGF(LogControlRig, Warning, "There were duplicated curves found possibly by 40400084, resave to remove duplicates and keep first one");
				Modify(); //we removed something so mark as dirty so animators can resave
			}
		}
	}
#endif
}

EMovieSceneChannelProxyType UMovieSceneControlRigParameterSection::CacheChannelProxy()
{
	HACK_FixMultipleParamsWithSameName();
	const FName UIMin("UIMin");
	const FName UIMax("UIMax");

	FMovieSceneChannelProxyData Channels;
	ControlChannelMap.Empty();

#if WITH_EDITOR
	const EElementNameDisplayMode ElementNameDisplayMode = UControlRigEditorSettings::Get()->ElementNameDisplayMode;
#else
	const EElementNameDisplayMode ElementNameDisplayMode = EElementNameDisplayMode::AssetDefault;
#endif

	// Need to create the channels in sorted orders, only if we have controls
	if (ControlRig)
	{
		TArray<FRigControlElement*> SortedControls;
		ControlRig->GetControlsInOrder(SortedControls);
		StoreLastControlsUsedToReconstruct(SortedControls);
		if (SortedControls.Num() > 0)
		{
			if (PlayRate.GetType() == EMovieSceneTimeWarpType::Custom)
			{
				UMovieSceneTimeWarpGetter* Custom = PlayRate.AsCustom();
				if (Custom)
				{
					Custom->PopulateChannelProxy(Channels, UMovieSceneTimeWarpGetter::EAllowTopLevelChannels::Yes);
				}
			}

			int32 ControlIndex = 0;
			int32 MaskIndex = 0;
			int32 SortOrder = 2; //start with 2 so Timewarp then Weight are first 
			int32 FloatChannelIndex = 0;
			int32 BoolChannelIndex = 0;
			int32 EnumChannelIndex = 0;
			int32 IntegerChannelIndex = 0;
			int32 SpaceChannelIndex = 0;
			int32 CategoryIndex = 0;
			int32 ConstraintsChannelIndex = 0;

			const FName BoolChannelTypeName = FMovieSceneBoolChannel::StaticStruct()->GetFName();
			const FName EnumChannelTypeName = FMovieSceneByteChannel::StaticStruct()->GetFName();
			const FName IntegerChannelTypeName = FMovieSceneIntegerChannel::StaticStruct()->GetFName();
			const FName SpaceName = FName(TEXT("Space"));

			// begin constraints
			auto AddConstrainChannels = [this, &ConstraintsChannelIndex, &SortOrder, &Channels](
				const FName& InControlName, const FText& InGroup, const bool bEnabled)
				{
					const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig.Get(), InControlName);
					for (FConstraintAndActiveChannel& ConstraintChannel : ConstraintsChannels)
					{
						UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(ConstraintChannel.GetConstraint());
						UTransformableControlHandle* Handle = Constraint ? Cast<UTransformableControlHandle>(Constraint->ChildTRSHandle.Get()) : nullptr;
						if (Handle && Handle->GetHash() == ControlHash)
						{
							const TWeakObjectPtr<UTickableConstraint>& WeakConstraint = MakeWeakObjectPtr(ConstraintChannel.GetConstraint());
							if (WeakConstraint.IsValid())
							{
								if (FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InControlName))
								{
									ChannelInfo->ConstraintsIndex.Add(ConstraintsChannelIndex);
								}

#if WITH_EDITOR
								ConstraintChannel.ActiveChannel.ExtraLabel = [WeakConstraint]
								{
									if (WeakConstraint.IsValid())
									{
										constexpr TCHAR ParentChildDelimiter[] = TEXT("|");
										FString ParentStr; WeakConstraint->GetLabel().Split(ParentChildDelimiter, &ParentStr, nullptr);
										if (!ParentStr.IsEmpty())
										{
											return ParentStr;
										}
									}
									static const FString DummyStr;
									return DummyStr;
								};

								const FText DisplayText = FText::FromString(WeakConstraint->GetTypeLabel());
								FMovieSceneChannelMetaData MetaData(WeakConstraint->GetFName(), DisplayText, InGroup, bEnabled);
								ConstraintsChannelIndex += 1;
								MetaData.SortOrder = SortOrder++;
								MetaData.bCanCollapseToTrack = true;

								Channels.Add(ConstraintChannel.ActiveChannel, MetaData, TMovieSceneExternalValue<bool>());
#else
								Channels.Add(ConstraintChannel.ActiveChannel);
#endif
							}
						}
					}
				};
			// end constraints

#if WITH_EDITOR
		// masking for per control channels based on control filters
			auto MaybeApplyChannelMask = [](FMovieSceneChannelMetaData& OutMetadata, const FRigControlElement* ControlElement, ERigControlTransformChannel InChannel)
				{
					if (!OutMetadata.bEnabled)
					{
						return;
					}

					const TArray<ERigControlTransformChannel>& FilteredChannels = ControlElement->Settings.FilteredChannels;
					if (!FilteredChannels.IsEmpty())
					{
						OutMetadata.bEnabled = FilteredChannels.Contains(InChannel);
					}
				};
#endif

			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

			TMap<const FRigControlElement*, FText> ControlToGroupName;
			TSet<FString> GroupNames;
			auto GetUniqueGroupName = [Hierarchy, &GroupNames, &ControlToGroupName, ElementNameDisplayMode](const FRigControlElement* Element) -> FText 
			{
				FString DisplayName = Hierarchy->GetDisplayNameForUI(Element, ElementNameDisplayMode).ToString();
				if (GroupNames.Contains(DisplayName))
				{
					int32 i=1;
					FString NewName;
					do
					{
						NewName = FString::Printf(TEXT("%s_%d"), *DisplayName, i++);
					}
					while (GroupNames.Contains(NewName));
					DisplayName = NewName;
				}

				GroupNames.Add(DisplayName);
				const FText Result = FText::FromString(DisplayName);
				ControlToGroupName.FindOrAdd(Element) = Result;
				return Result;
			};
			
			for (FRigControlElement* ControlElement : SortedControls)
			{
				if (!Hierarchy->IsAnimatable(ControlElement))
				{
					continue;
				}

				const FName& ControlName = ControlElement->GetFName();

				FName ParentControlName = NAME_None;
				FText Group;

				bool bParentIsEnabled = true;
				if (Hierarchy->ShouldBeGrouped(ControlElement))
				{
					if (const FRigControlElement* ParentControlElement = Cast<FRigControlElement>(Hierarchy->GetFirstParent(ControlElement)))
					{
						ParentControlName = ParentControlElement->GetFName();
						bParentIsEnabled = GetControlNameMask(ParentControlName);
						if (FText* Found = ControlToGroupName.Find(ParentControlElement))
						{
							Group = *Found;
						}
						else
						{
							Group = GetUniqueGroupName(ParentControlElement);
						}
					}
				}

				const bool bEnabled = GetControlNameMask(ControlElement->GetFName()) && bParentIsEnabled;

#if WITH_EDITOR
				switch (ControlElement->Settings.ControlType)
				{
				case ERigControlType::Float:
				case ERigControlType::ScaleFloat:
				{
					for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
					{
						if (ControlName == Scalar.ParameterName)
						{
							if (Group.IsEmpty())
							{
								ControlChannelMap.Add(Scalar.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex, CategoryIndex));
								Group = GetUniqueGroupName(ControlElement);
								if (bEnabled)
								{
									++CategoryIndex;
								}
							}
							else
							{
								const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
								const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
								ControlChannelMap.Add(Scalar.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, ParentControlIndex, NAME_None, MaskIndex, CategoryIndex));
							}

							FParameterFloatChannelEditorData EditorData(ControlRig, Scalar.ParameterName, bEnabled, Group, SortOrder);
							EditorData.MetaData.DisplayText = Hierarchy->GetDisplayNameForUI(ControlElement, ElementNameDisplayMode);
							EditorData.MetaData.PropertyMetaData.Add(UIMin, FString::SanitizeFloat(ControlElement->Settings.MinimumValue.Get<float>()));
							EditorData.MetaData.PropertyMetaData.Add(UIMax, FString::SanitizeFloat(ControlElement->Settings.MaximumValue.Get<float>()));
							Channels.Add(Scalar.ParameterCurve, EditorData.MetaData, EditorData.ExternalValues);
							FloatChannelIndex += 1;
							SortOrder += 1;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}
				case ERigControlType::Bool:
				{
					for (FBoolParameterNameAndCurve& Bool : GetBoolParameterNamesAndCurves())
					{
						if (ControlName == Bool.ParameterName)
						{
							if (Group.IsEmpty())
							{
								ControlChannelMap.Add(Bool.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, BoolChannelIndex, INDEX_NONE, BoolChannelTypeName, MaskIndex, CategoryIndex));
								Group = GetUniqueGroupName(ControlElement);
								if (bEnabled)
								{
									++CategoryIndex;
								}
							}
							else
							{
								const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
								const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
								ControlChannelMap.Add(Bool.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, BoolChannelIndex, ParentControlIndex, BoolChannelTypeName, MaskIndex, CategoryIndex));
							}

							FMovieSceneChannelMetaData MetaData(Bool.ParameterName, Group, Group, bEnabled);
							MetaData.DisplayText = Hierarchy->GetDisplayNameForUI(ControlElement, ElementNameDisplayMode);
							MetaData.SortOrder = SortOrder++;
							BoolChannelIndex += 1;
							ControlIndex += 1;
							// Prevent single channels from collapsing to the track node
							MetaData.bCanCollapseToTrack = true;
							Channels.Add(Bool.ParameterCurve, MetaData, TMovieSceneExternalValue<bool>());
							break;
						}
					}
					break;
				}
				case ERigControlType::Integer:
				{
					if (ControlElement->Settings.ControlEnum)
					{
						for (FEnumParameterNameAndCurve& Enum : GetEnumParameterNamesAndCurves())
						{
							if (ControlName == Enum.ParameterName)
							{
								if (Group.IsEmpty())
								{
									ControlChannelMap.Add(Enum.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, EnumChannelIndex, INDEX_NONE, EnumChannelTypeName, MaskIndex, CategoryIndex));
									Group = GetUniqueGroupName(ControlElement);
									if (bEnabled)
									{
										++CategoryIndex;
									}
								}
								else
								{
									const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
									const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
									ControlChannelMap.Add(Enum.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, EnumChannelIndex, ParentControlIndex, EnumChannelTypeName, MaskIndex, CategoryIndex));
								}

								FMovieSceneChannelMetaData MetaData(Enum.ParameterName, Group, Group, bEnabled);
								MetaData.DisplayText = Hierarchy->GetDisplayNameForUI(ControlElement, ElementNameDisplayMode);
								EnumChannelIndex += 1;
								ControlIndex += 1;
								MetaData.SortOrder = SortOrder++;
								// Prevent single channels from collapsing to the track node
								MetaData.bCanCollapseToTrack = true;
								Channels.Add(Enum.ParameterCurve, MetaData, TMovieSceneExternalValue<uint8>());
								break;
							}
						}
					}
					else
					{
						for (FIntegerParameterNameAndCurve& Integer : GetIntegerParameterNamesAndCurves())
						{
							if (ControlName == Integer.ParameterName)
							{
								if (Group.IsEmpty())
								{
									ControlChannelMap.Add(Integer.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, IntegerChannelIndex, INDEX_NONE, IntegerChannelTypeName, MaskIndex, CategoryIndex));
									Group = GetUniqueGroupName(ControlElement);
									if (bEnabled)
									{
										++CategoryIndex;
									}
								}
								else
								{
									const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
									const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
									ControlChannelMap.Add(Integer.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, IntegerChannelIndex, ParentControlIndex, IntegerChannelTypeName, MaskIndex, CategoryIndex));
								}

								FMovieSceneChannelMetaData MetaData(Integer.ParameterName, Group, Group, bEnabled);
								MetaData.DisplayText = Hierarchy->GetDisplayNameForUI(ControlElement, ElementNameDisplayMode);
								IntegerChannelIndex += 1;
								ControlIndex += 1;
								MetaData.SortOrder = SortOrder++;
								// Prevent single channels from collapsing to the track node
								MetaData.bCanCollapseToTrack = true;
								MetaData.PropertyMetaData.Add(UIMin, FString::FromInt(ControlElement->Settings.MinimumValue.Get<int32>()));
								MetaData.PropertyMetaData.Add(UIMax, FString::FromInt(ControlElement->Settings.MaximumValue.Get<int32>()));
								Channels.Add(Integer.ParameterCurve, MetaData, TMovieSceneExternalValue<int32>());
								break;
							}
						}

					}
					break;
				}
				case ERigControlType::Vector2D:
				{
					for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
					{
						if (ControlName == Vector2D.ParameterName)
						{
							if (Group.IsEmpty())
							{
								ControlChannelMap.Add(Vector2D.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex, CategoryIndex));
								if (bEnabled)
								{
									++CategoryIndex;
								}
								Group = GetUniqueGroupName(ControlElement);
							}
							else
							{
								const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
								const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
								ControlChannelMap.Add(Vector2D.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, ParentControlIndex, NAME_None, MaskIndex, CategoryIndex));
							}
							FParameterVectorChannelEditorData EditorData(ControlRig, Vector2D.ParameterName, bEnabled, Group, SortOrder, 2, ControlElement->Settings.ControlType);
							MaybeApplyChannelMask(EditorData.MetaData[0], ControlElement, ERigControlTransformChannel::TranslationX);
							MaybeApplyChannelMask(EditorData.MetaData[1], ControlElement, ERigControlTransformChannel::TranslationY);
							Channels.Add(Vector2D.XCurve, EditorData.MetaData[0], EditorData.ExternalValues[0]);
							Channels.Add(Vector2D.YCurve, EditorData.MetaData[1], EditorData.ExternalValues[1]);
							FloatChannelIndex += 2;
							SortOrder += 2;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}
				case ERigControlType::Position:
				case ERigControlType::Scale:
				case ERigControlType::Rotator:
				{
					for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
					{
						if (ControlName == Vector.ParameterName)
						{
							if (Group.IsEmpty())
							{
								ControlChannelMap.Add(Vector.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex, CategoryIndex));
								if (bEnabled)
								{
									++CategoryIndex;
								}
								Group = GetUniqueGroupName(ControlElement);
							}
							else
							{
								const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
								const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
								ControlChannelMap.Add(Vector.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, ParentControlIndex, NAME_None, MaskIndex, CategoryIndex));
							}
							if (FSpaceControlNameAndChannel* SpaceChannel = GetSpaceChannel(Vector.ParameterName))
							{

								FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(Vector.ParameterName);
								if (pChannelIndex)
								{
									pChannelIndex->bDoesHaveSpace = true;
									pChannelIndex->SpaceChannelIndex = SpaceChannelIndex;
								}

								FString TotalName = Vector.ParameterName.ToString(); //need ControlName.Space for selection to work.
								FString SpaceString = SpaceName.ToString();
								TotalName += ("." + SpaceString);
								FMovieSceneChannelMetaData SpaceMetaData(FName(*TotalName), Group, Group, bEnabled);
								SpaceMetaData.DisplayText = FText::FromName(SpaceName);
								SpaceChannelIndex += 1;
								SpaceMetaData.SortOrder = SortOrder++;
								// Prevent single channels from collapsing to the track node
								SpaceMetaData.bCanCollapseToTrack = true;
								Channels.Add(SpaceChannel->SpaceCurve, SpaceMetaData);
							}


							FParameterVectorChannelEditorData EditorData(ControlRig, Vector.ParameterName, bEnabled, Group, SortOrder, 3, ControlElement->Settings.ControlType);

							if (ControlElement->Settings.ControlType == ERigControlType::Position)
							{
								MaybeApplyChannelMask(EditorData.MetaData[0], ControlElement, ERigControlTransformChannel::TranslationX);
								MaybeApplyChannelMask(EditorData.MetaData[1], ControlElement, ERigControlTransformChannel::TranslationY);
								MaybeApplyChannelMask(EditorData.MetaData[2], ControlElement, ERigControlTransformChannel::TranslationZ);
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
							{
								MaybeApplyChannelMask(EditorData.MetaData[0], ControlElement, ERigControlTransformChannel::Pitch);
								MaybeApplyChannelMask(EditorData.MetaData[1], ControlElement, ERigControlTransformChannel::Yaw);
								MaybeApplyChannelMask(EditorData.MetaData[2], ControlElement, ERigControlTransformChannel::Roll);
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::Scale)
							{
								MaybeApplyChannelMask(EditorData.MetaData[0], ControlElement, ERigControlTransformChannel::ScaleX);
								MaybeApplyChannelMask(EditorData.MetaData[1], ControlElement, ERigControlTransformChannel::ScaleY);
								MaybeApplyChannelMask(EditorData.MetaData[2], ControlElement, ERigControlTransformChannel::ScaleZ);
							}

							Channels.Add(Vector.XCurve, EditorData.MetaData[0], EditorData.ExternalValues[0]);
							Channels.Add(Vector.YCurve, EditorData.MetaData[1], EditorData.ExternalValues[1]);
							Channels.Add(Vector.ZCurve, EditorData.MetaData[2], EditorData.ExternalValues[2]);
							FloatChannelIndex += 3;
							SortOrder += 3;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}

				case ERigControlType::TransformNoScale:
				case ERigControlType::Transform:
				case ERigControlType::EulerTransform:
				{
					for (FTransformParameterNameAndCurves& Transform : GetTransformParameterNamesAndCurves())
					{
						if (ControlName == Transform.ParameterName)
						{
							if (Group.IsEmpty())
							{
								ControlChannelMap.Add(Transform.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex, CategoryIndex));
								if (bEnabled)
								{
									++CategoryIndex;
								}
								Group = GetUniqueGroupName(ControlElement);
							}
							else
							{
								const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ParentControlName);
								const int32 ParentControlIndex = pChannelIndex ? pChannelIndex->ControlIndex : INDEX_NONE;
								ControlChannelMap.Add(Transform.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, ParentControlIndex, NAME_None, MaskIndex, CategoryIndex));
							}

							// constraints
							AddConstrainChannels(ControlName, Group, bEnabled);

							// spaces
							if (FSpaceControlNameAndChannel* SpaceChannel = GetSpaceChannel(Transform.ParameterName))
							{
								FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(Transform.ParameterName);
								if (pChannelIndex)
								{
									pChannelIndex->bDoesHaveSpace = true;
									pChannelIndex->SpaceChannelIndex = SpaceChannelIndex;
								}

								FString TotalName = Transform.ParameterName.ToString(); //need ControlName.Space for selection to work.
								FString SpaceString = SpaceName.ToString();
								TotalName += ("." + SpaceString);
								FMovieSceneChannelMetaData SpaceMetaData(FName(*TotalName), Group, Group, bEnabled);
								SpaceMetaData.DisplayText = FText::FromName(SpaceName);
								SpaceChannelIndex += 1;
								SpaceMetaData.SortOrder = SortOrder++;
								// Prevent single channels from collapsing to the track node
								SpaceMetaData.bCanCollapseToTrack = true;
								//TMovieSceneExternalValue<FMovieSceneControlRigSpaceBaseKey> ExternalData;
								Channels.Add(SpaceChannel->SpaceCurve, SpaceMetaData);
							}


							FParameterTransformChannelEditorData EditorData(ControlRig, Transform.ParameterName, bEnabled, TransformMask.GetChannels(), Group,
								SortOrder);

							MaybeApplyChannelMask(EditorData.MetaData[0], ControlElement, ERigControlTransformChannel::TranslationX);
							MaybeApplyChannelMask(EditorData.MetaData[1], ControlElement, ERigControlTransformChannel::TranslationY);
							MaybeApplyChannelMask(EditorData.MetaData[2], ControlElement, ERigControlTransformChannel::TranslationZ);

							Channels.Add(Transform.Translation[0], EditorData.MetaData[0], EditorData.ExternalValues[0]);
							Channels.Add(Transform.Translation[1], EditorData.MetaData[1], EditorData.ExternalValues[1]);
							Channels.Add(Transform.Translation[2], EditorData.MetaData[2], EditorData.ExternalValues[2]);

							// note the order here is different from the rotator
							MaybeApplyChannelMask(EditorData.MetaData[3], ControlElement, ERigControlTransformChannel::Roll);
							MaybeApplyChannelMask(EditorData.MetaData[4], ControlElement, ERigControlTransformChannel::Pitch);
							MaybeApplyChannelMask(EditorData.MetaData[5], ControlElement, ERigControlTransformChannel::Yaw);

							Channels.Add(Transform.Rotation[0], EditorData.MetaData[3], EditorData.ExternalValues[3]);
							Channels.Add(Transform.Rotation[1], EditorData.MetaData[4], EditorData.ExternalValues[4]);
							Channels.Add(Transform.Rotation[2], EditorData.MetaData[5], EditorData.ExternalValues[5]);

							if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
								ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
							{
								MaybeApplyChannelMask(EditorData.MetaData[6], ControlElement, ERigControlTransformChannel::ScaleX);
								MaybeApplyChannelMask(EditorData.MetaData[7], ControlElement, ERigControlTransformChannel::ScaleY);
								MaybeApplyChannelMask(EditorData.MetaData[8], ControlElement, ERigControlTransformChannel::ScaleZ);

								Channels.Add(Transform.Scale[0], EditorData.MetaData[6], EditorData.ExternalValues[6]);
								Channels.Add(Transform.Scale[1], EditorData.MetaData[7], EditorData.ExternalValues[7]);
								Channels.Add(Transform.Scale[2], EditorData.MetaData[8], EditorData.ExternalValues[8]);
								FloatChannelIndex += 9;
								SortOrder += 9;

							}
							else
							{
								FloatChannelIndex += 6;
								SortOrder += 6;

							}
							ControlIndex += 1;
							break;
						}
					}
				}
				default:
					break;
				}
#else
				switch (ControlElement->Settings.ControlType)
				{
				case ERigControlType::Float:
				{
					for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
					{
						if (ControlName == Scalar.ParameterName)
						{
							ControlChannelMap.Add(Scalar.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex));
							Channels.Add(Scalar.ParameterCurve);
							FloatChannelIndex += 1;
							SortOrder += 1;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}
				case ERigControlType::Bool:
				{
					for (FBoolParameterNameAndCurve& Bool : GetBoolParameterNamesAndCurves())
					{
						if (ControlName == Bool.ParameterName)
						{
							ControlChannelMap.Add(Bool.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, BoolChannelIndex, INDEX_NONE, NAME_None, MaskIndex));
							Channels.Add(Bool.ParameterCurve);
							BoolChannelIndex += 1;
							SortOrder += 1;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}
				case ERigControlType::Integer:
				{
					if (ControlElement->Settings.ControlEnum)
					{
						for (FEnumParameterNameAndCurve& Enum : GetEnumParameterNamesAndCurves())
						{
							if (ControlName == Enum.ParameterName)
							{
								ControlChannelMap.Add(Enum.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, EnumChannelIndex, INDEX_NONE, NAME_None, MaskIndex));
								Channels.Add(Enum.ParameterCurve);
								EnumChannelIndex += 1;
								SortOrder += 1;
								ControlIndex += 1;
								break;
							}
						}
					}
					else
					{
						for (FIntegerParameterNameAndCurve& Integer : GetIntegerParameterNamesAndCurves())
						{
							if (ControlName == Integer.ParameterName)
							{
								ControlChannelMap.Add(Integer.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, IntegerChannelIndex, INDEX_NONE, NAME_None, MaskIndex));
								Channels.Add(Integer.ParameterCurve);
								IntegerChannelIndex += 1;
								SortOrder += 1;
								ControlIndex += 1;
								break;
							}
						}
					}
					break;
				}
				case ERigControlType::Vector2D:
				{
					for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
					{
						if (ControlName == Vector2D.ParameterName)
						{
							ControlChannelMap.Add(Vector2D.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex));
							Channels.Add(Vector2D.XCurve);
							Channels.Add(Vector2D.YCurve);
							FloatChannelIndex += 2;
							SortOrder += 2;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}
				case ERigControlType::Position:
				case ERigControlType::Scale:
				case ERigControlType::Rotator:
				{
					for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
					{
						if (ControlName == Vector.ParameterName)
						{
							ControlChannelMap.Add(Vector.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex));
							bool bDoSpaceChannel = true;
							if (bDoSpaceChannel)
							{
								if (FSpaceControlNameAndChannel* SpaceChannel = GetSpaceChannel(Vector.ParameterName))
								{
									FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(Vector.ParameterName);
									if (pChannelIndex)
									{
										pChannelIndex->bDoesHaveSpace = true;
										pChannelIndex->SpaceChannelIndex = SpaceChannelIndex;
									}
									SpaceChannelIndex += 1;
									Channels.Add(SpaceChannel->SpaceCurve);
								}
							}

							Channels.Add(Vector.XCurve);
							Channels.Add(Vector.YCurve);
							Channels.Add(Vector.ZCurve);
							FloatChannelIndex += 3;
							SortOrder += 3;
							ControlIndex += 1;
							break;
						}
					}
					break;
				}
				/*
				for (FColorParameterNameAndCurves& Color : GetColorParameterNamesAndCurves())
				{
					Channels.Add(Color.RedCurve);
					Channels.Add(Color.GreenCurve);
					Channels.Add(Color.BlueCurve);
					Channels.Add(Color.AlphaCurve);
					break
				}
				*/
				case ERigControlType::TransformNoScale:
				case ERigControlType::Transform:
				case ERigControlType::EulerTransform:
				{
					for (FTransformParameterNameAndCurves& Transform : GetTransformParameterNamesAndCurves())
					{
						if (ControlName == Transform.ParameterName)
						{
							ControlChannelMap.Add(Transform.ParameterName, FChannelMapInfo(ControlIndex, SortOrder, FloatChannelIndex, INDEX_NONE, NAME_None, MaskIndex));

							bool bDoSpaceChannel = true;
							if (bDoSpaceChannel)
							{
								if (FSpaceControlNameAndChannel* SpaceChannel = GetSpaceChannel(Transform.ParameterName))
								{

									FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(Transform.ParameterName);
									if (pChannelIndex)
									{
										pChannelIndex->bDoesHaveSpace = true;
										pChannelIndex->SpaceChannelIndex = SpaceChannelIndex;
									}
									SpaceChannelIndex += 1;
									Channels.Add(SpaceChannel->SpaceCurve);
								}
							}

							Channels.Add(Transform.Translation[0]);
							Channels.Add(Transform.Translation[1]);
							Channels.Add(Transform.Translation[2]);

							Channels.Add(Transform.Rotation[0]);
							Channels.Add(Transform.Rotation[1]);
							Channels.Add(Transform.Rotation[2]);

							if (ControlElement->Settings.ControlType == ERigControlType::Transform || ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
							{
								Channels.Add(Transform.Scale[0]);
								Channels.Add(Transform.Scale[1]);
								Channels.Add(Transform.Scale[2]);
								FloatChannelIndex += 9;
								SortOrder += 9;
							}
							else
							{
								FloatChannelIndex += 6;
								SortOrder += 6;
							}

							ControlIndex += 1;
							break;
					}
				}
					break;
				}
			}
#endif
				++MaskIndex;

		}

#if WITH_EDITOR
			FMovieSceneChannelMetaData      MetaData;
			MetaData.SetIdentifiers("Weight", NSLOCTEXT("MovieSceneTransformSection", "Weight", "Weight"));
			MetaData.bEnabled = EnumHasAllFlags(TransformMask.GetChannels(), EMovieSceneTransformChannel::Weight);
			MetaData.SortOrder = 1;
			MetaData.bSortEmptyGroupsLast = false;
			MetaData.bCanCollapseToTrack = true;
			TMovieSceneExternalValue<float> ExVal;
			Channels.Add(Weight, MetaData, ExVal);
#else
			Channels.Add(Weight);

#endif
		}
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	
	return EMovieSceneChannelProxyType::Dynamic;
}

UE::MovieScene::FControlRigChannelMetaData UMovieSceneControlRigParameterSection::GetChannelMetaData(const FMovieSceneChannel* Channel) const
{
	using namespace UE::MovieScene;

	if (const FTransformParameterNameAndCurves* Transform = FindEntryWithinArrayByPtr(TransformParameterNamesAndCurves, Channel))
	{
		const FMovieSceneFloatChannel* ChannelStart = &Transform->Translation[0];

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Transform,
			Transform->ParameterName,
			static_cast<const FMovieSceneFloatChannel*>(Channel) - ChannelStart,
			EncodeControlRigEntityID(Transform - TransformParameterNamesAndCurves.GetData(), EControlRigEntityType::TransformParameter)
		);
	}

	if (const FVectorParameterNameAndCurves* Vector = FindEntryWithinArrayByPtr(VectorParameterNamesAndCurves, Channel))
	{
		const FMovieSceneFloatChannel* ChannelStart = &Vector->XCurve;

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Vector,
			Vector->ParameterName,
			static_cast<const FMovieSceneFloatChannel*>(Channel) - ChannelStart,
			EncodeControlRigEntityID(Vector - VectorParameterNamesAndCurves.GetData(), EControlRigEntityType::VectorParameter)
		);
	}

	if (const FEnumParameterNameAndCurve* Enum = FindEntryWithinArrayByPtr(EnumParameterNamesAndCurves, Channel))
	{
		check(Channel == &Enum->ParameterCurve);

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Enum,
			Enum->ParameterName,
			0,
			EncodeControlRigEntityID(Enum - EnumParameterNamesAndCurves.GetData(), EControlRigEntityType::EnumParameter)
			);
	}

	if (const FIntegerParameterNameAndCurve* Integer = FindEntryWithinArrayByPtr(IntegerParameterNamesAndCurves, Channel))
	{
		check(Channel == &Integer->ParameterCurve);

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Integer,
			Integer->ParameterName,
			0,
			EncodeControlRigEntityID(Integer - IntegerParameterNamesAndCurves.GetData(), EControlRigEntityType::IntegerParameter)
		);
	}

	if (const FSpaceControlNameAndChannel* Space = FindEntryWithinArrayByPtr(SpaceChannels, Channel))
	{
		check(Channel == &Space->SpaceCurve);

		return FControlRigChannelMetaData(
			EControlRigControlType::Space,
			Space->ControlName,
			0,
			EncodeControlRigEntityID(Space - SpaceChannels.GetData(), EControlRigEntityType::Space)
		);
	}

	if (const FBoolParameterNameAndCurve* Bool = FindEntryWithinArrayByPtr(BoolParameterNamesAndCurves, Channel))
	{
		check(Channel == &Bool->ParameterCurve);

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Bool,
			Bool->ParameterName,
			0,
			EncodeControlRigEntityID(Bool - BoolParameterNamesAndCurves.GetData(), EControlRigEntityType::BoolParameter)
		);
	}

	if (const FScalarParameterNameAndCurve* Scalar = FindEntryWithinArrayByPtr(ScalarParameterNamesAndCurves, Channel))
	{
		check(Channel == &Scalar->ParameterCurve);

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Scalar,
			Scalar->ParameterName,
			0,
			EncodeControlRigEntityID(Scalar - ScalarParameterNamesAndCurves.GetData(), EControlRigEntityType::ScalarParameter)
		);
	}

	if (const FVector2DParameterNameAndCurves* Vector2 = FindEntryWithinArrayByPtr(Vector2DParameterNamesAndCurves, Channel))
	{
		const FMovieSceneFloatChannel* ChannelStart = &Vector2->XCurve;

		return FControlRigChannelMetaData(
			EControlRigControlType::Parameter_Vector,
			Vector2->ParameterName,
			static_cast<const FMovieSceneFloatChannel*>(Channel) - ChannelStart,
			EncodeControlRigEntityID(Vector2 - Vector2DParameterNamesAndCurves.GetData(), EControlRigEntityType::VectorParameter)
		);
	}

	// @todo: Are colors even supported??
	// if (const FColorParameterNameAndCurves* Color = FindEntryWithinArrayByPtr(ColorParameterNamesAndCurves, Channel))
	// {
	// }

	return FControlRigChannelMetaData();
}

FMovieSceneInterrogationKey UMovieSceneControlRigParameterSection::GetFloatInterrogationKey()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

FMovieSceneInterrogationKey UMovieSceneControlRigParameterSection::GetVector2DInterrogationKey()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

FMovieSceneInterrogationKey UMovieSceneControlRigParameterSection::GetVectorInterrogationKey()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

FMovieSceneInterrogationKey UMovieSceneControlRigParameterSection::GetVector4InterrogationKey()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

FMovieSceneInterrogationKey UMovieSceneControlRigParameterSection::GetTransformInterrogationKey()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

float UMovieSceneControlRigParameterSection::GetTotalWeightValue(FFrameTime InTime) const
{
	float WeightVal = EvaluateEasing(InTime);
	if (EnumHasAllFlags(TransformMask.GetChannels(), EMovieSceneTransformChannel::Weight))
	{
		float ManualWeightVal = 1.f;
		Weight.Evaluate(InTime, ManualWeightVal);
		WeightVal *= ManualWeightVal;
	}
	return WeightVal;
}

void UMovieSceneControlRigParameterSection::KeyZeroValue(FFrameNumber InFrame, EMovieSceneKeyInterpolation DefaultInterpolation, bool bSelectedControls)
{
	TArray<FName> SelectedControls;
	if (bSelectedControls && ControlRig)
	{
		SelectedControls = ControlRig->CurrentControlSelection();
	}
	/* Don't set zero values on these doesn't make sense
	
	for (FBoolParameterNameAndCurve& Bool : GetBoolParameterNamesAndCurves())
	for (FEnumParameterNameAndCurve& Enum : GetEnumParameterNamesAndCurves())
	for (FIntegerParameterNameAndCurve& Integer : GetIntegerParameterNamesAndCurves())

	*/
	for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
	{
		if (SelectedControls.Num() == 0 || SelectedControls.Contains(Scalar.ParameterName))
		{
			AddKeyToChannel(&Scalar.ParameterCurve, InFrame, 0.0f, DefaultInterpolation);
			Scalar.ParameterCurve.AutoSetTangents();
		}
	}
	for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
	{
		if (SelectedControls.Num() == 0 || SelectedControls.Contains(Vector2D.ParameterName))
		{
			AddKeyToChannel(&Vector2D.XCurve, InFrame, 0.0f, DefaultInterpolation);
			Vector2D.XCurve.AutoSetTangents();
			AddKeyToChannel(&Vector2D.YCurve, InFrame, 0.0f, DefaultInterpolation);
			Vector2D.YCurve.AutoSetTangents();
		}
	}
	for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
	{
		if (SelectedControls.Num() == 0 || SelectedControls.Contains(Vector.ParameterName))
		{
			AddKeyToChannel(&Vector.XCurve, InFrame, 0.0f, DefaultInterpolation);
			Vector.XCurve.AutoSetTangents();
			AddKeyToChannel(&Vector.YCurve, InFrame, 0.0f, DefaultInterpolation);
			Vector.YCurve.AutoSetTangents();
			AddKeyToChannel(&Vector.ZCurve, InFrame, 0.0f, DefaultInterpolation);
			Vector.ZCurve.AutoSetTangents();
		}
	}
	for (FTransformParameterNameAndCurves& Transform : GetTransformParameterNamesAndCurves())
	{
		if (SelectedControls.Num() == 0 || SelectedControls.Contains(Transform.ParameterName))
		{
			for (int32 Index = 0; Index < 3; ++Index)
			{
				AddKeyToChannel(&Transform.Translation[Index], InFrame, 0.0f, DefaultInterpolation);
				Transform.Translation[Index].AutoSetTangents();
				AddKeyToChannel(&Transform.Rotation[Index], InFrame, 0.0f, DefaultInterpolation);
				Transform.Rotation[Index].AutoSetTangents();
				if (GetBlendType() == EMovieSceneBlendType::Additive)
				{
					AddKeyToChannel(&Transform.Scale[Index], InFrame, 0.0f, DefaultInterpolation);
				}
				else
				{
					AddKeyToChannel(&Transform.Scale[Index], InFrame, 1.0f, DefaultInterpolation);
				}
				Transform.Scale[Index].AutoSetTangents();

			}
		}
	}
}

void UMovieSceneControlRigParameterSection::KeyWeightValue(FFrameNumber InFrame, EMovieSceneKeyInterpolation DefaultInterpolation, float InVal)
{
	AddKeyToChannel(&Weight, InFrame, InVal, DefaultInterpolation);
	Weight.AutoSetTangents();
}

bool UMovieSceneControlRigParameterSection::RenameParameterName(const FName& OldParameterName, const FName& NewParameterName, TOptional<ERigControlType> ControlType)
{
	bool bWasReplaced = false;

	auto RenameParameterNameInner = [this, &bWasReplaced, OldParameterName, NewParameterName](auto& ParameterNamesAndCurves)
		{
			for (auto& ParameterNameAndCurve : ParameterNamesAndCurves)
			{
				if (ParameterNameAndCurve.ParameterName == OldParameterName)
				{
					if (!bWasReplaced)
					{
						Modify();
						bWasReplaced = true;
					}
					ParameterNameAndCurve.ParameterName = NewParameterName;
					break;
				}
			}
		};
	if (ControlType.IsSet())
	{
		switch (ControlType.GetValue())
		{
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			RenameParameterNameInner(ScalarParameterNamesAndCurves);
			break;
		}
		case ERigControlType::Bool:
		{
			RenameParameterNameInner(BoolParameterNamesAndCurves);
			break;
		}
		case ERigControlType::Integer:
		{
			RenameParameterNameInner(IntegerParameterNamesAndCurves);
			RenameParameterNameInner(EnumParameterNamesAndCurves);
			break;
		}
		case ERigControlType::Vector2D:
		{
			RenameParameterNameInner(Vector2DParameterNamesAndCurves);
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Rotator:
		case ERigControlType::Scale:
		{
			RenameParameterNameInner(VectorParameterNamesAndCurves);
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::EulerTransform:
		case ERigControlType::TransformNoScale:
		{
			RenameParameterNameInner(TransformParameterNamesAndCurves);
			break;
		}

		};
	}
	else
	{
		RenameParameterNameInner(ScalarParameterNamesAndCurves);
		RenameParameterNameInner(BoolParameterNamesAndCurves);
		RenameParameterNameInner(EnumParameterNamesAndCurves);
		RenameParameterNameInner(IntegerParameterNamesAndCurves);
		RenameParameterNameInner(Vector2DParameterNamesAndCurves);
		RenameParameterNameInner(VectorParameterNamesAndCurves);
		RenameParameterNameInner(ColorParameterNamesAndCurves);
		RenameParameterNameInner(TransformParameterNamesAndCurves);
	}
	for (FSpaceControlNameAndChannel& SpaceChannel : SpaceChannels)
	{
		if (SpaceChannel.ControlName == OldParameterName)
		{
			SpaceChannel.ControlName = NewParameterName;
			bWasReplaced = true;
		}
		//update spaces
		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = SpaceChannel.SpaceCurve.GetData();
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		TArrayView<FMovieSceneControlRigSpaceBaseKey> Values = ChannelData.GetValues();
		for (FMovieSceneControlRigSpaceBaseKey& Key : Values)
		{
			if (Key.ControlRigElement.Name == OldParameterName)
			{
				Key.ControlRigElement.Name = NewParameterName;
			}
		}
	}
	for (FConstraintAndActiveChannel& ConstraintChannel : ConstraintsChannels)
	{
		if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
		{
			if (UTransformableControlHandle* Handle = Cast<UTransformableControlHandle>(TransformConstraint->ChildTRSHandle))
			{
				if (Handle->ControlName == OldParameterName)
				{
					Handle->ControlName = NewParameterName;
					bWasReplaced = true;
				}
			}
		}
	}
	if (bWasReplaced)
	{
		ReconstructChannelProxy();
	}
	return bWasReplaced;
}

#if WITH_EDITOR

void UMovieSceneControlRigParameterSection::OnControlRigEditorSettingChanged(UObject* InSettingsChanged, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	if(InPropertyChangedEvent.Property)
	{
		if(InPropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, ElementNameDisplayMode))
		{
			ReconstructChannelProxy();
		}
	}
}

void UMovieSceneControlRigParameterSection::BindEditorDelegates()
{
	OnControlRigEditorSettingChangedHandle = UControlRigEditorSettings::Get()->OnSettingChanged().AddUObject(this, &UMovieSceneControlRigParameterSection::OnControlRigEditorSettingChanged);
}

void UMovieSceneControlRigParameterSection::UnbindEditorDelegates()
{
	UControlRigEditorSettings::Get()->OnSettingChanged().Remove(OnControlRigEditorSettingChangedHandle);
}
#endif

void UMovieSceneControlRigParameterSection::RecreateWithThisControlRig(UControlRig* InControlRig, bool bSetDefault)
{
	bool bSameControlRig = (ControlRig == InControlRig);
	SetControlRig(InControlRig);
	/* Don't delete old tracks but eventually show that they aren't associated.. but
	then how to delete?
	BoolParameterNamesAndCurves.Empty();
	EnumParameterNamesAndCurves.Empty();
	IntegerParameterNamesAndCurves.Empty();
	ScalarParameterNamesAndCurves.Empty();
	Vector2DParameterNamesAndCurves.Empty();
	VectorParameterNamesAndCurves.Empty();
	ColorParameterNamesAndCurves.Empty();
	TransformParameterNamesAndCurves.Empty();
	*/

	//update the mask array to the new mask name set
	//need to do it here since we won't get controls until here
	const int32 NumControls = ControlRig->AvailableControls().Num();
	const int32 MaskNum = ControlsMask.Num();
	if (NumControls > 0 && NumControls == MaskNum)
	{
		ConvertMaskArrayToNameSet();
	}
	/*
	//if we had the same with same number of controls keep the mask otherwise reset it.

	if (!bSameControlRig || (NumControls > 0  && (NumControls != MaskNum)))
	{
		TArray<bool> OnArray;
		OnArray.Init(true, ControlRig->AvailableControls().Num());
		SetControlsMask(OnArray);
	}
	*/
	
	TArray<FRigControlElement*> SortedControls;
	ControlRig->GetControlsInOrder(SortedControls);

	TMap<FName, FName> CurveControlNameRemapping;
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::FKControlNamingScheme)
	{
		for (FRigControlElement* ControlElement : SortedControls)
		{
			if (ControlElement->Settings.ControlType == ERigControlType::Float)
			{
				const FName TargetCurveName = UFKControlRig::GetControlTargetName(ControlElement->GetFName(), ERigElementType::Curve);
				const FRigElementKey CurveKey = FRigElementKey(TargetCurveName, ERigElementType::Curve);
				// Ensure name is valid, and curve actually exists in the hierarchy,
				// this means we could not be renaming some controls for which the curves do not exist anymore, which ties back to comment at the top op the function
				// with regards to non-associated curves
				if (TargetCurveName != NAME_None && Hierarchy->Find(CurveKey))
				{
					// Add mapping from old to new control naming scheme (previous was using uniform naming for both bones and curves)
					CurveControlNameRemapping.Add(ControlElement->GetFName(), UFKControlRig::GetControlName(TargetCurveName, ERigElementType::Bone));
				}
			}
		}
	}
	
	// rename all existing parameters based on short name vs long name.
	// this also recovers from controls being stored with an original long path
	// which may now be outdated. UControlRig::FindControl has backwards compat for that.
	if(InControlRig->IsModularRig())
	{
		if (URigHierarchy* RigHierarchy = InControlRig->GetHierarchy())
		{
			const FRigHierarchyNameSpaceWarningBracket SuspendNameSpaceWarnings(RigHierarchy); 
			
			ForEachParameter([this, InControlRig](FBaseParameterNameAndValue* InOutParameter)
			{
				if(const FRigControlElement* Control = InControlRig->FindControl(InOutParameter->ParameterName))
				{
					const FName& ControlName = Control->GetFName();
					if(ControlName != InOutParameter->ParameterName)
					{
						InOutParameter->ParameterName = ControlName;
					}
				}
			});
		}
	}

	//also need update Modular Rig FNames on the transformable handles on the constraint channels for those that are constraints. Update both child and parent
	auto UpdateHandleName = [](UTransformableControlHandle* Handle)
	{
		if (Handle)
		{
			if (UControlRig* HandleControlRig = Handle->ControlRig.Get())
			{
				if (HandleControlRig->IsModularRig())
				{
					if (URigHierarchy* RigHierarchy = HandleControlRig->GetHierarchy())
					{
						const FRigHierarchyNameSpaceWarningBracket SuspendNameSpaceWarnings(RigHierarchy); 
						if (const FRigControlElement* Control = HandleControlRig->FindControl(Handle->ControlName))
						{
							const FName& ControlName = Control->GetFName();
							if (ControlName != Handle->ControlName)
							{
								Handle->ControlName = ControlName;
							}
						}
					}
				}
			}
		}
	};

	for (FConstraintAndActiveChannel& ConstraintChannel : ConstraintsChannels)
	{
		if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
		{
			if (UTransformableControlHandle* Handle = Cast<UTransformableControlHandle>(TransformConstraint->ChildTRSHandle))
			{
				UpdateHandleName(Handle);
			}
			if (UTransformableControlHandle* Handle = Cast<UTransformableControlHandle>(TransformConstraint->ParentTRSHandle))
			{
				UpdateHandleName(Handle);
			}
		}
	}

	for (FRigControlElement* ControlElement : SortedControls)
	{
		if (!Hierarchy->IsAnimatable(ControlElement))
		{
			continue;
		}

		FName PreviousName = Hierarchy->GetPreviousName(ControlElement->GetKey());
		if (PreviousName != NAME_None && PreviousName != ControlElement->GetKey().Name)
		{
			TOptional<ERigControlType> ControlType = ControlElement->Settings.ControlType;
			RenameParameterName(PreviousName, ControlElement->GetKey().Name, ControlType);
		}
		if (ControlElement->Settings.ControlType == ERigControlType::Float ||
			ControlElement->Settings.ControlType == ERigControlType::ScaleFloat)
		{
			if (const FName* OldCurveControlName = CurveControlNameRemapping.Find(ControlElement->GetFName()))
			{
				TOptional<ERigControlType> ControlType = ControlElement->Settings.ControlType;
				RenameParameterName(*OldCurveControlName, ControlElement->GetKey().Name, ControlType);
			}
		}

		const FName& ControlName = ControlElement->GetFName();

		switch (ControlElement->Settings.ControlType)
		{
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			TOptional<float> DefaultValue;
			if (bSetDefault)
			{
				//or use IntialValue?
				DefaultValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
			}
			AddScalarParameter(ControlName, DefaultValue, false);
			break;
		}
		case ERigControlType::Bool:
		{
			TOptional<bool> DefaultValue;
			//only add bools,int, enums and space onto first sections, which is the same as the default one
			if (bSetDefault)
			{
				DefaultValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<bool>();
				AddBoolParameter(ControlName, DefaultValue, false);

			}
			break;
		}
		case ERigControlType::Integer:
		{
			if (ControlElement->Settings.ControlEnum)
			{
				TOptional<uint8> DefaultValue;
				//only add bools,int, enums and space onto first sections, which is the same as the default one
				if (bSetDefault)
				{
					DefaultValue = (uint8)ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int32>();
					AddEnumParameter(ControlName, ControlElement->Settings.ControlEnum, DefaultValue, false);
				}
			}
			else
			{
				TOptional<int32> DefaultValue;
				//only add bools,int, enums and space onto first sections, which is the same as the default one
				if (bSetDefault)
				{
					DefaultValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int32>();
					AddIntegerParameter(ControlName, DefaultValue, false);
				}
			}
			break;
		}
		case ERigControlType::Vector2D:
		{
			TOptional<FVector2D> DefaultValue;
			if (bSetDefault)
			{
				//or use IntialValue?
				const FVector3f TempValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
				DefaultValue = FVector2D(TempValue.X, TempValue.Y);
			}
			AddVector2DParameter(ControlName, DefaultValue, false);
			break;
		}

		case ERigControlType::Position:
		case ERigControlType::Scale:
		case ERigControlType::Rotator:
		{
			TOptional<FVector> DefaultValue;
			if (bSetDefault)
			{
				//or use IntialValue?
				DefaultValue = (FVector)ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
			}
			AddVectorParameter(ControlName, DefaultValue, false);
			//mz todo specify rotator special so we can do quat interps
			break;
		}
		case ERigControlType::EulerTransform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::Transform:
		{
			TOptional<FEulerTransform> DefaultValue;
			if (bSetDefault)
			{
				if (ControlElement->Settings.ControlType == ERigControlType::Transform)
				{
					DefaultValue = FEulerTransform(
						ControlRig->
						GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>().ToTransform());
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)

				{
					FEulerTransform Euler = 
						ControlRig->
						GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
					DefaultValue = Euler;
				}
				else
				{
					FTransformNoScale NoScale = 
						ControlRig->
						GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
					DefaultValue = FEulerTransform(NoScale.Rotation.Rotator(), NoScale.Location, FVector::OneVector);
				}
			}
			AddTransformParameter(ControlName, DefaultValue, false);
			break;
		}

		default:
			break;
		}
	}
	ReconstructChannelProxy();
}

void UMovieSceneControlRigParameterSection::SetControlRig(UControlRig* InControlRig)
{
	if(ControlRig)
	{
		ControlRig->OnOverrideAssetsChanged().Remove(OnOverrideAssetsChangedHandle);
#if WITH_EDITORONLY_DATA
		for(const TSoftObjectPtr<UControlRigOverrideAsset>& OverrideAssetPtr : OverrideAssets)
		{
			if(UControlRigOverrideAsset* OverrideAsset = OverrideAssetPtr.Get())
			{
				ControlRig->UnlinkOverrideAsset(OverrideAsset);
			}
		}
#endif
	}
	
	ControlRig = InControlRig;
	ControlRigAssetReference = ControlRig ? ControlRig->GetAssetReference() : nullptr;

	if(ControlRig)
	{
#if WITH_EDITORONLY_DATA
		for(const TSoftObjectPtr<UControlRigOverrideAsset>& OverrideAssetPtr : OverrideAssets)
		{
			if(UControlRigOverrideAsset* OverrideAsset = OverrideAssetPtr.LoadSynchronous())
			{
				ControlRig->LinkOverrideAsset(OverrideAsset);
			}
		}
#endif
		TWeakObjectPtr<UMovieSceneControlRigParameterSection> WeakThis(this);
		OnOverrideAssetsChangedHandle = ControlRig->OnOverrideAssetsChanged().AddLambda(
			[WeakThis](UControlRig* InControlRig)
			{
				if(WeakThis.IsValid())
				{
					WeakThis->HandleOverrideAssetsChanged(InControlRig);
				}
			}
		);
	}
}

void UMovieSceneControlRigParameterSection::ChangeControlRotationOrder(const FName& InControlName, const  TOptional<EEulerRotationOrder>& OldOrder,
                                                                       const  TOptional<EEulerRotationOrder>& NewOrder, EMovieSceneKeyInterpolation Interpolation)
{
	//this will call ReconstructChannelProxy which is needed before accessing the ControlChannelMap
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

	FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(InControlName);
	if (pChannelIndex == nullptr || GetControlRig() == nullptr)
	{
		return;
	}

	int32 ChannelIndex = pChannelIndex->ChannelIndex;
	if (FRigControlElement* ControlElement = GetControlRig()->FindControl(InControlName))
	{
		if (ControlElement->Settings.ControlType == ERigControlType::Rotator ||
			ControlElement->Settings.ControlType == ERigControlType::EulerTransform ||
			ControlElement->Settings.ControlType == ERigControlType::Transform ||
			ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
		{
			const int32 StartIndex = (ControlElement->Settings.ControlType == ERigControlType::Rotator) ? 0 : 3;
			const int32 XIndex = StartIndex + ChannelIndex;
			const int32 YIndex = XIndex + 1;
			const int32 ZIndex = XIndex + 2;
			
			if (!(FloatChannels.IsValidIndex(XIndex) && FloatChannels.IsValidIndex(YIndex) && FloatChannels.IsValidIndex(ZIndex)))
			{
				return;
			}
			
			auto AddArrayToSortedMap = [](const TArray<FFrameNumber>& InFrames, TSortedMap<FFrameNumber, FFrameNumber>& OutFrameMap)
			{
				for (const FFrameNumber& Frame : InFrames)
				{
					OutFrameMap.Add(Frame, Frame);
				}
			};

			TSortedMap<FFrameNumber, FFrameNumber> AllKeys;
			TArray<FFrameNumber> KeyTimes;
			TArray<FKeyHandle> Handles;
			for (int32 Index = XIndex; Index < XIndex + 3; ++Index)
			{
				KeyTimes.SetNum(0);
				Handles.SetNum(0);
				FloatChannels[Index]->GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
				AddArrayToSortedMap(KeyTimes, AllKeys);
			}
			KeyTimes.SetNum(0);
			AllKeys.GenerateKeyArray(KeyTimes);
			
			auto UpdateRotator = [&OldOrder, &NewOrder](FRotator& InOutRotator)
			{
				constexpr bool bUseUEHandiness = true;
				FQuat Quat;
				
				//if set use animation core conversion, else use rotator conversion
				if (OldOrder.IsSet())
				{
					const FVector Vector = InOutRotator.Euler();
					Quat = AnimationCore::QuatFromEuler(Vector, OldOrder.GetValue(), bUseUEHandiness);
				}
				else
				{
					Quat = FQuat(InOutRotator);
				}
				
				if (NewOrder.IsSet())
				{
					const FVector Vector = AnimationCore::EulerFromQuat(Quat, NewOrder.GetValue(), bUseUEHandiness);
					InOutRotator = FRotator::MakeFromEuler(Vector);
				}
				else
				{
					InOutRotator = FRotator(Quat);
				}
			};
			
			// no keys, update defaults
			if (KeyTimes.IsEmpty())
			{
				const TOptional<float>& OptRoll = FloatChannels[XIndex]->GetDefault();
				const TOptional<float>& OptPitch = FloatChannels[YIndex]->GetDefault();
				const TOptional<float>& OptYaw = FloatChannels[ZIndex]->GetDefault();
				if (OptRoll || OptPitch || OptYaw)
				{
					FRotator Rotator(OptPitch ? *OptPitch : 0.f, OptYaw ? *OptYaw : 0.f, OptRoll ? *OptRoll : 0.f);
					UpdateRotator(Rotator);
					
					FloatChannels[XIndex]->SetDefault(Rotator.Roll);
					FloatChannels[YIndex]->SetDefault(Rotator.Pitch);
					FloatChannels[ZIndex]->SetDefault(Rotator.Yaw);
				}
				return;
			}
			FRotator Rotator(0.0f, 0.0f, 0.0f);
			const FFrameNumber StartFrame = KeyTimes[0];
			const FFrameNumber EndFrame = KeyTimes[KeyTimes.Num() - 1];
			for (const FFrameNumber& Frame : KeyTimes)
			{
				float Roll = 0.0f, Pitch = 0.0f, Yaw = 0.0f;
				FloatChannels[XIndex]->Evaluate(Frame, Roll);
				FloatChannels[YIndex]->Evaluate(Frame, Pitch);
				FloatChannels[ZIndex]->Evaluate(Frame, Yaw);
				Rotator = FRotator(Pitch, Yaw, Roll);
				
				UpdateRotator(Rotator);
				
				//this will reuse tangent like we want and only add if new
				AddKeyToChannel(FloatChannels[XIndex], Frame, Rotator.Roll, Interpolation);
				AddKeyToChannel(FloatChannels[YIndex], Frame, Rotator.Pitch, Interpolation);
				AddKeyToChannel(FloatChannels[ZIndex], Frame, Rotator.Yaw, Interpolation);
			}
			FixRotationWinding(InControlName, StartFrame, EndFrame);
		}
	}
}

void UMovieSceneControlRigParameterSection::ConvertMaskArrayToNameSet()
{
	if (ControlRig && ControlsMask.Num() > 0)
	{
		TArray<FRigControlElement*> SortedControls;
		ControlRig->GetControlsInOrder(SortedControls);
		if (SortedControls.Num() == ControlsMask.Num())
		{
			ControlNameMask.Empty();
			for (int32 Index = 0; Index < SortedControls.Num(); ++Index)
			{
				if (ControlsMask[Index] == false)
				{
					ControlNameMask.Add(SortedControls[Index]->GetKey().Name);
				}
			}
		}
		//empty ControlsMask, no longer needed
		ControlsMask.Empty();
	}
}

void UMovieSceneControlRigParameterSection::FillControlNameMask(bool bValue)
{
	if (ControlRig)
	{
		ControlNameMask.Empty();
		if (bValue == false)
		{
			TArray<FRigControlElement*> SortedControls;
			ControlRig->GetControlsInOrder(SortedControls);
			for (FRigControlElement* ControlElement : SortedControls)
			{
				if (ControlElement)
				{
					ControlNameMask.Add(ControlElement->GetKey().Name);
				}
			}
		}
		ReconstructChannelProxy();
	}
}

void UMovieSceneControlRigParameterSection::SetControlNameMask(const FName& Name, bool bValue, bool bReconstructChannel)
{
	if (bValue == false)
	{
		ControlNameMask.Add(Name);
	}
	else
	{
		ControlNameMask.Remove(Name);
	}
	if (bReconstructChannel)
	{
		ReconstructChannelProxy();
	}
}

//get value, will return false if not found
bool UMovieSceneControlRigParameterSection::GetControlNameMask(const FName& Name) const
{
	return (ControlNameMask.Find(Name) == nullptr);
}

void UMovieSceneControlRigParameterSection::FixRotationWinding(const FName& ControlName, FFrameNumber StartFrame, FFrameNumber EndFrame)
{
	FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ControlName);
	if (pChannelIndex == nullptr || GetControlRig() == nullptr)
	{
		return;
	}
	int32 ChannelIndex = pChannelIndex->ChannelIndex;
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	if (FRigControlElement* ControlElement = GetControlRig()->FindControl(ControlName))
	{
		if (ControlElement->Settings.ControlType == ERigControlType::Rotator  ||
			ControlElement->Settings.ControlType == ERigControlType::EulerTransform ||
			ControlElement->Settings.ControlType == ERigControlType::Transform || 
			ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
		{
			int32 StartIndex = (ControlElement->Settings.ControlType == ERigControlType::Rotator) ? 0 : 3;
			for (int32 Index = 0; Index < 3; ++Index)
			{
				int32 RealIndex = StartIndex + Index + ChannelIndex;
				int32 NumKeys = FloatChannels[RealIndex]->GetNumKeys();
				bool bDidFrame = false;
				float PrevVal = 0.0f;
				for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
				{
					const FFrameNumber Frame = FloatChannels[RealIndex]->GetData().GetTimes()[KeyIndex];
					if (Frame >= StartFrame && Frame <= EndFrame)
					{
						FMovieSceneFloatValue Val = FloatChannels[RealIndex]->GetData().GetValues()[KeyIndex];
						if (bDidFrame == true)
						{
							FMath::WindRelativeAnglesDegrees(PrevVal, Val.Value);
							FloatChannels[RealIndex]->GetData().GetValues()[KeyIndex].Value = Val.Value;
						}
						else
						{
							bDidFrame = true;
						}
						PrevVal = Val.Value;
					}
					
				}
				FloatChannels[RealIndex]->AutoSetTangents();
			}
		}
	}
}

void UMovieSceneControlRigParameterSection::OptimizeSection(const FName& ControlName, const FKeyDataOptimizationParams& Params)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	TArrayView<FMovieSceneBoolChannel*> BoolChannels = GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
	TArrayView<FMovieSceneIntegerChannel*> IntegerChannels = GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
	TArrayView<FMovieSceneByteChannel*> EnumChannels = GetChannelProxy().GetChannels<FMovieSceneByteChannel>();
	FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ControlName);
	if (pChannelIndex != nullptr)
	{
		int32 ChannelIndex = pChannelIndex->ChannelIndex;

		if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
		{
			switch (ControlElement->Settings.ControlType)
			{
				case ERigControlType::Position:
				case ERigControlType::Scale:
				case ERigControlType::Rotator:
				{
					FloatChannels[ChannelIndex]->Optimize(Params);
					FloatChannels[ChannelIndex + 1]->Optimize(Params);
					FloatChannels[ChannelIndex + 2]->Optimize(Params);
					break;
				}

				case ERigControlType::Transform:
				case ERigControlType::TransformNoScale:
				case ERigControlType::EulerTransform:
				{

					FloatChannels[ChannelIndex]->Optimize(Params);
					FloatChannels[ChannelIndex + 1]->Optimize(Params);
					FloatChannels[ChannelIndex + 2]->Optimize(Params);
					FloatChannels[ChannelIndex + 3]->Optimize(Params);
					FloatChannels[ChannelIndex + 4]->Optimize(Params);
					FloatChannels[ChannelIndex + 5]->Optimize(Params);

					if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
						ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						FloatChannels[ChannelIndex + 6]->Optimize(Params);
						FloatChannels[ChannelIndex + 7]->Optimize(Params);
						FloatChannels[ChannelIndex + 8]->Optimize(Params);
					}
					break;

				}
				case ERigControlType::Bool:
				{
					BoolChannels[ChannelIndex]->Optimize(Params);
					break;
				}
				case ERigControlType::Integer:
				{
					if (ControlElement->Settings.ControlEnum)
					{
						EnumChannels[ChannelIndex]->Optimize(Params);
					}
					else
					{
						IntegerChannels[ChannelIndex]->Optimize(Params);
					}
					break;
				}
				default:
					break;
			}
		}
	}
}

void UMovieSceneControlRigParameterSection::AutoSetTangents(const FName& ControlName)
{
	TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ControlName);
	if (pChannelIndex != nullptr)
	{
		int32 ChannelIndex = pChannelIndex->ChannelIndex;

		if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
		{
			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Position:
			case ERigControlType::Scale:
			case ERigControlType::Rotator:
			{
				FloatChannels[ChannelIndex]->AutoSetTangents();
				FloatChannels[ChannelIndex + 1]->AutoSetTangents();
				FloatChannels[ChannelIndex + 2]->AutoSetTangents();
				break;
			}

			case ERigControlType::Transform:
			case ERigControlType::TransformNoScale:
			case ERigControlType::EulerTransform:
			{

				FloatChannels[ChannelIndex]->AutoSetTangents();
				FloatChannels[ChannelIndex + 1]->AutoSetTangents();
				FloatChannels[ChannelIndex + 2]->AutoSetTangents();
				FloatChannels[ChannelIndex + 3]->AutoSetTangents();
				FloatChannels[ChannelIndex + 4]->AutoSetTangents();
				FloatChannels[ChannelIndex + 5]->AutoSetTangents();

				if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
					ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					FloatChannels[ChannelIndex + 6]->AutoSetTangents();
					FloatChannels[ChannelIndex + 7]->AutoSetTangents();
					FloatChannels[ChannelIndex + 8]->AutoSetTangents();
				}
				break;

			}
			default:
				break;
			}
		}
	}
}
#if WITH_EDITOR
namespace UE::Private
{
	void SetOrAddKey(TMovieSceneChannelData<FMovieSceneFloatValue>& ChannelData, FFrameNumber Time, float Value, const EMovieSceneKeyInterpolation Interpolation)
	{
		int32 ExistingIndex = ChannelData.FindKey(Time);
		if (ExistingIndex != INDEX_NONE)
		{
			FMovieSceneFloatValue& FloatValue = ChannelData.GetValues()[ExistingIndex];
			FloatValue.Value = Value;
		}
		else
		{
			FMovieSceneFloatValue NewKey(Value);
			ERichCurveTangentWeightMode WeightedMode = RCTWM_WeightedNone;
			NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
			NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			NewKey.Tangent.ArriveTangent = 0.0f;
			NewKey.Tangent.LeaveTangent = 0.0f;
			NewKey.Tangent.TangentWeightMode = WeightedMode;
			NewKey.Tangent.ArriveTangentWeight = 0.0f;
			NewKey.Tangent.LeaveTangentWeight = 0.0f;

			switch (Interpolation)
			{
			case EMovieSceneKeyInterpolation::SmartAuto:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_SmartAuto;
			}
			break;
			case EMovieSceneKeyInterpolation::Auto:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			break;
			case EMovieSceneKeyInterpolation::User:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_User;
			}
			break;
			case EMovieSceneKeyInterpolation::Break:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Cubic;
				NewKey.TangentMode = ERichCurveTangentMode::RCTM_Auto;
			}
			break;

			case EMovieSceneKeyInterpolation::Linear:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Linear;
			}
			break;

			case EMovieSceneKeyInterpolation::Constant:
			{
				NewKey.InterpMode = ERichCurveInterpMode::RCIM_Constant;
			}
			break;

			}
			ChannelData.AddKey(Time, NewKey);
		}
	}
}

void UMovieSceneControlRigParameterSection::RecordControlRigKey(FFrameNumber FrameNumber, bool bSetDefault, EMovieSceneKeyInterpolation InInterpMode, bool bbOntoSelectedControls)
{
	if (ControlRig && ControlRig->GetHierarchy())
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		TArrayView<FMovieSceneBoolChannel*> BoolChannels = GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
		TArrayView<FMovieSceneIntegerChannel*> IntChannels = GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
		TArrayView<FMovieSceneByteChannel*> EnumChannels = GetChannelProxy().GetChannels<FMovieSceneByteChannel>();

		// Helper lambda to add a FVector key to the float channels
		auto AddVectorKeyToFloatChannels = [&FloatChannels, InInterpMode](int32& ChannelIndex, FFrameNumber FrameNumber, const FVector& Value)
		{
			TMovieSceneChannelData<FMovieSceneFloatValue> ChannelDataX = FloatChannels[ChannelIndex++]->GetData();
			UE::Private::SetOrAddKey(ChannelDataX, FrameNumber, Value.X, InInterpMode);
			TMovieSceneChannelData<FMovieSceneFloatValue> ChannelDataY = FloatChannels[ChannelIndex++]->GetData();
			UE::Private::SetOrAddKey(ChannelDataY, FrameNumber, Value.Y, InInterpMode);
			TMovieSceneChannelData<FMovieSceneFloatValue> ChannelDataZ = FloatChannels[ChannelIndex++]->GetData();
			UE::Private::SetOrAddKey(ChannelDataZ, FrameNumber, Value.Z, InInterpMode);
		};
			
		auto WindRotation = [&FloatChannels](FVector& InOutRotation, const int32 ChannelIndex, const uint8 Axis)
		{
			if ((Axis < 3) && FloatChannels.IsValidIndex(ChannelIndex+Axis))
			{
				FMovieSceneFloatChannel* RotChannel = FloatChannels[ChannelIndex+Axis];
				const int32 NumKeys = RotChannel->GetNumKeys();
				if (NumKeys > 0)
				{
					const float LastVal = RotChannel->GetValues().Last().Value;
					FMath::WindRelativeAnglesDegrees(LastVal, InOutRotation[Axis]);
				}
			}
		};

		TArray<FRigControlElement*> Controls;
		ControlRig->GetControlsInOrder(Controls);
		
		//if additive zero out scale
		const bool bIsAdditive = GetBlendType() == EMovieSceneBlendType::Additive;
		TArray<FName> SelectedControls;
		if (bbOntoSelectedControls)
		{
			SelectedControls = ControlRig->CurrentControlSelection();
		}

		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		
		for (FRigControlElement* ControlElement : Controls)
		{
			if (!Hierarchy->IsAnimatable(ControlElement))
			{
				continue;
			}
			
			FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ControlElement->GetFName());
			if (pChannelIndex == nullptr)
			{
				continue;
			}
			
			//if masked out don't do
			if (GetControlNameMask(ControlElement->GetFName()) == false)
			{
				continue;
			}

			if (bbOntoSelectedControls)
			{
				if (SelectedControls.Contains(ControlElement->GetFName()) == false)
				{
					continue;
				}
			}
			int32 ChannelIndex = pChannelIndex->ChannelIndex;

			const FRigControlValue CurrentValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);
			
			switch (ControlElement->Settings.ControlType)
			{
				case ERigControlType::Bool:
				{
					bool Val = CurrentValue.Get<bool>();
					if (bSetDefault)
					{
						BoolChannels[ChannelIndex]->SetDefault(Val);
					}
					int32 ExistingIndex = BoolChannels[ChannelIndex]->GetData().FindKey(FrameNumber);
					if (ExistingIndex != INDEX_NONE)
					{
						bool& BValue = BoolChannels[ChannelIndex]->GetData().GetValues()[ExistingIndex];
						BValue = Val;
					}
					else
					{
						BoolChannels[ChannelIndex]->GetData().AddKey(FrameNumber, Val);
					}
					break;
				}
				case ERigControlType::Integer:
				{
					if (ControlElement->Settings.ControlEnum)
					{
						uint8 Val = (uint8)CurrentValue.Get<uint8>();
						if (bSetDefault)
						{
							EnumChannels[ChannelIndex]->SetDefault(Val);
						}
						int32 ExistingIndex = EnumChannels[ChannelIndex]->GetData().FindKey(FrameNumber);
						if (ExistingIndex != INDEX_NONE)
						{
							uint8& BValue = EnumChannels[ChannelIndex]->GetData().GetValues()[ExistingIndex];
							BValue = Val;
						}
						else
						{
							EnumChannels[ChannelIndex]->GetData().AddKey(FrameNumber, Val);
						}
						break;

					}
					else
					{
						int32 Val = CurrentValue.Get<int32>();
						if (bSetDefault)
						{
							IntChannels[ChannelIndex]->SetDefault(Val);
						}

						int32 ExistingIndex = IntChannels[ChannelIndex]->GetData().FindKey(FrameNumber);
						if (ExistingIndex != INDEX_NONE)
						{
							int32& BValue = IntChannels[ChannelIndex]->GetData().GetValues()[ExistingIndex];
							BValue = Val;
						}
						else
						{
							IntChannels[ChannelIndex]->GetData().AddKey(FrameNumber, Val);
						}
					}
					break;
				}
				case ERigControlType::Float:
				case ERigControlType::ScaleFloat:
				{
					float Val = CurrentValue.Get<float>();
					if (bSetDefault)
					{
						FloatChannels[ChannelIndex]->SetDefault(Val);
					}
					TMovieSceneChannelData<FMovieSceneFloatValue> ChannelDataX = FloatChannels[ChannelIndex++]->GetData();
					UE::Private::SetOrAddKey(ChannelDataX, FrameNumber, Val, InInterpMode);
					break;
				}
				case ERigControlType::Vector2D:
				{
					FVector3f Val = CurrentValue.Get<FVector3f>();
					if (bSetDefault)
					{
						FloatChannels[ChannelIndex]->SetDefault(Val.X);
						FloatChannels[ChannelIndex + 1]->SetDefault(Val.Y);
					}
					TMovieSceneChannelData<FMovieSceneFloatValue> ChannelDataX = FloatChannels[ChannelIndex++]->GetData();
					UE::Private::SetOrAddKey(ChannelDataX, FrameNumber, Val.X, InInterpMode);

					TMovieSceneChannelData<FMovieSceneFloatValue> ChannelDataY = FloatChannels[ChannelIndex++]->GetData();
					UE::Private::SetOrAddKey(ChannelDataY, FrameNumber, Val.Y, InInterpMode);
					break;
				}
				case ERigControlType::Position:
				case ERigControlType::Scale:
				case ERigControlType::Rotator:
				{
					FVector3f Val = CurrentValue.Get<FVector3f>();
					if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
					{
						FVector Rotation = Hierarchy->GetControlSpecifiedEulerAngle(ControlElement);
						
						WindRotation(Rotation, ChannelIndex, 0);
						WindRotation(Rotation, ChannelIndex, 1);
						WindRotation(Rotation, ChannelIndex, 2);
						
						Val.X = static_cast<float>(Rotation.X);
						Val.Y = static_cast<float>(Rotation.Y);
						Val.Z = static_cast<float>(Rotation.Z);
					}
					//if additive and scale subtract out unity scale
					if (bIsAdditive && ControlElement->Settings.ControlType == ERigControlType::Scale)
					{
						Val.X -= 1.0f;
						Val.Y -= 1.0f;
						Val.Z -= 1.0f;
					}
					if (bSetDefault)
					{
						FloatChannels[ChannelIndex]->SetDefault(Val.X);
						FloatChannels[ChannelIndex + 1]->SetDefault(Val.Y);
						FloatChannels[ChannelIndex + 2]->SetDefault(Val.Z);
					}

					AddVectorKeyToFloatChannels(ChannelIndex, FrameNumber, FVector(Val));

					break;
				}

				case ERigControlType::Transform:
				case ERigControlType::TransformNoScale:
				case ERigControlType::EulerTransform:
				{
					FTransform Val = FTransform::Identity;
					if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
					{
						FTransformNoScale NoScale = CurrentValue.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
						Val = NoScale;
					}
					else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						FEulerTransform Euler = CurrentValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
						Val = Euler.ToFTransform();
					}
					else
					{
						Val = CurrentValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
					}
					
					// set translation
					{
						const FVector Translation = Val.GetTranslation();
						if (bSetDefault)
						{
							FloatChannels[ChannelIndex]->SetDefault(Translation.X);
							FloatChannels[ChannelIndex + 1]->SetDefault(Translation.Y);
							FloatChannels[ChannelIndex + 2]->SetDefault(Translation.Z);
						}
						AddVectorKeyToFloatChannels(ChannelIndex, FrameNumber, Translation);
					}

					// set rotation
					{
						FVector Rotation = Hierarchy->GetControlSpecifiedEulerAngle(ControlElement);
						
						WindRotation(Rotation, ChannelIndex, 0);
						WindRotation(Rotation, ChannelIndex, 1);
						WindRotation(Rotation, ChannelIndex, 2);
						
						if (bSetDefault)
						{
							FloatChannels[ChannelIndex]->SetDefault(Rotation.X);
							FloatChannels[ChannelIndex + 1]->SetDefault(Rotation.Y);
							FloatChannels[ChannelIndex + 2]->SetDefault(Rotation.Z);
						}
						AddVectorKeyToFloatChannels(ChannelIndex, FrameNumber, Rotation);
					}

					// set scale
					if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
						ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						FVector Scale = Val.GetScale3D();
						if (bIsAdditive)
						{
							Scale.X -= 1.0f;
							Scale.Y -= 1.0f;
							Scale.Z -= 1.0f;
						}
						if (bSetDefault)
						{
							FloatChannels[ChannelIndex]->SetDefault(Scale.X);
							FloatChannels[ChannelIndex + 1]->SetDefault(Scale.Y);
							FloatChannels[ChannelIndex + 2]->SetDefault(Scale.Z);
						}
						AddVectorKeyToFloatChannels(ChannelIndex, FrameNumber, Scale);
					}
					break;
				}
			}
		}
	}
}

//Function to load an Anim Sequence into this section. It will automatically resize to the section size.
//Will return false if fails or is canceled
bool UMovieSceneControlRigParameterSection::LoadAnimSequenceIntoThisSection(UAnimSequence* AnimSequence, const FFrameNumber& SequenceStart, UMovieScene* MovieScene, UObject* BoundObject, const FLoadAnimSequenceData& LoadData, EMovieSceneKeyInterpolation InInterpolation)
{
	USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(BoundObject);
	
	if (SkelMeshComp != nullptr && (SkelMeshComp->GetSkeletalMeshAsset() == nullptr || SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton() == nullptr))
	{
		return false;
	}
	
	USkeleton* Skeleton = SkelMeshComp ? SkelMeshComp->GetSkeletalMeshAsset()->GetSkeleton() : Cast<USkeleton>(BoundObject);
	if (Skeleton == nullptr)
	{
		return false;
	}

	UFKControlRig* AutoRig = Cast<UFKControlRig>(ControlRig);
	if (!AutoRig && !ControlRig->SupportsEvent(FRigUnit_InverseExecution::EventName))
	{
		return false;
	}

	TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	if (FloatChannels.Num() <= 0)
	{
		return false;
	}

	URigHierarchy* SourceHierarchy = ControlRig->GetHierarchy();
	
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	float Length = AnimSequence->GetPlayLength();
	const FFrameRate& FrameRate = AnimSequence->GetSamplingFrameRate();

	FFrameNumber StartFrame = SequenceStart + LoadData.StartFrame;
	FFrameNumber EndFrame = TickResolution.AsFrameNumber(Length) + StartFrame;

	Modify();
	if (HasStartFrame() && HasEndFrame())
	{
		StartFrame = GetInclusiveStartFrame();
		EndFrame = StartFrame + EndFrame;
		SetEndFrame(EndFrame);
	}
	ControlRig->Modify();

	
	const int32 NumberOfKeys = AnimSequence->GetDataModel()->GetNumberOfKeys();
	FFrameNumber FrameRateInFrameNumber = TickResolution.AsFrameNumber(FrameRate.AsInterval());
	int32 ExtraProgress = LoadData.bKeyReduce ? FloatChannels.Num() : 0;
	
	FScopedSlowTask Progress(NumberOfKeys + ExtraProgress, LOCTEXT("BakingToControlRig_SlowTask", "Baking To Control Rig..."));	
	Progress.MakeDialog(true);

	//Make sure we are reset and run construction event  before evaluating
	/*
	TArray<FRigElementKey>ControlsToReset = ControlRig->GetHierarchy()->GetAllKeys(true, ERigElementType::Control);
	for (const FRigElementKey& ControlToReset : ControlsToReset)
	{
		if (ControlToReset.Type == ERigElementType::Control)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ControlToReset.Name);
			if (ControlElement && !ControlElement->Settings.bIsTransientControl)
			{
				const FTransform InitialLocalTransform = ControlRig->GetHierarchy()->GetInitialLocalTransform(ControlToReset);
				ControlRig->GetHierarchy()->SetLocalTransform(ControlToReset, InitialLocalTransform);
			}
		}
	}
	SourceBones.ResetTransforms();
	SourceCurves.ResetValues();
	ControlRig->Execute(TEXT("Setup"));
	*/
	const IAnimationDataModel* DataModel = AnimSequence->GetDataModel();
	const FAnimationCurveData& CurveData = DataModel->GetCurveData();

	// copy the hierarchy from the CDO into the target control rig.
	// this ensures that the topology version matches in case of a dynamic hierarchy
	if (LoadData.bResetControls)
	{
		if(!ControlRig->GetAssetReference().IsNative())
		{
			SourceHierarchy->CopyHierarchy(ControlRig->GetAssetReference().GetDefaultHierarchy());
		}
	}

	// now set the hierarchies initial transforms based on the currently used skeletal mesh
	if (SkelMeshComp)
	{
		ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkelMeshComp, true);
	}
	else
	{
		ControlRig->SetBoneInitialTransformsFromRefSkeleton(Skeleton->GetReferenceSkeleton());
	}
	if (LoadData.bResetControls)
	{
		ControlRig->RequestConstruction();
		ControlRig->Evaluate_AnyThread();
	}

	int32 Index = 0;
	int32 EndIndex = NumberOfKeys - 1; 
	const int32 LastIndex = EndIndex;
	if (LoadData.AnimFrameRange.IsSet())
	{
		Index = (int32)(LoadData.AnimFrameRange.GetValue().GetLowerBoundValue().Value);
		Index = Index > (LastIndex) ? LastIndex : Index;
		EndIndex = (int32)(LoadData.AnimFrameRange.GetValue().GetUpperBoundValue().Value);
		EndIndex = EndIndex > (LastIndex) ? LastIndex : EndIndex;
	}
	int32 KeyIndex = 0;
	for (; Index <= EndIndex; ++Index)
	{
		const float SequenceSecond = AnimSequence->GetTimeAtFrame(Index);
		const FFrameNumber FrameNumber = StartFrame + (FMath::Max(FrameRateInFrameNumber.Value, 1) * KeyIndex);
		++KeyIndex;
		if (LoadData.bResetControls)
		{
			SourceHierarchy->ResetPoseToInitial();
			SourceHierarchy->ResetCurveValues();
		}

		for (const FFloatCurve& Curve : CurveData.FloatCurves)
		{
			const float Val = Curve.FloatCurve.Eval(SequenceSecond);
			SourceHierarchy->SetCurveValue(FRigElementKey(Curve.GetName(), ERigElementType::Curve), Val);
		}

		// retrieve the pose using the services that persona and sequencer rely on
		// rather than accessing the low level raw tracks.
		FAnimPoseEvaluationOptions EvaluationOptions;
		EvaluationOptions.OptionalSkeletalMesh = SkelMeshComp ? SkelMeshComp->GetSkeletalMeshAsset() : nullptr;
		//do not retarget if FKControlRig 
		EvaluationOptions.bShouldRetarget = (EvaluationOptions.OptionalSkeletalMesh && AutoRig == nullptr) ? LoadData.bShouldRetarget : false;
		if (EvaluationOptions.bShouldRetarget == true)
		{
			if (USkeleton* NewSkeleton = EvaluationOptions.OptionalSkeletalMesh->GetSkeleton())
			{
				if (NewSkeleton == AnimSequence->GetSkeleton() && NewSkeleton->GetGuid() == AnimSequence->GetSkeletonGuid())
				{
					EvaluationOptions.bShouldRetarget = false;
				}
			}
		}
		EvaluationOptions.EvaluationType = EAnimDataEvalType::Raw;

		FAnimPose AnimPose;
		UAnimPoseExtensions::GetAnimPoseAtTime(AnimSequence, SequenceSecond, EvaluationOptions, AnimPose);

		TArray<FName> BoneNames;
		UAnimPoseExtensions::GetBoneNames(AnimPose, BoneNames);
		for(const FName& BoneName : BoneNames)
		{
			if(FRigBoneElement* BoneElement = SourceHierarchy->Find<FRigBoneElement>(FRigElementKey(BoneName, ERigElementType::Bone)))
			{
				FTransform LocalTransform = UAnimPoseExtensions::GetBonePose(AnimPose, BoneName, EAnimPoseSpaces::Local);
				SourceHierarchy->SetLocalTransform(BoneElement->GetIndex(), LocalTransform, true, false);
			}
		}

		if (KeyIndex == 0)
		{
			//to make sure the first frame looks good we need to do this first. UE-100069
			ControlRig->Execute(FRigUnit_InverseExecution::EventName);
		}
		ControlRig->Execute(FRigUnit_InverseExecution::EventName);

		RecordControlRigKey(FrameNumber, KeyIndex == 0, InInterpolation, LoadData.bOntoSelectedControls);
		Progress.EnterProgressFrame(1);
		if (Progress.ShouldCancel())
		{
			return false;
		}
	}

	if (LoadData.bKeyReduce)
	{
		FKeyDataOptimizationParams Params;
		Params.bAutoSetInterpolation = true;
		Params.Tolerance = LoadData.Tolerance;
		for (FMovieSceneFloatChannel* Channel : FloatChannels)
		{
			Channel->Optimize(Params); //should also auto tangent
			Progress.EnterProgressFrame(1);
			if (Progress.ShouldCancel())
			{
				return false;
			}
		}

		TArrayView<FMovieSceneBoolChannel*> BoolChannels = GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
		for (FMovieSceneBoolChannel* Channel : BoolChannels)
		{
			Channel->Optimize(Params);
		}

		TArrayView<FMovieSceneIntegerChannel*> IntegerChannels = GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
		for (FMovieSceneIntegerChannel* Channel : IntegerChannels)
		{
			Channel->Optimize(Params);
		}

		TArrayView<FMovieSceneByteChannel*> EnumChannels = GetChannelProxy().GetChannels<FMovieSceneByteChannel>();
		for (FMovieSceneByteChannel* Channel : EnumChannels)
		{
			Channel->Optimize(Params);
		}
	}
	else
	{
		for (FMovieSceneFloatChannel* Channel : FloatChannels)
		{
			Channel->AutoSetTangents();
		}
	}
	return true;
}

#endif


void UMovieSceneControlRigParameterSection::AddEnumParameterKey(FName InParameterName, FFrameNumber InTime, uint8 InValue)
{
	FMovieSceneByteChannel* ExistingChannel = nullptr;
	for (FEnumParameterNameAndCurve& EnumParameterNameAndCurve : EnumParameterNamesAndCurves)
	{
		if (EnumParameterNameAndCurve.ParameterName == InParameterName)
		{
			ExistingChannel = &EnumParameterNameAndCurve.ParameterCurve;
			break;
		}
	}
	if (ExistingChannel == nullptr)
	{
		const int32 NewIndex = EnumParameterNamesAndCurves.Add(FEnumParameterNameAndCurve(InParameterName));
		ExistingChannel = &EnumParameterNamesAndCurves[NewIndex].ParameterCurve;

		ReconstructChannelProxy();
	}

	ExistingChannel->GetData().UpdateOrAddKey(InTime, InValue);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}


void UMovieSceneControlRigParameterSection::AddIntegerParameterKey(FName InParameterName, FFrameNumber InTime, int32 InValue)
{
	FMovieSceneIntegerChannel* ExistingChannel = nullptr;
	for (FIntegerParameterNameAndCurve& IntegerParameterNameAndCurve : IntegerParameterNamesAndCurves)
	{
		if (IntegerParameterNameAndCurve.ParameterName == InParameterName)
		{
			ExistingChannel = &IntegerParameterNameAndCurve.ParameterCurve;
			break;
		}
	}
	if (ExistingChannel == nullptr)
	{
		const int32 NewIndex = IntegerParameterNamesAndCurves.Add(FIntegerParameterNameAndCurve(InParameterName));
		ExistingChannel = &IntegerParameterNamesAndCurves[NewIndex].ParameterCurve;

		ReconstructChannelProxy();
	}

	ExistingChannel->GetData().UpdateOrAddKey(InTime, InValue);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

bool UMovieSceneControlRigParameterSection::RemoveEnumParameter(FName InParameterName)
{
	for (int32 i = 0; i < EnumParameterNamesAndCurves.Num(); i++)
	{
		if (EnumParameterNamesAndCurves[i].ParameterName == InParameterName)
		{
			EnumParameterNamesAndCurves.RemoveAt(i);
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneControlRigParameterSection::RemoveIntegerParameter(FName InParameterName)
{
	for (int32 i = 0; i < IntegerParameterNamesAndCurves.Num(); i++)
	{
		if (IntegerParameterNamesAndCurves[i].ParameterName == InParameterName)
		{
			IntegerParameterNamesAndCurves.RemoveAt(i);
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}


TArray<FEnumParameterNameAndCurve>& UMovieSceneControlRigParameterSection::GetEnumParameterNamesAndCurves()
{
	return EnumParameterNamesAndCurves;
}

const TArray<FEnumParameterNameAndCurve>& UMovieSceneControlRigParameterSection::GetEnumParameterNamesAndCurves() const
{
	return EnumParameterNamesAndCurves;
}

TArray<FIntegerParameterNameAndCurve>& UMovieSceneControlRigParameterSection::GetIntegerParameterNamesAndCurves()
{
	return IntegerParameterNamesAndCurves;
}

const TArray<FIntegerParameterNameAndCurve>& UMovieSceneControlRigParameterSection::GetIntegerParameterNamesAndCurves() const
{
	return IntegerParameterNamesAndCurves;
}
/*
void FillControlNameMask(bool bValue)
{
	TArray<FRigControlElement*> SortedControls;
	ControlRig->GetControlsInOrder(SortedControls);
	ControlNameMask.Empty();
	for (FRigControlElement* ControlElement : SortedControls)
	{
		if (ControlElement)
		{
			ControlNameMask.Add(ControlElement->GetKey().Name, bValue);
		}
	}
}

void SetControlNameMask(const FName& Name, bool bValue)
{
	ControlNameMask.Add(Name, bValue);
}

//get value, will return false if not found
bool GetControlNameMask(const FName& Name, bool& OutValue)
{
	bool* bValue = ControlNameMask.Find(Name);
	if (bValue)
	{
		OutValue = *bValue;
		return true;
	}
	return false;
}
*/
void UMovieSceneControlRigParameterSection::ClearAllParameters()
{
	BoolParameterNamesAndCurves.SetNum(0);
	ScalarParameterNamesAndCurves.SetNum(0);
	Vector2DParameterNamesAndCurves.SetNum(0);
	VectorParameterNamesAndCurves.SetNum(0);
	ColorParameterNamesAndCurves.SetNum(0);
	TransformParameterNamesAndCurves.SetNum(0);
	EnumParameterNamesAndCurves.SetNum(0);
	IntegerParameterNamesAndCurves.SetNum(0);
	SpaceChannels.SetNum(0);
	ConstraintsChannels.SetNum(0);
}

void UMovieSceneControlRigParameterSection::RemoveAllKeys(bool bIncludeSpaceKeys)
{
	TArray<FFrameNumber> KeyTimes;
	TArray<FKeyHandle> Handles;
	if (bIncludeSpaceKeys)
	{
		for (FSpaceControlNameAndChannel& Space : SpaceChannels)
		{
			KeyTimes.SetNum(0);
			Handles.SetNum(0);
			Space.SpaceCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
			Space.SpaceCurve.DeleteKeys(Handles);
		}
	}
	for (FBoolParameterNameAndCurve& Bool : GetBoolParameterNamesAndCurves())
	{
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Bool.ParameterCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Bool.ParameterCurve.DeleteKeys(Handles);
	}
	for (FEnumParameterNameAndCurve& Enum : GetEnumParameterNamesAndCurves())
	{
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Enum.ParameterCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Enum.ParameterCurve.DeleteKeys(Handles);
	}
	for (FIntegerParameterNameAndCurve& Integer : GetIntegerParameterNamesAndCurves())
	{
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Integer.ParameterCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Integer.ParameterCurve.DeleteKeys(Handles);
	}

	for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
	{
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Scalar.ParameterCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Scalar.ParameterCurve.DeleteKeys(Handles);
	}
	for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
	{
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Vector2D.XCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Vector2D.XCurve.DeleteKeys(Handles);
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Vector2D.YCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Vector2D.YCurve.DeleteKeys(Handles);
	}
	for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
	{
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Vector.XCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Vector.XCurve.DeleteKeys(Handles);
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Vector.YCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Vector.YCurve.DeleteKeys(Handles);
		KeyTimes.SetNum(0);
		Handles.SetNum(0);
		Vector.ZCurve.GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		Vector.ZCurve.DeleteKeys(Handles);
	}
	for (FTransformParameterNameAndCurves& Transform : GetTransformParameterNamesAndCurves())
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			KeyTimes.SetNum(0);
			Handles.SetNum(0);
			Transform.Translation[Index].GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
			Transform.Translation[Index].DeleteKeys(Handles);
			KeyTimes.SetNum(0);
			Handles.SetNum(0);
			Transform.Rotation[Index].GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
			Transform.Rotation[Index].DeleteKeys(Handles);
			KeyTimes.SetNum(0);
			Handles.SetNum(0);
			Transform.Scale[Index].GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
			Transform.Scale[Index].DeleteKeys(Handles);
		}
	}
}

UControlRig* UMovieSceneControlRigParameterSection::GetControlRig() const
{
	return GetControlRig(nullptr, nullptr);
}

UControlRig* UMovieSceneControlRigParameterSection::GetControlRig(UWorld* InGameWorld, UObject* InBoundObject) const
{
	if (InGameWorld == nullptr)
	{
		return ControlRig;
	}
	if (UMovieSceneControlRigParameterTrack* Track = GetTypedOuter<UMovieSceneControlRigParameterTrack>())
	{
		return Track->GetGameWorldControlRig(InGameWorld, InBoundObject);
	}
	return nullptr;
}


int32 UMovieSceneControlRigParameterSection::GetActiveCategoryIndex(FName ControlName) const
{
	int32 CategoryIndex = INDEX_NONE;
	const FChannelMapInfo* pChannelIndex = ControlChannelMap.Find(ControlName);
	if (pChannelIndex != nullptr && GetControlNameMask(ControlName))
	{
		CategoryIndex = pChannelIndex->CategoryIndex;
	}
	return CategoryIndex;
}


TOptional<float> UMovieSceneControlRigParameterSection::EvaluateScalarParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<float> OptValue;	
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		float Value = 0.0f;
		FloatChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value);
		OptValue = Value;
	}
	return OptValue;
}

TOptional<bool> UMovieSceneControlRigParameterSection::EvaluateBoolParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<bool> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneBoolChannel*> BoolChannels = GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
		bool Value = false;
		BoolChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value);
		OptValue = Value;
	}
	return OptValue;
}

TOptional<uint8> UMovieSceneControlRigParameterSection::EvaluateEnumParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<uint8> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneByteChannel*> EnumChannels = GetChannelProxy().GetChannels<FMovieSceneByteChannel>();
		uint8 Value = 0;
		EnumChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value);
		OptValue = Value;
	}
	return OptValue;
}

TOptional<int32> UMovieSceneControlRigParameterSection::EvaluateIntegerParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<int32> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneIntegerChannel*> IntChannels = GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
		int32 Value = 0;
		IntChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value);
		OptValue = Value;
	}
	return OptValue;
}

TOptional<FVector> UMovieSceneControlRigParameterSection::EvaluateVectorParameter(const FFrameTime& InTime, FName InParameterName)
{
	TOptional<FVector> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		FVector3f Value(0.0f, 0.0f, 0.0f);
		FloatChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value.X);
		FloatChannels[ChannelInfo->ChannelIndex + 1]->Evaluate(InTime, Value.Y);
		FloatChannels[ChannelInfo->ChannelIndex + 2]->Evaluate(InTime, Value.Z);
		OptValue = (FVector)Value;
	}
	return OptValue;
}

TOptional<FVector2D> UMovieSceneControlRigParameterSection::EvaluateVector2DParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<FVector2D> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		FVector2f Value(0.0f, 0.0f);
		FloatChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value.X);
		FloatChannels[ChannelInfo->ChannelIndex + 1]->Evaluate(InTime, Value.Y);
		OptValue = FVector2D(Value);
	}
	return OptValue;
}

TOptional<FLinearColor>UMovieSceneControlRigParameterSection:: EvaluateColorParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<FLinearColor> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		FLinearColor Value(0.0f, 0.0f, 0.0f, 1.0f);
		FloatChannels[ChannelInfo->ChannelIndex]->Evaluate(InTime, Value.R);
		FloatChannels[ChannelInfo->ChannelIndex + 1]->Evaluate(InTime, Value.G);		
		FloatChannels[ChannelInfo->ChannelIndex + 2]->Evaluate(InTime, Value.B);
		FloatChannels[ChannelInfo->ChannelIndex + 3]->Evaluate(InTime, Value.A);
		OptValue = Value;
	}
	return OptValue;
}

TOptional<FEulerTransform> UMovieSceneControlRigParameterSection::EvaluateTransformParameter(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<FEulerTransform> OptValue;
	if (const FChannelMapInfo* ChannelInfo = ControlChannelMap.Find(InParameterName))
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		FEulerTransform Value = FEulerTransform::Identity;
		FVector3f Translation(ForceInitToZero), Scale(FVector3f::OneVector);
		FRotator3f Rotator(0.0f, 0.0f, 0.0f);

		FloatChannels[ChannelInfo->ChannelIndex    ]->Evaluate(InTime, Translation.X);
		FloatChannels[ChannelInfo->ChannelIndex + 1]->Evaluate(InTime, Translation.Y);
		FloatChannels[ChannelInfo->ChannelIndex + 2]->Evaluate(InTime, Translation.Z);

		FloatChannels[ChannelInfo->ChannelIndex + 3]->Evaluate(InTime, Rotator.Roll);
		FloatChannels[ChannelInfo->ChannelIndex + 4]->Evaluate(InTime, Rotator.Pitch);
		FloatChannels[ChannelInfo->ChannelIndex + 5]->Evaluate(InTime, Rotator.Yaw);
		if (ControlRig)
		{
			if (FRigControlElement* ControlElement = ControlRig->FindControl(InParameterName))
			{
				if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
					ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					FloatChannels[ChannelInfo->ChannelIndex + 6]->Evaluate(InTime, Scale.X);
					FloatChannels[ChannelInfo->ChannelIndex + 7]->Evaluate(InTime, Scale.Y);
					FloatChannels[ChannelInfo->ChannelIndex + 8]->Evaluate(InTime, Scale.Z);
				}

			}
		}
		Value = FEulerTransform(FRotator(Rotator), (FVector)Translation, (FVector)Scale);
		OptValue = Value;
	}
	return OptValue;
}

TOptional<FMovieSceneControlRigSpaceBaseKey> UMovieSceneControlRigParameterSection::EvaluateSpaceChannel(const  FFrameTime& InTime, FName InParameterName)
{
	TOptional<FMovieSceneControlRigSpaceBaseKey> OptValue;
	if (FSpaceControlNameAndChannel* Channel = GetSpaceChannel(InParameterName))
	{
		FMovieSceneControlRigSpaceBaseKey Value;
		using namespace UE::MovieScene;
		EvaluateChannel(&(Channel->SpaceCurve),InTime, Value);
		OptValue = Value;
	}
	return OptValue;
}

FFrameTime UMovieSceneControlRigParameterSection::RemapTime(const FFrameTime& InFrameTime) const
{
	UMovieSceneTimeWarpGetter* Custom = PlayRate.GetType() == EMovieSceneTimeWarpType::Custom
		? PlayRate.AsCustom()
		: nullptr;

	if (Custom && Custom->IsMuted() == false)
	{
		return PlayRate.RemapTime(InFrameTime);
	}
	return InFrameTime;
}

void UMovieSceneControlRigParameterSection::DeleteTimeWarp()
{
	if (HasTimeWarp())
	{
		Modify();

		// Reset to the default 1:1 play rate so this section no longer reports a custom time warp
		PlayRate.Set(1.0);

		ReconstructChannelProxy();
	}
}

UObject* UMovieSceneControlRigParameterSection::GetImplicitObjectOwner()
{
	if (UControlRig* CR = GetControlRig())
	{
		return CR;
	}
	return Super::GetImplicitObjectOwner();
}

void UMovieSceneControlRigParameterSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate())
	{
		return;
	}

	const bool bExternalBlending = (BlendType.Get() == EMovieSceneBlendType::Absolute);

	EControlRigEntityType EntityType;
	int32 EntityIndex = 0;
	DecodeControlRigEntityID(Params.EntityID, EntityIndex, EntityType);

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();

	FGuid ObjectBindingID = Params.GetObjectBindingID();
	FControlRigSourceData ControlRigSource;
	ControlRigSource.Track = GetTypedOuter<UMovieSceneControlRigParameterTrack>();
	ControlRigSource.Section = const_cast<UMovieSceneControlRigParameterSection*>(this);
	ControlRigSource.ImportedControlsHash = GetLastControlsHash();
	ControlRigSource.ImportedEntityID = Params.EntityID;

	if (!ensure(ControlRigSource.Track))
	{
		return;
	}

	EMovieSceneTransformChannel ChannelMask = TransformMask.GetChannels();
	UMovieSceneTimeWarpGetter* Custom = PlayRate.GetType() == EMovieSceneTimeWarpType::Custom
		? PlayRate.AsCustom()
		: nullptr;

	switch (EntityType)
	{
		case EControlRigEntityType::Base:
		{
			auto NeverResolve = [](UObject*)->UObject*{ return nullptr; };

			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID)
				.Add(BuiltInComponentTypes->BoundObjectResolver, NeverResolve)
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(ControlRigComponents->BaseControlRigEvalData, FBaseControlRigEvalData{ this })
				.AddConditional(BuiltInComponentTypes->WeightChannel, &Weight, EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Weight) && Weight.HasAnyData())
				.AddDefaulted(BuiltInComponentTypes->EvalTime)
				.AddDefaulted(BuiltInComponentTypes->EvalSeconds)
				.AddTag(ControlRigComponents->Tags.BaseControlRig)
				.AddMutualComponents()
			);
			break;
		}

		case EControlRigEntityType::Space:
		{
			const FSpaceControlNameAndChannel& Space = SpaceChannels[EntityIndex];
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(TracksComponentTypes->GenericParameterName, Space.ControlName)
				.Add(ControlRigComponents->SpaceChannel, &Space.SpaceCurve)
				.AddTag(ControlRigComponents->Tags.ControlRigParameter)
				.AddTag(ControlRigComponents->Tags.Space)
				.AddMutualComponents()
			);
			break;
		}

		case EControlRigEntityType::BoolParameter:
		{
			const FBoolParameterNameAndCurve& Bool = BoolParameterNamesAndCurves[EntityIndex];
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(TracksComponentTypes->GenericParameterName, Bool.ParameterName)
				.Add(BuiltInComponentTypes->BoolChannel, &Bool.ParameterCurve)
				.AddDefaulted(BuiltInComponentTypes->EvalTime)
				.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom && Custom->IsMuted() == false)
				.AddTag(ControlRigComponents->Tags.ControlRigParameter)
				.AddTag(TracksComponentTypes->Parameters.Bool.PropertyTag)
				.AddMutualComponents()
			);
			break;
		}

		case EControlRigEntityType::EnumParameter:
		{
			const FEnumParameterNameAndCurve& Enum = EnumParameterNamesAndCurves[EntityIndex];
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(TracksComponentTypes->GenericParameterName, Enum.ParameterName)
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.AddDefaulted(BuiltInComponentTypes->EvalTime)
				.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom && Custom->IsMuted() == false)
				.AddTag(ControlRigComponents->Tags.ControlRigParameter)
				.AddTag(TracksComponentTypes->Parameters.Byte.PropertyTag)
				.Add(BuiltInComponentTypes->ByteChannel, &Enum.ParameterCurve)
				.AddMutualComponents()
			);
			break;
		}

		case EControlRigEntityType::IntegerParameter:
		{
			const FIntegerParameterNameAndCurve& Integer = IntegerParameterNamesAndCurves[EntityIndex];
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(TracksComponentTypes->GenericParameterName, Integer.ParameterName)
				.AddDefaulted(BuiltInComponentTypes->EvalTime)
				.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom&& Custom->IsMuted() == false)
				.AddTag(ControlRigComponents->Tags.ControlRigParameter)
				.AddTag(TracksComponentTypes->Parameters.Integer.PropertyTag)
				.Add(BuiltInComponentTypes->IntegerChannel, &Integer.ParameterCurve)
				.AddMutualComponents()
			);
			break;
		}

		case EControlRigEntityType::ScalarParameter:
		{
			const FScalarParameterNameAndCurve& Scalar = ScalarParameterNamesAndCurves[EntityIndex];
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(TracksComponentTypes->GenericParameterName, Scalar.ParameterName)
				.AddTag(ControlRigComponents->Tags.ControlRigParameter)
				.AddTag(TracksComponentTypes->Parameters.Scalar.PropertyTag)
				.Add(BuiltInComponentTypes->FloatChannel[0], &Scalar.ParameterCurve)
				.AddTagConditional(BuiltInComponentTypes->Tags.ExternalBlending, bExternalBlending)
				.AddDefaulted(BuiltInComponentTypes->EvalTime)
				.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom&& Custom->IsMuted() == false)
				.AddConditional(BuiltInComponentTypes->WeightChannel, &Weight, EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Weight) && Weight.HasAnyData())
				.AddMutualComponents()
			);
			break;
		}

		case EControlRigEntityType::VectorParameter:
			if (EntityIndex < Vector2DParameterNamesAndCurves.Num())
			{
				const FVector2DParameterNameAndCurves& Vector2D = Vector2DParameterNamesAndCurves[EntityIndex];
				OutImportedEntity->AddBuilder(
					FEntityBuilder()
					.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
					.Add(TracksComponentTypes->GenericParameterName, Vector2D.ParameterName)
					.AddTag(ControlRigComponents->Tags.ControlRigParameter)
					.AddTag(TracksComponentTypes->Parameters.Vector3.PropertyTag)
					.AddDefaulted(BuiltInComponentTypes->EvalTime)
					.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom&& Custom->IsMuted() == false)
					.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Vector2D.XCurve, Vector2D.XCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Vector2D.YCurve, Vector2D.YCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->WeightChannel, &Weight, EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Weight) && Weight.HasAnyData())
					.AddTagConditional(BuiltInComponentTypes->Tags.ExternalBlending, bExternalBlending)
					.AddMutualComponents()
				);
			}
			else
			{
				const FVectorParameterNameAndCurves& Vector = VectorParameterNamesAndCurves[EntityIndex - Vector2DParameterNamesAndCurves.Num()];
				OutImportedEntity->AddBuilder(
					FEntityBuilder()
					.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
					.Add(TracksComponentTypes->GenericParameterName, Vector.ParameterName)
					.AddTag(ControlRigComponents->Tags.ControlRigParameter)
					.AddTag(TracksComponentTypes->Parameters.Vector3.PropertyTag)
					.AddDefaulted(BuiltInComponentTypes->EvalTime)
					.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom&& Custom->IsMuted() == false)
					.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Vector.XCurve, Vector.XCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Vector.YCurve, Vector.YCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[2], &Vector.ZCurve, Vector.ZCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->WeightChannel, &Weight, EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Weight) && Weight.HasAnyData())
					.AddTagConditional(BuiltInComponentTypes->Tags.ExternalBlending, bExternalBlending)
					.AddMutualComponents()
				);
			}
			break;

		case EControlRigEntityType::TransformParameter:
		{
			const FTransformParameterNameAndCurves& Transform = TransformParameterNamesAndCurves[EntityIndex];

			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(ControlRigComponents->ControlRigSource, ControlRigSource)
				.Add(TracksComponentTypes->GenericParameterName, Transform.ParameterName)
				.AddTag(ControlRigComponents->Tags.ControlRigParameter)
				.AddTag(TracksComponentTypes->Parameters.Transform.PropertyTag)
				.AddDefaulted(BuiltInComponentTypes->EvalTime)
				.AddConditional(BuiltInComponentTypes->TimeWarp, Custom, Custom && Custom->IsMuted() == false)
				.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Transform.Translation[0], EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationX) && Transform.Translation[0].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Transform.Translation[1], EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationY) && Transform.Translation[1].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[2], &Transform.Translation[2], EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationZ) && Transform.Translation[2].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[3], &Transform.Rotation[0],    EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationX) && Transform.Rotation[0].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[4], &Transform.Rotation[1],    EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationY) && Transform.Rotation[1].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[5], &Transform.Rotation[2],    EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationZ) && Transform.Rotation[2].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[6], &Transform.Scale[0],       EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleX) && Transform.Scale[0].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[7], &Transform.Scale[1],       EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleY) && Transform.Scale[1].HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[8], &Transform.Scale[2],       EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleZ) && Transform.Scale[2].HasAnyData())
				.AddConditional(BuiltInComponentTypes->WeightChannel,   &Weight,				   EnumHasAnyFlags(ChannelMask, EMovieSceneTransformChannel::Weight) && Weight.HasAnyData())

				.AddTagConditional(BuiltInComponentTypes->Tags.ExternalBlending, bExternalBlending)
				.AddMutualComponents()
			);
			break;
		}
	}
}


bool UMovieSceneControlRigParameterSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// We use the top 8 bits of EntityID to encode the type of parameter
	const int32 NumSpaceID = SpaceChannels.Num();
	const int32 NumBoolID = BoolParameterNamesAndCurves.Num();
	const int32 NumEnumID = EnumParameterNamesAndCurves.Num();
	const int32 NumIntegerID = IntegerParameterNamesAndCurves.Num();
	const int32 NumScalarID = ScalarParameterNamesAndCurves.Num();
	const int32 NumVector2ID = Vector2DParameterNamesAndCurves.Num();
	const int32 NumVector3ID = VectorParameterNamesAndCurves.Num();
	const int32 NumTransformID = TransformParameterNamesAndCurves.Num();

	{
		// In the event there are multiple we will just pick one at runtime, but we need one entity per section for pre-animated state
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(0, EControlRigEntityType::Base));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}

	for (int32 Index = 0; Index < NumSpaceID; ++Index)
	{
		const FSpaceControlNameAndChannel& Space = SpaceChannels[Index];

		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::Space));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}

	for (int32 Index = 0; Index < NumBoolID; ++Index)
	{
		const FBoolParameterNameAndCurve& Bool = BoolParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Bool.ParameterName))
		{
			continue;
		}

		if (Bool.ParameterCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::BoolParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	for (int32 Index = 0; Index < NumEnumID; ++Index)
	{
		const FEnumParameterNameAndCurve& Enum = EnumParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Enum.ParameterName))
		{
			continue;
		}

		if (Enum.ParameterCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::EnumParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	for (int32 Index = 0; Index < NumIntegerID; ++Index)
	{
		const FIntegerParameterNameAndCurve& Integer = IntegerParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Integer.ParameterName))
		{
			continue;
		}

		if (Integer.ParameterCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::IntegerParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	for (int32 Index = 0; Index < NumScalarID; ++Index)
	{
		const FScalarParameterNameAndCurve& Scalar = ScalarParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Scalar.ParameterName))
		{
			continue;
		}

		if (Scalar.ParameterCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::ScalarParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	for (int32 Index = 0; Index < NumVector2ID; ++Index)
	{
		const FVector2DParameterNameAndCurves& Vector2D = Vector2DParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Vector2D.ParameterName))
		{
			continue;
		}

		if (Vector2D.XCurve.HasAnyData() || Vector2D.YCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::VectorParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	for (int32 Index = 0; Index < NumVector3ID; ++Index)
	{
		const FVectorParameterNameAndCurves& Vector = VectorParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Vector.ParameterName))
		{
			continue;
		}

		if (Vector.XCurve.HasAnyData() || Vector.YCurve.HasAnyData() || Vector.ZCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index + Vector2DParameterNamesAndCurves.Num(), EControlRigEntityType::VectorParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	for (int32 Index = 0; Index < NumTransformID; ++Index)
	{
		const FTransformParameterNameAndCurves& Transform = TransformParameterNamesAndCurves[Index];
		if (ControlNameMask.Contains(Transform.ParameterName))
		{
			continue;
		}

		if (Transform.Translation[0].HasAnyData() || Transform.Translation[1].HasAnyData() || Transform.Translation[2].HasAnyData() ||
			Transform.Rotation[0].HasAnyData() || Transform.Rotation[1].HasAnyData() || Transform.Rotation[2].HasAnyData() ||
			Transform.Scale[0].HasAnyData() || Transform.Scale[1].HasAnyData() || Transform.Scale[2].HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeControlRigEntityID(Index, EControlRigEntityType::TransformParameter));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE 






