// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigParameterTemplate.h"
#include "Evaluation/Blending/MovieSceneMultiChannelBlending.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "ControlRig.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneCommonHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "AnimCustomInstanceHelper.h"
#include "Sequencer/ControlRigLayerInstance.h"
#include "IControlRigObjectBinding.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "ControlRigObjectBinding.h"
#include "Evaluation/Blending/BlendableTokenStack.h"
#include "Evaluation/Blending/MovieSceneBlendingActuatorID.h"
#include "TransformNoScale.h"
#include "ControlRigComponent.h"
#include "SkeletalMeshRestoreState.h"
#include "Rigs/FKControlRig.h"
#include "UObject/UObjectAnnotation.h"
#include "Rigs/RigHierarchy.h"
#include "Transform/DependencyBuilder.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformableHandleUtils.h"
#include "AnimationCoreLibrary.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Misc/ScopeLock.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneControlRigParameterTemplate)

//#include "Particles/ParticleSystemComponent.h"

DECLARE_CYCLE_STAT(TEXT("ControlRig Parameter Track Evaluate"), MovieSceneEval_ControlRigTemplateParameter_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("ControlRig Parameter Track Token Execute"), MovieSceneEval_ControlRigParameterTrack_TokenExecute, STATGROUP_MovieSceneEval);

template<typename T>
struct TNameAndValue
{
	FName Name;
	T Value;
};


/**
 * Structure representing the animated value of a scalar parameter.
 */
struct FScalarParameterStringAndValue
{
	/** Creates a new FScalarParameterAndValue with a parameter name and a value. */
	FScalarParameterStringAndValue(FName InParameterName, float InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the scalar parameter. */
	FName ParameterName;
	/** The animated value of the scalar parameter. */
	float Value;
};

/**
 * Structure representing the animated value of a bool parameter.
 */
struct FBoolParameterStringAndValue
{
	/** Creates a new FBoolParameterAndValue with a parameter name and a value. */
	FBoolParameterStringAndValue(FName InParameterName, bool InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the bool parameter. */
	FName ParameterName;
	/** The animated value of the bool parameter. */
	bool Value;
};

/**
 * Structure representing the animated value of a int parameter.
 */
struct FIntegerParameterStringAndValue
{
	FIntegerParameterStringAndValue(FName InParameterName, int32 InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	FName ParameterName;
	int32 Value;
};

struct FControlSpaceAndValue
{
	FControlSpaceAndValue(FName InControlName, const FMovieSceneControlRigSpaceBaseKey& InValue)
	{
		ControlName = InControlName;
		Value = InValue;
	}
	FName ControlName;
	FMovieSceneControlRigSpaceBaseKey Value;
};
/**
 * Structure representing the animated value of a vector2D parameter.
 */
struct FVector2DParameterStringAndValue
{
	/** Creates a new FVector2DParameterAndValue with a parameter name and a value. */
	FVector2DParameterStringAndValue(FName InParameterName, FVector2D InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the vector2D parameter. */
	FName ParameterName;

	/** The animated value of the vector2D parameter. */
	FVector2D Value;
};

/**
 * Structure representing the animated value of a vector parameter.
 */
struct FVectorParameterStringAndValue
{
	/** Creates a new FVectorParameterAndValue with a parameter name and a value. */
	FVectorParameterStringAndValue(FName InParameterName, FVector InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the vector parameter. */
	FName ParameterName;

	/** The animated value of the vector parameter. */
	FVector Value;
};


/**
 * Structure representing the animated value of a color parameter.
 */
struct FColorParameterStringAndValue
{
	/** Creates a new FColorParameterAndValue with a parameter name and a value. */
	FColorParameterStringAndValue(FName InParameterName, FLinearColor InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the color parameter. */
	FName ParameterName;

	/** The animated value of the color parameter. */
	FLinearColor Value;
};

struct FEulerTransformParameterStringAndValue
{

	/** The name of the transform  parameter. */
	FName ParameterName;
	/** Transform component */
	FEulerTransform Transform;

	FEulerTransformParameterStringAndValue(FName InParameterName, const FEulerTransform& InTransform)
		: Transform(InTransform)
	{
		ParameterName = InParameterName;
	}
};

struct FConstraintAndActiveValue
{
	FConstraintAndActiveValue(TWeakObjectPtr<UTickableConstraint> InConstraint,  bool InValue)
		: Constraint(InConstraint)
		, Value(InValue)
	{}
	TWeakObjectPtr<UTickableConstraint> Constraint;
	bool Value;
};

struct FControlRigAnimTypeIDs;

/** Thread-safe because objects can be destroyed on background threads */
using FControlRigAnimTypeIDsPtr = TSharedPtr<FControlRigAnimTypeIDs, ESPMode::ThreadSafe>;

/**
 * Control rig anim type IDs are a little complex - they require a unique type ID for every bone
 * and they must be unique per-animating control rig. To efficiently support finding these each frame,
 * We store a cache of the type IDs in a container on an object annotation for each control rig.
 */
struct FControlRigAnimTypeIDs
{
	/** Get the anim type IDs for the specified section */
	static FControlRigAnimTypeIDsPtr Get(const UControlRig* ControlRig)
	{
		struct FControlRigAnimTypeIDsAnnotation
		{
			// IsDefault should really have been implemented as a trait rather than a function so that this type isn't necessary
			bool IsDefault() const
			{
				return Ptr == nullptr;
			}
			FControlRigAnimTypeIDsPtr Ptr;
		};

		// Function-local static so that this only gets created once it's actually required
		static FUObjectAnnotationSparse<FControlRigAnimTypeIDsAnnotation, true> AnimTypeIDAnnotation;

		FControlRigAnimTypeIDsAnnotation TypeIDs = AnimTypeIDAnnotation.GetAnnotation(ControlRig);
		if (TypeIDs.Ptr != nullptr)
		{
			return TypeIDs.Ptr;
		}

		FControlRigAnimTypeIDsPtr NewPtr = MakeShared<FControlRigAnimTypeIDs, ESPMode::ThreadSafe>();
		AnimTypeIDAnnotation.AddAnnotation(ControlRig, FControlRigAnimTypeIDsAnnotation{NewPtr});
		return NewPtr;
	}

	/** Find the anim-type ID for the specified scalar parameter */
	FMovieSceneAnimTypeID FindScalar(FName InParameterName)
	{
		return FindImpl(InParameterName, ScalarAnimTypeIDsByName);
	}
	/** Find the anim-type ID for the specified Vector2D parameter */
	FMovieSceneAnimTypeID FindVector2D(FName InParameterName)
	{
		return FindImpl(InParameterName, Vector2DAnimTypeIDsByName);
	}
	/** Find the anim-type ID for the specified vector parameter */
	FMovieSceneAnimTypeID FindVector(FName InParameterName)
	{
		return FindImpl(InParameterName, VectorAnimTypeIDsByName);
	}
	/** Find the anim-type ID for the specified transform parameter */
	FMovieSceneAnimTypeID FindTransform(FName InParameterName)
	{
		return FindImpl(InParameterName, TransformAnimTypeIDsByName);
	}
private:

	/** Sorted map should give the best trade-off for lookup speed with relatively small numbers of bones (O(log n)) */
	using MapType = TSortedMap<FName, FMovieSceneAnimTypeID, FDefaultAllocator, FNameFastLess>;

	static FMovieSceneAnimTypeID FindImpl(FName InParameterName, MapType& InOutMap)
	{
		if (const FMovieSceneAnimTypeID* Type = InOutMap.Find(InParameterName))
		{
			return *Type;
		}
		FMovieSceneAnimTypeID New = FMovieSceneAnimTypeID::Unique();
		InOutMap.Add(InParameterName, FMovieSceneAnimTypeID::Unique());
		return New;
	}
	/** Array of existing type identifiers */
	MapType ScalarAnimTypeIDsByName;
	MapType Vector2DAnimTypeIDsByName;
	MapType VectorAnimTypeIDsByName;
	MapType TransformAnimTypeIDsByName;
};

/**
 * Cache structure that is stored per-section that defines bitmasks for every
 * index within each curve type. Set bits denote that the curve should be 
 * evaluated. Only ever initialized once since the template will get re-created
 * whenever the control rig section changes
 */
struct FEvaluatedControlRigParameterSectionChannelMasks : IPersistentEvaluationData
{
	TBitArray<> ScalarCurveMask;
	TBitArray<> BoolCurveMask;
	TBitArray<> IntegerCurveMask;
	TBitArray<> EnumCurveMask;
	TBitArray<> Vector2DCurveMask;
	TBitArray<> VectorCurveMask;
	TBitArray<> ColorCurveMask;
	TBitArray<> TransformCurveMask;

	void Initialize(UMovieSceneControlRigParameterSection* Section,
		TArrayView<const FScalarParameterNameAndCurve> Scalars,
		TArrayView<const FBoolParameterNameAndCurve> Bools,
		TArrayView<const FIntegerParameterNameAndCurve> Integers,
		TArrayView<const FEnumParameterNameAndCurve> Enums,
		TArrayView<const FVector2DParameterNameAndCurves> Vector2Ds,
		TArrayView<const FVectorParameterNameAndCurves> Vectors,
		TArrayView<const FColorParameterNameAndCurves> Colors,
		TArrayView<const FTransformParameterNameAndCurves> Transforms
		)
	{
		const FChannelMapInfo* ChannelInfo = nullptr;

		ScalarCurveMask.Add(false, Scalars.Num());
		BoolCurveMask.Add(false, Bools.Num());
		IntegerCurveMask.Add(false, Integers.Num());
		EnumCurveMask.Add(false, Enums.Num());
		Vector2DCurveMask.Add(false, Vector2Ds.Num());
		VectorCurveMask.Add(false, Vectors.Num());
		ColorCurveMask.Add(false, Colors.Num());
		TransformCurveMask.Add(false, Transforms.Num());
		
		for (int32 Index = 0; Index < Scalars.Num(); ++Index)
		{
			const FScalarParameterNameAndCurve& Scalar = Scalars[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Scalar.ParameterName);
			ScalarCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Scalar.ParameterName));
		}
		for (int32 Index = 0; Index < Bools.Num(); ++Index)
		{
			const FBoolParameterNameAndCurve& Bool = Bools[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Bool.ParameterName);
			BoolCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Bool.ParameterName));
		}
		for (int32 Index = 0; Index < Integers.Num(); ++Index)
		{
			const FIntegerParameterNameAndCurve& Integer = Integers[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Integer.ParameterName);
			IntegerCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Integer.ParameterName));
		}
		for (int32 Index = 0; Index < Enums.Num(); ++Index)
		{
			const FEnumParameterNameAndCurve& Enum = Enums[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Enum.ParameterName);
			EnumCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Enum.ParameterName));
		}
		for (int32 Index = 0; Index < Vector2Ds.Num(); ++Index)
		{
			const FVector2DParameterNameAndCurves& Vector2D = Vector2Ds[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Vector2D.ParameterName);
			Vector2DCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Vector2D.ParameterName));
		}
		for (int32 Index = 0; Index < Vectors.Num(); ++Index)
		{
			const FVectorParameterNameAndCurves& Vector = Vectors[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Vector.ParameterName);
			VectorCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Vector.ParameterName));
		}
		for (int32 Index = 0; Index < Colors.Num(); ++Index)
		{
			const FColorParameterNameAndCurves& Color = Colors[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Color.ParameterName);
			ColorCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Color.ParameterName));
		}
		for (int32 Index = 0; Index < Transforms.Num(); ++Index)
		{
			const FTransformParameterNameAndCurves& Transform = Transforms[Index];
			ChannelInfo = Section->ControlChannelMap.Find(Transform.ParameterName);
			TransformCurveMask[Index] = (!ChannelInfo || Section->GetControlNameMask(Transform.ParameterName));
		}
	}
};
// Static hack because we cannot add this to the function parameters for EvaluateCurvesWithMasks due to hotfix restrictions
static FEvaluatedControlRigParameterSectionChannelMasks* HACK_ChannelMasks = nullptr;

struct FEvaluatedControlRigParameterSectionValues
{
	FEvaluatedControlRigParameterSectionValues() = default;

	FEvaluatedControlRigParameterSectionValues(FEvaluatedControlRigParameterSectionValues&&) = default;
	FEvaluatedControlRigParameterSectionValues& operator=(FEvaluatedControlRigParameterSectionValues&&) = default;

	// Non-copyable
	FEvaluatedControlRigParameterSectionValues(const FEvaluatedControlRigParameterSectionValues&) = delete;
	FEvaluatedControlRigParameterSectionValues& operator=(const FEvaluatedControlRigParameterSectionValues&) = delete;

	/** Array of evaluated scalar values */
	TArray<FScalarParameterStringAndValue, TInlineAllocator<2>> ScalarValues;
	/** Array of evaluated bool values */
	TArray<FBoolParameterStringAndValue, TInlineAllocator<2>> BoolValues;
	/** Array of evaluated integer values */
	TArray<FIntegerParameterStringAndValue, TInlineAllocator<2>> IntegerValues;
	/** Array of evaluated Spaces */
	TArray<FControlSpaceAndValue, TInlineAllocator<2>> SpaceValues;
	/** Array of evaluated vector2d values */
	TArray<FVector2DParameterStringAndValue, TInlineAllocator<2>> Vector2DValues;
	/** Array of evaluated vector values */
	TArray<FVectorParameterStringAndValue, TInlineAllocator<2>> VectorValues;
	/** Array of evaluated color values */
	TArray<FColorParameterStringAndValue, TInlineAllocator<2>> ColorValues;
	/** Array of evaluated transform values */
	TArray<FEulerTransformParameterStringAndValue, TInlineAllocator<2>> TransformValues;
	/** Array of evaluated constraint values */
	TArray<FConstraintAndActiveValue, TInlineAllocator<2>> ConstraintsValues;
};

/** Token for control rig control parameters */
struct FControlRigTrackTokenFloat
{
	FControlRigTrackTokenFloat() {}
	
	FControlRigTrackTokenFloat(float InValue)
		:Value(InValue)
	{}

	float Value;

};

struct FControlRigTrackTokenBool
{
	FControlRigTrackTokenBool() {}

	FControlRigTrackTokenBool(bool InValue)
		:Value(InValue)
	{}

	bool Value;
};

struct FControlRigTrackTokenVector2D
{
	FControlRigTrackTokenVector2D() {}
	FControlRigTrackTokenVector2D(FVector2D InValue)
		:Value(InValue)
	{}

	FVector2D Value;
};

struct FControlRigTrackTokenVector
{
	FControlRigTrackTokenVector() {}
	FControlRigTrackTokenVector(FVector InValue)
		:Value(InValue)
	{}

	FVector Value;
};

struct FControlRigTrackTokenTransform
{
	FControlRigTrackTokenTransform() {}
	FControlRigTrackTokenTransform(FEulerTransform InValue)
		: Value(InValue)
	{}
	FEulerTransform Value;

};



// Specify a unique runtime type identifier for rig control track tokens
template<> FMovieSceneAnimTypeID GetBlendingDataType<FControlRigTrackTokenFloat>()
{
	static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique();
	return TypeId;
}

template<> FMovieSceneAnimTypeID GetBlendingDataType<FControlRigTrackTokenBool>()
{
	static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique();
	return TypeId;
}

template<> FMovieSceneAnimTypeID GetBlendingDataType<FControlRigTrackTokenVector2D>()
{
	static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique();
	return TypeId;
}

template<> FMovieSceneAnimTypeID GetBlendingDataType<FControlRigTrackTokenVector>()
{
	static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique();
	return TypeId;
}

template<> FMovieSceneAnimTypeID GetBlendingDataType<FControlRigTrackTokenTransform>()
{
	static FMovieSceneAnimTypeID TypeId = FMovieSceneAnimTypeID::Unique();
	return TypeId;
}



/** Define working data types for blending calculations  */
template<>  struct TBlendableTokenTraits<FControlRigTrackTokenFloat>
{
	typedef UE::MovieScene::TMaskedBlendable<float, 1> WorkingDataType;
};

template<>  struct TBlendableTokenTraits<FControlRigTrackTokenBool>
{
	typedef UE::MovieScene::TMaskedBlendable<bool, 1> WorkingDataType;
};

template<> struct TBlendableTokenTraits<FControlRigTrackTokenVector2D>
{
	typedef UE::MovieScene::TMaskedBlendable<float, 2> WorkingDataType;
};

template<> struct TBlendableTokenTraits<FControlRigTrackTokenVector>
{
	typedef UE::MovieScene::TMaskedBlendable<float, 3> WorkingDataType;
};

template<>  struct TBlendableTokenTraits<FControlRigTrackTokenTransform>
{
	typedef UE::MovieScene::TMaskedBlendable<float, 9> WorkingDataType;
};




namespace UE
{
namespace MovieScene
{

	void MultiChannelFromData(const FControlRigTrackTokenFloat& In, TMultiChannelValue<float, 1>& Out)
	{
		Out = { In.Value };
	}

	void ResolveChannelsToData(const TMultiChannelValue<float, 1>& In, FControlRigTrackTokenFloat& Out)
	{
		Out.Value = In[0];
	}

	void MultiChannelFromData(const FControlRigTrackTokenBool& In, TMultiChannelValue<bool, 1>& Out)
	{
		Out = { In.Value };
	}

	void ResolveChannelsToData(const TMultiChannelValue<bool, 1>& In, FControlRigTrackTokenBool& Out)
	{
		Out.Value = In[0];
	}

	void MultiChannelFromData(const FControlRigTrackTokenVector2D& In, TMultiChannelValue<float, 2>& Out)
	{
		Out = { In.Value.X, In.Value.Y };
	}

	void ResolveChannelsToData(const TMultiChannelValue<float, 2>& In, FControlRigTrackTokenVector2D& Out)
	{
		Out.Value = FVector2D(In[0], In[1]);
	}

	void MultiChannelFromData(const FControlRigTrackTokenVector& In, TMultiChannelValue<float, 3>& Out)
	{
		Out = { In.Value.X, In.Value.Y, In.Value.Z };
	}

	void ResolveChannelsToData(const TMultiChannelValue<float, 3>& In, FControlRigTrackTokenVector& Out)
	{
		Out.Value = FVector(In[0], In[1], In[2]);
	}

	void MultiChannelFromData(const FControlRigTrackTokenTransform& In, TMultiChannelValue<float, 9>& Out)
	{
		FVector Translation = In.Value.GetLocation();
		FVector Rotation = In.Value.Rotator().Euler();
		FVector Scale = In.Value.GetScale3D();
		Out = { Translation.X, Translation.Y, Translation.Z, Rotation.X, Rotation.Y, Rotation.Z, Scale.X, Scale.Y, Scale.Z };

	}

	void ResolveChannelsToData(const TMultiChannelValue<float, 9>& In, FControlRigTrackTokenTransform& Out)
	{
		Out.Value = FEulerTransform(
			FRotator::MakeFromEuler(FVector(In[3], In[4], In[5])),
			FVector(In[0], In[1], In[2]),
			FVector(In[6], In[7], In[8])
		);
	}

} // namespace MovieScene
} // namespace UE

//since initialization can blow up selection, may need to just reselect, used in a few places
static void SelectControls(UControlRig* ControlRig, TArray<FName>& SelectedNames)
{
	if (ControlRig)
	{
		ControlRig->ClearControlSelection();
		for (const FName& Name : SelectedNames)
		{
			ControlRig->SelectControl(Name, true);
		}
	}
}

bool FControlRigBindingHelper::BindToSequencerInstance(UControlRig* ControlRig)
{
	if (!ControlRig)
	{
		return false;
	}
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
	{
		if(SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			bool bWasCreated = false;
			if (UControlRigLayerInstance* AnimInstance = FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UControlRigLayerInstance>(SkeletalMeshComponent, bWasCreated))
			{
				// Enter the refresh block when: this is a fresh AnimInstance, OR the rig is not
				// yet tracked, OR the rig has been re-initialized in place (Runtime Asset recompile).
				// The last case is specific to Runtime Asset-backed rigs — BP rigs get a new
				// UniqueID from reinstancing, so !HasControlRigTrack handles them naturally.
				if (bWasCreated || !AnimInstance->HasControlRigTrack(ControlRig->GetUniqueID()) || ControlRig->NeedsInstanceInitialization())
				{
					AnimInstance->RecalcRequiredBones();
					AnimInstance->AddControlRigTrack(ControlRig->GetUniqueID(), ControlRig);
					//initialization can blow up selection
					TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
					ControlRig->Initialize();
					ControlRig->RequestInit();
					ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkeletalMeshComponent, true);
					ControlRig->Evaluate_AnyThread();
					ControlRig->ClearInstanceInitializationPending();
					TArray<FName> NewSelectedControls = ControlRig->CurrentControlSelection();
					if (SelectedControls != NewSelectedControls)
					{
						SelectControls(ControlRig, SelectedControls);
					}
				}
			}
			return bWasCreated;
		}
	}
	else if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
	{
		if (ControlRigComponent->GetControlRig() != ControlRig)
		{
			ControlRigComponent->Initialize();
			/*
			Previously with Sequencer and CR Components we would assign the CR to a Component
			that the sequencer was using, in any world. This looks like it was causing issues
			with two worlds running with pre-forward solve events so now we only do that if
			in non-game, and if in game (which includes PIE), we don't re-set the
			CR Component's CR, but instead grab the CR from it and then use that for evaluation.
			*/
			if (ControlRigComponent->GetWorld())
			{
				if (ControlRigComponent->GetWorld()->IsGameWorld() == false)
				{
					ControlRigComponent->SetControlRig(ControlRig);
				}
			}
		}
	}
	return false;
}

void FControlRigBindingHelper::UnBindFromSequencerInstance(UControlRig* ControlRig)
{
	check(ControlRig);

	if (!ControlRig->IsValidLowLevel() ||
	    URigVMHost::IsGarbageOrDestroyed(ControlRig) ||
		!IsValid(ControlRig))
	{
		return;
	}
	
	if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
	{
		// todo: how do we reset the state?
		//ControlRig->Initialize();
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
	{
		if (!SkeletalMeshComponent->IsValidLowLevel() ||
			URigVMHost::IsGarbageOrDestroyed(SkeletalMeshComponent) ||
			!IsValid(SkeletalMeshComponent))
		{
			return;
		}

		UControlRigLayerInstance* AnimInstance = Cast<UControlRigLayerInstance>(SkeletalMeshComponent->GetAnimInstance());
		bool bShouldUnbind = true;
		if (AnimInstance)
		{
			if (!AnimInstance->IsValidLowLevel() ||
                URigVMHost::IsGarbageOrDestroyed(AnimInstance) ||
                !IsValid(AnimInstance))
			{
				return;
			}

			AnimInstance->ResetNodes();
			AnimInstance->RecalcRequiredBones();
			AnimInstance->RemoveControlRigTrack(ControlRig->GetUniqueID());

			bShouldUnbind = AnimInstance->GetFirstAvailableControlRig() == nullptr;
		}

		if (bShouldUnbind)
		{
			FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent< UControlRigLayerInstance>(SkeletalMeshComponent);
		}
	}
}

struct FControlRigSkeletalMeshComponentBindingTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	FControlRigSkeletalMeshComponentBindingTokenProducer(FMovieSceneSequenceIDRef InSequenceID, UControlRig* InControlRig)
		: SequenceID(InSequenceID), ControlRigUniqueID(InControlRig->GetUniqueID())
	{}

	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FControlRigSkeletalMeshComponentBindingTokenProducer>();
	}

	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			FToken(FMovieSceneSequenceIDRef InSequenceID, uint32 InControlRigUniqueID)
				: SequenceID(InSequenceID), ControlRigUniqueID(InControlRigUniqueID)
			{

			}
			
			virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params) override
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(&InObject))
				{
					if (UControlRigLayerInstance* ControlRigLayerInstance = Cast<UControlRigLayerInstance>(SkeletalMeshComponent->GetAnimInstance()))
					{
						ControlRigLayerInstance->RemoveControlRigTrack(ControlRigUniqueID);
						if (!ControlRigLayerInstance->GetFirstAvailableControlRig())
						{
							FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<UControlRigLayerInstance>(SkeletalMeshComponent);
						}
					}
				}
			}

			FMovieSceneSequenceID SequenceID;
			uint32 ControlRigUniqueID;
		};


		FToken Token(SequenceID, ControlRigUniqueID);
		return MoveTemp(Token);
	}

	FMovieSceneSequenceID SequenceID;
	uint32 ControlRigUniqueID;
};

struct FControlRigParameterPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	FControlRigParameterPreAnimatedTokenProducer(FMovieSceneSequenceIDRef InSequenceID)
		: SequenceID(InSequenceID)
	{}

	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{

		struct FToken : IMovieScenePreAnimatedToken
		{
			FToken(FMovieSceneSequenceIDRef InSequenceID)
				: SequenceID(InSequenceID)
			{

			}
			void SetSkelMesh(USkeletalMeshComponent* InComponent)
			{
				SkeletalMeshRestoreState.SaveState(InComponent);
				AnimationMode = InComponent->GetAnimationMode();
			}

			virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params) override
			{

				if (UControlRig* ControlRig = Cast<UControlRig>(&InObject))
				{
					if (ControlRig->GetObjectBinding())
					{
						{
							// Reduce the EvaluationMutex lock to only the absolutely necessary
							// After locking, the call to UnBindFromSequencerInstance might wait for parallel evaluations to finish
							// and some control rigs might have been queued to evaluate (but haven's started yet), so they could get stuck
							// waiting on the evaluation lock.

							
							// control rig evaluate critical section: when restoring the state, we can be poking into running instances of Control Rigs
							// on the anim thread so using a lock here to avoid this thread and anim thread both touching the rig
							// at the same time, which can lead to various issues like double-freeing some random array when doing SetControlValue
							// note: the critical section accepts recursive locking so it is ok that we
							// are calling evaluate_anythread later within the same scope
							UE::TScopeLock EvaluateLock(ControlRig->GetEvaluateMutex());
						
							//Restore control rig first
							const bool bSetupUndo = false;
							if (URigHierarchy* RigHierarchy = ControlRig->GetHierarchy())
							{
								FRigElementKey ControlKey;
								ControlKey.Type = ERigElementType::Control;
								for (FControlSpaceAndValue& SpaceNameAndValue : SpaceValues)
								{
									ControlKey.Name = SpaceNameAndValue.ControlName;
									switch (SpaceNameAndValue.Value.SpaceType)
									{
										case EMovieSceneControlRigSpaceType::Parent:
											ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
										break;
										case EMovieSceneControlRigSpaceType::World:
											ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
										break;
										case EMovieSceneControlRigSpaceType::ControlRig:
										{
#if WITH_EDITOR
											ControlRig->SwitchToParent(ControlKey, SpaceNameAndValue.Value.ControlRigElement, false, true);
#else
											ControlRig->SwitchToParent(ControlKey, SpaceNameAndValue.Value.ControlRigElement, false, true);
#endif	
										}
										break;
									}
								}
							

								for (TNameAndValue<float>& Value : ScalarValues)
								{
									if (ControlRig->FindControl(Value.Name))
									{
										ControlRig->SetControlValue<float>(Value.Name, Value.Value, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
									}
								}

								for (TNameAndValue<bool>& Value : BoolValues)
								{
									if (ControlRig->FindControl(Value.Name))
									{
										ControlRig->SetControlValue<bool>(Value.Name, Value.Value, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
									}
								}

								for (TNameAndValue<int32>& Value : IntegerValues)
								{
									if (ControlRig->FindControl(Value.Name))
									{
										ControlRig->SetControlValue<int32>(Value.Name, Value.Value, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
									}
								}
								for (int32 TwiceHack = 0; TwiceHack < 2; ++TwiceHack)
								{
									for (TNameAndValue<FVector2D>& Value : Vector2DValues)
									{
										if (ControlRig->FindControl(Value.Name))
										{
											const FVector3f Vector3(Value.Value.X, Value.Value.Y, 0.f);
											//okay to use vector3 for 2d here
											ControlRig->SetControlValue<FVector3f>(Value.Name, Vector3, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
										}
									}

									for (TNameAndValue<FVector>& Value : VectorValues)
									{
										if (FRigControlElement* ControlElement = ControlRig->FindControl(Value.Name))
										{
											if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
											{
												RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, Value.Value);
											}
											ControlRig->SetControlValue<FVector3f>(Value.Name, (FVector3f)Value.Value, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
										}
									}

									for (TNameAndValue<FEulerTransform>& Value : TransformValues)
									{
										if (FRigControlElement* ControlElement = ControlRig->FindControl(Value.Name))
										{
											switch (ControlElement->Settings.ControlType)
											{
											case ERigControlType::Transform:
												{
													FVector EulerAngle(Value.Value.Rotation.Roll, Value.Value.Rotation.Pitch, Value.Value.Rotation.Yaw);
													RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
													ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(Value.Name, Value.Value.ToFTransform(), true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
													break;
												}
											case ERigControlType::TransformNoScale:
												{
													FTransformNoScale NoScale = Value.Value.ToFTransform();
													FVector EulerAngle(Value.Value.Rotation.Roll, Value.Value.Rotation.Pitch, Value.Value.Rotation.Yaw);
													RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
													ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(Value.Name, NoScale, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
													break;
												}
											case ERigControlType::EulerTransform:
												{
													FEulerTransform EulerTransform = Value.Value;
													FVector EulerAngle(Value.Value.Rotation.Roll, Value.Value.Rotation.Pitch, Value.Value.Rotation.Yaw);
													RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
													ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(Value.Name, EulerTransform, true, FRigControlModifiedContext(EControlRigSetKey::Never), bSetupUndo);
													break;
												}
											default:
												{
													break;
												}
											}
										}
									}
								}
							}
							//make sure to evaluate the control rig
							ControlRig->Evaluate_AnyThread();
						}

						//unbind instances and reset animbp
						FControlRigBindingHelper::UnBindFromSequencerInstance(ControlRig);

						//do a tick and restore skel mesh
						if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
						{
							// If the skel mesh comp owner has been removed from the world, no need to restore anything
							if (SkeletalMeshComponent->IsRegistered())
							{
								// Restore pose after unbinding to force the restored pose
								SkeletalMeshComponent->SetUpdateAnimationInEditor(true);
								SkeletalMeshComponent->SetUpdateClothInEditor(true);
								if (!SkeletalMeshComponent->IsPostEvaluatingAnimation())
								{
									SkeletalMeshComponent->TickAnimation(0.f, false);
									SkeletalMeshComponent->RefreshBoneTransforms();
									SkeletalMeshComponent->RefreshFollowerComponents();
									SkeletalMeshComponent->UpdateComponentToWorld();
									SkeletalMeshComponent->FinalizeBoneTransform();
									SkeletalMeshComponent->MarkRenderTransformDirty();
									SkeletalMeshComponent->MarkRenderDynamicDataDirty();
								}
								SkeletalMeshRestoreState.RestoreState();

								if (SkeletalMeshComponent->GetAnimationMode() != AnimationMode)
								{
									SkeletalMeshComponent->SetAnimationMode(AnimationMode);
								}
							}
						}
						//only unbind if not a component
						if (Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()) == nullptr)
						{
							ControlRig->GetObjectBinding()->UnbindFromObject();
						}

					}
				}
			}

			FMovieSceneSequenceID SequenceID;
			TArray< FControlSpaceAndValue> SpaceValues;
			TArray< TNameAndValue<float> > ScalarValues;
			TArray< TNameAndValue<bool> > BoolValues;
			TArray< TNameAndValue<int32> > IntegerValues;
			TArray< TNameAndValue<FVector> > VectorValues;
			TArray< TNameAndValue<FVector2D> > Vector2DValues;
			TArray< TNameAndValue<FEulerTransform> > TransformValues;
			FSkeletalMeshRestoreState SkeletalMeshRestoreState;
			EAnimationMode::Type AnimationMode;
		};


		FToken Token(SequenceID);

		if (UControlRig* ControlRig = Cast<UControlRig>(&Object))
		{
			URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();
			TArray<FRigControlElement*> Controls = ControlRig->AvailableControls();
			FRigControlValue Value;
			
			for (FRigControlElement* ControlElement : Controls)
			{
				switch (ControlElement->Settings.ControlType)
				{
					case ERigControlType::Bool:
					{
						const bool Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<bool>();
						Token.BoolValues.Add(TNameAndValue<bool>{ ControlElement->GetFName(), Val });
						break;
					}
					case ERigControlType::Integer:
					{
						const int32 Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int32>();
						Token.IntegerValues.Add(TNameAndValue<int32>{ ControlElement->GetFName(), Val });
						break;
					}
					case ERigControlType::Float:
					case ERigControlType::ScaleFloat:
					{
						const float Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
						Token.ScalarValues.Add(TNameAndValue<float>{ ControlElement->GetFName(), Val });
						break;
					}
					case ERigControlType::Vector2D:
					{
						const FVector3f Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
						Token.Vector2DValues.Add(TNameAndValue<FVector2D>{ ControlElement->GetFName(), FVector2D(Val.X, Val.Y) });
						break;
					}
					case ERigControlType::Position:
					case ERigControlType::Scale:
					case ERigControlType::Rotator:
					{
						FMovieSceneControlRigSpaceBaseKey SpaceValue;
						//@helge how can we get the correct current space here? this is for restoring it.
						//for now just using parent space
						//FRigElementKey DefaultParent = RigHierarchy->GetFirstParent(ControlElement->GetKey());
						SpaceValue.ControlRigElement = ControlElement->GetKey();
						SpaceValue.SpaceType = EMovieSceneControlRigSpaceType::Parent;
						Token.SpaceValues.Add(FControlSpaceAndValue(ControlElement->GetFName(), SpaceValue));
						FVector3f Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
						if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
						{
							FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
							Val = FVector3f(Vector.X, Vector.Y, Vector.Z);
						}
						Token.VectorValues.Add(TNameAndValue<FVector>{ ControlElement->GetFName(), (FVector)Val });
						//mz todo specify rotator special so we can do quat interps
						break;
					}
					case ERigControlType::Transform:
					{
						FMovieSceneControlRigSpaceBaseKey SpaceValue;
						//@helge how can we get the correct current space here? this is for restoring it.
						//for now just using parent space
						//FRigElementKey DefaultParent = RigHierarchy->GetFirstParent(ControlElement->GetKey());
						SpaceValue.ControlRigElement = ControlElement->GetKey();
						SpaceValue.SpaceType = EMovieSceneControlRigSpaceType::Parent;
						Token.SpaceValues.Add(FControlSpaceAndValue(ControlElement->GetFName(), SpaceValue));
						const FTransform Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>().ToTransform();
						FEulerTransform EulerTransform(Val);
						FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
						EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
						Token.TransformValues.Add(TNameAndValue<FEulerTransform>{ ControlElement->GetFName(), EulerTransform });
						break;
					}
					case ERigControlType::TransformNoScale:
					{
						const FTransformNoScale NoScale = 
							ControlRig
							->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
						FEulerTransform EulerTransform(NoScale.ToFTransform());
						FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
						EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
						Token.TransformValues.Add(TNameAndValue<FEulerTransform>{ ControlElement->GetFName(), EulerTransform });
						break;
					}
					case ERigControlType::EulerTransform:
					{
						FMovieSceneControlRigSpaceBaseKey SpaceValue;
						//@helge how can we get the correct current space here? this is for restoring it.
						//for now just using parent space
						//FRigElementKey DefaultParent = RigHierarchy->GetFirstParent(ControlElement->GetKey());
						SpaceValue.ControlRigElement = ControlElement->GetKey();
						SpaceValue.SpaceType = EMovieSceneControlRigSpaceType::Parent;
						Token.SpaceValues.Add(FControlSpaceAndValue(ControlElement->GetFName(), SpaceValue));
						FEulerTransform EulerTransform = 
							ControlRig
							->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
						FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
						EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
						Token.TransformValues.Add(TNameAndValue<FEulerTransform>{ ControlElement->GetFName(), EulerTransform });
						break;
					}
				}
			}

			if (ControlRig->GetObjectBinding())
			{
				if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
				{
					if (ControlRigComponent->GetControlRig() != ControlRig)
					{
						ControlRigComponent->Initialize();
						/*
						Previously with Sequencer and CR Components we would assign the CR to a Component 
						that the sequencer was using, in any world. This looks like it was causing issues 
						with two worlds running with pre-forward solve events so now we only do that if 
						in non-game, and if in game (which includes PIE), we don't re-set the 
						CR Component's CR, but instead grab the CR from it and then use that for evaluation.
						*/
						if (ControlRigComponent->GetWorld())
						{
							if (ControlRigComponent->GetWorld()->IsGameWorld() == false)
							{
								ControlRigComponent->SetControlRig(ControlRig);
							}
						}
					}
					else
					{
						TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
						ControlRig->Initialize();
						TArray<FName> NewSelectedControls = ControlRig->CurrentControlSelection();
						if (SelectedControls != NewSelectedControls)
						{
							SelectControls(ControlRig,SelectedControls);
						}
					}
				}
				else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
				{
					Token.SetSkelMesh(SkeletalMeshComponent);
				}
			}
		}

		return MoveTemp(Token);
	}

	FMovieSceneSequenceID SequenceID;
	TArray< FControlSpaceAndValue> SpaceValues;
	TArray< TNameAndValue<bool> > BoolValues;
	TArray< TNameAndValue<int32> > IntegerValues;
	TArray< TNameAndValue<float> > ScalarValues;
	TArray< TNameAndValue<FVector2D> > Vector2DValues;
	TArray< TNameAndValue<FVector> > VectorValues;
	TArray< TNameAndValue<FTransform> > TransformValues;

};

static UControlRig* GetControlRig(const UMovieSceneControlRigParameterSection* Section, UObject* BoundObject)
{
	UWorld* GameWorld = (BoundObject && BoundObject->GetWorld() && BoundObject->GetWorld()->IsGameWorld()) ? BoundObject->GetWorld() : nullptr;
	UControlRig* ControlRig =  Section->GetControlRig(GameWorld, BoundObject);
	if (ControlRig && ControlRig->GetObjectBinding())
	{
		if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
		{
			if (AActor* Actor = Cast<AActor>(BoundObject))
			{
				if (UControlRigComponent* NewControlRigComponent = Actor->FindComponentByClass<UControlRigComponent>())
				{
					if (NewControlRigComponent->GetWorld())
					{
						if (NewControlRigComponent->GetWorld()->IsGameWorld())
						{
							ControlRig = NewControlRigComponent->GetControlRig();
							if (!ControlRig)
							{
								NewControlRigComponent->Initialize();
								ControlRig = NewControlRigComponent->GetControlRig();
							}
							if (ControlRig)
							{
								if (ControlRig->GetObjectBinding() == nullptr)
								{
									ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
								}
								if (ControlRig->GetObjectBinding()->GetBoundObject() != NewControlRigComponent)
								{
									ControlRig->GetObjectBinding()->BindToObject(BoundObject);
								}
							}
						}
						else if (NewControlRigComponent != ControlRigComponent)
						{
							NewControlRigComponent->SetControlRig(ControlRig);
						}
					}
				}
			}
		}
		else if (UControlRigComponent* NewControlRigComponent = Cast<UControlRigComponent>(BoundObject))
		{
			if (NewControlRigComponent->GetWorld())
			{
				if (NewControlRigComponent->GetWorld()->IsGameWorld())
				{
					ControlRig = NewControlRigComponent->GetControlRig();
					if (!ControlRig)
					{
						NewControlRigComponent->Initialize();
						ControlRig = NewControlRigComponent->GetControlRig();
					}
					if (ControlRig)
					{
						if (ControlRig->GetObjectBinding() == nullptr)
						{
							ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
						}
						if (ControlRig->GetObjectBinding()->GetBoundObject() != NewControlRigComponent)
						{
							ControlRig->GetObjectBinding()->BindToObject(BoundObject);
						}
					}
				}
				else if (NewControlRigComponent != ControlRigComponent)
				{
					NewControlRigComponent->SetControlRig(ControlRig);
				}
			}
		}
	}
	else
	{
		return nullptr;
	}
	return ControlRig;
}

static UTickableConstraint* CreateConstraintIfNeeded(UWorld* InWorld, const FConstraintAndActiveValue& InConstraintValue, UMovieSceneControlRigParameterSection* InSection)
{
	UTickableConstraint* Constraint = InConstraintValue.Constraint.Get();
	if (!Constraint) 
	{
		return nullptr;
	}
	
	// it's possible that we have it but it's not in the manager, due to manager not being saved with it (due to spawning or undo/redo).
	if (InWorld)
	{
		if (!FConstraintsManagerController::Get(InWorld).GetConstraint(Constraint->ConstraintID))
		{
			FConstraintsManagerController::Get(InWorld).AddConstraint(InConstraintValue.Constraint.Get());
			//need to reconstuct channels here.. note this is now lazy and so will recreate it next time view requests it
			//but only do it if the control rig has a valid world it may not for example in PIE
			if (InSection->GetControlRig() && InSection->GetControlRig()->GetWorld())
			{
				InSection->ReconstructChannelProxy();
				InSection->MarkAsChanged();
			}
		}
	}

	return Constraint;
}

/* Simple token used for non-blendables*/
struct FControlRigParameterExecutionToken : IMovieSceneExecutionToken
{
	FControlRigParameterExecutionToken(const UMovieSceneControlRigParameterSection* InSection,
		const FEvaluatedControlRigParameterSectionValues& Values, FMovieSceneControlRigParameterTemplate* InTemplate)
	:	Section(InSection),
		Template(InTemplate)
	{
		BoolValues = Values.BoolValues;
		IntegerValues = Values.IntegerValues;
		SpaceValues = Values.SpaceValues;
		ConstraintsValues = Values.ConstraintsValues;
	}
	FControlRigParameterExecutionToken(FControlRigParameterExecutionToken&&) = default;
	FControlRigParameterExecutionToken& operator=(FControlRigParameterExecutionToken&&) = default;

	// Non-copyable
	FControlRigParameterExecutionToken(const FControlRigParameterExecutionToken&) = delete;
	FControlRigParameterExecutionToken& operator=(const FControlRigParameterExecutionToken&) = delete;

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ControlRigParameterTrack_TokenExecute)
		
		if (!Section)
		{
			return;
		}
		FMovieSceneSequenceID SequenceID = Operand.SequenceID;
		TArrayView<TWeakObjectPtr<>> BoundObjects = Player.FindBoundObjects(Operand);
		const UMovieSceneSequence* Sequence = Player.GetEvaluationState()->FindSequence(Operand.SequenceID);
		UControlRig* ControlRig = nullptr;
		UObject* BoundObject = BoundObjects.Num() > 0 ? BoundObjects[0].Get() : nullptr;
		const FFrameTime Time = Section->RemapTime(Context.GetTime());

		if (Sequence && BoundObject)
		{
			UWorld* GameWorld = (BoundObject->GetWorld() && BoundObject->GetWorld()->IsGameWorld()) ? BoundObject->GetWorld() : nullptr;
			ControlRig = Section->GetControlRig(GameWorld, BoundObject);
			if (!ControlRig)
			{
				return;
			}
			if (!ControlRig->GetObjectBinding())
			{
				ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
			}

			if (ControlRig->GetObjectBinding()->GetBoundObject() != FControlRigObjectBinding::GetBindableObject(BoundObject))
			{
				ControlRig->GetObjectBinding()->BindToObject(BoundObject);
				TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
				ControlRig->Initialize();
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(BoundObject)))
				{
					ControlRig->RequestInit();
					ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkeletalMeshComponent, true);
					ControlRig->Evaluate_AnyThread();
				};
				if (GameWorld == nullptr && ControlRig->IsA<UFKControlRig>())// mz only in editor replace Fk Control rig, will look post 29.20 to see if this really needed but want to unblock folks
				{
					UMovieSceneControlRigParameterTrack* Track = Section->GetTypedOuter<UMovieSceneControlRigParameterTrack>();
					if (Track)
					{
						Track->ReplaceControlRig(ControlRig, true);
					}
				}
				TArray<FName> NewSelectedControls = ControlRig->CurrentControlSelection();
				if (SelectedControls != NewSelectedControls)
				{
					SelectControls(ControlRig, SelectedControls);
				}
			}

			// make sure to pick the correct CR instance for the  Components to bind.
			// In case of PIE + Spawnable Actor + CR component, sequencer should grab
			// CR component's CR instance for evaluation, see comment in BindToSequencerInstance
			// i.e. CR component should bind to the instance that it owns itself.
			ControlRig = GetControlRig(Section, BoundObjects[0].Get());
			if (!ControlRig)
			{
				return;
			}
			// ensure that pre animated state is saved, must be done before bind
			Player.SavePreAnimatedState(*ControlRig, FMovieSceneControlRigParameterTemplate::GetAnimTypeID(), FControlRigParameterPreAnimatedTokenProducer(Operand.SequenceID));
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(BoundObject)))
			{
				Player.SavePreAnimatedState(*SkeletalMeshComponent, FControlRigSkeletalMeshComponentBindingTokenProducer::GetAnimTypeID(), FControlRigSkeletalMeshComponentBindingTokenProducer(Operand.SequenceID, ControlRig));
			}

#if WITH_EDITOR
			TWeakObjectPtr<UAnimInstance> PreviousAnimInstanceWeakPtr = nullptr;
			
			if (ControlRig->GetObjectBinding())
			{
				if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
				{
					PreviousAnimInstanceWeakPtr = SkeletalMeshComponent->GetAnimInstance();
				}
			}
#endif

			bool bWasCreated = FControlRigBindingHelper::BindToSequencerInstance(ControlRig);

			if (ControlRig->GetObjectBinding())
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
				{
					if (UControlRigLayerInstance* AnimInstance = Cast<UControlRigLayerInstance>(SkeletalMeshComponent->GetAnimInstance()))
					{
#if WITH_EDITOR
						if (GEditor && bWasCreated)
						{
							TWeakObjectPtr<USkeletalMeshComponent> WeakSkeletalMeshComponentPtr = SkeletalMeshComponent;
							FDelegateHandle PreCompileHandle = GEditor->OnBlueprintPreCompile().AddLambda([PreviousAnimInstanceWeakPtr, WeakSkeletalMeshComponentPtr](const UBlueprint* InBlueprint)
							{
								const TStrongObjectPtr<UAnimInstance> PinnedAnimInstance = PreviousAnimInstanceWeakPtr.Pin();
								const TStrongObjectPtr<USkeletalMeshComponent> PinnedSkeletalMeshComponent = WeakSkeletalMeshComponentPtr.Pin();
								if (PinnedAnimInstance && PinnedSkeletalMeshComponent && InBlueprint && PinnedAnimInstance->GetClass() == InBlueprint->GeneratedClass)
								{
									FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<UControlRigLayerInstance>(PinnedSkeletalMeshComponent.Get());
								}
							});
							
							Template->PreCompileHandles.Add(PreCompileHandle);
						}
						
#endif
						float Weight = Section->EvaluateEasing(Time);
						if (EnumHasAllFlags(Section->TransformMask.GetChannels(), EMovieSceneTransformChannel::Weight))
						{
							float ManualWeight = 1.f;
							Section->Weight.Evaluate(Time, ManualWeight);
							Weight *= ManualWeight;
						}
						FControlRigIOSettings InputSettings;
						InputSettings.bUpdateCurves = true;
						InputSettings.bUpdatePose = true;
						//this is not great but assumes we have 1 absolute track that will be used for weighting
						if (Section->GetBlendType() == EMovieSceneBlendType::Absolute)
						{
							AnimInstance->UpdateControlRigTrack(ControlRig->GetUniqueID(), Weight, InputSettings, true);
						}
					}
				}
				else
				{
					ControlRig = GetControlRig(Section, BoundObject);
				}
			}
		}		

		//Do Bool straight up no blending
		if (Section->GetBlendType().IsValid() == false ||  Section->GetBlendType().Get() != EMovieSceneBlendType::Additive)
		{
			bool bWasDoNotKey = false;

			bWasDoNotKey = Section->GetDoNotKey();
			Section->SetDoNotKey(true);

			if (ControlRig)
			{
				const bool bSetupUndo = false;
				ControlRig->SetAbsoluteTime((float)Context.GetFrameRate().AsSeconds(Time));
				for (const FControlSpaceAndValue& SpaceNameAndValue : SpaceValues)
				{
					if (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(SpaceNameAndValue.ControlName))
					{
						if (URigHierarchy* RigHierarchy = ControlRig->GetHierarchy())
						{
							if (FRigControlElement* RigControl = ControlRig->FindControl(SpaceNameAndValue.ControlName))
							{
								const FRigElementKey ControlKey = RigControl->GetKey();

								switch (SpaceNameAndValue.Value.SpaceType)
								{
									case EMovieSceneControlRigSpaceType::Parent:
										ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
										break;
									case EMovieSceneControlRigSpaceType::World:
										ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
										break;
									case EMovieSceneControlRigSpaceType::ControlRig:
										{
#if WITH_EDITOR
											ControlRig->SwitchToParent(ControlKey, SpaceNameAndValue.Value.ControlRigElement, false, true);
#else
											ControlRig->SwitchToParent(ControlKey, SpaceNameAndValue.Value.ControlRigElement, false, true);
#endif	
										}
										break;
								}
							}
						}
						/*
						FRigControlElement* RigControl = ControlRig->FindControl(SpaceNameAndValue.ParameterName);
						if (RigControl && RigControl->Settings.ControlType == ERigControlType::Bool)
						{
							ControlRig->SetControlValue<bool>(SpaceNameAndValue.SpaceNameAndValue, SpaceNameAndValue.Value, true, EControlRigSetKey::Never, bSetupUndo);
						}
						*/
					}
				}


				for (const FBoolParameterStringAndValue& BoolNameAndValue : BoolValues)
				{
					if (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(BoolNameAndValue.ParameterName))
					{
						FRigControlElement* RigControl = ControlRig->FindControl(BoolNameAndValue.ParameterName);
						if (RigControl && RigControl->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
							RigControl->Settings.AnimationType != ERigControlAnimationType::VisualCue
							&& RigControl->Settings.ControlType == ERigControlType::Bool)
						{
							ControlRig->SetControlValue<bool>(BoolNameAndValue.ParameterName, BoolNameAndValue.Value, true, EControlRigSetKey::Never,bSetupUndo);
						}
					}
				}

				for (const FIntegerParameterStringAndValue& IntegerNameAndValue : IntegerValues)
				{
					if (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(IntegerNameAndValue.ParameterName))
					{
						FRigControlElement* RigControl = ControlRig->FindControl(IntegerNameAndValue.ParameterName);
						if (RigControl && RigControl->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
							RigControl->Settings.AnimationType != ERigControlAnimationType::VisualCue
							&& RigControl->Settings.ControlType == ERigControlType::Integer)
						{
							ControlRig->SetControlValue<int32>(IntegerNameAndValue.ParameterName, IntegerNameAndValue.Value, true, EControlRigSetKey::Never,bSetupUndo);
						}
					}
				}
				if (BoundObject)
				{
					UWorld* BoundObjectWorld = BoundObject->GetWorld();
					
					USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(BoundObject));
					auto MarkRigForEvaluationIfAdditive = [ControlRig, SkeletalMeshComponent]()
					{
						if (TransformableHandleUtils::SkipTicking())
						{
							if (SkeletalMeshComponent && ControlRig && ControlRig->IsAdditive())
							{
								TransformableHandleUtils::MarkComponentForEvaluation(SkeletalMeshComponent);
							}
						}
					};
										
					TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = Player.GetSharedPlaybackState();
					for (FConstraintAndActiveValue& ConstraintValue : ConstraintsValues)
					{
						UMovieSceneControlRigParameterSection* NonConstSection = const_cast<UMovieSceneControlRigParameterSection*>(Section);
						CreateConstraintIfNeeded(BoundObjectWorld, ConstraintValue, NonConstSection);

						if (ConstraintValue.Constraint.IsValid())
						{
							UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ConstraintValue.Constraint);
							if (TransformConstraint)
							{
								TransformConstraint->InitConstraint(BoundObjectWorld);
							}
							ConstraintValue.Constraint->ResolveBoundObjects(Operand.SequenceID, SharedPlaybackState, ControlRig);
							ConstraintValue.Constraint->SetActive(ConstraintValue.Value);

							if (TransformConstraint && ConstraintValue.Value && TransformableHandleUtils::SkipTicking())
							{
								MarkRigForEvaluationIfAdditive();
								
								if (UTransformableComponentHandle* ParentHandle = Cast<UTransformableComponentHandle>(TransformConstraint->ParentTRSHandle))
								{
									constexpr bool bRecursive = true;
									USceneComponent* ParentComponent = TransformableHandleUtils::GetTarget<USceneComponent>(TransformConstraint->ParentTRSHandle);
									TransformableHandleUtils::MarkComponentForEvaluation(ParentComponent, bRecursive);
								}
							}
						}
					}

					// for Constraints with ControlRig we need to resolve all Parents also
					// Don't need to do children since they wil be handled by the channel resolve above
					ResolveParentHandles(BoundObject, ControlRig, Operand, SharedPlaybackState);
				}
				else  //no bound object so turn off constraint
				{
					for (FConstraintAndActiveValue& ConstraintValue : ConstraintsValues)
					{
						if (ConstraintValue.Constraint.IsValid())
						{
							ConstraintValue.Constraint->SetActive(ConstraintValue.Value);
						}
					}
				}
			}
			Section->SetDoNotKey(bWasDoNotKey);
		}

	}

	void ResolveParentHandles(
		const UObject* InBoundObject, UControlRig* InControlRigInstance,
		const FMovieSceneEvaluationOperand& InOperand,
		const TSharedRef<UE::MovieScene::FSharedPlaybackState>& InSharedPlaybackState) const
	{
		if (!InBoundObject)
		{
			return;
		}

		UWorld* BoundObjectWorld = InBoundObject->GetWorld();
		const bool bIsGameWorld = InBoundObject->GetWorld() ? InBoundObject->GetWorld()->IsGameWorld() : false;
		
		UMovieSceneControlRigParameterTrack* ControlRigTrack = Section->GetTypedOuter<UMovieSceneControlRigParameterTrack>();

		// is this control rig a game world instance of this section's rig?
		auto WasAGameInstance = [ControlRigTrack](const UControlRig* InRigToTest)
		{
			return ControlRigTrack ? ControlRigTrack->IsAGameInstance(InRigToTest) : false;
		};

		// is the parent handle of this constraint related to this section?
		// this return true if the handle's control rig has been spawned by the ControlRigTrack (whether in Editor or Game)
		// if false, it means that the handle represents another control on another control rig so we don't need to resolve it here
		// note that it returns true if ControlRigTrack is null (is this possible?!) or if the ControlRig is null (we can't infer anything from this)
		auto ShouldResolveParent = [ControlRigTrack](const UTransformableControlHandle* ParentControlHandle)
		{
			if (!ParentControlHandle)
			{
				return false;
			}

			if (!ControlRigTrack)
			{
				// cf. UObjectBaseUtility::IsInOuter
				return true;	
			}
			
			return ParentControlHandle->ControlRig ? ParentControlHandle->ControlRig->IsInOuter(ControlRigTrack) : true;
		};

		// this is the default's section rig. when bIsGameWorld is false, InControlRigInstance should be equal to SectionRig
		const UControlRig* SectionRig = Section->GetControlRig();
		const bool bIsRigInstanceSectionRig = InControlRigInstance == SectionRig;

		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(BoundObjectWorld);
		const TArray< TWeakObjectPtr<UTickableConstraint>> Constraints = Controller.GetAllConstraints();

		for (const TWeakObjectPtr<UTickableConstraint>& TickConstraint : Constraints)
		{
			UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(TickConstraint.Get());
			UTransformableControlHandle* ParentControlHandle = TransformConstraint ? Cast<UTransformableControlHandle>(TransformConstraint->ParentTRSHandle) : nullptr;
			if (ParentControlHandle && ShouldResolveParent(ParentControlHandle))
			{
				if (bIsGameWorld)
				{
					// switch from section's rig to the game instance
					if (ParentControlHandle->ControlRig == SectionRig)
					{
						FConstraintDependencyScope Scope(TransformConstraint);
						ParentControlHandle->ResolveBoundObjects(InOperand.SequenceID, InSharedPlaybackState, InControlRigInstance);
						TransformConstraint->EnsurePrimaryDependency(BoundObjectWorld);
					}
				}
				else
				{
					// resolve to the incoming rig instance if it's the default rig for the section or if the previous one was a game instance
					if (WasAGameInstance(ParentControlHandle->ControlRig.Get()) || bIsRigInstanceSectionRig)
					{
						FConstraintDependencyScope Scope(TransformConstraint);
						ParentControlHandle->ResolveBoundObjects(InOperand.SequenceID, InSharedPlaybackState, InControlRigInstance);
						TransformConstraint->EnsurePrimaryDependency(BoundObjectWorld);
					}
				}
			}
		}
	}
	
	const UMovieSceneControlRigParameterSection* Section;
	/** Array of evaluated bool values */
	TArray<FBoolParameterStringAndValue, TInlineAllocator<2>> BoolValues;
	/** Array of evaluated integer values */
	TArray<FIntegerParameterStringAndValue, TInlineAllocator<2>> IntegerValues;
	/** Array of Space Values*/
	TArray<FControlSpaceAndValue, TInlineAllocator<2>> SpaceValues;
	/** Array of evaluated bool values */
	TArray<FConstraintAndActiveValue, TInlineAllocator<2>> ConstraintsValues;

	FMovieSceneControlRigParameterTemplate* Template;
};


FMovieSceneControlRigParameterTemplate::FMovieSceneControlRigParameterTemplate(
	const UMovieSceneControlRigParameterSection& Section,
	const UMovieSceneControlRigParameterTrack& Track)
	: FMovieSceneParameterSectionTemplate(Section)
	, Enums(Section.GetEnumParameterNamesAndCurves())
	, Integers(Section.GetIntegerParameterNamesAndCurves())
	, Spaces(Section.GetSpaceChannels())
	, Constraints(Section.GetConstraintsChannels())
{}



struct TControlRigParameterActuatorFloat : TMovieSceneBlendingActuator<FControlRigTrackTokenFloat>
{
	TControlRigParameterActuatorFloat(FMovieSceneAnimTypeID& InAnimID, const FName& InParameterName, const UMovieSceneControlRigParameterSection* InSection)
		: TMovieSceneBlendingActuator<FControlRigTrackTokenFloat>(FMovieSceneBlendingActuatorID(InAnimID))
		, ParameterName(InParameterName)
		, SectionData(InSection)
	{}


	FControlRigTrackTokenFloat RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();

		UControlRig* ControlRig = Section ? GetControlRig(Section, InObject) : nullptr;

		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
				ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue
				&& (ControlElement->Settings.ControlType == ERigControlType::Float || ControlElement->Settings.ControlType == ERigControlType::ScaleFloat))
			{
				const float Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
				return FControlRigTrackTokenFloat(Val);
			}

		}
		return FControlRigTrackTokenFloat();
	}

	void Actuate(UObject* InObject, const FControlRigTrackTokenFloat& InFinalValue, const TBlendableTokenStack<FControlRigTrackTokenFloat>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();
		
		bool bWasDoNotKey = false;
		if (Section)
		{
			const bool bSetupUndo = false;
			bWasDoNotKey = Section->GetDoNotKey();
			Section->SetDoNotKey(true);

			UControlRig* ControlRig = GetControlRig(Section, InObject);

			if (ControlRig && (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(ParameterName)))
			{
				FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
				if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
					ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue &&
					(ControlElement->Settings.ControlType == ERigControlType::Float || ControlElement->Settings.ControlType == ERigControlType::ScaleFloat))
				{
					ControlRig->SetControlValue<float>(ParameterName, InFinalValue.Value, true, EControlRigSetKey::Never,bSetupUndo);
				}
			}

			Section->SetDoNotKey(bWasDoNotKey);
		}
	}


	virtual void Actuate(FMovieSceneInterrogationData& InterrogationData, const FControlRigTrackTokenFloat& InValue, const TBlendableTokenStack<FControlRigTrackTokenFloat>& OriginalStack, const FMovieSceneContext& Context) const override
	{
		FFloatInterrogationData Data;
		Data.Val = InValue.Value;
		Data.ParameterName = ParameterName;
		InterrogationData.Add(FFloatInterrogationData(Data), UMovieSceneControlRigParameterSection::GetFloatInterrogationKey());
	}

	FName ParameterName;
	TWeakObjectPtr<const UMovieSceneControlRigParameterSection> SectionData;

};


struct TControlRigParameterActuatorVector2D : TMovieSceneBlendingActuator<FControlRigTrackTokenVector2D>
{
	TControlRigParameterActuatorVector2D(FMovieSceneAnimTypeID& InAnimID,  const FName& InParameterName, const UMovieSceneControlRigParameterSection* InSection)
		: TMovieSceneBlendingActuator<FControlRigTrackTokenVector2D>(FMovieSceneBlendingActuatorID(InAnimID))
		, ParameterName(InParameterName)
		, SectionData(InSection)
	{}



	FControlRigTrackTokenVector2D RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();

		UControlRig* ControlRig = Section ? GetControlRig(Section, InObject) : nullptr;

		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
				ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue
				&& (ControlElement->Settings.ControlType == ERigControlType::Vector2D))
			{
				FVector3f Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
				return FControlRigTrackTokenVector2D(FVector2D(Val.X, Val.Y));
			}
		}
		return FControlRigTrackTokenVector2D();
	}

	void Actuate(UObject* InObject, const FControlRigTrackTokenVector2D& InFinalValue, const TBlendableTokenStack<FControlRigTrackTokenVector2D>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();

		bool bWasDoNotKey = false;
		if (Section)
		{
			const bool bSetupUndo = false;
			bWasDoNotKey = Section->GetDoNotKey();
			Section->SetDoNotKey(true);

			UControlRig* ControlRig = GetControlRig(Section, InObject);


			if (ControlRig && (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(ParameterName)))
			{
				FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
				if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
					ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue
					&& (ControlElement->Settings.ControlType == ERigControlType::Vector2D))
				{
					const FVector3f Value(InFinalValue.Value.X, InFinalValue.Value.Y, 0.f);
					ControlRig->SetControlValue<FVector3f>(ParameterName, Value, true, EControlRigSetKey::Never,bSetupUndo);
				}
			}

			Section->SetDoNotKey(bWasDoNotKey);
		}
	}
	virtual void Actuate(FMovieSceneInterrogationData& InterrogationData, const FControlRigTrackTokenVector2D& InValue, const TBlendableTokenStack<FControlRigTrackTokenVector2D>& OriginalStack, const FMovieSceneContext& Context) const override
	{
		FVector2DInterrogationData Data;
		Data.Val = InValue.Value;
		Data.ParameterName = ParameterName;
		InterrogationData.Add(FVector2DInterrogationData(Data), UMovieSceneControlRigParameterSection::GetVector2DInterrogationKey());
	}

	FName ParameterName;
	TWeakObjectPtr<const UMovieSceneControlRigParameterSection> SectionData;

};


struct TControlRigParameterActuatorVector : TMovieSceneBlendingActuator<FControlRigTrackTokenVector>
{
	TControlRigParameterActuatorVector(FMovieSceneAnimTypeID& InAnimID, const FName& InParameterName, const UMovieSceneControlRigParameterSection* InSection)
		: TMovieSceneBlendingActuator<FControlRigTrackTokenVector>(FMovieSceneBlendingActuatorID(InAnimID))
		, ParameterName(InParameterName)
		, SectionData(InSection)
	{}



	FControlRigTrackTokenVector RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();

		UControlRig* ControlRig = Section ? GetControlRig(Section, InObject) : nullptr;

		if (ControlRig)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			
			if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
				ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue
				&& (ControlElement->Settings.ControlType == ERigControlType::Position || ControlElement->Settings.ControlType == ERigControlType::Scale || ControlElement->Settings.ControlType == ERigControlType::Rotator))
			{
				FVector3f Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
				if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
				{
					FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
					Val = FVector3f(Vector.X, Vector.Y, Vector.Z);
				}
				return FControlRigTrackTokenVector((FVector)Val);
			}
		}
		return FControlRigTrackTokenVector();
	}

	void Actuate(UObject* InObject, const FControlRigTrackTokenVector& InFinalValue, const TBlendableTokenStack<FControlRigTrackTokenVector>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();
		
		bool bWasDoNotKey = false;
		if (Section)
		{
			const bool bSetupUndo = false;
			bWasDoNotKey = Section->GetDoNotKey();
			Section->SetDoNotKey(true);
			UControlRig* ControlRig = GetControlRig(Section, InObject);

			if (ControlRig && (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(ParameterName)))
			{
				FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
				if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
					ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue
					&& (ControlElement->Settings.ControlType == ERigControlType::Position || ControlElement->Settings.ControlType == ERigControlType::Scale || ControlElement->Settings.ControlType == ERigControlType::Rotator))
				{
					if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::Rotator)
					{
						URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
						const FRotator Rotator(Hierarchy->GetControlQuaternion(ControlElement, InFinalValue.Value));
						
						Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, InFinalValue.Value);
						ControlRig->SetControlValue<FRotator>(ParameterName, Rotator, true, EControlRigSetKey::Never, bSetupUndo);
					}
					else
					{
						ControlRig->SetControlValue<FVector3f>(ParameterName, (FVector3f)InFinalValue.Value, true, EControlRigSetKey::Never, bSetupUndo);
					}
				}
			}

			Section->SetDoNotKey(bWasDoNotKey);
		}
	}
	virtual void Actuate(FMovieSceneInterrogationData& InterrogationData, const FControlRigTrackTokenVector& InValue, const TBlendableTokenStack<FControlRigTrackTokenVector>& OriginalStack, const FMovieSceneContext& Context) const override
	{
		FVectorInterrogationData Data;
		Data.Val = InValue.Value;
		Data.ParameterName = ParameterName;		
		InterrogationData.Add(FVectorInterrogationData(Data), UMovieSceneControlRigParameterSection::GetVectorInterrogationKey());
	}


	FName ParameterName;
	TWeakObjectPtr<const UMovieSceneControlRigParameterSection> SectionData;

};



struct TControlRigParameterActuatorTransform : TMovieSceneBlendingActuator<FControlRigTrackTokenTransform>
{
	TControlRigParameterActuatorTransform(FMovieSceneAnimTypeID& InAnimID, const FName& InParameterName, const UMovieSceneControlRigParameterSection* InSection)
		: TMovieSceneBlendingActuator<FControlRigTrackTokenTransform>(FMovieSceneBlendingActuatorID(InAnimID))
		, ParameterName(InParameterName)
		, SectionData(InSection)
	{}

	FControlRigTrackTokenTransform RetrieveCurrentValue(UObject* InObject, IMovieScenePlayer* Player) const
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();

		UControlRig* ControlRig = Section ? GetControlRig(Section, InObject) : nullptr;

		if (ControlRig && Section && (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(ParameterName)))
		{
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
			if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
				ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue)
			{
				if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::Transform)
				{
					const FTransform Val = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>().ToTransform();
					FEulerTransform EulerTransform(Val);
					FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
					EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
					return FControlRigTrackTokenTransform(EulerTransform);
				}
				else if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
				{
					FTransformNoScale ValNoScale = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
					FTransform Val = ValNoScale;
					FEulerTransform EulerTransform(Val);
					FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
					EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
					return FControlRigTrackTokenTransform(EulerTransform);
				}
				else if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					FEulerTransform EulerTransform = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
					FVector Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement);
					EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
					return FControlRigTrackTokenTransform(EulerTransform);
				}
			}
		}
		return FControlRigTrackTokenTransform();
	}

	void Actuate(UObject* InObject, const FControlRigTrackTokenTransform& InFinalValue, const TBlendableTokenStack<FControlRigTrackTokenTransform>& OriginalStack, const FMovieSceneContext& Context, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		const UMovieSceneControlRigParameterSection* Section = SectionData.Get();
		if (Section)
		{
			UMovieSceneTrack * Track = Cast<UMovieSceneTrack>(Section->GetOuter());
			if (Track && Track->GetSectionToKey())
			{
				Section = Cast<const UMovieSceneControlRigParameterSection>(Track->GetSectionToKey());
			}
		}

		bool bWasDoNotKey = false;
		if (Section)
		{
			bWasDoNotKey = Section->GetDoNotKey();
			Section->SetDoNotKey(true);
			const bool bSetupUndo = false;
			UControlRig* ControlRig = GetControlRig(Section, InObject);

			if (ControlRig && (Section->ControlsToSet.Num() == 0 || Section->ControlsToSet.Contains(ParameterName)))
			{
				URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
				FRigControlElement* ControlElement = ControlRig->FindControl(ParameterName);
				if (ControlElement && ControlElement->Settings.AnimationType != ERigControlAnimationType::ProxyControl &&
					ControlElement->Settings.AnimationType != ERigControlAnimationType::VisualCue)
				{
					if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::Transform)
					{
						FVector EulerAngle(InFinalValue.Value.Rotation.Roll, InFinalValue.Value.Rotation.Pitch, InFinalValue.Value.Rotation.Yaw);
						Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
						ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(ParameterName, InFinalValue.Value.ToFTransform(), true, EControlRigSetKey::Never, bSetupUndo);
					}
					else if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
					{
						const FTransformNoScale NoScale = InFinalValue.Value.ToFTransform();
						FVector EulerAngle(InFinalValue.Value.Rotation.Roll, InFinalValue.Value.Rotation.Pitch, InFinalValue.Value.Rotation.Yaw);
						Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
						ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(ParameterName, NoScale, true, EControlRigSetKey::Never, bSetupUndo);
					}
					else if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						FVector EulerAngle(InFinalValue.Value.Rotation.Roll, InFinalValue.Value.Rotation.Pitch, InFinalValue.Value.Rotation.Yaw);
						FQuat Quat = Hierarchy->GetControlQuaternion(ControlElement, EulerAngle);
						Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
						FRotator UERotator(Quat);
						FEulerTransform Transform = InFinalValue.Value;
						Transform.Rotation = UERotator;
						ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ParameterName, Transform, true, EControlRigSetKey::Never, bSetupUndo);	
						Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
					}
				}
			}

			Section->SetDoNotKey(bWasDoNotKey);
		}
	}
	virtual void Actuate(FMovieSceneInterrogationData& InterrogationData, const FControlRigTrackTokenTransform& InValue, const TBlendableTokenStack<FControlRigTrackTokenTransform>& OriginalStack, const FMovieSceneContext& Context) const override
	{
		FEulerTransformInterrogationData Data;
		Data.Val = InValue.Value;
		Data.ParameterName = ParameterName;
		InterrogationData.Add(FEulerTransformInterrogationData(Data), UMovieSceneControlRigParameterSection::GetTransformInterrogationKey());
	}
	FName ParameterName;
	TWeakObjectPtr<const UMovieSceneControlRigParameterSection> SectionData;

};

void FMovieSceneControlRigParameterTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	if (!UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate())
	{
		return;
	}

	const UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSourceSection());
	if (Section && Section->GetControlRig())
	{
		const FFrameTime Time = Section->RemapTime(Context.GetTime());

		UControlRig* ControlRig = Section->GetControlRig(); //this will be default(editor) control rig
		if (Operand.ObjectBindingID.IsValid())
		{
			TArrayView<TWeakObjectPtr<>> BoundObjects = PersistentData.GetMovieScenePlayer().FindBoundObjects(Operand);
			if(BoundObjects.Num() > 0 && BoundObjects[0].IsValid() && BoundObjects[0].Get()->GetWorld() && BoundObjects[0].Get()->GetWorld()->IsGameWorld()) //just support one bound object per control rig
			{
				UObject* BoundObject = BoundObjects[0].Get();
				UWorld* GameWorld =  BoundObject->GetWorld();
				ControlRig = Section->GetControlRig(GameWorld, BoundObject);
				if (ControlRig == nullptr)
				{
					return;
				}
			}
		}

		FEvaluatedControlRigParameterSectionChannelMasks* ChannelMasks = PersistentData.FindSectionData<FEvaluatedControlRigParameterSectionChannelMasks>();
		if (!ChannelMasks)
		{
			// Naughty const_cast here, but we can't create this inside Initialize because of hotfix restrictions
			// The cast is ok because we actually do not have any threading involved
			ChannelMasks = &const_cast<FPersistentEvaluationData&>(PersistentData).GetOrAddSectionData<FEvaluatedControlRigParameterSectionChannelMasks>();
			UMovieSceneControlRigParameterSection* NonConstSection = const_cast<UMovieSceneControlRigParameterSection*>(Section);
			ChannelMasks->Initialize(NonConstSection, Scalars, Bools, Integers, Enums, Vector2Ds, Vectors, Colors, Transforms);
		}

		UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Section->GetOuter());
		if (!Track)
		{
			return;
		}

		int32 BlendingOrder = Section->GetBlendingOrder();

		//Do blended tokens
		FEvaluatedControlRigParameterSectionValues Values;

		EvaluateCurvesWithMasks(Context, *ChannelMasks, Values);

		float Weight = EvaluateEasing(Time);
		if (EnumHasAllFlags(Section->TransformMask.GetChannels(), EMovieSceneTransformChannel::Weight))
		{
			float ManualWeight = 1.f;
			Section->Weight.Evaluate(Time, ManualWeight);
			Weight *= ManualWeight;
		}

		//Do basic token
		FControlRigParameterExecutionToken ExecutionToken(Section,Values, const_cast<FMovieSceneControlRigParameterTemplate*>(this));
		ExecutionTokens.Add(MoveTemp(ExecutionToken));

		EMovieSceneBlendType BlendType = Section->GetBlendType().IsValid() ? Section->GetBlendType().Get() : EMovieSceneBlendType::Absolute;

		FControlRigAnimTypeIDsPtr TypeIDs = FControlRigAnimTypeIDs::Get(ControlRig);

		for (const FScalarParameterStringAndValue& ScalarNameAndValue : Values.ScalarValues)
		{
			if (Section->GetControlNameMask(ScalarNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindScalar(ScalarNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!ExecutionTokens.GetBlendingAccumulator().FindActuator< FControlRigTrackTokenFloat>(ActuatorTypeID))
			{
				ExecutionTokens.GetBlendingAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorFloat>(AnimTypeID, ScalarNameAndValue.ParameterName, Section));
			}
			ExecutionTokens.BlendToken(ActuatorTypeID, TBlendableToken<FControlRigTrackTokenFloat>(ScalarNameAndValue.Value, BlendType, Weight, BlendingOrder));
		}

		UE::MovieScene::TMultiChannelValue<float, 3> VectorData;
		for (const FVectorParameterStringAndValue& VectorNameAndValue : Values.VectorValues)
		{
			if (Section->GetControlNameMask(VectorNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindVector(VectorNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!ExecutionTokens.GetBlendingAccumulator().FindActuator< FControlRigTrackTokenVector>(ActuatorTypeID))
			{
				ExecutionTokens.GetBlendingAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorVector>(AnimTypeID,  VectorNameAndValue.ParameterName,Section));
			}
			VectorData.Set(0, VectorNameAndValue.Value.X);
			VectorData.Set(1, VectorNameAndValue.Value.Y);
			VectorData.Set(2, VectorNameAndValue.Value.Z);

			ExecutionTokens.BlendToken(ActuatorTypeID, TBlendableToken<FControlRigTrackTokenVector>(VectorData, BlendType, Weight, BlendingOrder));
		}

		UE::MovieScene::TMultiChannelValue<float, 2> Vector2DData;
		for (const FVector2DParameterStringAndValue& Vector2DNameAndValue : Values.Vector2DValues)
		{
			if (Section->GetControlNameMask(Vector2DNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindVector2D(Vector2DNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!ExecutionTokens.GetBlendingAccumulator().FindActuator< FControlRigTrackTokenVector2D>(ActuatorTypeID))
			{
				ExecutionTokens.GetBlendingAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorVector2D>(AnimTypeID,  Vector2DNameAndValue.ParameterName, Section));
			}
			Vector2DData.Set(0, Vector2DNameAndValue.Value.X);
			Vector2DData.Set(1, Vector2DNameAndValue.Value.Y);

			ExecutionTokens.BlendToken(ActuatorTypeID, TBlendableToken<FControlRigTrackTokenVector2D>(Vector2DData, BlendType, Weight, BlendingOrder));
		}

		UE::MovieScene::TMultiChannelValue<float, 9> TransformData;
		for (const FEulerTransformParameterStringAndValue& TransformNameAndValue : Values.TransformValues)
		{
			if (Section->GetControlNameMask(TransformNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindTransform(TransformNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!ExecutionTokens.GetBlendingAccumulator().FindActuator< FControlRigTrackTokenTransform>(ActuatorTypeID))
			{
				ExecutionTokens.GetBlendingAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorTransform>(AnimTypeID,  TransformNameAndValue.ParameterName, Section));
			}

			const FEulerTransform& Transform = TransformNameAndValue.Transform;

			TransformData.Set(0, Transform.Location.X);
			TransformData.Set(1, Transform.Location.Y);
			TransformData.Set(2, Transform.Location.Z);

			TransformData.Set(3, Transform.Rotation.Roll);
			TransformData.Set(4, Transform.Rotation.Pitch);
			TransformData.Set(5, Transform.Rotation.Yaw);

			TransformData.Set(6, Transform.Scale.X);
			TransformData.Set(7, Transform.Scale.Y);
			TransformData.Set(8, Transform.Scale.Z);
			ExecutionTokens.BlendToken(ActuatorTypeID, TBlendableToken<FControlRigTrackTokenTransform>(TransformData, BlendType, Weight, BlendingOrder));
		}

	}
}


void FMovieSceneControlRigParameterTemplate::EvaluateCurvesWithMasks(const FMovieSceneContext& Context, FEvaluatedControlRigParameterSectionChannelMasks& InMasks, FEvaluatedControlRigParameterSectionValues& Values) const
{
	const UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSourceSection());
	if (Section)
	{
		const FFrameTime Time = Section->RemapTime(Context.GetTime());

		// Reserve the value arrays to avoid re-allocation
		Values.ScalarValues.Reserve(Scalars.Num());
		Values.BoolValues.Reserve(Bools.Num());
		Values.SpaceValues.Reserve(Spaces.Num());
		Values.IntegerValues.Reserve(Integers.Num() + Enums.Num()); // Both enums and integers output to the integer value array
		Values.Vector2DValues.Reserve(Vector2Ds.Num());
		Values.VectorValues.Reserve(Vectors.Num());
		Values.ColorValues.Reserve(Colors.Num());
		Values.TransformValues.Reserve(Transforms.Num());
		Values.ConstraintsValues.Reserve(Constraints.Num());

		const bool bIsAdditive = (Section->GetBlendType().IsValid() && Section->GetBlendType().Get() == EMovieSceneBlendType::Additive);
		const bool bIsAbsolute = (Section->GetBlendType().IsValid() && Section->GetBlendType().Get() == EMovieSceneBlendType::Absolute);

		// Populate each of the output arrays in turn
		for (int32 Index = 0; Index < Scalars.Num(); ++Index)
		{
			const FScalarParameterNameAndCurve& Scalar = this->Scalars[Index];
			float Value = 0;

			if (InMasks.ScalarCurveMask[Index])
			{
				Scalar.ParameterCurve.Evaluate(Time, Value);
			}
			else
			{
				Value = (!bIsAbsolute
					|| Scalar.ParameterCurve.GetDefault().IsSet() == false) ? 0.0f :
					Scalar.ParameterCurve.GetDefault().GetValue();
			}

			Values.ScalarValues.Emplace(Scalar.ParameterName, Value);
		}

		// when playing animation, instead of scrubbing/stepping thru frames, InTime might have a subframe of 0.999928
		// leading to a decimal value of 24399.999928 (for example). This results in evaluating one frame less than expected
		// (24399 instead of 24400) and leads to spaces and constraints switching parents/state after the control changes
		// its transform. (float/double channels will interpolate to a value pretty close to the one at 24400 as its based
		// on that 0.999928 subframe value.
		const FFrameTime RoundTime = Time.RoundToFrame();
		for (int32 Index = 0; Index < Spaces.Num(); ++Index)
		{
			FMovieSceneControlRigSpaceBaseKey Value;
			const FSpaceControlNameAndChannel& Space = Spaces[Index];
			Space.SpaceCurve.Evaluate(RoundTime, Value);
			
			Values.SpaceValues.Emplace(Space.ControlName, Value);
		}

		for (int32 Index = 0; Index < Constraints.Num(); ++Index)
		{
			bool Value = false;
			const FConstraintAndActiveChannel& ConstraintAndActiveChannel = Constraints[Index];
			ConstraintAndActiveChannel.ActiveChannel.Evaluate(RoundTime, Value);
			Values.ConstraintsValues.Emplace(ConstraintAndActiveChannel.GetConstraint().Get(), Value);
		}
		
		for (int32 Index = 0; Index < Bools.Num(); ++Index)
		{
			bool Value = false;
			const FBoolParameterNameAndCurve& Bool = Bools[Index];
			if (InMasks.BoolCurveMask[Index])
			{
				Bool.ParameterCurve.Evaluate(Time, Value);
			}
			else
			{
				Value = (!bIsAbsolute
					|| Bool.ParameterCurve.GetDefault().IsSet() == false) ? false :
					Bool.ParameterCurve.GetDefault().GetValue();
			}

			Values.BoolValues.Emplace(Bool.ParameterName, Value);
		}
		for (int32 Index = 0; Index < Integers.Num(); ++Index)
		{
			int32  Value = 0;
			const FIntegerParameterNameAndCurve& Integer = Integers[Index];
			if (InMasks.IntegerCurveMask[Index])
			{
				Integer.ParameterCurve.Evaluate(Time, Value);
			}
			else
			{
				Value = (!bIsAbsolute
					|| Integer.ParameterCurve.GetDefault().IsSet() == false) ? 0 :
					Integer.ParameterCurve.GetDefault().GetValue();
			}

			Values.IntegerValues.Emplace(Integer.ParameterName, Value);
		}
		for (int32 Index = 0; Index < Enums.Num(); ++Index)
		{
			uint8  Value = 0;
			const FEnumParameterNameAndCurve& Enum = Enums[Index];
			if (InMasks.EnumCurveMask[Index])
			{
				Enum.ParameterCurve.Evaluate(Time, Value);
			}
			else
			{
				Value = (!bIsAbsolute
					|| Enum.ParameterCurve.GetDefault().IsSet() == false) ? 0 :
					Enum.ParameterCurve.GetDefault().GetValue();

			}
			Values.IntegerValues.Emplace(Enum.ParameterName, (int32)Value);

		}
		for (int32 Index = 0; Index < Vector2Ds.Num(); ++Index)
		{
			FVector2f Value(ForceInitToZero);
			const FVector2DParameterNameAndCurves& Vector2D = Vector2Ds[Index];

			if (InMasks.Vector2DCurveMask[Index])
			{
				Vector2D.XCurve.Evaluate(Time, Value.X);
				Vector2D.YCurve.Evaluate(Time, Value.Y);
			}
			else
			{
				if (bIsAbsolute)
				{
					if (Vector2D.XCurve.GetDefault().IsSet())
					{
						Value.X = Vector2D.XCurve.GetDefault().GetValue();
					}
					if (Vector2D.YCurve.GetDefault().IsSet())
					{
						Value.Y = Vector2D.YCurve.GetDefault().GetValue();
					}
				}
			}

			Values.Vector2DValues.Emplace(Vector2D.ParameterName, FVector2D(Value));
		}

		for (int32 Index = 0; Index < Vectors.Num(); ++Index)
		{
			FVector3f Value(ForceInitToZero);
			const FVectorParameterNameAndCurves& Vector = Vectors[Index];

			if (InMasks.VectorCurveMask[Index])
			{
				Vector.XCurve.Evaluate(Time, Value.X);
				Vector.YCurve.Evaluate(Time, Value.Y);
				Vector.ZCurve.Evaluate(Time, Value.Z);
			}
			else
			{
				if (bIsAbsolute)
				{
					if (Vector.XCurve.GetDefault().IsSet())
					{
						Value.X = Vector.XCurve.GetDefault().GetValue();
					}
					if (Vector.YCurve.GetDefault().IsSet())
					{
						Value.Y = Vector.YCurve.GetDefault().GetValue();
					}
					if (Vector.ZCurve.GetDefault().IsSet())
					{
						Value.Z = Vector.ZCurve.GetDefault().GetValue();
					}
				}
			}

			Values.VectorValues.Emplace(Vector.ParameterName, (FVector)Value);
		}
		for (int32 Index = 0; Index < Colors.Num(); ++Index)
		{
			FLinearColor ColorValue = FLinearColor::White;
			const FColorParameterNameAndCurves& Color = Colors[Index];
			if (InMasks.ColorCurveMask[Index])
			{
				Color.RedCurve.Evaluate(Time, ColorValue.R);
				Color.GreenCurve.Evaluate(Time, ColorValue.G);
				Color.BlueCurve.Evaluate(Time, ColorValue.B);
				Color.AlphaCurve.Evaluate(Time, ColorValue.A);
			}
			else
			{
				if (bIsAbsolute)
				{
					if (Color.RedCurve.GetDefault().IsSet())
					{
						ColorValue.R = Color.RedCurve.GetDefault().GetValue();
					}
					if (Color.GreenCurve.GetDefault().IsSet())
					{
						ColorValue.G = Color.GreenCurve.GetDefault().GetValue();
					}
					if (Color.BlueCurve.GetDefault().IsSet())
					{
						ColorValue.B = Color.BlueCurve.GetDefault().GetValue();
					}
					if (Color.AlphaCurve.GetDefault().IsSet())
					{
						ColorValue.A = Color.AlphaCurve.GetDefault().GetValue();
					}
				}
			}

			Values.ColorValues.Emplace(Color.ParameterName, ColorValue);
		}
		EMovieSceneTransformChannel ChannelMask = Section->GetTransformMask().GetChannels();
		for (int32 Index = 0; Index < Transforms.Num(); ++Index)
		{
			FVector3f Translation(ForceInitToZero), Scale(FVector3f::OneVector);
			if (bIsAdditive)
			{
				Scale = FVector3f(0.0f, 0.0f, 0.0f);
			}

			FRotator3f Rotator(0.0f, 0.0f, 0.0f);

			const FTransformParameterNameAndCurves& Transform = Transforms[Index];
			if (InMasks.TransformCurveMask[Index])
			{
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationX))
				{
					Transform.Translation[0].Evaluate(Time, Translation[0]);
				}
				else
				{
					if (bIsAbsolute && Transform.Translation[0].GetDefault().IsSet())
					{
						Translation[0] = Transform.Translation[0].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationY))
				{
					Transform.Translation[1].Evaluate(Time, Translation[1]);
				}
				else
				{
					if (bIsAbsolute && Transform.Translation[1].GetDefault().IsSet())
					{
						Translation[1] = Transform.Translation[1].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationZ))
				{
					Transform.Translation[2].Evaluate(Time, Translation[2]);
				}
				else
				{
					if (bIsAbsolute && Transform.Translation[2].GetDefault().IsSet())
					{
						Translation[2] = Transform.Translation[2].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationX))
				{
					Transform.Rotation[0].Evaluate(Time, Rotator.Roll);
				}
				else
				{
					if (bIsAbsolute && Transform.Rotation[0].GetDefault().IsSet())
					{
						Rotator.Roll = Transform.Rotation[0].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationY))
				{
					Transform.Rotation[1].Evaluate(Time, Rotator.Pitch);
				}
				else
				{
					if (bIsAbsolute && Transform.Rotation[1].GetDefault().IsSet())
					{
						Rotator.Pitch = Transform.Rotation[1].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationZ))
				{
					Transform.Rotation[2].Evaluate(Time, Rotator.Yaw);
				}
				else
				{
					if (bIsAbsolute && Transform.Rotation[2].GetDefault().IsSet())
					{
						Rotator.Yaw = Transform.Rotation[2].GetDefault().GetValue();
					}
				}
				//mz todo quat interp...
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleX))
				{
					Transform.Scale[0].Evaluate(Time, Scale[0]);
				}
				else
				{
					if (bIsAbsolute && Transform.Scale[0].GetDefault().IsSet())
					{
						Scale[0] = Transform.Scale[0].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleY))
				{
					Transform.Scale[1].Evaluate(Time, Scale[1]);
				}
				else
				{
					if (bIsAbsolute && Transform.Scale[1].GetDefault().IsSet())
					{
						Scale[1] = Transform.Scale[1].GetDefault().GetValue();
					}
				}
				if (EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleZ))
				{
					Transform.Scale[2].Evaluate(Time, Scale[2]);
				}
				else
				{
					if (bIsAbsolute && Transform.Scale[2].GetDefault().IsSet())
					{
						Scale[2] = Transform.Scale[2].GetDefault().GetValue();
					}
				}
			}
			else //completely masked use default or zeroed, which is already set if additive
			{
				if (bIsAbsolute)
				{

					if (Transform.Translation[0].GetDefault().IsSet())
					{
						Translation[0] = Transform.Translation[0].GetDefault().GetValue();
					}
					if (Transform.Translation[1].GetDefault().IsSet())
					{
						Translation[1] = Transform.Translation[1].GetDefault().GetValue();
					}
					if (Transform.Translation[2].GetDefault().IsSet())
					{
						Translation[2] = Transform.Translation[2].GetDefault().GetValue();
					}

					if (Transform.Rotation[0].GetDefault().IsSet())
					{
						Rotator.Roll = Transform.Rotation[0].GetDefault().GetValue();
					}

					if (Transform.Rotation[1].GetDefault().IsSet())
					{
						Rotator.Pitch = Transform.Rotation[1].GetDefault().GetValue();
					}

					if (Transform.Rotation[2].GetDefault().IsSet())
					{
						Rotator.Yaw = Transform.Rotation[2].GetDefault().GetValue();
					}

					if (Transform.Scale[0].GetDefault().IsSet())
					{
						Scale[0] = Transform.Scale[0].GetDefault().GetValue();
					}

					if (Transform.Scale[1].GetDefault().IsSet())
					{
						Scale[1] = Transform.Scale[1].GetDefault().GetValue();
					}

					if (Transform.Scale[2].GetDefault().IsSet())
					{
						Scale[2] = Transform.Scale[2].GetDefault().GetValue();
					}
				}
			}
			FEulerTransformParameterStringAndValue NameAndValue(Transform.ParameterName, FEulerTransform(FRotator(Rotator), (FVector)Translation, (FVector)Scale));
			Values.TransformValues.Emplace(NameAndValue);
		}
	}
}


FMovieSceneAnimTypeID FMovieSceneControlRigParameterTemplate::GetAnimTypeID()
{
	return TMovieSceneAnimTypeID<FMovieSceneControlRigParameterTemplate>();
}

#if WITH_EDITOR
FMovieSceneControlRigParameterTemplate::~FMovieSceneControlRigParameterTemplate()
{
	if (GEditor)
	{
		for (FDelegateHandle& Handle : PreCompileHandles)
		{
			GEditor->OnBlueprintPreCompile().Remove(Handle);
		}
	}
}
#endif

void FMovieSceneControlRigParameterTemplate::Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ControlRigTemplateParameter_Evaluate)

	if (!UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate())
	{
		return;
	}

	const UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(GetSourceSection());
	if (Section && Section->GetControlRig() && MovieSceneHelpers::IsSectionKeyable(Section))
	{
		const FFrameTime Time = Section->RemapTime(Context.GetTime());

		FEvaluatedControlRigParameterSectionChannelMasks ChannelMasks;
		UMovieSceneControlRigParameterSection* NonConstSection = const_cast<UMovieSceneControlRigParameterSection*>(Section);
		ChannelMasks.Initialize(NonConstSection, Scalars, Bools, Integers, Enums, Vector2Ds, Vectors, Colors, Transforms);

		//Do blended tokens
		FEvaluatedControlRigParameterSectionValues Values;
		EvaluateCurvesWithMasks(Context, ChannelMasks, Values);

		FControlRigAnimTypeIDsPtr TypeIDs = FControlRigAnimTypeIDs::Get(Section->GetControlRig());
		EMovieSceneBlendType BlendType = Section->GetBlendType().IsValid() ? Section->GetBlendType().Get() : EMovieSceneBlendType::Absolute;
		int32 BlendingOrder = Section->GetBlendingOrder();

		float Weight = EvaluateEasing(Time);
		if (EnumHasAllFlags(Section->TransformMask.GetChannels(), EMovieSceneTransformChannel::Weight))
		{
			float ManualWeight = 1.f;
			Section->Weight.Evaluate(Time, ManualWeight);
			Weight *= ManualWeight;
		}

		for (const FScalarParameterStringAndValue& ScalarNameAndValue : Values.ScalarValues)
		{
			if (Section->GetControlNameMask(ScalarNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindScalar(ScalarNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!Container.GetAccumulator().FindActuator< FControlRigTrackTokenFloat>(ActuatorTypeID))
			{
				Container.GetAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorFloat>(AnimTypeID, ScalarNameAndValue.ParameterName, Section));
			}
			Container.GetAccumulator().BlendToken(FMovieSceneEvaluationOperand(), ActuatorTypeID, FMovieSceneEvaluationScope(), Context,TBlendableToken<FControlRigTrackTokenFloat>(ScalarNameAndValue.Value, BlendType, Weight, BlendingOrder));
		}

		UE::MovieScene::TMultiChannelValue<float, 2> Vector2DData;
		for (const FVector2DParameterStringAndValue& Vector2DNameAndValue : Values.Vector2DValues)
		{
			if (Section->GetControlNameMask(Vector2DNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindVector2D(Vector2DNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!Container.GetAccumulator().FindActuator< FControlRigTrackTokenVector>(ActuatorTypeID))
			{
				Container.GetAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorVector2D>(AnimTypeID, Vector2DNameAndValue.ParameterName, Section));
			}
			Vector2DData.Set(0, Vector2DNameAndValue.Value.X);
			Vector2DData.Set(1, Vector2DNameAndValue.Value.Y);

			Container.GetAccumulator().BlendToken(FMovieSceneEvaluationOperand(), ActuatorTypeID, FMovieSceneEvaluationScope(), Context, TBlendableToken<FControlRigTrackTokenVector2D>(Vector2DData, BlendType, Weight, BlendingOrder));
		}

		UE::MovieScene::TMultiChannelValue<float, 3> VectorData;
		for (const FVectorParameterStringAndValue& VectorNameAndValue : Values.VectorValues)
		{
			if (Section->GetControlNameMask(VectorNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindVector(VectorNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!Container.GetAccumulator().FindActuator< FControlRigTrackTokenVector>(ActuatorTypeID))
			{
				Container.GetAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorVector>(AnimTypeID, VectorNameAndValue.ParameterName, Section));
			}
			VectorData.Set(0, VectorNameAndValue.Value.X);
			VectorData.Set(1, VectorNameAndValue.Value.Y);
			VectorData.Set(2, VectorNameAndValue.Value.Z);

			Container.GetAccumulator().BlendToken(FMovieSceneEvaluationOperand(), ActuatorTypeID, FMovieSceneEvaluationScope(),Context,TBlendableToken<FControlRigTrackTokenVector>(VectorData, BlendType, Weight, BlendingOrder));
		}

		UE::MovieScene::TMultiChannelValue<float, 9> TransformData;
		for (const FEulerTransformParameterStringAndValue& TransformNameAndValue : Values.TransformValues)
		{
			if (Section->GetControlNameMask(TransformNameAndValue.ParameterName) == false)
			{
				continue;
			}
			FMovieSceneAnimTypeID AnimTypeID = TypeIDs->FindTransform(TransformNameAndValue.ParameterName);
			FMovieSceneBlendingActuatorID ActuatorTypeID(AnimTypeID);

			if (!Container.GetAccumulator().FindActuator< FControlRigTrackTokenTransform>(ActuatorTypeID))
			{
				Container.GetAccumulator().DefineActuator(ActuatorTypeID, MakeShared <TControlRigParameterActuatorTransform>(AnimTypeID, TransformNameAndValue.ParameterName, Section));
			}

			const FEulerTransform& Transform = TransformNameAndValue.Transform;

			TransformData.Set(0, Transform.Location.X);
			TransformData.Set(1, Transform.Location.Y);
			TransformData.Set(2, Transform.Location.Z);

			TransformData.Set(3, Transform.Rotation.Roll);
			TransformData.Set(4, Transform.Rotation.Pitch);
			TransformData.Set(5, Transform.Rotation.Yaw);

			TransformData.Set(6, Transform.Scale.X);
			TransformData.Set(7, Transform.Scale.Y);
			TransformData.Set(8, Transform.Scale.Z);
			Container.GetAccumulator().BlendToken(FMovieSceneEvaluationOperand(),ActuatorTypeID, FMovieSceneEvaluationScope(), Context, TBlendableToken<FControlRigTrackTokenTransform>(TransformData, BlendType, Weight, BlendingOrder));
		}

	}
}

