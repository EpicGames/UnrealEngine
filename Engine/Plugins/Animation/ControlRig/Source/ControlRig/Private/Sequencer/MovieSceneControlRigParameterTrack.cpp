// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Sequencer/MovieSceneControlRigParameterTemplate.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Rigs/FKControlRig.h"
#include "UObject/Package.h"
#include "Async/Async.h"
#include "Sequencer/MovieSceneControlRigSystem.h"
#include "Transform/TransformConstraint.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "ControlRigObjectBinding.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "Editor.h"
#endif//WITH_EDITOR

#include "Tracks/MovieScene3DTransformTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneControlRigParameterTrack)

#define LOCTEXT_NAMESPACE "MovieSceneParameterControlRigTrack"

namespace UE::MovieScene
{
	MOVIESCENETRACKS_API extern bool (*ShouldUseLegacyControlRigTemplate)();

	bool GUseLegacyControlRigTemplate = true;
	FAutoConsoleVariableRef CVarUseLegacyControlRigTemplate(
		TEXT("ControlRig.UseLegacySequencerTemplate"),
		GUseLegacyControlRigTemplate,
		TEXT("(Default: true) Specifies whether to use legacy template evaluation for control rig tracks in Sequencer."),
		ECVF_Default
	);

	bool GCanUseLegacyControlRigTemplate = true;
	FAutoConsoleVariableRef CVarCanUseLegacyControlRigTemplate(
		TEXT("ControlRig.CanUseLegacySequencerTemplate"),
		GCanUseLegacyControlRigTemplate,
		TEXT("(Default: true) Specifies whether control rig tracks compile their legacy template as a back up. When disabled, ControlRig.UseLegacySequencerTemplate has no effect."),
		ECVF_Default
	);
	
	bool GUseScopedLegacyControlRigTemplate = false;
	FAutoConsoleVariableRef CVarUseScopedLegacyControlRigTemplate(
		TEXT("ControlRig.UseScopedLegacySequencerTemplate"),
		GUseScopedLegacyControlRigTemplate,
		TEXT("(Default: false) Specifies whether to use scoped legacy template evaluation for control rig tracks in Sequencer."),
		ECVF_Default
	);

} // namespace UE::MovieScene

bool UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate()
{
	// We use the legacy template if we were asked to, or if we're not using the custom scheduler (since the code is not implemented there....)
	return UE::MovieScene::GUseLegacyControlRigTemplate || !UMovieSceneEntitySystem::IsCustomSchedulingEnabled();
}

bool UMovieSceneControlRigParameterTrack::ShouldCacheLegacyTemplate()
{
	return ShouldUseLegacyTemplate() && UE::MovieScene::GUseScopedLegacyControlRigTemplate;
}

FColor UMovieSceneControlRigParameterTrack::AbsoluteRigTrackColor = FColor(65, 89, 194, 65);
FColor UMovieSceneControlRigParameterTrack::LayeredRigTrackColor = FColor(173, 151, 114);

FControlRotationOrder::FControlRotationOrder()
	:RotationOrder(EEulerRotationOrder::YZX),
	bOverrideSetting(false)

{

}

UMovieSceneControlRigParameterTrack::UMovieSceneControlRigParameterTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ControlRig(nullptr)
	, PriorityOrder(INDEX_NONE)
{
#if WITH_EDITORONLY_DATA
	TrackTint = AbsoluteRigTrackColor;
#endif

	SupportedBlendTypes = FMovieSceneBlendTypeField::None();
	SupportedBlendTypes.Add(EMovieSceneBlendType::Additive);
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
	SupportedBlendTypes.Add(EMovieSceneBlendType::Override);

	UE::MovieScene::ShouldUseLegacyControlRigTemplate = &UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate;

#if WITH_EDITOR
	// If we are in the process of loading, we can't be certain that we aren't already inside one of these delegate's broadcasts
	if (!IsInAsyncLoadingThread())
	{
		BindPIEDelegates();
	}
#endif
}

void UMovieSceneControlRigParameterTrack::BeginDestroy()
{
	Super::BeginDestroy();
	UnbindControlRigDelegates();
	
#if WITH_EDITOR
	UnbindPIEDelegates();
#endif
}

// UMovieSceneTrack interface
FMovieSceneEvalTemplatePtr UMovieSceneControlRigParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	if (UE::MovieScene::GCanUseLegacyControlRigTemplate)
	{
		return FMovieSceneControlRigParameterTemplate(*CastChecked<UMovieSceneControlRigParameterSection>(&InSection), *this);
	}

	return FMovieSceneEvalTemplatePtr();
}

bool UMovieSceneControlRigParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneControlRigParameterSection::StaticClass();
}

//we have possible name changes with modular rigs, need to update the rotation order map with this new name
void UMovieSceneControlRigParameterTrack::CheckForNameChanges()
{
	if (ControlRig && ControlRig->IsModularRig())
	{
		TArray<FName> RotationOrderNames;
		ControlsRotationOrder.GenerateKeyArray(RotationOrderNames);
		for (const FName& OrderControlName : RotationOrderNames)
		{
			if (FControlRotationOrder* RotationOrder = ControlsRotationOrder.Find(OrderControlName))
			{
				if (const FRigControlElement* Control = ControlRig->FindControl(OrderControlName)) //this will find old names also
				{
					const FName& ControlName = Control->GetFName();
					if (ControlName != OrderControlName)
					{
						FControlRotationOrder NewRotationOrder = *RotationOrder;
						ControlsRotationOrder.Remove(OrderControlName);
						ControlsRotationOrder.Add(ControlName, NewRotationOrder);
					}
				}
			}
		}
	}
}

#if WITH_EDITOR

void UMovieSceneControlRigParameterTrack::OnPreBeginPIE(bool bIsSimulating)
{
	OnCleanupGameControlRigs();
}

void UMovieSceneControlRigParameterTrack::OnEndPIE(bool bIsSimulating)
{
	UControlRig* DefaultRig = ControlRig.Get();
	if (!DefaultRig || GameWorldControlRigs.IsEmpty())
	{
		return;
	}

	constexpr bool bEvenIfPendingKill = true;
	for (const auto& [WeakGameWorldPtr, GameRigPtr]: GameWorldControlRigs)
	{
		UWorld* World = WeakGameWorldPtr.Get(bEvenIfPendingKill);
		UControlRig* GameRig = GameRigPtr.Get();
		if (World && GameRig)
		{
			auto ResetHandle = [GameRig, DefaultRig](const TObjectPtr<UTransformableHandle>& Handle)
			{
				UTransformableControlHandle* ControlHandle = Cast<UTransformableControlHandle>(Handle);
				if (ControlHandle && ControlHandle->ControlRig == GameRig)
				{
					ControlHandle->UpdateControlRig(DefaultRig);
				}						
			};

			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			for (const TWeakObjectPtr<UTickableConstraint>& Constraint: Controller.GetAllConstraints())
			{
				if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(Constraint.Get()))
				{
					ResetHandle(TransformConstraint->ChildTRSHandle);
					ResetHandle(TransformConstraint->ParentTRSHandle);
				}
			}
		}
	}
}

void UMovieSceneControlRigParameterTrack::OnShutdownPIE(bool bIsSimulating)
{
	OnCleanupGameControlRigs();
}

void UMovieSceneControlRigParameterTrack::OnCleanupGameControlRigs()
{
	TArray<TWeakObjectPtr<UWorld>> KeysToRemove;
	for (const TPair<TWeakObjectPtr<UWorld>,TObjectPtr<UControlRig>>& Pair : GameWorldControlRigs)
	{
		if (!Pair.Key.IsValid() || Pair.Key.IsStale())
		{
			KeysToRemove.Add(Pair.Key);
			if (UControlRig* GameWorldControlRig = Pair.Value)
			{
				if (IsValid(GameWorldControlRig))
				{
					UnbindGameWorldControlRigDelegates(GameWorldControlRig);
					GameWorldControlRig->MarkAsGarbage();
				}
			}
		}
	}
	for (const TWeakObjectPtr<UWorld>& KeyToRemove : KeysToRemove)
	{
		GameWorldControlRigs.Remove(KeyToRemove);
	}
}

#endif

UMovieSceneSection* UMovieSceneControlRigParameterTrack::CreateNewSection()
{
	UMovieSceneControlRigParameterSection* NewSection = NewObject<UMovieSceneControlRigParameterSection>(this, NAME_None, RF_Transactional);
	NewSection->SetControlRig(ControlRig);
	bool bSetDefault = false;
	if (Sections.Num() == 0)
	{
		NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
		bSetDefault = true;
	}
	else
	{
		NewSection->SetBlendType(EMovieSceneBlendType::Additive);
	}

	NewSection->SpaceChannelAdded().AddUObject(this, &UMovieSceneControlRigParameterTrack::HandleOnSpaceAdded);
	if (!NewSection->ConstraintChannelAdded().IsBoundToObject(this))
	{
		NewSection->ConstraintChannelAdded().AddUObject(this, &UMovieSceneControlRigParameterTrack::HandleOnConstraintAdded);
	}

	if (ControlRig)
	{
		NewSection->RecreateWithThisControlRig(ControlRig,bSetDefault);
	}
	return  NewSection;
}

void UMovieSceneControlRigParameterTrack::HandleOnSpaceAdded(UMovieSceneControlRigParameterSection* Section, const FName& InControlName, FMovieSceneControlRigSpaceChannel* Channel)
{
	OnSpaceChannelAdded.Broadcast(Section, InControlName, Channel);
}

void UMovieSceneControlRigParameterTrack::HandleOnConstraintAdded(
	IMovieSceneConstrainedSection* InSection,
	FMovieSceneConstraintChannel* InChannel) const
{
	OnConstraintChannelAdded.Broadcast(InSection, InChannel);
}

void UMovieSceneControlRigParameterTrack::RemoveAllConstraints() const
{
	// no need to compensate, we just want to get rig of all the animation data
	constexpr bool bDoNotCompensate = true;
	
	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneControlRigParameterSection* RigSection = Cast<UMovieSceneControlRigParameterSection>(Section))
		{
			const UControlRig* Rig = RigSection->GetControlRig();
			if (UWorld* World = Rig ? Rig->GetWorld() : nullptr)
			{
				RigSection->RemoveAllConstraints(World, bDoNotCompensate);
			}
		}
	}
	
	// also remove the constraints related to this ControlRig object that live within other constrained sections
	UWorld* World = ::IsValid(ControlRig) ? ControlRig->GetWorld() : nullptr;
	const UMovieScene* OuterMovieScene = GetTypedOuter<UMovieScene>();
	if (World && OuterMovieScene)
	{
		static const TSet< TSubclassOf<UMovieSceneTrack> > ConstrainedTracks({
			UMovieSceneControlRigParameterTrack::StaticClass(),
			UMovieScene3DTransformTrack::StaticClass()
		});

		const UObject* ThisTrack = this;
		const FSoftObjectPath RigPath(ControlRig);
		TSet<UTickableConstraint*> ConstraintsToRemove;
		
		auto ProcessSection = [ThisTrack, &ConstraintsToRemove, &RigPath](UMovieSceneSection* InSection)
		{
			if (InSection && InSection->GetOuter() != ThisTrack)
			{
				if (IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(InSection))
				{
					for (const FConstraintAndActiveChannel& ConstraintChannel: ConstrainedSection->GetConstraintsChannels())
					{
						if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
						{
							if (UTransformableControlHandle* ControlHandle = ::IsValid(TransformConstraint->ParentTRSHandle) ?
								Cast<UTransformableControlHandle>(TransformConstraint->ParentTRSHandle.Get()) : nullptr)
							{
								const FSoftObjectPath& HandleRigPath = ControlHandle->ControlRig.ToSoftObjectPath();
								if (HandleRigPath == RigPath)
								{
									ConstraintsToRemove.Add(TransformConstraint);
								}
							}
						}
					}
				}
			}
		};

		auto ProcessTrack = [&ProcessSection, ThisTrack](const UMovieSceneTrack* InTrack)
		{
			if (InTrack && InTrack != ThisTrack)
			{
				if (ConstrainedTracks.Contains(InTrack->GetClass()))
				{
					for (UMovieSceneSection* Section : InTrack->GetAllSections())
					{
						ProcessSection(Section);
					}
				}
			}
		};
			
		for (const FMovieSceneBinding& Binding : OuterMovieScene->GetBindings())
		{
			for (const TSubclassOf<UMovieSceneTrack>& TrackClass: ConstrainedTracks)
			{
				for (UMovieSceneTrack* Track: OuterMovieScene->FindTracks(TrackClass, Binding.GetObjectGuid()))
				{
					ProcessTrack(Track);
				}
			}
		}

		for (UMovieSceneTrack* Track: OuterMovieScene->GetTracks())
		{
			ProcessTrack(Track);
		}
		
		// remove constraints
		for (UTickableConstraint* ConstraintToRemove: ConstraintsToRemove)
		{
			FConstraintsManagerController::Get(World).RemoveConstraint(ConstraintToRemove, bDoNotCompensate);
		}
	}
}

void UMovieSceneControlRigParameterTrack::RemoveAllAnimationData()
{
	RemoveAllConstraints();
	Sections.Empty();
	SectionToKey = nullptr;
}

bool UMovieSceneControlRigParameterTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

void UMovieSceneControlRigParameterTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
	if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(&Section))
	{
		if (CRSection->GetControlRig() != ControlRig)
		{
			CRSection->SetControlRig(ControlRig);
		}
		CRSection->ReconstructChannelProxy();
	}

	if (Sections.Num() > 1)
	{
		SetSectionToKey(&Section);
	}
}

void UMovieSceneControlRigParameterTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	if (SectionToKey == &Section)
	{
		if (Sections.Num() > 0)
		{
			SectionToKey = Sections[0];
		}
		else
		{
			SectionToKey = nullptr;
		}
	}
}

void UMovieSceneControlRigParameterTrack::RemoveSectionAt(int32 SectionIndex)
{
	bool bResetSectionToKey = (SectionToKey == Sections[SectionIndex]);

	Sections.RemoveAt(SectionIndex);

	if (bResetSectionToKey)
	{
		SectionToKey = Sections.Num() > 0 ? Sections[0] : nullptr;
	}
}

bool UMovieSceneControlRigParameterTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

const TArray<UMovieSceneSection*>& UMovieSceneControlRigParameterTrack::GetAllSections() const
{
	return Sections;
}

void UMovieSceneControlRigParameterTrack::OnRemovedFromMovieSceneImpl()
{
	RemoveAllConstraints();
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneControlRigParameterTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("DisplayName", "Control Rig Parameter");
}
#endif


UMovieSceneSection* UMovieSceneControlRigParameterTrack::CreateControlRigSection(FFrameNumber StartTime, UControlRig* InControlRig, bool bInOwnsControlRig)
{
	if (InControlRig == nullptr)
	{
		return nullptr;
	}
	if (!bInOwnsControlRig)
	{
		InControlRig->Rename(nullptr, this);
	}

	UnbindControlRigDelegates();

	ControlRig = InControlRig;

	BindControlRigDelegates();

	UMovieSceneControlRigParameterSection* NewSection = Cast<UMovieSceneControlRigParameterSection>(CreateNewSection());

	UMovieScene* OuterMovieScene = GetTypedOuter<UMovieScene>();
	NewSection->SetRange(TRange<FFrameNumber>::All());

	//mz todo tbd maybe just set it to animated range? TRange<FFrameNumber> Range = OuterMovieScene->GetPlaybackRange();
	//Range.SetLowerBoundValue(StartTime);
	//NewSection->SetRange(Range);

	AddSection(*NewSection);

	return NewSection;
}

TArray<UMovieSceneSection*, TInlineAllocator<4>> UMovieSceneControlRigParameterTrack::FindAllSections(FFrameNumber Time)
{
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections;

	for (UMovieSceneSection* Section : Sections)
	{
		if (MovieSceneHelpers::IsSectionKeyable(Section) && Section->GetRange().Contains(Time))
		{
			OverlappingSections.Add(Section);
		}
	}

	Algo::Sort(OverlappingSections, MovieSceneHelpers::SortOverlappingSections);

	return OverlappingSections;
}


UMovieSceneSection* UMovieSceneControlRigParameterTrack::FindSection(FFrameNumber Time)
{
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections = FindAllSections(Time);

	if (OverlappingSections.Num())
	{
		if (SectionToKey && OverlappingSections.Contains(SectionToKey))
		{
			return SectionToKey;
		}
		else
		{
			return OverlappingSections[0];
		}
	}

	return nullptr;
}


UMovieSceneSection* UMovieSceneControlRigParameterTrack::FindOrExtendSection(FFrameNumber Time, float& Weight)
{
	Weight = 1.0f;
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections = FindAllSections(Time);
	if (SectionToKey && MovieSceneHelpers::IsSectionKeyable(SectionToKey))
	{
		bool bCalculateWeight = false;
		if (!OverlappingSections.Contains(SectionToKey))
		{
			if (SectionToKey->HasEndFrame() && SectionToKey->GetExclusiveEndFrame() <= Time)
			{
				if (SectionToKey->GetExclusiveEndFrame() != Time)
				{
					SectionToKey->SetEndFrame(Time);
				}
			}
			else
			{
				SectionToKey->SetStartFrame(Time);
			}
			if (OverlappingSections.Num() > 0)
			{
				bCalculateWeight = true;
			}
		}
		else
		{
			if (OverlappingSections.Num() > 1)
			{
				bCalculateWeight = true;
			}
		}
		//we need to calculate weight also possibly
		FOptionalMovieSceneBlendType BlendType = SectionToKey->GetBlendType();
		if (bCalculateWeight)
		{
			Weight = MovieSceneHelpers::CalculateWeightForBlending(SectionToKey, Time);
		}
		return SectionToKey;
	}
	else
	{
		if (OverlappingSections.Num() > 0)
		{
			return OverlappingSections[0];
		}
	}

	// Find a spot for the section so that they are sorted by start time
	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		UMovieSceneSection* Section = Sections[SectionIndex];

		// Check if there are no more sections that would overlap the time 
		if (!Sections.IsValidIndex(SectionIndex + 1) || (Sections[SectionIndex + 1]->HasEndFrame() && Sections[SectionIndex + 1]->GetExclusiveEndFrame() > Time))
		{
			// No sections overlap the time

			if (SectionIndex > 0)
			{
			// Append and grow the previous section
			UMovieSceneSection* PreviousSection = Sections[SectionIndex ? SectionIndex - 1 : 0];

			PreviousSection->SetEndFrame(Time);
			return PreviousSection;
			}
			else if (Sections.IsValidIndex(SectionIndex + 1))
			{
			// Prepend and grow the next section because there are no sections before this one
			UMovieSceneSection* NextSection = Sections[SectionIndex + 1];
			NextSection->SetStartFrame(Time);
			return NextSection;
			}
			else
			{
			// SectionIndex == 0 
			UMovieSceneSection* PreviousSection = Sections[0];
			if (PreviousSection->HasEndFrame() && PreviousSection->GetExclusiveEndFrame() <= Time)
			{
				// Append and grow the section
				if (PreviousSection->GetExclusiveEndFrame() != Time)
				{
					PreviousSection->SetEndFrame(Time);
				}
			}
			else
			{
				// Prepend and grow the section
				PreviousSection->SetStartFrame(Time);
			}
			return PreviousSection;
			}
		}
	}

	return nullptr;
}

UMovieSceneSection* UMovieSceneControlRigParameterTrack::FindOrAddSection(FFrameNumber Time, bool& bSectionAdded)
{
	bSectionAdded = false;

	UMovieSceneSection* FoundSection = FindSection(Time);
	if (FoundSection)
	{
		return FoundSection;
	}

	// Add a new section that starts and ends at the same time
	UMovieSceneSection* NewSection = CreateNewSection();
	ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
	NewSection->SetFlags(RF_Transactional);
	NewSection->SetRange(TRange<FFrameNumber>::Inclusive(Time, Time));

	Sections.Add(NewSection);

	bSectionAdded = true;

	return NewSection;
}

TArray<TWeakObjectPtr<UMovieSceneSection>> UMovieSceneControlRigParameterTrack::GetSectionsToKey() const
{
	TArray<TWeakObjectPtr<UMovieSceneSection>> SectionsToKey;
	if (SectionToKeyPerControl.Num() > 0)
	{
		SectionToKeyPerControl.GenerateValueArray(SectionsToKey);
	}
	else
	{
		SectionsToKey.Add(SectionToKey);
	}
	return SectionsToKey;
}

UMovieSceneSection* UMovieSceneControlRigParameterTrack::GetSectionToKey(const FName& InControlName) const
{
	if (const TWeakObjectPtr<UMovieSceneSection>* Section = SectionToKeyPerControl.Find(InControlName))
	{
		if (Section->IsValid())
		{
			return Section->Get();
		}
	}
	return GetSectionToKey();
}

void UMovieSceneControlRigParameterTrack::SetSectionToKey(UMovieSceneSection* InSection, const FName& InControlName)
{
	if (Sections.Num() < 1 || InSection == nullptr)
	{
		return;
	}
	Modify();
	SectionToKeyPerControl.Add(InControlName, InSection);
	SectionToKey = Sections[0];
}

void UMovieSceneControlRigParameterTrack::SetSectionToKey(UMovieSceneSection* InSection)
{
	if (Sections.IsEmpty())
	{
		return;
	}

	UMovieSceneControlRigParameterSection* ControlRigSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!ControlRigSection)
	{
		return;
	}

	if (!SectionToKeyPerControl.IsEmpty()) //we have sections that are in layers so need to respect them
	{
		bool bAlreadyModified = false;
		auto ModifyThis = [&bAlreadyModified](UMovieSceneControlRigParameterTrack* InTrack)
		{
			if (!bAlreadyModified)
			{
				InTrack->Modify();
				bAlreadyModified = true;
			}
		};
		
		if (bSetSectionToKeyPerControl)
		{
			for (TPair<FName, TWeakObjectPtr<UMovieSceneSection>>& SectionToKeyItem : SectionToKeyPerControl)
			{
				const FName& ControlName = SectionToKeyItem.Key;
				
				//only set it as the section to key if it's in that section, otherwise leave it alone
				if (ControlRigSection->ControlChannelMap.Contains(ControlName))
				{
					if (ControlRigSection->GetControlNameMask(ControlName))
					{
						if (InSection != SectionToKeyItem.Value)
						{
							ModifyThis(this);
							SectionToKeyItem.Value = ControlRigSection;
						}
					}
				}
			}
		}

		if (Sections[0] != SectionToKey)
		{
			ModifyThis(this);
			SectionToKey = Sections[0];
		}
	}
	else
	{
		if (ControlRigSection != SectionToKey)
		{
			Modify();
			SectionToKey = ControlRigSection;
		}
	}
}

UMovieSceneSection* UMovieSceneControlRigParameterTrack::GetSectionToKey() const
{
	if (SectionToKey)
	{
		return SectionToKey;
	}
	else if(Sections.Num() >0)
	{
		return Sections[0];
	}
	return nullptr;
}

void UMovieSceneControlRigParameterTrack::ReconstructControlRig()
{
	if (ControlRig && !ControlRig->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedInitialization))
	{
		ControlRig->ConditionalPostLoad();
		ControlRig->Initialize();
		CheckForNameChanges();

		for (UMovieSceneSection* Section : Sections)
		{
			if (Section)
			{
				UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section);
				if (CRSection)
				{
					if (CRSection->SpaceChannelAdded().IsBoundToObject(this) == false)
					{
						CRSection->SpaceChannelAdded().AddUObject(this, &UMovieSceneControlRigParameterTrack::HandleOnSpaceAdded);
					}

					if (!CRSection->ConstraintChannelAdded().IsBoundToObject(this))
					{
						CRSection->ConstraintChannelAdded().AddUObject(this, &UMovieSceneControlRigParameterTrack::HandleOnConstraintAdded);
					}
					
					CRSection->RecreateWithThisControlRig(ControlRig, CRSection->GetBlendType() == EMovieSceneBlendType::Absolute);
				}
			}
		}
	}
}

void UMovieSceneControlRigParameterTrack::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	BindPIEDelegates();

	FCoreUObjectDelegates::OnEndLoadPackage.AddUObject(this, &UMovieSceneControlRigParameterTrack::HandlePackageDone);
	// If we have a control Rig and it's not a native one, register OnEndLoadPackage callback on instance directly
	if (ControlRig && !ControlRig->GetAssetReference().IsNative())
	{
		ControlRig->OnEndLoadPackage().AddUObject(this, &UMovieSceneControlRigParameterTrack::HandleControlRigPackageDone);
	}
	// Otherwise try and reconstruct the control rig directly (which is fine for native classes)
	else
#endif
	{		
		ReconstructControlRig();
	}

	BindControlRigDelegates();
}

//mz todo this is coming from BuildPatchServices in Runtime/Online/BuildPatchServices/Online/Core/AsyncHelper.h
//looking at moving this over
//this is very useful since it properly handles passing in a this pointer to the async task.
namespace MovieSceneControlRigTrack
{
	/**
	 * Helper functions for wrapping async functionality.
	 */
	namespace AsyncHelpers
	{
		template<typename ResultType, typename... Args>
		static TFunction<void()> MakePromiseKeeper(const TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe>& Promise, const TFunction<ResultType(Args...)>& Function, Args... FuncArgs)
		{
			return [Promise, Function, FuncArgs...]()
			{
				Promise->SetValue(Function(FuncArgs...));
			};
		}

		template<typename... Args>
		static TFunction<void()> MakePromiseKeeper(const TSharedRef<TPromise<void>, ESPMode::ThreadSafe>& Promise, const TFunction<void(Args...)>& Function, Args... FuncArgs)
		{
			return [Promise, Function, FuncArgs...]()
			{
				Function(FuncArgs...);
				Promise->SetValue();
			};
		}

		template<typename ResultType, typename... Args>
		static TFuture<ResultType> ExecuteOnGameThread(const TFunction<ResultType(Args...)>& Function, Args... FuncArgs)
		{
			TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<ResultType>());
			TFunction<void()> PromiseKeeper = MakePromiseKeeper(Promise, Function, FuncArgs...);
			if (!IsInGameThread())
			{
				AsyncTask(ENamedThreads::GameThread, MoveTemp(PromiseKeeper));
			}
			else
			{
				PromiseKeeper();
			}
			return Promise->GetFuture();
		}

		template<typename ResultType>
		static TFuture<ResultType> ExecuteOnGameThread(const TFunction<ResultType()>& Function)
		{
			TSharedRef<TPromise<ResultType>, ESPMode::ThreadSafe> Promise = MakeShareable(new TPromise<ResultType>());
			TFunction<void()> PromiseKeeper = MakePromiseKeeper(Promise, Function);
			if (!IsInGameThread())
			{
				AsyncTask(ENamedThreads::GameThread, MoveTemp(PromiseKeeper));
			}
			else
			{
				PromiseKeeper();
			}
			return Promise->GetFuture();
		}
	}
}

void UMovieSceneControlRigParameterTrack::BindControlRigDelegates()
{
	if (IsValid(ControlRig))
	{
		if (!ControlRig->OnPreConstruction_AnyThread().IsBoundToObject(this))
		{
			ControlRig->OnPreConstruction_AnyThread().AddUObject(this,  &UMovieSceneControlRigParameterTrack::HandleOnPreConstruct_AnyThread);
			ControlRig->OnPostConstruction_AnyThread().AddUObject(this, &UMovieSceneControlRigParameterTrack::HandleOnPostConstructed);	
		}
	}
}

void UMovieSceneControlRigParameterTrack::UnbindControlRigDelegates()
{
	if (IsValid(ControlRig))
	{
		ControlRig->OnPreConstruction_AnyThread().RemoveAll(this);
		ControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
	}
}

void UMovieSceneControlRigParameterTrack::BindGameWorldControlRigDelegates(UControlRig* GameWorldControlRig)
{
	if (IsValid(GameWorldControlRig))
	{
		if (!GameWorldControlRig->OnPreConstruction_AnyThread().IsBoundToObject(this))
		{
			GameWorldControlRig->OnPreConstruction_AnyThread().AddUObject(this,  &UMovieSceneControlRigParameterTrack::HandleOnPreConstruct_AnyThread);
		}
	}	
}

void UMovieSceneControlRigParameterTrack::UnbindGameWorldControlRigDelegates(UControlRig* GameWorldControlRig)
{
	if (IsValid(GameWorldControlRig))
	{
		GameWorldControlRig->OnPreConstruction_AnyThread().RemoveAll(this);
	}
}

void UMovieSceneControlRigParameterTrack::HandleOnPreConstruct_AnyThread(UControlRig* Subject, const FName& InEventName)
{
	if (IsValid(Subject))
	{
		ApplyControlRigSettingsOverrides(Subject);
	}
}


void UMovieSceneControlRigParameterTrack::HandleOnPostConstructed_GameThread()
{
	if (IsValid(ControlRig))
	{
		CheckForNameChanges();
		TArray<FRigControlElement*> SortedControls;
		ControlRig->GetControlsInOrder(SortedControls);
#if WITH_EDITOR
		const FScopedTransaction PostConstructTransation(NSLOCTEXT("ControlRig", "PostConstructTransation", "Post Construct"), !GIsTransacting);
#endif		
		bool bSectionWasDifferent = false;
		for (UMovieSceneSection* BaseSection : GetAllSections())
		{
			if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(BaseSection))
			{
				if (Section->IsDifferentThanLastControlsUsedToReconstruct(SortedControls))
				{
					Section->RecreateWithThisControlRig(ControlRig, Section->GetBlendType() == EMovieSceneBlendType::Absolute);
					bSectionWasDifferent = true;
				}
			}
		}
		if (bSectionWasDifferent)
		{
			BroadcastChanged();
		}
		if (SortedControls.Num() > 0) //really set up
		{
			TArray<FName> Names = GetControlsWithDifferentRotationOrders();
			ResetControlsToSettingsRotationOrder(Names);
		}
	}
}

void UMovieSceneControlRigParameterTrack::HandleOnPostConstructed(UControlRig* Subject, const FName& InEventName)
{
	if(IsInGameThread())
	{
		HandleOnPostConstructed_GameThread();

	}
}




#if WITH_EDITORONLY_DATA
void UMovieSceneControlRigParameterTrack::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieSceneSection::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieSceneControlRigParameterSection::StaticClass()));
}
#endif

#if WITH_EDITOR
void UMovieSceneControlRigParameterTrack::BindPIEDelegates()
{
	OnPreBeginPIEHandle = FEditorDelegates::PreBeginPIE.AddUObject(this, &UMovieSceneControlRigParameterTrack::OnPreBeginPIE);
	EndPIEHandle = FEditorDelegates::EndPIE.AddUObject(this, &UMovieSceneControlRigParameterTrack::OnEndPIE);
	ShutdownPIEHandle = FEditorDelegates::ShutdownPIE.AddUObject(this, &UMovieSceneControlRigParameterTrack::OnShutdownPIE);
}

void UMovieSceneControlRigParameterTrack::UnbindPIEDelegates()
{
	FEditorDelegates::PreBeginPIE.Remove(OnPreBeginPIEHandle);
	FEditorDelegates::EndPIE.Remove(EndPIEHandle);
	FEditorDelegates::ShutdownPIE.Remove(ShutdownPIEHandle);
}

void UMovieSceneControlRigParameterTrack::HandlePackageDone(const FEndLoadPackageContext& Context)
{
	if (!ControlRig || ControlRig->GetAssetReference().IsNative())
	{
		// EndLoad is never called for native packages, so skip work
		FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
		return;
	}
	
	// ensure both the track package and the control rig package are fully end-loaded	
	if (!GetPackage()->GetHasBeenEndLoaded())
	{
		return;
	}

	// make sure the asset reference exists
	UObject* AssetReference = ControlRig->GetAssetReference().Get();
	if (!AssetReference)
	{
		return;
	}
	
	if (const UPackage* ControlRigPackage = Cast<UPackage>(AssetReference->GetOutermost()))
	{
		if (!ControlRigPackage->GetHasBeenEndLoaded())
		{
			return;
		}
	}

	// All dependent packages ready, no need to wait/check for any other packages
	// ReconstructControlRig may trigger loading of packages that we don't care about, so unregister from the delegate
	// before reconstruction to avoid infinite loop
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);

	// Only reconstruct in case it is not a native ControlRig class
	ReconstructControlRig();	
}

void UMovieSceneControlRigParameterTrack::HandleControlRigPackageDone(URigVMHost* InControlRig)
{
	if (ensure(ControlRig == InControlRig))
	{
		ControlRig->OnEndLoadPackage().RemoveAll(this);
		ReconstructControlRig();
	}
}
#endif


void UMovieSceneControlRigParameterTrack::PostEditImport()
{
	Super::PostEditImport();
	if (ControlRig)
	{
		BindControlRigDelegates();
		
		ControlRig->ClearFlags(RF_Transient); //when copied make sure it's no longer transient, sequencer does this for tracks/sections 
											  //but not for all objects in them since the control rig itself has transient objects.
	}
	ReconstructControlRig();
}

void UMovieSceneControlRigParameterTrack::RenameParameterName(const FName& OldParameterName, const FName& NewParameterName)
{
	if (OldParameterName != NewParameterName)
	{
		for (UMovieSceneSection* Section : Sections)
		{
			if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
			{
				bool bRequiredToRebuildProxy = false;
				CRSection->ForEachParameter([OldParameterName, NewParameterName, CRSection, &bRequiredToRebuildProxy](FBaseParameterNameAndValue* InOutParameter)
				{
					if(InOutParameter->ParameterName == OldParameterName)
					{
						CRSection->Modify();
						InOutParameter->ParameterName = NewParameterName;
						bRequiredToRebuildProxy = true;
					}
				});
				if(bRequiredToRebuildProxy)
				{
					CRSection->ReconstructChannelProxy();
				}
			}
		}
		if (FControlRotationOrder* RotationOrder = ControlsRotationOrder.Find(OldParameterName))
		{
			FControlRotationOrder NewRotationOrder = *RotationOrder;
			ControlsRotationOrder.Remove(OldParameterName);
			ControlsRotationOrder.Add(NewParameterName, NewRotationOrder);
		}
	}
}

void UMovieSceneControlRigParameterTrack::ReplaceControlRig(UControlRig* NewControlRig, bool RecreateChannels)
{
	if (ControlRig == NewControlRig)
	{
		return;
	}

	UnbindControlRigDelegates();


	ControlRig = NewControlRig;

	if (IsValid(ControlRig))
	{
		BindControlRigDelegates();

		if (ControlRig->GetOuter() != this)
		{
			ControlRig->Rename(nullptr, this);
		}
	}

	CheckForNameChanges();

	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
		{
			if (RecreateChannels)
			{
				CRSection->RecreateWithThisControlRig(NewControlRig, CRSection->GetBlendType() == EMovieSceneBlendType::Absolute);
			}
			else
			{
				CRSection->SetControlRig(NewControlRig);
			}
		}
	}
}
TOptional<EEulerRotationOrder> UMovieSceneControlRigParameterTrack::GetControlRotationOrder(const FRigControlElement* ControlElement,
	bool bCurrent) const
{
	TOptional<EEulerRotationOrder> Order;
	if (bCurrent)
	{
		const FControlRotationOrder* RotationOrder = ControlsRotationOrder.Find(ControlElement->GetFName());
		if (RotationOrder)
		{
			Order = RotationOrder->RotationOrder;
		}
	}
	else //use setting
	{
		if (ControlRig->GetHierarchy()->GetUsePreferredRotationOrder(ControlElement))
		{
			Order = ControlRig->GetHierarchy()->GetControlPreferredEulerRotationOrder(ControlElement);
		}
	}
	return Order;
}

void UMovieSceneControlRigParameterTrack::UpdateControlRigSettingsOverrides(const FInstancedPropertyBag& InNewOverrides)
{
	Modify();
	ControlRigSettingsOverrides = InNewOverrides;
}

void UMovieSceneControlRigParameterTrack::ApplyControlRigSettingsOverrides(UControlRig* InControlRig)
{
	if (!InControlRig)
	{
		return;
	}

	if (!ControlRigSettingsOverrides.IsValid())
	{
		return;
	}
	
	for (const FPropertyBagPropertyDesc& PropertyDesc : ControlRigSettingsOverrides.GetPropertyBagStruct()->GetPropertyDescs())
	{
		FRigVMExternalVariable Variable = InControlRig->GetPublicVariableByName(PropertyDesc.Name);
		if (Variable.IsValid())
		{
			uint8* TargetAddress = Variable.GetProperty()->ContainerPtrToValuePtr<uint8>(InControlRig->GetVariablesMemory());
			const uint8* SourceAddress = ControlRigSettingsOverrides.GetValue().GetMemory() + PropertyDesc.CachedProperty->GetOffset_ForInternal();
			FRigVMMemoryStorageStruct::CopyProperty(
				PropertyDesc.CachedProperty,
				TargetAddress,
				Variable.GetProperty(),
				SourceAddress
				);
		}
	}
	
}

void UMovieSceneControlRigParameterTrack::RecreateControlRig()
{
	Modify();
	ControlRig->Modify();
	
	ControlRig->Initialize();
	ControlRig->Evaluate_AnyThread();

	for (UMovieSceneSection* Section : GetAllSections())
	{
		if (Section)
		{
			UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section);
			if (CRSection)
			{
				Section->Modify();
				CRSection->RecreateWithThisControlRig(CRSection->GetControlRig(), true);
			}
		}
	}	
}


TArray<FName> UMovieSceneControlRigParameterTrack::GetControlsWithDifferentRotationOrders() const
{
	TArray<FName> Names;
	if (ControlRig && ControlRig->GetHierarchy())
	{
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		TArray<FRigControlElement*> SortedControls;
		ControlRig->GetControlsInOrder(SortedControls);
		for (const FRigControlElement* ControlElement : SortedControls)
		{
			if (!Hierarchy->IsAnimatable(ControlElement))
			{
				continue;
			}
			TOptional<EEulerRotationOrder> Current = GetControlRotationOrder(ControlElement, true);
			TOptional<EEulerRotationOrder> Setting = GetControlRotationOrder(ControlElement, false);
			if (Current != Setting)
			{
				Names.Add(ControlElement->GetFName());
			}

		}
	}
	return Names;
}

void UMovieSceneControlRigParameterTrack::ResetControlsToSettingsRotationOrder(const TArray<FName>& Names, EMovieSceneKeyInterpolation Interpolation)
{
	if (ControlRig && ControlRig->GetHierarchy())
	{
		for (const FName& Name : Names)
		{
			if (FRigControlElement* ControlElement = GetControlRig()->FindControl(Name))
			{

				TOptional<EEulerRotationOrder> Current = GetControlRotationOrder(ControlElement, true);
				TOptional<EEulerRotationOrder> Setting = GetControlRotationOrder(ControlElement, false);
				if (Current != Setting)
				{
					ChangeControlRotationOrder(Name, Setting, Interpolation);
				}
			}
		}
	}
}
void UMovieSceneControlRigParameterTrack::ChangeControlRotationOrder(const FName& InControlName, const TOptional<EEulerRotationOrder>& NewOrder, EMovieSceneKeyInterpolation Interpolation)
{
	if (ControlRig && ControlRig->GetHierarchy())
	{
		if (FRigControlElement* ControlElement = GetControlRig()->FindControl(InControlName))
		{
			TOptional<EEulerRotationOrder> Current = GetControlRotationOrder(ControlElement, true);
			if (Current != NewOrder)
			{
				if (NewOrder.IsSet())
				{
					FControlRotationOrder& RotationOrder = ControlsRotationOrder.FindOrAdd(ControlElement->GetFName());
					RotationOrder.RotationOrder = NewOrder.GetValue();
					TOptional<EEulerRotationOrder> Setting = GetControlRotationOrder(ControlElement, false);
					if (Setting != NewOrder)
					{
						RotationOrder.bOverrideSetting = true;
					}
				}
				else //no longer set so just remove
				{
					ControlsRotationOrder.Remove(InControlName);
				}
				for (UMovieSceneSection* Section : Sections)
				{
					if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
					{
						CRSection->ChangeControlRotationOrder(ControlElement->GetFName(), Current, NewOrder, Interpolation);
					}
				}
			}
		}
	}	
}

void UMovieSceneControlRigParameterTrack::GetSelectedNodes(TArray<FName>& SelectedControlNames)
{
	if (GetControlRig())
	{
		SelectedControlNames = GetControlRig()->CurrentControlSelection();
	}
}

int32 UMovieSceneControlRigParameterTrack::GetPriorityOrder() const
{
	return PriorityOrder;
}

void UMovieSceneControlRigParameterTrack::SetPriorityOrder(int32 InPriorityIndex)
{
	if (InPriorityIndex >= 0)
	{
		PriorityOrder = InPriorityIndex;
	}
	else
	{
		PriorityOrder = 0;
	}
}

#if WITH_EDITOR
bool UMovieSceneControlRigParameterTrack::GetFbxCurveDataFromChannelMetadata(const FMovieSceneChannelMetaData& MetaData, FControlRigFbxCurveData& OutCurveData)
{
	const FString ChannelName = MetaData.Name.ToString();
	TArray<FString> ChannelParts;

	// The channel has an attribute
	if (ChannelName.ParseIntoArray(ChannelParts, TEXT(".")) > 1)
	{
		// Retrieve the attribute
		OutCurveData.AttributeName = ChannelParts.Last();
		
		// The control name (left part) will be used as the node name
		OutCurveData.NodeName = ChannelParts[0];
		OutCurveData.ControlName = *OutCurveData.NodeName;

		// The channel has 3 parts, the middle one (i.e. Location) will be treated as the property name
		if (ChannelParts.Num() > 2)
		{
			OutCurveData.AttributePropertyName = ChannelParts[1];
		}
	}
	// The channel does not have an attribute
	else
	{
		// Thus no property above the attribute
		OutCurveData.AttributePropertyName.Empty();
		
		// The channel group will be used as the node name (name of the control this channel is grouped under - i.e. for animation channels)
		OutCurveData.NodeName = MetaData.Group.ToString();

		// The channel name will be used as the control name and attribute name (i.e. Weight)
		OutCurveData.ControlName = *ChannelName;
		OutCurveData.AttributeName = OutCurveData.ControlName.ToString();
	}

	if (OutCurveData.NodeName.IsEmpty() || OutCurveData.AttributeName.IsEmpty())
	{
		return false;
	}
	
	// Retrieve the control type
	if (GetControlRig())
	{
		if (FRigControlElement* Control = GetControlRig()->FindControl(OutCurveData.ControlName))
		{
			OutCurveData.ControlType = (FFBXControlRigTypeProxyEnum)(uint8)Control->Settings.ControlType;
			return true;
		}
	}
	return false;
}
#endif

FInstancedPropertyBag UMovieSceneControlRigParameterTrack::GetCurrentRigSettings()
{
	if (!ControlRig)
	{
		return {};
	}
	
	TArray<FPropertyBagPropertyDesc> BagProperties;
	TArray<const FProperty*> SourceProperties;
	TArray<FRigVMExternalVariable> PublicVariables = ControlRig->GetPublicVariables();
	for (const FRigVMExternalVariable& Variable : PublicVariables)
	{
		const FProperty* SourceProperty = ControlRig->GetAssetReference().FindVariablePropertyByName(Variable.GetName());
		SourceProperties.Add(SourceProperty);
		
		BagProperties.Add(FPropertyBagPropertyDesc(Variable.GetName(), SourceProperty));
	}

	if (BagProperties.Num() == 0)
	{
		return {};
	}

	FInstancedPropertyBag Settings;
	Settings.AddProperties(BagProperties);

	SyncRigSettingsToCurrentRig(Settings);

	return Settings;
}

void UMovieSceneControlRigParameterTrack::SyncRigSettingsToCurrentRig(FInstancedPropertyBag& InSettings)
{
	if (!ControlRig)
	{
		return;
	}
	
	for (const FPropertyBagPropertyDesc& PropertyDesc : InSettings.GetPropertyBagStruct()->GetPropertyDescs())
	{
		FRigVMExternalVariable Variable = ControlRig->GetPublicVariableByName(PropertyDesc.Name);
		if (Variable.IsValid())
		{
			const uint8* SourceAddress  = Variable.GetProperty()->ContainerPtrToValuePtr<uint8>(ControlRig->GetVariablesMemory());
			uint8* TargetAddress = InSettings.GetMutableValue().GetMemory() + PropertyDesc.CachedProperty->GetOffset_ForInternal();
			FRigVMMemoryStorageStruct::CopyProperty(
				PropertyDesc.CachedProperty,
				TargetAddress,
				Variable.GetProperty(),
				SourceAddress
				);
		}
	}
}

FInstancedPropertyBag UMovieSceneControlRigParameterTrack::BuildNewRigSettings(const TMap<FName, FString>& InSettingsToOverride)
{
	FInstancedPropertyBag Result = ControlRigSettingsOverrides;
	if (!Result.IsValid())
	{
		Result = GetCurrentRigSettings();
	}

	const UPropertyBag* BagStruct = Result.GetPropertyBagStruct();

	bool bSuccess = true;
	for (const TPair<FName, FString>& Pair : InSettingsToOverride)
	{
		const FPropertyBagPropertyDesc* Desc = Result.FindPropertyDescByName(Pair.Key);
		if (!Desc)
		{
			bSuccess = false;
			break;
		}

		// Making sure string parsing is the same as current Control Rig APIs
		FControlRigOverrideValue StringParser(Pair.Key.ToString(), BagStruct, Result.GetValue().GetMemory(), TrackName);
		const FControlRigOverrideValueErrorPipe::TReportFunction ReportFunction = [this, Pair, &bSuccess](const TCHAR* V, ELogVerbosity::Type Verbosity)
			{
				bSuccess = false;
				UE_LOGF(LogControlRig, Warning, "Problem during import of property '%ls' for Control Rig Track '%ls': %ls", *Pair.Key.ToString(), *TrackName.ToString(), V);
			};
		StringParser.SetFromString(Pair.Value, ReportFunction);
		if (!StringParser.IsValid())
		{
			bSuccess = false;
			break;
		}

		(void)StringParser.CopyToSubject(Result.GetMutableValue().GetMemory(), BagStruct);
	}

	// Avoid changing anything unless all overrides can be applied correctly
	if (!bSuccess)
	{
		return {};
	}

	return Result;
}

UControlRig* UMovieSceneControlRigParameterTrack::GetGameWorldControlRig(UWorld* InWorld, UObject* InBoundObject) 
{
	if (GameWorldControlRigs.Find(InWorld) == nullptr && ControlRig)
	{
		FControlRigAssetStrongReference AssetReference = ControlRig->GetAssetReference();
		UControlRig* NewGameWorldControlRig = AssetReference.CreateInstance(this, NAME_None, RF_Transient);
		BindGameWorldControlRigDelegates(NewGameWorldControlRig);

		if (!ControlRig->IsA<UFKControlRig>())
		{
			if (InBoundObject)
			{
				NewGameWorldControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
				NewGameWorldControlRig->GetObjectBinding()->BindToObject(InBoundObject);
			}
		}
		
		NewGameWorldControlRig->Initialize();
		if (UFKControlRig* FKControlRig = Cast<UFKControlRig>(Cast<UControlRig>(ControlRig)))
		{
			if (UFKControlRig* NewFKControlRig = Cast<UFKControlRig>(Cast<UControlRig>(NewGameWorldControlRig)))
			{
				NewFKControlRig->SetApplyMode(FKControlRig->GetApplyMode());
			}
		}
		else
		{
			NewGameWorldControlRig->SetIsAdditive(ControlRig->IsAdditive());
		}
		GameWorldControlRigs.Add(InWorld, NewGameWorldControlRig);
	}
	TObjectPtr<UControlRig> * GameWorldControlRig = GameWorldControlRigs.Find(InWorld);
	if (GameWorldControlRig != nullptr)
	{
		return GameWorldControlRig->Get();
	}
	return nullptr;
}

bool UMovieSceneControlRigParameterTrack::IsAGameInstance(const UControlRig* InControlRig, const bool bCheckValidWorld) const
{
	if (!InControlRig || GameWorldControlRigs.IsEmpty())
	{
		return false;
	}

	for (const TPair<TWeakObjectPtr<UWorld>, TObjectPtr<UControlRig>>& WorldAndControlRig: GameWorldControlRigs)
	{
		if (WorldAndControlRig.Value == InControlRig)
		{
			return bCheckValidWorld ? WorldAndControlRig.Key.IsValid() : true;
		}
	}
	
	return false;
}

TArray<FRigControlFBXNodeAndChannels>* UMovieSceneControlRigParameterTrack::GetNodeAndChannelMappings(UMovieSceneSection* InSection )
{
#if WITH_EDITOR
	if (GetControlRig() == nullptr)
	{
		return nullptr;
	}
	bool bSectionAdded;
	//use passed in section if available, else section to key if available, else first section or create one.
	UMovieSceneControlRigParameterSection* CurrentSectionToKey = InSection ? Cast<UMovieSceneControlRigParameterSection>(InSection) : Cast<UMovieSceneControlRigParameterSection>(GetSectionToKey());
	if (CurrentSectionToKey == nullptr)
	{
		CurrentSectionToKey = Cast<UMovieSceneControlRigParameterSection>(FindOrAddSection(0, bSectionAdded));
	} 
	if (!CurrentSectionToKey)
	{
		return nullptr;
	}

	const FName DoubleChannelTypeName = FMovieSceneDoubleChannel::StaticStruct()->GetFName();
	const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();
	const FName BoolChannelTypeName = FMovieSceneBoolChannel::StaticStruct()->GetFName();
	const FName EnumChannelTypeName = FMovieSceneByteChannel::StaticStruct()->GetFName();
	const FName IntegerChannelTypeName = FMovieSceneIntegerChannel::StaticStruct()->GetFName();

	// Our resulting mapping containing FBX node & UE channels data for each control
	TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels = new TArray<FRigControlFBXNodeAndChannels>();
	
	FMovieSceneChannelProxy& ChannelProxy = CurrentSectionToKey->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : CurrentSectionToKey->GetChannelProxy().GetAllEntries())
	{
		const FName ChannelTypeName = Entry.GetChannelTypeName();
		if (ChannelTypeName != DoubleChannelTypeName && ChannelTypeName != FloatChannelTypeName && ChannelTypeName != BoolChannelTypeName
			&& ChannelTypeName != EnumChannelTypeName && ChannelTypeName != IntegerChannelTypeName)
		{
			continue;
		}

		TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();
		TArrayView<const FMovieSceneChannelMetaData> AllMetaData = Entry.GetMetaData();

		for (int32 Index = 0; Index < Channels.Num(); ++Index)
		{
			FMovieSceneChannelHandle Channel = ChannelProxy.MakeHandle(ChannelTypeName, Index);
			const FMovieSceneChannelMetaData& MetaData = AllMetaData[Index];

			FControlRigFbxCurveData FbxCurveData;
			if (!GetFbxCurveDataFromChannelMetadata(MetaData, FbxCurveData))
			{
				continue;
			}

			// Retrieve the current control node, usually the last one but not given
			FRigControlFBXNodeAndChannels* CurrentNodeAndChannel;

			const int i = NodeAndChannels->FindLastByPredicate([FbxCurveData](const FRigControlFBXNodeAndChannels& A)
				{ return A.NodeName == FbxCurveData.NodeName && A.ControlName == FbxCurveData.ControlName; }
			);
			if (i != INDEX_NONE)
			{
				CurrentNodeAndChannel = &(*NodeAndChannels)[i];
			}
			// Create the node if not created yet
			else
			{
				NodeAndChannels->Add(FRigControlFBXNodeAndChannels());
				
				CurrentNodeAndChannel = &NodeAndChannels->Last();

				CurrentNodeAndChannel->MovieSceneTrack = this;
				CurrentNodeAndChannel->ControlType = FbxCurveData.ControlType;
				CurrentNodeAndChannel->NodeName = FbxCurveData.NodeName;
				CurrentNodeAndChannel->ControlName = FbxCurveData.ControlName;
			}

			if (ChannelTypeName == DoubleChannelTypeName)
			{
				FMovieSceneDoubleChannel* DoubleChannel = Channel.Cast<FMovieSceneDoubleChannel>().Get();
				CurrentNodeAndChannel->DoubleChannels.Add(DoubleChannel);
			}
			else if (ChannelTypeName == FloatChannelTypeName)
			{
				FMovieSceneFloatChannel* FloatChannel = Channel.Cast<FMovieSceneFloatChannel>().Get();
				CurrentNodeAndChannel->FloatChannels.Add(FloatChannel);
			}
			else if (ChannelTypeName == BoolChannelTypeName)
			{
				FMovieSceneBoolChannel* BoolChannel = Channel.Cast<FMovieSceneBoolChannel>().Get();
				CurrentNodeAndChannel->BoolChannels.Add(BoolChannel);
			}
			else if (ChannelTypeName == EnumChannelTypeName)
			{
				FMovieSceneByteChannel* EnumChannel = Channel.Cast<FMovieSceneByteChannel>().Get();
				CurrentNodeAndChannel->EnumChannels.Add(EnumChannel);
			}
			else if (ChannelTypeName == IntegerChannelTypeName)
			{
				FMovieSceneIntegerChannel* IntegerChannel = Channel.Cast<FMovieSceneIntegerChannel>().Get();
				CurrentNodeAndChannel->IntegerChannels.Add(IntegerChannel);
			}
		}
	}

	return NodeAndChannels;
#else
	return nullptr;
#endif
}

const TUniquePtr<FMovieSceneEvaluationTrack>& UMovieSceneControlRigParameterTrack::GetLegacyEvaluationTrack() const
{
	return LegacyEvaluationTrack;
}

FScopedLegacyEvaluationTrack::FScopedLegacyEvaluationTrack(UMovieSceneControlRigParameterTrack* InTrack)
	: WeakTrack(InTrack)
	, bCacheTemplate(UMovieSceneControlRigParameterTrack::ShouldCacheLegacyTemplate())
{
	if (InTrack && bCacheTemplate)
	{
		if (InTrack->LegacyEvaluationTrackCount == 0)
		{
			ensure(!InTrack->LegacyEvaluationTrack.IsValid());
			FMovieSceneEvaluationTrack EvaluationTrack = InTrack->GenerateTrackTemplate(InTrack);
			InTrack->LegacyEvaluationTrack = MakeUnique<FMovieSceneEvaluationTrack>(MoveTemp(EvaluationTrack));
		}
		else
		{
			ensure(InTrack->LegacyEvaluationTrack.IsValid());
		}
		InTrack->LegacyEvaluationTrackCount++;
	}
}

FScopedLegacyEvaluationTrack::FScopedLegacyEvaluationTrack(const UMovieSceneSection* InSection)
	: FScopedLegacyEvaluationTrack(InSection ? Cast<UMovieSceneControlRigParameterTrack>(InSection->GetOuter()) : nullptr)
{}

FScopedLegacyEvaluationTrack::~FScopedLegacyEvaluationTrack()
{
	if (UMovieSceneControlRigParameterTrack* Track = WeakTrack.Get())
	{
		ensure(bCacheTemplate == UMovieSceneControlRigParameterTrack::ShouldCacheLegacyTemplate());
		if (bCacheTemplate)
		{
			Track->LegacyEvaluationTrackCount--;
			ensure(Track->LegacyEvaluationTrackCount >= 0);
			if (Track->LegacyEvaluationTrackCount == 0)
			{
				Track->LegacyEvaluationTrack.Reset();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

