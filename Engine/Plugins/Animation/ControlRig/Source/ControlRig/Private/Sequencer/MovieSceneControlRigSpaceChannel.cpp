// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigSpaceChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneControlRigSpaceChannel)

bool FMovieSceneControlRigSpaceChannel::Evaluate(FFrameTime InTime, FMovieSceneControlRigSpaceBaseKey& OutValue) const
{
	if (!KeyTimes.IsEmpty())
	{
		const int32 Index = FMath::Max(0, Algo::UpperBound(KeyTimes, InTime.FrameNumber) - 1);
		OutValue = KeyValues[Index];
		return true;
	}
	
	if (bHasDefaultValue)
	{
		OutValue = DefaultValue;
		return true;
	}

	OutValue = FMovieSceneControlRigSpaceBaseKey();
	return true;
}

void FMovieSceneControlRigSpaceChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* DeleteKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, DeleteKeyHandles);
}

FKeyHandle FMovieSceneControlRigSpaceChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneControlRigSpaceChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneControlRigSpaceChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneControlRigSpaceChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneControlRigSpaceChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneControlRigSpaceChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	TArray<FRigElementKey> BeforeKeys, AfterKeys;
	GetUniqueSpaceList(&BeforeKeys);
	
	GetData().DeleteKeys(InHandles);

	GetUniqueSpaceList(&AfterKeys);
	BroadcastSpaceNoLongerUsed(BeforeKeys, AfterKeys);
}

void FMovieSceneControlRigSpaceChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	TArray<FRigElementKey> BeforeKeys, AfterKeys;
	GetUniqueSpaceList(&BeforeKeys);
	
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		FMovieSceneControlRigSpaceBaseKey Value;
		Evaluate(InTime, Value);
		GetData().UpdateOrAddKey(InTime, Value);
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);

	GetUniqueSpaceList(&AfterKeys);
	BroadcastSpaceNoLongerUsed(BeforeKeys, AfterKeys);
}

void FMovieSceneControlRigSpaceChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneControlRigSpaceChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneControlRigSpaceChannel::GetNumKeys() const
{
	return KeyTimes.Num();
}

void FMovieSceneControlRigSpaceChannel::Reset()
{
	KeyTimes.Reset();
	KeyValues.Reset();
	KeyHandles.Reset();
}

void FMovieSceneControlRigSpaceChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

void FMovieSceneControlRigSpaceChannel::GetUniqueSpaceList(TArray<FRigElementKey>* OutList)
{
	if(OutList)
	{
		OutList->Reset();
		
		for(const FMovieSceneControlRigSpaceBaseKey& KeyValue : KeyValues)
		{
			OutList->AddUnique(KeyValue.ControlRigElement);
		}
	}
}

//this will also delete any duplicated keys.
TArray <FSpaceRange> FMovieSceneControlRigSpaceChannel::FindSpaceIntervals()
{
	TArray<FSpaceRange> Ranges;
	TArray<int32> ToDelete;
	for (int32 Index = 0; Index < KeyTimes.Num(); ++Index)
	{
		FSpaceRange Range;
		Range.Key = KeyValues[Index];
		if (Index == KeyTimes.Num() - 1)
		{
			Range.Range.SetLowerBound(TRangeBound<FFrameNumber>(FFrameNumber(KeyTimes[Index])));
			Range.Range.SetUpperBound(TRangeBound<FFrameNumber>(FFrameNumber(KeyTimes[Index])));
		}
		else
		{
			int32 NextIndex = Index;
			FFrameNumber LowerBound = KeyTimes[Index],UpperBound = KeyTimes[Index];
			while (NextIndex < KeyTimes.Num() -1)
			{
				NextIndex = Index + 1;
				if (Range.Key == KeyValues[NextIndex])
				{
					Index++;
					ToDelete.Add(NextIndex);
				}
				else
				{
					UpperBound = KeyTimes[NextIndex];
					Index = NextIndex - 1;
					NextIndex = KeyTimes.Num();
				}
			}
			Range.Range.SetLowerBound(TRangeBound<FFrameNumber>(LowerBound));
			Range.Range.SetUpperBound(TRangeBound<FFrameNumber>(UpperBound));
		}
		Ranges.Add(Range);
	}
	//now delete duplicate values
	if (ToDelete.Num() > 0)
	{
		TArray<FKeyHandle> DeleteKeyHandles;
		for (int32 Index : ToDelete)
		{
			FKeyHandle KeyHandle = GetData().GetHandle(Index);
			DeleteKeyHandles.Add(KeyHandle);
		}

		GetData().DeleteKeys(DeleteKeyHandles);

	}
	return Ranges;
}

void FMovieSceneControlRigSpaceChannel::BroadcastSpaceNoLongerUsed(const TArray<FRigElementKey>& BeforeKeys, const TArray<FRigElementKey>& AfterKeys)
{
	if(BeforeKeys == AfterKeys)
	{
		return;
	}

	if(!SpaceNoLongerUsedEvent.IsBound())
	{
		return;
	}

	TArray<FRigElementKey> SpacesNoLongerUsed;
	for(const FRigElementKey& BeforeKey : BeforeKeys)
	{
		if(!AfterKeys.Contains(BeforeKey))
		{
			SpacesNoLongerUsed.Add(BeforeKey);
		}
	}

	if(!SpacesNoLongerUsed.IsEmpty())
	{
		SpaceNoLongerUsedEvent.Broadcast(this, SpacesNoLongerUsed);
	}
}

void FMovieSceneControlRigSpaceChannel::SetDefault(const FMovieSceneControlRigSpaceBaseKey& InDefaultValue)
{
	bHasDefaultValue = true;
	DefaultValue = InDefaultValue;
}

TOptional<FMovieSceneControlRigSpaceBaseKey> FMovieSceneControlRigSpaceChannel::GetDefault() const
{
	return bHasDefaultValue ? TOptional<FMovieSceneControlRigSpaceBaseKey>(DefaultValue) : TOptional<FMovieSceneControlRigSpaceBaseKey>();
}

void FMovieSceneControlRigSpaceChannel::ClearDefault()
{
	bHasDefaultValue = false;
	DefaultValue = FMovieSceneControlRigSpaceBaseKey();
}

FName FMovieSceneControlRigSpaceBaseKey::GetName() const
{
	switch (SpaceType)
	{
	case EMovieSceneControlRigSpaceType::Parent:
		return FName(TEXT("Parent"));

	case EMovieSceneControlRigSpaceType::World:
		return FName(TEXT("World"));

	case EMovieSceneControlRigSpaceType::ControlRig:
		return ControlRigElement.Name;
	};
	return NAME_None;
}
