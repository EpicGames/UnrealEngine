// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigSystem.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterBuffer.h"
#include "Sequencer/MovieSceneControlRigComponentTypes.h"
#include "Algo/IndexOf.h"
#include "ConstraintsManager.h"
#include "MovieSceneTracksComponentTypes.h"
#include "EntitySystem/MovieSceneInterrogation.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneInitialValueSystem.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "ControlRig.h"
#include "Components/SkeletalMeshComponent.h"
#include "Sequencer/ControlRigLayerInstance.h"
#include "IControlRigObjectBinding.h"
#include "ControlRigObjectBinding.h"
#include "TransformNoScale.h"
#include "ControlRigComponent.h"
#include "SkeletalMeshRestoreState.h"
#include "Rigs/FKControlRig.h"
#include "UObject/UObjectAnnotation.h"
#include "Rigs/RigHierarchy.h"
#include "Transform/DependencyBuilder.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformableHandleUtils.h"
#include "Constraints/ControlRigTransformableHandle.h"


#if WITH_EDITOR
#include "Editor.h"
#include "AnimCustomInstanceHelper.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneControlRigSystem)

namespace UE::MovieScene
{

TAutoRegisterPreAnimatedStorageID<FPreAnimatedControlRigStorage> FPreAnimatedControlRigStorage::StorageID;
TAutoRegisterPreAnimatedStorageID<FPreAnimatedControlRigParameterStorage> FPreAnimatedControlRigParameterStorage::StorageID;



UControlRig* GetControlRig(UMovieSceneControlRigParameterTrack* Track, UObject* BoundObject)
{
	UWorld* GameWorld = (BoundObject && BoundObject->GetWorld() && BoundObject->GetWorld()->IsGameWorld()) ? BoundObject->GetWorld() : nullptr;

	UControlRig* ControlRig = GameWorld ? Track->GetGameWorldControlRig(GameWorld, BoundObject) : Track->GetControlRig();
	if (!ControlRig)
	{
		return nullptr;
	}

	if (ControlRig->GetObjectBinding())
	{
		if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
		{
			if (AActor* Actor = Cast<AActor>(BoundObject))
			{
				if (UControlRigComponent* NewControlRigComponent = Actor->FindComponentByClass<UControlRigComponent>())
				{
					if (NewControlRigComponent->GetWorld())
					{
						if (NewControlRigComponent->GetWorld()->IsGameWorld())
						{
							ControlRig = NewControlRigComponent->GetControlRig();
							if (!ControlRig)
							{
								NewControlRigComponent->Initialize();
								ControlRig = NewControlRigComponent->GetControlRig();
							}
							if (ControlRig)
							{
								if (ControlRig->GetObjectBinding() == nullptr)
								{
									ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
								}
								if (ControlRig->GetObjectBinding()->GetBoundObject() != NewControlRigComponent)
								{
									ControlRig->GetObjectBinding()->BindToObject(BoundObject);
								}
							}
						}
						else if (NewControlRigComponent != ControlRigComponent)
						{
							NewControlRigComponent->SetControlRig(ControlRig);
						}
					}
				}
			}
		}
		else if (UControlRigComponent* NewControlRigComponent = Cast<UControlRigComponent>(BoundObject))
		{
			if (NewControlRigComponent->GetWorld())
			{
				if (NewControlRigComponent->GetWorld()->IsGameWorld())
				{
					ControlRig = NewControlRigComponent->GetControlRig();
					if (!ControlRig)
					{
						NewControlRigComponent->Initialize();
						ControlRig = NewControlRigComponent->GetControlRig();
					}
					if (ControlRig)
					{
						if (ControlRig->GetObjectBinding() == nullptr)
						{
							ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
						}
						if (ControlRig->GetObjectBinding()->GetBoundObject() != NewControlRigComponent)
						{
							ControlRig->GetObjectBinding()->BindToObject(BoundObject);
						}
					}
				}
				else if (NewControlRigComponent != ControlRigComponent)
				{
					NewControlRigComponent->SetControlRig(ControlRig);
				}
			}
		}
	}

	return ControlRig;
}

UTickableConstraint* CreateConstraintIfNeeded(UWorld* InWorld, UTickableConstraint* Constraint, UMovieSceneControlRigParameterSection* InSection)
{
	if (!Constraint)
	{
		return nullptr;
	}

	// it's possible that we have it but it's not in the manager, due to manager not being saved with it (due to spawning or undo/redo).
	if (InWorld)
	{
		if (!FConstraintsManagerController::Get(InWorld).GetConstraint(Constraint->ConstraintID))
		{
			FConstraintsManagerController::Get(InWorld).AddConstraint(Constraint);
			//need to reconstruct channels here.. note this is now lazy and so will recreate it next time view requests it
			//but only do it if the control rig has a valid world it may not for example in PIE
			if (InSection->GetControlRig() && InSection->GetControlRig()->GetWorld())
			{
				InSection->ReconstructChannelProxy();
				InSection->MarkAsChanged();
			}
		}
	}

	return Constraint;
}


/**
 * Initial value processor for control rig parameters.
 * Responsible for caching the current value of scalar, vector and transform parameters into
 * either an initial value cache index, initial value component, or both.
 */
struct FControlRigInitialValueProcessor : IInitialValueProcessor
{
	FBuiltInComponentTypes* BuiltInComponents;
	FControlRigComponentTypes* ControlRigComponents;
	FMovieSceneTracksComponentTypes* TracksComponents;

	/*~ Transient properties reset on each usage (through Initialize/Finalize). */
	IInterrogationExtension* Interrogation;
	FInitialValueCache* InitialValueCache;
	FEntityAllocationWriteContext WriteContext;
	UMovieSceneControlRigParameterEvaluatorSystem* ParameterSystem;

	FControlRigInitialValueProcessor()
		: BuiltInComponents(FBuiltInComponentTypes::Get())
		, ControlRigComponents(FControlRigComponentTypes::Get())
		, TracksComponents(FMovieSceneTracksComponentTypes::Get())
		, Interrogation(nullptr)
		, InitialValueCache(nullptr)
		, WriteContext(FEntityAllocationWriteContext::NewAllocation())
		, ParameterSystem(nullptr)
	{}

	virtual void Initialize(UMovieSceneEntitySystemLinker* Linker, FInitialValueCache* InInitialValueCache) override
	{
		Interrogation     = Linker->FindExtension<IInterrogationExtension>();
		ParameterSystem   = Linker->FindSystem<UMovieSceneControlRigParameterEvaluatorSystem>();
		InitialValueCache = InInitialValueCache;
		WriteContext      = FEntityAllocationWriteContext(Linker->EntityManager);
	}

	virtual void PopulateFilter(FEntityComponentFilter& OutFilter) const override
	{
		OutFilter.All({ ControlRigComponents->ControlRigSource, TracksComponents->GenericParameterName, ControlRigComponents->Tags.ControlRigParameter });
	}

	virtual void Process(const FEntityAllocation* Allocation, const FComponentMask& AllocationType) override
	{
		if (!ParameterSystem)
		{
			return;
		}
		if (Interrogation && AllocationType.Contains(BuiltInComponents->Interrogation.OutputKey))
		{
			VisitInterrogationAllocation(Allocation);
		}
		else
		{
			VisitAllocation(Allocation);
		}
	}

	virtual void Finalize() override
	{
		Interrogation     = nullptr;
		InitialValueCache = nullptr;
		ParameterSystem   = nullptr;
	}

	void VisitAllocation(const FEntityAllocation* Allocation)
	{
		if (TryCacheInitialValues<FFloatPropertyTraits, float>(Allocation, TracksComponents->Parameters.Scalar.InitialValue))
		{
			return;
		}
		if (TryCacheInitialValues<FFloatVectorPropertyTraits, FVector3f>(Allocation, TracksComponents->Parameters.Vector3.InitialValue))
		{
			return;
		}
		if (TryCacheInitialValues<FEulerTransformPropertyTraits, FEulerTransform>(Allocation, TracksComponents->Parameters.Transform.InitialValue))
		{
			return;
		}
	}

	void VisitInterrogationAllocation(const FEntityAllocation* Allocation)
	{
		if (TryInterrogateValues<FFloatPropertyTraits, float>(Allocation, TracksComponents->Parameters.Scalar.InitialValue))
		{
			return;
		}
		if (TryInterrogateValues<FFloatVectorPropertyTraits, FVector3f>(Allocation, TracksComponents->Parameters.Vector3.InitialValue))
		{
			return;
		}
		if (TryInterrogateValues<FEulerTransformPropertyTraits, FEulerTransform>(Allocation, TracksComponents->Parameters.Transform.InitialValue))
		{
			return;
		}
	}

private:

	/** Default cache function for retrieving a control rig variable */
	template<typename RigControlType, typename InitialValueType>
	void ControlValueToInitialValue(UControlRig* Rig, FRigControlElement* Control, InitialValueType& OutInitialValue)
	{
		OutInitialValue = Rig->GetControlValue(Control, ERigControlValueType::Current).Get<RigControlType>();
	}

	/** Specialization for vector controls */
	template<>
	void ControlValueToInitialValue<FVector3f, FFloatIntermediateVector>(UControlRig* Rig, FRigControlElement* Control, FFloatIntermediateVector& OutInitialValue)
	{
		FVector3f Value = Rig->GetControlValue(Control, ERigControlValueType::Current).Get<FVector3f>();
		OutInitialValue = FFloatIntermediateVector(Value.X, Value.Y, Value.Z);
	}

	/** Specialization for transform controls */
	template<>
	void ControlValueToInitialValue<FEulerTransform, FIntermediate3DTransform>(UControlRig* Rig, FRigControlElement* Control, FIntermediate3DTransform& OutInitialValue)
	{
		FEulerTransform EulerTransform;

		switch (Control->Settings.ControlType)
		{
			case ERigControlType::Transform:
			{
				const FTransform Val = Rig->GetControlValue(Control, ERigControlValueType::Current)
					.Get<FRigControlValue::FTransform_Float>().ToTransform();

				EulerTransform = FEulerTransform(Val);
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				const FTransformNoScale NoScale = Rig->GetControlValue(Control, ERigControlValueType::Current)
					.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();

				EulerTransform = FEulerTransform(NoScale.ToFTransform());
				break;
			}
			case ERigControlType::EulerTransform:
			{
				EulerTransform = Rig->GetControlValue(Control, ERigControlValueType::Current)
					.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
				break;
			}
		}

		FVector Vector = Rig->GetControlSpecifiedEulerAngle(Control);
		OutInitialValue = FIntermediate3DTransform(EulerTransform.Location, FRotator(Vector.Y, Vector.Z, Vector.X), EulerTransform.Scale);
	}

	template<typename PropertyTraits, typename RigControlType, typename InitialValueType>
	bool TryCacheInitialValues(const FEntityAllocation* Allocation, TComponentTypeID<InitialValueType> InitialValue)
	{
		TOptionalComponentWriter<InitialValueType> InitialValues = Allocation->TryWriteComponents(InitialValue, WriteContext);
		if (!InitialValues)
		{
			return false;
		}

		TPropertyValueStorage<PropertyTraits>* CacheStorage = InitialValueCache ? InitialValueCache->GetStorage<PropertyTraits>(InitialValue) : nullptr;

		TComponentReader<FControlRigSourceData> ControlRigSources = Allocation->ReadComponents(ControlRigComponents->ControlRigSource);
		TComponentReader<FName>                    ParameterNames  = Allocation->ReadComponents(TracksComponents->GenericParameterName);

		const int32 Num = Allocation->Num();
		if (CacheStorage)
		{
			TComponentWriter<FInitialValueIndex> InitialValueIndices = Allocation->WriteComponents(BuiltInComponents->InitialValueIndex, WriteContext);

			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = ParameterSystem->GetRigFromTrack(ControlRigSources[Index].Track);
				FRigControlElement* ControlElement = Rig ? Rig->FindControl(ParameterNames[Index]) : nullptr;

				if (Rig && ControlElement)
				{
					TOptional<FInitialValueIndex> ExistingIndex = CacheStorage->FindPropertyIndex(Rig, ParameterNames[Index]);
					if (ExistingIndex)
					{
						InitialValues[Index] = CacheStorage->GetCachedValue(ExistingIndex.GetValue());
					}
					else
					{
						ControlValueToInitialValue<RigControlType>(Rig, ControlElement, InitialValues[Index]);
						InitialValueIndices[Index] = CacheStorage->AddInitialValue(Rig, InitialValues[Index], ParameterNames[Index]);
					}
				}
			}
		}
		else
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = ParameterSystem->GetRigFromTrack(ControlRigSources[Index].Track);
				FRigControlElement* ControlElement = Rig ? Rig->FindControl(ParameterNames[Index]) : nullptr;

				if (Rig && ControlElement)
				{
					ControlValueToInitialValue<RigControlType>(Rig, ControlElement, InitialValues[Index]);
				}
			}
		}

		return true;
	}


	template<typename PropertyTraits, typename RigControlType, typename InitialValueType>
	bool TryInterrogateValues(const FEntityAllocation* Allocation, TComponentTypeID<InitialValueType> InitialValue)
	{
		TOptionalComponentWriter<InitialValueType> InitialValues = Allocation->TryWriteComponents(InitialValue, WriteContext);
		if (!InitialValues)
		{
			return false;
		}

		const int32 Num = Allocation->Num();

		TComponentReader<FInterrogationKey>    OutputKeys   = Allocation->ReadComponents(BuiltInComponents->Interrogation.OutputKey);
		TPropertyValueStorage<PropertyTraits>* CacheStorage = InitialValueCache ? InitialValueCache->GetStorage<PropertyTraits>(InitialValue) : nullptr;

		const FSparseInterrogationChannelInfo& SparseChannelInfo = Interrogation->GetSparseChannelInfo();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			FInterrogationChannel Channel = OutputKeys[Index].Channel;

			const FInterrogationChannelInfo* ChannelInfo = SparseChannelInfo.Find(Channel);
			UControlRig* Rig = ChannelInfo ? Cast<UControlRig>(ChannelInfo->WeakObject.Get()) : nullptr;
			if (!ChannelInfo || !Rig || ChannelInfo->PropertyBinding.PropertyName.IsNone())
			{
				continue;
			}

			FRigControlElement* ControlElement = Rig->FindControl(ChannelInfo->PropertyBinding.PropertyName);
			if (!ControlElement)
			{
				continue;
			}

			// Retrieve a cached value if possible
			if (CacheStorage)
			{
				const InitialValueType* CachedValue = CacheStorage->FindCachedValue(Rig, ChannelInfo->PropertyBinding.PropertyName);
				if (CachedValue)
				{
					InitialValues[Index] = *CachedValue;
					continue;
				}
			}

			// No cached value available, must retrieve it now
			ControlValueToInitialValue<RigControlType>(Rig, ControlElement, InitialValues[Index]);
		};
		return true;
	}
};




/**
 * Mutation that adds and assigns accumulation buffer indices to parameters
 */
struct FControlRigAccumulationEntryIndexMutation : IMovieSceneEntityMutation
{
	FAccumulatedControlRigValues* AccumulatedValues;
	FPreAnimatedControlRigParameterStorage* PreAnimatedParameters;

	FControlRigAccumulationEntryIndexMutation(FAccumulatedControlRigValues* InAccumulatedValues, FPreAnimatedControlRigParameterStorage* InPreAnimatedParameters)
		: AccumulatedValues(InAccumulatedValues)
		, PreAnimatedParameters(InPreAnimatedParameters)
	{
	}
	void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
		InOutEntityComponentTypes->Set(ControlRigComponents->AccumulatedControlEntryIndex);
	}
	void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const override
	{
		ProcessAllocation(Allocation, AllocationType);
	}
	void InitializeUnmodifiedAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const override
	{
		ProcessAllocation(Allocation, AllocationType);
	}
	void ProcessAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
	{
		FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();

		FControlRigComponentTypes*       ControlRigComponents = FControlRigComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents     = FMovieSceneTracksComponentTypes::Get();

		TComponentReader<FName>                 ParameterNames    = Allocation->ReadComponents(TracksComponents->GenericParameterName);
		TComponentWriter<FControlRigSourceData> ControlRigSources = Allocation->WriteComponents(ControlRigComponents->ControlRigSource, WriteContext);

		EControlRigControlType ControlType;
		if (AllocationType.Contains(TracksComponents->Parameters.Bool.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Bool;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Byte.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Enum;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Integer.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Integer;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Scalar.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Scalar;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Vector3.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Vector;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Transform.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Transform;
		}
		else if (AllocationType.Contains(ControlRigComponents->Tags.Space))
		{
			ControlType = EControlRigControlType::Space;
		}
		else
		{
			return;
		}

		TComponentWriter<FAccumulatedControlEntryIndex> AccumulatorEntryIndices = Allocation->WriteComponents(ControlRigComponents->AccumulatedControlEntryIndex, WriteContext);

		const int32 Num = Allocation->Num();

		for (int32 Index = 0; Index < Num; ++Index)
		{
			AccumulatorEntryIndices[Index] = AccumulatedValues->AllocateEntryIndex(ControlRigSources[Index].Track, ParameterNames[Index], ControlType);
		}
	}
};


struct FRemoveInvalidControlRigAccumulationComponents : IMovieSceneConditionalEntityMutation
{
	FAccumulatedControlRigValues* AccumulatedValues;

	FRemoveInvalidControlRigAccumulationComponents(FAccumulatedControlRigValues* InAccumulatedValues)
		: AccumulatedValues(InAccumulatedValues)
	{
	}
	void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const override
	{
		FControlRigComponentTypes*              ControlRigComponents = FControlRigComponentTypes::Get();
		TComponentReader<FControlRigSourceData> ControlRigSources    = Allocation->ReadComponents(ControlRigComponents->ControlRigSource);

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			// If the entity does not have a valid entry, set the bit, resulting in the component being removed
			if (!AccumulatedValues->DoesEntryExistForTrack(ControlRigSources[Index].Track))
			{
				OutEntitiesToMutate.PadToNum(Index+1, false);
				OutEntitiesToMutate[Index] = true;
			}
		}
	}
	void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
		InOutEntityComponentTypes->Remove(ControlRigComponents->AccumulatedControlEntryIndex);
	}
};


struct FInitialControlRigParameterValueMutation : IMovieSceneEntityMutation
{
	FAccumulatedControlRigValues* AccumulatedValues;
	FInitialControlRigParameterValueMutation(FAccumulatedControlRigValues* InAccumulatedValues)
		: AccumulatedValues(InAccumulatedValues)
	{
	}
	void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		FControlRigComponentTypes*       ControlRigComponents = FControlRigComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents     = FMovieSceneTracksComponentTypes::Get();

		if (InOutEntityComponentTypes->Contains(TracksComponents->Parameters.Bool.PropertyTag))
		{
			InOutEntityComponentTypes->Set(TracksComponents->Parameters.Bool.InitialValue);
		}
		else if (InOutEntityComponentTypes->Contains(TracksComponents->Parameters.Byte.PropertyTag))
		{
			InOutEntityComponentTypes->Set(TracksComponents->Parameters.Byte.InitialValue);
		}
		else if (InOutEntityComponentTypes->Contains(TracksComponents->Parameters.Integer.PropertyTag))
		{
			InOutEntityComponentTypes->Set(TracksComponents->Parameters.Integer.InitialValue);
		}
		else if (InOutEntityComponentTypes->Contains(TracksComponents->Parameters.Scalar.PropertyTag))
		{
			InOutEntityComponentTypes->Set(TracksComponents->Parameters.Scalar.InitialValue);
		}
		else if (InOutEntityComponentTypes->Contains(TracksComponents->Parameters.Vector3.PropertyTag))
		{
			InOutEntityComponentTypes->Set(TracksComponents->Parameters.Vector3.InitialValue);
		}
		else if (InOutEntityComponentTypes->Contains(TracksComponents->Parameters.Transform.PropertyTag))
		{
			InOutEntityComponentTypes->Set(TracksComponents->Parameters.Transform.InitialValue);
		}
	}
	void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const override
	{
		FEntityAllocationWriteContext WriteContext = FEntityAllocationWriteContext::NewAllocation();

		FControlRigComponentTypes*       ControlRigComponents = FControlRigComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents     = FMovieSceneTracksComponentTypes::Get();

		TComponentReader<FAccumulatedControlEntryIndex> AccumulatorEntryIndices = Allocation->ReadComponents(ControlRigComponents->AccumulatedControlEntryIndex);
		TComponentReader<FName> ParameterNames = Allocation->ReadComponents(TracksComponents->GenericParameterName);

		const int32 Num = Allocation->Num();
		if (AllocationType.Contains(TracksComponents->Parameters.Bool.PropertyTag))
		{
			TComponentWriter<bool> InitialValues = Allocation->WriteComponents(TracksComponents->Parameters.Bool.InitialValue, WriteContext);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = AccumulatedValues->FindControlRig(AccumulatorEntryIndices[Index]);
				if (FRigControlElement* ControlElement = Rig->FindControl(ParameterNames[Index]))
				{
					InitialValues[Index] = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<bool>();
				}
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Byte.PropertyTag))
		{
			TComponentWriter<uint8> InitialValues = Allocation->WriteComponents(TracksComponents->Parameters.Byte.InitialValue, WriteContext);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = AccumulatedValues->FindControlRig(AccumulatorEntryIndices[Index]);
				if (FRigControlElement* ControlElement = Rig->FindControl(ParameterNames[Index]))
				{
					const int32 EnumAsInt = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int32>();
					InitialValues[Index] = static_cast<uint8>(EnumAsInt);
				}
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Integer.PropertyTag))
		{
			TComponentWriter<int32> InitialValues = Allocation->WriteComponents(TracksComponents->Parameters.Integer.InitialValue, WriteContext);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = AccumulatedValues->FindControlRig(AccumulatorEntryIndices[Index]);
				if (FRigControlElement* ControlElement = Rig->FindControl(ParameterNames[Index]))
				{
					InitialValues[Index] = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<int32>();
				}
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Scalar.PropertyTag))
		{
			TComponentWriter<double> InitialValues = Allocation->WriteComponents(TracksComponents->Parameters.Scalar.InitialValue, WriteContext);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = AccumulatedValues->FindControlRig(AccumulatorEntryIndices[Index]);
				if (FRigControlElement* ControlElement = Rig->FindControl(ParameterNames[Index]))
				{
					InitialValues[Index] = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<float>();
				}
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Vector3.PropertyTag))
		{
			TComponentWriter<FFloatIntermediateVector> InitialValues = Allocation->WriteComponents(TracksComponents->Parameters.Vector3.InitialValue, WriteContext);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = AccumulatedValues->FindControlRig(AccumulatorEntryIndices[Index]);
				if (FRigControlElement* ControlElement = Rig->FindControl(ParameterNames[Index]))
				{
					FVector3f Value = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FVector3f>();
					InitialValues[Index] = FFloatIntermediateVector(Value.X, Value.Y, Value.Z);
				}
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Transform.PropertyTag))
		{
			TComponentWriter<FIntermediate3DTransform> InitialValues = Allocation->WriteComponents(TracksComponents->Parameters.Transform.InitialValue, WriteContext);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = AccumulatedValues->FindControlRig(AccumulatorEntryIndices[Index]);
				if (FRigControlElement* ControlElement = Rig->FindControl(ParameterNames[Index]))
				{
					switch (ControlElement->Settings.ControlType)
					{

						case ERigControlType::Transform:
						{
							const FTransform Val = Rig->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransform_Float>().ToTransform();
							FEulerTransform EulerTransform(Val);
							FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
							EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
							InitialValues[Index] = FIntermediate3DTransform(EulerTransform.Location, EulerTransform.Rotation, EulerTransform.Scale);
							break;
						}
						case ERigControlType::TransformNoScale:
						{
							const FTransformNoScale NoScale =
								Rig
								->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
							FEulerTransform EulerTransform(NoScale.ToFTransform());
							FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
							EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
							InitialValues[Index] = FIntermediate3DTransform(EulerTransform.Location, EulerTransform.Rotation, EulerTransform.Scale);
							break;
						}
						case ERigControlType::EulerTransform:
						{
							FEulerTransform EulerTransform =
								Rig
								->GetControlValue(ControlElement, ERigControlValueType::Current).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
							FVector Vector = Rig->GetControlSpecifiedEulerAngle(ControlElement);
							EulerTransform.Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
							InitialValues[Index] = FIntermediate3DTransform(EulerTransform.Location, EulerTransform.Rotation, EulerTransform.Scale);
							break;
						}
					}
				}
			}
		}
	}
	void InitializeUnmodifiedAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const override
	{
	}
};


FAnimatedControlRigParameterInfo::~FAnimatedControlRigParameterInfo()
{
}

void CollectGarbageForOutput(FAnimatedControlRigParameterInfo* Output)
{
	// This should only happen during garbage collection
	if (Output->OutputEntityID.IsValid())
	{
		UMovieSceneEntitySystemLinker* Linker = Output->WeakLinker.Get();
		if (Linker)
		{
			Linker->EntityManager.AddComponent(Output->OutputEntityID, FBuiltInComponentTypes::Get()->Tags.NeedsUnlink);
		}

		Output->OutputEntityID = FMovieSceneEntityID();
	}

	if (Output->BlendChannelID.IsValid())
	{
		UMovieSceneBlenderSystem* BlenderSystem = Output->WeakBlenderSystem.Get();
		if (BlenderSystem)
		{
			BlenderSystem->ReleaseBlendChannel(Output->BlendChannelID);
		}
		Output->BlendChannelID = FMovieSceneBlendChannelID();
	}
}


TSharedPtr<FPreAnimatedControlRigParameterTraits::FPreAnimatedBufferPairs> FPreAnimatedControlRigParameterTraits::GetBuffers(UControlRig* Rig)
{
	check(Rig);

	TWeakPtr<FPreAnimatedBufferPairs>& WeakBuffers = PreAnimatedBuffers.FindOrAdd(Rig);
	TSharedPtr<FPreAnimatedBufferPairs> Buffers = WeakBuffers.Pin();
	if (!Buffers)
	{
		Buffers = MakeShared<FPreAnimatedBufferPairs>();
		WeakBuffers = Buffers;
	}
	return Buffers;
}

FPreAnimatedControlRigParameterTraits::FPreAnimatedParameterKey FPreAnimatedControlRigParameterTraits::CachePreAnimatedValue(UControlRig* Rig, FName ParameterName)
{
	check(Rig);
	TSharedPtr<FPreAnimatedBufferPairs> Buffers = GetBuffers(Rig);

	if (FRigControlElement* ControlElement = Rig->FindControl(ParameterName))
	{
		Buffers->Transient.AddCurrentValue(Rig, ControlElement);
	}
	return FPreAnimatedControlRigParameterTraits::FPreAnimatedParameterKey(Buffers, ParameterName);
}

void FPreAnimatedControlRigParameterTraits::RestorePreAnimatedValue(const TTuple<FObjectKey, FName>& InKey, FPreAnimatedParameterKey& ParameterBufferKey, const FRestoreStateParams& Params)
{
	UControlRig* Rig = Cast<UControlRig>(InKey.Get<0>().ResolveObjectPtr());
	if (Rig)
	{
		if (ParameterBufferKey.bPersistent)
		{
			ParameterBufferKey.Buffer->Persistent.ApplyAndRemove(Rig, InKey.Get<1>());
		}
		else
		{
			ParameterBufferKey.Buffer->Transient.ApplyAndRemove(Rig, InKey.Get<1>());
		}
	}
}

FPreAnimatedControlRigState FPreAnimatedControlRigTraits::CachePreAnimatedValue(FObjectKey InObject)
{
	UControlRig* ControlRig = Cast<UControlRig>(InObject.ResolveObjectPtr());
	check(ControlRig);

	FPreAnimatedControlRigState State;
	USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject());
	if (SkeletalMeshComponent)
	{
		State.SetSkelMesh(SkeletalMeshComponent);
	}
	return State;
}

void FPreAnimatedControlRigTraits::RestorePreAnimatedValue(const FObjectKey& InObject, FPreAnimatedControlRigState& State, const FRestoreStateParams& Params)
{
	if (UControlRig* ControlRig = Cast<UControlRig>(InObject.ResolveObjectPtr()))
	{
		// When the anim mixer is the pose producer it owns the anim instance and skel-mesh restore,
		// so skip the evaluate / sequencer-instance unbind / skel-mesh tick path.
		if (!State.bAnimMixerPoseProducer)
		{
			ControlRig->Evaluate_AnyThread();

			//unbind instances and reset animbp
			FControlRigBindingHelper::UnBindFromSequencerInstance(ControlRig);

			//do a tick and restore skel mesh
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
			{
				// If the skel mesh comp owner has been removed from the world, no need to restore anything
				if (SkeletalMeshComponent->IsRegistered())
				{
					// Restore pose after unbinding to force the restored pose
					SkeletalMeshComponent->SetUpdateAnimationInEditor(true);
					SkeletalMeshComponent->SetUpdateClothInEditor(true);
					if (!SkeletalMeshComponent->IsPostEvaluatingAnimation())
					{
						SkeletalMeshComponent->TickAnimation(0.f, false);
						SkeletalMeshComponent->RefreshBoneTransforms();
						SkeletalMeshComponent->RefreshFollowerComponents();
						SkeletalMeshComponent->UpdateComponentToWorld();
						SkeletalMeshComponent->FinalizeBoneTransform();
						SkeletalMeshComponent->MarkRenderTransformDirty();
						SkeletalMeshComponent->MarkRenderDynamicDataDirty();
					}
					State.SkeletalMeshRestoreState.RestoreState();

					if (SkeletalMeshComponent->GetAnimationMode() != State.AnimationMode)
					{
						SkeletalMeshComponent->SetAnimationMode(State.AnimationMode);
					}
				}
			}
		}

		//only unbind if not a component
		if (Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()) == nullptr)
		{
			ControlRig->GetObjectBinding()->UnbindFromObject();
		}
	}
}



struct FControlRigDataGroupingPolicy
{
	using GroupKeyType = TTuple<FObjectKey, FName, EControlRigControlType>;

	void InitializeGroupKeys(
		TEntityGroupingHandlerBase<FControlRigDataGroupingPolicy>& Handler,
		FEntityGroupBuilder* Builder,
		FEntityAllocationIteratorItem Item,
		FReadEntityIDs EntityIDs,
		TWrite<FEntityGroupID> GroupIDs,
		TRead<FControlRigSourceData> ControlRigSources,
		TRead<FName> ParameterNames)
	{
		FMovieSceneTracksComponentTypes* TracksComponents     = FMovieSceneTracksComponentTypes::Get();
		FControlRigComponentTypes*       ControlRigComponents = FControlRigComponentTypes::Get();

		const FEntityAllocation* Allocation     = Item.GetAllocation();
		const FComponentMask&    AllocationType = Item.GetAllocationType();
		const int32              Num            = Allocation->Num();

		EControlRigControlType ControlType;
		if (AllocationType.Contains(TracksComponents->Parameters.Bool.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Bool;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Byte.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Enum;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Integer.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Integer;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Scalar.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Scalar;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Vector3.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Vector;
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Transform.PropertyTag))
		{
			ControlType = EControlRigControlType::Parameter_Transform;
		}
		else if (AllocationType.Contains(ControlRigComponents->Tags.Space))
		{
			ControlType = EControlRigControlType::Space;
		}
		else
		{
			check(false);
		}

		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (GroupIDs[Index].IsValid())
			{
				continue;
			}
			GroupKeyType Key = MakeTuple(ControlRigSources[Index].Track, ParameterNames[Index], ControlType);

			const int32    NewGroupIndex = Handler.GetOrAllocateGroupIndex(Key, Builder);
			FEntityGroupID NewGroupID = Builder->MakeGroupID(NewGroupIndex);

			Builder->AddEntityToGroup(EntityIDs[Index], NewGroupID);
			// Write out the group ID component
			GroupIDs[Index] = NewGroupID;
		}
	}

#if WITH_EDITOR
	bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		return false;
	}
#endif
};

struct FOverlappingControlRigParameterHandler
{
	UMovieSceneEntitySystemLinker* Linker;
	UMovieSceneControlRigParameterEvaluatorSystem* System;

	FOverlappingControlRigParameterHandler(UMovieSceneControlRigParameterEvaluatorSystem* InSystem)
		: Linker(InSystem->GetLinker())
		, System(InSystem)
	{}

	void InitializeOutput(FEntityGroupID GroupID, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedControlRigParameterInfo* Output, FEntityOutputAggregate Aggregate)
	{
		UpdateOutput(GroupID, Inputs, Output, Aggregate);
	}

	void UpdateOutput(FEntityGroupID GroupID, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedControlRigParameterInfo* Output, FEntityOutputAggregate Aggregate)
	{
		using namespace UE::MovieScene;

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
		FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();

		const int32 NumContributors = Inputs.Num();
		if (!ensure(NumContributors != 0))
		{
			return;
		}

		bool bUseBlending = NumContributors > 1 ||
			!Linker->EntityManager.HasComponent(Inputs[0], BuiltInComponents->Tags.AbsoluteBlend) ||
			 Linker->EntityManager.HasComponent(Inputs[0], BuiltInComponents->WeightAndEasingResult);

		if (!bUseBlending)
		{
			// Check if the parameter is fully keyed
			const FComponentMask& EntityType = Linker->EntityManager.GetEntityType(Inputs[0]);
			TArrayView<const FPropertyDefinition> Properties = BuiltInComponents->PropertyRegistry.GetProperties();

			const int32 PropertyDefinitionIndex = Algo::IndexOfByPredicate(Properties, [=](const FPropertyDefinition& InDefinition){ return EntityType.Contains(InDefinition.PropertyType); });
			if (PropertyDefinitionIndex != INDEX_NONE)
			{
				const FPropertyDefinition& Property = Properties[PropertyDefinitionIndex];

				TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(Property);
				for (int32 CompositeIndex = 0; CompositeIndex < Property.CompositeSize; ++CompositeIndex)
				{
					if (!EntityType.Contains(Composites[CompositeIndex].ComponentTypeID))
					{
						bUseBlending = true;
						break;
					}
				}
			}
		}

		if (bUseBlending || Output->OutputEntityID)
		{
			if (!Output->OutputEntityID)
			{
				if (!System->DoubleBlenderSystem)
				{
					System->DoubleBlenderSystem = Linker->LinkSystem<UMovieScenePiecewiseDoubleBlenderSystem>();
					Linker->SystemGraph.AddReference(System, System->DoubleBlenderSystem);
				}

				Output->WeakLinker = Linker;
				Output->WeakBlenderSystem = System->DoubleBlenderSystem;

				// Initialize the blend channel ID
				Output->BlendChannelID = System->DoubleBlenderSystem->AllocateBlendChannel();
			}

			const FComponentTypeID BlenderTypeTag = System->DoubleBlenderSystem->GetBlenderTypeTag();
			InitializeBlendOutput(BlenderTypeTag, Inputs, Output);

			for (FMovieSceneEntityID Input : Inputs)
			{
				if (!Linker->EntityManager.HasComponent(Input, BuiltInComponents->BlendChannelInput))
				{
					Linker->EntityManager.AddComponent(Input, BuiltInComponents->BlendChannelInput, Output->BlendChannelID);
				}
				else
				{
					// If the bound material changed, we might have been re-assigned a different blend channel so make sure it's up to date
					Linker->EntityManager.WriteComponentChecked(Input, BuiltInComponents->BlendChannelInput, Output->BlendChannelID);
				}


				// Ensure we have the blender type tag on the inputs.
				Linker->EntityManager.AddComponent(Input, BlenderTypeTag);
			}
		}
		else if (!Output->OutputEntityID && Inputs.Num() == 1)
		{
			Linker->EntityManager.RemoveComponent(Inputs[0], BuiltInComponents->BlendChannelInput);
		}

		Output->NumContributors = NumContributors;
	}

	void DestroyOutput(FEntityGroupID GroupID, FAnimatedControlRigParameterInfo* Output, FEntityOutputAggregate Aggregate)
	{
		if (Output->OutputEntityID)
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			Linker->EntityManager.AddComponent(Output->OutputEntityID, BuiltInComponents->Tags.NeedsUnlink);
			Output->OutputEntityID = FMovieSceneEntityID();

			if (System->DoubleBlenderSystem)
			{
				System->DoubleBlenderSystem->ReleaseBlendChannel(Output->BlendChannelID);
			}
			Output->BlendChannelID = FMovieSceneBlendChannelID();
		}
	}

	void InitializeBlendOutput(FComponentTypeID BlenderTypeTag, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedControlRigParameterInfo* Output)
	{
		FBuiltInComponentTypes*          BuiltInComponents    = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents     = FMovieSceneTracksComponentTypes::Get();
		FControlRigComponentTypes*       ControlRigComponents = FControlRigComponentTypes::Get();

		FComponentMask ChannelResults = {
			BuiltInComponents->DoubleResult[0],
			BuiltInComponents->DoubleResult[1],
			BuiltInComponents->DoubleResult[2],
			BuiltInComponents->DoubleResult[3],
			BuiltInComponents->DoubleResult[4],
			BuiltInComponents->DoubleResult[5],
			BuiltInComponents->DoubleResult[6],
			BuiltInComponents->DoubleResult[7],
			BuiltInComponents->DoubleResult[8]
		};

		FTypelessMutation Mutation;
		for (FMovieSceneEntityID Input : Inputs)
		{
			FComponentMask Type = Linker->EntityManager.GetEntityType(Input);
			Type.CombineWithBitwiseAND(ChannelResults, EBitwiseOperatorFlags::MinSize);
			Mutation.AddMask.CombineWithBitwiseOR(Type, EBitwiseOperatorFlags::MaxSize);
		}
		
		// Remove any channels not present in any of the inputs by taking an XOR of all the channels with the AddMask
		Mutation.RemoveMask = MoveTemp(ChannelResults);
		Mutation.RemoveMask.CombineWithBitwiseXOR(Mutation.AddMask, EBitwiseOperatorFlags::MaintainSize);

		auto Builder = FEntityBuilder()
			.Add(BuiltInComponents->BlendChannelOutput, Output->BlendChannelID)
			.AddTag(ControlRigComponents->Tags.ControlRigParameter)
			.AddTag(BlenderTypeTag)
			.AddTag(BuiltInComponents->Tags.NeedsLink)
			.AddMutualComponents();

		if (Output->OutputEntityID)
		{
			Builder.MutateExisting(&Linker->EntityManager, Output->OutputEntityID, Mutation);
		}
		else
		{
			Output->OutputEntityID = Builder.CreateEntity(&Linker->EntityManager, Mutation.AddMask);
			Linker->EntityManager.CopyComponents(Inputs[0], Output->OutputEntityID, Linker->EntityManager.GetComponents()->GetCopyAndMigrationMask());
		}
	}

};

struct FEvaluateBaseControlRigs
{
	FInstanceRegistry* InstanceRegistry;

	FEvaluateBaseControlRigs(FInstanceRegistry* InInstanceRegistry)
		: InstanceRegistry(InInstanceRegistry)
	{
	}

	void ForEachEntity(
		FRootInstanceHandle RootInstanceHandle,
		const FMovieSceneSequenceID* OptSequenceID,
		FFrameTime EvalTime,
		const double* OptWeightAndEasing,
		const FControlRigSourceData& ControlRigSource,
		FBaseControlRigEvalData& InOutBaseEvalData) const
	{
		UControlRig*   ControlRig   = InOutBaseEvalData.WeakControlRig.Get();
		URigHierarchy* RigHierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;

		if (ControlRig == nullptr || RigHierarchy == nullptr)
		{
			return;
		}

		InOutBaseEvalData.bWasDoNotKey = InOutBaseEvalData.Section->GetDoNotKey();
		InOutBaseEvalData.Section->SetDoNotKey(false);

		InOutBaseEvalData.WarpedTime = InOutBaseEvalData.Section->RemapTime(EvalTime);
		const FFrameRate TickResolution = InOutBaseEvalData.Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		float EvalTimeSeconds = TickResolution.AsSeconds(InOutBaseEvalData.WarpedTime);
		UObject* BoundObject = ControlRig->GetObjectBinding()->GetBoundObject();
		if (!BoundObject)
		{
			return;
		}

		if (InOutBaseEvalData.bIsActive)
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
			{
				UControlRigLayerInstance* AnimInstance = Cast<UControlRigLayerInstance>(SkeletalMeshComponent->GetAnimInstance());
				if (!AnimInstance && !InOutBaseEvalData.bAnimMixerPoseProducer)
				{
					// The ControlRigLayerInstance may have been destroyed by pre-animated state
					// restoration during PostInstantiation (e.g. when skeletal animation entities
					// are unlinked after a BakeToControlRig operation). Re-bind to recreate it.
					FControlRigBindingHelper::BindToSequencerInstance(ControlRig);
					AnimInstance = Cast<UControlRigLayerInstance>(SkeletalMeshComponent->GetAnimInstance());
				}
				if (AnimInstance)
				{
					float Weight = InOutBaseEvalData.Section->EvaluateEasing(InOutBaseEvalData.WarpedTime);
					if (EnumHasAllFlags(InOutBaseEvalData.Section->TransformMask.GetChannels(), EMovieSceneTransformChannel::Weight))
					{
						float ManualWeight = 1.f;
						InOutBaseEvalData.Section->Weight.Evaluate(InOutBaseEvalData.WarpedTime, ManualWeight);
						Weight *= ManualWeight;
					}

					FControlRigIOSettings InputSettings;
					InputSettings.bUpdateCurves = true;
					InputSettings.bUpdatePose = true;

					// Only update for Absolute sections - additive sections handle weight through control value blending
					if (InOutBaseEvalData.Section->GetBlendType() == EMovieSceneBlendType::Absolute)
					{
						AnimInstance->UpdateControlRigTrack(ControlRig->GetUniqueID(), Weight, InputSettings, true);
					}
				}
			}

			const bool bSetupUndo = false;
			ControlRig->SetAbsoluteTime(EvalTimeSeconds);
		}

		// when playing animation, instead of scrubbing/stepping thru frames, InTime might have a subframe of 0.999928
		// leading to a decimal value of 24399.999928 (for example). This results in evaluating one frame less than expected
		// (24399 instead of 24400) and leads to spaces and constraints switching parents/state after the control changes
		// its transform. (float/double channels will interpolate to a value pretty close to the one at 24400 as its based
		// on that 0.999928 subframe value.
		const FFrameTime RoundTime = InOutBaseEvalData.WarpedTime.RoundToFrame();

		TSharedRef<FSharedPlaybackState> SharedPlaybackState = InstanceRegistry->GetInstance(RootInstanceHandle).GetSharedPlaybackState();

		FMovieSceneSequenceID SequenceID = OptSequenceID ? *OptSequenceID : MovieSceneSequenceID::Root;

		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(BoundObject));
		auto MarkRigForEvaluationIfAdditive = [ControlRig, SkeletalMeshComponent]()
		{
			if (TransformableHandleUtils::SkipTicking())
			{
				if (SkeletalMeshComponent && ControlRig && ControlRig->IsAdditive())
				{
					TransformableHandleUtils::MarkComponentForEvaluation(SkeletalMeshComponent);
				}
			}
		};
		
		for (const FConstraintAndActiveChannel& ConstraintAndActiveChannel : InOutBaseEvalData.Section->GetConstraintsChannels())
		{
			bool Value = false;
			ConstraintAndActiveChannel.ActiveChannel.Evaluate(RoundTime, Value);
			CreateConstraintIfNeeded(BoundObject->GetWorld(), ConstraintAndActiveChannel.GetConstraint().Get(), InOutBaseEvalData.Section);

			if (TObjectPtr<UTickableConstraint> Constraint = ConstraintAndActiveChannel.GetConstraint())
			{
				//For Control Rig we may need to explicitly set the control rig
				UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint);
				if (TransformConstraint)
				{
					TransformConstraint->InitConstraint(BoundObject->GetWorld());
				}
				Constraint->ResolveBoundObjects(SequenceID, SharedPlaybackState, ControlRig);
				Constraint->SetActive(Value);

				if (TransformConstraint && Value && TransformableHandleUtils::SkipTicking())
				{
					MarkRigForEvaluationIfAdditive();
					
					if (UTransformableComponentHandle* ParentHandle = Cast<UTransformableComponentHandle>(TransformConstraint->ParentTRSHandle))
					{
						constexpr bool bRecursive = true;
						USceneComponent* ParentComponent = TransformableHandleUtils::GetTarget<USceneComponent>(TransformConstraint->ParentTRSHandle);
						TransformableHandleUtils::MarkComponentForEvaluation(ParentComponent, bRecursive);
					}
				}
			}
		}

		// for Constraints with ControlRig we need to resolve all Parents also
		// Don't need to do children since they wil be handled by the channel resolve above
		ResolveParentHandles(BoundObject, ControlRig, InOutBaseEvalData, SequenceID, SharedPlaybackState);
	}

	static void ResolveParentHandles(
		const UObject* InBoundObject, UControlRig* InControlRigInstance,
		const FBaseControlRigEvalData& InBaseEvalData,
		const FMovieSceneSequenceID& InSequenceID,
		const TSharedRef<FSharedPlaybackState>& InSharedPlaybackState)
	{
		if (!InBoundObject)
		{
			return;
		}

		UWorld* BoundObjectWorld = InBoundObject->GetWorld();
		const bool bIsGameWorld = InBoundObject->GetWorld() ? InBoundObject->GetWorld()->IsGameWorld() : false;
		
		UMovieSceneControlRigParameterTrack* ControlRigTrack = InBaseEvalData.Section->GetTypedOuter<UMovieSceneControlRigParameterTrack>();

		// is this control rig a game world instance of this section's rig?
		auto WasAGameInstance = [ControlRigTrack](const UControlRig* InRigToTest)
		{
			return ControlRigTrack ? ControlRigTrack->IsAGameInstance(InRigToTest) : false;
		};

		// is the parent handle of this constraint related to this section?
		// this return true if the handle's control rig has been spawned by the ControlRigTrack (whether in Editor or Game)
		// if false, it means that the handle represents another control on another control rig so we don't need to resolve it here
		// note that it returns true if ControlRigTrack is null (is this possible?!) or if the ControlRig is null (we can't infer anything from this)
		auto ShouldResolveParent = [ControlRigTrack](const UTransformableControlHandle* ParentControlHandle)
		{
			if (!ParentControlHandle)
			{
				return false;
			}

			if (!ControlRigTrack)
			{
				// cf. UObjectBaseUtility::IsInOuter
				return true;
			}
			
			return ParentControlHandle->ControlRig ? ParentControlHandle->ControlRig->IsInOuter(ControlRigTrack) : true;
		};

		// this is the default's section rig. when bIsGameWorld is false, InControlRigInstance should be equal to SectionRig
		const UControlRig* SectionRig = InBaseEvalData.Section->GetControlRig();
		const bool bIsRigInstanceSectionRig = InControlRigInstance == SectionRig;

		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(BoundObjectWorld);
		const TArray< TWeakObjectPtr<UTickableConstraint>> Constraints = Controller.GetAllConstraints();

		for (const TWeakObjectPtr<UTickableConstraint>& TickConstraint : Constraints)
		{
			UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(TickConstraint.Get());
			UTransformableControlHandle* ParentControlHandle = TransformConstraint ? Cast<UTransformableControlHandle>(TransformConstraint->ParentTRSHandle) : nullptr;
			if (ParentControlHandle && ShouldResolveParent(ParentControlHandle))
			{
				if (bIsGameWorld)
				{
					// switch from section's rig to the game instance
					if (ParentControlHandle->ControlRig == SectionRig)
					{
						FConstraintDependencyScope Scope(TransformConstraint);
						ParentControlHandle->ResolveBoundObjects(InSequenceID, InSharedPlaybackState, InControlRigInstance);
						TransformConstraint->EnsurePrimaryDependency(BoundObjectWorld);
					}
				}
				else
				{
					// resolve to the incoming rig instance if it's the default rig for the section or if the previous one was a game instance
					if (WasAGameInstance(ParentControlHandle->ControlRig.Get()) || bIsRigInstanceSectionRig)
					{
						FConstraintDependencyScope Scope(TransformConstraint);
						ParentControlHandle->ResolveBoundObjects(InSequenceID, InSharedPlaybackState, InControlRigInstance);
						TransformConstraint->EnsurePrimaryDependency(BoundObjectWorld);
					}
				}
			}
		}
	}
};


struct FEvaluateControlRigSpaceChannels
{
	static void ForEachEntity(const FMovieSceneControlRigSpaceChannel* Channel,
		const FControlRigSourceData& ControlRigSource, FFrameTime EvalTime, FMovieSceneControlRigSpaceBaseKey& OutValue)
	{
		UMovieSceneControlRigParameterSection* Section = ControlRigSource.Section.Get();
		if (!Section)
		{
			return;
		}

		// Apply the section's time remapping (play rate), matching what FEvaluateBaseControlRigs does.
		const FFrameTime WarpedTime = Section->RemapTime(EvalTime);

		// when playing animation, instead of scrubbing/stepping thru frames, InTime might have a subframe of 0.999928
		// leading to a decimal value of 24399.999928 (for example). This results in evaluating one frame less than expected
		// (24399 instead of 24400) and leads to spaces and constraints switching parents/state after the control changes
		// its transform. (float/double channels will interpolate to a value pretty close to the one at 24400 as its based
		// on that 0.999928 subframe value.
		Channel->Evaluate(WarpedTime.RoundToFrame(), OutValue);
	}
};



struct FGatherControlRigParameterValues
{
	FAccumulatedControlRigValues* AccumulatedValues;
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	void ForEachAllocation(FEntityAllocationIteratorItem Item, const FAccumulatedControlEntryIndex* EntryIndices, const FName* ParameterNames, 
		const FMovieSceneControlRigSpaceBaseKey* OptSpaceResults, const bool* OptBoolResults, const uint8* OptByteResults, const int64* OptIntegerResults,
		const double* OptDoubleResults0, const double* OptDoubleResults1, const double* OptDoubleResults2,
		const double* OptDoubleResults3, const double* OptDoubleResults4, const double* OptDoubleResults5,
		const double* OptDoubleResults6, const double* OptDoubleResults7, const double* OptDoubleResults8
		) const
	{
		const FComponentMask&    AllocationType = Item.GetAllocationType();
		const FEntityAllocation* Allocation     = Item.GetAllocation();

		const int32 Num = Allocation->Num();

		if (AllocationType.Contains(ControlRigComponents->Tags.Space))
		{
			check(OptSpaceResults);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				AccumulatedValues->Store(EntryIndices[Index], OptSpaceResults[Index]);
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Bool.PropertyTag))
		{
			check(OptBoolResults);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				AccumulatedValues->Store(EntryIndices[Index], OptBoolResults[Index]);
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Byte.PropertyTag))
		{
			check(OptByteResults);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				AccumulatedValues->Store(EntryIndices[Index], OptByteResults[Index]);
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Integer.PropertyTag))
		{
			check(OptIntegerResults);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				AccumulatedValues->Store(EntryIndices[Index], static_cast<int32>(OptIntegerResults[Index]));
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Scalar.PropertyTag))
		{
			check(OptDoubleResults0);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				AccumulatedValues->Store(EntryIndices[Index], static_cast<float>(OptDoubleResults0[Index]));
			}
		}
		// Vector2 is applied as vector3 in Control Rig
		else if (AllocationType.Contains(TracksComponents->Parameters.Vector3.PropertyTag))
		{
			const bool bFullyAnimated = OptDoubleResults0 && OptDoubleResults1 && OptDoubleResults2;

			if (bFullyAnimated)
			{
				for (int32 Index = 0; Index < Num; ++Index)
				{
					FVector3f Result(
						OptDoubleResults0[Index],
						OptDoubleResults1[Index],
						OptDoubleResults2[Index]
					);
					AccumulatedValues->Store(EntryIndices[Index], Result);
				}
			}
			else
			{
				TOptionalComponentReader<FFloatIntermediateVector> OptInitialValues =
					Allocation->TryReadComponents(TracksComponents->Parameters.Vector3.InitialValue);

				for (int32 Index = 0; Index < Num; ++Index)
				{
					FVector3f Vector = OptInitialValues
						? OptInitialValues[Index].AsVector3f()
						: FVector3f(0.f, 0.f, 0.f);

					if (OptDoubleResults0)
					{
						Vector.X = OptDoubleResults0[Index];
					}
					if (OptDoubleResults1)
					{
						Vector.Y = OptDoubleResults1[Index];
					}
					if (OptDoubleResults2)
					{
						Vector.Z = OptDoubleResults2[Index];
					}
					AccumulatedValues->Store(EntryIndices[Index], Vector);
				}
			}
		}
		else if (AllocationType.Contains(TracksComponents->Parameters.Transform.PropertyTag))
		{
			const bool bFullyAnimated =
				OptDoubleResults0 && OptDoubleResults1 && OptDoubleResults2 &&
				OptDoubleResults3 && OptDoubleResults4 && OptDoubleResults5 &&
				OptDoubleResults6 && OptDoubleResults7 && OptDoubleResults8;

			if (bFullyAnimated)
			{
				for (int32 Index = 0; Index < Num; ++Index)
				{
					FEulerTransform Result(
						FVector( OptDoubleResults0[Index], OptDoubleResults1[Index], OptDoubleResults2[Index]),
						FRotator(OptDoubleResults4[Index], OptDoubleResults5[Index], OptDoubleResults3[Index]),
						FVector( OptDoubleResults6[Index], OptDoubleResults7[Index], OptDoubleResults8[Index])
					);
					AccumulatedValues->Store(EntryIndices[Index], Result);
				}
			}
			else
			{
				TOptionalComponentReader<FIntermediate3DTransform> OptInitialValues =
					Allocation->TryReadComponents(TracksComponents->Parameters.Transform.InitialValue);

				for (int32 Index = 0; Index < Num; ++Index)
				{
					FEulerTransform Transform = OptInitialValues
						? OptInitialValues[Index].AsEuler()
						: FEulerTransform::Identity;

					if (OptDoubleResults0) { Transform.Location.X     = OptDoubleResults0[Index]; }
					if (OptDoubleResults1) { Transform.Location.Y     = OptDoubleResults1[Index]; }
					if (OptDoubleResults2) { Transform.Location.Z     = OptDoubleResults2[Index]; }

					if (OptDoubleResults3) { Transform.Rotation.Roll  = OptDoubleResults3[Index]; }
					if (OptDoubleResults4) { Transform.Rotation.Pitch = OptDoubleResults4[Index]; }
					if (OptDoubleResults5) { Transform.Rotation.Yaw   = OptDoubleResults5[Index]; }

					if (OptDoubleResults6) { Transform.Scale.X        = OptDoubleResults6[Index]; }
					if (OptDoubleResults7) { Transform.Scale.Y        = OptDoubleResults7[Index]; }
					if (OptDoubleResults8) { Transform.Scale.Z        = OptDoubleResults8[Index]; }

					AccumulatedValues->Store(EntryIndices[Index], Transform);
				}
			}
		}
	}
};


struct FApplyControlRigParameterValuesTask
{
	const FAccumulatedControlRigValues* AccumulatedValues;
	bool bApplyRigs = true;

	FApplyControlRigParameterValuesTask(const FAccumulatedControlRigValues* InAccumulatedValues, bool bInApplyRigs)
		: AccumulatedValues(InAccumulatedValues)
		, bApplyRigs(bInApplyRigs)
	{
	}

	void Run(FEntityAllocationWriteContext WriteContext) const
	{
		if (bApplyRigs)
		{
			AccumulatedValues->Apply();
		}
	}
};


struct FResetDoNotKey
{
	void ForEachEntity(const FControlRigSourceData& ControlRigSource, const FBaseControlRigEvalData& InOutBaseEvalData) const
	{
		InOutBaseEvalData.Section->SetDoNotKey(InOutBaseEvalData.bWasDoNotKey);
	}
};

} // namespace UE::MovieScene


UMovieSceneControlRigParameterEvaluatorSystem::UMovieSceneControlRigParameterEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		TSharedRef<FControlRigInitialValueProcessor> InitialValueProcessor =
			MakeShared<FControlRigInitialValueProcessor>();

		UMovieSceneInitialValueSystem::RegisterProcessor(TracksComponents->Parameters.Scalar.InitialValue, InitialValueProcessor);
		UMovieSceneInitialValueSystem::RegisterProcessor(TracksComponents->Parameters.Vector2.InitialValue, InitialValueProcessor);
		UMovieSceneInitialValueSystem::RegisterProcessor(TracksComponents->Parameters.Vector3.InitialValue, InitialValueProcessor);
		UMovieSceneInitialValueSystem::RegisterProcessor(TracksComponents->Parameters.Color.InitialValue, InitialValueProcessor);
		UMovieSceneInitialValueSystem::RegisterProcessor(TracksComponents->Parameters.Transform.InitialValue, InitialValueProcessor);


		DefineComponentConsumer(GetClass(), ControlRigComponents->ControlRigSource);
		DefineComponentConsumer(GetClass(), BuiltInComponents->HierarchicalBlendTarget);
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);

		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());


		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[Index]);
		}
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneHierarchicalEasingInstantiatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneInitialValueSystem::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());

		DefineImplicitPrerequisite(UMovieSceneSkeletalAnimationSystem::StaticClass(), GetClass());
	}
}

bool UMovieSceneControlRigParameterEvaluatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	const bool bCanBeEnabled = true;// !UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate();
	return bCanBeEnabled && InLinker->EntityManager.ContainsComponent(FControlRigComponentTypes::Get()->ControlRigSource);
}

void UMovieSceneControlRigParameterEvaluatorSystem::OnLink()
{
	using namespace UE::MovieScene;

	ControlRigParameterTracker.Initialize(this);

	ControlRigStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedControlRigStorage>();
	ControlRigParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedControlRigParameterStorage>();

	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();

	ParameterGroupingKey = GroupingSystem->AddGrouping(
		FControlRigDataGroupingPolicy(),
		ControlRigComponents->ControlRigSource,
		TracksComponents->GenericParameterName
	);
}

UControlRig* UMovieSceneControlRigParameterEvaluatorSystem::GetRigFromTrack(UMovieSceneControlRigParameterTrack* Track) const
{
	return AccumulatedValues.FindControlRig(Track);
}

const UE::MovieScene::FControlRigParameterBuffer* UMovieSceneControlRigParameterEvaluatorSystem::FindParameters(UMovieSceneControlRigParameterTrack* Track) const
{
	return AccumulatedValues.FindParameterBuffer(Track);
}

void UMovieSceneControlRigParameterEvaluatorSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	// Always reset the float blender system on link to ensure that recycled systems are correctly initialized.
	DoubleBlenderSystem = nullptr;

	ControlRigParameterTracker.Destroy(FOverlappingControlRigParameterHandler(this));

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(ParameterGroupingKey);
	}
	ParameterGroupingKey = FEntityGroupingPolicyKey();
}

void UMovieSceneControlRigParameterEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();
	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		OnInstantiation();
	}
	else if (CurrentPhase == ESystemPhase::Evaluation)
	{
		OnEvaluation(InPrerequisites, Subsequents);
	}
}

void UMovieSceneControlRigParameterEvaluatorSystem::OnInstantiation()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();


	FEntityComponentFilter ChangedFilter;
	ChangedFilter.Any({ ControlRigComponents->BaseControlRigEvalData, ControlRigComponents->Tags.ControlRigParameter });
	ChangedFilter.Any({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink });
	if (!Linker->EntityManager.Contains(ChangedFilter))
	{
		return;
	}

	// Prime the existing container so we can track what needs to be destroyed
	AccumulatedValues.PrimeForInstantiation();

	// Keep track of which base eval component is the 'active' one based on whether it is blended or not
	TMap<UControlRig*, FBaseControlRigEvalData*> EncounteredRigs;

	// --------------------------------------------------------------------------------------------------------
	// Initialize base control rig components.
	if (Linker->GetLinkerRole() == EEntitySystemLinkerRole::Interrogation)
	{
		IInterrogationExtension* Interrogation = Linker->FindExtension<IInterrogationExtension>();
		check(Interrogation);

		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->Interrogation.InputKey)
		.Read(ControlRigComponents->ControlRigSource)
		.Write(ControlRigComponents->BaseControlRigEvalData)
		.PassthroughFilter(FEntityComponentFilter().All({ BuiltInComponents->WeightAndEasingResult }))
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, [this, Interrogation, &EncounteredRigs](FMovieSceneEntityID EntityID, FInterrogationKey InterrogationKey, const FControlRigSourceData& ControlRigSource, FBaseControlRigEvalData& OutBaseData, bool bHasWeight)
			{
				UControlRig* Rig = Cast<UControlRig>(Interrogation->GetSparseChannelInfo().FindObject(InterrogationKey.Channel));

				OutBaseData.bIsActive = false;
				OutBaseData.bHasWeight = bHasWeight;
				OutBaseData.WeakControlRig = Rig;

				FBaseControlRigEvalData** ExistingData = EncounteredRigs.Find(Rig);
				if (ExistingData)
				{
					if ((*ExistingData)->bHasWeight)
					{
						return;
					}

					// Previous one wasn't weighted but this is, this should take over
					if (bHasWeight)
					{
						(*ExistingData)->bIsActive = false;
						OutBaseData.bIsActive = true;
						*ExistingData = &OutBaseData;

						// Don't reinitialize
						return;
					}
				}
				else
				{
					OutBaseData.bIsActive = true;
					EncounteredRigs.Add(Rig, &OutBaseData);
				}

				this->AccumulatedValues.InitializeRig(ControlRigSource.Track, Rig);
			}
		);
	}
	else
	{
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->GenericObjectBinding)
		.Read(BuiltInComponents->RootInstanceHandle)
		.ReadOptional(BuiltInComponents->SequenceID)
		.Read(ControlRigComponents->ControlRigSource)
		.Write(ControlRigComponents->BaseControlRigEvalData)
		.PassthroughFilter(FEntityComponentFilter().All({ BuiltInComponents->Tags.RestoreState }))
		.PassthroughFilter(FEntityComponentFilter().All({ BuiltInComponents->WeightAndEasingResult }))
		.PassthroughFilter(FEntityComponentFilter().All({ TracksComponents->Tags.AnimMixerPoseProducer}))
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, [this, &EncounteredRigs](FMovieSceneEntityID EntityID, const FGuid& ObjectBindingID, FRootInstanceHandle RootInstanceHandle, const FMovieSceneSequenceID* OptSequenceID,
				const FControlRigSourceData& ControlRigSource, FBaseControlRigEvalData& OutBaseData, bool bWantsRestoreState, bool bHasWeight, bool bAnimMixerPoseProducer)
			{

				OutBaseData.bIsActive = false;
				OutBaseData.bHasWeight = bHasWeight;
				OutBaseData.bAnimMixerPoseProducer = bAnimMixerPoseProducer;
				OutBaseData.WeakControlRig = nullptr;

				FMovieSceneSequenceID SequenceID = OptSequenceID ? *OptSequenceID : MovieSceneSequenceID::Root;
				TArrayView<TWeakObjectPtr<>> BoundObjects = Linker->GetInstanceRegistry()->GetInstance(RootInstanceHandle).GetSharedPlaybackState()->FindBoundObjects(ObjectBindingID, SequenceID);

				for (TWeakObjectPtr<> WeakBoundObject : BoundObjects)
				{
					UObject* BoundObject = WeakBoundObject.Get();
					if (BoundObject)
					{
						this->InitializeBaseRigComponent(
							BoundObject,
							EntityID,
							RootInstanceHandle,
							bWantsRestoreState,
							bHasWeight,
							bAnimMixerPoseProducer,
							ControlRigSource,
							OutBaseData,
							EncounteredRigs);
						return;
					}
				}
			}
		);
	}

	// Compact the accululation buffers if necessary
	AccumulatedValues.Compact();

	// --------------------------------------------------------------------------------------------------------
	// Check for parameter entities whose sections were reconstructed during rig binding
	// (e.g., procedural rigs adding controls). Repair stale channels if possible or mark stale entities for reimport.
	RepairOrInvalidateStaleParameterEntities();

	// --------------------------------------------------------------------------------------------------------
	// Process overlapping control rig parameter entities that animate the same parameter based on the group ID
	{
		FEntityTaskBuilder()
		.Read(BuiltInComponents->Group)
		.PassthroughFilter(FEntityComponentFilter().All({ BuiltInComponents->Tags.NeedsLink }))
		.FilterAny({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink })
		.FilterAny({ TracksComponents->Parameters.Transform.PropertyTag, TracksComponents->Parameters.Scalar.PropertyTag, TracksComponents->Parameters.Vector3.PropertyTag })
		.FilterAll({ ControlRigComponents->ControlRigSource, TracksComponents->GenericParameterName, ControlRigComponents->Tags.ControlRigParameter })
		.Iterate_PerAllocation(&Linker->EntityManager, [this](const FEntityAllocation* Allocation, TRead<FEntityGroupID> Group, bool bIsBeingLinked)
			{
				if (bIsBeingLinked)
				{
					this->ControlRigParameterTracker.VisitActiveAllocation(Allocation, Group);
				}
				else
				{
					this->ControlRigParameterTracker.VisitUnlinkedAllocation(Allocation);
				}
			}
		);

		FOverlappingControlRigParameterHandler Handler(this);
		ControlRigParameterTracker.ProcessInvalidatedOutputs(Linker, Handler);
	}

	// --------------------------------------------------------------------------------------------------------
	// Track pre-animated state for parameters
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(ControlRigComponents->ControlRigSource)
	.Read(TracksComponents->GenericParameterName)
	.PassthroughFilter(FEntityComponentFilter().All({ BuiltInComponents->Tags.RestoreState }))
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.Iterate_PerAllocation(&Linker->EntityManager, [this](FEntityAllocationIteratorItem Item,
		TRead<FMovieSceneEntityID> EntityIDs,
		TRead<FRootInstanceHandle> RootInstanceHandles,
		TRead<FControlRigSourceData> ControlRigSources,
		TRead<FName> ParameterNames,
		const bool bWantsRestore)
	{
		if (this->ControlRigParameterStorage->IsCapturingGlobalState() || bWantsRestore)
		{
			const int32 Num = Item.GetAllocation()->Num();

			for (int32 Index = 0; Index < Num; ++Index)
			{
				UControlRig* Rig = this->GetRigFromTrack(ControlRigSources[Index].Track);
				if (Rig)
				{
					this->ControlRigParameterStorage->BeginTrackingEntity(EntityIDs[Index], bWantsRestore, RootInstanceHandles[Index], Rig, ParameterNames[Index]);
				}
			}
		}
	});


	// --------------------------------------------------------------------------------------------------------
	// Regather all active parameters into our allocation buffers
	{
		FEntityComponentFilter MutationFilter;
		MutationFilter.All({
			TracksComponents->GenericParameterName,
			ControlRigComponents->Tags.ControlRigParameter,
		});
		
		MutationFilter.None({ BuiltInComponents->BlendChannelInput, BuiltInComponents->Tags.NeedsUnlink });

		// Now mutate it all to assign new allocation entries
		FControlRigAccumulationEntryIndexMutation AccumulationEntryMutation(&AccumulatedValues, ControlRigParameterStorage.Get());
		Linker->EntityManager.MutateAll(MutationFilter, AccumulationEntryMutation);

		FRemoveInvalidControlRigAccumulationComponents RemoveInvalidMutation(&AccumulatedValues);
		Linker->EntityManager.MutateConditional(MutationFilter, RemoveInvalidMutation);

		// Initialize the parameter buffers
		AccumulatedValues.InitializeParameters(*this->ControlRigParameterStorage.Get());
	}

	// --------------------------------------------------------------------------------------------------------
	// Initialize initial values
	{
		FEntityComponentFilter MutationFilter;
		MutationFilter.All({
			ControlRigComponents->ControlRigSource,
			ControlRigComponents->AccumulatedControlEntryIndex,
			TracksComponents->GenericParameterName,
			ControlRigComponents->Tags.ControlRigParameter,
			BuiltInComponents->Tags.NeedsLink,
		});
		MutationFilter.None({ BuiltInComponents->BlendChannelInput });

		FInitialControlRigParameterValueMutation InitialValueMutation(&AccumulatedValues);
		Linker->EntityManager.MutateAll(MutationFilter, InitialValueMutation);
	}
}


void UMovieSceneControlRigParameterEvaluatorSystem::RepairOrInvalidateStaleParameterEntities()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FControlRigComponentTypes* ControlRigComponents = FControlRigComponentTypes::Get();

	// Collect entities that need to be unlinked (can't add/remove components during iteration)
	TArray<FMovieSceneEntityID> EntitiesToUnlink;

	// Iterate all control rig parameter entities and check if their section has been
	// reconstructed since they were imported. This can happen when procedural rigs
	// add controls during binding (in InitializeBaseRigComponent), which causes
	// RecreateWithThisControlRig to be called, potentially invalidating channel pointers.
	// We repair stale entities by updating their channel pointers directly.
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Write(ControlRigComponents->ControlRigSource)
	.WriteOptional(ControlRigComponents->SpaceChannel)
	.WriteOptional(BuiltInComponents->BoolChannel)
	.WriteOptional(BuiltInComponents->ByteChannel)
	.WriteOptional(BuiltInComponents->IntegerChannel)
	.WriteOptional(BuiltInComponents->FloatChannel[0])
	.WriteOptional(BuiltInComponents->FloatChannel[1])
	.WriteOptional(BuiltInComponents->FloatChannel[2])
	.WriteOptional(BuiltInComponents->FloatChannel[3])
	.WriteOptional(BuiltInComponents->FloatChannel[4])
	.WriteOptional(BuiltInComponents->FloatChannel[5])
	.WriteOptional(BuiltInComponents->FloatChannel[6])
	.WriteOptional(BuiltInComponents->FloatChannel[7])
	.WriteOptional(BuiltInComponents->FloatChannel[8])
	.FilterAll({ ControlRigComponents->Tags.ControlRigParameter })
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->BlendChannelOutput })
	.Iterate_PerEntity(&Linker->EntityManager,
		[&EntitiesToUnlink](
			FMovieSceneEntityID EntityID,
			FControlRigSourceData& ControlRigSource,
			const FMovieSceneControlRigSpaceChannel** SpaceChannel,
			FSourceBoolChannel* BoolChannel,
			FSourceByteChannel* ByteChannel,
			FSourceIntegerChannel* IntegerChannel,
			FSourceFloatChannel* FloatChannel0,
			FSourceFloatChannel* FloatChannel1,
			FSourceFloatChannel* FloatChannel2,
			FSourceFloatChannel* FloatChannel3,
			FSourceFloatChannel* FloatChannel4,
			FSourceFloatChannel* FloatChannel5,
			FSourceFloatChannel* FloatChannel6,
			FSourceFloatChannel* FloatChannel7,
			FSourceFloatChannel* FloatChannel8)
		{
			UMovieSceneControlRigParameterSection* Section = ControlRigSource.Section.Get();
			if (!Section)
			{
				return;
			}

			uint32 CurrentHash = Section->GetLastControlsHash();
			if (CurrentHash == ControlRigSource.ImportedControlsHash)
			{
				// Not stale, nothing to do
				return;
			}

			// Section was reconstructed since this entity was imported - repair channel pointers
			int32 EntityIndex = 0;
			EControlRigEntityType EntityType;
			DecodeControlRigEntityID(ControlRigSource.ImportedEntityID, EntityIndex, EntityType);

			bool bFoundChannel = false;

			switch (EntityType)
			{
			case EControlRigEntityType::Space:
			{
				TArray<FSpaceControlNameAndChannel>& SpaceChannels = Section->GetSpaceChannels();
				if (SpaceChannel && EntityIndex < SpaceChannels.Num())
				{
					*SpaceChannel = &SpaceChannels[EntityIndex].SpaceCurve;
					ControlRigSource.ImportedControlsHash = CurrentHash;
					bFoundChannel = true;
				}
				break;
			}
			case EControlRigEntityType::BoolParameter:
			{
				TArray<FBoolParameterNameAndCurve>& BoolParams = Section->GetBoolParameterNamesAndCurves();
				if (BoolChannel && EntityIndex < BoolParams.Num())
				{
					BoolChannel->Source = &BoolParams[EntityIndex].ParameterCurve;
					ControlRigSource.ImportedControlsHash = CurrentHash;
					bFoundChannel = true;
				}
				break;
			}
			case EControlRigEntityType::EnumParameter:
			{
				TArray<FEnumParameterNameAndCurve>& EnumParams = Section->GetEnumParameterNamesAndCurves();
				if (ByteChannel && EntityIndex < EnumParams.Num())
				{
					ByteChannel->Source = &EnumParams[EntityIndex].ParameterCurve;
					ControlRigSource.ImportedControlsHash = CurrentHash;
					bFoundChannel = true;
				}
				break;
			}
			case EControlRigEntityType::IntegerParameter:
			{
				TArray<FIntegerParameterNameAndCurve>& IntParams = Section->GetIntegerParameterNamesAndCurves();
				if (IntegerChannel && EntityIndex < IntParams.Num())
				{
					IntegerChannel->Source = &IntParams[EntityIndex].ParameterCurve;
					ControlRigSource.ImportedControlsHash = CurrentHash;
					bFoundChannel = true;
				}
				break;
			}
			case EControlRigEntityType::ScalarParameter:
			{
				TArray<FScalarParameterNameAndCurve>& ScalarParams = Section->GetScalarParameterNamesAndCurves();
				if (FloatChannel0 && EntityIndex < ScalarParams.Num())
				{
					FloatChannel0->Source = &ScalarParams[EntityIndex].ParameterCurve;
					ControlRigSource.ImportedControlsHash = CurrentHash;
					bFoundChannel = true;
				}
				break;
			}
			case EControlRigEntityType::VectorParameter:
			{
				// Vector entities can be Vector2D or Vector3 based on index
				TArray<FVector2DParameterNameAndCurves>& Vector2DParams = Section->GetVector2DParameterNamesAndCurves();
				if (EntityIndex < Vector2DParams.Num())
				{
					FVector2DParameterNameAndCurves& Vec2D = Vector2DParams[EntityIndex];
					// Update whichever float channels exist on the entity.
					// Channels are conditionally added based on HasAnyData() at import time,
					// so a valid Vector2D entity may have only FloatChannel0 or FloatChannel1.
					if (FloatChannel0)
					{
						FloatChannel0->Source = &Vec2D.XCurve;
					}
					if (FloatChannel1)
					{
						FloatChannel1->Source = &Vec2D.YCurve;
					}
					if (FloatChannel0 || FloatChannel1)
					{
						ControlRigSource.ImportedControlsHash = CurrentHash;
						bFoundChannel = true;
					}
				}
				else
				{
					int32 VectorIndex = EntityIndex - Vector2DParams.Num();
					TArray<FVectorParameterNameAndCurves>& VectorParams = Section->GetVectorParameterNamesAndCurves();
					if (VectorIndex >= 0 && VectorIndex < VectorParams.Num())
					{
						if (FloatChannel0 && FloatChannel1 && FloatChannel2)
						{
							FVectorParameterNameAndCurves& Vec = VectorParams[VectorIndex];
							FloatChannel0->Source = &Vec.XCurve;
							FloatChannel1->Source = &Vec.YCurve;
							FloatChannel2->Source = &Vec.ZCurve;
							ControlRigSource.ImportedControlsHash = CurrentHash;
							bFoundChannel = true;
						}
					}
				}
				break;
			}
			case EControlRigEntityType::TransformParameter:
			{
				TArray<FTransformParameterNameAndCurves>& TransformParams = Section->GetTransformParameterNamesAndCurves();
				if (EntityIndex < TransformParams.Num())
				{
					if (FloatChannel0 && FloatChannel1 && FloatChannel2 &&
						FloatChannel3 && FloatChannel4 && FloatChannel5 &&
						FloatChannel6 && FloatChannel7 && FloatChannel8)
					{
						FTransformParameterNameAndCurves& Transform = TransformParams[EntityIndex];
						FloatChannel0->Source = &Transform.Translation[0];
						FloatChannel1->Source = &Transform.Translation[1];
						FloatChannel2->Source = &Transform.Translation[2];
						FloatChannel3->Source = &Transform.Rotation[0];
						FloatChannel4->Source = &Transform.Rotation[1];
						FloatChannel5->Source = &Transform.Rotation[2];
						FloatChannel6->Source = &Transform.Scale[0];
						FloatChannel7->Source = &Transform.Scale[1];
						FloatChannel8->Source = &Transform.Scale[2];
						ControlRigSource.ImportedControlsHash = CurrentHash;
						bFoundChannel = true;
					}
				}
				break;
			}
			default:
				break;
			}

			if (!bFoundChannel)
			{
				// Channel no longer exists - collect for unlinking outside iteration
				EntitiesToUnlink.Add(EntityID);
			}
		}
	);

	// Mark entities whose channels no longer exist for unlink
	for (FMovieSceneEntityID EntityID : EntitiesToUnlink)
	{
		if (Linker->EntityManager.HasComponent(EntityID, BuiltInComponents->Tags.NeedsLink))
		{
			Linker->EntityManager.RemoveComponent(EntityID, BuiltInComponents->Tags.NeedsLink);
		}
		Linker->EntityManager.AddComponent(EntityID, BuiltInComponents->Tags.NeedsUnlink);
	}
}

void UMovieSceneControlRigParameterEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents    = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents     = FMovieSceneTracksComponentTypes::Get();
	FControlRigComponentTypes*       ControlRigComponents = FControlRigComponentTypes::Get();

	// Evaluate base rigs before anything else
	FTaskID EvalBaseRigs = FEntityTaskBuilder()
	.Read(BuiltInComponents->RootInstanceHandle)
	.ReadOptional(BuiltInComponents->SequenceID)
	.Read(BuiltInComponents->EvalTime)
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.Read(ControlRigComponents->ControlRigSource)
	.Write(ControlRigComponents->BaseControlRigEvalData)
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Schedule_PerEntity<FEvaluateBaseControlRigs>(&Linker->EntityManager, TaskScheduler, Linker->GetInstanceRegistry());

	// Evaluate space channels. Space entities have SpaceChannel, ControlRigSource, and EvalTime
	// (EvalTime is added via mutual inclusion with SpaceChannel, see MovieSceneControlRigComponentTypes.cpp).
	// We read ControlRigSource to apply the section's time remapping (play rate) inside the task.
	FTaskID EvalSpaces = FEntityTaskBuilder()
	.Read(ControlRigComponents->SpaceChannel)
	.Read(ControlRigComponents->ControlRigSource)
	.Read(BuiltInComponents->EvalTime)
	.Write(ControlRigComponents->SpaceResult)
	.Fork_PerEntity<FEvaluateControlRigSpaceChannels>(&Linker->EntityManager, TaskScheduler);

	// Gather all (potentially blended) parameter values
	FTaskID GatherAnimatedControlRigs = FEntityTaskBuilder()
	.Read(ControlRigComponents->AccumulatedControlEntryIndex)
	.Read(TracksComponents->GenericParameterName)
	.ReadAnyOf(
		ControlRigComponents->SpaceResult, BuiltInComponents->BoolResult, BuiltInComponents->ByteResult, BuiltInComponents->IntegerResult,
		BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2],
		BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5],
		BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8])
	.FilterAll({ ControlRigComponents->Tags.ControlRigParameter })
	.FilterNone({ BuiltInComponents->BlendChannelInput, BuiltInComponents->Tags.Ignored })
	.Schedule_PerAllocation<FGatherControlRigParameterValues>(&Linker->EntityManager, TaskScheduler, &AccumulatedValues);

	FTaskID ApplyTask = TaskScheduler->AddTask<FApplyControlRigParameterValuesTask>(
		FTaskParams(TEXT("Apply Control Rig Parameters")).ForceGameThread(),
		&AccumulatedValues,
		Linker->GetLinkerRole() != EEntitySystemLinkerRole::Interrogation
	);

	// Reset Do Not Key states on any thread
	FTaskID ResetDoNotKey = FEntityTaskBuilder()
	.Read(ControlRigComponents->ControlRigSource)
	.Read(ControlRigComponents->BaseControlRigEvalData)
	.Fork_PerEntity<FResetDoNotKey>(&Linker->EntityManager, TaskScheduler);

	// Spaces must be evaluated before we gather results
	TaskScheduler->AddPrerequisite(EvalSpaces, GatherAnimatedControlRigs);
	// We must finish gathering parameter values before we apply them
	TaskScheduler->AddPrerequisite(GatherAnimatedControlRigs, ApplyTask);
	// Base control rigs must be evaluated before we apply parameters
	TaskScheduler->AddPrerequisite(EvalBaseRigs, ApplyTask);
	// Reset do not key states has to happen last (after parameters have been applied)
	TaskScheduler->AddPrerequisite(ApplyTask, ResetDoNotKey);
}

void UMovieSceneControlRigParameterEvaluatorSystem::OnEvaluation(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	// Not enabled for this codepath
	ensure(false);
}

void UMovieSceneControlRigParameterEvaluatorSystem::InitializeBaseRigComponent(
	UObject* BoundObject,
	UE::MovieScene::FMovieSceneEntityID EntityID,
	UE::MovieScene::FRootInstanceHandle RootInstanceHandle,
	bool bWantsRestoreState,
	bool bHasWeight,
	bool bAnimMixerPoseProducer,
	UE::MovieScene::FControlRigSourceData ControlRigSource,
	UE::MovieScene::FBaseControlRigEvalData& OutBaseData,
	TMap<UControlRig*, UE::MovieScene::FBaseControlRigEvalData*>& OutBaseComponentTracker)
{
	using namespace UE::MovieScene;

	UMovieSceneControlRigParameterTrack* Track = ControlRigSource.Track;
	if (!Track || !BoundObject)
	{
		return;
	}

	UWorld* GameWorld = (BoundObject->GetWorld() && BoundObject->GetWorld()->IsGameWorld()) ? BoundObject->GetWorld() : nullptr;

	UControlRig* ControlRig = Cast<UControlRig>(BoundObject);
	if (!ControlRig)
	{
		if (GameWorld)
		{
			ControlRig = Track->GetGameWorldControlRig(GameWorld, BoundObject);
		}
		else
		{
			ControlRig = Track->GetControlRig();
		}
	}

	if (!ControlRig)
	{
		return;
	}

	// Begin tracking this entity for the base pre-animated state that keeps the rig bound to our target object.
	// The anim-mixer case is differentiated inside RestorePreAnimatedValue via FPreAnimatedControlRigState::bAnimMixerPoseProducer.
	this->ControlRigStorage->BeginTrackingEntity(
		EntityID,
		bWantsRestoreState,
		RootInstanceHandle,
		ControlRig);

	if (!ControlRig->GetObjectBinding())
	{
		ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
	}

	if (ControlRig->GetObjectBinding()->GetBoundObject() != FControlRigObjectBinding::GetBindableObject(BoundObject))
	{ 
		ControlRig->GetObjectBinding()->BindToObject(BoundObject);

		TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
		ControlRig->Initialize();
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(FControlRigObjectBinding::GetBindableObject(BoundObject)))
		{
			ControlRig->RequestInit();
			ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkeletalMeshComponent, true);
			ControlRig->Evaluate_AnyThread();
		};
		if (GameWorld == nullptr && ControlRig->IsA<UFKControlRig>())// mz only in editor replace Fk Control rig, will look post 29.20 to see if this really needed but want to unblock folks
		{
			Track->ReplaceControlRig(ControlRig, true);
		}
		TArray<FName> NewSelectedControls = ControlRig->CurrentControlSelection();
		if (SelectedControls != NewSelectedControls)
		{
			ControlRig->ClearControlSelection();
			for (const FName& Name : SelectedControls)
			{
				ControlRig->SelectControl(Name, true);
			}
		}
	}

	// make sure to pick the correct CR instance for the  Components to bind.
	// In case of PIE + Spawnable Actor + CR component, sequencer should grab
	// CR component's CR instance for evaluation, see comment in BindToSequencerInstance
	// i.e. CR component should bind to the instance that it owns itself.
	ControlRig = GetControlRig(Track, BoundObject);
	if (!ControlRig)
	{
		return;
	}

	OutBaseData.WeakControlRig = ControlRig;

	// Cache pre-animated value now that the control rig is bound to the correct object.
	// Stamp bAnimMixerPoseProducer onto the state so the restore path can branch the same way the init path did.
	this->ControlRigStorage->CachePreAnimatedValue(
		FCachePreAnimatedValueParams(), ControlRig,
		[bAnimMixerPoseProducer](const FObjectKey& InObject)
		{
			FPreAnimatedControlRigState State = FPreAnimatedControlRigTraits::CachePreAnimatedValue(InObject);
			State.bAnimMixerPoseProducer = bAnimMixerPoseProducer;
			return State;
		});

	FBaseControlRigEvalData** ExistingData = OutBaseComponentTracker.Find(ControlRig);
	if (ExistingData)
	{
		if ((*ExistingData)->bHasWeight)
		{
			return;
		}

		// Previous one wasn't weighted but this is, this should take over
		if (bHasWeight)
		{
			(*ExistingData)->bIsActive = false;
			OutBaseData.bIsActive = true;
			*ExistingData = &OutBaseData;

			// Don't reinitialize
			return;
		}
	}
	else
	{
		OutBaseData.bIsActive = true;
		OutBaseComponentTracker.Add(ControlRig, &OutBaseData);
	}
	

#if WITH_EDITOR	
	// The anim mixer handles binding to the skeletal mesh component
	TWeakObjectPtr<UAnimInstance> PreviousAnimInstanceWeakPtr = nullptr;
	if (!bAnimMixerPoseProducer)
	{
		if (ControlRig->GetObjectBinding())
		{
			if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
			{
				PreviousAnimInstanceWeakPtr = SkeletalMeshComponent->GetAnimInstance();
			}
		}
	}
#endif

	// We bind to sequencer instance if we're not an anim mixer pose producer or if our bound object is not a skm component.
	bool bWasCreated = false;
	if (!bAnimMixerPoseProducer || (ControlRig->GetObjectBinding() && Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()) == nullptr))
	{
		bWasCreated = FControlRigBindingHelper::BindToSequencerInstance(ControlRig);
	}
	
	this->AccumulatedValues.InitializeRig(Track, ControlRig);

#if WITH_EDITOR
	if (!bAnimMixerPoseProducer && GEditor && bWasCreated)
	{
		if (ControlRig->GetObjectBinding())
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
			{
				TWeakObjectPtr<USkeletalMeshComponent> WeakSkeletalMeshComponentPtr = SkeletalMeshComponent;
				FDelegateHandle PreCompileHandle = GEditor->OnBlueprintPreCompile().AddLambda([PreviousAnimInstanceWeakPtr, WeakSkeletalMeshComponentPtr](const UBlueprint* InBlueprint)
				{
					const TStrongObjectPtr<UAnimInstance> PinnedAnimInstance = PreviousAnimInstanceWeakPtr.Pin();
					const TStrongObjectPtr<USkeletalMeshComponent> PinnedSkeletalMeshComponent = WeakSkeletalMeshComponentPtr.Pin();
					if (PinnedAnimInstance && PinnedSkeletalMeshComponent && InBlueprint && PinnedAnimInstance->GetClass() == InBlueprint->GeneratedClass)
					{
						FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<UControlRigLayerInstance>(PinnedSkeletalMeshComponent.Get());
					}
				});
								

				PreCompileHandles.Add(PreCompileHandle);
			}
		}
	}
#endif
}

#if WITH_EDITOR
UMovieSceneControlRigParameterEvaluatorSystem::~UMovieSceneControlRigParameterEvaluatorSystem()
{
	if (GEditor)
	{
		for (FDelegateHandle& Handle : PreCompileHandles)
		{
			GEditor->OnBlueprintPreCompile().Remove(Handle);
		}
	}
}
#endif


