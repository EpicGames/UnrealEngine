// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/ControlRigSettings.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSettings)

UControlRigSettings::UControlRigSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	DefaultShapeLibrary = LoadObject<UControlRigShapeLibrary>(nullptr, TEXT("/ControlRig/Controls/DefaultGizmoLibraryNormalized.DefaultGizmoLibraryNormalized"));
	DefaultRootModule = TEXT("/ControlRig/Modules/Modules56/Root.Root");
#endif
}

UControlRigEditorSettings::UControlRigEditorSettings(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
    , bResetControlTransformsOnCompile(true)
#endif
{
#if WITH_EDITORONLY_DATA
	bResetControlsOnCompile = true;
	bResetControlsOnPinValueInteraction = false;
	bResetPoseWhenTogglingEventQueue = true;
	bEnableUndoForPoseInteraction = true;
	bEnableFlashlightInDependencyViewer = true;

	ConstructionEventBorderColor = FLinearColor::Red;
	BackwardsSolveBorderColor = FLinearColor::Yellow;
	BackwardsAndForwardsBorderColor = FLinearColor::Blue;
	bShowSchematicViewInModularRig = true;
	bShowStackedHierarchy = false;
	MaxStackSize = 16;
	bLeftMouseDragDoesMarquee = true;
	bArrangeByModules = true;
	bFlattenModules = false;
	bFocusOnSelection = true;
	ElementNameDisplayMode = EElementNameDisplayMode::AssetDefault;
	OutlinerMultiRigDisplayMode = EMultiRigTreeDisplayMode::All;
#endif
}

#if WITH_EDITOR
bool UControlRigEditorSettings::HasAnyModularRigHierarchyConnectorVisibilityFlags(const EModularRigHierarchyEditorConnectorVisibilityFlags Flags) const
{
	return EnumHasAnyFlags(GetModularRigHierarchyConnectorVisibilityFlags(), Flags);
}

EModularRigHierarchyEditorConnectorVisibilityFlags UControlRigEditorSettings::GetModularRigHierarchyConnectorVisibilityFlags() const
{
	return static_cast<EModularRigHierarchyEditorConnectorVisibilityFlags>(ModularRigHierarchyConnectorVisibilityFlags);
}

void UControlRigEditorSettings::SetModularRigHierarchyConnectorVisibilityFlags(const EModularRigHierarchyEditorConnectorVisibilityFlags Flags) 
{
	ModularRigHierarchyConnectorVisibilityFlags = static_cast<uint8>(Flags);
}
#endif // WITH_EDITOR
