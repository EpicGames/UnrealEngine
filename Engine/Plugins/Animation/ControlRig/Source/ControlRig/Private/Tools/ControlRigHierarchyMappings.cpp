// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigHierarchyMappings.h"

#include "ControlRig.h"
#include "BoneContainer.h"
#include "Rigs/RigHierarchy.h"
#include "Animation/NodeMappingContainer.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimCurveTypes.h"
#include "HAL/IConsoleManager.h"
#include "Tools/ControlRigIOSettings.h"
#include "Tools/ControlRigPoseAdapter.h"
#include "Components/SkeletalMeshComponent.h"

// CVar to enable performance optimizations within the anim node pose exchange
TAutoConsoleVariable<int32> CVarControlRigEnableAnimNodePerformanceOptimizations(
	TEXT("ControlRig.EnableAnimNodePerformanceOptimizations"),
	1,
	TEXT("if nonzero we enable the (experimental) execution performance optimizations of Control Rig AnimNodes."));


void FControlRigHierarchyMappings::InitializeInstance()
{
	SetEnablePoseAdapter(CVarControlRigEnableAnimNodePerformanceOptimizations->GetInt() != 0);
}

void FControlRigHierarchyMappings::LinkToHierarchy(URigHierarchy* InHierarchy)
{
	if (bEnablePoseAdapter)
	{
		if (InHierarchy)
		{
			if (!PoseAdapter.IsValid())
			{
				PoseAdapter = MakeShareable(new FControlRigPoseAdapter());
			}
			InHierarchy->LinkPoseAdapter(PoseAdapter);
		}
	}
}

void FControlRigHierarchyMappings::UpdateControlRigRefPoseIfNeeded(UControlRig* ControlRig
	, const UObject* InstanceObject
	, const USkeletalMeshComponent* SkeletalMeshComponent
	, const FBoneContainer& RequiredBones
	, bool bInSetRefPoseFromSkeleton
	, bool bIncludePoseInHash)
{
	check(ControlRig);

	if (!bInSetRefPoseFromSkeleton)
	{
		return;
	}

	int32 ExpectedHash = 0;
	ExpectedHash = HashCombine(ExpectedHash, (int32)reinterpret_cast<uintptr_t>(InstanceObject));
	ExpectedHash = HashCombine(ExpectedHash, (int32)reinterpret_cast<uintptr_t>(SkeletalMeshComponent));

	if (SkeletalMeshComponent)
	{
		ExpectedHash = HashCombine(ExpectedHash, (int32)reinterpret_cast<uintptr_t>(SkeletalMeshComponent->GetSkeletalMeshAsset()));
	}

	FMemMark Mark(FMemStack::Get());
	FCompactPose RefPose;
	RefPose.ResetToRefPose(RequiredBones);

	if (bIncludePoseInHash)
	{
		for (const FCompactPoseBoneIndex& BoneIndex : RefPose.ForEachBoneIndex())
		{
			const FTransform& Transform = RefPose.GetRefPose(BoneIndex);
			const FQuat Rotation = Transform.GetRotation();

			ExpectedHash = HashCombine(ExpectedHash, GetTypeHash(Transform.GetTranslation()));
			ExpectedHash = HashCombine(ExpectedHash, GetTypeHash(FVector4(Rotation.X, Rotation.Y, Rotation.Z, Rotation.W)));
			ExpectedHash = HashCombine(ExpectedHash, GetTypeHash(Transform.GetScale3D()));
		}
	}

	if (RefPoseSetterHash.IsSet() && (ExpectedHash == RefPoseSetterHash.GetValue()))
	{
		return;
	}
	ControlRig->SetBoneInitialTransformsFromCompactPose(&RefPose);

	RefPoseSetterHash = ExpectedHash;
}

void FControlRigHierarchyMappings::UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
	, URigHierarchy* InHierarchy
	, const FBoneContainer& InRequiredBones
	, const TArray<FBoneReference>& InInputBonesToTransfer
	, const TArray<FBoneReference>& InOutputBonesToTransfer
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInTransferPoseInGlobalSpace
	, bool bInResetInputPoseToInitial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InHierarchy == nullptr)
	{
		return;
	}

	if (bEnablePoseAdapter)
	{
		InHierarchy->UnlinkPoseAdapter();

		if (!PoseAdapter.IsValid())
		{
			return;
		}

		PoseAdapter->UpdateInputOutputMappingIfRequired(InControlRig
			, InHierarchy
			, InRequiredBones
			, InNodeMappingContainer
			, bInTransferPoseInGlobalSpace
			, bInResetInputPoseToInitial);
	}
	else
	{
		UpdateInputOutputMappingIfRequiredImpl(InControlRig
			, InHierarchy
			, InRequiredBones
			, InInputBonesToTransfer
			, InOutputBonesToTransfer
			, InNodeMappingContainer);
	}
}

void FControlRigHierarchyMappings::UpdateInput(UControlRig* ControlRig
	, FPoseContext& InOutput
	, const FControlRigIOSettings& InInputSettings
	, const FControlRigIOSettings& InOutputSettings
	, TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInExecute
	, bool bInTransferInputPose
	, bool bInResetInputPoseToInitial
	, bool bInTransferPoseInGlobalSpace
	, bool bInTransferInputCurves)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (!IsValid(Hierarchy))
	{
		return;
	}

#if WITH_EDITOR
	// if we are recording any change - let's clear the undo stack
	if (bInExecute && Hierarchy->IsTracingChanges())
	{
		Hierarchy->ResetTransformStack();
	}
#endif

	if (bEnablePoseAdapter)
	{
		if (PoseAdapter && InInputSettings.bUpdatePose)
		{
			if (InInputSettings.bUpdatePose)
			{
				// reset transforms here to prevent additive transforms from accumulating to INF
				// we only update transforms from the mesh pose for bones in the current LOD, 
				// so the reset here ensures excluded bones are also reset
				if (!PoseAdapter->GetBonesToResetToInitial().IsEmpty())
				{
					FRigHierarchyValidityBracket ValidityBracket(Hierarchy);
					{
#if WITH_EDITOR
						// make sure transient controls don't get reset
						UControlRig::FTransientControlPoseScope PoseScope(ControlRig);
#endif
						for (const int32& BoneElementIndex : PoseAdapter->GetBonesToResetToInitial())
						{
							if (FRigTransformElement* BoneElement = Hierarchy->Get<FRigBoneElement>(BoneElementIndex))
							{
								const FTransform InitialLocalTransform = Hierarchy->GetTransform(BoneElement, ERigTransformType::InitialLocal);
								BoneElement->GetTransform().Set(ERigTransformType::CurrentLocal, InitialLocalTransform);
								BoneElement->GetDirtyState().MarkClean(ERigTransformType::CurrentLocal);
								BoneElement->GetDirtyState().MarkDirty(ERigTransformType::CurrentGlobal);
							}
						}
					}
				}

				if (bInTransferInputPose && InOutput.Pose.GetNumBones() == PoseAdapter->GetLocalPose().Num())
				{
					PoseAdapter->MarkDependentsDirty();
					PoseAdapter->CopyBonesFrom(InOutput.Pose);
					PoseAdapter->UpdateDirtyStates(true);
				}
			}

			if (InInputSettings.bUpdateCurves || InOutputSettings.bUpdateCurves)
			{
				Hierarchy->UnsetCurveValues();

				const TArray<FRigBaseElement*> HierarchyCurves = Hierarchy->GetCurvesFast();
				PoseAdapter->SetHierarchyCurvesLookup(HierarchyCurves);
				PoseAdapter->SetPoseCurveToHierarchyCurve(HierarchyCurves, InOutput.Curve);
			}
		}
	}
	else
	{
		if (InInputSettings.bUpdatePose && bInTransferInputPose)
		{
			const FBoneContainer& RequiredBones = InOutput.Pose.GetBoneContainer();

			// reset transforms here to prevent additive transforms from accumulating to INF
			// we only update transforms from the mesh pose for bones in the current LOD, 
			// so the reset here ensures excluded bones are also reset
			if (!GetControlRigBoneInputMappingByName().IsEmpty() || bInResetInputPoseToInitial)
			{
				FRigHierarchyValidityBracket ValidityBracket(Hierarchy);

				{
#if WITH_EDITOR
					// make sure transient controls don't get reset
					UControlRig::FTransientControlPoseScope PoseScope(ControlRig);
#endif 
					Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
				}
			}

			if (bInTransferPoseInGlobalSpace || InNodeMappingContainer.IsValid())
			{
				// get component pose from control rig
				FCSPose<FCompactPose> MeshPoses;
				// first I need to convert to local pose
				MeshPoses.InitPose(InOutput.Pose);

				if (!GetControlRigBoneInputMappingByIndex().IsEmpty())
				{
					for (const TPair<uint16, uint16>& Pair : GetControlRigBoneInputMappingByIndex())
					{
						const uint16 ControlRigIndex = Pair.Key;
						const uint16 SkeletonIndex = Pair.Value;

						FCompactPoseBoneIndex CompactPoseIndex(SkeletonIndex);
						const FTransform& ComponentTransform = MeshPoses.GetComponentSpaceTransform(CompactPoseIndex);
						Hierarchy->SetGlobalTransformByIndex(ControlRigIndex, ComponentTransform, false);
					}
				}
				else
				{
					for (auto Iter = GetControlRigBoneInputMappingByName().CreateConstIterator(); Iter; ++Iter)
					{
						const FName& Name = Iter.Key();
						const uint16 Index = Iter.Value();
						const FRigElementKey Key(Name, ERigElementType::Bone);

						FCompactPoseBoneIndex CompactPoseIndex(Index);

						const FTransform& ComponentTransform = MeshPoses.GetComponentSpaceTransform(CompactPoseIndex);
						if (InNodeMappingContainer.IsValid())
						{
							const FTransform& RelativeTransformReverse = InNodeMappingContainer->GetSourceToTargetTransform(Name).GetRelativeTransformReverse(ComponentTransform);
							Hierarchy->SetGlobalTransform(Key, RelativeTransformReverse, false);
						}
						else
						{
							Hierarchy->SetGlobalTransform(Key, ComponentTransform, false);
						}

					}
				}
			}
			else
			{
				if (!GetControlRigBoneInputMappingByIndex().IsEmpty())
				{
					for (const TPair<uint16, uint16>& Pair : GetControlRigBoneInputMappingByIndex())
					{
						const uint16 ControlRigIndex = Pair.Key;
						const uint16 SkeletonIndex = Pair.Value;

						FCompactPoseBoneIndex CompactPoseIndex(SkeletonIndex);
						const FTransform& LocalTransform = InOutput.Pose[CompactPoseIndex];
						Hierarchy->SetLocalTransformByIndex(ControlRigIndex, LocalTransform, false);
					}
				}
				else
				{
					for (auto Iter = GetControlRigBoneInputMappingByName().CreateConstIterator(); Iter; ++Iter)
					{
						const FName& Name = Iter.Key();
						const uint16 SkeletonIndex = Iter.Value();
						const FRigElementKey Key(Name, ERigElementType::Bone);

						FCompactPoseBoneIndex CompactPoseIndex(SkeletonIndex);
						const FTransform& LocalTransform = InOutput.Pose[CompactPoseIndex];
						Hierarchy->SetLocalTransform(Key, LocalTransform, false);
					}
				}
			}

		}

		if (InInputSettings.bUpdateCurves && bInTransferInputCurves)
		{
			Hierarchy->UnsetCurveValues();

			InOutput.Curve.ForEachElement([Hierarchy](const UE::Anim::FCurveElement& InCurveElement)
				{
					const FRigElementKey Key(InCurveElement.Name, ERigElementType::Curve);
					Hierarchy->SetCurveValue(Key, InCurveElement.Value);
				});
		}
	}

#if WITH_EDITOR
	if (bInExecute && Hierarchy->IsTracingChanges())
	{
		Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::UpdateInput"));
	}
#endif
}

void FControlRigHierarchyMappings::UpdateOutput(UControlRig* ControlRig
	, FPoseContext& InOutput
	, const FControlRigIOSettings& InOutputSettings
	, TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInExecute
	, bool bInTransferPoseInGlobalSpace)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (!IsValid(Hierarchy))
	{
		return;
	}

	if (bEnablePoseAdapter)
	{
		if (InOutputSettings.bUpdatePose)
		{
			if (PoseAdapter)
			{
				// make sure the local respectively global transforms are all ready
				if (PoseAdapter->GetTransferInLocalSpace())
				{
					PoseAdapter->ConvertToLocalPose();
				}
				else
				{
					PoseAdapter->ConvertToGlobalPose();
				}

				// if we transfered in global - let's convert back to local
				if (!PoseAdapter->GetTransferInLocalSpace())
				{
					if (PoseAdapter->GetGlobalPose().Num() == InOutput.Pose.GetNumBones())
					{
						PoseAdapter->ConvertToLocalPose();
					}
				}

				InOutput.Pose.CopyBonesFrom(PoseAdapter->GetLocalPose());
			}
		}

		if (InOutputSettings.bUpdateCurves)
		{
			const TArray<int32>& ChangedCurveIndices = Hierarchy->GetChangedCurveIndices();
			if (ChangedCurveIndices.Num() > 0)
			{
				const TArray<FRigBaseElement*> HierarchyCurves = Hierarchy->GetCurvesFast();

				int32 CurveIndex = 0;
				int32 CurvesCopied = 0;
				InOutput.Curve.ForEachElement([this, &CurveIndex, &CurvesCopied, &HierarchyCurves](const UE::Anim::FCurveElement& InCurveElement)
					{
						// the index stored here is the sub index of the curve (the index of the curve within the list of curves)
						const int32& HierarchyIndex = PoseAdapter->GetPoseCurveToHierarchyCurve()[CurveIndex];
						if (HierarchyIndex != INDEX_NONE)
						{
							const FRigCurveElement* HierarchyCurve = CastChecked<FRigCurveElement>(HierarchyCurves[HierarchyIndex]);
							if (HierarchyCurve->IsValueSet())
							{
								const_cast<UE::Anim::FCurveElement*>(&InCurveElement)->Value = HierarchyCurve->Get();
							}
							CurvesCopied++;
						}
						CurveIndex++;
					});

				if (CurvesCopied < HierarchyCurves.Num())
				{
					HierarchyCurveCopied.SetNumUninitialized(Hierarchy->Num());
					FMemory::Memzero(HierarchyCurveCopied.GetData(), HierarchyCurveCopied.GetAllocatedSize());

					FBlendedCurve ControlRigCurves;
					ControlRigCurves.Reserve(ChangedCurveIndices.Num());
					for (const int32& ChangedCurveIndex : ChangedCurveIndices)
					{
						if (!HierarchyCurveCopied[ChangedCurveIndex])
						{
							if (const FRigCurveElement* HierarchyCurve = Hierarchy->Get<FRigCurveElement>(ChangedCurveIndex))
							{
								if (HierarchyCurve->IsValueSet())
								{
									ControlRigCurves.Add(HierarchyCurve->GetFName(), HierarchyCurve->Get());
								}
							}
							HierarchyCurveCopied[ChangedCurveIndex] = true;
						}
					}

					InOutput.Curve.Combine(ControlRigCurves);
				}
			}
		}
	}
	else
	{
		if (InOutputSettings.bUpdatePose)
		{
			// copy output of the rig
			const FBoneContainer& RequiredBones = InOutput.Pose.GetBoneContainer();

			TMap<FName, uint16>& NameBasedMapping = GetControlRigBoneOutputMappingByName();
			TArray<TPair<uint16, uint16>>& IndexBasedMapping = GetControlRigBoneOutputMappingByIndex();

			// if we don't have a different mapping for outputs, use the input mapping
			if (NameBasedMapping.IsEmpty() && IndexBasedMapping.IsEmpty())
			{
				NameBasedMapping = GetControlRigBoneInputMappingByName();
				IndexBasedMapping = GetControlRigBoneInputMappingByIndex();
			}

			if (bInTransferPoseInGlobalSpace || InNodeMappingContainer.IsValid())
			{
				// get component pose from control rig
				FCSPose<FCompactPose> MeshPoses;
				MeshPoses.InitPose(InOutput.Pose);

				if (!IndexBasedMapping.IsEmpty())
				{
					for (const TPair<uint16, uint16>& Pair : IndexBasedMapping)
					{
						const uint16 ControlRigIndex = Pair.Key;
						const uint16 SkeletonIndex = Pair.Value;

						FCompactPoseBoneIndex CompactPoseIndex(SkeletonIndex);
						FTransform ComponentTransform = Hierarchy->GetGlobalTransformByIndex(ControlRigIndex);
						MeshPoses.SetComponentSpaceTransform(CompactPoseIndex, ComponentTransform);
					}
				}
				else
				{
					for (auto Iter = NameBasedMapping.CreateConstIterator(); Iter; ++Iter)
					{
						const FName& Name = Iter.Key();
						const uint16 SkeletonIndex = Iter.Value();
						const FRigElementKey Key(Name, ERigElementType::Bone);

						FCompactPoseBoneIndex CompactPoseIndex(SkeletonIndex);
						FTransform ComponentTransform = Hierarchy->GetGlobalTransform(Key);
						if (InNodeMappingContainer.IsValid())
						{
							ComponentTransform = InNodeMappingContainer->GetSourceToTargetTransform(Name) * ComponentTransform;
						}

						MeshPoses.SetComponentSpaceTransform(CompactPoseIndex, ComponentTransform);
					}
				}

				FCSPose<FCompactPose>::ConvertComponentPosesToLocalPosesSafe(MeshPoses, InOutput.Pose);
				InOutput.Pose.NormalizeRotations();
			}
			else
			{
				if (!IndexBasedMapping.IsEmpty())
				{
					for (const TPair<uint16, uint16>& Pair : IndexBasedMapping)
					{
						const uint16 ControlRigIndex = Pair.Key;
						const uint16 SkeletonIndex = Pair.Value;

						FCompactPoseBoneIndex CompactPoseIndex(SkeletonIndex);
						FTransform LocalTransform = Hierarchy->GetLocalTransformByIndex(ControlRigIndex);
						InOutput.Pose[CompactPoseIndex] = LocalTransform;
					}
				}
				else
				{
					for (auto Iter = NameBasedMapping.CreateConstIterator(); Iter; ++Iter)
					{
						const FName& Name = Iter.Key();
						const uint16 Index = Iter.Value();
						const FRigElementKey Key(Name, ERigElementType::Bone);

						FCompactPoseBoneIndex CompactPoseIndex(Index);
						FTransform LocalTransform = Hierarchy->GetLocalTransform(Key);
						InOutput.Pose[CompactPoseIndex] = LocalTransform;
					}
				}
			}
		}

		if (InOutputSettings.bUpdateCurves)
		{
			FBlendedCurve ControlRigCurves;
			ControlRigCurves.Reserve(Hierarchy->Num(ERigElementType::Curve));
			Hierarchy->ForEach<FRigCurveElement>([&ControlRigCurves](const FRigCurveElement* InElement)
				{
					if (InElement->IsValueSet())
					{
						ControlRigCurves.Add(InElement->GetFName(), InElement->Get());
					}
					return true;
				});

			InOutput.Curve.Combine(ControlRigCurves);
		}
	}

#if WITH_EDITOR
	if (bInExecute && Hierarchy->IsTracingChanges())
	{
		Hierarchy->StorePoseForTrace(TEXT("FAnimNode_ControlRigBase::UpdateOutput"));
		Hierarchy->DumpTransformStackToFile();
	}
#endif
}

void FControlRigHierarchyMappings::UpdateInputOutputMappingIfRequiredImpl(UControlRig* InControlRig
	, URigHierarchy* InHierarchy
	, const FBoneContainer& InRequiredBones
	, const TArray<FBoneReference>& InInputBonesToTransfer
	, const TArray<FBoneReference>& InOutputBonesToTransfer
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ControlRigBoneInputMappingByIndex.Reset();
	ControlRigBoneOutputMappingByIndex.Reset();
	ControlRigCurveMappingByIndex.Reset();
	ControlRigBoneInputMappingByName.Reset();
	ControlRigBoneOutputMappingByName.Reset();
	ControlRigCurveMappingByName.Reset();

	if (InRequiredBones.IsValid())
	{
		const TArray<FBoneIndexType>& RequiredBonesArray = InRequiredBones.GetBoneIndicesArray();
		const int32 NumBones = RequiredBonesArray.Num();

		const FReferenceSkeleton& RefSkeleton = InRequiredBones.GetReferenceSkeleton();

		// @todo: thread-safe? probably not in editor, but it may not be a big issue in editor
		if (InNodeMappingContainer.IsValid())
		{
			// get target to source mapping table - this is reversed mapping table
			TMap<FName, FName> TargetToSourceMappingTable;
			InNodeMappingContainer->GetTargetToSourceMappingTable(TargetToSourceMappingTable);

			// now fill up node name
			for (uint16 Index = 0; Index < NumBones; ++Index)
			{
				// get bone name, and find reverse mapping
				FName TargetNodeName = RefSkeleton.GetBoneName(RequiredBonesArray[Index]);
				FName* SourceName = TargetToSourceMappingTable.Find(TargetNodeName);
				if (SourceName)
				{
					ControlRigBoneInputMappingByName.Add(*SourceName, Index);
				}
			}
		}
		else
		{
			TArray<FName> NodeNames;
			TArray<FNodeItem> NodeItems;
			InControlRig->GetMappableNodeData(NodeNames, NodeItems);

			// even if not mapped, we map only node that exists in the controlrig
			for (uint16 Index = 0; Index < NumBones; ++Index)
			{
				const FName& BoneName = RefSkeleton.GetBoneName(RequiredBonesArray[Index]);
				if (NodeNames.Contains(BoneName))
				{
					ControlRigBoneInputMappingByName.Add(BoneName, Index);
				}
			}
		}

		auto UpdatingMappingFromSpecificTransferList = [](
			const TArray<FBoneReference>& InTransferList,
			const TWeakObjectPtr<UNodeMappingContainer>& InMappingContainer,
			const FBoneContainer& InRequiredBones,
			const FReferenceSkeleton& InRefSkeleton,
			const TArray<FBoneIndexType>& InRequiredBonesArray,
			const UControlRig* InControlRig,
			TMap<FName, uint16>& OutMapping
			) {
				OutMapping.Reset();

				if (InMappingContainer.IsValid())
				{
					// get target to source mapping table - this is reversed mapping table
					TMap<FName, FName> TargetToSourceMappingTable;
					InMappingContainer->GetTargetToSourceMappingTable(TargetToSourceMappingTable);

					for (const FBoneReference& InputBoneToTransfer : InTransferList)
					{
						const int32 BoneIndex = InRequiredBones.GetPoseBoneIndexForBoneName(InputBoneToTransfer.BoneName);
						if (BoneIndex == INDEX_NONE)
						{
							continue;
						}
						const FName TargetNodeName = InRefSkeleton.GetBoneName(BoneIndex/*InputBoneToTransfer.BoneIndex*/);
						if (const FName* SourceName = TargetToSourceMappingTable.Find(TargetNodeName))
						{
							OutMapping.Add(*SourceName, BoneIndex/*InputBoneToTransfer.BoneIndex*/);
						}
					}
				}
				else
				{
					TArray<FName> NodeNames;
					TArray<FNodeItem> NodeItems;
					InControlRig->GetMappableNodeData(NodeNames, NodeItems);

					for (const FBoneReference& InputBoneToTransfer : InTransferList)
					{
						const int32 BoneIndex = InRequiredBones.GetPoseBoneIndexForBoneName(InputBoneToTransfer.BoneName);
						if (BoneIndex == INDEX_NONE)
						{
							continue;
						}
						if (InRequiredBonesArray.IsValidIndex(BoneIndex/*InputBoneToTransfer.BoneIndex*/))
						{
							const FName& BoneName = InRefSkeleton.GetBoneName(InRequiredBonesArray[BoneIndex/*InputBoneToTransfer.BoneIndex*/]);
							if (NodeNames.Contains(BoneName))
							{
								OutMapping.Add(BoneName, BoneIndex/*InputBoneToTransfer.BoneIndex*/);
							}
						}
					}
				}
			};

		if (!InInputBonesToTransfer.IsEmpty())
		{
			ControlRigBoneOutputMappingByName = ControlRigBoneInputMappingByName;

			UpdatingMappingFromSpecificTransferList(
				InInputBonesToTransfer,
				InNodeMappingContainer,
				InRequiredBones,
				RefSkeleton,
				RequiredBonesArray,
				InControlRig,
				ControlRigBoneInputMappingByName);
		}

		if (!InOutputBonesToTransfer.IsEmpty())
		{
			UpdatingMappingFromSpecificTransferList(
				InOutputBonesToTransfer,
				InNodeMappingContainer,
				InRequiredBones,
				RefSkeleton,
				RequiredBonesArray,
				InControlRig,
				ControlRigBoneOutputMappingByName);
		}

		// check if we can switch the bones to an index based mapping.
		// we can only do that if there is no node mapping container set.
		if (!InNodeMappingContainer.IsValid())
		{
			for (int32 InputOutput = 0; InputOutput < 2; InputOutput++)
			{
				bool bIsMappingByIndex = true;
				TMap<FName, uint16>& NameBasedMapping = InputOutput == 0 ? ControlRigBoneInputMappingByName : ControlRigBoneOutputMappingByName;
				if (NameBasedMapping.IsEmpty())
				{
					continue;
				}

				TArray<TPair<uint16, uint16>>& IndexBasedMapping = InputOutput == 0 ? ControlRigBoneInputMappingByIndex : ControlRigBoneOutputMappingByIndex;

				for (auto Iter = NameBasedMapping.CreateConstIterator(); Iter; ++Iter)
				{
					const uint16 SkeletonIndex = Iter.Value();
					const int32 ControlRigIndex = InHierarchy->GetIndex(FRigElementKey(Iter.Key(), ERigElementType::Bone));
					if (ControlRigIndex != INDEX_NONE)
					{
						IndexBasedMapping.Add(TPair<uint16, uint16>((uint16)ControlRigIndex, SkeletonIndex));
					}
					else
					{
						bIsMappingByIndex = false;
					}
				}

				if (bIsMappingByIndex)
				{
					NameBasedMapping.Reset();
				}
				else
				{
					IndexBasedMapping.Reset();
				}
			}
		}
	}
}

bool FControlRigHierarchyMappings::CheckPoseAdapter() const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (bEnablePoseAdapter && !PoseAdapter.IsValid())
	{
		return false;
	}

	return true;
}

bool FControlRigHierarchyMappings::IsUpdateToDate(const URigHierarchy* InHierarchy) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (bEnablePoseAdapter)
	{
		if (!PoseAdapter->IsUpdateToDate(InHierarchy))
		{
			return false;
		}
	}
	return true;
}

void FControlRigHierarchyMappings::PerformUpdateToDate(UControlRig* ControlRig
	, URigHierarchy* InHierarchy
	, const FBoneContainer& InRequiredBones
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInTransferPoseInGlobalSpace
	, bool bInResetInputPoseToInitial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (bEnablePoseAdapter)
	{
		if (!PoseAdapter->IsUpdateToDate(InHierarchy))
		{
			InHierarchy->UnlinkPoseAdapter();
			PoseAdapter->UpdateInputOutputMappingIfRequired(ControlRig, InHierarchy, InRequiredBones, InNodeMappingContainer, bInTransferPoseInGlobalSpace, bInResetInputPoseToInitial);
			InHierarchy->LinkPoseAdapter(PoseAdapter);
		}
	}
}

void FControlRigHierarchyMappings::SetEnablePoseAdapter(bool bInEnablePoseAdapter)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (bEnablePoseAdapter != bInEnablePoseAdapter)
	{
		bEnablePoseAdapter = bInEnablePoseAdapter;

		if (bEnablePoseAdapter)
		{
			PoseAdapter = MakeShareable(new FControlRigPoseAdapter());
		}
		else
		{
			PoseAdapter.Reset();
		}
	}
}
