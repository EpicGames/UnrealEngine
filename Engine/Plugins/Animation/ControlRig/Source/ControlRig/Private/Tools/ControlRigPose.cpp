// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tools/ControlRigPose.h"
#include "Tools/ControlRigPoseProjectSettings.h"
#include "Tools/ControlRigPoseMirrorSettings.h"

#include "IControlRigObjectBinding.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigPose)

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "ControlRigPose"

void FControlRigControlPose::SavePose(UControlRig* ControlRig, bool bUseAll, bool bSaveValues)
{
	if (!ControlRig)
	{
		return;
	}
	
	TArray<FRigControlElement*> CurrentControls;
	ControlRig->GetControlsInOrder(CurrentControls);
	CopyOfControls.SetNum(0);
	
	for (FRigControlElement* ControlElement : CurrentControls)
	{
		if (ControlRig->GetHierarchy()->IsAnimatable(ControlElement) && (bUseAll || ControlRig->IsControlSelected(ControlElement->GetFName())))
		{
			if (bSaveValues)
			{
				//we store poses in default parent space so if not in that space we need to compensate it
				bool bHasNonDefaultParent = false;
				FTransform GlobalTransform;
				FRigElementKey SpaceKey = ControlRig->GetHierarchy()->GetActiveParent(ControlElement->GetKey());
				if (SpaceKey != ControlRig->GetHierarchy()->GetDefaultParentKey())
				{
					bHasNonDefaultParent = true;
					//to compensate we get the global, switch space, then reset global in that new space
					GlobalTransform = ControlRig->GetHierarchy()->GetGlobalTransform(ControlElement->GetKey());
					ControlRig->GetHierarchy()->SwitchToDefaultParent(ControlElement->GetKey());
					ControlRig->GetHierarchy()->SetGlobalTransform(ControlElement->GetKey(), GlobalTransform);

					ControlRig->Evaluate_AnyThread();
				}

				FRigControlCopy Copy(ControlElement, ControlRig->GetHierarchy(), bSaveValues);
				CopyOfControls.Add(Copy);

				if (bHasNonDefaultParent == true)
				{
					ControlRig->GetHierarchy()->SwitchToParent(ControlElement->GetKey(), SpaceKey);
					ControlRig->GetHierarchy()->SetGlobalTransform(ControlElement->GetKey(), GlobalTransform);
				}
			}
			else
			{
				FRigControlCopy Copy(ControlElement, ControlRig->GetHierarchy(), bSaveValues);
				CopyOfControls.Add(Copy);

			}
		}
	}
	SetUpControlMap();
}

void FControlRigControlPose::PastePose(UControlRig* ControlRig, bool bDoKey, bool bDoMirror)
{
	if (!ControlRig)
	{
		return;
	}
	PastePoseInternal(ControlRig, bDoKey, bDoMirror, CopyOfControls);
	ControlRig->Evaluate_AnyThread();
	PastePoseInternal(ControlRig, bDoKey, bDoMirror, CopyOfControls);

}

void FControlRigControlPose::SetControlMirrorTransform(bool bDoLocal, UControlRig* ControlRig, const FName& Name, bool bIsMatched,
	const FTransform& GlobalTransform, const FTransform& LocalTransform, bool bNotify, const  FRigControlModifiedContext& Context, bool bSetupUndo)
{
	if (!ControlRig || !ControlRig->GetHierarchy())
	{
		return;
	}
	FRigPose ControlsAfterBackwardsSolve;
	int32 AdditiveControlIndex = INDEX_NONE;
	if (ControlRig->IsAdditive())
	{
		if (FRigControlElement* ControlElement = ControlRig->GetHierarchy()->Find<FRigControlElement>((FRigElementKey(Name, ERigElementType::Control))))
		{
			ControlsAfterBackwardsSolve = ControlRig->GetControlsAfterBackwardsSolve();
			AdditiveControlIndex = ControlsAfterBackwardsSolve.GetIndex(ControlElement->GetKey());
		}
	}
	if (bDoLocal || bIsMatched)
	{
		FTransform Transform(LocalTransform);
		if (AdditiveControlIndex != INDEX_NONE)
		{
			const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
			Transform = Transform.GetRelativeTransform(AnimPose.LocalTransform);
		}
		ControlRig->SetControlLocalTransform(Name, Transform, bNotify,Context,bSetupUndo, true/* bFixEulerFlips*/);

	}
	else
	{
		FTransform Transform(GlobalTransform);
		if (AdditiveControlIndex != INDEX_NONE)
		{
			const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
			Transform = Transform.GetRelativeTransform(AnimPose.GlobalTransform);
		}
		ControlRig->SetControlGlobalTransform(Name, Transform, bNotify,Context,bSetupUndo, false /*bPrintPython*/, true/* bFixEulerFlips*/);
	}	
}

//Set of functions to figure how to possible invert translation or rotation values when doing mirrroring
namespace UE::AIE
{
	//Get Directions of a transforms vector by either grabbing them directly or getting them based on cross product of the other axis
	TTuple<FVector, FVector, FVector> GetDirections(const FTransform& Transform, bool bUseCrossProduct)
	{
		TTuple<FVector, FVector, FVector> Vectors;
		FTransform StrippedTransform(Transform.GetRotation().Rotator(), FVector(0, 0, 0), Transform.GetScale3D());
		FVector XTrans = StrippedTransform.TransformPosition(FVector(1, 0, 0));
		FVector YTrans = StrippedTransform.TransformPosition(FVector(0, 1, 0));
		FVector ZTrans = StrippedTransform.TransformPosition(FVector(0, 0, 1));
		if (bUseCrossProduct)
		{
			FVector XCross = YTrans.Cross(ZTrans);
			FVector YCross = ZTrans.Cross(XTrans);
			FVector ZCross = XTrans.Cross(YTrans);

			Vectors.Get<0>() = XCross;
			Vectors.Get<1>() = YCross;
			Vectors.Get<2>() = ZCross;
			return Vectors;
		}
		Vectors.Get<0>() = XTrans;
		Vectors.Get<1>() = YTrans;
		Vectors.Get<2>() = ZTrans;
		return Vectors;
	}
	//get optimal mirrored transform based upon mirror axis
	FTransform  GetMirroredTransform(const FTransform& InTransform, TEnumAsByte<EAxis::Type> MirrorAxis)
	{
		FMatrix Matrix = InTransform.ToMatrixWithScale();
		FVector X = Matrix.GetScaledAxis(EAxis::X);
		FVector Y = Matrix.GetScaledAxis(EAxis::Y);
		FVector Z = Matrix.GetScaledAxis(EAxis::Z);
		FVector Origin = Matrix.GetOrigin();
		FVector MultFactor{ -1.0,1.0,1.0 }; //EAxis::X is default
	    if (MirrorAxis == EAxis::Y)
		{
			MultFactor = FVector(1.0, -1.0, 1.0);
		}
		else if (MirrorAxis == EAxis::Z)
		{
			MultFactor = FVector(1.0, 1.0, -1.0);
		}
		X *= MultFactor;
		Y *= MultFactor;
		Z *= MultFactor;
		Origin *= MultFactor;
		Matrix = FMatrix(X, Y, Z, FVector::ZeroVector);
		Matrix.SetOrigin(Origin);
		FTransform Transform;
		Transform.SetFromMatrix(Matrix);
		return Transform;
	}

	//get the transalation or rotation factors to see which axis needs to get mirrored and how, if bUseCross is true we are getting the rotation factors
	FVector GetFactors(const FTransform& OrigTransform, const FTransform& MirredTransform, TEnumAsByte<EAxis::Type> MirrorAxis, bool bUseCross)
	{
		FTransform IdealMirrored = GetMirroredTransform(OrigTransform, MirrorAxis);
		TTuple<FVector, FVector, FVector> IdealDir = GetDirections(IdealMirrored, bUseCross);
		TTuple<FVector, FVector, FVector> MirroredDir = GetDirections(MirredTransform, bUseCross);
		FVector Factor;
		//dot products
		Factor.X = IdealDir.Get<0>() | MirroredDir.Get<0>();
		Factor.Y = IdealDir.Get<1>() | MirroredDir.Get<1>();
		Factor.Z = IdealDir.Get<2>() | MirroredDir.Get<2>();

		return Factor;
	}
}

void FControlRigControlPose::PastePoseInternal(UControlRig* ControlRig, bool bDoKey, bool bDoMirror, const TArray<FRigControlCopy>& ControlsToPaste)
{
	if (!ControlRig || !ControlRig->GetHierarchy())
	{
		return;
	}
	TEnumAsByte<EAxis::Type> MirrorAxis = EAxis::X;
	if (const UControlRigPoseMirrorSettings* Settings = GetDefault<UControlRigPoseMirrorSettings>())
	{
		MirrorAxis = Settings->MirrorAxis;
	}
	FRigControlModifiedContext Context;
	Context.SetKey = bDoKey ? EControlRigSetKey::Always : EControlRigSetKey::DoNotCare;
	FControlRigPoseMirrorTable MirrorTable;
	
	TArray<FRigControlElement*> SortedControls;
	ControlRig->GetControlsInOrder(SortedControls);
	
	const bool bDoLocal = true;
	const bool bSetupUndo = false;
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

	FRigPose ControlsAfterBackwardsSolve;
	if (ControlRig->IsAdditive())
	{
		ControlsAfterBackwardsSolve = ControlRig->GetControlsAfterBackwardsSolve();
	}

	for (FRigControlElement* ControlElement : SortedControls)
	{
		if (!ControlRig->IsControlSelected(ControlElement->GetFName()))
		{
			continue;
		}
		
		FRigControlCopy* CopyRigControl = MirrorTable.GetControl(ControlRig, *this, ControlElement->GetFName(), bDoMirror);
		if (CopyRigControl)
		{
			//if not in default parent space we need to move it to default parent space first and then reset the global transforms
			bool bHasNonDefaultParent = false;
			FRigElementKey SpaceKey = Hierarchy->GetActiveParent(ControlElement->GetKey());
			if (SpaceKey != Hierarchy->GetDefaultParentKey())
			{
				bHasNonDefaultParent = true;
				Hierarchy->SwitchToDefaultParent(ControlElement->GetKey());
				ControlRig->Evaluate_AnyThread();
			}

			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Transform:
			case ERigControlType::TransformNoScale:
			case ERigControlType::EulerTransform:
			case ERigControlType::Position:
			case ERigControlType::Scale:
			case ERigControlType::Rotator:
			{
				int32 AdditiveControlIndex = INDEX_NONE;
				if (ControlRig->IsAdditive())
				{
					AdditiveControlIndex = ControlsAfterBackwardsSolve.GetIndex(ControlElement->GetKey());
				}
				if (bDoMirror == false)
				{
					if (bDoLocal) // -V547  
					{
						FTransform Transform = CopyRigControl->LocalTransform;
						if (AdditiveControlIndex != INDEX_NONE)
						{
							const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
							Transform = Transform.GetRelativeTransform(AnimPose.LocalTransform);
						}
						ControlRig->SetControlLocalTransform(ControlElement->GetFName(), Transform, true, Context, bSetupUndo, true/* bFixEulerFlips*/);
					}
					else
					{
						FTransform Transform = CopyRigControl->GlobalTransform;
						if (AdditiveControlIndex != INDEX_NONE)
						{
							const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
							Transform = Transform.GetRelativeTransform(AnimPose.GlobalTransform);
						}
						ControlRig->SetControlGlobalTransform(ControlElement->GetFName(), Transform, true, Context, bSetupUndo, false /*bPrintPython*/, true/* bFixEulerFlips*/);
					}
				}
				else
				{
					FTransform GlobalTransform;
					FTransform LocalTransform;
					bool bIsMatched = MirrorTable.IsMatched(ControlRig, CopyRigControl->Name);
					const FName NewName = CopyRigControl->Name;
					FControlRigPoseMirrorContext MirrorContext;
					MirrorContext.bIsMatched = bIsMatched;
					MirrorContext.bDoLocal = bDoLocal;
					//if matched calculator translation/rotation factors to see if we need to flip any axis there.
					if (bIsMatched == true)
					{
						FTransform OrigTransform = Hierarchy->GetTransform(ControlElement, ERigTransformType::InitialGlobal);
						if (FRigControlElement* NewElement = ControlRig->FindControl(NewName))
						{
							FTransform NewTransform = Hierarchy->GetTransform(NewElement, ERigTransformType::InitialGlobal);
							MirrorContext.TranslationFactor = UE::AIE::GetFactors(OrigTransform, NewTransform, MirrorAxis, false /*bUseCross*/);
							MirrorContext.RotationFactor = UE::AIE::GetFactors(OrigTransform, NewTransform, MirrorAxis, true /*bUseCross*/);
						}
					}
					MirrorTable.GetMirrorTransform(*CopyRigControl, MirrorContext, GlobalTransform, LocalTransform);
					SetControlMirrorTransform(bDoLocal,ControlRig, ControlElement->GetFName(), bIsMatched, GlobalTransform,LocalTransform,true,Context,bSetupUndo);
				}				
				break;
			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				float Val = CopyRigControl->Value.Get<float>();
				ControlRig->SetControlValue<float>(ControlElement->GetFName(), Val, true, Context,bSetupUndo);
				break;
			}
			case ERigControlType::Bool:
			{
				bool Val = CopyRigControl->Value.Get<bool>();
				ControlRig->SetControlValue<bool>(ControlElement->GetFName(), Val, true, Context,bSetupUndo);
				break;
			}
			case ERigControlType::Integer:
			{
				int32 Val = CopyRigControl->Value.Get<int32>();
				ControlRig->SetControlValue<int32>(ControlElement->GetFName(), Val, true, Context,bSetupUndo);
				break;
			}
			case ERigControlType::Vector2D:
			{
				FVector3f Val = CopyRigControl->Value.Get<FVector3f>();
				ControlRig->SetControlValue<FVector3f>(ControlElement->GetFName(), Val, true, Context,bSetupUndo);
				break;
			}
			default:
				//TODO add log
				break;
			};

			if (bHasNonDefaultParent == true)
			{
				FTransform GlobalTransform = ControlRig->GetHierarchy()->GetGlobalTransform(ControlElement->GetKey());
				ControlRig->GetHierarchy()->SwitchToParent(ControlElement->GetKey(), SpaceKey);
				ControlRig->SetControlGlobalTransform(ControlElement->GetFName(), GlobalTransform, true, Context, bSetupUndo, false /*bPrintPython*/, true/* bFixEulerFlips*/);
			}
		}
	}
}

void FControlRigControlPose::BlendWithInitialPoses(FControlRigControlPose& InitialPose, UControlRig* ControlRig, bool bDoKey, bool bDoMirror, float BlendValue, bool bDoAdditive)
{
	if (!ControlRig || !ControlRig->GetHierarchy())
	{
		return;
	}

	auto BlendTransforms = [bDoAdditive, BlendValue](FTransform& InitialVal, FTransform& Val) 
	{ 
		FVector Translation, Scale;
		FQuat Rotation;
		if (bDoAdditive == false)
		{
			Translation = FMath::Lerp(InitialVal.GetTranslation(), Val.GetTranslation(), BlendValue);
			Rotation = FQuat::Slerp(InitialVal.GetRotation(), Val.GetRotation(), BlendValue); //doing slerp here not fast lerp, can be slow this is for content creation
			Scale = FMath::Lerp(InitialVal.GetScale3D(), Val.GetScale3D(), BlendValue);
		}
		else
		{
			Translation = InitialVal.GetTranslation() + (Val.GetTranslation() * BlendValue);
			//do additive as rotators
			FRotator RotatorVal = Val.GetRotation().Rotator();
			FRotator RotatorInitial = InitialVal.GetRotation().Rotator();
			RotatorVal = RotatorInitial + (RotatorVal * BlendValue);
			Rotation = FQuat(RotatorVal);
			Scale = FMath::Lerp(InitialVal.GetScale3D(), Val.GetScale3D(), BlendValue); //scale is just blended
		}
		Val = FTransform(Rotation, Translation, Scale);
	};

	if (InitialPose.CopyOfControls.Num() == 0)
	{
		return;
	}
	TEnumAsByte<EAxis::Type> MirrorAxis = EAxis::X;
	if (const UControlRigPoseMirrorSettings* Settings = GetDefault<UControlRigPoseMirrorSettings>())
	{
		MirrorAxis = Settings->MirrorAxis;
	}
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

	//though can be n^2 should be okay, we search from current Index which in most cases will be the same
	//not run often anyway
	FRigControlModifiedContext Context;
	Context.SetKey = bDoKey ? EControlRigSetKey::Always : EControlRigSetKey::DoNotCare;
	FControlRigPoseMirrorTable MirrorTable;

	TArray<FRigControlElement*> SortedControls;
	ControlRig->GetControlsInOrder(SortedControls);
	
	FRigPose ControlsAfterBackwardsSolve;
	if (ControlRig->IsAdditive())
	{
		ControlsAfterBackwardsSolve = ControlRig->GetControlsAfterBackwardsSolve();
	}

	const bool bDoLocal = true;
	const bool bSetupUndo = false;
	for (FRigControlElement* ControlElement : SortedControls)
	{
		if (!ControlRig->IsControlSelected(ControlElement->GetFName()))
		{
			continue;
		}
		FRigControlCopy* CopyRigControl = MirrorTable.GetControl(ControlRig,*this, ControlElement->GetFName(),bDoMirror);
		if (CopyRigControl)
		{
			FRigControlCopy* InitialFound = nullptr;
			int32* Index = InitialPose.CopyOfControlsNameToIndex.Find(ControlElement->GetFName());
			if (Index)
			{
				InitialFound = &(InitialPose.CopyOfControls[*Index]);
			}
			if (InitialFound && InitialFound->ControlType == CopyRigControl->ControlType)
			{
				if ((CopyRigControl->ControlType == ERigControlType::Transform || CopyRigControl->ControlType == ERigControlType::EulerTransform ||
					CopyRigControl->ControlType == ERigControlType::TransformNoScale || CopyRigControl->ControlType == ERigControlType::Position ||
					CopyRigControl->ControlType == ERigControlType::Rotator || CopyRigControl->ControlType == ERigControlType::Scale
					))
				{
					//if not in default parent space we need to move it to default parent space first and then reset the global transforms
					bool bHasNonDefaultParent = false;
					FRigElementKey SpaceKey = Hierarchy->GetActiveParent(ControlElement->GetKey());
					if (SpaceKey != Hierarchy->GetDefaultParentKey())
					{
						bHasNonDefaultParent = true;
						Hierarchy->SwitchToDefaultParent(ControlElement->GetKey());
						ControlRig->Evaluate_AnyThread();
					}
					
					int32 AdditiveControlIndex = INDEX_NONE;
					if (ControlRig->IsAdditive())
					{
						AdditiveControlIndex = ControlsAfterBackwardsSolve.GetIndex(ControlElement->GetKey());
					}

					if (bDoMirror == false)
					{
						if (bDoLocal == true)    // -V547  
						{
							FTransform Val = CopyRigControl->LocalTransform;
							FTransform InitialVal = InitialFound->LocalTransform;
							BlendTransforms(InitialVal, Val);

							if (AdditiveControlIndex != INDEX_NONE)
							{
								const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
								Val = Val.GetRelativeTransform(AnimPose.LocalTransform);
							}

							ControlRig->SetControlLocalTransform(ControlElement->GetFName(), Val, true,Context,bSetupUndo, true/* bFixEulerFlips*/);
						}
						else
						{
							FTransform Val = CopyRigControl->GlobalTransform;
							FTransform InitialVal = InitialFound->GlobalTransform;
							BlendTransforms(InitialVal, Val);
							if (AdditiveControlIndex != INDEX_NONE)
							{
								const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
								Val = Val.GetRelativeTransform(AnimPose.GlobalTransform);
							}
							ControlRig->SetControlGlobalTransform(ControlElement->GetFName(), Val, true,Context,bSetupUndo, false /*bPrintPython*/, true/* bFixEulerFlips*/);
						}
					}
					else
					{				
						FTransform GlobalTransform;
						FTransform LocalTransform;
						bool bIsMatched = MirrorTable.IsMatched(ControlRig, CopyRigControl->Name);
						const FName NewName = CopyRigControl->Name;
						FControlRigPoseMirrorContext MirrorContext;
						MirrorContext.bIsMatched = bIsMatched;
						MirrorContext.bDoLocal = bDoLocal;
						//if matched calculator translation/rotation factors to see if we need to flip any axis there.
						if (bIsMatched == true)
						{
							FTransform OrigTransform = Hierarchy->GetTransform(ControlElement, ERigTransformType::InitialGlobal);
							if (FRigControlElement* NewElement = ControlRig->FindControl(NewName))
							{
								FTransform NewTransform = Hierarchy->GetTransform(NewElement, ERigTransformType::InitialGlobal);
								MirrorContext.TranslationFactor = UE::AIE::GetFactors(OrigTransform, NewTransform, MirrorAxis, false /*bUseCross*/);
								MirrorContext.RotationFactor = UE::AIE::GetFactors(OrigTransform, NewTransform, MirrorAxis, true /*bUseCross*/);
							}
						}
						MirrorTable.GetMirrorTransform(*CopyRigControl, MirrorContext, GlobalTransform, LocalTransform);


						BlendTransforms(InitialFound->GlobalTransform, GlobalTransform);
						BlendTransforms(InitialFound->LocalTransform, LocalTransform);

						SetControlMirrorTransform(bDoLocal,ControlRig, ControlElement->GetFName(), bIsMatched, GlobalTransform,LocalTransform,bDoKey,Context,bSetupUndo);							
					}
					if (bHasNonDefaultParent == true)
					{
						FTransform GlobalTransform = Hierarchy->GetGlobalTransform(ControlElement->GetKey());
						Hierarchy->SwitchToParent(ControlElement->GetKey(), SpaceKey);
						FTransform Transform(GlobalTransform);
						if (AdditiveControlIndex != INDEX_NONE)
						{
							const FRigPoseElement& AnimPose = ControlsAfterBackwardsSolve[AdditiveControlIndex];
							Transform = Transform.GetRelativeTransform(AnimPose.LocalTransform);
						}

						ControlRig->SetControlGlobalTransform(ControlElement->GetFName(), Transform, true, Context, bSetupUndo, false /*bPrintPython*/, true/* bFixEulerFlips*/);
					}
				}
				else if(CopyRigControl->ControlType == ERigControlType::Float || 
						CopyRigControl->ControlType == ERigControlType::ScaleFloat)
				{
					float InitialVal = InitialFound->Value.Get<float>();
					float Val = CopyRigControl->Value.Get<float>();
					Val = bDoAdditive == false ? FMath::Lerp(InitialVal, Val, BlendValue) : InitialVal + (Val * BlendValue);
					ControlRig->SetControlValue<float>(ControlElement->GetFName(), Val, true, Context, bSetupUndo);
				}
				else if (CopyRigControl->ControlType == ERigControlType::Vector2D)
				{
					FVector3f InitialVal = InitialFound->Value.Get<FVector3f>();
					FVector3f Val = CopyRigControl->Value.Get<FVector3f>();
					Val = bDoAdditive == false ? FMath::Lerp(InitialVal, Val, BlendValue) : InitialVal + (Val * BlendValue);
					ControlRig->SetControlValue<FVector3f>(ControlElement->GetFName(), Val, true, Context, bSetupUndo);
				}
			}
		}
	}
}

bool FControlRigControlPose::ContainsName(const FName& Name) const
{
	const int32* Index = CopyOfControlsNameToIndex.Find(Name);
	return (Index && *Index >= 0);
}

void FControlRigControlPose::ReplaceControlName(const FName& Name, const FName& NewName)
{
	int32* Index = CopyOfControlsNameToIndex.Find(Name);
	if (Index && *Index >= 0)
	{
		FRigControlCopy& Control = CopyOfControls[*Index];
		Control.Name = NewName;
		CopyOfControlsNameToIndex.Remove(Name);
		CopyOfControlsNameToIndex.Add(Control.Name, *Index);
	}
}

TArray<FName> FControlRigControlPose::GetControlNames() const
{
	TArray<FName> Controls;
	Controls.Reserve(CopyOfControls.Num());
	for (const FRigControlCopy& Control : CopyOfControls)
	{
		Controls.Add(Control.Name);
	}
	return Controls;
}

void FControlRigControlPose::SetUpControlMap()
{
	CopyOfControlsNameToIndex.Reset();

	for (int32 Index = 0; Index < CopyOfControls.Num(); ++Index)
	{
		const FRigControlCopy& Control = CopyOfControls[Index];
		CopyOfControlsNameToIndex.Add(Control.Name, Index);
	}
}


UControlRigPoseAsset::UControlRigPoseAsset(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UControlRigPoseAsset::PostLoad()
{
	Super::PostLoad();
	Pose.SetUpControlMap();
}

void UControlRigPoseAsset::SavePose(UControlRig* InControlRig, bool bUseAll, bool bSaveValues)
{
	if (!InControlRig)
	{
		return;
	}
	Pose.SavePose(InControlRig,bUseAll, bSaveValues);
}

void UControlRigPoseAsset::PastePose(UControlRig* InControlRig, bool bDoKey, bool bDoMirror, bool bDoAdditive)
{
	if (!InControlRig)
	{
		return;
	}
#if WITH_EDITOR
	FScopedTransaction ScopedTransaction(LOCTEXT("PastePoseTransaction", "Paste Pose"));
	InControlRig->Modify();
#endif
	if (bDoAdditive == false)
	{
		Pose.PastePose(InControlRig, bDoKey, bDoMirror);
	}
	else //if additive we blend with 1 since that uses current pose
	{
		FControlRigControlPose TempPose;
		GetCurrentPose(InControlRig, TempPose);
		Pose.BlendWithInitialPoses(TempPose, InControlRig, bDoKey, bDoMirror, 1.0, bDoAdditive);
		InControlRig->Evaluate_AnyThread();
		Pose.BlendWithInitialPoses(TempPose, InControlRig, bDoKey, bDoMirror, 1.0, bDoAdditive);

	}
}

void UControlRigPoseAsset::SelectControls(UControlRig* InControlRig, bool bDoMirror, bool bClearSelection)
{
	if (!InControlRig)
	{
		return;
	}
	
#if WITH_EDITOR
	FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"));
	InControlRig->Modify();
#endif
	if (bClearSelection)
	{
		InControlRig->ClearControlSelection();
	}
	TArray<FName> Controls = Pose.GetControlNames();
	FControlRigPoseMirrorTable MirrorTable;
	FControlRigControlPose TempPose;
	if (bDoMirror)
	{
		TempPose.SavePose(InControlRig, true , false/*no values*/);
	}
	const bool bSelect = true;
	const bool bUndo = true;
	for (const FName& Name : Controls)
	{
		if (bDoMirror)
		{
			if (FRigControlElement* ControlElement = InControlRig->GetHierarchy()->Find<FRigControlElement>((FRigElementKey(Name, ERigElementType::Control))))
			{
				if (ControlElement->IsAnimationChannel())
				{
					continue;
				}
			}
			FRigControlCopy* CopyRigControl = MirrorTable.GetControl(InControlRig, TempPose, Name, bDoMirror);
			if (CopyRigControl)
			{
				InControlRig->SelectControl(CopyRigControl->Name, bSelect, bUndo);
			}
			else
			{
				InControlRig->SelectControl(Name, bSelect, bUndo);
			}
		}
		else
		{
			InControlRig->SelectControl(Name, bSelect, bUndo);
		}
	}
}

void UControlRigPoseAsset::GetCurrentPose(UControlRig* InControlRig, FControlRigControlPose& OutPose)
{
	if (!InControlRig)
	{
		return;
	}
	OutPose.SavePose(InControlRig, true);
}


TArray<FRigControlCopy> UControlRigPoseAsset::GetCurrentPose(UControlRig* InControlRig) 
{
	FControlRigControlPose TempPose;
	if (InControlRig)
	{
		TempPose.SavePose(InControlRig,true);
	}
	return TempPose.GetPoses();
}

void UControlRigPoseAsset::BlendWithInitialPoses(FControlRigControlPose& InitialPose, UControlRig* InControlRig, bool bDoKey, bool bDoMirror, float BlendValue, bool bDoAdditive)
{
	Pose.BlendWithInitialPoses(InitialPose, InControlRig, bDoKey, bDoMirror, BlendValue, bDoAdditive);
}

TArray<FName> UControlRigPoseAsset::GetControlNames() const
{
	return Pose.GetControlNames();
}

void UControlRigPoseAsset::ReplaceControlName(const FName& CurrentName, const FName& NewName)
{
	Pose.ReplaceControlName(CurrentName, NewName);
}

bool UControlRigPoseAsset::DoesMirrorMatch(UControlRig* ControlRig, const FName& ControlName) 
{
	return (MirrorMatchTable.IsMatched(ControlRig,ControlName));
}


#undef LOCTEXT_NAMESPACE

