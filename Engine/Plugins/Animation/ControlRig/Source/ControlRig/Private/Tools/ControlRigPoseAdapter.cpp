// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigPoseAdapter.h"
#include "Rigs/RigHierarchy.h"
#include "ControlRig.h"
#include "BoneContainer.h"
#include "Animation/NodeMappingContainer.h"

void FControlRigPoseAdapter::PostLinked(URigHierarchy* InHierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigHierarchyPoseAdapter::PostLinked(InHierarchy);

	// 1. sort the hierarchies storage so that we can make sure initial and local is grouped correctly
	SortHierarchyStorage();

	// 2. make sure to compute all local transforms in initial and local
	const TArray<FRigTransformElement*> TransformElements = InHierarchy->GetElementsOfType<FRigTransformElement>(false);
	for (FRigTransformElement* TransformElement : TransformElements)
	{
		(void)InHierarchy->GetTransform(TransformElement, ERigTransformType::InitialLocal);
		(void)InHierarchy->GetTransform(TransformElement, ERigTransformType::CurrentLocal);
	}

	// 3. set up a list of dependents in the hierarchy of the rig to be reset to initial on execute
	TArray<FRigTransformElement*> DependentTransformElements;

	struct Local
	{
		static bool ProcessTransformElement(FControlRigPoseAdapter* InPoseAdapter, TArray<FRigTransformElement*>& InDependentTransformElements, FRigTransformElement* InTransformElement)
		{
			if (InTransformElement == nullptr)
			{
				return false;
			}

			if (InPoseAdapter->ElementIndexToPoseIndex.Contains((uint16)InTransformElement->GetIndex()))
			{
				return true;
			}

			if (InDependentTransformElements.Contains(InTransformElement))
			{
				return true;
			}

			const FRigBaseElementParentArray& ParentElements = InPoseAdapter->GetHierarchy()->GetParents(InTransformElement);
			for (const FRigBaseElement* ParentElement : ParentElements)
			{
				if (ProcessTransformElement(InPoseAdapter, InDependentTransformElements, Cast<FRigTransformElement>(const_cast<FRigBaseElement*>(ParentElement))))
				{
					InDependentTransformElements.AddUnique(InTransformElement);
					return true;
				}
			}

			return false;
		}
	};

	for (FRigTransformElement* TransformElement : TransformElements)
	{
		Local::ProcessTransformElement(this, DependentTransformElements, TransformElement);
	}

	Dependents.Reset();
	Dependents.Reserve(DependentTransformElements.Num() * 3);

	for (FRigTransformElement* DependentTransformElement : DependentTransformElements)
	{
		// skip bones - since they are taking care of by BonesToResetToInitial
		if (const FRigBoneElement* BoneElement = Cast<FRigBoneElement>(DependentTransformElement))
		{
			// skip bones if they are not user defined - and if the parent of the bone is not a dependent as well
			if (BoneElement->BoneType == ERigBoneType::Imported &&
				!DependentTransformElements.Contains(BoneElement->ParentElement))
			{
				continue;
			}
		}

		FRigControlElement* DependentControlElement = Cast<FRigControlElement>(DependentTransformElement);
		if (DependentControlElement && DependentControlElement->IsAnimationChannel())
		{
			continue;
		}

		Dependents.Emplace(DependentTransformElement->GetKeyAndIndex(), ERigTransformType::CurrentGlobal, ERigTransformStorageType::Pose, &DependentTransformElement->GetDirtyState().Current);
		if (DependentControlElement)
		{
			Dependents.Emplace(DependentTransformElement->GetKeyAndIndex(), ERigTransformType::CurrentGlobal, ERigTransformStorageType::Offset, &DependentControlElement->GetOffsetDirtyState().Current);
			Dependents.Emplace(DependentTransformElement->GetKeyAndIndex(), ERigTransformType::CurrentGlobal, ERigTransformStorageType::Shape, &DependentControlElement->GetShapeDirtyState().Current);
		}
	}

	// 4. relink the storage for the transforms (local or global or both) and dirty states to our local storage
	static constexpr bool bLocalIsPrimary = true;
	UpdateDirtyStates(bLocalIsPrimary);
	for (int32 PoseIndex = 0; PoseIndex < PoseIndexToElementIndex.Num(); PoseIndex++)
	{
		const int32& TransformElementIndex = PoseIndexToElementIndex[PoseIndex];
		if (TransformElementIndex != INDEX_NONE)
		{
			const FRigElementKeyAndIndex& KeyAndIndex = InHierarchy->GetKeyAndIndex(TransformElementIndex);
			RelinkTransformStorage(KeyAndIndex, ERigTransformType::CurrentLocal, ERigTransformStorageType::Pose, &LocalPose[PoseIndex], &LocalPoseIsDirty[PoseIndex]);
			RelinkTransformStorage(KeyAndIndex, ERigTransformType::CurrentGlobal, ERigTransformStorageType::Pose, &GlobalPose[PoseIndex], &GlobalPoseIsDirty[PoseIndex]);
		}
	}
	UpdateDirtyStates(bLocalIsPrimary); // do this again to make sure local is flagged as clean and global as dirty

	// 5. Shrink the storage on the hierarchy now that we've relinked it
	ShrinkHierarchyStorage();
}

void FControlRigPoseAdapter::PreUnlinked(URigHierarchy* InHierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	UnlinkTransformStorage();
	UpdateHierarchyStorage();

	FRigHierarchyPoseAdapter::PreUnlinked(InHierarchy);
}

bool FControlRigPoseAdapter::IsUpdateToDate(const URigHierarchy* InHierarchy) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!FRigHierarchyPoseAdapter::IsUpdateToDate(InHierarchy))
	{
		return false;
	}
	return !LocalPose.IsEmpty() && !GlobalPose.IsEmpty();
}

void FControlRigPoseAdapter::UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
	, URigHierarchy* InHierarchy
	, const FBoneContainer& InRequiredBones
	, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
	, bool bInTransferPoseInGlobalSpace
	, bool bInResetInputPoseToInitial)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InHierarchy == nullptr)
	{
		return;
	}

	ParentPoseIndices.Reset();
	RequiresHierarchyForSpaceConversion.Reset();
	ElementIndexToPoseIndex.Reset();
	PoseIndexToElementIndex.Reset();
	GlobalPose.Reset();
	LocalPose.Reset();
	HierarchyCurveLookup.Reset();

	const int32 NumBonesInPose = InRequiredBones.GetCompactPoseNumBones();

	ParentPoseIndices.Reserve(NumBonesInPose);
	RequiresHierarchyForSpaceConversion.Reserve(NumBonesInPose);
	GlobalPose.AddDefaulted(NumBonesInPose);
	LocalPose.AddDefaulted(NumBonesInPose);

	bTransferInLocalSpace = !(bInTransferPoseInGlobalSpace || InNodeMappingContainer.IsValid());

	for (int32 Index = 0; Index < NumBonesInPose; Index++)
	{
		ParentPoseIndices.AddDefaulted();

		const FCompactPoseBoneIndex ParentBoneIndex = InRequiredBones.GetParentBoneIndex(FCompactPoseBoneIndex(Index));
		if (ParentBoneIndex.IsValid())
		{
			ParentPoseIndices[Index] = ParentBoneIndex;
		}
		RequiresHierarchyForSpaceConversion.Add(false);
	}
	UpdateDirtyStates();

	TArray<int32> MappedBoneElementIndices;
	if (InRequiredBones.IsValid())
	{
		ElementIndexToPoseIndex.Reserve(NumBonesInPose);
		PoseIndexToElementIndex.Reserve(NumBonesInPose);

		const FReferenceSkeleton* RefSkeleton = &InRequiredBones.GetReferenceSkeleton();
		if (const USkeleton* Skeleton = InRequiredBones.GetSkeletonAsset())
		{
			RefSkeleton = &Skeleton->GetReferenceSkeleton();
		}

		// @todo: thread-safe? probably not in editor, but it may not be a big issue in editor
		if (InNodeMappingContainer.IsValid())
		{
			// get target to source mapping table - this is reversed mapping table
			TMap<FName, FName> TargetToSourceMappingTable;
			InNodeMappingContainer->GetTargetToSourceMappingTable(TargetToSourceMappingTable);

			// now fill up node name
			for (uint16 Index = 0; Index < NumBonesInPose; ++Index)
			{
				// get bone name, and find reverse mapping
				const FSkeletonPoseBoneIndex BoneIndex = InRequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(FCompactPoseBoneIndex(Index));
				if (BoneIndex.IsValid() && RefSkeleton->IsValidIndex(BoneIndex.GetInt()))
				{
					FName TargetNodeName = RefSkeleton->GetBoneName(BoneIndex.GetInt());
					FName* SourceName = TargetToSourceMappingTable.Find(TargetNodeName);
					if (SourceName)
					{
						const int32 ElementIndex = InHierarchy->GetIndex({ *SourceName, ERigElementType::Bone });
						if (ElementIndex != INDEX_NONE)
						{
							MappedBoneElementIndices.Add(ElementIndex);
							ElementIndexToPoseIndex.Add(static_cast<uint16>(ElementIndex), Index);
							PoseIndexToElementIndex.Add(ElementIndex);
							LocalPose[Index] = InHierarchy->GetLocalTransform(ElementIndex);
							GlobalPose[Index] = InHierarchy->GetGlobalTransform(ElementIndex);
							continue;
						}
					}
				}
				PoseIndexToElementIndex.Add(INDEX_NONE);
			}
		}
		else
		{
			TArray<FName> NodeNames;
			TArray<FNodeItem> NodeItems;
			InControlRig->GetMappableNodeData(NodeNames, NodeItems);

			// even if not mapped, we map only node that exists in the controlrig
			for (uint16 Index = 0; Index < NumBonesInPose; ++Index)
			{
				const FSkeletonPoseBoneIndex BoneIndex = InRequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(FCompactPoseBoneIndex(Index));
				if (BoneIndex.IsValid() && RefSkeleton->IsValidIndex(BoneIndex.GetInt()))
				{
					const FName& BoneName = RefSkeleton->GetBoneName(BoneIndex.GetInt());
					if (NodeNames.Contains(BoneName))
					{
						const int32 ElementIndex = InHierarchy->GetIndex({ BoneName, ERigElementType::Bone });
						if (ElementIndex != INDEX_NONE)
						{
							MappedBoneElementIndices.Add(ElementIndex);
							ElementIndexToPoseIndex.Add(static_cast<uint16>(ElementIndex), Index);
							PoseIndexToElementIndex.Add(ElementIndex);
							LocalPose[Index] = InHierarchy->GetLocalTransform(ElementIndex);
							GlobalPose[Index] = InHierarchy->GetGlobalTransform(ElementIndex);
							continue;
						}
					}
				}
				PoseIndexToElementIndex.Add(INDEX_NONE);
			}
		}

		// once we know all of the bones we are going to transfer - we can check if any of these bones has a different parenting
		// relationship in the skeleton used in the anim graph vs the hierarchy in the rig. in that case we have to transfer in global
		if (bTransferInLocalSpace)
		{
			for (const int32& BoneElementIndex : MappedBoneElementIndices)
			{
				const int32 HierarchyParentIndex = InHierarchy->GetFirstParent(BoneElementIndex);
				const int16 PoseIndex = ElementIndexToPoseIndex.FindChecked(BoneElementIndex);
				const FCompactPoseBoneIndex CompactPoseParentIndex(ParentPoseIndices[PoseIndex]);

				FName HierarchyParentName(NAME_None);
				FName PoseParentName(NAME_None);

				if (HierarchyParentIndex != INDEX_NONE)
				{
					HierarchyParentName = InHierarchy->Get(HierarchyParentIndex)->GetFName();
				}
				if (CompactPoseParentIndex.IsValid())
				{
					const FSkeletonPoseBoneIndex SkeletonIndex = InRequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(CompactPoseParentIndex);
					if (SkeletonIndex.IsValid() && RefSkeleton->IsValidIndex(SkeletonIndex.GetInt()))
					{
						PoseParentName = RefSkeleton->GetBoneName(SkeletonIndex.GetInt());
					}
				}

				if (HierarchyParentName.IsEqual(PoseParentName, ENameCase::CaseSensitive))
				{
					continue;
				}

				RequiresHierarchyForSpaceConversion[PoseIndex] = true;
				check(PoseIndexToElementIndex[PoseIndex] != INDEX_NONE);
				bTransferInLocalSpace = false;
			}
		}

		// only reset the full pose if we are not mapping all bones
		const TArray<FRigBaseElement*>& HierarchyBones = InHierarchy->GetBonesFast();
		const bool bMapsAllBones = MappedBoneElementIndices.Num() == HierarchyBones.Num();
		BonesToResetToInitial.Reset();
		bRequiresResetPoseToInitial = bInResetInputPoseToInitial && !bMapsAllBones;

		if (bRequiresResetPoseToInitial)
		{
			BonesToResetToInitial.Reserve(HierarchyBones.Num() - MappedBoneElementIndices.Num());

			// bone is mapped stores sub indices (bone index within the list of bones)
			TArray<bool> BoneIsMapped;
			BoneIsMapped.AddZeroed(HierarchyBones.Num());
			for (const int32& MappedTransformIndex : MappedBoneElementIndices)
			{
				const FRigBaseElement* MappedElement = InHierarchy->Get(MappedTransformIndex);
				check(MappedElement);
				BoneIsMapped[MappedElement->GetSubIndex()] = true;
			}

			// when we want to know which bones to reset we want to convert back to a global index
			for (int32 UnmappedBoneIndex = 0; UnmappedBoneIndex < BoneIsMapped.Num(); UnmappedBoneIndex++)
			{
				if (!BoneIsMapped[UnmappedBoneIndex])
				{
					BonesToResetToInitial.Add(HierarchyBones[UnmappedBoneIndex]->GetIndex());
				}
			}
		}
	}
}

void FControlRigPoseAdapter::SetHierarchyCurvesLookup(const TArray<FRigBaseElement*>& InHierarchyCurves)
{
	const int32 NumHierarchyCurves = InHierarchyCurves.Num();
	if (GetHierarchyCurveLookup().Num() != NumHierarchyCurves)
	{
		HierarchyCurveLookup.Reset();
		HierarchyCurveLookup.Reserve(NumHierarchyCurves);
		for (int32 Index = 0; Index < NumHierarchyCurves; Index++)
		{
			HierarchyCurveLookup.Add(InHierarchyCurves[Index]->GetFName(), Index);
		}
	}
}

void FControlRigPoseAdapter::SetPoseCurveToHierarchyCurve(const TArray<FRigBaseElement*>& InHierarchyCurves, const FBlendedCurve& InCurve)
{
	PoseCurveToHierarchyCurve.SetNumUninitialized(InCurve.Num(), EAllowShrinking::No);

	int32 CurveIndex = 0;
	InCurve.ForEachElement([this, &CurveIndex, &InHierarchyCurves](const UE::Anim::FCurveElement& InCurveElement)
		{
			PoseCurveToHierarchyCurve[CurveIndex] = INDEX_NONE;

			// the index stored here is the sub index of the curve (the index of the curve within the list of curves)
			if (const int32* IndexPtr = HierarchyCurveLookup.Find(InCurveElement.Name))
			{
				const int32& Index = *IndexPtr;
				FRigCurveElement* HierarchyCurve = CastChecked<FRigCurveElement>(InHierarchyCurves[Index]);

				// when setting the curve we need to mark it as "value set", otherwise the copy
				// pose may reset it to unset - thus we'll loose the value that was just copied in.
				HierarchyCurve->Set(InCurveElement.Value, true);
				PoseCurveToHierarchyCurve[CurveIndex] = Index;
			}
			CurveIndex++;
		});
}

void FControlRigPoseAdapter::UnlinkTransformStorage()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		for (int32 PoseIndex = 0; PoseIndex < PoseIndexToElementIndex.Num(); PoseIndex++)
		{
			const int32& TransformElementIndex = PoseIndexToElementIndex[PoseIndex];
			if (TransformElementIndex != INDEX_NONE)
			{
				const FRigElementKeyAndIndex& KeyAndIndex = Hierarchy->GetKeyAndIndex(TransformElementIndex);
				RestoreTransformStorage(KeyAndIndex, ERigTransformType::CurrentLocal, ERigTransformStorageType::Pose, false);
				RestoreTransformStorage(KeyAndIndex, ERigTransformType::CurrentGlobal, ERigTransformStorageType::Pose, false);
			}
		}
	}

	ElementIndexToPoseIndex.Reset();
	PoseIndexToElementIndex.Reset();
}

void FControlRigPoseAdapter::ConvertToLocalPose()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	check(ParentPoseIndices.Num() == GlobalPose.Num());
	LocalPose.SetNum(GlobalPose.Num());

	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		for (int32 Index = 0; Index < ParentPoseIndices.Num(); Index++)
		{
			(void)GetLocalTransform(Index);
		}
	}
}
void FControlRigPoseAdapter::ConvertToGlobalPose()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	check(ParentPoseIndices.Num() == LocalPose.Num());
	GlobalPose.SetNum(LocalPose.Num());

	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		for (int32 Index = 0; Index < ParentPoseIndices.Num(); Index++)
		{
			(void)GetGlobalTransform(Index);
		}
	}
}

const FTransform& FControlRigPoseAdapter::GetLocalTransform(int32 InIndex)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	check(LocalPose.IsValidIndex(InIndex));
	check(LocalPoseIsDirty.IsValidIndex(InIndex));
	check(ParentPoseIndices.IsValidIndex(InIndex));
	check(RequiresHierarchyForSpaceConversion.IsValidIndex(InIndex));

	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);

	if (LocalPoseIsDirty[InIndex])
	{
		check(GlobalPoseIsDirty[InIndex] == false);
		if (RequiresHierarchyForSpaceConversion[InIndex] && PoseIndexToElementIndex.IsValidIndex(InIndex))
		{
			LocalPose[InIndex] = Hierarchy->GetLocalTransformByIndex(PoseIndexToElementIndex[InIndex]);
		}
		else
		{
			const FCompactPoseBoneIndex ParentIndex = ParentPoseIndices[InIndex];
			if (!ParentIndex.IsValid())
			{
				LocalPose[InIndex] = GetGlobalTransform(InIndex);
			}
			else
			{
				LocalPose[InIndex] = GetGlobalTransform(InIndex).GetRelativeTransform(GetGlobalTransform(ParentIndex.GetInt()));
			}
			LocalPose[InIndex].NormalizeRotation();
		}
		LocalPoseIsDirty[InIndex] = false;
	}
	return LocalPose[InIndex];
}

const FTransform& FControlRigPoseAdapter::GetGlobalTransform(int32 InIndex)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	check(GlobalPose.IsValidIndex(InIndex));
	check(GlobalPoseIsDirty.IsValidIndex(InIndex));
	check(ParentPoseIndices.IsValidIndex(InIndex));
	check(RequiresHierarchyForSpaceConversion.IsValidIndex(InIndex));

	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);

	if (GlobalPoseIsDirty[InIndex])
	{
		check(LocalPoseIsDirty[InIndex] == false);
		if (RequiresHierarchyForSpaceConversion[InIndex] && PoseIndexToElementIndex.IsValidIndex(InIndex))
		{
			GlobalPose[InIndex] = Hierarchy->GetGlobalTransformByIndex(PoseIndexToElementIndex[InIndex]);
		}
		else
		{
			const FCompactPoseBoneIndex ParentIndex = ParentPoseIndices[InIndex];
			if (!ParentIndex.IsValid())
			{
				GlobalPose[InIndex] = GetLocalTransform(InIndex);
			}
			else
			{
				GlobalPose[InIndex] = GetLocalTransform(InIndex) * GetGlobalTransform(ParentIndex.GetInt());
			}
			GlobalPose[InIndex].NormalizeRotation();
		}
		GlobalPoseIsDirty[InIndex] = false;
	}
	return GlobalPose[InIndex];
}

void FControlRigPoseAdapter::UpdateDirtyStates(const TOptional<bool> InLocalIsPrimary)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	bool bLocalIsPrimary = bTransferInLocalSpace;
	if (InLocalIsPrimary.IsSet())
	{
		bLocalIsPrimary = InLocalIsPrimary.GetValue();
	}
	
	if (LocalPoseIsDirty.Num() != LocalPose.Num())
	{
		LocalPoseIsDirty.Reset();
		LocalPoseIsDirty.AddZeroed(LocalPose.Num());
	}
	else if (bLocalIsPrimary)
	{
		FMemory::Memzero(LocalPoseIsDirty.GetData(), LocalPoseIsDirty.GetAllocatedSize());
	}
	if (!bLocalIsPrimary)
	{
		for (bool& Flag : LocalPoseIsDirty)
		{
			Flag = true;
		}
	}

	if (GlobalPoseIsDirty.Num() != GlobalPose.Num())
	{
		GlobalPoseIsDirty.Reset();
		GlobalPoseIsDirty.AddZeroed(GlobalPose.Num());
	}
	else if (!bLocalIsPrimary)
	{
		FMemory::Memzero(GlobalPoseIsDirty.GetData(), GlobalPoseIsDirty.GetAllocatedSize());
	}
	if (bLocalIsPrimary)
	{
		for (bool& Flag : GlobalPoseIsDirty)
		{
			Flag = true;
		}
	}
}

void FControlRigPoseAdapter::ComputeDependentTransforms()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		// ensure to compute all local transforms
		for (const FDependentTransform& Dependent : Dependents)
		{
			if (FRigTransformElement* TransformElement = Hierarchy->Get<FRigTransformElement>(Dependent.KeyAndIndex.Index))
			{
				switch(Dependent.StorageType)
				{
					case ERigTransformStorageType::Pose:
					{
						(void)Hierarchy->GetTransform(TransformElement, MakeLocal(Dependent.TransformType));
						break;
					}
					case ERigTransformStorageType::Offset:
					{
						FRigControlElement* ControlElement = CastChecked<FRigControlElement>(TransformElement);
						(void)Hierarchy->GetControlOffsetTransform(ControlElement, MakeLocal(Dependent.TransformType));
						break;
					}
					case ERigTransformStorageType::Shape:
					{
						FRigControlElement* ControlElement = CastChecked<FRigControlElement>(TransformElement);
						(void)Hierarchy->GetControlShapeTransform(ControlElement, MakeLocal(Dependent.TransformType));
						break;
					}
					default:
					{
						break;
					}
				}
				check(Dependent.DirtyState->Local.Get() == false);
			}
		}
	}
}

void FControlRigPoseAdapter::MarkDependentsDirty()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (IsValid(GetHierarchy()))
	{
		ComputeDependentTransforms();

		// mark the global dependent as dirty
		for (const FDependentTransform& Dependent : Dependents)
		{
			check(Dependent.DirtyState->Local.Get() == false);
			Dependent.DirtyState->Global.Set(true);
		}
	}
}
