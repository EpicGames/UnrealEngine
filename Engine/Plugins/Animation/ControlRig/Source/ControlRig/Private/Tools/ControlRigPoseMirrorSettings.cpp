// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tools/ControlRigPoseMirrorSettings.h"

#include "Tools/ControlRigPoseMirrorTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigPoseMirrorSettings)

UControlRigPoseMirrorSettings::UControlRigPoseMirrorSettings()
{
	MirrorAxis = EAxis::X;
	AxisToFlip = EAxis::X;
	MirrorMatchTolerance = .28;
}

#if WITH_EDITOR
//property changed so reset the cached mirror tables
void UControlRigPoseMirrorSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FControlRigPoseMirrorTable MirrorTable;
	MirrorTable.Reset();
	SaveConfig();
}
#endif
