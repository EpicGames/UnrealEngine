// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigPoseMirrorTable.h"
#include "Tools/ControlRigPoseMirrorSettings.h"
#include "ControlRig.h"
#include "Tools/ControlRigPose.h"
#include "Tools/MirrorCalculator.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "Animation/AnimationSettings.h"
#include "Animation/MirrorDataTable.h"
#include "Internationalization/Regex.h"


struct FControlRigToMirrorItems
{
	FControlRigToMirrorItems(const UControlRig* InControlRig);
	FControlRigToMirrorItems() = delete;

	const UControlRig* ControlRig;
	TArray<FRigControlElement*> Elements;
	TArray<UE::AIE::FMirrorItem> MirrorItems;
	UE::AIE::FMirrorItemResults Results;
	UE::AIE::FMirrorCalculator Calculator;
};


FControlRigToMirrorItems::FControlRigToMirrorItems(const UControlRig* InControlRig)
{
	using namespace UE::AIE;
	ControlRig = InControlRig;
	if (ControlRig)
	{
		if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			if (const UControlRigPoseMirrorSettings* Settings = GetDefault<UControlRigPoseMirrorSettings>())
			{
				FTransform GlobalTransform;
				TArray<FRigControlElement*> CurrentControls = ControlRig->AvailableControls();
				for (FRigControlElement* ControlElement : CurrentControls)
				{
					if (Hierarchy->IsAnimatable(ControlElement))
					{
						GlobalTransform = Hierarchy->GetTransform(ControlElement, ERigTransformType::InitialGlobal);

						FMirrorItem MirrorItem(GlobalTransform.GetLocation());
						MirrorItems.Add(MirrorItem);
						Elements.Add(ControlElement);

					}
				}

				TEnumAsByte<EAxis::Type> MirrorAxis = Settings->MirrorAxis;
				constexpr double AxisLocation = 0.0;
				const double Tolerance = Settings->MirrorMatchTolerance;
				Calculator.FindMirroredItems(MirrorItems, Results, MirrorAxis, AxisLocation, Tolerance);
			}
		}
	}
}


FString FControlRigPoseMirrorTable::GetMirrorString(const FString& InNameString, const TArray<FMirrorFindReplaceExpression>& MirrorFindReplaceExpressions)
{
	
	for (const FMirrorFindReplaceExpression& RegExStr : MirrorFindReplaceExpressions)
	{
		FString FindString = RegExStr.FindExpression.ToString();
		FString ReplaceString = RegExStr.ReplaceExpression.ToString();
		if (RegExStr.FindReplaceMethod == EMirrorFindReplaceMethod::Prefix)
		{
			// convert prefix expression to regex that matches start of string, prefix, and any number of characters
			FindString = FindString + TEXT("([^}]*)");
			FindString = TEXT("^") + FindString;
			ReplaceString = ReplaceString + TEXT("$1");

		}
		else if (RegExStr.FindReplaceMethod == EMirrorFindReplaceMethod::Suffix)
		{
			// convert suffix expression to regex that matches any number of characters start, the suffix, and end of string
			FindString = TEXT("([^}]*)") + FindString + TEXT("$");
			ReplaceString = TEXT("$1") + ReplaceString;
		}

		FRegexPattern MatherPatter(FindString);
		FRegexMatcher Matcher(MatherPatter, InNameString);
		bool bFound = false;
		while (Matcher.FindNext())
		{
			for (int32 CaptureIndex = 1; CaptureIndex < 10; CaptureIndex++)
			{
				FString CaptureResult = Matcher.GetCaptureGroup(CaptureIndex);
				int32 CaptureBegin = Matcher.GetCaptureGroupBeginning(CaptureIndex);
				int32 CaptureEnd = Matcher.GetCaptureGroupEnding(CaptureIndex);
				FString CaptureRegion = CaptureResult.Mid(CaptureBegin, CaptureEnd - CaptureBegin);
				if (CaptureResult.IsEmpty())
				{
					break;
				}
				FString MatchString = FString::Printf(TEXT("$%i"), CaptureIndex);
				ReplaceString = ReplaceString.Replace(*MatchString, *CaptureResult);
			}
			bFound = true;
		}
		if (bFound)
		{
			return ReplaceString;
		}
	}
	return FString();
}

TMap<const UControlRig*, FControlRigPoseMirrorTable::FMatchedControls> FControlRigPoseMirrorTable::MatchedControls;

void FControlRigPoseMirrorTable::Reset()
{
	FControlRigPoseMirrorTable::MatchedControls.Reset();
}

void FControlRigPoseMirrorTable::SetUpMirrorTable(const UControlRig* ControlRig)
{
	FMatchedControls* CRMatchedControls = FControlRigPoseMirrorTable::MatchedControls.Find(ControlRig);
	if (CRMatchedControls == nullptr || CRMatchedControls->Match.Num() == 0)
	{
		FMatchedControls CurrentMatchedControls;
		//first try by name matching
		TArray<FRigControlElement*> CurrentControls = ControlRig->AvailableControls();
		if (UAnimationSettings* AnimSettings = UAnimationSettings::Get())
		{
			for (FRigControlElement* ControlElement : CurrentControls)
			{
				FString CurrentString = ControlElement->GetName();
				FString NewString = FControlRigPoseMirrorTable::GetMirrorString(CurrentString, AnimSettings->MirrorFindReplaceExpressions);
				if (NewString.IsEmpty() == false && CurrentString != NewString)
				{
					FName CurrentName(*CurrentString);
					FName NewName(*NewString);
					CurrentMatchedControls.Match.Add(NewName, CurrentName);
				}
			}
		}
		FControlRigToMirrorItems MirrorItems(ControlRig);
		if (MirrorItems.Results.MirroredItems.Num() > 0)
		{
			for (TPair<int32, int32>& Pair : MirrorItems.Results.MirroredItems)
			{
				FName CurrentName(MirrorItems.Elements[Pair.Key]->GetFName());
				FName NewName(MirrorItems.Elements[Pair.Value]->GetFName());
				if (CurrentMatchedControls.Match.Contains(CurrentName) == false)
				{
					CurrentMatchedControls.Match.Add(CurrentName, NewName);
				}
			}
		}
		FControlRigPoseMirrorTable::MatchedControls.Add(ControlRig, CurrentMatchedControls);
	}
}

FRigControlCopy* FControlRigPoseMirrorTable::GetControl(const UControlRig* ControlRig, FControlRigControlPose& Pose, FName Name, bool bDoMirror)
{
	if (bDoMirror)
	{
		SetUpMirrorTable(ControlRig);

		if (FMatchedControls* CRMatchedControls = FControlRigPoseMirrorTable::MatchedControls.Find(ControlRig))
		{
			TArray<FRigControlCopy> CopyOfControls;

			if (const FName* MatchedName = CRMatchedControls->Match.Find(Name))
			{
				int32* Index = Pose.CopyOfControlsNameToIndex.Find(*MatchedName);
				if (Index != nullptr && (*Index) >= 0 && (*Index) < Pose.CopyOfControls.Num())
				{
					return &(Pose.CopyOfControls[*Index]);
				}
			}
		}
	}
	//okay not mirror or matched so just find it.
	int32* Index = Pose.CopyOfControlsNameToIndex.Find(Name);
	if (Index != nullptr && (*Index) >= 0 && (*Index) < Pose.CopyOfControls.Num())
	{
		return &(Pose.CopyOfControls[*Index]);
	}
	return nullptr;
}

bool FControlRigPoseMirrorTable::IsMatched(const UControlRig* ControlRig,const FName& Name) 
{
	SetUpMirrorTable(ControlRig);

	if (FMatchedControls* CRMatchedControls = FControlRigPoseMirrorTable::MatchedControls.Find(ControlRig))
	{
		if (CRMatchedControls->Match.Num() > 0)
		{
			if (const FName* MatchedName = CRMatchedControls->Match.Find(Name))
			{
				return true;
			}
		}
	}
	return false;
}

void FControlRigPoseMirrorTable::GetMirrorTransform(const FRigControlCopy& ControlCopy, bool bDoLocal, bool bIsMatched, FTransform& OutGlobalTransform,
	FTransform& OutLocalTransform) const
{
	FControlRigPoseMirrorContext Context;
	Context.bDoLocal = bDoLocal;
	Context.bIsMatched = bIsMatched;
	GetMirrorTransform(ControlCopy, Context, OutGlobalTransform, OutLocalTransform);
}

void FControlRigPoseMirrorTable::GetMirrorTransform(const FRigControlCopy& ControlCopy, const FControlRigPoseMirrorContext& InContext, FTransform& OutGlobalTransform, FTransform& OutLocalTransform) const
{
	const UControlRigPoseMirrorSettings* Settings = GetDefault<UControlRigPoseMirrorSettings>();
	FTransform GlobalTransform = ControlCopy.GlobalTransform;
	OutGlobalTransform = GlobalTransform;
	FTransform LocalTransform = ControlCopy.LocalTransform;
	OutLocalTransform = LocalTransform;
	if (Settings)
	{
		if (!InContext.bIsMatched) //still do it if not a matching, need to mirror translation
		{
			FRigVMMirrorSettings MirrorSettings;
			MirrorSettings.MirrorAxis = Settings->MirrorAxis;
			MirrorSettings.AxisToFlip = Settings->AxisToFlip;
			FTransform NewTransform = MirrorSettings.MirrorTransform(GlobalTransform);
			OutGlobalTransform.SetTranslation(NewTransform.GetTranslation());
			OutGlobalTransform.SetRotation(NewTransform.GetRotation());

			FTransform NewLocalTransform = MirrorSettings.MirrorTransform(LocalTransform);
			OutLocalTransform.SetTranslation(NewLocalTransform.GetTranslation());
			OutLocalTransform.SetRotation(NewLocalTransform.GetRotation());
		}
		else
		{
			const FVector Identity(1.0, 1.0, 1.0);
			if (InContext.TranslationFactor.Equals(Identity, KINDA_SMALL_NUMBER) == false)
			{
				FVector Translate = GlobalTransform.GetTranslation();
				Translate *= InContext.TranslationFactor;
				OutGlobalTransform.SetTranslation(Translate);
				Translate = LocalTransform.GetTranslation();
				Translate *= InContext.TranslationFactor;
				OutLocalTransform.SetTranslation(Translate);
			}
			if (InContext.RotationFactor.Equals(Identity, KINDA_SMALL_NUMBER) == false)
			{
				FVector Euler = GlobalTransform.GetRotation().Euler();
				Euler *= InContext.RotationFactor;
				FQuat Quat;
				Quat = Quat.MakeFromEuler(Euler);
				OutGlobalTransform.SetRotation(Quat);

				Euler = LocalTransform.GetRotation().Euler();
				Euler *= InContext.RotationFactor;
				Quat = Quat.MakeFromEuler(Euler);
				OutLocalTransform.SetRotation(Quat);
			}
		}
	}
}

