// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tools/ControlRigPoseProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigPoseProjectSettings)

UControlRigPoseProjectSettings::UControlRigPoseProjectSettings()
{
	FDirectoryPath RootSaveDir;
 	RootSaveDir.Path = TEXT("ControlRig/Pose");
	RootSaveDirs.Add(RootSaveDir);
}

TArray<FString> UControlRigPoseProjectSettings::GetAssetPaths() const
{
	TArray<FString> Paths;
	for (const FDirectoryPath& Path : RootSaveDirs)
	{
		Paths.Add(Path.Path);
	}
	return Paths;
}

