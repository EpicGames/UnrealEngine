// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigVariableMappings.h"
#include "ControlRig.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimCurveUtils.h"
#include "Algo/ForEach.h"

namespace AnimNodeLocals
{

const TCHAR* UseFunctionsName = TEXT("ControlRig.AnimNode.UseFunctions");

static bool bUseFunctions = true;
static FAutoConsoleVariableRef CVarUseCachedFunctions(
   UseFunctionsName,
   bUseFunctions,
   TEXT("Cache and use propagate functions.")
   );

template<typename ValueType, typename HierarchyValueType = ValueType>
void SetControlValue(FRigControlElement* InControlElement, FProperty* InSourceProperty, URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
{
	const ValueType* ValuePtr = InSourceProperty->ContainerPtrToValuePtr<ValueType>(InSourceInstance);
	const FRigControlValue Value = FRigControlValue::Make<HierarchyValueType>(*ValuePtr);
	InTargetHierarchy->SetControlValue(InControlElement, Value, ERigControlValueType::Current);
}

template<typename ValueType, typename HierarchyValueType = ValueType>
void SetControlValue(FRigControlElement* InControlElement, const uint8* ValuePtr, URigHierarchy* InTargetHierarchy)
{
	const FRigControlValue Value = FRigControlValue::Make<HierarchyValueType>(*((const ValueType*)ValuePtr));
	InTargetHierarchy->SetControlValue(InControlElement, Value, ERigControlValueType::Current);
}

template<typename ValueType, typename VariableValueType = ValueType>
void SetVariableValue(FRigVMExternalVariable& InVariable, FProperty* InSourceProperty, const UObject* InSourceInstance)
{
	const ValueType* ValuePtr = InSourceProperty->ContainerPtrToValuePtr<ValueType>(InSourceInstance);
	InVariable.SetValue<VariableValueType>(*ValuePtr);
}

template<typename ValueType, typename VariableValueType = ValueType>
void SetVariableValue(FRigVMExternalVariable& InVariable, const uint8* InValuePtr)
{
	const ValueType* ValuePtr = (const ValueType*)InValuePtr;
	InVariable.SetValue<VariableValueType>(*ValuePtr);
}

} // end namespace AnimNodeLocals


void FControlRigVariableMappings::InitializeProperties(const UClass* InSourceClass, UObject* TargetInstance,
	FControlRigAssetStrongReference InTargetClass, const TArray<FName>& InSourcePropertyNames, TArray<FName>& InDestPropertyNames)
{
	const bool bUseFunctions = AnimNodeLocals::bUseFunctions;

	// Build property lists
	SourceProperties.Reset(InSourcePropertyNames.Num());
	DestProperties.Reset(InSourcePropertyNames.Num());
	UpdateFunctions.Reset(InSourcePropertyNames.Num());
	Variables.Reset();

	check(InSourcePropertyNames.Num() == InDestPropertyNames.Num());

	UControlRig* TargetControlRig = Cast<UControlRig>(TargetInstance);
	URigHierarchy* TargetHierarchy = TargetControlRig ? TargetControlRig->GetHierarchy() : nullptr;

	
	for (int32 Idx = 0; Idx < InSourcePropertyNames.Num(); ++Idx)
	{
		const FName& SourceName = InSourcePropertyNames[Idx];

		FProperty* SourceProperty = FindFProperty<FProperty>(InSourceClass, SourceName);
		SourceProperties.Add(SourceProperty);
		DestProperties.Add(nullptr);

		if (bUseFunctions)
		{
			if (TargetControlRig && TargetHierarchy && SourceProperty)
			{
				const FName& DestPropertyName = InDestPropertyNames[Idx];
				if (FRigControlElement* ControlElement = TargetControlRig->FindControl(DestPropertyName))
				{
					AddControlFunction(ControlElement, SourceProperty, TargetHierarchy);
				}
				else
				{
					const FRigVMExternalVariable Variable = TargetControlRig->GetPublicVariableByName(DestPropertyName);
					if (Variable.IsValid() && !Variable.IsReadOnly())
					{
						AddVariableFunction(Variable, SourceProperty);
					}
				}
			}
		}
	}
}

void FControlRigVariableMappings::InitializeProperties(const UClass* InSourceClass
                                                       , UObject* TargetInstance
                                                       , UClass* InTargetClass
                                                       , const TArray<FName>& InSourcePropertyNames
                                                       , TArray<FName>& InDestPropertyNames)
{
	FControlRigAssetStrongReference TargetSource;
	TargetSource.Set(TSubclassOf<UControlRig>(InTargetClass));
	InitializeProperties(InSourceClass, TargetInstance, TargetSource, InSourcePropertyNames, InDestPropertyNames);
}

bool FControlRigVariableMappings::RequiresInitAfterConstruction() const
{
	const bool bUseFunctions = AnimNodeLocals::bUseFunctions;

	return bUseFunctions && UpdateFunctions.Num() > 0;
}

void FControlRigVariableMappings::InitializeCustomProperties(UControlRig* TargetControlRig, const FCustomPropertyMappings& CustomPropertyMapping)
{
	const TArray<FCustomPropertyData>& Mappings = CustomPropertyMapping.GetMappings();

	ResetCustomProperties(Mappings.Num());

	if (TargetControlRig != nullptr)
	{
		URigHierarchy* TargetHierarchy = TargetControlRig->GetHierarchy();

		for (const FCustomPropertyData& Mapping : Mappings)
		{
			switch (Mapping.Type)
			{
				case FCustomPropertyData::ECustomPropertyType::Variable:
				{
					const FRigVMExternalVariable Variable = TargetControlRig->GetPublicVariableByName(Mapping.TargetName);
					if (Variable.IsValid() && !Variable.IsReadOnly())
					{
						AddCustomVariableFunction(Variable, Mapping.Property, Mapping.SourceMemory);
					}
					break;
				}
				case FCustomPropertyData::ECustomPropertyType::Control:
				{
					if (FRigControlElement* ControlElement = TargetControlRig->FindControl(Mapping.TargetName))
					{
						AddCustomControlFunction(ControlElement, Mapping.ControlType, Mapping.SourceMemory, TargetHierarchy);
					}
					break;
				}
				default:
				{
					ensureMsgf(false, TEXT("Unsupported or Invalid Custom Property Type"));
					break;
				}
			}
		}
	}
}

void FControlRigVariableMappings::ResetCustomProperties(int32 NewSize)
{
	CustomUpdateFunctions.Reset(NewSize);
}

void FControlRigVariableMappings::PropagateInputProperties(const UObject* InSourceInstance, UControlRig* InTargetControlRig, TArray<FName>& InDestPropertyNames)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(FControlRigVariableMappings_PropagateInputProperties);

	if (!InSourceInstance)
	{
		return;
	}

	if (!InTargetControlRig)
	{
		return;
	}

	URigHierarchy* TargetHierarchy = InTargetControlRig->GetHierarchy();
	if (!TargetHierarchy)
	{
		return;
	}

	if (AnimNodeLocals::bUseFunctions && !UpdateFunctions.IsEmpty())
	{
		Algo::ForEach(UpdateFunctions, [TargetHierarchy, InSourceInstance](const PropertyUpdateFunction& InFunc)
			{
				InFunc(TargetHierarchy, InSourceInstance);
			});
	}
	else
	{
		PropagateInputPropertiesNoCache(InSourceInstance, InTargetControlRig, TargetHierarchy, InDestPropertyNames);
	}
}

void FControlRigVariableMappings::PropagateCustomInputProperties(UControlRig* InTargetControlRig)
{
	if (!InTargetControlRig)
	{
		return;
	}
	
	URigHierarchy* TargetHierarchy = InTargetControlRig->GetHierarchy();
	if (!TargetHierarchy)
	{
		return;
	}

	if (!CustomUpdateFunctions.IsEmpty())
	{
		Algo::ForEach(CustomUpdateFunctions, [](const CustomPropertyUpdateFunction& InFunc)
			{
				InFunc();
			});
	}
}

void FControlRigVariableMappings::UpdateCurveInputs(UControlRig* InControlRig, const TMap<FName, FName> &InputMapping, const FBlendedCurve& InCurveData) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// now go through variable mapping table and see if anything is mapping through input
	if (InputMapping.Num() > 0 && InControlRig)
	{
		UE::Anim::FCurveUtils::BulkGet(InCurveData, InputCurveMappings,
			[&InControlRig](const FControlRigCurveMapping& InBulkElement, float InValue)
			{
				FRigVMExternalVariable Variable = InControlRig->GetPublicVariableByName(InBulkElement.SourceName);
				if (!Variable.IsReadOnly() && Variable.GetExtendedCPPType() == TEXT("float"))
				{
					Variable.SetValue<float>(InValue);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "[%ls] Missing Input Variable [%ls]", *GetNameSafe(InControlRig->GetClass()), *InBulkElement.SourceName.ToString());
				}
			});
	}
}

void FControlRigVariableMappings::UpdateCurveOutputs(UControlRig* InControlRig, const TMap<FName, FName> &OutputMapping, FBlendedCurve& InCurveData) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (OutputMapping.Num() > 0 && InControlRig)
	{
		UE::Anim::FCurveUtils::BulkSet(InCurveData, OutputCurveMappings,
			[&InControlRig](const FControlRigCurveMapping& InBulkElement) -> float
			{
				FRigVMExternalVariable Variable = InControlRig->GetPublicVariableByName(InBulkElement.SourceName);
				if (Variable.GetExtendedCPPType() == TEXT("float"))
				{
					return Variable.GetValue<float>();
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "[%ls] Missing Output Variable [%ls]", *GetNameSafe(InControlRig->GetClass()), *InBulkElement.SourceName.ToString());
				}

				return 0.0f;
			});
	}
}

void FControlRigVariableMappings::PropagateInputPropertiesNoCache(const UObject* InSourceInstance, const UControlRig* InTargetControlRig, URigHierarchy* InTargetHierarchy, TArray<FName>& InDestPropertyNames)
{
	check(SourceProperties.Num() == DestProperties.Num());

	for (int32 PropIdx = 0; PropIdx < SourceProperties.Num(); ++PropIdx)
	{
		FProperty* CallerProperty = SourceProperties[PropIdx];

		if (FRigControlElement* ControlElement = InTargetControlRig->FindControl(InDestPropertyNames[PropIdx]))
		{
			const uint8* SrcPtr = CallerProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);

			bool bIsValid = false;
			FRigControlValue Value;
			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Bool:
			{
				if (ensure(CastField<FBoolProperty>(CallerProperty)))
				{
					Value = FRigControlValue::Make<bool>(*(bool*)SrcPtr);
					bIsValid = true;
				}
				break;
			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				if (ensure(CastField<FFloatProperty>(CallerProperty)))
				{
					Value = FRigControlValue::Make<float>(*(float*)SrcPtr);
					bIsValid = true;
				}
				break;
			}
			case ERigControlType::Integer:
			{
				if (ensure(CastField<FIntProperty>(CallerProperty)))
				{
					Value = FRigControlValue::Make<int32>(*(int32*)SrcPtr);
					bIsValid = true;
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty);
				if (ensure(StructProperty))
				{
					if (ensure(StructProperty->Struct == TBaseStructure<FVector2D>::Get()))
					{
						const FVector2D& SrcVector = *(FVector2D*)SrcPtr;
						Value = FRigControlValue::Make<FVector2D>(SrcVector);
						bIsValid = true;
					}
				}
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Scale:
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty);
				if (ensure(StructProperty))
				{
					if (ensure(StructProperty->Struct == TBaseStructure<FVector>::Get()))
					{
						const FVector& SrcVector = *(FVector*)SrcPtr;
						Value = FRigControlValue::Make<FVector>(SrcVector);
						bIsValid = true;
					}
				}
				break;
			}
			case ERigControlType::Rotator:
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty);
				if (ensure(StructProperty))
				{
					if (ensure(StructProperty->Struct == TBaseStructure<FRotator>::Get()))
					{
						const FRotator& SrcRotator = *(FRotator*)SrcPtr;
						Value = FRigControlValue::Make<FRotator>(SrcRotator);
						bIsValid = true;
					}
				}
				break;
			}
			case ERigControlType::Transform:
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty);
				if (ensure(StructProperty))
				{
					if (ensure(StructProperty->Struct == TBaseStructure<FTransform>::Get()))
					{
						const FTransform& SrcTransform = *(FTransform*)SrcPtr;
						Value = FRigControlValue::Make<FTransform>(SrcTransform);
						bIsValid = true;
					}
				}
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty);
				if (ensure(StructProperty))
				{
					if (ensure(StructProperty->Struct == TBaseStructure<FTransform>::Get()))
					{
						const FTransform& SrcTransform = *(FTransform*)SrcPtr;
						Value = FRigControlValue::Make<FTransformNoScale>(SrcTransform);
						bIsValid = true;
					}
				}
				break;
			}
			case ERigControlType::EulerTransform:
			{
				FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty);
				if (ensure(StructProperty))
				{
					if (ensure(StructProperty->Struct == TBaseStructure<FTransform>::Get()))
					{
						const FTransform& SrcTransform = *(FTransform*)SrcPtr;
						Value = FRigControlValue::Make<FEulerTransform>(FEulerTransform(SrcTransform));
						bIsValid = true;
					}
				}
				break;
			}
			default:
			{
				checkNoEntry();
			}
			}

			if (bIsValid)
			{
				InTargetHierarchy->SetControlValue(ControlElement, Value, ERigControlValueType::Current);
			}
			continue;
		}

		FRigVMExternalVariable Variable = InTargetControlRig->GetPublicVariableByName(InDestPropertyNames[PropIdx]);
		if (Variable.IsValid())
		{
			if (Variable.IsReadOnly())
			{
				continue;
			}

			const uint8* SrcPtr = CallerProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);

			if (CastField<FBoolProperty>(CallerProperty) != nullptr && Variable.GetExtendedCPPType() == RigVMTypeUtils::BoolTypeName)
			{
				const bool Value = *(const bool*)SrcPtr;
				Variable.SetValue<bool>(Value);
			}
			else if (CastField<FFloatProperty>(CallerProperty) != nullptr && (Variable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName || Variable.GetExtendedCPPType() == RigVMTypeUtils::DoubleTypeName))
			{
				const float Value = *(const float*)SrcPtr;
				if (Variable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName)
				{
					Variable.SetValue<float>(Value);
				}
				else
				{
					Variable.SetValue<double>(Value);
				}
			}
			else if (CastField<FDoubleProperty>(CallerProperty) != nullptr && (Variable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName || Variable.GetExtendedCPPType() == RigVMTypeUtils::DoubleTypeName))
			{
				const double Value = *(const double*)SrcPtr;
				if (Variable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName)
				{
					Variable.SetValue<float>((float)Value);
				}
				else
				{
					Variable.SetValue<double>(Value);
				}
			}
			else if (CastField<FIntProperty>(CallerProperty) != nullptr && Variable.GetExtendedCPPType() == RigVMTypeUtils::Int32TypeName)
			{
				const int32 Value = *(const int32*)SrcPtr;
				Variable.SetValue<int32>(Value);
			}
			else if (CastField<FNameProperty>(CallerProperty) != nullptr && Variable.GetExtendedCPPType() == RigVMTypeUtils::FNameTypeName)
			{
				const FName Value = *(const FName*)SrcPtr;
				Variable.SetValue<FName>(Value);
			}
			else if (CastField<FNameProperty>(CallerProperty) != nullptr && Variable.GetExtendedCPPType() == RigVMTypeUtils::FStringTypeName)
			{
				const FString Value = *(const FString*)SrcPtr;
				Variable.SetValue<FString>(Value);
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(CallerProperty))
			{
				if (StructProperty->Struct == Variable.GetCPPTypeObject())
				{
					StructProperty->Struct->CopyScriptStruct(Variable.GetMemory(), SrcPtr, 1);
				}
			}
			else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CallerProperty))
			{
				if (ensure(ArrayProperty->SameType(Variable.GetProperty())))
				{
					ArrayProperty->CopyCompleteValue(Variable.GetMemory(), SrcPtr);
				}
			}
			else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(CallerProperty))
			{
				if (ensure(ObjectProperty->SameType(Variable.GetProperty())))
				{
					ObjectProperty->CopyCompleteValue(Variable.GetMemory(), SrcPtr);
				}
			}
			else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(CallerProperty))
			{
				if (ensure(EnumProperty->SameType(Variable.GetProperty())))
				{
					EnumProperty->CopyCompleteValue(Variable.GetMemory(), SrcPtr);
				}
			}
			else if (FByteProperty* ByteProperty = CastField<FByteProperty>(CallerProperty))
			{
				if (ensure(ByteProperty->SameType(Variable.GetProperty())))
				{
					ByteProperty->CopyCompleteValue(Variable.GetMemory(), SrcPtr);
				}
			}
			else
			{
				ensureMsgf(false, TEXT("Property %s type %s not recognized"), *CallerProperty->GetName(), *CallerProperty->GetCPPType());
			}
		}
	}
}

void FControlRigVariableMappings::AddControlFunction(FRigControlElement* InControlElement, FProperty* InSourceProperty, URigHierarchy* InTargetHierarchy)
{
	using namespace AnimNodeLocals;

	switch (InControlElement->Settings.ControlType)
	{
		case ERigControlType::Bool:
		{
			if (ensure(CastField<FBoolProperty>(InSourceProperty)))
			{
				return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
					{
						if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
						{
							if (BaseElement->GetKey() == ElementKey)
							{
								SetControlValue<bool>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
							}
						}
					});
			}
			break;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			if (ensure(CastField<FFloatProperty>(InSourceProperty)))
			{
				return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
					{
						if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
						{
							if (BaseElement->GetKey() == ElementKey)
							{
								SetControlValue<float>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
							}
						}
					});
			}
			break;
		}
		case ERigControlType::Integer:
		{
			if (ensure(CastField<FIntProperty>(InSourceProperty)))
			{
				return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
					{
						if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
						{
							if (BaseElement->GetKey() == ElementKey)
							{
								SetControlValue<int32>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
							}
						}
					});
			}
			break;
		}
		case ERigControlType::Vector2D:
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty);
			if (ensure(StructProperty))
			{
				if (ensure(StructProperty->Struct == TBaseStructure<FVector2D>::Get()))
				{
					return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
						{
							if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
							{
								if (BaseElement->GetKey() == ElementKey)
								{
									SetControlValue<FVector2D>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
								}
							}
						});
				}
			}
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty);
			if (ensure(StructProperty))
			{
				if (ensure(StructProperty->Struct == TBaseStructure<FVector>::Get()))
				{
					return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
						{
							if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
							{
								if (BaseElement->GetKey() == ElementKey)
								{
									SetControlValue<FVector>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
								}
							}
						});
				}
			}
			break;
		}
		case ERigControlType::Rotator:
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty);
			if (ensure(StructProperty))
			{
				if (ensure(StructProperty->Struct == TBaseStructure<FRotator>::Get()))
				{
					return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
						{
							if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
							{
								if (BaseElement->GetKey() == ElementKey)
								{
									SetControlValue<FRotator>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
								}
							}
						});
				}
			}
			break;
		}
		case ERigControlType::Transform:
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty);
			if (ensure(StructProperty))
			{
				if (ensure(StructProperty->Struct == TBaseStructure<FTransform>::Get()))
				{
					return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
						{
							if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
							{
								if (BaseElement->GetKey() == ElementKey)
								{
									SetControlValue<FTransform>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
								}
							}
						});
				}
			}
			break;
		}
		case ERigControlType::TransformNoScale:
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty);
			if (ensure(StructProperty))
			{
				if (ensure(StructProperty->Struct == TBaseStructure<FTransform>::Get()))
				{
					return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
						{
							if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
							{
								if (BaseElement->GetKey() == ElementKey)
								{
									SetControlValue<FTransform, FTransformNoScale>(CastChecked<FRigControlElement>(BaseElement), InSourceProperty, InTargetHierarchy, InSourceInstance);
								}
							}
						});
				}
			}
			break;
		}
		case ERigControlType::EulerTransform:
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty);
			if (ensure(StructProperty))
			{
				if (ensure(StructProperty->Struct == TBaseStructure<FTransform>::Get()))
				{
					return AddUpdateFunction([InSourceProperty, ElementKey = InControlElement->GetKey(), ElementIndex = InControlElement->GetIndex()](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
						{
							if (FRigBaseElement* BaseElement = InTargetHierarchy->Get(ElementIndex))
							{
								if (BaseElement->GetKey() == ElementKey)
								{
									const FTransform* ValuePtr = InSourceProperty->ContainerPtrToValuePtr<FTransform>(InSourceInstance);
									const FRigControlValue Value = FRigControlValue::Make<FEulerTransform>(FEulerTransform(*ValuePtr));
									InTargetHierarchy->SetControlValue(CastChecked<FRigControlElement>(BaseElement), Value, ERigControlValueType::Current);
								}
							}
						});
				}
			}
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}
}

void FControlRigVariableMappings::AddUpdateFunction(PropertyUpdateFunction&& InFunction)
{
	UpdateFunctions.Add(MoveTemp(InFunction));
}

void FControlRigVariableMappings::AddCustomUpdateFunction(CustomPropertyUpdateFunction&& InFunction)
{
	CustomUpdateFunctions.Add(MoveTemp(InFunction));
}

void FControlRigVariableMappings::AddVariableFunction(const FRigVMExternalVariable& InVariable, FProperty* InSourceProperty)
{
	using namespace AnimNodeLocals;
	
	const int32 VariableIndex = Variables.Add(InVariable);
	
	if (CastField<FBoolProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::BoolTypeName)
	{
		return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
		{
			SetVariableValue<bool>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
		});
	}
	else if (CastField<FFloatProperty>(InSourceProperty) && (InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName || InVariable.GetExtendedCPPType() == RigVMTypeUtils::DoubleTypeName))
	{
		if(InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName)
		{
			return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				SetVariableValue<float>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
			});
		}
		else
		{
			return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				SetVariableValue<float, double>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
			});
		}
	}
	else if (CastField<FDoubleProperty>(InSourceProperty) && (InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName || InVariable.GetExtendedCPPType() == RigVMTypeUtils::DoubleTypeName))
	{
		if(InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName)
		{
			return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				SetVariableValue<double, float>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
			});
		}
		else
		{
			return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				SetVariableValue<double>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
			});
		}
	}
	else if (CastField<FIntProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::Int32TypeName)
	{
		return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
		{
			SetVariableValue<int32>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
		});
	}
	else if (CastField<FNameProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::FNameTypeName)
	{
		return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
		{
			SetVariableValue<FName>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
		});
	}
	else if (CastField<FNameProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::FStringTypeName)
	{
		return AddUpdateFunction([InSourceProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
		{
			SetVariableValue<FString>(Variables[VariableIndex], InSourceProperty, InSourceInstance);
		});
	}
	else if (FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty))
	{
		return AddUpdateFunction([StructProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
		{
			const uint8* SrcPtr = StructProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);
			StructProperty->Struct->CopyScriptStruct(Variables[VariableIndex].GetMemory(), SrcPtr, 1);
		});
	}
	else if(FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InSourceProperty))
	{
		if(ensure(ArrayProperty->SameType(InVariable.GetProperty())))
		{
			return AddUpdateFunction([ArrayProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				const uint8* SrcPtr = ArrayProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);
				ArrayProperty->CopyCompleteValue(Variables[VariableIndex].GetMemory(), SrcPtr);
			});
		}
	}
	else if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InSourceProperty))
	{
		if(ensure(ObjectProperty->SameType(InVariable.GetProperty())))
		{
			return AddUpdateFunction([ObjectProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				const uint8* SrcPtr = ObjectProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);
				ObjectProperty->CopyCompleteValue(Variables[VariableIndex].GetMemory(), SrcPtr);
			});
		}
	}
	else if(FEnumProperty* EnumProperty = CastField<FEnumProperty>(InSourceProperty))
	{
		if(ensure(EnumProperty->SameType(InVariable.GetProperty())))
		{
			return AddUpdateFunction([EnumProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
			{
				const uint8* SrcPtr = EnumProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);
				EnumProperty->CopyCompleteValue(Variables[VariableIndex].GetMemory(), SrcPtr);
			});
		}
	}
	else if (FByteProperty* ByteProperty = CastField<FByteProperty>(InSourceProperty))
	{
		if (ensure(ByteProperty->SameType(InVariable.GetProperty())))
		{
			return AddUpdateFunction([ByteProperty, this, VariableIndex](URigHierarchy* InTargetHierarchy, const UObject* InSourceInstance)
				{
					const uint8* SrcPtr = ByteProperty->ContainerPtrToValuePtr<uint8>(InSourceInstance);
					ByteProperty->CopyCompleteValue(Variables[VariableIndex].GetMemory(), SrcPtr);
				});
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Property %s type %s not recognized"), *InSourceProperty->GetName(), *InSourceProperty->GetCPPType());
	}
}

void FControlRigVariableMappings::AddCustomControlFunction(FRigControlElement* InControlElement, ERigControlType InControlType, const uint8* InSourcePropertyMemory, URigHierarchy* InTargetHierarchy)
{
	using namespace AnimNodeLocals;

	if (!ensure(InControlType == InControlElement->Settings.ControlType))
	{
		return;
	}

	switch (InControlElement->Settings.ControlType)
	{
		case ERigControlType::Bool:
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<bool>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
			break;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<float>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
			break;
		}
		case ERigControlType::Integer:
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<int32>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
			break;
		}
		case ERigControlType::Vector2D:
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<FVector2D>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			return AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<FVector>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
		}
		case ERigControlType::Rotator:
		{
			return AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<FRotator>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
		}
		case ERigControlType::Transform:
		{
			return AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<FTransform>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
		}
		case ERigControlType::TransformNoScale:
		{
			return AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					SetControlValue<FTransform, FTransformNoScale>(InControlElement, InSourcePropertyMemory, InTargetHierarchy);
				});
		}
		case ERigControlType::EulerTransform:
		{
			return AddCustomUpdateFunction([InSourcePropertyMemory, InTargetHierarchy, InControlElement]()
				{
					const FTransform* ValuePtr = (const FTransform*)InSourcePropertyMemory;
					const FRigControlValue Value = FRigControlValue::Make<FEulerTransform>(FEulerTransform(*ValuePtr));
					InTargetHierarchy->SetControlValue(InControlElement, Value, ERigControlValueType::Current);
				});
		}
		default:
		{
			checkNoEntry();
		}
	}
}

void FControlRigVariableMappings::AddCustomVariableFunction(const FRigVMExternalVariable& InVariable, const FProperty* InSourceProperty, const uint8* InSourcePropertyMemory)
{
	using namespace AnimNodeLocals;

	if (CastField<FBoolProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::BoolTypeName)
	{
		AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
			{
				SetVariableValue<bool>(Variable, InSourcePropertyMemory);
			});
	}
	else if (CastField<FFloatProperty>(InSourceProperty) && (InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName || InVariable.GetExtendedCPPType() == RigVMTypeUtils::DoubleTypeName))
	{
		if (InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName)
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
				{
					SetVariableValue<float>(Variable, InSourcePropertyMemory);
				});
		}
		else
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
				{
					SetVariableValue<float, double>(Variable, InSourcePropertyMemory);
				});
		}
	}
	else if (CastField<FDoubleProperty>(InSourceProperty) && (InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName || InVariable.GetExtendedCPPType() == RigVMTypeUtils::DoubleTypeName))
	{
		if (InVariable.GetExtendedCPPType() == RigVMTypeUtils::FloatTypeName)
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
				{
					SetVariableValue<double, float>(Variable, InSourcePropertyMemory);
				});
		}
		else
		{
			AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
				{
					SetVariableValue<double>(Variable, InSourcePropertyMemory);
				});
		}
	}
	else if (CastField<FIntProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::Int32TypeName)
	{
		AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
			{
				SetVariableValue<int32>(Variable, InSourcePropertyMemory);
			});
	}
	else if (CastField<FNameProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::FNameTypeName)
	{
		AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
			{
				SetVariableValue<FName>(Variable, InSourcePropertyMemory);
			});
	}
	else if (CastField<FNameProperty>(InSourceProperty) && InVariable.GetExtendedCPPType() == RigVMTypeUtils::FStringTypeName)
	{
		AddCustomUpdateFunction([InSourcePropertyMemory, Variable = InVariable]() mutable
			{
				SetVariableValue<FString>(Variable, InSourcePropertyMemory);
			});
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty))
	{
		AddCustomUpdateFunction([StructProperty, InSourcePropertyMemory, Variable = InVariable]()
			{
				StructProperty->Struct->CopyScriptStruct(const_cast<uint8*>(Variable.GetMemory()), InSourcePropertyMemory, 1);
			});
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InSourceProperty))
	{
		if (ensure(ArrayProperty->SameType(InVariable.GetProperty())))
		{
			AddCustomUpdateFunction([ArrayProperty, InSourcePropertyMemory, Variable = InVariable]()
				{
					ArrayProperty->CopyCompleteValue(const_cast<uint8*>(Variable.GetMemory()), InSourcePropertyMemory);
				});
		}
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InSourceProperty))
	{
		if (ensure(ObjectProperty->SameType(InVariable.GetProperty())))
		{
			AddCustomUpdateFunction([ObjectProperty, InSourcePropertyMemory, Variable = InVariable]()
				{
					ObjectProperty->CopyCompleteValue(const_cast<uint8*>(Variable.GetMemory()), InSourcePropertyMemory);
				});
		}
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InSourceProperty))
	{
		if (ensure(EnumProperty->SameType(InVariable.GetProperty())))
		{
			AddCustomUpdateFunction([EnumProperty, InSourcePropertyMemory, Variable = InVariable]()
				{
					EnumProperty->CopyCompleteValue(const_cast<uint8*>(Variable.GetMemory()), InSourcePropertyMemory);
				});
		}
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InSourceProperty))
	{
		if (ensure(ByteProperty->SameType(InVariable.GetProperty())))
		{
			AddCustomUpdateFunction([ByteProperty, InSourcePropertyMemory, Variable = InVariable]()
				{
					ByteProperty->CopyCompleteValue(const_cast<uint8*>(Variable.GetMemory()), InSourcePropertyMemory);
				});
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Property %s type %s not recognized"), *InSourceProperty->GetName(), *InSourceProperty->GetCPPType());
	}
}

void FControlRigVariableMappings::CacheCurveMappings(const TMap<FName, FName>& InInputMapping, const TMap<FName, FName>& InOutputMapping, URigHierarchy* InHierarchy)
{
	CacheCurveMappings(InInputMapping, InputCurveMappings, InHierarchy);
	CacheCurveMappings(InOutputMapping, OutputCurveMappings, InHierarchy);
}

void FControlRigVariableMappings::CacheCurveMappings(const TMap<FName, FName>& InMapping, FControlRigVariableMappings::FCurveMappings& InCurveMappings,	URigHierarchy* InHierarchy)
{
	for (auto Iter = InMapping.CreateConstIterator(); Iter; ++Iter)
	{
		// we need to have list of variables using pin
		const FName SourcePath = Iter.Key();
		const FName TargetPath = Iter.Value();

		if (SourcePath != NAME_None && TargetPath != NAME_None)
		{
			InCurveMappings.Add(SourcePath, TargetPath);

			if (InHierarchy)
			{
				const FRigElementKey Key(TargetPath, ERigElementType::Control);
				if (const FRigControlElement* ControlElement = InHierarchy->Find<FRigControlElement>(Key))
				{
					CurveInputToControlIndex.Add(TargetPath, ControlElement->GetIndex());
					continue;
				}
			}
		}

		// @todo: should we clear the item if not found?
	}
}
