// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tools/MirrorCalculator.h"

using namespace UE::AIE;

bool FMirrorCalculator::FindMirroredItems(const TArray<FMirrorItem>& Items, FMirrorItemResults& OutMirrorItemResults,
	TEnumAsByte<EAxis::Type> MirrorAxis, double AxisLocation, double Tolerance)
{

	int32 MirrorIndex = 0;
	int32 OtherIndex[2] = { 1,2 };
	if (MirrorAxis == EAxis::Y)
	{
		MirrorIndex = 1;
		OtherIndex[0] = 0;
		OtherIndex[1] = 2;
	}
	else if (MirrorAxis == EAxis::Z)
	{
		MirrorIndex = 2;
		OtherIndex[0] = 0;
		OtherIndex[1] = 1;
	}

	//items found on first pass less than and greater than the Axis Location, those on the Axis will go straight into
	TArray<int32> ItemsLess;
	TArray<int32> ItemsMore;
	
	for (int32 Index = 0; Index < Items.Num(); ++Index)
	{
		const FMirrorItem& Item = Items[Index];
		if (FMath::IsNearlyZero(Item.Location[MirrorIndex] - AxisLocation, Tolerance))
		{
			OutMirrorItemResults.ExactlyOnAxis.Add(Index);
		}
		else if (Item.Location[MirrorIndex] < AxisLocation)
		{
			ItemsLess.Add(Index);
		}
		else // > AxisLocation
		{
			ItemsMore.Add(Index);
		}
	}

	for (const int32 LessIndex : ItemsLess)
	{

		const FMirrorItem& LessItem = Items[LessIndex];
		bool bWasFound = false;
		for (const int32 MoreIndex : ItemsMore)
		{
			const FMirrorItem& MoreItem = Items[MoreIndex];
				
			if (IsMirror(LessItem, MoreItem, AxisLocation, Tolerance, MirrorIndex, OtherIndex))
			{
				OutMirrorItemResults.MirroredItems.Add(LessIndex, MoreIndex);
				OutMirrorItemResults.MirroredItems.Add(MoreIndex, LessIndex);
				ItemsMore.Remove(MoreIndex);
				bWasFound = true;
				break;
			}
		}
		if (bWasFound == false)
		{
			OutMirrorItemResults.NotOnAxis.Add(LessIndex);
		}
	}
	//if any MoreItems left they weren't mirrored 
	for (int32 MoreIndex : ItemsMore)
	{
		OutMirrorItemResults.NotOnAxis.Add(MoreIndex);
	}
	return (OutMirrorItemResults.MirroredItems.Num() > 0);
	
}
