// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/ComposedSampleTrack.h"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FComposedSampleTrack
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template<>
TArray<FSampleTrackBase::ETrackType> FComposedSampleTrack<FTransform3f>::GetChildTrackTypes() const
{
	return {
		ETrackType_Vector3f,
		ETrackType_Quatf,
		ETrackType_Vector3f
	};
}

template<>
FString FComposedSampleTrack<FTransform3f>::GetChildTrackNameSuffix(int32 InChildTrackIndex) const
{
	switch(InChildTrackIndex)
	{
		case 0:
		{
			return TEXT("Location");
		}
		case 1:
		{
			return TEXT("Rotation");
		}
		case 2:
		{
			return TEXT("Scale");
		}
		default:
		{
			break;
		}
	}
	return FSampleTrack<FTransform3f>::GetChildTrackNameSuffix(InChildTrackIndex);
}

template<>
FTransform3f FComposedSampleTrack<FTransform3f>::GetValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const
{
	check(ChildTracks.Num() == 3);

	const FSampleTrack<FVector3f>* LocationTrack = static_cast<const FSampleTrack<FVector3f>*>(ChildTracks[0].Get());
	const FSampleTrack<FQuat4f>* RotationTrack = static_cast<const FSampleTrack<FQuat4f>*>(ChildTracks[1].Get());
	const FSampleTrack<FVector3f>* ScaleTrack = static_cast<const FSampleTrack<FVector3f>*>(ChildTracks[2].Get());

	FTransform3f Result = FTransform3f::Identity;

	Result.SetLocation(LocationTrack->GetValueAtTimeIndex(InTimeIndex, InOutIndex));
	Result.SetRotation(RotationTrack->GetValueAtTimeIndex(InTimeIndex, InOutIndex));
	Result.SetScale3D(ScaleTrack->GetValueAtTimeIndex(InTimeIndex, InOutIndex));
	return Result;
}
	
template<>
void FComposedSampleTrack<FTransform3f>::AddSample(const FTransform3f& InValue, float InTolerance)
{
	check(!FSampleTrackBase::IsReferenced());
	check(ChildTracks.Num() == 3);

	FSampleTrack<FVector3f>* LocationTrack = static_cast<FSampleTrack<FVector3f>*>(ChildTracks[0].Get());
	FSampleTrack<FQuat4f>* RotationTrack = static_cast<FSampleTrack<FQuat4f>*>(ChildTracks[1].Get());
	FSampleTrack<FVector3f>* ScaleTrack = static_cast<FSampleTrack<FVector3f>*>(ChildTracks[2].Get());

	LocationTrack->AddSample(InValue.GetLocation(), InTolerance);
	RotationTrack->AddSample(InValue.GetRotation(), InTolerance);
	ScaleTrack->AddSample(InValue.GetScale3D(), InTolerance);
}