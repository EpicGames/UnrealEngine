// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/SampleTrack.h"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrack
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FSampleTrack<FInstancedStruct>::Serialize(FArchive& InArchive)
{
	FSampleTrackBase::Serialize(InArchive);
	
	int32 ArraySize = ValuesStorage.Num();
	InArchive << ArraySize;
	
	if(InArchive.IsLoading())
	{
		ValuesStorage.Reset();
		ValuesStorage.SetNum(ArraySize);
	}

	for(int32 Index = 0; Index < ArraySize; Index++)
	{
		ValuesStorage[Index].Serialize(InArchive);
	}

	if(InArchive.IsLoading())
	{
		UpdateValueArrayView();
	}
}

int32 FSampleTrack<FInstancedStruct>::GetSizePerValue() const
{
	if(ScriptStruct)
	{
		return sizeof(FInstancedStruct) + ScriptStruct->GetStructureSize();
	}
	return sizeof(FInstancedStruct);
}

void FSampleTrack<FInstancedStruct>::AddSampleFromProperty(const FProperty* InProperty, const uint8* InMemory)
{
	const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InProperty);
	check(StructProperty->Struct == GetScriptStruct());

	FInstancedStruct Value(GetScriptStruct());
	StructProperty->CopyCompleteValue(Value.GetMutableMemory(), InMemory);
	AddSample(Value);
}

void FSampleTrack<FInstancedStruct>::GetSampleForProperty(int32 InTimeIndex, FSampleTrackIndex& InOutSampleTrackIndex, const FProperty* InProperty, uint8* OutMemory) const
{
	const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InProperty);
	check(StructProperty->Struct == GetScriptStruct());

	const FInstancedStruct& Value = GetValueAtTimeIndex(InTimeIndex, InOutSampleTrackIndex);
	StructProperty->CopyCompleteValue(OutMemory, Value.GetMemory());
}


void FSampleTrack<TArray<FInstancedStruct>>::Serialize(FArchive& InArchive)
{
	FSampleTrackBase::Serialize(InArchive);
	
	int32 OuterArraySize = ValuesStorage.Num();
	InArchive << OuterArraySize;
	
	if(InArchive.IsLoading())
	{
		ValuesStorage.Reset();
		ValuesStorage.SetNum(OuterArraySize);
	}

	for(int32 OuterIndex = 0; OuterIndex < OuterArraySize; OuterIndex++)
	{
		int32 InnerArraySize = ValuesStorage[OuterIndex].Num();
		InArchive << InnerArraySize;
	
		if(InArchive.IsLoading())
		{
			ValuesStorage[OuterIndex].Reset();
			ValuesStorage[OuterIndex].SetNum(InnerArraySize);
		}

		for(int32 InnerIndex = 0; InnerIndex < InnerArraySize; InnerIndex++)
		{
			ValuesStorage[OuterIndex][InnerIndex].Serialize(InArchive);
		}
	}

	if(InArchive.IsLoading())
	{
		UpdateValueArrayView();
	}
}

void FSampleTrack<TArray<FInstancedStruct>>::AddSampleFromProperty(const FProperty* InProperty, const uint8* InMemory)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(InProperty);
	const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
	check(StructProperty->Struct == GetScriptStruct());
	FScriptArrayHelper ArrayHelper(ArrayProperty, InMemory);

	TArray<FInstancedStruct> NewValues;
	NewValues.Reserve(ArrayHelper.Num());
	for(int32 Index = 0; Index < ArrayHelper.Num(); Index++)
	{
		FInstancedStruct Value(GetScriptStruct());
		StructProperty->CopyCompleteValue(Value.GetMutableMemory(), ArrayHelper.GetElementPtr(Index));
		NewValues.Add(Value);
	}
	AddSample(NewValues);
}

void FSampleTrack<TArray<FInstancedStruct>>::GetSampleForProperty(int32 InTimeIndex, FSampleTrackIndex& InOutSampleTrackIndex, const FProperty* InProperty, uint8* OutMemory) const
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(InProperty);
	const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(ArrayProperty->Inner);
	check(StructProperty->Struct == GetScriptStruct());

	const TArray<FInstancedStruct>& ValuesAtTime = GetValueAtTimeIndex(InTimeIndex, InOutSampleTrackIndex);

	FScriptArrayHelper ArrayHelper(ArrayProperty, OutMemory);
	ArrayHelper.Resize(ValuesAtTime.Num());

	for(int32 Index = 0; Index < ValuesAtTime.Num(); Index++)
	{
		const FInstancedStruct& Value = ValuesAtTime[Index];
		StructProperty->CopyCompleteValue(ArrayHelper.GetElementPtr(Index), Value.GetMemory());
	}
}
