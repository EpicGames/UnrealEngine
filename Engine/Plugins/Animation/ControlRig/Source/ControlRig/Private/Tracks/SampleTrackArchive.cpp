// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/SampleTrackArchive.h"
#include "Misc/Compression.h"
#include "RigVMTypeUtils.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SampleTrackArchive)

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackMemoryData
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FSampleTrackMemoryData::Serialize(FArchive& Ar)
{
	Ar << Buffer;
	Ar << Names;
	Ar << ObjectPaths;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackMemoryWriter
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSampleTrackMemoryWriter::FSampleTrackMemoryWriter(FSampleTrackMemoryData& InOutData, bool bIsPersistent)
: FMemoryWriter(InOutData.Buffer, bIsPersistent)
, Data(InOutData)
{
}
	
FArchive& FSampleTrackMemoryWriter::operator<<(FName& Value)
{
	int32 Index = INDEX_NONE;
	if(const int32* ExistingIndexPtr = NameToIndex.Find(Value))
	{
		Index = *ExistingIndexPtr;
	}
	else
	{
		Index = Data.Names.Find(Value);
		if(Index == INDEX_NONE)
		{
			Index = Data.Names.Add(Value);
		}
		NameToIndex.Add(Value, Index);
	}
	*this << Index;
	return *this;
}

FArchive& FSampleTrackMemoryWriter::operator<<(FText& Value)
{
	FString ValueString = Value.ToString();
	*this << ValueString;
	return *this;
}

FArchive& FSampleTrackMemoryWriter::operator<<(class UObject*& Value)
{
	int32 Index = INDEX_NONE;
	if(Value)
	{
		if(const int32* ExistingIndex = ObjectToIndex.Find(Value))
		{
			Index = *ExistingIndex;
		}
		else
		{
			const FString ObjectPath = Value->GetPathName();
			Index = Data.ObjectPaths.Find(ObjectPath);
			if(Index == INDEX_NONE)
			{
				Index = Data.ObjectPaths.Add(ObjectPath);
			}
			ObjectToIndex.Add(Value, Index);
		}
	}
	*this << Index;
	return *this;
}

FArchive& FSampleTrackMemoryWriter::operator<<(FObjectPtr& Value)
{
	UObject* Object = Value.Get();
	*this << Object;
	return *this;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackMemoryReader
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSampleTrackMemoryReader::FSampleTrackMemoryReader(FSampleTrackMemoryData& InOutData, bool bIsPersistent)
: FMemoryReader(InOutData.Buffer, bIsPersistent)
, Data(InOutData)
{
	static TMap<FString, TWeakObjectPtr<UObject>> PathToObject;
	static FCriticalSection PathToObjectMutex;
	FScopeLock _(&PathToObjectMutex);

	for(const FString& ObjectPath : Data.ObjectPaths)
	{
		if(TWeakObjectPtr<UObject>* ExistingObject = PathToObject.Find(ObjectPath))
		{
			Objects.Add(ExistingObject->Get());
		}
		else
		{
			UObject* Object = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(ObjectPath);
			if(Object == nullptr)
			{
				UE_LOGF(LogControlRig, Error, "FSampleTrackMemoryReader: The object '%ls' could not be resolved.", *ObjectPath);
			}
			else
			{
				PathToObject.Add(ObjectPath, Object);
			}
			Objects.Add(Object);
		}
	}
}

FArchive& FSampleTrackMemoryReader::operator<<(FName& Value)
{
	int32 NameIndex = INDEX_NONE;
	*this << NameIndex;
	check(Data.Names.IsValidIndex(NameIndex));
	Value = Data.Names[NameIndex];
	return *this;
}
	
FArchive& FSampleTrackMemoryReader::operator<<(FText& Value)
{
	FString ValueString;
	*this << ValueString;
	Value = FText::FromString(ValueString);
	return *this;
}

FArchive& FSampleTrackMemoryReader::operator<<(class UObject*& Value)
{
	int32 Index = INDEX_NONE;
	*this << Index;

	if(Index == INDEX_NONE)
	{
		Value = nullptr;
	}
	else
	{
		check(Objects.IsValidIndex(Index));
		Value = Objects[Index];
	}
	return *this;
}

FArchive& FSampleTrackMemoryReader::operator<<(FObjectPtr& Value)
{
	UObject* Object = nullptr;
	*this << Object;
	Value = Object;
	return *this;
}
