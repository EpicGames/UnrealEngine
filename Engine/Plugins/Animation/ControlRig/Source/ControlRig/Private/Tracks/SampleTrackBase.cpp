// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/SampleTrackBase.h"
#include "Tracks/SampleTrackContainer.h"

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackIndex
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSampleTrackIndex::FSampleTrackIndex()
: bIsSingleton(false)
{
}

FSampleTrackIndex::FSampleTrackIndex(int32 NumTracks)
: bIsSingleton(false)
{
	Allocate(NumTracks);
}

FSampleTrackIndex::FSampleTrackIndex(const FSampleTrackContainer& InContainer)
: bIsSingleton(false)
{
	Update(InContainer);
}

int32& FSampleTrackIndex::GetSample(int32 InTrackIndex)
{
	if(bIsSingleton)
	{
		check(Samples.Num() == 1);
		return Samples[0];
	}
	
	InTrackIndex = FMath::Max(InTrackIndex, 0);
	Allocate(InTrackIndex+1);
	return Samples[InTrackIndex];
}

FSampleTrackIndex FSampleTrackIndex::MakeSingleton()
{
	FSampleTrackIndex Index(1);
	Index.bIsSingleton = true;
	return Index;
}

void FSampleTrackIndex::Update(const FSampleTrackContainer& InContainer)
{
	Allocate(InContainer.NumTracks());
}

void FSampleTrackIndex::Allocate(int32 NumTracks)
{
	if(Samples.Num() >= NumTracks)
	{
		return;
	}
	Samples.AddZeroed(NumTracks - Samples.Num());
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackBase
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSampleTrackBase::FSampleTrackBase()
: Names()
, TrackIndex(INDEX_NONE)
, ReferencedTrackIndex(INDEX_NONE)
, ReferencedTimeIndicesRange({INDEX_NONE,INDEX_NONE})
, ReferencedAtlasRange({INDEX_NONE,INDEX_NONE})
, ReferencedValuesRange({INDEX_NONE,INDEX_NONE})
, ScriptStruct(nullptr)
, Mode(EMode_Invalid)
, NumTimesInContainer(0)
, Container(nullptr)
{
}

FSampleTrackBase::FSampleTrackBase(const FName& InName)
: Names({InName})
, TrackIndex(INDEX_NONE)
, ReferencedTrackIndex(INDEX_NONE)
, ReferencedTimeIndicesRange({INDEX_NONE,INDEX_NONE})
, ReferencedAtlasRange({INDEX_NONE,INDEX_NONE})
, ReferencedValuesRange({INDEX_NONE,INDEX_NONE})
, ScriptStruct(nullptr)
, Mode(EMode_Invalid)
, NumTimesInContainer(0)
, Container(nullptr)
{
}

const FName& FSampleTrackBase::GetName() const
{
	if(Names.IsEmpty())
	{
		static const FName InvalidName = FName(NAME_None);
		return InvalidName;
	}
	return Names[0];
}

int32 FSampleTrackBase::NumTimes() const
{
	return NumTimesInContainer;
}

int32 FSampleTrackBase::NumSamples() const
{
	switch(Mode)
	{
		case EMode_Singleton:
		{
			return 1;
		}
		case EMode_Sampled:
		{
			return TimeIndices.Num();
		}
		case EMode_Complete:
		{
			return NumTimes();
		}
		case EMode_Raw:
		case EMode_Invalid:
		default:
		{
			break;
		}
	}
	return 0;
}

bool FSampleTrackBase::GetSampleIndexForTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const
{
	InTimeIndex = FMath::Clamp(InTimeIndex, 0, NumTimes() - 1);
	
	int32& SampleIndex = InOutIndex.GetSample(TrackIndex);
	
	switch(Mode)
	{
		case EMode_Singleton:
		{
			SampleIndex = 0;
			return true;
		}
		case EMode_Sampled:
		{
			SampleIndex = FMath::Clamp(SampleIndex, 0, TimeIndices.Num()-1);

			if(TimeIndices.IsEmpty())
			{
				break;
			}
			if(!TimeIndices.IsValidIndex(SampleIndex))
			{
				SampleIndex = 0;
			}
			while(TimeIndices.IsValidIndex(SampleIndex - 1) && TimeIndices[SampleIndex] > InTimeIndex)
			{
				SampleIndex--;
			}
			while(TimeIndices.IsValidIndex(SampleIndex + 1) && TimeIndices[SampleIndex + 1] <= InTimeIndex)
			{
				SampleIndex++;
			}
			return true;
		}
		case EMode_Complete:
		{
			SampleIndex = InTimeIndex;
			return true;
		}
		case EMode_Invalid:
		case EMode_Raw:
		default:
		{
			break;
		}
	}
	return false;
}

bool FSampleTrackBase::StoresValueForTimeIndex(int32 InTimeIndex) const
{
	switch(Mode)
	{
		case EMode_Singleton:
		{
			return InTimeIndex == 0;
		}
		case EMode_Sampled:
		{
			return TimeIndices.Contains(InTimeIndex);
		}
		case EMode_Complete:
		{
			return true;
		}
		case EMode_Invalid:
		default:
		{
			break;
		}
	}
	return false;
}

bool FSampleTrackBase::IsIdentical(const FSampleTrackBase* InOther, float InTolerance)
{
	if(IsReferenced() || InOther->IsReferenced())
	{
		return false;
	}
	if(GetTrackType() != InOther->GetTrackType())
	{
		return false;
	}
	if(GetScriptStruct() != InOther->GetScriptStruct())
	{
		return false;
	}
	if(GetMode() != InOther->GetMode())
	{
		return false;
	}
	if(NumTimes() != InOther->NumTimes())
	{
		return false;
	}
	if(NumSamples() != InOther->NumSamples())
	{
		return false;
	}
	if(GetMode() == EMode_Sampled)
	{
		if(TimeIndices.Num() != InOther->TimeIndices.Num())
		{
			return false;
		}
		for(int32 Index = 0; Index < TimeIndices.Num(); Index++)
		{
			if(TimeIndices[Index] != InOther->TimeIndices[Index])
			{
				return false;
			}
		}
	}
	if(Atlas.Num() != InOther->Atlas.Num())
	{
		return false;
	}
	for(int32 Index = 0; Index < Atlas.Num(); Index++)
	{
		if(Atlas[Index] != InOther->Atlas[Index])
		{
			return false;
		}
	}
	return true;
}

void FSampleTrackBase::Reserve(int32 InSampleCount, int32 InValueCount)
{
	check(!IsReferenced());
	TimeIndicesStorage.Reserve(InSampleCount);
}

void FSampleTrackBase::Reset()
{
	Mode = EMode_Invalid;
	TimeIndicesStorage.Reset();
	AtlasStorage.Reset(0);
	NumTimesInContainer = 0;
	ReferencedTrackIndex = INDEX_NONE;
	ReferencedTimeIndicesRange = {INDEX_NONE, INDEX_NONE};
	ReferencedAtlasRange = {INDEX_NONE, INDEX_NONE};
	ReferencedValuesRange = {INDEX_NONE, INDEX_NONE};
	UpdateTimeAndAtlasArrayViews();
}

void FSampleTrackBase::Shrink()
{
	if(Mode == EMode_Sampled || Mode == EMode_Raw)
	{
		TimeIndicesStorage.Shrink();
	}
	else
	{
		TimeIndicesStorage.Empty();
	}
	AtlasStorage.Shrink();
	UpdateTimeAndAtlasArrayViews();
}

void FSampleTrackBase::Empty()
{
	TimeIndicesStorage.Empty();
	AtlasStorage.Empty();
	UpdateTimeAndAtlasArrayViews();
}

void FSampleTrackBase::Serialize(FArchive& InArchive)
{
	if(InArchive.IsLoading())
	{
		Reset();
	}
		
	InArchive << Names;
	InArchive << (uint8&)Mode;
	InArchive << NumTimesInContainer;
	InArchive << TrackIndex;
	InArchive << ReferencedTrackIndex;
	InArchive << ReferencedTimeIndicesRange;
	InArchive << ReferencedAtlasRange;
	InArchive << ReferencedValuesRange;

	bool bHasScriptStruct = ScriptStruct != nullptr;
	InArchive << bHasScriptStruct;
	
	if(InArchive.IsLoading())
	{
		if(bHasScriptStruct)
		{
			UScriptStruct* SerializedScriptStruct = nullptr;
			InArchive << SerializedScriptStruct;
			ScriptStruct = SerializedScriptStruct;
		}
		else
		{
			ScriptStruct = nullptr;
		}
	}
	else
	{
		if(bHasScriptStruct)
		{
			UScriptStruct* ScriptStructToSerialize = const_cast<UScriptStruct*>(ScriptStruct);
			InArchive << ScriptStructToSerialize;
		}
	}

	switch(Mode)
	{
		// only sampled tracks require the indices array
		case EMode_Sampled:
		case EMode_Raw:
		{
			InArchive << TimeIndicesStorage;
		}
		default:
		{
			break;
		}
	}

	InArchive << AtlasStorage;

	if(InArchive.IsLoading())
	{
		UpdateTimeAndAtlasArrayViews();
		Shrink();
	}
}

const FSampleTrackBase* FSampleTrackBase::GetReferencedTrack() const
{
	check(IsReferenced());
	check(Container);
	check(ReferencedTrackIndex != INDEX_NONE);
	if(Container->NumTracks() <= ReferencedTrackIndex)
	{
		return nullptr;
	}
	return Container->GetTrack(ReferencedTrackIndex).Get();
}

FSampleTrackBase* FSampleTrackBase::GetReferencedTrack()
{
	check(IsReferenced());
	check(Container);
	check(ReferencedTrackIndex != INDEX_NONE);
	if(Container->NumTracks() <= ReferencedTrackIndex)
	{
		return nullptr;
	}
	return Container->GetTrack(ReferencedTrackIndex).Get();
}

void FSampleTrackBase::UpdateArrayViews()
{
	UpdateTimeAndAtlasArrayViews();
}

void FSampleTrackBase::UpdateTimeAndAtlasArrayViews()
{
	if(IsReferenced())
	{
		FSampleTrackBase* ReferencedTrack = GetReferencedTrack();
		if(ReferencedTrack == nullptr)
		{
			TimeIndices = TArrayView<int32>();
			Atlas = TArrayView<int32>();
			return;
		}

		if(ReferencedTimeIndicesRange.Get<1>() <= 0)
		{
			TimeIndices = TArrayView<int32>();
		}
		else
		{
			check(ReferencedTrack->TimeIndicesStorage.IsValidIndex(ReferencedTimeIndicesRange.Get<0>()));
			check(ReferencedTrack->TimeIndicesStorage.IsValidIndex(ReferencedTimeIndicesRange.Get<0>() + ReferencedTimeIndicesRange.Get<1>() - 1));
		
			TimeIndices = TArrayView<int32>(
				ReferencedTrack->TimeIndicesStorage.GetData() + ReferencedTimeIndicesRange.Get<0>(),
				ReferencedTimeIndicesRange.Get<1>());
		}

		if(ReferencedAtlasRange.Get<1>() <= 0)
		{
			Atlas = TArrayView<int32>();
		}
		else
		{
			check(ReferencedTrack->AtlasStorage.IsValidIndex(ReferencedAtlasRange.Get<0>()));
			check(ReferencedTrack->AtlasStorage.IsValidIndex(ReferencedAtlasRange.Get<0>() + ReferencedAtlasRange.Get<1>() - 1));
			
			Atlas = TArrayView<int32>(
				ReferencedTrack->AtlasStorage.GetData() + ReferencedAtlasRange.Get<0>(),
				ReferencedAtlasRange.Get<1>());
		}
	}
	else
	{
		TimeIndices = TArrayView<int32>(TimeIndicesStorage.GetData(), TimeIndicesStorage.Num());
		Atlas = TArrayView<int32>(AtlasStorage.GetData(), AtlasStorage.Num());
	}
}

bool FSampleTrackBase::UsesAtlas(int32 InAtlasIndex) const
{
	if(Atlas.IsEmpty())
	{
		return false;
	}
#if WITH_EDITOR	
	if(InAtlasIndex != INDEX_NONE)
	{
		check(Atlas.IsValidIndex(InAtlasIndex));
	}
#endif
	return true;
}

void FSampleTrackBase::AddTimeIndex(bool bOnlyIncreaseUpperBound)
{
	check(!IsReferenced());

	if(!bOnlyIncreaseUpperBound)
	{
		TimeIndicesStorage.Add(NumTimesInContainer);
		UpdateTimeAndAtlasArrayViews();
	}
	NumTimesInContainer++;

	switch(Mode)
	{
		case EMode_Singleton:
		case EMode_Sampled:
		case EMode_Complete:
		{
			if(TimeIndices.Num() == NumTimesInContainer)
			{
				Mode = EMode_Complete;
			}
			else if(TimeIndices.Num() == 1)
			{
				Mode = EMode_Singleton;
			}
			else
			{
				Mode = EMode_Sampled;
			}
			break;
		}
		case EMode_Invalid:
		default:
		{
			Mode = EMode_Singleton;
			break;
		}
	}
}

const TArray<int32>& FSampleTrackBase::GetChildTracks() const
{
	static const TArray<int32> EmptyIndices;
	return EmptyIndices;
}

int32 FSampleTrackBase::RemoveObsoleteTimes(int32 InNumSamplesToRemove, const TArray<bool>& InRemoveIndexMap, const TArray<int32>& InNewIndexMap)
{
	if(!IsValid() || IsReferenced() || GetMode() == EMode_Singleton)
	{
		return 0;
	}

	check(NumTimesInContainer == InRemoveIndexMap.Num());
	check(NumTimesInContainer == InNewIndexMap.Num());
	
	if(InNumSamplesToRemove == 0)
	{
		return 0;
	}

	switch(Mode)
	{
		case EMode_Singleton:
		{
			// tracks can't be empty and still be valid.
			// so we'll keep one value in here at all times (no pun intended).
			break;
		}
		case EMode_Sampled:
		{
			int32 NumRemoved = TimeIndicesStorage.RemoveAll([InRemoveIndexMap](int32 TimeIndex)
			{
				return InRemoveIndexMap[TimeIndex];
			});

			for(int32& TimeIndex : TimeIndicesStorage)
			{
				if(InNewIndexMap[TimeIndex] != TimeIndex)
				{
					TimeIndex = InNewIndexMap[TimeIndex];
				}
			}

			NumTimesInContainer = FMath::Max(1, NumTimesInContainer - InNumSamplesToRemove);
			UpdateTimeAndAtlasArrayViews();
			return NumRemoved;
		}
		case EMode_Complete:
		{
			// the indices array is empty - there's no need to shrink them,
			// only the values have to be adapted.
			return InNumSamplesToRemove;
		}
		case EMode_Invalid:
		default:
		{
			checkNoEntry();
			break;
		}
	}

	return 0;
}

bool FSampleTrackBase::LocalizeValues()
{
	if(IsReferenced())
	{
		// first update all array views so that we can copy the data off of the raw track
		UpdateArrayViews();

		ReferencedTrackIndex = INDEX_NONE;
		ReferencedTimeIndicesRange = {INDEX_NONE, INDEX_NONE};
		ReferencedAtlasRange = {INDEX_NONE, INDEX_NONE};
		ReferencedValuesRange = {INDEX_NONE, INDEX_NONE};
		TimeIndicesStorage = TimeIndices;
		AtlasStorage = Atlas;
		UpdateTimeAndAtlasArrayViews();
		return true;
	}
	return false;
}
