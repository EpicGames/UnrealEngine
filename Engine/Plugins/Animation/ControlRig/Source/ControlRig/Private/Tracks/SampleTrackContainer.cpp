// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/SampleTrackContainer.h"
#include "Tracks/SampleTrackArchive.h"
#include "Tracks/ComposedSampleTrack.h"
#include "Misc/Compression.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SampleTrackContainer)

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackContainer
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSampleTrackHost::FSampleTrackHost()
: Container(new FSampleTrackContainer)
{
}

FSampleTrackHost::FSampleTrackHost(const FSampleTrackHost& InOther)
: Container(new FSampleTrackContainer)
{
	*this = InOther;
}

FSampleTrackHost& FSampleTrackHost::operator=(const FSampleTrackHost& InOther)
{
	if(Container == nullptr)
	{
		Container = MakeUnique<FSampleTrackContainer>();
	}
	
	if(InOther.GetContainer())
	{
		FSampleTrackMemoryData ArchiveData;
		FSampleTrackMemoryWriter ArchiveWriter(ArchiveData);
		const_cast<FSampleTrackHost&>(InOther).GetContainer()->Serialize(ArchiveWriter);

		FSampleTrackMemoryReader ArchiveReader(ArchiveData);
		Container->Serialize(ArchiveReader);
	}
	else
	{
		Container->Reset();
	}
	
	return *this;
}

void FSampleTrackHost::Reset()
{
	Container->Reset();
}

void FSampleTrackHost::Compact()
{
	Container->Compact();
}

bool FSampleTrackHost::Serialize(FArchive& InArchive)
{
	if(Container == nullptr)
	{
		Container = MakeUnique<FSampleTrackContainer>();
	}

	// Serialize normal tagged property data
	UScriptStruct* const Struct = FSampleTrackHost::StaticStruct();
	Struct->SerializeTaggedProperties(InArchive, reinterpret_cast<uint8*>(this), Struct, nullptr);

	return Container->Serialize(InArchive);
}

const FSampleTrackContainer* FSampleTrackHost::GetContainer() const
{
	return Container.Get();
}

FSampleTrackContainer* FSampleTrackHost::GetContainer()
{
	return Container.Get();
}

int32 FSampleTrackHost::AddTimeSample(float InAbsoluteTime, float InDeltaTime)
{
	return GetContainer()->AddTimeSample(InAbsoluteTime, InDeltaTime);
}

int32 FSampleTrackHost::AddTimeSampleFromDeltaTime(float InDeltaTime)
{
	return GetContainer()->AddTimeSampleFromDeltaTime(InDeltaTime);
}

int32 FSampleTrackHost::GetNumTimes() const
{
	return GetContainer()->GetNumTimes();
}

FVector2f FSampleTrackHost::GetTimeRange() const
{
	return GetContainer()->GetTimeRange();
}

int32 FSampleTrackHost::GetTimeIndex(float InAbsoluteTime, FSampleTrackIndex& InOutTrackIndex) const
{
	return GetContainer()->GetTimeIndex(InAbsoluteTime, InOutTrackIndex);
}

float FSampleTrackHost::GetAbsoluteTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const
{
	return GetContainer()->GetAbsoluteTime(InTimeIndex, InOutTrackIndex);
}

float FSampleTrackHost::GetDeltaTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const
{
	return GetContainer()->GetDeltaTime(InTimeIndex, InOutTrackIndex);
}

float FSampleTrackHost::GetAbsoluteTime(int32 InTimeIndex) const
{
	return GetContainer()->GetAbsoluteTime(InTimeIndex);
}

int32 FSampleTrackHost::GetTimeIndex(float InAbsoluteTime) const
{
	return GetContainer()->GetTimeIndex(InAbsoluteTime);
}

float FSampleTrackHost::GetDeltaTime(int32 InTimeIndex) const
{
	return GetContainer()->GetDeltaTime(InTimeIndex);
}

float FSampleTrackHost::GetLastAbsoluteTime() const
{
	return GetContainer()->GetLastAbsoluteTime();
}

float FSampleTrackHost::GetLastDeltaTime() const
{
	return GetContainer()->GetLastDeltaTime();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
/////// FSampleTrackContainer
//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const FLazyName FSampleTrackContainer::AbsoluteTimeName = FLazyName(TEXT("AbsoluteTime"));
const FLazyName FSampleTrackContainer::DeltaTimeName = FLazyName(TEXT("DeltaTime"));

FSampleTrackContainer::FSampleTrackContainer()
: bForceToUseCompression(false)
, TimeSampleTrackIndex(FSampleTrackIndex::MakeSingleton())
{
}

FSampleTrackContainer::~FSampleTrackContainer()
{
}

void FSampleTrackContainer::Reset()
{
	NameToIndex.Reset();
	Tracks.Reset();
	TimeSampleTrackIndex = FSampleTrackIndex::MakeSingleton();
}

void FSampleTrackContainer::Shrink()
{
	NameToIndex.Shrink();

	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		Track->Shrink();
	}
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		Track->UpdateArrayViews();
	}
}

void FSampleTrackContainer::Compact(float InTolerance)
{
	RemoveInvalidTracks(false);
	RemoveRedundantTracks(false, InTolerance);
	MergeTypedTracks(false, InTolerance);
	ConvertTracksToSampled();
	ConvertTracksToComplete();
	EnableTrackAtlas(InTolerance);
	UpdateNameToIndexMap();
	Shrink();
}

void FSampleTrackContainer::Reserve(int32 InNum)
{
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		Track->Reserve(InNum, InNum);
	}
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		Track->UpdateArrayViews();
	}
}

bool FSampleTrackContainer::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FControlRigObjectVersion::GUID);
	
	bool bUseCompression = bForceToUseCompression || (
		!((InArchive.GetPortFlags() & PPF_Duplicate) != 0) &&
		InArchive.IsPersistent() &&
		!InArchive.IsObjectReferenceCollector() &&
		!InArchive.IsCountingMemory() &&
		!InArchive.ShouldSkipBulkData() &&
		!InArchive.IsTransacting()
	);

	// compact ourselves to as small as possible
	if(InArchive.IsSaving() && bUseCompression && InArchive.IsPersistent())
	{
		Compact();
	}

	InArchive << bUseCompression;

	FArchive* InnerArchive = &InArchive;
	TUniquePtr<FMemoryArchive> MemoryArchive;

	FSampleTrackMemoryData ArchiveData;
	TArray<uint8> CompressedBytes;

	if(InArchive.IsLoading())
	{
		Reset();

		if(bUseCompression)
		{
			InArchive << ArchiveData;
			
			int32 UncompressedSize = 0;
			InArchive << UncompressedSize;

			bool bStoreCompressedBytes = false;
			InArchive << bStoreCompressedBytes;

			if(bStoreCompressedBytes)
			{
				Swap(CompressedBytes, ArchiveData.Buffer);
				ArchiveData.Buffer.SetNumUninitialized(UncompressedSize);

				const bool bSuccess = FCompression::UncompressMemory(NAME_Oodle, ArchiveData.Buffer.GetData(), UncompressedSize, CompressedBytes.GetData(), CompressedBytes.Num());
				verify(bSuccess);
			}

			MemoryArchive = MakeUnique<FSampleTrackMemoryReader>(ArchiveData, InArchive.IsPersistent());
			InnerArchive = MemoryArchive.Get();
			InnerArchive->SetCustomVersions(InArchive.GetCustomVersions());
		}
	}
	else
	{
		if(bUseCompression)
		{
			MemoryArchive = MakeUnique<FSampleTrackMemoryWriter>(ArchiveData, InArchive.IsPersistent());
			InnerArchive = MemoryArchive.Get();
			InnerArchive->SetCustomVersions(InArchive.GetCustomVersions());
		}
	}
	
	int32 NumTracks = Tracks.Num();
	(*InnerArchive) << NumTracks;
	
	if(InArchive.IsLoading())
	{
		// first create all of the tracks
		for(int32 TrackIndex = 0; TrackIndex < NumTracks; TrackIndex++)
		{
			uint8 TrackType = 0;
			(*InnerArchive) << TrackType;
			AddTrack(MakeTrack((FSampleTrackBase::ETrackType)TrackType), false);
		}

		// now load the track in the order they were saved (raw first, everything else after)
		for(int32 TrackIndex = 0; TrackIndex < NumTracks; TrackIndex++)
		{
			int32 TrackIndexToLoad = INDEX_NONE;
			(*InnerArchive) << TrackIndexToLoad;
			Tracks[TrackIndexToLoad]->Serialize((*InnerArchive));
		}

		// update the array views on the tracks
		for(int32 TrackIndex = 0; TrackIndex < NumTracks; TrackIndex++)
		{
			GetTrack(TrackIndex)->UpdateArrayViews();
			GetTrack(TrackIndex)->UpdateChildTracks();
		}

		// the tracks were added with no names present,
		// so let's take care of updating the map now
		UpdateNameToIndexMap();
	}
	else
	{
		// first save all of the types of tracks
		for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
		{
			uint8 TrackType = (uint8)Track->GetTrackType();
			(*InnerArchive) << TrackType;
		}

		// then save all of the raw tracks (tracks referenced by other tracks)
		for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
		{
			if(Track->GetMode() != FSampleTrackBase::EMode_Raw)
			{
				continue;
			}
			int32 TrackIndex = Track->GetTrackIndex();
			(*InnerArchive) << TrackIndex;
			Track->Serialize((*InnerArchive));
		}

		// finally save all of the other tracks
		for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
		{
			if(Track->GetMode() == FSampleTrackBase::EMode_Raw)
			{
				continue;
			}
			int32 TrackIndex = Track->GetTrackIndex();
			(*InnerArchive) << TrackIndex;
			Track->Serialize((*InnerArchive));
		}
		
		if(bUseCompression)
		{
			// It is possible for compression to actually increase the size of the data, so we over allocate here to handle that.
			int32 UncompressedSize = ArchiveData.Buffer.Num();
			int32 CompressedSize = FCompression::CompressMemoryBound(NAME_Oodle, UncompressedSize);

			CompressedBytes.SetNumUninitialized(CompressedSize);

			bool bStoreCompressedBytes = FCompression::CompressMemory(NAME_Oodle, CompressedBytes.GetData(), CompressedSize, ArchiveData.Buffer.GetData(), ArchiveData.Buffer.Num(), COMPRESS_BiasMemory);
			if(bStoreCompressedBytes)
			{
				if(CompressedSize < UncompressedSize)
				{
					CompressedBytes.SetNum(CompressedSize);
					ArchiveData.Buffer = MoveTemp(CompressedBytes);
				}
				else
				{
					bStoreCompressedBytes = false;
				}
			}
			
			InArchive << ArchiveData;
			InArchive << UncompressedSize;
			InArchive << bStoreCompressedBytes;
		}
	}

	return true;
}

void FSampleTrackContainer::SetForceToUseCompression(bool InForce)
{
	bForceToUseCompression = InForce;
}

TSharedPtr<FSampleTrackBase> FSampleTrackContainer::AddTrack(const FName& InName, FSampleTrackBase::ETrackType InTrackType, const UScriptStruct* InScriptStruct)
{
	TSharedPtr<FSampleTrackBase> Track = MakeTrack(InTrackType);
	Track->Names.Reset();
	Track->Names.Add(InName);
	Track->ScriptStruct = InScriptStruct;
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrackBase> FSampleTrackContainer::FindOrAddTrack(const FName& InName, FSampleTrackBase::ETrackType InTrackType,
	const UScriptStruct* InScriptStruct)
{
	TSharedPtr<FSampleTrackBase> Track = FindTrack(InName);
	if(!Track)
	{
		Track = AddTrack(InName, InTrackType, InScriptStruct);
	}
	return Track;
}

TSharedPtr<FSampleTrack<bool>> FSampleTrackContainer::AddBoolTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<bool>> Track = MakeShareable(new FSampleTrack<bool>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<int32>> FSampleTrackContainer::AddInt32Track(const FName& InName)
{
	TSharedPtr<FSampleTrack<int32>> Track = MakeShareable(new FSampleTrack<int32>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<uint32>> FSampleTrackContainer::AddUint32Track(const FName& InName)
{
	TSharedPtr<FSampleTrack<uint32>> Track = MakeShareable(new FSampleTrack<uint32>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<float>> FSampleTrackContainer::AddFloatTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<float>> Track = MakeShareable(new FSampleTrack<float>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FName>> FSampleTrackContainer::AddNameTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FName>> Track = MakeShareable(new FSampleTrack<FName>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FString>> FSampleTrackContainer::AddStringTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FString>> Track = MakeShareable(new FSampleTrack<FString>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FVector3f>> FSampleTrackContainer::AddVectorTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FVector3f>> Track = MakeShareable(new FSampleTrack<FVector3f>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FQuat4f>> FSampleTrackContainer::AddQuatTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FQuat4f>> Track = MakeShareable(new FSampleTrack<FQuat4f>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FTransform3f>> FSampleTrackContainer::AddTransformTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FTransform3f>> Track = MakeShareable(new FComposedSampleTrack<FTransform3f>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FLinearColor>> FSampleTrackContainer::AddLinearColorTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FLinearColor>> Track = MakeShareable(new FSampleTrack<FLinearColor>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FRigElementKey>> FSampleTrackContainer::AddRigElementKeyTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FRigElementKey>> Track = MakeShareable(new FSampleTrack<FRigElementKey>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FRigComponentKey>> FSampleTrackContainer::AddRigComponentKeyTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<FRigComponentKey>> Track = MakeShareable(new FSampleTrack<FRigComponentKey>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<FInstancedStruct>> FSampleTrackContainer::AddStructTrack(const FName& InName, const UScriptStruct* InScriptStruct)
{
	check(InScriptStruct);
	TSharedPtr<FSampleTrack<FInstancedStruct>> Track = MakeShareable(new FSampleTrack<FInstancedStruct>(InName));
	Track->ScriptStruct = InScriptStruct;
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<bool>>> FSampleTrackContainer::AddBoolArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<bool>>> Track = MakeShareable(new FSampleTrack<TArray<bool>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<int32>>> FSampleTrackContainer::AddInt32ArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<int32>>> Track = MakeShareable(new FSampleTrack<TArray<int32>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<uint32>>> FSampleTrackContainer::AddUint32ArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<uint32>>> Track = MakeShareable(new FSampleTrack<TArray<uint32>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<float>>> FSampleTrackContainer::AddFloatArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<float>>> Track = MakeShareable(new FSampleTrack<TArray<float>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FName>>> FSampleTrackContainer::AddNameArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FName>>> Track = MakeShareable(new FSampleTrack<TArray<FName>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FString>>> FSampleTrackContainer::AddStringArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FString>>> Track = MakeShareable(new FSampleTrack<TArray<FString>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FVector3f>>> FSampleTrackContainer::AddVectorArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FVector3f>>> Track = MakeShareable(new FSampleTrack<TArray<FVector3f>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FQuat4f>>> FSampleTrackContainer::AddQuatArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FQuat4f>>> Track = MakeShareable(new FSampleTrack<TArray<FQuat4f>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FTransform3f>>> FSampleTrackContainer::AddTransformArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FTransform3f>>> Track = MakeShareable(new FSampleTrack<TArray<FTransform3f>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FLinearColor>>> FSampleTrackContainer::AddLinearColorArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FLinearColor>>> Track = MakeShareable(new FSampleTrack<TArray<FLinearColor>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FRigElementKey>>> FSampleTrackContainer::AddRigElementKeyArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FRigElementKey>>> Track = MakeShareable(new FSampleTrack<TArray<FRigElementKey>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FRigComponentKey>>> FSampleTrackContainer::AddRigComponentKeyArrayTrack(const FName& InName)
{
	TSharedPtr<FSampleTrack<TArray<FRigComponentKey>>> Track = MakeShareable(new FSampleTrack<TArray<FRigComponentKey>>(InName));
	AddTrack(Track);
	return Track;
}

TSharedPtr<FSampleTrack<TArray<FInstancedStruct>>> FSampleTrackContainer::AddStructArrayTrack(const FName& InName, const UScriptStruct* InScriptStruct)
{
	check(InScriptStruct);
	TSharedPtr<FSampleTrack<TArray<FInstancedStruct>>> Track = MakeShareable(new FSampleTrack<TArray<FInstancedStruct>>(InName));
	Track->ScriptStruct = InScriptStruct;
	AddTrack(Track);
	return Track;
}

FVector2f FSampleTrackContainer::GetTimeRange() const
{
	const TSharedPtr<const FSampleTrack<float>> AbsoluteTimeTrack = FindTrack<float>(AbsoluteTimeName);
	if(!AbsoluteTimeTrack.IsValid() || AbsoluteTimeTrack->NumSamples() == 0)
	{
		return FVector2f::ZeroVector;
	}
	return {
		AbsoluteTimeTrack->GetValueAtSampleIndex(0),
		AbsoluteTimeTrack->GetValueAtSampleIndex(AbsoluteTimeTrack->NumSamples()-1)
	};
}

int32 FSampleTrackContainer::GetNumTimes() const
{
	const TSharedPtr<const FSampleTrack<float>> AbsoluteTimeTrack = FindTrack<float>(AbsoluteTimeName);
	if(!AbsoluteTimeTrack.IsValid() || AbsoluteTimeTrack->NumTimes() == 0)
	{
		return 0;
	}
	return AbsoluteTimeTrack->NumTimes(); 
}

float FSampleTrackContainer::GetAbsoluteTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const
{
	const TSharedPtr<const FSampleTrack<float>> AbsoluteTimeTrack = FindTrack<float>(AbsoluteTimeName);
	if(!AbsoluteTimeTrack.IsValid() || AbsoluteTimeTrack->NumTimes() == 0)
	{
		return 0.f;
	}
	return AbsoluteTimeTrack->GetValueAtTimeIndex(InTimeIndex, InOutTrackIndex); 
}

float FSampleTrackContainer::GetDeltaTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const
{
	const TSharedPtr<const FSampleTrack<float>> DeltaTimeTrack = FindTrack<float>(DeltaTimeName);
	if(!DeltaTimeTrack.IsValid() || DeltaTimeTrack->NumTimes() == 0)
	{
		return 0.f;
	}
	return DeltaTimeTrack->GetValueAtTimeIndex(InTimeIndex, InOutTrackIndex); 
}

float FSampleTrackContainer::GetAbsoluteTime(int32 InTimeIndex) const
{
	return GetAbsoluteTime(InTimeIndex, TimeSampleTrackIndex);
}

float FSampleTrackContainer::GetDeltaTime(int32 InTimeIndex) const
{
	return GetDeltaTime(InTimeIndex, TimeSampleTrackIndex);
}

float FSampleTrackContainer::GetLastAbsoluteTime() const
{
	const TSharedPtr<const FSampleTrack<float>> AbsoluteTimeTrack = FindTrack<float>(AbsoluteTimeName);
	if(!AbsoluteTimeTrack.IsValid() || AbsoluteTimeTrack->NumSamples() == 0)
	{
		return 0.f;
	}
	return AbsoluteTimeTrack->GetValueAtSampleIndex(AbsoluteTimeTrack->NumSamples()-1);
}

float FSampleTrackContainer::GetLastDeltaTime() const
{
	const TSharedPtr<const FSampleTrack<float>> DeltaTimeTrack = FindTrack<float>(DeltaTimeName);
	if(!DeltaTimeTrack.IsValid() || DeltaTimeTrack->NumSamples() == 0)
	{
		return 0.f;
	}
	return DeltaTimeTrack->GetValueAtSampleIndex(DeltaTimeTrack->NumSamples()-1);
}

int32 FSampleTrackContainer::GetTimeIndex(float InAbsoluteTime, FSampleTrackIndex& InOutTrackIndex) const
{
	const TSharedPtr<const FSampleTrack<float>> AbsoluteTimeTrack = FindTrack<float>(AbsoluteTimeName);
	check(AbsoluteTimeTrack.IsValid() && AbsoluteTimeTrack->NumTimes() > 0);

	int32& InOutTimeIndex = InOutTrackIndex.GetSample(AbsoluteTimeTrack->GetTrackIndex());

	if(AbsoluteTimeTrack->GetMode() == FSampleTrackBase::EMode_Singleton)
	{
		InOutTimeIndex = 0;
		return InOutTimeIndex;
	}

	const int32 NumSamples = AbsoluteTimeTrack->NumSamples();
	InOutTimeIndex = FMath::Clamp(InOutTimeIndex, 0, NumSamples - 1);

	while((InOutTimeIndex > 1) && AbsoluteTimeTrack->GetValueAtSampleIndex(InOutTimeIndex) > InAbsoluteTime - SMALL_NUMBER)
	{
		InOutTimeIndex--;
	}
	while((InOutTimeIndex < NumSamples - 1) && AbsoluteTimeTrack->GetValueAtSampleIndex(InOutTimeIndex+1) <= InAbsoluteTime + SMALL_NUMBER)
	{
		InOutTimeIndex++;
	}

	if(AbsoluteTimeTrack->GetMode() == FSampleTrackBase::EMode_Sampled)
	{
		InOutTimeIndex = AbsoluteTimeTrack->TimeIndices[InOutTimeIndex];
	}
	return InOutTimeIndex;
}

int32 FSampleTrackContainer::GetTimeIndex(float InAbsoluteTime) const
{
	return GetTimeIndex(InAbsoluteTime, TimeSampleTrackIndex);
}

int32 FSampleTrackContainer::GetTrackIndex(const FName& InName) const
{
	if(const int32* Index = NameToIndex.Find(InName))
	{
		return *Index;
	}
	return INDEX_NONE;
}

TSharedPtr<const FSampleTrackBase> FSampleTrackContainer::GetTrack(int32 InIndex) const
{
	return ConstCastSharedPtr<FSampleTrackBase>(Tracks[InIndex]);
}

TSharedPtr<FSampleTrackBase> FSampleTrackContainer::GetTrack(int32 InIndex)
{
	return Tracks[InIndex];
}

TSharedPtr<const FSampleTrackBase> FSampleTrackContainer::FindTrack(const FName& InName) const
{
	const int32 TrackIndex = GetTrackIndex(InName);
	if(Tracks.IsValidIndex(TrackIndex))
	{
		return GetTrack(TrackIndex);
	}
	return nullptr;
}

TSharedPtr<FSampleTrackBase> FSampleTrackContainer::FindTrack(const FName& InName)
{
	const int32 TrackIndex = GetTrackIndex(InName);
	if(Tracks.IsValidIndex(TrackIndex))
	{
		return GetTrack(TrackIndex);
	}
	return nullptr;
}

void FSampleTrackContainer::AddTrack(TSharedPtr<FSampleTrackBase> InTrack, bool bCreateChildTracks)
{
	TArray<int32> ChildTrackIndices;
	if(InTrack->IsComposed() && bCreateChildTracks)
	{
		const TArray<FSampleTrackBase::ETrackType> ChildTrackTypes = InTrack->GetChildTrackTypes();
		for(int32 ChildTrackIndex = 0; ChildTrackIndex < ChildTrackTypes.Num(); ChildTrackIndex++)
		{
			const FSampleTrackBase::ETrackType ChildTrackType = ChildTrackTypes[ChildTrackIndex];
			
			FName ChildTrackName = *(TEXT("ChildTrack_") + InTrack->GetChildTrackNameSuffix(ChildTrackIndex));
			if(!InTrack->Names.IsEmpty())
			{
				ChildTrackName = *(InTrack->GetName().ToString() + TEXT("_") + InTrack->GetChildTrackNameSuffix(ChildTrackIndex));
			}
			TSharedPtr<FSampleTrackBase> ChildTrack = MakeTrack(ChildTrackType);
			ChildTrack->Names.Reset();
			ChildTrack->Names.Add(ChildTrackName);
			AddTrack(ChildTrack);
			ChildTrackIndices.Add(ChildTrack->GetTrackIndex());
		}
	}
	
	InTrack->TrackIndex = Tracks.Num();
	InTrack->Container = this;
	InTrack->SetChildTracks(ChildTrackIndices);

	// a track can have many names - so we need to make sure each one is unique
	for(int32 Index = 0; Index < InTrack->Names.Num(); Index++)
	{
		FName& Name = InTrack->Names[Index];
		if(Name.IsNone())
		{
			continue;
		}
		
		int32 NameSuffix = 3; // causes the first duplicate name to be called "Foo_2"
		while (NameToIndex.Contains(Name))
		{
			if(NameSuffix == 3)
			{
				Name = FName(Name, NameSuffix);
			}
			else
			{
				Name.SetNumber(NameSuffix);
			}
			NameSuffix++;
		}
		NameToIndex.Add(Name, InTrack->TrackIndex);
	}
	
	Tracks.Add(InTrack);
}

TSharedPtr<FSampleTrackBase> FSampleTrackContainer::MakeTrack(FSampleTrackBase::ETrackType InTrackType)
{
	switch(InTrackType)
	{
		case FSampleTrackBase::ETrackType_Bool:
		{
			return MakeShareable(new FSampleTrack<bool>());
		}
		case FSampleTrackBase::ETrackType_Int32:
		{
			return MakeShareable(new FSampleTrack<int32>());
		}
		case FSampleTrackBase::ETrackType_Uint32:
		{
			return MakeShareable(new FSampleTrack<uint32>());
		}
		case FSampleTrackBase::ETrackType_Float:
		{
			return MakeShareable(new FSampleTrack<float>());
		}
		case FSampleTrackBase::ETrackType_Name:
		{
			return MakeShareable(new FSampleTrack<FName>());
		}
		case FSampleTrackBase::ETrackType_String:
		{
			return MakeShareable(new FSampleTrack<FString>());
		}
		case FSampleTrackBase::ETrackType_Vector3f:
		{
			return MakeShareable(new FSampleTrack<FVector3f>());
		}
		case FSampleTrackBase::ETrackType_Quatf:
		{
			return MakeShareable(new FSampleTrack<FQuat4f>());
		}
		case FSampleTrackBase::ETrackType_Transformf:
		{
			return MakeShareable(new FComposedSampleTrack<FTransform3f>());
		}
		case FSampleTrackBase::ETrackType_LinearColor:
		{
			return MakeShareable(new FSampleTrack<FLinearColor>());
		}
		case FSampleTrackBase::ETrackType_ElementKey:
		{
			return MakeShareable(new FSampleTrack<FRigElementKey>());
		}
		case FSampleTrackBase::ETrackType_ComponentKey:
		{
			return MakeShareable(new FSampleTrack<FRigComponentKey>());
		}
		case FSampleTrackBase::ETrackType_Struct:
		{
			return MakeShareable(new FSampleTrack<FInstancedStruct>());
		}
		case FSampleTrackBase::ETrackType_BoolArray:
		{
			return MakeShareable(new FSampleTrack<TArray<bool>>());
		}
		case FSampleTrackBase::ETrackType_Int32Array:
		{
			return MakeShareable(new FSampleTrack<TArray<int32>>());
		}
		case FSampleTrackBase::ETrackType_Uint32Array:
		{
			return MakeShareable(new FSampleTrack<TArray<uint32>>());
		}
		case FSampleTrackBase::ETrackType_FloatArray:
		{
			return MakeShareable(new FSampleTrack<TArray<float>>());
		}
		case FSampleTrackBase::ETrackType_NameArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FName>>());
		}
		case FSampleTrackBase::ETrackType_StringArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FString>>());
		}
		case FSampleTrackBase::ETrackType_Vector3fArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FVector3f>>());
		}
		case FSampleTrackBase::ETrackType_QuatfArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FQuat4f>>());
		}
		case FSampleTrackBase::ETrackType_TransformfArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FTransform3f>>());
		}
		case FSampleTrackBase::ETrackType_LinearColorArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FLinearColor>>());
		}
		case FSampleTrackBase::ETrackType_ElementKeyArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FRigElementKey>>());
		}
		case FSampleTrackBase::ETrackType_ComponentKeyArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FRigComponentKey>>());
		}
		case FSampleTrackBase::ETrackType_StructArray:
		{
			return MakeShareable(new FSampleTrack<TArray<FInstancedStruct>>());
		}

		default:
		{
			checkNoEntry();
			break;
		}
	}
	return nullptr;
}

void FSampleTrackContainer::RemoveInvalidTracks(bool bUpdateNameToIndexMap)
{
	TArray<int32> OldTrackIndexToNewTrackIndex;
	OldTrackIndexToNewTrackIndex.AddUninitialized(Tracks.Num());
	int32 MaxTrackIndex = 0;
	for(int32 TrackIndex = 0; TrackIndex < Tracks.Num(); TrackIndex++)
	{
		if(Tracks[TrackIndex].IsValid() && Tracks[TrackIndex]->IsValid())
		{
			OldTrackIndexToNewTrackIndex[TrackIndex] = MaxTrackIndex++;
		}
		else
		{
			OldTrackIndexToNewTrackIndex[TrackIndex] = INDEX_NONE;
		}
	}
	const int32 NumRemoved = Tracks.RemoveAll([](const TSharedPtr<FSampleTrackBase>& Track)
	{
		return !Track.IsValid() || !Track->IsValid();
	});

	if(NumRemoved > 0)
	{
		UpdateTrackIndices(OldTrackIndexToNewTrackIndex);
		if(bUpdateNameToIndexMap)
		{
			UpdateNameToIndexMap();
		}
	}
}

void FSampleTrackContainer::RemoveRedundantTracks(bool bUpdateNameToIndexMap, float InTolerance)
{
	struct FTrackInfo
	{
		FSampleTrackBase::ETrackType Type;
		FSampleTrackBase::EMode Mode;
		int32 Num;
		int32 NumSamples;

		bool operator!=(const FTrackInfo& InOther) const
		{
			return (Type != InOther.Type) || (Mode != InOther.Mode) || (Num != InOther.Num) || (NumSamples != InOther.NumSamples);
		}
	};
	
	TArray<FTrackInfo> TrackInfos;
	TrackInfos.Reserve(Tracks.Num());
	
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		FTrackInfo Info;
		if(!Track.IsValid() || !Track->IsValid())
		{
			Info.Mode = FSampleTrackBase::EMode_Invalid;
			TrackInfos.Add(Info);
			continue;
		}
		Info.Type = Track->GetTrackType();
		Info.Mode = Track->GetMode();
		Info.Num = Track->NumTimes();
		Info.NumSamples = Track->NumSamples();
		TrackInfos.Add(Info);
	}

	int32 NumTracksMerged = 0;
	TArray<bool> MergedTracks;
	MergedTracks.AddZeroed(Tracks.Num());
	TArray<int32> OldTrackIndexToNewTrackIndex;
	OldTrackIndexToNewTrackIndex.Reserve(Tracks.Num());

	for(int32 Index = 0; Index < Tracks.Num(); Index++)
	{
		OldTrackIndexToNewTrackIndex.Add(Index);
	}

	for(int32 IndexA = 0; IndexA < Tracks.Num() - 1; IndexA++)
	{
		if(MergedTracks[IndexA])
		{
			continue;
		}

		if(TrackInfos[IndexA].Mode == FSampleTrackBase::EMode_Invalid)
		{
			continue;
		}
		
		TSharedPtr<FSampleTrackBase>& TrackA = Tracks[IndexA];

		for(int32 IndexB = IndexA + 1; IndexB < Tracks.Num(); IndexB++)
		{
			if(MergedTracks[IndexB])
			{
				continue;
			}
			if(TrackInfos[IndexA].Mode == FSampleTrackBase::EMode_Invalid)
			{
				continue;
			}
			
			if(TrackInfos[IndexA] != TrackInfos[IndexB])
			{
				continue;
			}
			
			TSharedPtr<FSampleTrackBase>& TrackB = Tracks[IndexB];
			if(!TrackA->IsIdentical(TrackB.Get(), InTolerance))
			{
				continue;
			}

			// merge the track
			TrackA->Names.Append(TrackB->Names);
			TrackA->Names.Remove(NAME_None);
			MergedTracks[IndexB] = true;
			OldTrackIndexToNewTrackIndex[IndexB] = IndexA;
			NumTracksMerged++;
		}
	}
	
	if(NumTracksMerged == 0)
	{
		return;
	}

	for(int32 Index = 0; Index < Tracks.Num(); Index++)
	{
		if(MergedTracks[Index])
		{
			Tracks[Index].Reset();
		}
	}

	UpdateTrackIndices(OldTrackIndexToNewTrackIndex);
	RemoveInvalidTracks(bUpdateNameToIndexMap);
}

void FSampleTrackContainer::MergeTypedTracks(bool bUpdateNameToIndexMap, float InTolerance)
{
	struct FTrackGroup
	{
		FTrackGroup()
		: TrackType(FSampleTrackBase::ETrackType_Unknown)
		, ScriptStruct(nullptr)
		{
		}

		FTrackGroup(const TSharedPtr<FSampleTrackBase>& InTrack)
		: TrackType(InTrack->GetTrackType())
		, ScriptStruct(InTrack->GetScriptStruct())
		{
			Tracks.Add(InTrack);
		}

		bool MergeTrack(const TSharedPtr<FSampleTrackBase>& InTrack)
		{
			if(TrackType != InTrack->GetTrackType())
			{
				return false;
			}
			if(ScriptStruct != InTrack->GetScriptStruct())
			{
				return false;
			}
			Tracks.Add(InTrack); 
			return true;
		}
		
		FSampleTrackBase::ETrackType TrackType;
		const UScriptStruct* ScriptStruct;
		TArray<TSharedPtr<FSampleTrackBase>> Tracks;
	};
	
	TArray<FTrackGroup> TrackGroups;
	for(int32 TrackIndex = 0; TrackIndex < Tracks.Num(); TrackIndex++)
	{
		TSharedPtr<FSampleTrackBase>& Track = Tracks[TrackIndex];
		if(!Track.IsValid() || !Track->IsValid() || Track->IsReferenced() || Track->IsComposed())
		{
			continue;
		}

		// make sure this track's index is up2date
		Track->TrackIndex = TrackIndex;

		bool bMergedTrack = false;
		for(FTrackGroup& TrackGroup : TrackGroups)
		{
			if(TrackGroup.MergeTrack(Track))
			{
				bMergedTrack = true;
				break;
			}
		}

		if(!bMergedTrack)
		{
			TrackGroups.Emplace(Track);
		}
	}

	TrackGroups.RemoveAll([](const FTrackGroup& TrackGroup)
	{
		return TrackGroup.Tracks.Num() <= 1;
	});

	for(FTrackGroup& TrackGroup : TrackGroups)
	{
		TSharedPtr<FSampleTrackBase> CombinedTrack = MakeTrack(TrackGroup.TrackType);
		CombinedTrack->Names.Reset();
		CombinedTrack->Mode = FSampleTrackBase::EMode_Raw;
		CombinedTrack->ScriptStruct = TrackGroup.ScriptStruct;
		AddTrack(CombinedTrack);

		int32 CombinedNumSamples = 0;
		int32 CombinedNumValues = 0;
		for(const TSharedPtr<FSampleTrackBase>& Track : TrackGroup.Tracks)
		{
			CombinedNumSamples += Track->NumSamples();
			CombinedNumValues += Track->NumStoredValues();
		}
		CombinedTrack->Reserve(CombinedNumSamples, CombinedNumValues);

		for(TSharedPtr<FSampleTrackBase>& Track : TrackGroup.Tracks)
		{
			// remove the atlas - this gives us the opportunity
			// to apply an atlas on the combined track
			Track->RemoveAtlas();

			// also give the track a chance to unroll its values again to save
			// on time index list memory.
			Track->ConvertToComplete();

			// shrink the memory use of the track as it is right now
			Track->Shrink();

			// copy samples and values over into the combined track 
			const int32 FirstTimeIndex = CombinedTrack->TimeIndicesStorage.Num();
			CombinedTrack->TimeIndicesStorage.Append(Track->TimeIndicesStorage);
			const int32 FirstValueIndex = CombinedTrack->AppendValuesFromTrack(Track.Get());

			const TTuple<int32,int32> TimeIndicesRange = {FirstTimeIndex, Track->TimeIndicesStorage.Num()}; 
			const TTuple<int32,int32> ValuesRange = {FirstValueIndex, Track->NumStoredValues()}; 

			// set up the referencing track
			Track->Empty();
			Track->ReferencedTrackIndex = CombinedTrack->GetTrackIndex();
			Track->ReferencedTimeIndicesRange = TimeIndicesRange;
			Track->ReferencedAtlasRange = {INDEX_NONE, INDEX_NONE};
			Track->ReferencedValuesRange = ValuesRange;
			Track->UpdateArrayViews();
		}

		// create an atlas for the merged track. if that's successful we'll have to upgrade our tracks' atlas range
		if(CombinedTrack->AddAtlas(false, InTolerance))
		{
			for(TSharedPtr<FSampleTrackBase>& Track : TrackGroup.Tracks)
			{
				// the size of used atlas is going to be size of the values
				Track->ReferencedAtlasRange = Track->ReferencedValuesRange;

				// for the values we'll use the whole value array (since the atlas indexes into that anyway) 
				Track->ReferencedValuesRange = {0, CombinedTrack->NumStoredValues()};
				Track->UpdateArrayViews();
			}
		}
	}

	if(!TrackGroups.IsEmpty() && bUpdateNameToIndexMap)
	{
		UpdateNameToIndexMap();
	}
}

void FSampleTrackContainer::EnableTrackAtlas(float InTolerance)
{
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		(void)Track->AddAtlas(false, InTolerance);
	}
}

void FSampleTrackContainer::ConvertTracksToComplete()
{
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		(void)Track->ConvertToComplete();
	}
}

void FSampleTrackContainer::ConvertTracksToSampled(float InTolerance)
{
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		(void)Track->ConvertToSampled(false, InTolerance);
	}
}

void FSampleTrackContainer::UpdateTrackIndices(const TArray<int32>& InNewTrackIndices)
{
	for(TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		if(!Track.IsValid() || !Track->IsValid())
		{
			continue;
		}
		if(Track->ReferencedTrackIndex != INDEX_NONE)
		{
			Track->ReferencedTrackIndex = InNewTrackIndices[Track->ReferencedTrackIndex];
			Track->UpdateArrayViews();
		}
		if(Track->IsComposed())
		{
			TArray<int32> ChildTrackIndices = Track->GetChildTracks();
			for(int32& ChildTrackIndex : ChildTrackIndices)
			{
				ChildTrackIndex = InNewTrackIndices[ChildTrackIndex];
			}
			Track->SetChildTracks(ChildTrackIndices);
		}
	}
}

void FSampleTrackContainer::UpdateNameToIndexMap()
{
	NameToIndex.Reset();

	for(int32 TrackIndex = 0; TrackIndex < Tracks.Num(); TrackIndex++)
	{
		Tracks[TrackIndex]->TrackIndex = TrackIndex;
		for(const FName& Name : Tracks[TrackIndex]->GetAllNames())
		{
			if(!Name.IsNone())
			{
				check(!NameToIndex.Contains(Name));
				NameToIndex.Add(Name, TrackIndex);
			}
		}
	}
}

int32 FSampleTrackContainer::AddTimeSample(float InAbsoluteTime, float InDeltaTime)
{
	const TSharedPtr<FSampleTrack<float>> AbsoluteTimeTrack = FindOrAddTrack<float>(AbsoluteTimeName, FSampleTrackBase::ETrackType_Float);
	const TSharedPtr<FSampleTrack<float>> DeltaTimeTrack = FindOrAddTrack<float>(DeltaTimeName, FSampleTrackBase::ETrackType_Float);
	const int32 AddedIndex = AbsoluteTimeTrack->NumTimes();
	AbsoluteTimeTrack->AddSample(InAbsoluteTime);
	DeltaTimeTrack->AddSample(InDeltaTime);
	return AddedIndex;
}

int32 FSampleTrackContainer::AddTimeSampleFromDeltaTime(float InDeltaTime)
{
	const float LastTime = GetLastAbsoluteTime() + GetLastDeltaTime();
	return AddTimeSample(LastTime, InDeltaTime);
}

bool FSampleTrackContainer::IsEditable() const
{
	for(const TSharedPtr<FSampleTrackBase>& Track : Tracks)
	{
		if(Track->IsReferenced())
		{
			return false;
		}
		if(Track->UsesAtlas())
		{
			return false;
		}
		if(Track->GetMode() != FSampleTrackBase::EMode_Complete &&
			Track->GetMode() != FSampleTrackBase::EMode_Singleton)
		{
			return false;
		}
	}
	return true;
}

bool FSampleTrackContainer::MakeEditable()
{
	if(IsEditable())
	{
		return true;
	}

	bool bChangedSomething = false;
	TArray<int32> TracksToRemove;
	for(int32 TrackIndex = 0; TrackIndex < Tracks.Num(); TrackIndex++)
	{
		if(Tracks[TrackIndex]->GetMode() == FSampleTrackBase::EMode_Raw)
		{
			TracksToRemove.Add(TrackIndex);
		}

		Tracks[TrackIndex]->UpdateArrayViews();
		if(Tracks[TrackIndex]->LocalizeValues())
		{
			bChangedSomething = true;
		}
		if(Tracks[TrackIndex]->RemoveAtlas())
		{
			bChangedSomething = true;
		}
		if(Tracks[TrackIndex]->ConvertToComplete(true))
		{
			bChangedSomething = true;
		}
	}

	Algo::Reverse(TracksToRemove);
	for(const int32& IndexToRemove : TracksToRemove)
	{
		Tracks.RemoveAt(IndexToRemove);
		bChangedSomething = true;
	}

	return bChangedSomething;
}


