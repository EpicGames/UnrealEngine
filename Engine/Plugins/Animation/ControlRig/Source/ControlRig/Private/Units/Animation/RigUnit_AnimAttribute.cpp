// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Animation/RigUnit_AnimAttribute.h"

#include "Animation/AttributeTypes.h"
#include "Engine/SkeletalMesh.h"
#include "Units/RigUnitContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "StructUtils/UserDefinedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimAttribute)

bool FRigDispatch_AnimAttributeBase::IsTypeSupported(const TRigVMTypeIndex& InTypeIndex, const FRigVMRegistryHandle& InRegistry) const
{
	if(SpecialTypes.IsEmpty())
	{
		SpecialTypes = {
			RigVMTypeUtils::TypeIndex::Float,
			RigVMTypeUtils::TypeIndex::Int32,
			RigVMTypeUtils::TypeIndex::FString,
			InRegistry->GetTypeIndex_NoLock<FTransform>(false),
			InRegistry->GetTypeIndex_NoLock<FVector>(false),
			InRegistry->GetTypeIndex_NoLock<FQuat>(false),
		};
	}

	if (SpecialTypes.Contains(InTypeIndex))
	{
		return true;
	}

	const FRigVMTemplateArgumentType& InType = InRegistry->GetType_NoLock(InTypeIndex);

	// cpp type object can become invalid because users can choose to delete
	// user defined structs
	if (IsValid(InType.CPPTypeObject))
	{
		if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InType.CPPTypeObject))
		{
			if (ScriptStruct->IsA<UUserDefinedStruct>())
			{
				// allow all user defined structs because even if a struct is not registered with anim attribute system,
				// it could be added to or removed from the system easily. allowing all of them as valid permutations
				// avoids having to create orphan pins.
				return true;
			}

			static const TArray<TWeakObjectPtr<const UScriptStruct>> SpecialAttributeTypes = {
				FFloatAnimationAttribute::StaticStruct(),
				FIntegerAnimationAttribute::StaticStruct(),
				FStringAnimationAttribute::StaticStruct(),
				FTransformAnimationAttribute::StaticStruct(),
				FVectorAnimationAttribute::StaticStruct(),
				FQuaternionAnimationAttribute::StaticStruct()
			};

			if (SpecialAttributeTypes.Contains(ScriptStruct))
			{
				// these type have been added, above, rejecting them here so we don't have duplicated types
				return false;
			}
		
			return UE::Anim::AttributeTypes::IsTypeRegistered(ScriptStruct);
		}
	}

	return false;
}


const TArray<FRigVMTemplateArgument::ETypeCategory>& FRigDispatch_AnimAttributeBase::GetValueTypeCategory()
{
	static const TArray<FRigVMTemplateArgument::ETypeCategory> TypeCategories = {
		FRigVMTemplateArgument::ETypeCategory_SingleSimpleValue,
		FRigVMTemplateArgument::ETypeCategory_SingleMathStructValue,
		FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue
	};
	return TypeCategories;
}


static uint8* GetAnimAttributeValue(
	bool bAddIfNotFound,
	UScriptStruct* InAttributeScriptStruct,
	const FControlRigExecuteContext& Context,
	const FName& Name,
	const FName& BoneName,
	FName& CachedBoneName,
	int32& CachedBoneIndex)
{
	if (!ensure(InAttributeScriptStruct))
	{
		return nullptr;
	}
	
	if (Name.IsNone())
	{
		return nullptr;
	}

	if (!Context.UnitContext.AnimAttributeContainer)
	{
		return nullptr;
	}
	
	const USkeletalMeshComponent* OwningComponent = Cast<USkeletalMeshComponent>(Context.GetOwningComponent());

	if (!OwningComponent ||
		!OwningComponent->GetSkeletalMeshAsset())
	{
		return nullptr;
	}

	if (BoneName == NAME_None)
	{
		// default to use root bone
		CachedBoneIndex = 0;
	}
	else
	{
		// Invalidate cache if input changed
		if (CachedBoneName != BoneName)
		{
			CachedBoneIndex = OwningComponent->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(BoneName);
		}
	}
	
	CachedBoneName = BoneName;

	if (CachedBoneIndex != INDEX_NONE)
	{
		const UE::Anim::FAttributeId Id = {Name, FCompactPoseBoneIndex(CachedBoneIndex)} ;
		uint8* Attribute = bAddIfNotFound ?
			Context.UnitContext.AnimAttributeContainer->FindOrAdd(InAttributeScriptStruct, Id) :
			Context.UnitContext.AnimAttributeContainer->Find(InAttributeScriptStruct, Id);
		
		if (Attribute)
		{
			return Attribute;
		}
	}

	return nullptr;
}

void FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch(FRigVMExtendedExecuteContext& InContext,
	FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FRigDispatch_GetAnimAttribute* Factory = static_cast<const FRigDispatch_GetAnimAttribute*>(InContext.Factory);
		
#if WITH_EDITOR
	if(!Factory->CheckArgumentTypes(Handles))
	{
		return;
	}
#endif

	// unpack the memory
	const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
	const FName& BoneName = *(const FName*)Handles[Factory->BoneNameArgIndex].GetInputData();
	const uint8* Default = (const uint8*)Handles[Factory->DefaultArgIndex].GetInputData();
	
	uint8* Value = (uint8*)Handles[Factory->ValueArgIndex].GetOutputData();
	bool& Found = *(bool*)Handles[Factory->FoundArgIndex].GetOutputData();
	Found = false;
	
	FName& CachedBoneName = *(FName*)Handles[Factory->CachedBoneNameArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
	int32& CachedBoneIndex = *(int32*)Handles[Factory->CachedBoneIndexArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
	
	if (const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Handles[Factory->ValueArgIndex].GetResolvedProperty()))
	{
		UScriptStruct* ScriptStruct = StructProperty->Struct;
		bool Registered = true;
		const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();

#if WITH_EDITOR
		{
			FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
			if (!UE::Anim::AttributeTypes::IsTypeRegistered(ScriptStruct))
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(
					TEXT("Type: '%s' is not registered with the Animation Attribute System. "
						"Please register the type in Project Settings - Animation - CustomAttributes - User Defined Struct Attributes."),
					*ScriptStruct->GetAuthoredName());
				Registered = false;
			}
		}
#endif
		
		if (Registered)
		{
			// extract the animation attribute
			const uint8* ValuePtr = GetAnimAttributeValue(false, ScriptStruct, Context, Name, BoneName, CachedBoneName, CachedBoneIndex);
			Found = ValuePtr ? true : false;

			if (Found)
			{
				ScriptStruct->CopyScriptStruct(Value, ValuePtr);		
			}
		}

		if (!Found)
		{
			ScriptStruct->CopyScriptStruct(Value, Default);		
		}
	}
}

void FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch(FRigVMExtendedExecuteContext& InContext,
	FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FRigDispatch_SetAnimAttribute* Factory = static_cast<const FRigDispatch_SetAnimAttribute*>(InContext.Factory);
		
#if WITH_EDITOR
	if(!Factory->CheckArgumentTypes(Handles))
	{
		return;
	}
#endif

	// unpack the memory
	const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
	const FName& BoneName = *(const FName*)Handles[Factory->BoneNameArgIndex].GetInputData();
	const uint8* Value = (uint8*)Handles[Factory->ValueArgIndex].GetInputData();
	
	bool& Success = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();
	Success = false;
	
	FName& CachedBoneName = *(FName*)Handles[Factory->CachedBoneNameArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
	int32& CachedBoneIndex = *(int32*)Handles[Factory->CachedBoneIndexArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
	
	if (const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Handles[Factory->ValueArgIndex].GetResolvedProperty()))
	{
		UScriptStruct* ScriptStruct = StructProperty->Struct;
		bool Registered = true;
		const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();

#if WITH_EDITOR
		{
			FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
			if (!UE::Anim::AttributeTypes::IsTypeRegistered(ScriptStruct))
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(
					TEXT("Type: '%s' is not registered in the Animation Attribute System. "
						"Please register the type in Project Settings - Animation - CustomAttributes - User Defined Struct Animation Attributes."),
					*ScriptStruct->GetAuthoredName());
				Registered = false;
			}
		}
#endif
		
		if (Registered)
		{
			// extract the animation attribute
			uint8* ValuePtr = GetAnimAttributeValue(true, ScriptStruct, Context, Name, BoneName, CachedBoneName, CachedBoneIndex);

			if (ValuePtr)
			{
				Success = true;
				ScriptStruct->CopyScriptStruct(ValuePtr, Value);		
			}
		}
	}	
}

void FRigDispatch_AnimAttributeBase::RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	Super::RegisterDependencyTypes_NoLock(InRegistry);

	InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(), true);
	
	TArray<TWeakObjectPtr<const UScriptStruct>> AttributeTypes = UE::Anim::AttributeTypes::GetRegisteredTypes();
	for (TWeakObjectPtr<const UScriptStruct> Type : AttributeTypes)
	{
		UScriptStruct* TypePtr = const_cast<UScriptStruct*>(Type.Get());
		if (TypePtr)
		{
			InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(TypePtr));
		}
	}
}

#if WITH_EDITOR
FString FRigDispatch_AnimAttributeBase::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	if(const TRigVMTypeIndex* ValueTypeIndexPtr = InTypes.Find(ValueArgName))
	{
		const TRigVMTypeIndex& ValueTypeIndex = *ValueTypeIndexPtr;
		if(ValueTypeIndex != RigVMTypeUtils::TypeIndex::WildCard &&
			ValueTypeIndex != RigVMTypeUtils::TypeIndex::WildCardArray)
		{
			const FRigVMTemplateArgumentType& ValueType = FRigVMRegistry::Get().GetType(ValueTypeIndex);
			FString ValueName;

			if(ValueType.CPPTypeObject)
			{
				ValueName = ValueType.CPPTypeObject->GetName();
			}
			else
			{
				ValueName = ValueType.GetBaseCPPType();
				ValueName = ValueName.Left(1).ToUpper() + ValueName.Mid(1);
			}

			return FString::Printf(TEXT("%s %s Animation Attribute"), IsSet() ? TEXT("Set") : TEXT("Get"), *ValueName);
		}
	}
	return FRigDispatchFactory::GetNodeTitle(InTypes);	
}
#endif

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_AnimAttributeBase::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		NameArgIndex = CachedArgumentInfos.Emplace(NameArgName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);
		BoneNameArgIndex = CachedArgumentInfos.Emplace(BoneNameArgName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);

		CachedBoneNameArgIndex = CachedArgumentInfos.Emplace(CachedBoneNameArgName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::FName);
		CachedBoneIndexArgIndex = CachedArgumentInfos.Emplace(CachedBoneIndexArgName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::Int32);
	}
	
	return CachedArgumentInfos;
}

#if WITH_EDITOR
FText FRigDispatch_AnimAttributeBase::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == NameArgName)
	{
		return NSLOCTEXT("FRigDispatch_AnimAttributeBase", "NameArgTooltip", "The name of the animation attribute");
	}
	if(InArgumentName == BoneNameArgName)
	{
		return NSLOCTEXT("FRigDispatch_AnimAttributeBase", "BoneNameArgTooltip", "The name of the bone that stores the attribute, default to root bone if set to none");
	}
	if(InArgumentName == DefaultArgName)
	{
		return NSLOCTEXT("FRigDispatch_AnimAttributeBase", "DefaultArgTooltip", "The default value used as a fallback if the animation attribute does not exist");
	}
	if(InArgumentName == ValueArgName)
	{
		return NSLOCTEXT("FRigDispatch_AnimAttributeBase", "ValueArgTooltip", "The value to get / set");
	}
	if(InArgumentName == FoundArgName)
	{
		return NSLOCTEXT("FRigDispatch_AnimAttributeBase", "FoundArgTooltip", "Returns true if the animation attribute exists with the specific type");
	}
	if(InArgumentName == SuccessArgName)
	{
		return NSLOCTEXT("FRigDispatch_AnimAttributeBase", "SuccessArgTooltip", "Returns true if the animation attribute was successfully stored");
	}
	return FRigDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);	
}




#endif



const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_GetAnimAttribute::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos = Super::GetArgumentInfos(InRegistry); 

		// Will be open to any category, but will filter the type through our IsTypeSupported callback
		// If we reduce this to multiple (more targeted) categories, and any of those categories have common types, bUseCategories will not be true
		// and the template will not receive notifications of newly added types
		const TArray<FRigVMTemplateArgument::ETypeCategory> Categories = {FRigVMTemplateArgument::ETypeCategory_SingleAnyValue};
		DefaultArgIndex = CachedArgumentInfos.Emplace(DefaultArgName, ERigVMPinDirection::Input, Categories, [this](const TRigVMTypeIndex& Type, const FRigVMRegistryHandle& InRegistry) { return IsTypeSupported(Type, InRegistry); });
		ValueArgIndex = CachedArgumentInfos.Emplace(ValueArgName, ERigVMPinDirection::Output, Categories, [this](const TRigVMTypeIndex& Type, const FRigVMRegistryHandle& InRegistry) { return IsTypeSupported(Type, InRegistry); });
		
		FoundArgIndex = CachedArgumentInfos.Emplace(FoundArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}

	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigDispatch_GetAnimAttribute::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;

	// similar pattern to FRigVMDispatch_ArrayGetAtIndex::OnNewArgumentType
	// we used to need the if check below to avoid double registration for both default and value arguments, but since CL 29742624
	// only the primary argument is used to add new permutations and thus this check becomes optional.
	// primary argument is defined as the first argument that uses category, see FRigVMTemplate::UpdateAllArgumentTypesSlow(), and in this case
	// default is the first argument instead of the value
	
	if (ensure(InArgumentName == DefaultArgName))
	{
		if (IsTypeSupported(InTypeIndex, InRegistry))
		{
			Types.Add(NameArgName, RigVMTypeUtils::TypeIndex::FName);
			Types.Add(BoneNameArgName, RigVMTypeUtils::TypeIndex::FName);
			Types.Add(DefaultArgName, InTypeIndex);
			Types.Add(ValueArgName, InTypeIndex);
			Types.Add(FoundArgName, RigVMTypeUtils::TypeIndex::Bool);
			Types.Add(CachedBoneNameArgName, RigVMTypeUtils::TypeIndex::FName);
			Types.Add(CachedBoneIndexArgName, RigVMTypeUtils::TypeIndex::Int32);
		}
	}
	return Types;
}

FRigVMFunctionPtr FRigDispatch_GetAnimAttribute::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));

	if (ValueTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch<float>;
	}
	if (ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch<int32>;
	}
	if (ValueTypeIndex == RigVMTypeUtils::TypeIndex::FString)
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch<FString>;
	}
	if (ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(false))
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch<FTransform>;
	}
	if (ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(false))
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch<FQuat>;
	}
	if (ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(false))
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch<FVector>;
	}
	
	const FRigVMTemplateArgumentType& ValueType = InRegistry->GetType_NoLock(ValueTypeIndex);
	
	if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ValueType.CPPTypeObject))
	{
		return &FRigDispatch_GetAnimAttribute::GetAnimAttributeDispatch;
	}

	return nullptr;
}

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_SetAnimAttribute::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos = Super::GetArgumentInfos(InRegistry);

		// Will be open to any category, but will filter the type through our IsTypeSupported callback
		// If we reduce this to multiple (more targeted) categories, and any of those categories have common types, bUseCategories will not be true
		// and the template will not receive notifications of newly added types
		static const TArray<FRigVMTemplateArgument::ETypeCategory> Categories = {FRigVMTemplateArgument::ETypeCategory_SingleAnyValue};
		
		ValueArgIndex = CachedArgumentInfos.Emplace(ValueArgName, ERigVMPinDirection::Input, Categories, [this](const TRigVMTypeIndex& Type, const FRigVMRegistryHandle& InRegistry) { return IsTypeSupported(Type, InRegistry); });		
		SuccessArgIndex = CachedArgumentInfos.Emplace(SuccessArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	
	return CachedArgumentInfos;
}

const TArray<FRigVMExecuteArgument>& FRigDispatch_SetAnimAttribute::GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const
{
	static const TArray<FRigVMExecuteArgument> ExecuteArguments = {
		{TEXT("ExecuteContext"), ERigVMPinDirection::IO}
	};
	return ExecuteArguments;
}

FRigVMTemplateTypeMap FRigDispatch_SetAnimAttribute::OnNewArgumentType(const FName& InArgumentName,
                                                                       TRigVMTypeIndex InTypeIndex,
                                                                       FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;

	// similar pattern to URigVMArrayNode's FRigVMTemplate_NewArgumentTypeDelegate to avoid double registration
	// likely not strictly needed for this node since it only has a single non-singleton argument
	if (InArgumentName == ValueArgName)
	{
		if (IsTypeSupported(InTypeIndex, InRegistry))
		{
			Types.Add(NameArgName, RigVMTypeUtils::TypeIndex::FName);
			Types.Add(BoneNameArgName, RigVMTypeUtils::TypeIndex::FName);
			Types.Add(ValueArgName, InTypeIndex);
			Types.Add(SuccessArgName, RigVMTypeUtils::TypeIndex::Bool);
			Types.Add(CachedBoneNameArgName, RigVMTypeUtils::TypeIndex::FName);
			Types.Add(CachedBoneIndexArgName, RigVMTypeUtils::TypeIndex::Int32);
		}
	}
	
	return Types;	
}

FRigVMFunctionPtr FRigDispatch_SetAnimAttribute::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));

	if (ValueTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch<float>;
	}
	if (ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch<int32>;
	}
	if (ValueTypeIndex == RigVMTypeUtils::TypeIndex::FString)
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch<FString>;
	}
	if (ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(false))
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch<FTransform>;
	}
	if (ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(false))
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch<FQuat>;
	}
	if (ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(false))
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch<FVector>;
	}
	
	const FRigVMTemplateArgumentType& ValueType = InRegistry->GetType_NoLock(ValueTypeIndex);
	
	if (Cast<UScriptStruct>(ValueType.CPPTypeObject))
	{
		return &FRigDispatch_SetAnimAttribute::SetAnimAttributeDispatch;
	}

	return nullptr;
}


