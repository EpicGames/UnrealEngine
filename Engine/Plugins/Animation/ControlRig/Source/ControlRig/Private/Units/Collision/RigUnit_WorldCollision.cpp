// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Collision/RigUnit_WorldCollision.h"
#include "Engine/World.h"
#include "Math/ControlRigMathLibrary.h"
#include "Units/RigUnitContext.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_WorldCollision)

FRigUnit_SphereTraceWorld_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    bHit = false;
	HitLocation = FVector::ZeroVector;
	HitNormal = FVector(0.f, 0.f, 1.f);

	if(ExecuteContext.GetWorld() == nullptr)
	{
		WorkData.Reset();
		return;
	}

	// Note: we are not adding the world transform to the hash since
	// the world transform cannot change during one execution of the 
	// rig. By adding NumExecutions to the hash we make sure to
	// trace again once the world transform has changed.
	uint32 Hash = GetTypeHash(ExecuteContext.GetNumExecutions());
	Hash = HashCombine(Hash, GetTypeHash(Start));
	Hash = HashCombine(Hash, GetTypeHash(End));
	Hash = HashCombine(Hash, GetTypeHash((int32)Channel));
	Hash = HashCombine(Hash, GetTypeHash(Radius));

	if(WorkData.Hash == Hash)
	{
		bHit = WorkData.bHit;
		HitLocation = WorkData.HitLocation;
		HitNormal = WorkData.HitNormal;
		return;
	}
	
	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = true;

	if (ExecuteContext.GetOwningActor())
	{
		QueryParams.AddIgnoredActor(ExecuteContext.GetOwningActor());
	}
	else if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ExecuteContext.GetOwningComponent()))
	{
		QueryParams.AddIgnoredComponent(PrimitiveComponent);
	}
	
	FCollisionResponseParams ResponseParams(ECR_Ignore);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldStatic, ECR_Block);

	const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(Radius);

	FHitResult HitResult;
	bHit = ExecuteContext.GetWorld()->SweepSingleByChannel(HitResult, ExecuteContext.ToWorldSpace(Start), ExecuteContext.ToWorldSpace(End), 
			FQuat::Identity, Channel, CollisionShape, QueryParams, ResponseParams);

	if (bHit)
	{
		HitLocation = ExecuteContext.ToVMSpace(HitResult.ImpactPoint);
		HitNormal = ExecuteContext.GetToWorldSpaceTransform().InverseTransformVector(HitResult.ImpactNormal);
	}

	WorkData.Hash = Hash;
	WorkData.bHit = bHit;
	WorkData.HitLocation = HitLocation;
	WorkData.HitNormal = HitNormal;
}

FRigVMStructUpgradeInfo FRigUnit_SphereTraceWorld::GetUpgradeInfo() const
{
	FRigUnit_SphereTraceByTraceChannel NewNode;
	NewNode.Start = Start;
	NewNode.End = End;
	NewNode.Radius = Radius;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_SphereTraceByTraceChannel_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	bHit = false;
	HitLocation = FVector::ZeroVector;
	HitNormal = FVector(0.f, 0.f, 1.f);

	if (ExecuteContext.GetWorld() == nullptr)
	{
		WorkData.Reset();
		return;
	}

	// Note: we are not adding the world transform to the hash since
	// the world transform cannot change during one execution of the 
	// rig. By adding NumExecutions to the hash we make sure to
	// trace again once the world transform has changed.
	uint32 Hash = GetTypeHash(ExecuteContext.GetNumExecutions());
	Hash = HashCombine(Hash, GetTypeHash(Start));
	Hash = HashCombine(Hash, GetTypeHash(End));
	Hash = HashCombine(Hash, GetTypeHash((int32)TraceChannel));
	Hash = HashCombine(Hash, GetTypeHash(Radius));

	if(WorkData.Hash == Hash)
	{
		bHit = WorkData.bHit;
		HitLocation = WorkData.HitLocation;
		HitNormal = WorkData.HitNormal;
		return;
	}

	const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(TraceChannel); 
	const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(Radius);

	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = true;

	if (ExecuteContext.GetOwningActor())
	{
		QueryParams.AddIgnoredActor(ExecuteContext.GetOwningActor());
	}
	else if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ExecuteContext.GetOwningComponent()))
	{
		QueryParams.AddIgnoredComponent(PrimitiveComponent);
	}

	FHitResult HitResult;
	bHit = ExecuteContext.GetWorld()->SweepSingleByChannel(HitResult, ExecuteContext.ToWorldSpace(Start), ExecuteContext.ToWorldSpace(End), FQuat::Identity, CollisionChannel, CollisionShape, QueryParams);
	
	if (bHit)
	{
		HitLocation = ExecuteContext.ToVMSpace(HitResult.ImpactPoint);
		HitNormal = ExecuteContext.GetToWorldSpaceTransform().InverseTransformVector(HitResult.ImpactNormal);
	}

	WorkData.Hash = Hash;
	WorkData.bHit = bHit;
	WorkData.HitLocation = HitLocation;
	WorkData.HitNormal = HitNormal;
}

FRigUnit_LineTraceByTraceChannel_Execute()
{
	FRigUnit_SphereTraceByTraceChannel::StaticExecute(ExecuteContext, Start, End, TraceChannel, 0, bHit, HitLocation, HitNormal, WorkData);
}

FRigUnit_SphereTraceByObjectTypes_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	bHit = false;
	HitLocation = FVector::ZeroVector;
	HitNormal = FVector(0.f, 0.f, 1.f);

	if (ExecuteContext.GetWorld() == nullptr)
	{
		WorkData.Reset();
		return;
	}

	// Note: we are not adding the world transform to the hash since
	// the world transform cannot change during one execution of the 
	// rig. By adding NumExecutions to the hash we make sure to
	// trace again once the world transform has changed.
	uint32 Hash = GetTypeHash(ExecuteContext.GetNumExecutions());
	Hash = HashCombine(Hash, GetTypeHash(Start));
	Hash = HashCombine(Hash, GetTypeHash(End));
	Hash = HashCombine(Hash, GetTypeHash(ObjectTypes.Num()));
	for(const TEnumAsByte<EObjectTypeQuery>& ObjectType : ObjectTypes)
	{
		Hash = HashCombine(Hash, GetTypeHash((int32)ObjectType));
	}
	Hash = HashCombine(Hash, GetTypeHash(Radius));

	if(WorkData.Hash == Hash)
	{
		bHit = WorkData.bHit;
		HitLocation = WorkData.HitLocation;
		HitNormal = WorkData.HitNormal;
		return;
	}

	const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(Radius); 

	// similar to KismetTraceUtils.h -> ConfigureCollisionObjectParams
	TArray<ECollisionChannel> CollisionChannels;
	CollisionChannels.AddUninitialized(ObjectTypes.Num());

	for (int Index = 0; Index < ObjectTypes.Num(); Index++)
	{ 
		CollisionChannels[Index] = UEngineTypes::ConvertToCollisionChannel(ObjectTypes[Index]);
	}

	FCollisionObjectQueryParams ObjectParams;
	for (const ECollisionChannel& Channel : CollisionChannels)
	{
		if (FCollisionObjectQueryParams::IsValidObjectQuery(Channel))
		{
			ObjectParams.AddObjectTypesToQuery(Channel);
		}
	}

	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = true;

	if (ExecuteContext.GetOwningActor())
	{
		QueryParams.AddIgnoredActor(ExecuteContext.GetOwningActor());
	}
	else if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ExecuteContext.GetOwningComponent()))
	{
		QueryParams.AddIgnoredComponent(PrimitiveComponent);
	} 

	FHitResult HitResult;
	bHit = ExecuteContext.GetWorld()->SweepSingleByObjectType(HitResult, ExecuteContext.ToWorldSpace(Start), ExecuteContext.ToWorldSpace(End), FQuat::Identity, ObjectParams, CollisionShape, QueryParams);

	if (bHit)
	{
		HitLocation = ExecuteContext.ToVMSpace(HitResult.ImpactPoint);
		HitNormal = ExecuteContext.GetToWorldSpaceTransform().InverseTransformVector(HitResult.ImpactNormal);
	}

	WorkData.Hash = Hash;
	WorkData.bHit = bHit;
	WorkData.HitLocation = HitLocation;
	WorkData.HitNormal = HitNormal;
}

FRigUnit_LineTraceByObjectTypes_Execute()
{
	FRigUnit_SphereTraceByObjectTypes::StaticExecute(ExecuteContext, Start, End, ObjectTypes, 0, bHit, HitLocation, HitNormal, WorkData);
}

FRigUnit_SphereTraceByProfile_Execute()
{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	bHit = false;
	HitLocation = FVector::ZeroVector;
	HitNormal = FVector(0.f, 0.f, 1.f);

	if (ExecuteContext.GetWorld() == nullptr)
	{
		WorkData.Reset();
		return;
	}

	// Note: we are not adding the world transform to the hash since
	// the world transform cannot change during one execution of the 
	// rig. By adding NumExecutions to the hash we make sure to
	// trace again once the world transform has changed.
	uint32 Hash = GetTypeHash(ExecuteContext.GetNumExecutions());
	Hash = HashCombine(Hash, GetTypeHash(Start));
	Hash = HashCombine(Hash, GetTypeHash(End));
	Hash = HashCombine(Hash, GetTypeHash(ProfileName));
	Hash = HashCombine(Hash, GetTypeHash(Radius));

	if(WorkData.Hash == Hash)
	{
		bHit = WorkData.bHit;
		HitLocation = WorkData.HitLocation;
		HitNormal = WorkData.HitNormal;
		return;
	}

	const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(Radius); 

	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = true;

	if (ExecuteContext.GetOwningActor())
	{
		QueryParams.AddIgnoredActor(ExecuteContext.GetOwningActor());
	}
	else if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(ExecuteContext.GetOwningComponent()))
	{
		QueryParams.AddIgnoredComponent(PrimitiveComponent);
	} 

	FHitResult HitResult;
	bHit = ExecuteContext.GetWorld()->SweepSingleByProfile(HitResult, ExecuteContext.ToWorldSpace(Start), ExecuteContext.ToWorldSpace(End), FQuat::Identity, ProfileName, CollisionShape, QueryParams);

	if (bHit)
	{
		HitLocation = ExecuteContext.ToVMSpace(HitResult.ImpactPoint);
		HitNormal = ExecuteContext.GetToWorldSpaceTransform().InverseTransformVector(HitResult.ImpactNormal);
	}

	WorkData.Hash = Hash;
	WorkData.bHit = bHit;
	WorkData.HitLocation = HitLocation;
	WorkData.HitNormal = HitNormal;
}

FRigUnit_LineTraceByProfile_Execute()
{
	FRigUnit_SphereTraceByProfile::StaticExecute(ExecuteContext, Start, End, 0.0f, ProfileName, bHit, HitLocation, HitNormal, WorkData);
}

