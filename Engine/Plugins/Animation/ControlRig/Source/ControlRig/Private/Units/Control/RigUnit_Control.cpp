// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Control/RigUnit_Control.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Control)

FRigUnit_Control_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (!bIsInitialized)
	{
		Transform.FromFTransform(InitTransform);
		bIsInitialized = true;
	}

	Result = StaticGetResultantTransform(Transform, Base, Filter);
}

FTransform FRigUnit_Control::GetResultantTransform() const
{
	return StaticGetResultantTransform(Transform, Base, Filter);
}

FTransform FRigUnit_Control::StaticGetResultantTransform(const FEulerTransform& InTransform, const FTransform& InBase, const FTransformFilter& InFilter)
{
	return StaticGetFilteredTransform(InTransform, InFilter).ToFTransform() * InBase;
}

FMatrix FRigUnit_Control::GetResultantMatrix() const
{
	return StaticGetResultantMatrix(Transform, Base, Filter);
}

FMatrix FRigUnit_Control::StaticGetResultantMatrix(const FEulerTransform& InTransform, const FTransform& InBase, const FTransformFilter& InFilter)
{
	const FEulerTransform FilteredTransform = StaticGetFilteredTransform(InTransform, InFilter);
	return FScaleRotationTranslationMatrix(FilteredTransform.Scale, FilteredTransform.Rotation, FilteredTransform.Location) * InBase.ToMatrixWithScale();
}

void FRigUnit_Control::SetResultantTransform(const FTransform& InResultantTransform)
{
	StaticSetResultantTransform(InResultantTransform, Base, Transform);
}

void FRigUnit_Control::StaticSetResultantTransform(const FTransform& InResultantTransform, const FTransform& InBase, FEulerTransform& OutTransform)
{
	OutTransform.FromFTransform(InResultantTransform.GetRelativeTransform(InBase));
}

void FRigUnit_Control::SetResultantMatrix(const FMatrix& InResultantMatrix)
{
	StaticSetResultantMatrix(InResultantMatrix, Base, Transform);
}

void FRigUnit_Control::StaticSetResultantMatrix(const FMatrix& InResultantMatrix, const FTransform& InBase, FEulerTransform& OutTransform)
{
	const FMatrix RelativeTransform = InResultantMatrix * InBase.ToMatrixWithScale().Inverse();

	OutTransform.Location = RelativeTransform.GetOrigin();
	OutTransform.Rotation = RelativeTransform.Rotator();
	OutTransform.Scale = RelativeTransform.GetScaleVector();
}

FEulerTransform FRigUnit_Control::GetFilteredTransform() const
{
	return StaticGetFilteredTransform(Transform, Filter);
}

FEulerTransform FRigUnit_Control::StaticGetFilteredTransform(const FEulerTransform& InTransform, const FTransformFilter& InFilter)
{
	FEulerTransform FilteredTransform = InTransform;
	InFilter.FilterTransform(FilteredTransform);
	return FilteredTransform;
}

FRigVMStructUpgradeInfo FRigUnit_Control::GetUpgradeInfo() const
{
	return FRigUnit::GetUpgradeInfo();
}

