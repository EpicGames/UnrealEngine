// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Control/RigUnit_Control_StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Control_StaticMesh)

FRigUnit_Control_StaticMesh::FRigUnit_Control_StaticMesh()
{
}

FRigUnit_Control_StaticMesh_Execute()
{
	FRigUnit_Control::StaticExecute(ExecuteContext, Transform, Base, InitTransform, Result, Filter, bIsInitialized);
}

FRigVMStructUpgradeInfo FRigUnit_Control_StaticMesh::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}
