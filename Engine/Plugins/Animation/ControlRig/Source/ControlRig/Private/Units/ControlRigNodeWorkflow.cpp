// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/ControlRigNodeWorkflow.h"
#include "RigVMCore/RigVMStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigNodeWorkflow)

#if WITH_EDITOR
#include "RigVMModel/RigVMNode.h"
#include "RigVMModel/RigVMController.h"
#endif

bool UControlRigWorkflowOptions::EnsureAtLeastOneRigElementSelected() const
{
	if(Selection.IsEmpty())
	{
		static constexpr TCHAR SelectAtLeastOneRigElementMessage[] = TEXT("Please select at least one element in the hierarchy!");
		Reportf(EMessageSeverity::Error, SelectAtLeastOneRigElementMessage);
		return false;
	}
	return true;
}

// provides the default workflows for any pin
TArray<FRigVMUserWorkflow> UControlRigTransformWorkflowOptions::ProvideWorkflows(const UObject* InSubject)
{
	TArray<FRigVMUserWorkflow> Workflows;
#if WITH_EDITOR
	if(const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
	{
		if(!Pin->IsArray())
		{
			if(Pin->GetCPPType() == RigVMTypeUtils::GetUniqueStructTypeName(TBaseStructure<FTransform>::Get()))
			{
				Workflows.Emplace(
					TEXT("Set from hierarchy"),
					TEXT("Sets the pin to match the global transform of the selected element in the hierarchy"),
					ERigVMUserWorkflowType::PinContext,
					FRigVMPerformUserWorkflowDelegate::CreateStatic(&UControlRigTransformWorkflowOptions::PerformTransformWorkflow),
					StaticClass()
				);
			}
		}
	}
#endif
	return Workflows;
}

#if WITH_EDITOR

bool UControlRigTransformWorkflowOptions::PerformTransformWorkflow(const URigVMUserWorkflowOptions* InOptions,
	UObject* InController)
{
	URigVMController* Controller = CastChecked<URigVMController>(InController);
	
	if(const UControlRigTransformWorkflowOptions* Options = Cast<UControlRigTransformWorkflowOptions>(InOptions))
	{
		if(Options->EnsureAtLeastOneRigElementSelected())
		{
			const FRigElementKey& Key = Options->Selection[0];
			if(FRigTransformElement* TransformElement = (FRigTransformElement*)Options->Hierarchy->Find<FRigTransformElement>(Key))
			{
				const FTransform Transform = Options->Hierarchy->GetTransform(TransformElement, Options->TransformType);

				return Controller->SetPinDefaultValue(
					InOptions->GetSubject<URigVMPin>()->GetPinPath(),
					FRigVMStruct::ExportToFullyQualifiedText<FTransform>(Transform)
				);
			}
		}
	}
	
	return false;
}
#endif
