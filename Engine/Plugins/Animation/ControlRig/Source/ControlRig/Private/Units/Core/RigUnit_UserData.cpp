// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Core/RigUnit_UserData.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_UserData)
#define LOCTEXT_NAMESPACE "RigUnit_UserData"

#if WITH_EDITOR

FString FRigDispatch_GetUserData::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InMetaDataKey == FRigVMStruct::CustomWidgetMetaName)
	{
		if(InArgumentName == ArgNameSpaceName)
		{
			return TEXT("UserDataNameSpace");
		}
		if(InArgumentName == ArgPathName)
		{
			return TEXT("UserDataPath");
		}
	}
	return FRigDispatchFactory::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FText FRigDispatch_GetUserData::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ArgNameSpaceName)
	{
		return LOCTEXT("GetUserDataNameSpaceToolTip", "The namespace to look up the argument within");
	}
	if (InArgumentName == ArgPathName)
	{
		return LOCTEXT("GetUserDataPathToolTip", "The path to the argument within the namespace");
	}
	if (InArgumentName == ArgDefaultName)
	{
		return LOCTEXT("GetUserDataDefaultToolTip", "The default value for the argument. This is returned if the argument could not be found.");
	}
	if (InArgumentName == ArgResultName)
	{
		return LOCTEXT("GetUserDataResultToolTip", "The resulting value for the argument. This may be the default value if the argument could not be found.");
	}
	if (InArgumentName == ArgFoundName)
	{
		return LOCTEXT("GetUserDataFoundToolTip", "A boolean flag indicating if the argument could be found.");
	}
	return FRigDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_GetUserData::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		
		CachedArgumentInfos.Emplace(ArgNameSpaceName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FString);
		CachedArgumentInfos.Emplace(ArgPathName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FString);
		CachedArgumentInfos.Emplace(ArgDefaultName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(ArgResultName, ERigVMPinDirection::Output, ValueCategories);
		CachedArgumentInfos.Emplace(ArgFoundName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigDispatch_GetUserData::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(ArgNameSpaceName, RigVMTypeUtils::TypeIndex::FString);
	Types.Add(ArgPathName, RigVMTypeUtils::TypeIndex::FString);
	Types.Add(ArgDefaultName, InTypeIndex);
	Types.Add(ArgResultName, InTypeIndex);
	Types.Add(ArgFoundName, RigVMTypeUtils::TypeIndex::Bool);
	return Types;
}

FRigVMFunctionPtr FRigDispatch_GetUserData::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InTypes.FindChecked(ArgNameSpaceName) == RigVMTypeUtils::TypeIndex::FString);
	check(InTypes.FindChecked(ArgPathName) == RigVMTypeUtils::TypeIndex::FString);
	const TRigVMTypeIndex TypeIndex = InTypes.FindChecked(ArgDefaultName);
	check(InRegistry->CanMatchTypes_NoLock(TypeIndex, InTypes.FindChecked(ArgResultName), true));
	check(InTypes.FindChecked(ArgFoundName) == RigVMTypeUtils::TypeIndex::Bool);
	return &FRigDispatch_GetUserData::Execute;
}

void FRigDispatch_GetUserData::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	const FControlRigExecuteContext& ControlRigContext = InContext.GetPublicData<FControlRigExecuteContext>();

	if(ControlRigContext.GetEventName().IsNone())
	{
		// may happen during init
		return;
	}
	
	// allow to run only in construction event
	if(ControlRigContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		static constexpr TCHAR ErrorMessageFormat[] = TEXT("It's recommended to use this node in the %s event. When using it in other events please consider caching the results.");
		ControlRigContext.Logf(EMessageSeverity::Info, ErrorMessageFormat, *FRigUnit_PrepareForExecution::EventName.ToString());
	}

	const FProperty* PropertyDefault = Handles[2].GetResolvedProperty(); 
	const FProperty* PropertyResult = Handles[3].GetResolvedProperty(); 
#if WITH_EDITOR
	check(PropertyDefault);
	check(PropertyResult);
	check(Handles[0].IsString());
	check(Handles[1].IsString());
	check(Handles[4].IsBool());
#endif

	const FString& NameSpace = *(FString*)Handles[0].GetInputData();
	const FString& Path = *(FString*)Handles[1].GetInputData();
	const uint8* Default = Handles[2].GetInputData();
	uint8* Result = Handles[3].GetOutputData();
	bool& bFound = *(bool*)Handles[4].GetOutputData();
	bFound = false;

	auto Log = [&ControlRigContext](const FString& Message)
	{
#if WITH_EDITOR
		if(ControlRigContext.GetLog())
		{
			ControlRigContext.Report(EMessageSeverity::Info, ControlRigContext.GetFunctionName(), ControlRigContext.GetInstructionIndex(), Message);
		}
		else
#endif
		{
			ControlRigContext.Log(EMessageSeverity::Info, Message);
		}
	};

	
	if(const UNameSpacedUserData* UserDataObject = ControlRigContext.FindUserData(NameSpace))
	{
		FString ErrorMessage;
		if(const UNameSpacedUserData::FUserData* UserData = UserDataObject->GetUserData(Path, &ErrorMessage))
		{
			if(UserData->GetProperty())
			{
#if WITH_EDITOR
				const FString CPPType = RigVMTypeUtils::GetCPPTypeFromProperty(PropertyResult);
				const FRigVMRegistry& Registry = FRigVMRegistry::Get();
				const TRigVMTypeIndex ResultTypeIndex = Registry.GetTypeIndexFromCPPType(CPPType);

				// Object type will be null for non-uobjects. We deal with this in PostProcessCPPType
				UObject* CPPObjectType = RigVMTypeUtils::GetCPPTypeObjectFromProperty(UserData->GetProperty());
				const FString UserDataCPPType = RigVMTypeUtils::PostProcessCPPType(UserData->GetCPPType().ToString(), CPPObjectType);
				const TRigVMTypeIndex UserDataTypeIndex = Registry.GetTypeIndexFromCPPType(UserDataCPPType);

				if(Registry.CanMatchTypes(ResultTypeIndex, UserDataTypeIndex, true))
#endif
				{
					URigVMMemoryStorage::CopyProperty(PropertyResult, Result, UserData->GetProperty(), UserData->GetMemory());
					bFound = true;
					return;
				}

#if WITH_EDITOR
				static constexpr TCHAR Format[] = TEXT("User data of type '%s' not compatible with pin of type '%s'.");
				ControlRigContext.Logf(EMessageSeverity::Info, Format, *UserData->GetCPPType().ToString(), *CPPType);
#endif
			}
		}
#if WITH_EDITOR
		else
		{
			ControlRigContext.Log(EMessageSeverity::Info, ErrorMessage);
		}
#endif
	}
#if WITH_EDITOR
	else
	{
		static constexpr TCHAR Format[] = TEXT("User data namespace '%s' cannot be found.");
		ControlRigContext.Logf(EMessageSeverity::Info, Format, *NameSpace);
	}
#endif

	// fall back behavior - copy default to result
	URigVMMemoryStorage::CopyProperty(PropertyResult, Result, PropertyDefault, Default);
}

FRigUnit_SetupShapeLibraryFromUserData_Execute()
{
	bFound = false;

	// this may happen during init
	if(ExecuteContext.GetEventName().IsNone())
	{
		return;
	}

	auto Log = [&ExecuteContext](const FString& Message)
	{
#if WITH_EDITOR
		if(ExecuteContext.GetLog())
		{
			ExecuteContext.Report(EMessageSeverity::Info, ExecuteContext.GetFunctionName(), ExecuteContext.GetInstructionIndex(), Message);
		}
		else
#endif
		{
			ExecuteContext.Log(EMessageSeverity::Info, Message);
		}
	};
	
	// allow to run only in construction event
	if(ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		static constexpr TCHAR MessageFormat[] = TEXT("It's recommended to use this node in the %s event. When using it in other events please consider caching the results.");
		ExecuteContext.Logf(EMessageSeverity::Info, MessageFormat, *FRigUnit_PrepareForExecution::EventName.ToString());
	}

	if(const UNameSpacedUserData* UserDataObject = ExecuteContext.FindUserData(NameSpace))
	{
		FString ErrorMessage;
		if(const UNameSpacedUserData::FUserData* UserData = UserDataObject->GetUserData(Path, &ErrorMessage))
		{
			if(const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(UserData->GetProperty()))
			{
				if(ObjectProperty->PropertyClass == UControlRigShapeLibrary::StaticClass())
				{
					if(ExecuteContext.OnAddShapeLibraryDelegate.IsBound())
					{
						ExecuteContext.OnAddShapeLibraryDelegate.Execute(&ExecuteContext, LibraryName, (UControlRigShapeLibrary*)UserData->GetMemory(), LogShapeLibraries);
					}
					bFound = true;
					return;
				}
			}
#if WITH_EDITOR
			static constexpr TCHAR Format[] = TEXT("User data for path '%s' is of type '%s' - not a shape library.");
			Log(FString::Printf(Format, *Path, *UserData->GetCPPType().ToString()));
#endif
		}
#if WITH_EDITOR
		else
		{
			Log(ErrorMessage);
		}
#endif
	}
#if WITH_EDITOR
	else
	{
		static constexpr TCHAR Format[] = TEXT("User data namespace '%s' cannot be found.");
		Log(FString::Printf(Format, *NameSpace));
	}
#endif
}

FRigUnit_ShapeExists_Execute()
{
	Result = false;
	if (ExecuteContext.OnShapeExistsDelegate.IsBound())
	{
		Result = ExecuteContext.OnShapeExistsDelegate.Execute(ShapeName);
	}
}

#undef LOCTEXT_NAMESPACE
