// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_DebugBezier.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DebugBezier)

FRigUnit_DebugBezier_Execute()
{
	FRigUnit_DebugBezierItemSpace::StaticExecute(
		ExecuteContext, 
		Bezier, 
		MinimumU, 
		MaximumU, 
		Color, 
		Thickness, 
		Detail, 
		FRigElementKey(Space, ERigElementType::Bone), 
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_DebugBezier::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_DebugBezierItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	FTransform Transform = WorldOffset;
	if (Space.IsValid())
	{
		//Transform = Transform * ExecuteContext.Hierarchy->GetGlobalTransform(Space);
	}

	DrawBezier(ExecuteContext, Transform, Bezier, MinimumU, MaximumU, Color, Thickness, Detail);
}

FRigVMStructUpgradeInfo FRigUnit_DebugBezierItemSpace::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

void FRigUnit_DebugBezierItemSpace::DrawBezier(const FRigVMExecuteContext& InContext, const FTransform& WorldOffset, const FRigVMFourPointBezier& InBezier, float MinimumU, float MaximumU, const FLinearColor& Color, float Thickness, int32 Detail)
{
	FRigVMDrawInterface* DrawInterface = InContext.GetDrawInterface();
	if(DrawInterface == nullptr)
	{
		return;
	}
	
	if (!DrawInterface->IsEnabled())
	{
		return;
	}

	int32 Count = FMath::Clamp<int32>(Detail, 4, 64);
	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::LineStrip, Color, Thickness, WorldOffset);
	Instruction.Positions.SetNumUninitialized(Count);

	float T = MinimumU;
	float Step = (MaximumU - MinimumU) / float(Detail-1);
	for(int32 Index=0;Index<Count;Index++)
	{
		FVector Tangent;
		FControlRigMathLibrary::FourPointBezier(InBezier, T, Instruction.Positions[Index], Tangent);
		T += Step;
	}

	DrawInterface->DrawInstruction(Instruction);
}
