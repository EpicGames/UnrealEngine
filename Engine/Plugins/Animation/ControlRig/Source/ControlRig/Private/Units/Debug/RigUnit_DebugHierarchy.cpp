// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_DebugHierarchy.h"
#include "Units/RigUnitContext.h"
#include "ControlRigDefines.h"
#include "RigVMCore/RigVMDebugDrawSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DebugHierarchy)

FRigUnit_DebugHierarchy_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		DrawHierarchy(ExecuteContext, WorldOffset, Hierarchy, EControlRigDrawHierarchyMode::Axes, Scale, Color, Thickness, nullptr, &Items, DebugDrawSettings);
	}
}

FRigUnit_DebugPose_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		FRigUnit_DebugHierarchy::DrawHierarchy(ExecuteContext, WorldOffset, Hierarchy, EControlRigDrawHierarchyMode::Axes, Scale, Color, Thickness, &Pose, &Items, DebugDrawSettings);
	}
}

void FRigUnit_DebugHierarchy::DrawHierarchy(const FRigVMExecuteContext& InContext, const FTransform& WorldOffset, URigHierarchy* Hierarchy, EControlRigDrawHierarchyMode::Type Mode, float Scale, const FLinearColor& Color, float Thickness, const FRigPose* InPose, const TArrayView<const FRigElementKey>* InItems, const FRigVMDebugDrawSettings& DebugDrawSettings)
{
	FRigVMDrawInterface* DrawInterface = InContext.GetDrawInterface();
	if(DrawInterface == nullptr)
	{
		return;
	}
	
	if (!DrawInterface->IsEnabled())
	{
		return;
	}

	TMap<const FRigBaseElement*,  bool> ElementMap;
	if(InItems)
	{
		for(const FRigElementKey& Item : *InItems)
		{
			if(const FRigBaseElement* Element = Hierarchy->Find(Item))
			{
				ElementMap.Add(Element, true);
			}
		}
	}

	switch (Mode)
	{
		case EControlRigDrawHierarchyMode::Axes:
		{
			FRigVMDrawInstruction InstructionX(ERigVMDrawSettings::Lines, FLinearColor::Red, Thickness, WorldOffset, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			FRigVMDrawInstruction InstructionY(ERigVMDrawSettings::Lines, FLinearColor::Green, Thickness, WorldOffset, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			FRigVMDrawInstruction InstructionZ(ERigVMDrawSettings::Lines, FLinearColor::Blue, Thickness, WorldOffset, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			FRigVMDrawInstruction InstructionParent(ERigVMDrawSettings::Lines, Color, Thickness, WorldOffset, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			InstructionX.Positions.Reserve(Hierarchy->Num() * 2);
			InstructionY.Positions.Reserve(Hierarchy->Num() * 2);
			InstructionZ.Positions.Reserve(Hierarchy->Num() * 2);
			InstructionParent.Positions.Reserve(Hierarchy->Num() * 6);

			Hierarchy->ForEach<FRigTransformElement>([&](FRigTransformElement* Child) -> bool
			{
				auto GetTransformLambda = [InPose, Hierarchy](FRigTransformElement* InElement, FTransform& Transform) -> bool
				{
					bool bValid = true;
					if(InPose)
					{
						const int32 ElementIndex = InPose->GetIndex(InElement->GetKey());
						if(ElementIndex != INDEX_NONE)
						{
							Transform = InPose->operator[](ElementIndex).GlobalTransform;
							return true;
						}
						else
						{
							bValid = false;
						}
					}

					Transform = Hierarchy->GetTransform(InElement, ERigTransformType::CurrentGlobal);
					return bValid;
				};

				// use the optional filter
				if(!ElementMap.IsEmpty() && !ElementMap.Contains(Child))
				{
					return true;
				}
				
				FTransform Transform = FTransform::Identity;
				if(!GetTransformLambda(Child, Transform))
				{
					return true;
				}

				const FVector P0 = Transform.GetLocation();
				const FVector PX = Transform.TransformPosition(FVector(Scale, 0.f, 0.f));
				const FVector PY = Transform.TransformPosition(FVector(0.f, Scale, 0.f));
				const FVector PZ = Transform.TransformPosition(FVector(0.f, 0.f, Scale));
				InstructionX.Positions.Add(P0);
				InstructionX.Positions.Add(PX);
				InstructionY.Positions.Add(P0);
				InstructionY.Positions.Add(PY);
				InstructionZ.Positions.Add(P0);
				InstructionZ.Positions.Add(PZ);

				FRigBaseElementParentArray Parents = Hierarchy->GetParents(Child);
				TArray<FRigElementWeight> Weights = Hierarchy->GetParentWeightArray(Child);
				
				for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
				{
					if(FRigTransformElement* ParentTransformElement = Cast<FRigTransformElement>(Parents[ParentIndex]))
					{
						// use the optional filter
						if(!ElementMap.IsEmpty() && !ElementMap.Contains(ParentTransformElement))
						{
							continue;
						}

						if(Weights.IsValidIndex(ParentIndex))
						{
							if(Weights[ParentIndex].IsAlmostZero())
							{
								continue;
							}
						}
						
						FTransform ParentTransform = FTransform::Identity;
						GetTransformLambda(ParentTransformElement, ParentTransform);

						const FVector P1 = ParentTransform.GetLocation();
						InstructionParent.Positions.Add(P0);
						InstructionParent.Positions.Add(P1);
					}
				}
				return true;
			});

			DrawInterface->DrawInstruction(InstructionX);
			DrawInterface->DrawInstruction(InstructionY);
			DrawInterface->DrawInstruction(InstructionZ);
			DrawInterface->DrawInstruction(InstructionParent);
			break;
		}
		default:
		{
			break;
		}
	}
}
