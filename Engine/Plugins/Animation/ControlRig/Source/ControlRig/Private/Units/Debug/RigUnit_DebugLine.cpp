// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_DebugLine.h"
#include "Units/RigUnitContext.h"
#include "RigVMFunctions/Debug/RigVMFunction_DebugLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DebugLine)

FRigUnit_DebugLine_Execute()
{
	FRigUnit_DebugLineItemSpace::StaticExecute(
		ExecuteContext, 
		A,
		B,
		Color,
		Thickness,
		FRigElementKey(Space, ERigElementType::Bone), 
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_DebugLine::GetUpgradeInfo() const
{
	FRigVMFunction_DebugLineNoSpace NewNode;
	NewNode.A = A;
	NewNode.B = B;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_DebugLineItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FVector DrawA = A, DrawB = B;
	if (Space.IsValid())
	{
		FTransform Transform = ExecuteContext.Hierarchy->GetGlobalTransform(Space);
		DrawA = Transform.TransformPosition(DrawA);
		DrawB = Transform.TransformPosition(DrawB);
	}

	ExecuteContext.GetDrawInterface()->DrawLine(WorldOffset, DrawA, DrawB, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}

FRigVMStructUpgradeInfo FRigUnit_DebugLineItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_DebugLineNoSpace NewNode;
	NewNode.A = A;
	NewNode.B = B;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}