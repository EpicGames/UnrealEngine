// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_DebugLineStrip.h"
#include "Units/RigUnitContext.h"
#include "RigVMFunctions/Debug/RigVMFunction_DebugLineStrip.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DebugLineStrip)

FRigUnit_DebugLineStrip_Execute()
{
	FRigUnit_DebugLineStripItemSpace::StaticExecute(
		ExecuteContext, 
		Points,
		Color,
		Thickness,
		FRigElementKey(Space, ERigElementType::Bone), 
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_DebugLineStrip::GetUpgradeInfo() const
{
	FRigVMFunction_DebugLineStripNoSpace NewNode;
	NewNode.Points = Points;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_DebugLineStripItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	if (Space.IsValid())
	{
		FTransform Transform = ExecuteContext.Hierarchy->GetGlobalTransform(Space);
		TArray<FVector> PointsTransformed;
		PointsTransformed.Reserve(Points.Num());
		for(const FVector& Point : Points)
		{
			PointsTransformed.Add(Transform.TransformPosition(Point));
		}
		ExecuteContext.GetDrawInterface()->DrawLineStrip(WorldOffset, PointsTransformed, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
	}
	else
	{
		ExecuteContext.GetDrawInterface()->DrawLineStrip(WorldOffset, TArrayView<const FVector>(Points.GetData(), Points.Num()), Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
	}
}

FRigVMStructUpgradeInfo FRigUnit_DebugLineStripItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_DebugLineStripNoSpace NewNode;
	NewNode.Points = Points;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

