// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_DebugPrimitives.h"
#include "Units/RigUnitContext.h"
#include "RigVMFunctions/Debug/RigVMFunction_DebugPrimitives.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DebugPrimitives)

FRigUnit_DebugRectangle_Execute()
{
	FRigUnit_DebugRectangleItemSpace::StaticExecute(
		ExecuteContext, 
		Transform,
		Color,
		Scale,
		Thickness,
		FRigElementKey(Space, ERigElementType::Bone), 
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_DebugRectangle::GetUpgradeInfo() const
{
	FRigVMFunction_DebugRectangleNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Color = Color;
	NewNode.Scale = Scale;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_DebugRectangleItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_DebugRectangleNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Color = Color;
	NewNode.Scale = Scale;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_DebugRectangleItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform DrawTransform = Transform;
	if (Space.IsValid())
	{
		DrawTransform = DrawTransform * ExecuteContext.Hierarchy->GetGlobalTransform(Space);
	}

	ExecuteContext.GetDrawInterface()->DrawRectangle(WorldOffset, DrawTransform, Scale, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}

FRigUnit_DebugArc_Execute()
{
	FRigUnit_DebugArcItemSpace::StaticExecute(
		ExecuteContext, 
		Transform,
		Color,
		Radius,
		MinimumDegrees,
		MaximumDegrees,
		Thickness,
		Detail,
		FRigElementKey(Space, ERigElementType::Bone), 
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_DebugArc::GetUpgradeInfo() const
{
	FRigVMFunction_DebugArcNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Color = Color;
	NewNode.Radius = Radius;
	NewNode.MinimumDegrees = MinimumDegrees;
	NewNode.MaximumDegrees = MaximumDegrees;
	NewNode.Thickness = Thickness;
	NewNode.Detail = Detail;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_DebugArcItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_DebugArcNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Color = Color;
	NewNode.Radius = Radius;
	NewNode.MinimumDegrees = MinimumDegrees;
	NewNode.MaximumDegrees = MaximumDegrees;
	NewNode.Thickness = Thickness;
	NewNode.Detail = Detail;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_DebugArcItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform DrawTransform = Transform;
	if (Space.IsValid())
	{
		DrawTransform = DrawTransform * ExecuteContext.Hierarchy->GetGlobalTransform(Space);
	}

	ExecuteContext.GetDrawInterface()->DrawArc(WorldOffset, DrawTransform, Radius, FMath::DegreesToRadians(MinimumDegrees), FMath::DegreesToRadians(MaximumDegrees), Color, Thickness, Detail, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}
