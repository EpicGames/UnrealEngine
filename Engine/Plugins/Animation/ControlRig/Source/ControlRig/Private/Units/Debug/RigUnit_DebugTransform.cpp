// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_DebugTransform.h"
#include "Units/Debug/RigUnit_VisualDebug.h"
#include "Units/RigUnitContext.h"
#include "RigVMFunctions/Debug/RigVMFunction_DebugTransform.h"
#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DebugTransform)

FRigUnit_DebugTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform DrawTransform = Transform;
	if (Space != NAME_None && ExecuteContext.Hierarchy != nullptr)
	{
		DrawTransform = Transform * ExecuteContext.Hierarchy->GetGlobalTransform(FRigElementKey(Space, ERigElementType::Bone));
	}

	switch (Mode)
	{
		case ERigUnitDebugTransformMode::Axes:
		{
			ExecuteContext.GetDrawInterface()->DrawAxes(WorldOffset, DrawTransform, Scale, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugTransformMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, DrawTransform.GetLocation(), Scale, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugTransformMode::Box:
		{
			DrawTransform.SetScale3D(DrawTransform.GetScale3D() * Scale);
			ExecuteContext.GetDrawInterface()->DrawBox(WorldOffset, DrawTransform, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_DebugTransform::GetUpgradeInfo() const
{
	FRigVMFunction_DebugTransformMutableNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_DebugTransformMutable_Execute()
{
	FRigUnit_DebugTransformMutableItemSpace::StaticExecute(
		ExecuteContext, 
		Transform,
		Mode,
		Color,
		Thickness,
		Scale,
		FRigElementKey(Space, ERigElementType::Bone), 
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_DebugTransformMutable::GetUpgradeInfo() const
{
	FRigVMFunction_DebugTransformMutableNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_DebugTransformMutableItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_DebugTransformMutableNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_DebugTransformMutableItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform DrawTransform = Transform;
	if (Space.IsValid() && ExecuteContext.Hierarchy != nullptr)
	{
		DrawTransform = Transform * ExecuteContext.Hierarchy->GetGlobalTransform(Space);
	}

	switch (Mode)
	{
		case ERigUnitDebugTransformMode::Axes:
		{
			ExecuteContext.GetDrawInterface()->DrawAxes(WorldOffset, DrawTransform, Scale, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugTransformMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, DrawTransform.GetLocation(), Scale, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugTransformMode::Box:
		{
			DrawTransform.SetScale3D(DrawTransform.GetScale3D() * Scale);
			ExecuteContext.GetDrawInterface()->DrawBox(WorldOffset, DrawTransform, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
	}
}

FRigUnit_DebugTransformArrayMutable_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	WorkData.DrawTransforms.SetNumUninitialized(Transforms.Num());
	if (Space != NAME_None && ExecuteContext.Hierarchy)
	{
		FTransform SpaceTransform = ExecuteContext.Hierarchy->GetGlobalTransform(FRigElementKey(Space, ERigElementType::Bone));
		for(int32 Index=0;Index<Transforms.Num();Index++)
		{
			WorkData.DrawTransforms[Index] = Transforms[Index] * SpaceTransform;
		}
	}
	else
	{
		for(int32 Index=0;Index<Transforms.Num();Index++)
		{
			WorkData.DrawTransforms[Index] = Transforms[Index];
		}
	}

	for(FTransform& DrawTransform : WorkData.DrawTransforms)
	{
		switch (Mode)
		{
			case ERigUnitDebugTransformMode::Axes:
			{
				ExecuteContext.GetDrawInterface()->DrawAxes(WorldOffset, DrawTransform, Scale, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
				break;
			}
			case ERigUnitDebugTransformMode::Point:
			{
				ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, DrawTransform.GetLocation(), Scale, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
				break;
			}
			case ERigUnitDebugTransformMode::Box:
			{
				DrawTransform.SetScale3D(DrawTransform.GetScale3D() * Scale);
				ExecuteContext.GetDrawInterface()->DrawBox(WorldOffset, DrawTransform, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
				break;
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_DebugTransformArrayMutable::GetUpgradeInfo() const
{
	FRigVMFunction_DebugTransformArrayMutableNoSpace NewNode;
	NewNode.Transforms = Transforms;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_DebugTransformArrayMutableItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_DebugTransformArrayMutableNoSpace NewNode;
	NewNode.Transforms = Transforms;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_DebugTransformArrayMutableItemSpace_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled || Transforms.Num() == 0)
	{
		return;
	}

	for(const FTransform& Transform : Transforms)
	{
		FRigUnit_DebugTransformMutableItemSpace::StaticExecute(
			ExecuteContext, 
			Transform,
			Mode,
			Color,
			Thickness,
			Scale,
			Space, 
			WorldOffset, 
			bEnabled,
			DebugDrawSettings);
	}

	if(ParentIndices.Num() == Transforms.Num())
	{
		for(int32 Index = 0; Index < ParentIndices.Num(); Index++)
		{
			if(Transforms.IsValidIndex(ParentIndices[Index]))
			{
				ExecuteContext.GetDrawInterface()->DrawLine(
					WorldOffset,
					Transforms[Index].GetTranslation(),
					Transforms[ParentIndices[Index]].GetTranslation(),
					Color,
					Thickness,
					DebugDrawSettings.DepthPriority,
					DebugDrawSettings.Lifetime
				);
			}
		}
	}
}

