// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_ProfilingBracket.h"
#include "Units/RigUnitContext.h"
#include "ControlRigDefines.h"
#include "KismetAnimationLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ProfilingBracket)

FRigUnit_StartProfilingTimer_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	UKismetAnimationLibrary::K2_StartProfilingTimer();
}

FRigUnit_EndProfilingTimer_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (!bIsInitialized)
	{
		AccumulatedTime = 0.f;
		MeasurementsLeft = FMath::Max(NumberOfMeasurements, 1);
		bIsInitialized = true;
	}

	float Delta = UKismetAnimationLibrary::K2_EndProfilingTimer(false);
	if(MeasurementsLeft > 0)
	{
		AccumulatedTime += Delta;
		MeasurementsLeft--;

		if (MeasurementsLeft == 0)
		{
			AverageTime = AccumulatedTime / float(FMath::Max(NumberOfMeasurements, 1));
			if (Prefix.IsEmpty())
			{
				UE_CONTROLRIG_RIGUNIT_LOG_MESSAGE(TEXT("%0.3f ms (%d runs)."), AverageTime, FMath::Max(NumberOfMeasurements, 1));
			}
			else
			{
				UE_CONTROLRIG_RIGUNIT_LOG_MESSAGE(TEXT("[%s] %0.3f ms (%d runs)."), *Prefix, AverageTime, FMath::Max(NumberOfMeasurements, 1));
			}
		}
	}
}
