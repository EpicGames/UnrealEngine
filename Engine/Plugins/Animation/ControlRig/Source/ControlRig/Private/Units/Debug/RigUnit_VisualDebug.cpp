// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Debug/RigUnit_VisualDebug.h"
#include "Units/RigUnitContext.h"
#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_VisualDebug)

FRigUnit_VisualDebugVector_Execute()
{
	FRigUnit_VisualDebugVectorItemSpace::StaticExecute(ExecuteContext, Value, bEnabled, Mode, Color, Thickness, Scale, FRigElementKey(BoneSpace, ERigElementType::Bone), DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_VisualDebugVector::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugVectorNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.bEnabled = bEnabled;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_VisualDebugVectorItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform WorldOffset = FTransform::Identity;
	if (Space.IsValid())
	{
		WorldOffset = ExecuteContext.Hierarchy->GetGlobalTransform(Space);
	}

	switch(Mode)
	{
		case ERigUnitVisualDebugPointMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, Value, Thickness, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitVisualDebugPointMode::Vector:
		{
			ExecuteContext.GetDrawInterface()->DrawLine(WorldOffset, FVector::ZeroVector, Value * Scale, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}
}

FRigUnit_VisualDebugQuat_Execute()
{
	FRigUnit_VisualDebugQuatItemSpace::StaticExecute(ExecuteContext, Value, bEnabled, Thickness, Scale, FRigElementKey(BoneSpace, ERigElementType::Bone), DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_VisualDebugVectorItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugVectorNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.bEnabled = bEnabled;
	NewNode.Mode = Mode;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_VisualDebugQuat::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugQuatNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;
	
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_VisualDebugQuatItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    FTransform Transform = FTransform::Identity;
    Transform.SetRotation(Value);

	FRigUnit_VisualDebugTransformItemSpace::StaticExecute(ExecuteContext, Transform, bEnabled, Thickness, Scale, Space, DebugDrawSettings);
}

FRigUnit_VisualDebugTransform_Execute()
{
	FRigUnit_VisualDebugTransformItemSpace::StaticExecute(ExecuteContext, Value, bEnabled, Thickness, Scale, FRigElementKey(BoneSpace, ERigElementType::Bone), DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigUnit_VisualDebugQuatItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugQuatNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;
	
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_VisualDebugTransform::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugTransformNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMStructUpgradeInfo FRigUnit_VisualDebugTransformItemSpace::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugTransformNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_VisualDebugTransformItemSpace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform WorldOffset = FTransform::Identity;
	if (Space.IsValid())
	{
		WorldOffset = ExecuteContext.Hierarchy->GetGlobalTransform(Space);
	}

	ExecuteContext.GetDrawInterface()->DrawAxes(WorldOffset, Value, Scale, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}
