// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/Math/RigUnit_Converter.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathQuaternion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Converter)

FRigUnit_ConvertTransform_Execute()
{
	Result.FromFTransform(Input);
}

FRigVMStructUpgradeInfo FRigUnit_ConvertTransform::GetUpgradeInfo() const
{
	FRigVMFunction_MathTransformToEulerTransform NewNode;
	NewNode.Value = Input;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Input"), TEXT("Value"));
	return Info;
}

FRigUnit_ConvertEulerTransform_Execute()
{
	Result = Input.ToFTransform();
}

FRigVMStructUpgradeInfo FRigUnit_ConvertEulerTransform::GetUpgradeInfo() const
{
	FRigVMFunction_MathTransformFromEulerTransform NewNode;
	NewNode.EulerTransform = Input;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Input"), TEXT("EulerTransform"));
	return Info;
}

FRigUnit_ConvertRotation_Execute()
{
	Result = Input.Quaternion();
}

FRigVMStructUpgradeInfo FRigUnit_ConvertRotation::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionFromRotator NewNode;
	NewNode.Rotator = Input;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Input"), TEXT("Rotator"));
	return Info;
}

FRigUnit_ConvertQuaternion_Execute()
{
	Result = Input.Rotator();
}

FRigVMStructUpgradeInfo FRigUnit_ConvertQuaternion::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionToRotator NewNode;
	NewNode.Value = Input;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Input"), TEXT("Value"));
	return Info;
}

FRigUnit_ConvertVectorToRotation_Execute()
{
	Result = Input.Rotation();
}

FRigVMStructUpgradeInfo FRigUnit_ConvertVectorToRotation::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_ConvertVectorToQuaternion_Execute()
{
	Result = Input.Rotation().Quaternion();
	Result.Normalize();
}

FRigVMStructUpgradeInfo FRigUnit_ConvertVectorToQuaternion::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_ConvertRotationToVector_Execute()
{
	Result = Input.RotateVector(FVector(1.f, 0.f, 0.f));
}

FRigVMStructUpgradeInfo FRigUnit_ConvertRotationToVector::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_ConvertQuaternionToVector_Execute()
{
	Result = Input.RotateVector(FVector(1.f, 0.f, 0.f));
}

FRigVMStructUpgradeInfo FRigUnit_ConvertQuaternionToVector::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_ToSwingAndTwist_Execute()
{
	if (!TwistAxis.IsZero())
	{
		FVector NormalizedAxis = TwistAxis.GetSafeNormal();
		Input.ToSwingTwist(TwistAxis, Swing, Twist);
	}
}

FRigVMStructUpgradeInfo FRigUnit_ToSwingAndTwist::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionSwingTwist NewNode;
	NewNode.Input = Input;
	NewNode.TwistAxis = TwistAxis;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

