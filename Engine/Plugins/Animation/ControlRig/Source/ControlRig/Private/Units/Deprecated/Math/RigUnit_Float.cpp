// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/Math/RigUnit_Float.h"
#include "RigVMFunctions/Math/RigVMFunction_MathFloat.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Float)

FRigUnit_Multiply_FloatFloat_Execute()
{
	Result = FRigMathLibrary::Multiply(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Multiply_FloatFloat::GetUpgradeInfo() const
{
	FRigVMFunction_MathFloatMul NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Add_FloatFloat_Execute()
{
	Result = FRigMathLibrary::Add(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Add_FloatFloat::GetUpgradeInfo() const
{
	FRigVMFunction_MathFloatAdd NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Subtract_FloatFloat_Execute()
{
	Result = FRigMathLibrary::Subtract(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Subtract_FloatFloat::GetUpgradeInfo() const
{
	FRigVMFunction_MathFloatSub NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Divide_FloatFloat_Execute()
{
	Result = FRigMathLibrary::Divide(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Divide_FloatFloat::GetUpgradeInfo() const
{
	FRigVMFunction_MathFloatDiv NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Clamp_Float_Execute()
{
	Result = FMath::Clamp(Value, Min, Max);
}

FRigVMStructUpgradeInfo FRigUnit_Clamp_Float::GetUpgradeInfo() const
{
	FRigVMFunction_MathFloatClamp NewNode;
	NewNode.Value = Value;
	NewNode.Minimum = Min;
	NewNode.Maximum = Max;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Min"), TEXT("Minimum"));
	Info.AddRemappedPin(TEXT("Max"), TEXT("Maximum"));
	return Info;
}

FRigUnit_MapRange_Float_Execute()
{
	Result = FMath::GetMappedRangeValueClamped(FVector2f(MinIn, MaxIn), FVector2f(MinOut, MaxOut), Value);
}

FRigVMStructUpgradeInfo FRigUnit_MapRange_Float::GetUpgradeInfo() const
{
	FRigVMFunction_MathFloatRemap NewNode;
	NewNode.Value = Value;
	NewNode.SourceMinimum = MinIn;
	NewNode.SourceMaximum = MaxIn;
	NewNode.TargetMinimum = MinOut;
	NewNode.TargetMaximum = MaxOut;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("MinIn"), TEXT("SourceMinimum"));
	Info.AddRemappedPin(TEXT("MaxIn"), TEXT("SourceMaximum"));
	Info.AddRemappedPin(TEXT("MinOut"), TEXT("TargetMinimum"));
	Info.AddRemappedPin(TEXT("MaxOut"), TEXT("TargetMaximum"));
	return Info;
}
