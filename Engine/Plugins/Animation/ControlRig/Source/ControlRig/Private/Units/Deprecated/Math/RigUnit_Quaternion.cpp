// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/Math/RigUnit_Quaternion.h"
#include "RigVMFunctions/Math/RigVMFunction_MathQuaternion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Quaternion)

FRigUnit_MultiplyQuaternion_Execute()
{
	Result = Argument0*Argument1;
	Result.Normalize();
}

FRigVMStructUpgradeInfo FRigUnit_MultiplyQuaternion::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionMul NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_InverseQuaterion_Execute()
{
	Result = Argument.Inverse();
	Result.Normalize();
}

FRigVMStructUpgradeInfo FRigUnit_InverseQuaterion::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionInverse NewNode;
	NewNode.Value = Argument;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument"), TEXT("Value"));
	return Info;
}

FRigUnit_QuaternionToAxisAndAngle_Execute()
{
	FVector NewAxis = Axis.GetSafeNormal();
	Argument.ToAxisAndAngle(NewAxis, Angle);
	Angle = FMath::RadiansToDegrees(Angle);
}

FRigVMStructUpgradeInfo FRigUnit_QuaternionToAxisAndAngle::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionToAxisAndAngle NewNode;
	NewNode.Value = Argument;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument"), TEXT("Value"));
	return Info;
}

FRigUnit_QuaternionFromAxisAndAngle_Execute()
{
	FVector NewAxis = Axis.GetSafeNormal();
	Result = FQuat(NewAxis, FMath::DegreesToRadians(Angle));
}

FRigVMStructUpgradeInfo FRigUnit_QuaternionFromAxisAndAngle::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionToAxisAndAngle NewNode;
	NewNode.Axis = Axis;
	NewNode.Angle = Angle;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Result"), TEXT("Value"));
	return Info;
}

FRigUnit_QuaternionToAngle_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	FQuat Swing, Twist;
	FVector SafeAxis = Axis.GetSafeNormal();

	FQuat Input = Argument;
	Input.Normalize();
	Input.ToSwingTwist(SafeAxis, Swing, Twist);

	FVector TwistAxis;
	float Radian;
	Twist.ToAxisAndAngle(TwistAxis, Radian);
	// Our range here is from [0, 360)
	Angle = FMath::Fmod(FMath::RadiansToDegrees(Radian), 360);
	if ((TwistAxis | SafeAxis) < 0)
	{
		Angle = 360 - Angle;
	}
}

FRigVMStructUpgradeInfo FRigUnit_QuaternionToAngle::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

