// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/Math/RigUnit_Transform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Transform)

FRigUnit_MultiplyTransform_Execute()
{
	Result = Argument0*Argument1;
}

FRigVMStructUpgradeInfo FRigUnit_MultiplyTransform::GetUpgradeInfo() const
{
	FRigVMFunction_MathTransformMul NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_GetRelativeTransform_Execute()
{
	Result = Argument0.GetRelativeTransform(Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_GetRelativeTransform::GetUpgradeInfo() const
{
	FRigVMFunction_MathTransformMakeRelative NewNode;
	NewNode.Global = Argument0;
	NewNode.Parent = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("Global"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("Parent"));
	Info.AddRemappedPin(TEXT("Result"), TEXT("Local"));
	return Info;
}

