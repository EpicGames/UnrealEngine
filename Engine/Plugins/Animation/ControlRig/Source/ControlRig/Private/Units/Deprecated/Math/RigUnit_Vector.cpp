// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/Math/RigUnit_Vector.h"
#include "RigVMFunctions/Math/RigVMFunction_MathVector.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Vector)

FRigUnit_Multiply_VectorVector_Execute()
{
	Result = FRigMathLibrary::Multiply(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Multiply_VectorVector::GetUpgradeInfo() const
{
	FRigVMFunction_MathVectorMul NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Add_VectorVector_Execute()
{
	Result = FRigMathLibrary::Add(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Add_VectorVector::GetUpgradeInfo() const
{
	FRigVMFunction_MathVectorAdd NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Subtract_VectorVector_Execute()
{
	Result = FRigMathLibrary::Subtract(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Subtract_VectorVector::GetUpgradeInfo() const
{
	FRigVMFunction_MathVectorSub NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Divide_VectorVector_Execute()
{
	Result = FRigMathLibrary::Divide(Argument0, Argument1);
}

FRigVMStructUpgradeInfo FRigUnit_Divide_VectorVector::GetUpgradeInfo() const
{
	FRigVMFunction_MathVectorDiv NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}

FRigUnit_Distance_VectorVector_Execute()
{
	Result = (Argument0 - Argument1).Size();
}

FRigVMStructUpgradeInfo FRigUnit_Distance_VectorVector::GetUpgradeInfo() const
{
	FRigVMFunction_MathVectorDistance NewNode;
	NewNode.A = Argument0;
	NewNode.B = Argument1;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Argument0"), TEXT("A"));
	Info.AddRemappedPin(TEXT("Argument1"), TEXT("B"));
	return Info;
}
