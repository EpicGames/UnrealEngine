// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/RigUnit_AimConstraint.h"
#include "Units/RigUnitContext.h"
#include "HelperUtil.h"
#include "AnimationCoreLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AimConstraint)

FRigUnit_AimConstraint_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	TArray<FConstraintData>& ConstraintData = WorkData.ConstraintData;

	if (ConstraintData.Num() != AimTargets.Num())
	{
		ConstraintData.Reset();

		URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		if (Hierarchy)
		{
			const FRigElementKey Key(Joint, ERigElementType::Bone); 
			int32 BoneIndex = Hierarchy->GetIndex(Key);
			if (BoneIndex != INDEX_NONE)
			{
				const int32 TargetNum = AimTargets.Num();
				if (TargetNum > 0)
				{
					const FTransform SourceTransform = Hierarchy->GetGlobalTransform(BoneIndex);
					for (int32 TargetIndex = 0; TargetIndex < TargetNum; ++TargetIndex)
					{
						const FAimTarget& Target = AimTargets[TargetIndex];

						int32 NewIndex = ConstraintData.AddDefaulted();
						check(NewIndex != INDEX_NONE);
						FConstraintData& NewData = ConstraintData[NewIndex];
						NewData.Constraint = FAimConstraintDescription();
						NewData.bMaintainOffset = false; // for now we don't support maintain offset for aim
						NewData.Weight = Target.Weight;
					}
				}
			}
		}
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Joint, ERigElementType::Bone); 
		int32 BoneIndex = Hierarchy->GetIndex(Key);
		if (BoneIndex != INDEX_NONE)
		{
			const int32 TargetNum = AimTargets.Num();
			if (TargetNum > 0)
			{
				for (int32 ConstraintIndex= 0; ConstraintIndex< ConstraintData.Num(); ++ConstraintIndex)
				{
					FAimConstraintDescription* AimConstraintDesc = ConstraintData[ConstraintIndex].Constraint.GetTypedConstraint<FAimConstraintDescription>();
					AimConstraintDesc->LookAt_Axis = FAxis(AimVector);

					if (UpTargets.IsValidIndex(ConstraintIndex))
					{
						AimConstraintDesc->LookUp_Axis = FAxis(UpVector);
						AimConstraintDesc->bUseLookUp = UpVector.Size() > 0.f;
						AimConstraintDesc->LookUpTarget = UpTargets[ConstraintIndex].Transform.GetLocation();
					}

					ConstraintData[ConstraintIndex].CurrentTransform = AimTargets[ConstraintIndex].Transform;
					ConstraintData[ConstraintIndex].Weight = AimTargets[ConstraintIndex].Weight;
				}

				FTransform BaseTransform = FTransform::Identity;
				int32 ParentIndex = Hierarchy->GetIndex(Hierarchy->GetFirstParent(Key));
				if (ParentIndex != INDEX_NONE)
				{
					BaseTransform = Hierarchy->GetGlobalTransform(ParentIndex);
				}

				FTransform SourceTransform = Hierarchy->GetGlobalTransform(BoneIndex);

				// @todo: ignore maintain offset for now
				FTransform ConstrainedTransform = AnimationCore::SolveConstraints(SourceTransform, BaseTransform, ConstraintData);

				Hierarchy->SetGlobalTransform(BoneIndex, ConstrainedTransform);
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_AimConstraint::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

/*
void FRigUnit_AimConstraint::AddConstraintData(EAimConstraintType ConstraintType, const int32 TargetIndex, const FTransform& SourceTransform, const FTransform& InBaseTransform)
{
	const FConstraintTarget& Target = Targets[TargetIndex];

	int32 NewIndex = ConstraintData.AddDefaulted();
	check(NewIndex != INDEX_NONE);
	FConstraintData& NewData = ConstraintData[NewIndex];
	NewData.Constraint = FAimConstraintDescription(ConstraintType);
	NewData.bMaintainOffset = Target.bMaintainOffset;
	NewData.Weight = Target.Weight;

	if (Target.bMaintainOffset)
	{
		NewData.SaveInverseOffset(SourceTransform, Target.Transform, InBaseTransform);
	}

	ConstraintDataToTargets.Add(NewIndex, TargetIndex);
}*/
