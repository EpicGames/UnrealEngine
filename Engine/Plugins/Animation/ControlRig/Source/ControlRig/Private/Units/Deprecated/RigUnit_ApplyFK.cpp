// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/RigUnit_ApplyFK.h"
#include "Units/RigUnitContext.h"
#include "HelperUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ApplyFK)

FRigUnit_ApplyFK_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Joint, ERigElementType::Bone);
		int32 Index = Hierarchy->GetIndex(Key);
		if (Index != INDEX_NONE)
		{
			// first filter input transform
			FTransform InputTransform = Transform;
			Filter.FilterTransform(InputTransform);

			FTransform InputBaseTransform = UtilityHelpers::GetBaseTransformByMode(
				ApplyTransformSpace,
				[Hierarchy](const FRigElementKey& BoneKey) { return Hierarchy->GetGlobalTransform(BoneKey); },
				Hierarchy->GetFirstParent(Key),
				FRigElementKey(BaseJoint, ERigElementType::Bone),
				BaseTransform
			);

			// now get override or additive
			// whether I'd like to apply whole thing or not
			if (ApplyTransformMode == EApplyTransformMode::Override)
			{
				// get base transform
				FTransform ApplyTransform = InputTransform * InputBaseTransform;
				Hierarchy->SetGlobalTransform(Index, ApplyTransform);
			}
			else
			{
				// if additive, we get current transform and calculate base transform and apply in their local space
				FTransform CurrentTransform = Hierarchy->GetGlobalTransform(Index);
				FTransform LocalTransform = InputTransform * CurrentTransform.GetRelativeTransform(InputBaseTransform);
				// apply additive
				Hierarchy->SetGlobalTransform(Index, LocalTransform * InputBaseTransform);
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_ApplyFK::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ApplyFK)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(1.f, 2.f, 3.f)), true, ERigBoneType::User);

	Unit.Joint = TEXT("BoneA");
	Unit.ApplyTransformMode = EApplyTransformMode::Override;
	Unit.ApplyTransformSpace = ETransformSpaceMode::GlobalSpace;
	Unit.Transform = FTransform(FVector(0.f, 5.f, 0.f));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(0.f, 5.f, 0.f)), TEXT("unexpected global transform"));
	AddErrorIfFalse(Hierarchy->GetLocalTransform(1).GetTranslation().Equals(FVector(-1.f, 5.f, 0.f)), TEXT("unexpected local transform"));

	Unit.ApplyTransformMode = EApplyTransformMode::Override;
	Unit.ApplyTransformSpace = ETransformSpaceMode::LocalSpace;

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 5.f, 0.f)), TEXT("unexpected global transform"));
	AddErrorIfFalse(Hierarchy->GetLocalTransform(1).GetTranslation().Equals(FVector(0.f, 5.f, 0.f)), TEXT("unexpected local transform"));

	Unit.ApplyTransformMode = EApplyTransformMode::Additive;

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 7.f, 3.f)), TEXT("unexpected global transform"));
	AddErrorIfFalse(Hierarchy->GetLocalTransform(1).GetTranslation().Equals(FVector(0.f, 7.f, 3.f)), TEXT("unexpected local transform"));
	return true;
}
#endif
