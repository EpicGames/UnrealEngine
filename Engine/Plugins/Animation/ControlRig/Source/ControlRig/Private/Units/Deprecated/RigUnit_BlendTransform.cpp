// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/RigUnit_BlendTransform.h"
#include "Units/RigUnitContext.h"
#include "AnimationRuntime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_BlendTransform)

FRigUnit_BlendTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	if (Targets.Num() > 0)
	{
		float TotalSum = 0.f;
		TArray<FTransform> BlendTransform;
		TArray<float> BlendWeights;
		for (int32 Index = 0; Index < Targets.Num(); ++Index)
		{
			if (Targets[Index].Weight > ZERO_ANIMWEIGHT_THRESH)
			{
				BlendTransform.Add(Targets[Index].Transform);
				BlendWeights.Add(Targets[Index].Weight);
				TotalSum += Targets[Index].Weight;
			}
		}

		if (BlendTransform.Num() > 0)
		{
			if (TotalSum > 1.f )
			{
				for (int32 Index = 0; Index < BlendWeights.Num(); ++Index)
				{
					BlendWeights[Index] /= TotalSum;
				}
			}

			const float SourceWeight = FMath::Clamp(1.f - (TotalSum), 0.f, 1.f);
			if (SourceWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				BlendTransform.Add(Source);
				BlendWeights.Add(SourceWeight);
			}
			
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// deprecated node calls a deprecated function, suppress warning for this case
			FAnimationRuntime::BlendTransformsByWeight(Result, BlendTransform, BlendWeights);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	// if failed on any of the above, it will just use source as target pose
	Result = Source;
}

FRigVMStructUpgradeInfo FRigUnit_BlendTransform::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

