// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/RigUnit_GetJointTransform.h"
#include "Units/RigUnitContext.h"
#include "HelperUtil.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetJointTransform)

FRigUnit_GetJointTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    
	if (URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		const FRigElementKey Key(Joint, ERigElementType::Bone);
		
		switch (Type)
		{
		case ETransformGetterType::Current:
			{
				const FTransform ComputedBaseTransform = UtilityHelpers::GetBaseTransformByMode(TransformSpace, [Hierarchy](const FRigElementKey& JointKey) { return Hierarchy->GetGlobalTransform(JointKey); },
				Hierarchy->GetFirstParent(Key), FRigElementKey(BaseJoint, ERigElementType::Bone), BaseTransform);

				Output = Hierarchy->GetInitialGlobalTransform(Key).GetRelativeTransform(ComputedBaseTransform);
				break;
			}
		case ETransformGetterType::Initial:
		default:
			{
				const FTransform ComputedBaseTransform = UtilityHelpers::GetBaseTransformByMode(TransformSpace, [Hierarchy](const FRigElementKey& JointKey) { return Hierarchy->GetInitialGlobalTransform(JointKey); },
				Hierarchy->GetFirstParent(Key), FRigElementKey(BaseJoint, ERigElementType::Bone), BaseTransform);

				Output = Hierarchy->GetInitialGlobalTransform(Key).GetRelativeTransform(ComputedBaseTransform);
				break;
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_GetJointTransform::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

