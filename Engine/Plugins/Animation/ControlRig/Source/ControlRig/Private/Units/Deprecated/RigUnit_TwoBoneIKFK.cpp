// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Deprecated/RigUnit_TwoBoneIKFK.h"
#include "Units/RigUnitContext.h"
#include "HelperUtil.h"
#include "TwoBoneIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_TwoBoneIKFK)

FRigUnit_TwoBoneIKFK_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (!bIsInitialized)
	{
		URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		if (Hierarchy)
		{
			// reset
			StartJointIndex = MidJointIndex = EndJointIndex = INDEX_NONE;
			UpperLimbLength = LowerLimbLength = 0.f;

			const FRigElementKey StartJointKey(StartJoint, ERigElementType::Bone);
			const FRigElementKey EndJointKey(EndJoint, ERigElementType::Bone);
			
			// verify the chain
			int32 StartIndex = Hierarchy->GetIndex(StartJointKey);
			int32 EndIndex = Hierarchy->GetIndex(EndJointKey);
			if (StartIndex != INDEX_NONE && EndIndex != INDEX_NONE)
			{
				// ensure the chain
				int32 EndParentIndex = Hierarchy->GetFirstParent(EndIndex);
				if (EndParentIndex != INDEX_NONE)
				{
					int32 MidParentIndex = Hierarchy->GetFirstParent(EndParentIndex);
					if (MidParentIndex == StartIndex)
					{
						StartJointIndex = StartIndex;
						MidJointIndex = EndParentIndex;
						EndJointIndex = EndIndex;

						// set length for upper/lower length
						FTransform StartTransform = Hierarchy->GetInitialGlobalTransform(StartJointIndex);
						FTransform MidTransform = Hierarchy->GetInitialGlobalTransform(MidJointIndex);
						FTransform EndTransform = Hierarchy->GetInitialGlobalTransform(EndJointIndex);

						FVector UpperLimb = StartTransform.GetLocation() - MidTransform.GetLocation();
						FVector LowerLimb = MidTransform.GetLocation() - EndTransform.GetLocation();

						UpperLimbLength = UpperLimb.Size();
						LowerLimbLength = LowerLimb.Size();
						StartJointIKTransform = StartJointFKTransform = StartTransform;
						MidJointIKTransform = MidJointFKTransform = MidTransform;
						EndJointIKTransform = EndJointFKTransform = EndTransform;
					}
				}
			}
		}
		bIsInitialized = true;
	}

	if (StartJointIndex != INDEX_NONE && MidJointIndex != INDEX_NONE && EndJointIndex != INDEX_NONE)
	{
		FTransform StartJointTransform;
		FTransform MidJointTransform;
		FTransform EndJointTransform;

		// FK only
		if (FMath::IsNearlyZero(IKBlend))
		{
			StartJointTransform = StartJointFKTransform;
			MidJointTransform = MidJointFKTransform;
			EndJointTransform = EndJointFKTransform;
		}
		// IK only
		else if (FMath::IsNearlyEqual(IKBlend, 1.f))
		{
			// update transform before going through IK
			const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
			check(Hierarchy);

			StartJointIKTransform = Hierarchy->GetGlobalTransform(StartJointIndex);
			MidJointIKTransform = Hierarchy->GetGlobalTransform(MidJointIndex);
			EndJointIKTransform = Hierarchy->GetGlobalTransform(EndJointIndex);

			AnimationCore::SolveTwoBoneIK(StartJointIKTransform, MidJointIKTransform, EndJointIKTransform, PoleTarget, EndEffector.GetLocation(), UpperLimbLength, LowerLimbLength, false, 1.0f, 1.05f);
			EndJointIKTransform.SetRotation(EndEffector.GetRotation());

			StartJointTransform = StartJointIKTransform;
			MidJointTransform = MidJointIKTransform;
			EndJointTransform = EndJointIKTransform;
		}
		else
		{
			// update transform before going through IK
			const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
			check(Hierarchy);

			StartJointIKTransform = Hierarchy->GetGlobalTransform(StartJointIndex);
			MidJointIKTransform = Hierarchy->GetGlobalTransform(MidJointIndex);
			EndJointIKTransform = Hierarchy->GetGlobalTransform(EndJointIndex);

			AnimationCore::SolveTwoBoneIK(StartJointIKTransform, MidJointIKTransform, EndJointIKTransform, PoleTarget, EndEffector.GetLocation(), UpperLimbLength, LowerLimbLength, false, 1.0f, 1.05f);
			EndJointIKTransform.SetRotation(EndEffector.GetRotation());

			StartJointTransform.Blend(StartJointFKTransform, StartJointIKTransform, IKBlend);
			MidJointTransform.Blend(MidJointFKTransform, MidJointIKTransform, IKBlend);
			EndJointTransform.Blend(MidJointFKTransform, EndJointIKTransform, IKBlend);
		}

		URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		check(Hierarchy);
		Hierarchy->SetGlobalTransform(StartJointIndex, StartJointTransform);
		Hierarchy->SetGlobalTransform(MidJointIndex, MidJointTransform);
		Hierarchy->SetGlobalTransform(EndJointIndex, EndJointTransform);

		PreviousFKIKBlend = IKBlend;
	}
}

FRigVMStructUpgradeInfo FRigUnit_TwoBoneIKFK::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

