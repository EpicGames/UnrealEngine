// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Drawing/RigUnit_DrawContainer.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DrawContainer)

FRigUnit_DrawContainerGetInstruction_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    if(ExecuteContext.GetDrawContainer() == nullptr)
    {
    	return;
    }
	
	int32 Index = ExecuteContext.GetDrawContainer()->GetIndex(InstructionName);
	if (Index != INDEX_NONE)
	{
		const FRigVMDrawInstruction& Instruction = (*ExecuteContext.GetDrawContainer())[Index];
		Color = Instruction.Color;
		Transform = Instruction.Transform;
	}
	else
	{
		Color = FLinearColor::Red;
		Transform = FTransform::Identity;
	}
}

FRigVMStructUpgradeInfo FRigUnit_DrawContainerGetInstruction::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_DrawContainerSetThickness_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (ExecuteContext.GetDrawContainer() == nullptr)
	{
		return;
	}

	int32 Index = ExecuteContext.GetDrawContainer()->GetIndex(InstructionName);
	if (Index != INDEX_NONE)
	{
		FRigVMDrawInstruction& Instruction = (*ExecuteContext.GetDrawContainer())[Index];
		Instruction.Thickness = Thickness;
	}
}

FRigVMStructUpgradeInfo FRigUnit_DrawContainerSetThickness::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_DrawContainerSetColor_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (ExecuteContext.GetDrawContainer() == nullptr)
	{
		return;
	}

	int32 Index = ExecuteContext.GetDrawContainer()->GetIndex(InstructionName);
	if (Index != INDEX_NONE)
	{
		FRigVMDrawInstruction& Instruction = (*ExecuteContext.GetDrawContainer())[Index];
		Instruction.Color = Color;
	}
}

FRigVMStructUpgradeInfo FRigUnit_DrawContainerSetColor::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_DrawContainerSetTransform_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (ExecuteContext.GetDrawContainer() == nullptr)
	{
		return;
	}

	int32 Index = ExecuteContext.GetDrawContainer()->GetIndex(InstructionName);
	if (Index != INDEX_NONE)
	{
		FRigVMDrawInstruction& Instruction = (*ExecuteContext.GetDrawContainer())[Index];
		Instruction.Transform = Transform;
	}
}

FRigVMStructUpgradeInfo FRigUnit_DrawContainerSetTransform::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

