// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_BeginExecution)

FRigUnit_BeginExecution_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	ExecuteContext.SetEventName(FRigUnit_BeginExecution::EventName);
}

FRigUnit_PreBeginExecution_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	ExecuteContext.SetEventName(FRigUnit_PreBeginExecution::EventName);
}

FRigUnit_PostBeginExecution_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	ExecuteContext.SetEventName(FRigUnit_PostBeginExecution::EventName);
}

