// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_Collection.h"
#include "Units/Execution/RigUnit_Item.h"
#include "Units/Execution/RigUnit_Hierarchy.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Collection)

#if WITH_EDITOR
#include "Units/RigUnitTest.h"
#endif

FRigUnit_CollectionChain_Execute()
{
	FRigUnit_CollectionChainArray::StaticExecute(ExecuteContext, FirstItem, LastItem, Reverse, Collection.Keys);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionChain::GetUpgradeInfo() const
{
	FRigUnit_HierarchyGetChainItemArray NewNode;
	NewNode.Start = FirstItem;
	NewNode.End = LastItem;
	NewNode.bReverse = Reverse;
	NewNode.bIncludeStart = NewNode.bIncludeEnd = true;
	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_CollectionChainArray_Execute()
{
	FCachedRigElement FirstCache, LastCache;
	FRigElementKeyCollection CachedChain;
	FRigUnit_HierarchyGetChainItemArray::StaticExecute(ExecuteContext, FirstItem, LastItem, true, true, false, Items, FirstCache, LastCache, CachedChain);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionChainArray::GetUpgradeInfo() const
{
	FRigUnit_HierarchyGetChainItemArray NewNode;
	NewNode.Start = FirstItem;
	NewNode.End = LastItem;
	NewNode.bReverse = Reverse;
	NewNode.bIncludeStart = NewNode.bIncludeEnd = true;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_CollectionNameSearch_Execute()
{
	FRigUnit_CollectionNameSearchArray::StaticExecute(ExecuteContext, PartialName, TypeToSearch, Collection.Keys);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionNameSearch::GetUpgradeInfo() const
{
	FRigUnit_CollectionNameSearchArray NewNode;
	NewNode.PartialName = PartialName;
	NewNode.TypeToSearch = TypeToSearch;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_CollectionNameSearchArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(ExecuteContext.Hierarchy == nullptr)
	{
		Items.Reset();
		return;
	}

	uint32 Hash = GetTypeHash(StaticStruct()) + ExecuteContext.Hierarchy->GetTopologyVersion() * 17;
	Hash = HashCombine(Hash, GetTypeHash(PartialName));
	Hash = HashCombine(Hash, (int32)TypeToSearch * 8);

	FRigElementKeyCollection Collection;
	if(const FRigElementKeyCollection* Cache = ExecuteContext.Hierarchy->FindCachedCollection(Hash))
	{
		Collection = *Cache;
	}
	else
	{
		Collection = FRigElementKeyCollection::MakeFromName(ExecuteContext.Hierarchy, PartialName, (uint8)TypeToSearch);
		ExecuteContext.Hierarchy->AddCachedCollection(Hash, Collection);
	}

	Items = Collection.Keys;
}

FRigUnit_CollectionChildren_Execute()
{
	FRigUnit_CollectionChildrenArray::StaticExecute(ExecuteContext, Parent, bIncludeParent, bRecursive, true, TypeToSearch, Collection.Keys);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionChildren::GetUpgradeInfo() const
{
	FRigUnit_CollectionChildrenArray NewNode;
	NewNode.Parent = Parent;
	NewNode.bIncludeParent = bIncludeParent;
	NewNode.bRecursive = bRecursive;
	NewNode.TypeToSearch = TypeToSearch;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Collection.Keys"), TEXT("Items"));
	return Info;
}

FRigUnit_CollectionChildrenArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(ExecuteContext.Hierarchy == nullptr)
	{
		Items.Reset();
		return;
	}

	if (bDefaultChildren)
	{
		uint32 Hash = GetTypeHash(StaticStruct()) + ExecuteContext.Hierarchy->GetTopologyVersion() * 17;
		Hash = HashCombine(Hash, GetTypeHash(ExecuteContext.Hierarchy->GetResolvedTarget(Parent)));
		Hash = HashCombine(Hash, bRecursive ? 2 : 0);
		Hash = HashCombine(Hash, bIncludeParent ? 1 : 0);
		Hash = HashCombine(Hash, (int32)TypeToSearch * 8);

		FRigElementKeyCollection Collection;
		if(const FRigElementKeyCollection* Cache = ExecuteContext.Hierarchy->FindCachedCollection(Hash))
		{
			Collection = *Cache;
		}
		else
		{
			Collection = FRigElementKeyCollection::MakeFromChildren(ExecuteContext.Hierarchy, Parent, bRecursive, bIncludeParent, (uint8)TypeToSearch);
			if (Collection.IsEmpty())
			{
				if (ExecuteContext.Hierarchy->GetIndex(Parent) == INDEX_NONE)
				{
					UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent '%s' is not valid."), *Parent.ToString());
				}
			}
			ExecuteContext.Hierarchy->AddCachedCollection(Hash, Collection);
		}

		Items = Collection.Keys;
	}
	else
	{
		const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		Items.Reset();

		if (bIncludeParent)
		{
			Items.Add(Parent);
		}

		FRigBaseElementChildrenArray Children = Hierarchy->GetActiveChildren(Hierarchy->Find(Parent), bRecursive);

		Children = Children.FilterByPredicate([TypeToSearch](const FRigBaseElement* Child)
		{
			return ((uint8)Child->GetType() & (uint8)TypeToSearch) != 0;
		});

		Items.Reserve(Items.Num() + Children.Num());
		for (FRigBaseElement* Child : Children)
		{
			Items.Add(Child->GetKey());
		}
	}
}

FRigUnit_CollectionGetAll_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(ExecuteContext.Hierarchy == nullptr)
	{
		Items.Reset();
		return;
	}
	
	uint32 Hash = GetTypeHash(StaticStruct()) + ExecuteContext.Hierarchy->GetTopologyVersion() * 17;
	Hash = HashCombine(Hash, (int32)TypeToSearch * 8);

	FRigElementKeyCollection Collection;
	if(const FRigElementKeyCollection* Cache = ExecuteContext.Hierarchy->FindCachedCollection(Hash))
	{
		Collection = *Cache;
	}
	else
	{
		ExecuteContext.Hierarchy->Traverse([&Collection, TypeToSearch](FRigBaseElement* InElement, bool &bContinue)
		{
			bContinue = true;
			
			const FRigElementKey Key = InElement->GetKey();
			if(((uint8)TypeToSearch & (uint8)Key.Type) == (uint8)Key.Type)
			{
				Collection.AddUnique(Key);
			}
		});
		
		if (Collection.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("%s"), TEXT("No elements found for given filter."));
		}
		ExecuteContext.Hierarchy->AddCachedCollection(Hash, Collection);
	}

	Items = Collection.Keys;
}

#if WITH_EDITOR

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_CollectionChildren)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), BoneA, FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneC = Controller->AddBone(TEXT("BoneC"), Root, FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);

	Unit.Parent = Root;
	Unit.bIncludeParent = false;
	Unit.bRecursive = false;
	Execute();
	AddErrorIfFalse(Unit.Collection.Num() == 2, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[0] == BoneA, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[1] == BoneC, TEXT("unexpected result"));

	Unit.bIncludeParent = true;
	Unit.bRecursive = false;
	Execute();
	AddErrorIfFalse(Unit.Collection.Num() == 3, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[0] == Root, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[1] == BoneA, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[2] == BoneC, TEXT("unexpected result"));

	Unit.bIncludeParent = true;
	Unit.bRecursive = true;
	Execute();
	AddErrorIfFalse(Unit.Collection.Num() == 4, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[0] == Root, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[1] == BoneA, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[2] == BoneC, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.Collection[3] == BoneB, TEXT("unexpected result"));

	return true;
}

#endif

FRigUnit_CollectionReplaceItems_Execute()
{
	FRigUnit_CollectionReplaceItemsArray::StaticExecute(ExecuteContext, Items.Keys, Old, New, RemoveInvalidItems, bAllowDuplicates, Collection.Keys);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionReplaceItems::GetUpgradeInfo() const
{
	FRigUnit_CollectionReplaceItemsArray NewNode;
	NewNode.Items = Items.Keys;
	NewNode.Old = Old;
	NewNode.New = New;
	NewNode.RemoveInvalidItems = RemoveInvalidItems;
	NewNode.bAllowDuplicates = bAllowDuplicates;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_CollectionReplaceItemsArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(ExecuteContext.Hierarchy == nullptr)
	{
		Result.Reset();
		return;
	}
	
	uint32 Hash = GetTypeHash(StaticStruct()) + ExecuteContext.Hierarchy->GetTopologyVersion() * 17;
	Hash = HashCombine(Hash, GetTypeHash(Items));
	Hash = HashCombine(Hash, 12 * GetTypeHash(Old));
	Hash = HashCombine(Hash, 13 * GetTypeHash(New));
	Hash = HashCombine(Hash, RemoveInvalidItems ? 14 : 0);

	FRigElementKeyCollection Collection;
	if(const FRigElementKeyCollection* Cache = ExecuteContext.Hierarchy->FindCachedCollection(Hash))
	{
		Collection = *Cache;
	}
	else
	{
		Collection.Reset();
		
		for (int32 Index = 0; Index < Items.Num(); Index++)
		{
			FRigElementKey Key = Items[Index];
			FRigUnit_ItemReplace::StaticExecute(ExecuteContext, Key, Old, New, Key);

			if (ExecuteContext.Hierarchy->GetIndex(Key) != INDEX_NONE)
			{
				if(bAllowDuplicates)
				{
					Collection.Add(Key);
				}
				else
				{
					Collection.AddUnique(Key);
				}
			}
			else if(!RemoveInvalidItems)
			{
				Collection.Add(FRigElementKey());
			}
		}

		ExecuteContext.Hierarchy->AddCachedCollection(Hash, Collection);
	}
	
	Result = Collection.Keys;
}

FRigUnit_CollectionItems_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Collection.Reset();
	for (const FRigElementKey& Key : Items)
	{
		if(bAllowDuplicates)
		{
			Collection.Add(Key);
		}
		else
		{
			Collection.AddUnique(Key);
		}
	}
}

FRigUnit_CollectionGetItems_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Items = Collection.GetKeys();
}

FRigUnit_CollectionGetParentIndices_Execute()
{
	FRigUnit_CollectionGetParentIndicesItemArray::StaticExecute(ExecuteContext, Collection.Keys, ParentIndices);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionGetParentIndices::GetUpgradeInfo() const
{
	FRigUnit_CollectionGetParentIndicesItemArray NewNode;
	NewNode.Items = Collection.Keys;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Collection"), TEXT("Items"));
	return Info;
}

FRigUnit_CollectionGetParentIndicesItemArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(ExecuteContext.Hierarchy == nullptr)
	{
		ParentIndices.Reset();
		return;
	}
	
	ParentIndices.SetNumUninitialized(Items.Num());

	for(int32 Index=0;Index<Items.Num();Index++)
	{
		ParentIndices[Index] = INDEX_NONE;

		const int32 ItemIndex = ExecuteContext.Hierarchy->GetIndex(Items[Index]);
		if(ItemIndex == INDEX_NONE)
		{
			continue;;
		}

		switch(Items[Index].Type)
		{
			case ERigElementType::Curve:
			{
				continue;
			}
			case ERigElementType::Bone:
			{
				ParentIndices[Index] = ExecuteContext.Hierarchy->GetFirstParent(ItemIndex);
				break;
			}
			default:
			{
				if(const FRigBaseElement* ChildElement = ExecuteContext.Hierarchy->Get(ItemIndex))
				{
					TArray<int32> ItemParents = ExecuteContext.Hierarchy->GetParents(ItemIndex);
					for(int32 ParentIndex = 0; ParentIndex < ItemParents.Num(); ParentIndex++)
					{
						const FRigElementWeight Weight = ExecuteContext.Hierarchy->GetParentWeight(ChildElement, ParentIndex, false);
						if(!Weight.IsAlmostZero())
						{
							ParentIndices[Index] = ItemParents[ParentIndex];
						}
					}
				}
				break;
			}
		}

		if(ParentIndices[Index] != INDEX_NONE)
		{
			int32 ParentIndex = ParentIndices[Index];
			ParentIndices[Index] = INDEX_NONE;

			while(ParentIndices[Index] == INDEX_NONE && ParentIndex != INDEX_NONE)
			{
				ParentIndices[Index] = Items.Find(ExecuteContext.Hierarchy->GetKey(ParentIndex));
				ParentIndex = ExecuteContext.Hierarchy->GetFirstParent(ParentIndex);
			}
		}
	}
}

FRigUnit_CollectionUnion_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Collection = FRigElementKeyCollection::MakeUnion(A, B, bAllowDuplicates);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionUnion::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionIntersection_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Collection = FRigElementKeyCollection::MakeIntersection(A, B);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionIntersection::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionDifference_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Collection = FRigElementKeyCollection::MakeDifference(A, B);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionDifference::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionReverse_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Reversed = FRigElementKeyCollection::MakeReversed(Collection);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionReverse::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionCount_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Count = Collection.Num();
}

FRigVMStructUpgradeInfo FRigUnit_CollectionCount::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionItemAtIndex_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Collection.IsValidIndex(Index))
	{
		Item = Collection[Index];
	}
	else
	{
		Item = FRigElementKey();
	}
}

FRigVMStructUpgradeInfo FRigUnit_CollectionItemAtIndex::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionLoop_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	
	if(BlockToRun.IsNone())
	{
	    Count = Collection.Num();
		Index = 0;
		BlockToRun = ExecutePinName;
	}
	else if(BlockToRun == ExecutePinName)
	{
		Index++;
	}

	if(Collection.IsValidIndex(Index))
	{
		Item = Collection[Index];
	}
	else
	{
		Item = FRigElementKey();
		BlockToRun = ControlFlowCompletedName;
	}
	
	Ratio = GetRatioFromIndex(Index, Count);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionLoop::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CollectionAddItem_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();

	Result = Collection;
	Result.AddUnique(Item);
}

FRigVMStructUpgradeInfo FRigUnit_CollectionAddItem::GetUpgradeInfo() const
{
	// this node is no longer supported. you can rely on generic array nodes for this now
	return FRigVMStructUpgradeInfo();
}
