// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_DynamicHierarchy.h"
#include "Engine/SkeletalMesh.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "ControlRig.h"
#include "ModularRig.h"
#include "Components/SkeletalMeshComponent.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DynamicHierarchy)

bool FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(
	const FControlRigExecuteContext& InExecuteContext,
	bool bAllowOnlyConstructionEvent,
	FString* OutErrorMessage)
{
	if(InExecuteContext.Hierarchy == nullptr)
	{
		return false;
	}

	if(bAllowOnlyConstructionEvent)
	{
		if(!InExecuteContext.IsRunningAConstructionEvent())
		{
			if(OutErrorMessage)
			{
				static constexpr TCHAR ErrorMessageFormat[] = TEXT("Node can only run in %s or %s Event");
				*OutErrorMessage = FString::Printf(ErrorMessageFormat, *FRigUnit_PrepareForExecution::EventName.ToString(), *FRigUnit_PostPrepareForExecution::EventName.ToString());
			}
			return false;
		}
	}

	if(InExecuteContext.Hierarchy->Num() >= InExecuteContext.UnitContext.HierarchySettings.ProceduralElementLimit)
	{
		if(OutErrorMessage)
		{
			static constexpr TCHAR ErrorMessageFormat[] = TEXT("Node has hit the Procedural Element Limit. Check the Class Settings under Hierarchy.");
			*OutErrorMessage = ErrorMessageFormat;
		}
		return false;
	}

	return true;
}

FRigUnit_AddParent_Execute()
{
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true))
	{
		return;
	}
	
	FRigTransformElement* ChildElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Child);
	if(ChildElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Child item %s does not exist."), *Child.ToString())
		return;
	}

	FRigTransformElement* ParentElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Parent);
	if(ParentElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent item %s does not exist."), *Parent.ToString())
		return;
	}

	FRigHierarchyEnableControllerBracket EnableController(ExecuteContext.Hierarchy, true);
	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		Controller->AddParent(ChildElement, ParentElement, 0.f, true, false, DisplayLabel);
	}
}

FRigUnit_SetDefaultParent_Execute()
{
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true))
	{
		return;
	}
	
	FRigTransformElement* ChildElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Child);
	if(ChildElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Child item %s does not exist."), *Child.ToString())
		return;
	}

	FRigTransformElement* ParentElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Parent);
	if(ParentElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent item %s does not exist."), *Parent.ToString())
		return;
	}

	FRigHierarchyEnableControllerBracket EnableController(ExecuteContext.Hierarchy, true);
	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		Controller->AddParent(ChildElement, ParentElement, 1.0f, true, true);
	}
}

FRigUnit_AddAvailableSpaces_Execute()
{
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true))
	{
		return;
	}
	
	FRigControlElement* ControlElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Control);
	if(ControlElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control item %s does not exist."), *Control.ToString())
		return;
	}

	FRigHierarchyEnableControllerBracket EnableController(ExecuteContext.Hierarchy, true);
	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		for(const FRigElementKeyWithLabel& Space : Spaces)
		{
			if(const FRigTransformElement* TransformElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Space.Key))
			{
				(void)Controller->AddAvailableSpace(ControlElement, TransformElement, Space.Label);
			}
			else
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Space %s does not exist."), *Space.Key.ToString())
			}
		}
	}
}

FRigUnit_SetChannelHosts_Execute()
{
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true))
	{
		return;
	}
	
	for(const FRigElementKey& Host : Hosts)
	{
		FRigHierarchyEnableControllerBracket EnableController(ExecuteContext.Hierarchy, true);
		if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
		{
			(void)Controller->AddChannelHost(Channel, Host);
		}
	}
}

FRigUnit_SwitchParent_Execute()
{
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, false))
	{
		return;
	}

	FRigTransformElement* ChildElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Child);
	if(ChildElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Child item %s does not exist."), *Child.ToString())
		return;
	}
	if(!ChildElement->IsA<FRigMultiParentElement>())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Child item %s cannot be space switched (only Nulls and Controls can)."), *Child.ToString())
		return;
	}

	FRigTransformElement* ParentElement = nullptr;

	if(Mode == ERigSwitchParentMode::ParentItem)
	{
		ParentElement = ExecuteContext.Hierarchy->Find<FRigTransformElement>(Parent);
		if(ParentElement == nullptr)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent item %s does not exist."), *Parent.ToString())
			return;
		}

		if(!ParentElement->IsA<FRigTransformElement>())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent item %s does not have a transform."), *Parent.ToString())
			return;
		}
	}

	const ERigTransformType::Type TransformTypeToMaintain =
		bMaintainGlobal ? ERigTransformType::CurrentGlobal : ERigTransformType::CurrentLocal;
	
	const FTransform Transform = ExecuteContext.Hierarchy->GetTransform(ChildElement, TransformTypeToMaintain);

	switch(Mode)
	{
		case ERigSwitchParentMode::World:
		{
			if(!ExecuteContext.Hierarchy->SwitchToWorldSpace(ChildElement, false, true))
			{
				return;
			}
			break;
		}
		case ERigSwitchParentMode::DefaultParent:
		{
			if(!ExecuteContext.Hierarchy->SwitchToDefaultParent(ChildElement, false, true))
			{
				return;
			}
			break;
		}
		case ERigSwitchParentMode::ParentItem:
		default:
		{
			FString FailureReason;
			static const FEmptyRigDependenciesProvider NoDependencyProvider;
			if(!ExecuteContext.Hierarchy->SwitchToParent(ChildElement, ParentElement, false, true, NoDependencyProvider, &FailureReason))
			{
				if(!FailureReason.IsEmpty())
				{
					UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("%s"), *FailureReason);
				}
				return;
			}

			// during construction and post-construction events also change the initial weights
			if(ExecuteContext.IsRunningAConstructionEvent())
			{
				if(!ExecuteContext.Hierarchy->SwitchToParent(ChildElement, ParentElement, true, true, NoDependencyProvider, &FailureReason))
				{
					if(!FailureReason.IsEmpty())
					{
						UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("%s"), *FailureReason);
					}
					return;
				}
			}
			break;
		}
	}
	
	ExecuteContext.Hierarchy->SetTransform(ChildElement, Transform, TransformTypeToMaintain, true);
}

FRigUnit_HierarchyGetParentWeights_Execute()
{
	FRigUnit_HierarchyGetParentWeightsArray::StaticExecute(ExecuteContext, Child, Weights, Parents.Keys);
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyGetParentWeights::GetUpgradeInfo() const
{
	FRigUnit_HierarchyGetParentWeightsArray NewNode;
	NewNode.Child = Child;
	NewNode.Weights = Weights;
	NewNode.Parents = Parents.Keys;

	return FRigVMStructUpgradeInfo();
}

FRigUnit_HierarchyGetParentWeightsArray_Execute()
{
	if(ExecuteContext.Hierarchy == nullptr)
	{
		return;
	}

	FRigBaseElement* ChildElement = ExecuteContext.Hierarchy->Find(Child);
	if(ChildElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item %s does not exist."), *Child.ToString())
		return;
	}
	
	Weights = ExecuteContext.Hierarchy->GetParentWeightArray(ChildElement, false);
	Parents = ExecuteContext.Hierarchy->GetParents(ChildElement->GetKey(), false);
}

FRigUnit_HierarchySetParentWeights_Execute()
{
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, false))
	{
		return;
	}

	FRigBaseElement* ChildElement = ExecuteContext.Hierarchy->Find(Child);
	if(ChildElement == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item %s does not exist."), *Child.ToString())
		return;
	}

	if(Weights.Num() != ExecuteContext.Hierarchy->GetNumberOfParents(ChildElement))
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Provided incorrect number of weights(%d) for %s - expected %d."), Weights.Num(), *Child.ToString(), ExecuteContext.Hierarchy->GetNumberOfParents(Child))
		return;
	}

	ExecuteContext.Hierarchy->SetParentWeightArray(ChildElement, Weights, false, true);

	// during construction and post-construction events also change the initial weights
	if(ExecuteContext.IsRunningAConstructionEvent())
	{
		ExecuteContext.Hierarchy->SetParentWeightArray(ChildElement, Weights, true, true);
	}
}

FRigUnit_HierarchyReset_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}
	ExecuteContext.Hierarchy->ResetToDefault();
}

FRigUnit_HierarchyImportFromSkeleton_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Items.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		
		USkeleton* Skeleton = nullptr;
		const FReferenceSkeleton* ReferenceSkeleton = nullptr;
		USkeletalMesh* SkeletalMesh = nullptr;

		const USkeletalMeshComponent* SkelMeshComponent = ExecuteContext.UnitContext.DataSourceRegistry->RequestSource<USkeletalMeshComponent>(UControlRig::OwnerComponent);
		if (SkelMeshComponent != nullptr)
		{
			if (SkelMeshComponent->GetSkeletalMeshAsset())
			{
				SkeletalMesh = SkelMeshComponent->GetSkeletalMeshAsset();
				Skeleton = SkeletalMesh->GetSkeleton();
				ReferenceSkeleton = &SkeletalMesh->GetRefSkeleton();
			}
		}
		else
		{
			Skeleton = ExecuteContext.UnitContext.DataSourceRegistry->RequestSource<USkeleton>(UControlRig::OwnerComponent);
			if (Skeleton != nullptr)
			{
				ReferenceSkeleton = &Skeleton->GetReferenceSkeleton();
			}
		}

		if(ReferenceSkeleton != nullptr)
		{
			if (bIncludeVirtualBones)
			{
				Items = Controller->ImportBones(*ReferenceSkeleton, NameSpace, false, false, false, false);
			}
			else
			{
				const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRefBoneInfo();
				const TArray<FTransform>& BoneTransforms = ReferenceSkeleton->GetRefBonePose();

				TArray<FMeshBoneInfo> FilteredBoneInfos;
				TArray<FTransform> FilteredBoneTransforms;

				const int32 NumBones = BoneInfos.Num();
				for (int32 Index = 0; Index < NumBones; ++Index)
				{
					const FMeshBoneInfo& MeshBoneInfo = BoneInfos[Index];
					const FTransform& BoneTransform = BoneTransforms[Index];

					if (VirtualBoneNameHelpers::CheckVirtualBonePrefix(MeshBoneInfo.Name.ToString()))
					{
						continue;
					}
					FilteredBoneInfos.Add(MeshBoneInfo);
					FilteredBoneTransforms.Add(BoneTransform);
				}

				Items = Controller->ImportBones(FilteredBoneInfos, FilteredBoneTransforms, NameSpace, false, false, false, false);
			}

			if(bIncludeCurves)
			{
				if (SkeletalMesh)
				{
					Items.Append(Controller->ImportCurvesFromSkeletalMesh(SkeletalMesh, NameSpace, false, false, false));
				}
				else if (Skeleton)
				{
					Items.Append(Controller->ImportCurves(Skeleton, NameSpace, false, false, false));
				}
			}
			if(bIncludeMeshSockets && SkeletalMesh != nullptr)
			{
				Items.Append(Controller->ImportSocketsFromSkeletalMesh(SkeletalMesh, NameSpace, false, false, false));
			}
		}
	}
}

FRigUnit_HierarchyRemoveElement_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}
	
	bSuccess = false;

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		bSuccess = Controller->RemoveElement(Item);
	}
}

FRigUnit_HierarchyAddBone_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddBone(Name, Parent, Transform, Space == ERigVMTransformSpace::GlobalSpace, ERigBoneType::Imported, false, false);
	}
}

FRigUnit_HierarchyAddNull_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddNull(Name, Parent, Transform, Space == ERigVMTransformSpace::GlobalSpace, false, false);
	}
}

FTransform FRigUnit_HierarchyAddControlElement::ProjectOffsetTransform(const FTransform& InOffsetTransform,
	ERigVMTransformSpace InOffsetSpace, const FRigElementKey& InParent, const URigHierarchy* InHierarchy)
{
	if(InOffsetSpace == ERigVMTransformSpace::GlobalSpace && InParent.IsValid())
	{
		const FTransform ParentTransform = InHierarchy->GetGlobalTransform(InParent);
		FTransform OffsetTransform = InOffsetTransform.GetRelativeTransform(ParentTransform);
		OffsetTransform.NormalizeRotation();
		return OffsetTransform;
	}
	return InOffsetTransform;
}

void FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	DisplayName = InSettings.DisplayName;
	bSelectable = InSettings.AnimationType != ERigControlAnimationType::VisualCue;
}

void FRigUnit_HierarchyAddControl_Settings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.DisplayName = DisplayName;
	OutSettings.AnimationType = bSelectable ? ERigControlAnimationType::AnimationControl : ERigControlAnimationType::VisualCue;
}

void FRigUnit_HierarchyAddControl_ShapeSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	bVisible = InSettings.bShapeVisible;
	Name = InSettings.ShapeName;
	Color = InSettings.ShapeColor;
	Transform = InControlElement->GetShapeTransform().Get(ERigTransformType::InitialLocal);
}

void FRigUnit_HierarchyAddControl_ShapeSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.bShapeVisible = bVisible;
	OutSettings.ShapeName = Name;
	OutSettings.ShapeColor = Color;
	OutSettings.ShapeTransform = Transform;
}

void FRigUnit_HierarchyAddControl_ProxySettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	bIsProxy = InSettings.AnimationType == ERigControlAnimationType::ProxyControl;
	DrivenControls = InSettings.DrivenControls;
	ShapeVisibility = InSettings.ShapeVisibility;
}

void FRigUnit_HierarchyAddControl_ProxySettings::Configure(FRigControlSettings& OutSettings) const
{
	if (bIsProxy)
	{
		OutSettings.AnimationType = ERigControlAnimationType::ProxyControl;
	}
	OutSettings.DrivenControls = DrivenControls;
	OutSettings.ShapeVisibility = ShapeVisibility;
}

void FRigUnit_HierarchyAddControlFloat_LimitSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	Limit = InSettings.LimitEnabled[0];
	MinValue = InSettings.MinimumValue.Get<float>();
	MaxValue = InSettings.MaximumValue.Get<float>();
	bDrawLimits = InSettings.bDrawLimits;
}

void FRigUnit_HierarchyAddControlFloat_LimitSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.SetupLimitArrayForType(false, false, false);
	OutSettings.LimitEnabled[0] = Limit;
	OutSettings.MinimumValue = FRigControlValue::Make<float>(MinValue);
	OutSettings.MaximumValue = FRigControlValue::Make<float>(MaxValue);
	OutSettings.bDrawLimits = bDrawLimits;
}

void FRigUnit_HierarchyAddControlFloat_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(InControlElement, InSettings);

	bIsScale = InSettings.ControlType == ERigControlType::ScaleFloat;
	PrimaryAxis = InSettings.PrimaryAxis;

	Proxy.ConfigureFrom(InControlElement, InSettings);
	Limits.ConfigureFrom(InControlElement, InSettings);
	Shape.ConfigureFrom(InControlElement, InSettings);
}

void FRigUnit_HierarchyAddControlFloat_Settings::Configure(FRigControlSettings& OutSettings) const
{
	Super::Configure(OutSettings);
	
	OutSettings.ControlType = bIsScale ? ERigControlType::ScaleFloat : ERigControlType::Float;
	OutSettings.PrimaryAxis = PrimaryAxis;

	Proxy.Configure(OutSettings);
	Limits.Configure(OutSettings);
	Shape.Configure(OutSettings);
}

FRigUnit_HierarchyAddControlFloat_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		Settings.Configure(ControlSettings);

		const FTransform Offset = ProjectOffsetTransform(OffsetTransform, OffsetSpace, Parent, ExecuteContext.Hierarchy);

		const FRigControlValue Value = FRigControlValue::Make<float>(InitialValue);
		const FTransform ShapeTransform = Settings.Shape.Transform;
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddControl(Name, Parent, ControlSettings, Value, Offset, ShapeTransform, false, false);
	}
}

void FRigUnit_HierarchyAddControlInteger_LimitSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	Limit = InSettings.LimitEnabled[0];
	MinValue = InSettings.MinimumValue.Get<int32>();
	MaxValue = InSettings.MaximumValue.Get<int32>();
	bDrawLimits = InSettings.bDrawLimits;
}

void FRigUnit_HierarchyAddControlInteger_LimitSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.SetupLimitArrayForType(false, false, false);
	OutSettings.LimitEnabled[0] = Limit;
	OutSettings.MinimumValue = FRigControlValue::Make<int32>(MinValue);
	if (OutSettings.ControlEnum)
	{
		OutSettings.MaximumValue = FRigControlValue::Make<int32>(OutSettings.ControlEnum->GetMaxEnumValue());
	}
	else
	{
		OutSettings.MaximumValue = FRigControlValue::Make<int32>(MaxValue);
	}
	OutSettings.bDrawLimits = bDrawLimits;
}

void FRigUnit_HierarchyAddControlInteger_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(InControlElement, InSettings);

	PrimaryAxis = InSettings.PrimaryAxis;
	ControlEnum = InSettings.ControlEnum;

	Proxy.ConfigureFrom(InControlElement, InSettings);
	Limits.ConfigureFrom(InControlElement, InSettings);
	Shape.ConfigureFrom(InControlElement, InSettings);
}

void FRigUnit_HierarchyAddControlInteger_Settings::Configure(FRigControlSettings& OutSettings) const
{
	Super::Configure(OutSettings);
	
	OutSettings.ControlType = ERigControlType::Integer;
	OutSettings.PrimaryAxis = PrimaryAxis;
	OutSettings.ControlEnum = ControlEnum;

	Proxy.Configure(OutSettings);
	Limits.Configure(OutSettings);
	Shape.Configure(OutSettings);
}

FRigUnit_HierarchyAddControlInteger_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		Settings.Configure(ControlSettings);

		const FTransform Offset = ProjectOffsetTransform(OffsetTransform, OffsetSpace, Parent, ExecuteContext.Hierarchy);

		const FRigControlValue Value = FRigControlValue::Make<int32>(InitialValue);
		const FTransform ShapeTransform = Settings.Shape.Transform;
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddControl(Name, Parent, ControlSettings, Value, Offset, ShapeTransform, false, false);
	}
}

void FRigUnit_HierarchyAddControlVector2D_LimitSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	LimitX = InSettings.LimitEnabled[0];
	LimitY = InSettings.LimitEnabled[1];
	MinValue = FVector2D(InSettings.MinimumValue.Get<FVector3f>().X, InSettings.MinimumValue.Get<FVector3f>().Y);
	MaxValue = FVector2D(InSettings.MaximumValue.Get<FVector3f>().X, InSettings.MaximumValue.Get<FVector3f>().Y);
	bDrawLimits = InSettings.bDrawLimits;
}

void FRigUnit_HierarchyAddControlVector2D_LimitSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.SetupLimitArrayForType(false, false, false);
	OutSettings.LimitEnabled[0] = LimitX;
	OutSettings.LimitEnabled[1] = LimitY;
	OutSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinValue.X, MinValue.Y, 0.f));
	OutSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaxValue.X, MaxValue.Y, 0.f));
	OutSettings.bDrawLimits = bDrawLimits;
}

void FRigUnit_HierarchyAddControlVector2D_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(InControlElement, InSettings);

	PrimaryAxis = InSettings.PrimaryAxis;
	FilteredChannels = InSettings.FilteredChannels;

	Proxy.ConfigureFrom(InControlElement, InSettings);
	Limits.ConfigureFrom(InControlElement, InSettings);
	Shape.ConfigureFrom(InControlElement, InSettings);
}

void FRigUnit_HierarchyAddControlVector2D_Settings::Configure(FRigControlSettings& OutSettings) const
{
	Super::Configure(OutSettings);
	
	OutSettings.ControlType = ERigControlType::Vector2D;
	OutSettings.PrimaryAxis = PrimaryAxis;
	OutSettings.FilteredChannels = FilteredChannels;

	Proxy.Configure(OutSettings);
	Limits.Configure(OutSettings);
	Shape.Configure(OutSettings);
}

FRigUnit_HierarchyAddControlVector2D_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		Settings.Configure(ControlSettings);

		const FTransform Offset = ProjectOffsetTransform(OffsetTransform, OffsetSpace, Parent, ExecuteContext.Hierarchy);

		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(InitialValue.X, InitialValue.Y, 0.f));
		const FTransform ShapeTransform = Settings.Shape.Transform;
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddControl(Name, Parent, ControlSettings, Value, Offset, ShapeTransform, false, false);
	}
}

void FRigUnit_HierarchyAddControlVector_LimitSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	LimitX = InSettings.LimitEnabled[0];
	LimitY = InSettings.LimitEnabled[1];
	LimitZ = InSettings.LimitEnabled[2];
	MinValue = FVector(InSettings.MinimumValue.Get<FVector3f>());
	MaxValue = FVector(InSettings.MaximumValue.Get<FVector3f>());
	bDrawLimits = InSettings.bDrawLimits;
}

void FRigUnit_HierarchyAddControlVector_LimitSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.SetupLimitArrayForType(false, false, false);
	OutSettings.LimitEnabled[0] = LimitX;
	OutSettings.LimitEnabled[1] = LimitY;
	OutSettings.LimitEnabled[2] = LimitZ;
	OutSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinValue));
	OutSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaxValue));
	OutSettings.bDrawLimits = bDrawLimits;
}

void FRigUnit_HierarchyAddControlVector_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(InControlElement, InSettings);

	bIsPosition = InSettings.ControlType == ERigControlType::Position;
	FilteredChannels = InSettings.FilteredChannels;

	Proxy.ConfigureFrom(InControlElement, InSettings);
	Limits.ConfigureFrom(InControlElement, InSettings);
	Shape.ConfigureFrom(InControlElement, InSettings);
}

void FRigUnit_HierarchyAddControlVector_Settings::Configure(FRigControlSettings& OutSettings) const
{
	Super::Configure(OutSettings);
	
	OutSettings.ControlType = bIsPosition ? ERigControlType::Position : ERigControlType::Scale;
	OutSettings.FilteredChannels = FilteredChannels;

	Proxy.Configure(OutSettings);
	Limits.Configure(OutSettings);
	Shape.Configure(OutSettings);
}

FRigUnit_HierarchyAddControlVector_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		Settings.Configure(ControlSettings);

		const FTransform Offset = ProjectOffsetTransform(OffsetTransform, OffsetSpace, Parent, ExecuteContext.Hierarchy);

		FVector Vector = InitialValue;
		if(Settings.InitialSpace == ERigVMTransformSpace::GlobalSpace)
		{
			if(Settings.bIsPosition)
			{
				const FTransform ParentTransform = ExecuteContext.Hierarchy->GetGlobalTransform(Parent);
				const FTransform GlobalOffsetTransform = Offset * ParentTransform;
				const FTransform LocalTransform = FTransform(Vector).GetRelativeTransform(GlobalOffsetTransform);
				Vector = LocalTransform.GetLocation();
			}
			else
			{
				Vector = Vector * ExecuteContext.Hierarchy->GetGlobalTransform(Parent).GetScale3D(); 
			}
		}

		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(Vector));
		const FTransform ShapeTransform = Settings.Shape.Transform;
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddControl(Name, Parent, ControlSettings, Value, Offset, ShapeTransform, false, false);
	}
}

void FRigUnit_HierarchyAddControlRotator_LimitSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	LimitPitch = InSettings.LimitEnabled[0];
	LimitYaw = InSettings.LimitEnabled[1];
	LimitRoll = InSettings.LimitEnabled[2];
	MinValue = FRotator::MakeFromEuler(FVector(InSettings.MinimumValue.Get<FVector3f>()));
	MaxValue = FRotator::MakeFromEuler(FVector(InSettings.MaximumValue.Get<FVector3f>()));
	bDrawLimits = InSettings.bDrawLimits;
}

void FRigUnit_HierarchyAddControlRotator_LimitSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.SetupLimitArrayForType(false, false, false);
	OutSettings.LimitEnabled[0] = LimitPitch;
	OutSettings.LimitEnabled[1] = LimitYaw;
	OutSettings.LimitEnabled[2] = LimitRoll;
	OutSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinValue.Euler()));
	OutSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaxValue.Euler()));
	OutSettings.bDrawLimits = bDrawLimits;
}

void FRigUnit_HierarchyAddControlRotator_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(InControlElement, InSettings);

	FilteredChannels = InSettings.FilteredChannels;

	Proxy.ConfigureFrom(InControlElement, InSettings);
	Limits.ConfigureFrom(InControlElement, InSettings);
	Shape.ConfigureFrom(InControlElement, InSettings);

	bUsePreferredRotationOrder = InSettings.bUsePreferredRotationOrder;
	PreferredRotationOrder = InSettings.PreferredRotationOrder;
}

void FRigUnit_HierarchyAddControlRotator_Settings::Configure(FRigControlSettings& OutSettings) const
{
	Super::Configure(OutSettings);
	
	OutSettings.ControlType = ERigControlType::Rotator;
	OutSettings.FilteredChannels = FilteredChannels;

	Proxy.Configure(OutSettings);
	Limits.Configure(OutSettings);
	Shape.Configure(OutSettings);

	OutSettings.bUsePreferredRotationOrder = bUsePreferredRotationOrder;
	OutSettings.PreferredRotationOrder = PreferredRotationOrder;
}

FRigUnit_HierarchyAddControlRotator_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		Settings.Configure(ControlSettings);

		const FTransform Offset = ProjectOffsetTransform(OffsetTransform, OffsetSpace, Parent, ExecuteContext.Hierarchy);

		FRotator Rotator = InitialValue;
		if(Settings.InitialSpace == ERigVMTransformSpace::GlobalSpace)
		{
			const FTransform ParentTransform = ExecuteContext.Hierarchy->GetGlobalTransform(Parent);
			const FTransform GlobalOffsetTransform = Offset * ParentTransform;
			FTransform LocalTransform = FTransform(Rotator).GetRelativeTransform(GlobalOffsetTransform);
			LocalTransform.NormalizeRotation();
			Rotator = LocalTransform.Rotator(); 
		}

		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(Rotator.Euler()));
		const FTransform ShapeTransform = Settings.Shape.Transform;
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddControl(Name, Parent, ControlSettings, Value, Offset, ShapeTransform, false, false);
	}
}

void FRigUnit_HierarchyAddControlTransform_LimitSettings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	switch (InSettings.ControlType)
	{
		case ERigControlType::EulerTransform:
		case ERigControlType::Transform:
		{
			LimitScaleX = InSettings.LimitEnabled[6];
			LimitScaleY = InSettings.LimitEnabled[7];
			LimitScaleZ = InSettings.LimitEnabled[8];
			// fall through
		}
		case ERigControlType::TransformNoScale:
		{
			LimitPitch = InSettings.LimitEnabled[3];
			LimitYaw = InSettings.LimitEnabled[4];
			LimitRoll = InSettings.LimitEnabled[5];
			// fall through
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		case ERigControlType::Rotator:
		{
			LimitTranslationZ = InSettings.LimitEnabled[2];
			// fall through
		}
		case ERigControlType::Vector2D:
		{
			LimitTranslationY = InSettings.LimitEnabled[1];
			// fall through
		}
		case ERigControlType::Integer:
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			LimitTranslationX = InSettings.LimitEnabled[0];
			// fall through
		}
		case ERigControlType::Bool:
		default:
		{
		}
	}

	MinValue = InSettings.MinimumValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
	MaxValue = InSettings.MaximumValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
	bDrawLimits = InSettings.bDrawLimits;
}

void FRigUnit_HierarchyAddControlTransform_LimitSettings::Configure(FRigControlSettings& OutSettings) const
{
	OutSettings.SetupLimitArrayForType(false, false, false);

	switch (OutSettings.ControlType)
	{
	case ERigControlType::EulerTransform:
	case ERigControlType::Transform:
		{
			OutSettings.LimitEnabled[6] = LimitScaleX;
			OutSettings.LimitEnabled[7] = LimitScaleY;
			OutSettings.LimitEnabled[8] = LimitScaleZ;
			// fall through
		}
	case ERigControlType::TransformNoScale:
		{
			OutSettings.LimitEnabled[3] = LimitPitch;
			OutSettings.LimitEnabled[4] = LimitYaw;
			OutSettings.LimitEnabled[5] = LimitRoll;
			// fall through
		}
	case ERigControlType::Position:
	case ERigControlType::Scale:
	case ERigControlType::Rotator:
		{
			OutSettings.LimitEnabled[2] = LimitTranslationZ;
			// fall through
		}
	case ERigControlType::Vector2D:
		{
			OutSettings.LimitEnabled[1] = LimitTranslationY;
			// fall through
		}
	case ERigControlType::Integer:
	case ERigControlType::Float:
	case ERigControlType::ScaleFloat:
		{
			OutSettings.LimitEnabled[0] = LimitTranslationX;
		}
	case ERigControlType::Bool:
	default:
		{
		}
	}
	OutSettings.MinimumValue = FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(MinValue);
	OutSettings.MaximumValue = FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(MaxValue);
	OutSettings.bDrawLimits = bDrawLimits;
}

void FRigUnit_HierarchyAddControlTransform_Settings::ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings)
{
	FRigUnit_HierarchyAddControl_Settings::ConfigureFrom(InControlElement, InSettings);

	FilteredChannels = InSettings.FilteredChannels;
	bUsePreferredRotationOrder = InSettings.bUsePreferredRotationOrder;
	PreferredRotationOrder = InSettings.PreferredRotationOrder;

	Proxy.ConfigureFrom(InControlElement, InSettings);
	Limits.ConfigureFrom(InControlElement, InSettings);
	Shape.ConfigureFrom(InControlElement, InSettings);
}

void FRigUnit_HierarchyAddControlTransform_Settings::Configure(FRigControlSettings& OutSettings) const
{
	Super::Configure(OutSettings);
	
	OutSettings.ControlType = ERigControlType::EulerTransform;
	OutSettings.FilteredChannels = FilteredChannels;
	OutSettings.bUsePreferredRotationOrder = bUsePreferredRotationOrder;
	OutSettings.PreferredRotationOrder = PreferredRotationOrder;

	Proxy.Configure(OutSettings);
	Limits.Configure(OutSettings);
	Shape.Configure(OutSettings);
}

FRigUnit_HierarchyAddControlTransform_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		Settings.Configure(ControlSettings);

		const FTransform Offset = ProjectOffsetTransform(OffsetTransform, OffsetSpace, Parent, ExecuteContext.Hierarchy);

		FTransform Transform = InitialValue;
		if(Settings.InitialSpace == ERigVMTransformSpace::GlobalSpace)
		{
			const FTransform ParentTransform = ExecuteContext.Hierarchy->GetGlobalTransform(Parent);
			const FTransform GlobalOffsetTransform = Offset * ParentTransform;
			Transform = Transform.GetRelativeTransform(GlobalOffsetTransform);
			Transform.NormalizeRotation();
		}
		
		const FEulerTransform EulerTransform(Transform);
		const FRigControlValue Value = FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(EulerTransform);
		const FTransform ShapeTransform = Settings.Shape.Transform;
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddControl(Name, Parent, ControlSettings, Value, Offset, ShapeTransform, false, false);
	}
}

FRigUnit_HierarchyAddAnimationChannelBool_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Bool;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.MinimumValue = FRigControlValue::Make<bool>(MinimumValue);
		ControlSettings.MaximumValue = FRigControlValue::Make<bool>(MaximumValue);
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<bool>(InitialValue);

		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelFloat_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Float;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.Enabled;
		ControlSettings.MinimumValue = FRigControlValue::Make<float>(MinimumValue);
		ControlSettings.MaximumValue = FRigControlValue::Make<float>(MaximumValue);
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<float>(InitialValue);
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelScaleFloat_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::ScaleFloat;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.Enabled;
		ControlSettings.MinimumValue = FRigControlValue::Make<float>(MinimumValue);
		ControlSettings.MaximumValue = FRigControlValue::Make<float>(MaximumValue);
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<float>(InitialValue);
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelInteger_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Integer;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.Enabled;
		ControlSettings.MinimumValue = FRigControlValue::Make<int32>(MinimumValue);
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		ControlSettings.ControlEnum = ControlEnum;

		if (ControlEnum)
		{
			ControlSettings.MaximumValue = FRigControlValue::Make<int32>(ControlEnum->GetMaxEnumValue());
		}
		else
		{
			ControlSettings.MaximumValue = FRigControlValue::Make<int32>(MaximumValue);
		}
		const FRigControlValue Value = FRigControlValue::Make<int32>(InitialValue);
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelVector2D_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Vector2D;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.X;
		ControlSettings.LimitEnabled[1] = LimitsEnabled.Y;
		ControlSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinimumValue.X, MinimumValue.Y, 0.f));
		ControlSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaximumValue.X, MaximumValue.Y, 0.f));
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(InitialValue.X, InitialValue.Y, 0.f));
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelVector_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Position;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.X;
		ControlSettings.LimitEnabled[1] = LimitsEnabled.Y;
		ControlSettings.LimitEnabled[2] = LimitsEnabled.Z;
		ControlSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinimumValue));
		ControlSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaximumValue));
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(InitialValue));
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelScaleVector_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Scale;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.X;
		ControlSettings.LimitEnabled[1] = LimitsEnabled.Y;
		ControlSettings.LimitEnabled[2] = LimitsEnabled.Z;
		ControlSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinimumValue));
		ControlSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaximumValue));
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(InitialValue));
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyAddAnimationChannelRotator_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	const FRigHierarchyModulePath ModulePath(Name);
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid ':' character."), *ModulePath.GetPath());
	}
	if (ModulePath.UsesModuleNameFormat())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation channel name %s contains invalid '/' character."), *ModulePath.GetPath());
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigControlSettings ControlSettings;
		ControlSettings.ControlType = ERigControlType::Rotator;
		ControlSettings.SetupLimitArrayForType(true, true, true);
		ControlSettings.LimitEnabled[0] = LimitsEnabled.Pitch;
		ControlSettings.LimitEnabled[1] = LimitsEnabled.Yaw;
		ControlSettings.LimitEnabled[2] = LimitsEnabled.Roll;
		ControlSettings.MinimumValue = FRigControlValue::Make<FVector3f>(FVector3f(MinimumValue.Euler()));
		ControlSettings.MaximumValue = FRigControlValue::Make<FVector3f>(FVector3f(MaximumValue.Euler()));
		ControlSettings.DisplayName = Controller->GetHierarchy()->GetSafeNewDisplayName(Parent, Name);
		const FRigControlValue Value = FRigControlValue::Make<FVector3f>(FVector3f(InitialValue.Euler()));
		
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddAnimationChannel(Name, Parent, ControlSettings, false, false);

		if(Item.IsValid())
		{
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Initial, false, false);
			ExecuteContext.Hierarchy->SetControlValue(Item, Value, ERigControlValueType::Current, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel could not be added. Is Parent valid?"));
		}
	}
}

FRigUnit_HierarchyGetShapeSettings_Execute()
{
	Settings = FRigUnit_HierarchyAddControl_ShapeSettings();

	if(URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Item))
		{
			Settings.bVisible = ControlElement->Settings.bShapeVisible;
			Settings.Color = ControlElement->Settings.ShapeColor;
			Settings.Name = ControlElement->Settings.ShapeName;
			Settings.Transform = Hierarchy->GetControlShapeTransform((FRigControlElement*)ControlElement, ERigTransformType::InitialLocal);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Control '%s' does not exist."), *Item.ToString());
		}
	}
}

FRigUnit_HierarchySetShapeSettings_Execute()
{
	if(URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if(FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Item))
		{
			FRigControlSettings& CurrentSettings = ControlElement->Settings;

			const bool bNotifySettingsChanged =
				CurrentSettings.bShapeVisible != Settings.bVisible ||
				CurrentSettings.ShapeColor != Settings.Color ||
				CurrentSettings.ShapeName != Settings.Name;

			CurrentSettings.bShapeVisible = Settings.bVisible;
			CurrentSettings.ShapeColor = Settings.Color;
			CurrentSettings.ShapeName = Settings.Name;

			if (bNotifySettingsChanged)
			{
				Hierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);
			}
			
			Hierarchy->SetControlShapeTransformByIndex(ControlElement->GetIndex(), Settings.Transform, true, false);
			Hierarchy->SetControlShapeTransformByIndex(ControlElement->GetIndex(), Settings.Transform, false, false);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Control '%s' does not exist."), *Item.ToString());
		}
	}
}

FRigUnit_HierarchyAddSocket_Execute()
{
	FString ErrorMessage;
	if(!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, true, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	Item.Reset();

	if(URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController(true))
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());
		Item = Controller->AddSocket(Name, Parent, Transform, Space == ERigVMTransformSpace::GlobalSpace, Color, Description, false, false);
	}
}
