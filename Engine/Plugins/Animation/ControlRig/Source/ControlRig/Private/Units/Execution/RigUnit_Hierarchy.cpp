// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_Hierarchy.h"
#include "Units/Execution/RigUnit_Item.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_Collection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Hierarchy)

FRigUnit_HierarchyGetParent_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (bDefaultParent)
	{
		if(CachedChild.IsIdentical(Child, ExecuteContext.Hierarchy))
		{
			Parent = CachedParent.GetKey();
		}
		else
		{
			Parent.Reset();
			CachedParent.Reset();

			if(CachedChild.UpdateCache(Child, ExecuteContext.Hierarchy))
			{
				Parent = ExecuteContext.Hierarchy->GetFirstParent(CachedChild.GetResolvedKey());
				if(Parent.IsValid())
				{
					CachedParent.UpdateCache(Parent, ExecuteContext.Hierarchy);
				}
			}
		}
	}
	else
	{
		CachedChild.UpdateCache(Child, ExecuteContext.Hierarchy);
		const FRigElementKey& ChildKey = CachedChild.GetResolvedKey();

		Parent = ExecuteContext.Hierarchy->GetActiveParent(ChildKey, false);
	}
}

FRigUnit_HierarchyGetParents_Execute()
{
	FRigUnit_HierarchyGetParentsItemArray::StaticExecute(ExecuteContext, Child, bIncludeChild, bReverse, true, Parents.Keys, CachedChild, CachedParents);
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyGetParents::GetUpgradeInfo() const
{
	FRigUnit_HierarchyGetParentsItemArray NewNode;
	NewNode.Child = Child;
	NewNode.bIncludeChild = bIncludeChild;
	NewNode.bReverse = bReverse;
	NewNode.Parents = Parents.Keys;
	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_HierarchyGetParentsItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (bDefaultParent)
	{
		if(!CachedChild.IsIdentical(Child, ExecuteContext.Hierarchy))
		{
			CachedParents.Reset();

			if(CachedChild.UpdateCache(Child, ExecuteContext.Hierarchy))
			{
				TArray<FRigElementKey> Keys;
				FRigElementKey Parent = CachedChild.GetResolvedKey();
				do
				{
					if(bIncludeChild || Parent != Child)
					{
						Keys.Add(Parent);
					}
					Parent = ExecuteContext.Hierarchy->GetFirstParent(Parent);
				}
				while(Parent.IsValid());

				CachedParents = FRigElementKeyCollection(Keys);
				if(bReverse)
				{
					CachedParents = FRigElementKeyCollection::MakeReversed(CachedParents);
				}
			}
		}

		Parents = CachedParents.Keys;
	}
	else
	{
		CachedChild.UpdateCache(Child, ExecuteContext.Hierarchy);

		TArray<FRigElementKey> Keys;
		FRigElementKey Parent = CachedChild.GetResolvedKey();
		do
		{
			if(bIncludeChild || Parent != Child)
			{
				Keys.Add(Parent);
			}
			const FRigElementKey PreviousParent = Parent;

			Parent = ExecuteContext.Hierarchy->GetActiveParent(Parent, false);
		}
		while(Parent.IsValid());

		if(bReverse)
		{
			Algo::Reverse(Keys);
		}

		Parents = Keys;
	}
}

FRigUnit_HierarchyGetChildren_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(!CachedParent.IsIdentical(Parent, ExecuteContext.Hierarchy))
	{
		CachedChildren.Reset();

		if(CachedParent.UpdateCache(Parent, ExecuteContext.Hierarchy))
		{
			const FRigElementKey ResolvedParent = CachedParent.GetResolvedKey();

			TArray<FRigElementKey> Keys;

			if(bIncludeParent)
			{
				Keys.Add(ResolvedParent);
			}

			Keys.Append(ExecuteContext.Hierarchy->GetChildren(ResolvedParent, bRecursive));

			CachedChildren = FRigElementKeyCollection(Keys);
		}
	}

	Children = CachedChildren;
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyGetChildren::GetUpgradeInfo() const
{
	FRigUnit_CollectionChildrenArray NewNode;
	NewNode.Parent = Parent;
	NewNode.bRecursive = bRecursive;
	NewNode.bIncludeParent = bIncludeParent;
	NewNode.TypeToSearch = ERigElementType::All;
	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_HierarchyGetSiblings_Execute()
{
	FRigUnit_HierarchyGetSiblingsItemArray::StaticExecute(ExecuteContext, Item, bIncludeItem, true, Siblings.Keys, CachedItem, CachedSiblings);
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyGetSiblings::GetUpgradeInfo() const
{
	FRigUnit_HierarchyGetSiblingsItemArray NewNode;
	NewNode.Item = Item;
	NewNode.bIncludeItem = bIncludeItem;
	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_HierarchyGetSiblingsItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (bDefaultSiblings)
	{
		if(!CachedItem.IsIdentical(Item, ExecuteContext.Hierarchy))
		{
			CachedSiblings.Reset();

			if(CachedItem.UpdateCache(Item, ExecuteContext.Hierarchy))
			{
				TArray<FRigElementKey> Keys;

				FRigElementKey Parent = ExecuteContext.Hierarchy->GetFirstParent(CachedItem.GetResolvedKey());
				if(Parent.IsValid())
				{
					TArray<FRigElementKey> Children = ExecuteContext.Hierarchy->GetChildren(Parent, false);
					for(FRigElementKey Child : Children)
					{
						if(bIncludeItem || Child != Item)
						{
							Keys.Add(Child);
						}
					}
				}

				if(Keys.Num() == 0 && bIncludeItem)
				{
					Keys.Add(Item);
				}

				CachedSiblings = FRigElementKeyCollection(Keys);
			}
		}

		Siblings = CachedSiblings.Keys;
	}
	else
	{
		Siblings.Reset();
		
		const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		FRigElementKey ParentKey = Hierarchy->GetActiveParent(Item, false);

		if (!ParentKey.IsValid())
		{
			ParentKey = URigHierarchy::GetWorldSpaceReferenceKey();
		}

		FRigBaseElementChildrenArray SiblingsPtrs = Hierarchy->GetActiveChildren(Hierarchy->Find(ParentKey), false);
		Algo::Transform(SiblingsPtrs, Siblings, [](const FRigBaseElement* Element) { return Element->GetKey(); });

		if (!bIncludeItem)
		{
			Siblings.Remove(Item);
		}
	}
}

FRigUnit_HierarchyGetChainItemArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(!CachedStart.IsIdentical(Start, ExecuteContext.Hierarchy) ||
		!CachedEnd.IsIdentical(End, ExecuteContext.Hierarchy))
	{
		CachedChain.Reset();

		if(CachedStart.UpdateCache(Start, ExecuteContext.Hierarchy) &&
			CachedEnd.UpdateCache(End, ExecuteContext.Hierarchy))
		{
			TArray<FRigElementKey> Keys;

			const FRigTransformElement* StartElement = Cast<FRigTransformElement>(CachedStart.GetElement());
			if(StartElement && (StartElement->GetType() == ERigElementType::Socket))
			{
				StartElement = Cast<FRigTransformElement>(ExecuteContext.Hierarchy->GetFirstParent(StartElement));
			}
			const FRigTransformElement* EndElement = Cast<FRigTransformElement>(CachedEnd.GetElement());
			if(EndElement && (EndElement->GetType() == ERigElementType::Socket))
			{
				EndElement = Cast<FRigTransformElement>(ExecuteContext.Hierarchy->GetFirstParent(EndElement));
			}
			
			if(StartElement == nullptr || EndElement == nullptr)
			{
				Keys.Reset();
			}
			else
			{
				if(bIncludeEnd)
				{
					Keys.Add(EndElement->GetKey());
				}

				const FRigTransformElement* Parent = Cast<FRigTransformElement>(ExecuteContext.Hierarchy->GetFirstParent(EndElement));
				while(Parent && Parent != StartElement)
				{
					Keys.Add(Parent->GetKey());
					Parent = Cast<FRigTransformElement>(ExecuteContext.Hierarchy->GetFirstParent(Parent));
				}

				if(Parent != StartElement)
				{
					Keys.Reset();
#if WITH_EDITOR
					ExecuteContext.Report(EMessageSeverity::Info, ExecuteContext.GetFunctionName(), ExecuteContext.GetInstructionIndex(), TEXT("Start and End are not part of the same chain."));
#endif
				}
				else if(bIncludeStart)
				{
					Keys.Add(StartElement->GetKey());
				}

				CachedChain = FRigElementKeyCollection(Keys);

				// we are collecting the chain in reverse order (from tail to head)
				// so we'll reverse it again if we are expecting the chain to be in order parent to child.
				if(!bReverse)
				{
					CachedChain = FRigElementKeyCollection::MakeReversed(CachedChain);
				}
			}
		}
	}

	Chain = CachedChain.Keys;
}

FRigUnit_HierarchyGetPose_Execute()
{
	FRigUnit_HierarchyGetPoseItemArray::StaticExecute(ExecuteContext, Initial, ElementType, ItemsToGet.Keys, Pose);
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyGetPose::GetUpgradeInfo() const
{
	FRigUnit_HierarchyGetPoseItemArray NewNode;
	NewNode.Initial = Initial;
	NewNode.ElementType = ElementType;
	NewNode.ItemsToGet = ItemsToGet.Keys;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_HierarchyGetPoseItemArray_Execute()
{
	Pose = ExecuteContext.Hierarchy->GetPose(Initial, ElementType, ItemsToGet);
}

FRigUnit_HierarchySetPose_Execute()
{
	FRigUnit_HierarchySetPoseItemArray::StaticExecute(ExecuteContext, Pose, ElementType, Space, ItemsToSet.Keys, Weight);
}

FRigVMStructUpgradeInfo FRigUnit_HierarchySetPose::GetUpgradeInfo() const
{
	FRigUnit_HierarchySetPoseItemArray NewNode;
	NewNode.Pose = Pose;
	NewNode.ElementType = ElementType;
	NewNode.Space = Space;
	NewNode.ItemsToSet = ItemsToSet.Keys;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_HierarchySetPoseItemArray_Execute()
{
	ExecuteContext.Hierarchy->SetPose(
		Pose,
		Space == ERigVMTransformSpace::GlobalSpace ? ERigTransformType::CurrentGlobal : ERigTransformType::CurrentLocal,
		ElementType,
		ItemsToSet,
		Weight
	);
}

FRigUnit_PoseIsEmpty_Execute()
{
	IsEmpty = Pose.Num() == 0;
}

FRigUnit_PoseGetItems_Execute()
{
	FRigUnit_PoseGetItemsItemArray::StaticExecute(ExecuteContext, Pose, ElementType, Items.Keys);
}

FRigVMStructUpgradeInfo FRigUnit_PoseGetItems::GetUpgradeInfo() const
{
	FRigUnit_PoseGetItemsItemArray NewNode;
	NewNode.Pose = Pose;
	NewNode.ElementType = ElementType;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_PoseGetItemsItemArray_Execute()
{
	Items.Reset();

	for(const FRigPoseElement& PoseElement : Pose)
	{
		// filter by type
		if (((uint8)ElementType & (uint8)PoseElement.Index.GetKey().Type) == 0)
		{
			continue;
		}
		Items.Add(PoseElement.Index.GetKey());
	}
}

FRigUnit_PoseGetDelta_Execute()
{
	PosesAreEqual = true;
	ItemsWithDelta.Reset();

	if(PoseA.Num() == 0 && PoseB.Num() == 0)
	{
		PosesAreEqual = true;
		return;
	}

	if(PoseA.Num() == 0 && PoseB.Num() != 0)
	{
		PosesAreEqual = false;
		FRigUnit_PoseGetItems::StaticExecute(ExecuteContext, PoseB, ElementType, ItemsWithDelta);
		return;
	}

	if(PoseA.Num() != 0 && PoseB.Num() == 0)
	{
		PosesAreEqual = false;
		FRigUnit_PoseGetItems::StaticExecute(ExecuteContext, PoseA, ElementType, ItemsWithDelta);
		return;
	}

	const float PositionU = FMath::Abs(PositionThreshold);
	const float RotationU = FMath::Abs(RotationThreshold);
	const float ScaleU = FMath::Abs(ScaleThreshold);
	const float CurveU = FMath::Abs(CurveThreshold);

	// if we should compare a sub set of things
	if(!ItemsToCompare.IsEmpty())
	{
		for(int32 Index = 0; Index < ItemsToCompare.Num(); Index++)
		{
			const FRigElementKey& Key = ItemsToCompare[Index];
			if (((uint8)ElementType & (uint8)Key.Type) == 0)
			{
				continue;
			}

			const int32 IndexA = PoseA.GetIndex(Key);
			if(IndexA == INDEX_NONE)
			{
				PosesAreEqual = false;
				ItemsWithDelta.Add(Key);
				continue;
			}
			
			const int32 IndexB = PoseB.GetIndex(Key);
			if(IndexB == INDEX_NONE)
			{
				PosesAreEqual = false;
				ItemsWithDelta.Add(Key);
				continue;
			}

			const FRigPoseElement& A = PoseA[IndexA];
			const FRigPoseElement& B = PoseB[IndexB];

			if(!ArePoseElementsEqual(A, B, Space, PositionU, RotationU, ScaleU, CurveU))
			{
				PosesAreEqual = false;
				ItemsWithDelta.Add(Key);
			}
		}		
	}
	
	// if the poses have the same hash we can accelerate this
	// since they are structurally the same
	else if(PoseA.PoseHash == PoseB.PoseHash)
	{
		for(int32 Index = 0; Index < PoseA.Num(); Index++)
		{
			const FRigPoseElement& A = PoseA[Index];
			const FRigPoseElement& B = PoseB[Index];
			const FRigElementKey& KeyA = A.Index.GetKey(); 
			const FRigElementKey& KeyB = B.Index.GetKey(); 
			check(KeyA == KeyB);
			
			if (((uint8)ElementType & (uint8)KeyA.Type) == 0)
			{
				continue;
			}

			if(!ArePoseElementsEqual(A, B, Space, PositionU, RotationU, ScaleU, CurveU))
			{
				PosesAreEqual = false;
				ItemsWithDelta.Add(KeyA);
			}
		}
	}
	
	// if the poses have different hashes they might not contain the same elements
	else
	{
		for(int32 IndexA = 0; IndexA < PoseA.Num(); IndexA++)
		{
			const FRigPoseElement& A = PoseA[IndexA];
			const FRigElementKey& KeyA = A.Index.GetKey(); 
			
			if (((uint8)ElementType & (uint8)KeyA.Type) == 0)
			{
				continue;
			}

			const int32 IndexB = PoseB.GetIndex(KeyA);
			if(IndexB == INDEX_NONE)
			{
				PosesAreEqual = false;
				ItemsWithDelta.Add(KeyA);
				continue;
			}

			const FRigPoseElement& B = PoseB[IndexB];
			
			if(!ArePoseElementsEqual(A, B, Space, PositionU, RotationU, ScaleU, CurveU))
			{
				PosesAreEqual = false;
				ItemsWithDelta.Add(KeyA);
			}
		}

		// let's loop the other way as well and find relevant
		// elements which are not part of this
		for(int32 IndexB = 0; IndexB < PoseB.Num(); IndexB++)
		{
			const FRigPoseElement& B = PoseB[IndexB];
			const FRigElementKey& KeyB = B.Index.GetKey(); 
			
			if (((uint8)ElementType & (uint8)KeyB.Type) == 0)
			{
				continue;
			}

			const int32 IndexA = PoseA.GetIndex(KeyB);
			if(IndexA == INDEX_NONE)
			{
				PosesAreEqual = false;
				ItemsWithDelta.AddUnique(KeyB);
			}
		}
	}
}

bool FRigUnit_PoseGetDelta::ArePoseElementsEqual(const FRigPoseElement& A, const FRigPoseElement& B,
                                                 ERigVMTransformSpace Space, float PositionU, float RotationU, float ScaleU, float CurveU)
{
	const FRigElementKey& KeyA = A.Index.GetKey();
	const FRigElementKey& KeyB = B.Index.GetKey();
	check(KeyA == KeyB);
		
	if(KeyA.Type == ERigElementType::Curve)
	{
		return AreCurvesEqual(A.CurveValue, B.CurveValue, CurveU);
	}

	return AreTransformsEqual(
		(Space == ERigVMTransformSpace::GlobalSpace) ? A.GlobalTransform : A.LocalTransform,
		(Space == ERigVMTransformSpace::GlobalSpace) ? B.GlobalTransform : B.LocalTransform,
		PositionU, RotationU, ScaleU);
}

bool FRigUnit_PoseGetDelta::AreTransformsEqual(const FTransform& A, const FTransform& B, float PositionU,
	float RotationU, float ScaleU)
{
	if(PositionU > SMALL_NUMBER)
	{
		const FVector PositionA = A.GetLocation();
		const FVector PositionB = B.GetLocation();
		const FVector Delta = (PositionA - PositionB).GetAbs();
		if( (Delta.X >= PositionU) ||
			(Delta.Y >= PositionU) ||
			(Delta.Z >= PositionU))
		{
			return false;
		}
	}

	if(RotationU > SMALL_NUMBER)
	{
		const FVector RotationA = A.GetRotation().Rotator().Euler();
		const FVector RotationB = B.GetRotation().Rotator().Euler();
		const FVector Delta = (RotationA - RotationB).GetAbs();
		if( (Delta.X >= RotationU) ||
			(Delta.Y >= RotationU) ||
			(Delta.Z >= RotationU))
		{
			return false;
		}
	}

	if(ScaleU > SMALL_NUMBER)
	{
		const FVector ScaleA = A.GetScale3D();
		const FVector ScaleB = B.GetScale3D();
		const FVector Delta = (ScaleA - ScaleB).GetAbs();
		if( (Delta.X >= ScaleU) ||
			(Delta.Y >= ScaleU) ||
			(Delta.Z >= ScaleU))
		{
			return false;
		}
	}

	return true;
}

bool FRigUnit_PoseGetDelta::AreCurvesEqual(float A, float B, float CurveU)
{
	if(CurveU > SMALL_NUMBER)
	{
		return FMath::Abs(A - B) < CurveU;
	}
	return true;
}

FRigUnit_PoseGetTransform_Execute()
{
	// set up defaults
	Valid = false;
	Transform = FTransform::Identity;

	if(CachedPoseHash != Pose.HierarchyTopologyVersion)
	{
		CachedPoseHash = Pose.PoseHash;
		CachedPoseElementIndex = Pose.GetIndex(Item);
	}

	if(CachedPoseElementIndex == INDEX_NONE)
	{
		return;
	}

	Valid = true;

	const FRigPoseElement& PoseElement = Pose[CachedPoseElementIndex];

	if(Space == ERigVMTransformSpace::GlobalSpace)
	{
		Transform = PoseElement.GlobalTransform;
	}
	else
	{
		Transform = PoseElement.LocalTransform;
	}
}

FRigUnit_PoseGetTransformArray_Execute()
{
	// set up defaults
	Valid = false;
	Transforms.Reset();

	Valid = true;

	Transforms.SetNum(Pose.Num());

	if(Space == ERigVMTransformSpace::GlobalSpace)
	{
		for(int32 Index=0; Index<Pose.Num(); Index++)
		{
			Transforms[Index] = Pose[Index].GlobalTransform;
		}
	}
	else
	{
		for(int32 Index=0; Index<Pose.Num(); Index++)
		{
			Transforms[Index] = Pose[Index].LocalTransform;
		}
	}
}

FRigUnit_PoseGetCurve_Execute()
{
	// set up defaults
	Valid = false;
	CurveValue = 0.f;

	if(CachedPoseHash != Pose.HierarchyTopologyVersion)
	{
		CachedPoseHash = Pose.PoseHash;
		CachedPoseElementIndex = Pose.GetIndex(FRigElementKey(Curve, ERigElementType::Curve));
	}

	if(CachedPoseElementIndex == INDEX_NONE)
	{
		return;
	}

	Valid = true;

	const FRigPoseElement& PoseElement = Pose[CachedPoseElementIndex];
	CurveValue = PoseElement.CurveValue;
}

FRigUnit_PoseLoop_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(BlockToRun.IsNone())
	{
		Count = Pose.Num();
		Index = 0;
		BlockToRun = ExecutePinName;
	}
	else if(BlockToRun == ExecutePinName)
	{
		Index++;
	}

	if(Index == Count)
	{
		Item = FRigElementKey();
		GlobalTransform = LocalTransform = FTransform::Identity;
		CurveValue = 0.f;
		BlockToRun = ControlFlowCompletedName;
	}
	else
	{
		const FRigPoseElement& PoseElement = Pose.Elements[Index];
		Item = PoseElement.Index.GetKey();
		GlobalTransform = PoseElement.GlobalTransform;
		LocalTransform = PoseElement.LocalTransform;
		CurveValue = PoseElement.CurveValue;
	}

	Ratio = GetRatioFromIndex(Index, Count);
}

FRigUnit_HierarchyCreatePoseItemArray_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if(Hierarchy == nullptr)
	{
		Pose.Reset();
		return;
	}
	
	Pose.Elements.Reset(Entries.Num());
	Pose.HierarchyTopologyVersion = Hierarchy->GetTopologyVersion();
	Pose.PoseHash = Pose.HierarchyTopologyVersion;

	for(const FRigUnit_HierarchyCreatePoseItemArray_Entry& Entry : Entries)
	{
		if(const FRigTransformElement* TransformElement = Hierarchy->Find<FRigTransformElement>(Entry.Item))
		{
			const FRigElementKey& Key = Entry.Item;
			FRigPoseElement Element;
			Element.Index = FCachedRigElement(Key, Hierarchy);
			Element.ActiveParent = Hierarchy->GetActiveParent(Key);
			Element.LocalTransform = Entry.LocalTransform;
			Element.GlobalTransform = Entry.GlobalTransform;
			Element.CurveValue = Entry.CurveValue;
			Element.PreferredEulerAngle = FVector::ZeroVector;

			if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
			{
				if(Entry.UseEulerAngles)
				{
					Element.PreferredEulerAngle = Entry.EulerAngles;
				}
				else
				{
					FRigPreferredEulerAngles EulerAngles = ControlElement->PreferredEulerAngles;
					EulerAngles.SetRotator(Entry.LocalTransform.Rotator(), false, false);
					Element.PreferredEulerAngle = EulerAngles.GetAngles(false, EulerAngles.RotationOrder); 
				}
			}

			Pose.Elements.Add(Element);
			Pose.PoseHash = HashCombine(Pose.PoseHash, GetTypeHash(Key));
		}
	}
}