// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_InverseExecution)

FRigUnit_InverseExecution_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	ExecuteContext.SetEventName(FRigUnit_InverseExecution::EventName);
}

