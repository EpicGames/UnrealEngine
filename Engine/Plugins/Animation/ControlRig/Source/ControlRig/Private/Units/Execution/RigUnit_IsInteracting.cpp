// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_IsInteracting.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_IsInteracting)

FRigUnit_IsInteracting_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	bIsInteracting = ExecuteContext.UnitContext.IsInteracting();
	bIsTranslating = ( ExecuteContext.UnitContext.InteractionType & (uint8)EControlRigInteractionType::Translate) != 0;
	bIsRotating = ( ExecuteContext.UnitContext.InteractionType & (uint8)EControlRigInteractionType::Rotate) != 0;
	bIsScaling = ( ExecuteContext.UnitContext.InteractionType & (uint8)EControlRigInteractionType::Scale) != 0;
	Items = ExecuteContext.UnitContext.ElementsBeingInteracted;
}

