// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_Item.h"
#include "RigVMFunctions/RigVMFunction_Name.h"
#include "Units/RigUnitContext.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Item)

FRigUnit_ItemExists_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Exists = CachedIndex.UpdateCache(Item, ExecuteContext.Hierarchy);
}

FRigUnit_ItemReplace_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Result = Item;
	FRigVMFunction_NameReplace::StaticExecute(ExecuteContext, Item.Name, Old, New, Result.Name);
}

FRigUnit_ItemEquals_Execute()
{
	Result = (A == B);
}

FRigVMStructUpgradeInfo FRigUnit_ItemEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigUnit_ItemNotEquals_Execute()
{
	Result = (A != B);
}

FRigVMStructUpgradeInfo FRigUnit_ItemNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigUnit_ItemTypeEquals_Execute()
{
	Result = (A.Type == B.Type);
}

FRigUnit_ItemTypeNotEquals_Execute()
{
	Result = (A.Type != B.Type);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ItemReplace)
{
	Unit.Item.Name = FName(TEXT("OldItemName"));
	Unit.Item.Type = ERigElementType::Bone;

	Unit.Old = FName(TEXT("Old"));
	Unit.New = FName(TEXT("New"));
	
	Execute();
	AddErrorIfFalse(Unit.Result == FRigElementKey(TEXT("NewItemName"), ERigElementType::Bone), TEXT("unexpected result"));

	Unit.Item.Name = FName(TEXT("OldItemName"));
	Unit.Item.Type = ERigElementType::Bone;

	Unit.Old = FName(TEXT("Old"));
	Unit.New = NAME_None;

	Execute();
	AddErrorIfFalse(Unit.Result == FRigElementKey(TEXT("ItemName"), ERigElementType::Bone), TEXT("unexpected result when New is None"));

	Unit.Item.Name = FName(TEXT("OldItemName"));
	Unit.Item.Type = ERigElementType::Bone;

	Unit.Old = NAME_None;
	Unit.New = FName(TEXT("New")); 

	Execute();
	AddErrorIfFalse(Unit.Result == FRigElementKey(TEXT("OldItemName"), ERigElementType::Bone), TEXT("unexpected result when Old is None"));
	return true;
}
#endif

FRigUnit_ItemToName_Execute()
{
	Result = Value.Name;
}