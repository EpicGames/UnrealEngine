// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_Physics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Physics)

FRigUnit_HierarchyAddPhysicsSolver_Execute()
{
	UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("This node has been deprecated."));
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyAddPhysicsSolver::GetUpgradeInfo() const
{
	// nothing to upgrade to.
	return FRigVMStructUpgradeInfo();
}

FRigUnit_HierarchyAddPhysicsJoint_Execute()
{
	UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("This node has been deprecated."));
}

FRigVMStructUpgradeInfo FRigUnit_HierarchyAddPhysicsJoint::GetUpgradeInfo() const
{
	// nothing to upgrade to.
	return FRigVMStructUpgradeInfo();
}
