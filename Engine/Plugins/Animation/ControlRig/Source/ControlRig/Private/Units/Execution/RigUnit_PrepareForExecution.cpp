// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_PrepareForExecution)

FRigUnit_PrepareForExecution_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	ExecuteContext.SetEventName(FRigUnit_PrepareForExecution::EventName);
}

FRigUnit_PostPrepareForExecution_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	ExecuteContext.SetEventName(FRigUnit_PostPrepareForExecution::EventName);
}
