// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_RigModules.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/RigUnitContext.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_RigModules)

FRigUnit_ResolveConnector_Execute()
{
	TArray<FRigElementKey> ResultArray;
	FRigUnit_ResolveArrayConnector::StaticExecute(ExecuteContext, Connector, SkipSocket, ResultArray, bIsConnected);
	if(ResultArray.IsEmpty())
	{
		Result = Connector;
	}
	else
	{
		Result = ResultArray[0];
	}
}

FRigUnit_ResolveArrayConnector_Execute()
{
	Result = {Connector};

	if (const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		// make sure to only avoid post connectors during construction
		if(const FRigConnectorElement* ConnectorElement = Hierarchy->Find<FRigConnectorElement>(Connector, false))
		{
			if(ConnectorElement->IsPostConstructionConnector())
			{
				if(ExecuteContext.GetEventName() == FRigUnit_PrepareForExecution::EventName)
				{
					static constexpr TCHAR ErrorMessageFormat[] = TEXT("Connector '%s' is only valid during the %s event.");
					const FString ErrorMessage = FString::Printf(ErrorMessageFormat, *Connector.Name.ToString(), *FRigUnit_PostPrepareForExecution::EventName.ToString());
					UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
					bIsConnected = false;
					return;
				}
			}
		}
		
		Result = Hierarchy->GetResolvedTargets(Connector);
		if(SkipSocket && !Result.IsEmpty())
		{
			for(FRigElementKey& ResultKey : Result)
			{
				if(ResultKey.Type == ERigElementType::Socket)
				{
					const FRigElementKey ParentOfSocket = Hierarchy->GetFirstParent(ResultKey);
					if(ParentOfSocket.IsValid())
					{
						ResultKey = ParentOfSocket;
					}
				}
			}
		}
	}

	bIsConnected = false;
	if(Result.Num() == 1)
	{
		bIsConnected = Result[0] != Connector;
		if (!bIsConnected)
		{
			Result.Reset();
		}
	}
	else if(Result.Num() > 1)
	{
		bIsConnected = true;
	}
}

FRigUnit_GetCurrentNameSpace_Execute()
{
	FRigUnit_GetModuleName::StaticExecute(ExecuteContext, NameSpace);
}

FRigVMStructUpgradeInfo FRigUnit_GetCurrentNameSpace::GetUpgradeInfo() const
{
	FRigUnit_GetModuleName NewNode;
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(
		GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetCurrentNameSpace, NameSpace),
		GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetModuleName, Module)
	);
	return Info;
}

FRigUnit_GetItemShortName_Execute()
{
	ShortName = NAME_None;

	if(const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		ShortName = *Hierarchy->GetDisplayNameForUI(Item, EElementNameDisplayMode::ForceShort).ToString();
	}

	if(ShortName.IsNone())
	{
		ShortName = Item.Name;
	}
}

FRigVMStructUpgradeInfo FRigUnit_GetItemShortName::GetUpgradeInfo() const
{
	// we don't have a node to upgrade to for this one
	return FRigVMStructUpgradeInfo();
}


FRigUnit_GetItemNameSpace_Execute()
{
	FRigUnit_GetItemModuleName::StaticExecute(ExecuteContext, Item, HasNameSpace, NameSpace);
}

FRigVMStructUpgradeInfo FRigUnit_GetItemNameSpace::GetUpgradeInfo() const
{
	FRigUnit_GetItemModuleName NewNode;
	NewNode.Item = Item;
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(
		GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetItemNameSpace, NameSpace),
		GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetItemModuleName, Module)
	);
	Info.AddRemappedPin(
		GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetItemNameSpace, HasNameSpace),
		GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetItemModuleName, IsPartOfModule)
	);
	return Info;
}

FRigUnit_IsItemInCurrentNameSpace_Execute()
{
	FRigUnit_IsItemInCurrentModule::StaticExecute(ExecuteContext, Item, Result);
}

FRigVMStructUpgradeInfo FRigUnit_IsItemInCurrentNameSpace::GetUpgradeInfo() const
{
	FRigUnit_IsItemInCurrentModule NewNode;
	NewNode.Item = Item;
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_GetItemsInNameSpace_Execute()
{
	FRigUnit_GetItemsInModule::StaticExecute(ExecuteContext, TypeToSearch, Items);
}

FRigVMStructUpgradeInfo FRigUnit_GetItemsInNameSpace::GetUpgradeInfo() const
{
	FRigUnit_GetItemsInModule NewNode;
	NewNode.TypeToSearch = TypeToSearch;
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigUnit_GetModuleName_Execute()
{
	if(!ExecuteContext.IsRigModule())
	{
#if WITH_EDITOR
		
		static const FString Message = TEXT("This node should only be used in a Rig Module."); 
		ExecuteContext.Report(EMessageSeverity::Warning, ExecuteContext.GetFunctionName(), ExecuteContext.GetInstructionIndex(), Message);
		
#endif
	}
	Module = *ExecuteContext.GetRigModulePrefix();
	if(!Module.IsEmpty())
	{
		// remove the module suffix character from the end
		Module.LeftChopInline(1);
	}
}

FRigUnit_GetItemModuleName_Execute()
{
	Module.Reset();
	IsPartOfModule = false;

	if(const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		const FString ModuleForItem = Hierarchy->GetModuleName(Item);
		if(!ModuleForItem.IsEmpty())
		{
			Module = ModuleForItem;
			IsPartOfModule = true;
		}
	}
}

FRigUnit_IsItemInCurrentModule_Execute()
{
	FString CurrentModule;
	FRigUnit_GetModuleName::StaticExecute(ExecuteContext, CurrentModule);
	bool bHasModule = false;
	FString ItemModule;
	FRigUnit_GetItemModuleName::StaticExecute(ExecuteContext, Item, bHasModule, ItemModule);

	Result = false;
	if(!CurrentModule.IsEmpty() && !ItemModule.IsEmpty())
	{
		Result = ItemModule.Equals(CurrentModule, ESearchCase::IgnoreCase); 
	}
}

FRigUnit_GetItemsInModule_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		Items.Reset();
		return;
	}

	FString Module;
	FRigUnit_GetModuleName::StaticExecute(ExecuteContext, Module);
	if(Module.IsEmpty())
	{
		Items.Reset();
		return;
	}
	const FName ModuleName = *Module;
	
	uint32 Hash = GetTypeHash(StaticStruct());
	Hash = HashCombine(Hash, GetTypeHash((int32)TypeToSearch));
	Hash = HashCombine(Hash, GetTypeHash(ModuleName));

	if(const FRigElementKeyCollection* Cache = Hierarchy->FindCachedCollection(Hash))
	{
		Items = Cache->Keys;
	}
	else
	{
		FRigElementKeyCollection Collection;
		Hierarchy->Traverse([Hierarchy, ModuleName, &Collection, TypeToSearch]
			(const FRigBaseElement* InElement, bool &bContinue)
			{
				bContinue = true;
						
				const FRigElementKey Key = InElement->GetKey();
				if(((uint8)TypeToSearch & (uint8)Key.Type) == (uint8)Key.Type)
				{
					const FName ItemModule = Hierarchy->GetModuleFName(Key);
					if(!ItemModule.IsNone())
					{
						if(ItemModule.IsEqual(ModuleName, ENameCase::IgnoreCase))
						{
							Collection.AddUnique(Key);
						}
					}
				}
			}
		);

		Hierarchy->AddCachedCollection(Hash, Collection);
		Items = Collection.Keys;
	}
}
