// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Execution/RigUnit_SequenceExecution.h"
#include "RigVMFunctions/Execution/RigVMFunction_Sequence.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SequenceExecution)

FRigUnit_SequenceExecution_Execute()
{
	// nothing to do here. the execute context is actually
	// the same shared memory for all pins
}

FRigVMStructUpgradeInfo FRigUnit_SequenceExecution::GetUpgradeInfo() const
{
	FRigVMFunction_Sequence NewNode;
	
	FRigVMStructUpgradeInfo Info(*this, NewNode);

	// add two more pins
	Info.AddAggregatePin(); // "C"
	Info.AddAggregatePin(); // "D"

	return Info;
}


