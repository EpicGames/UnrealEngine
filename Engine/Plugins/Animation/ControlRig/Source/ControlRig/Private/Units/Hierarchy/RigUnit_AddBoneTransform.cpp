// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_AddBoneTransform.h"
#include "Units/Hierarchy/RigUnit_OffsetTransform.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AddBoneTransform)

FRigUnit_AddBoneTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Bone, ERigElementType::Bone); 
		if (!CachedBone.UpdateCache(Key, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else
		{
			FTransform TargetTransform;
			const FTransform PreviousTransform = Hierarchy->GetGlobalTransform(CachedBone);

			if (bPostMultiply)
			{
				TargetTransform = PreviousTransform * Transform;
			}
			else
			{
				TargetTransform = Transform * PreviousTransform;
			}

			if (!FMath::IsNearlyEqual(Weight, 1.f))
			{
				float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
				TargetTransform = FControlRigMathLibrary::LerpTransform(PreviousTransform, TargetTransform, T);
			}

			Hierarchy->SetGlobalTransform(CachedBone, TargetTransform, bPropagateToChildren);
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_AddBoneTransform::GetUpgradeInfo() const
{
	FRigUnit_OffsetTransformForItem NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Weight = Weight;
	NewNode.bPropagateToChildren = bPropagateToChildren;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	return Info;
}
