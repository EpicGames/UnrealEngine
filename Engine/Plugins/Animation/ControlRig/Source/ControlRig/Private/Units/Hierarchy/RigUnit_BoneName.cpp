// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_BoneName.h"

#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_BoneName)

FRigUnit_Item_Execute()
{
}

FRigUnit_ItemArray_Execute()
{
}

FRigVMStructUpgradeInfo FRigUnit_ItemArray::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(
		StaticStruct(),
		FRigVMDispatch_Constant::StaticStruct(),
		{{TEXT("Items"), TEXT("Value")}});
}

FRigUnit_BoneName_Execute()
{
}

FRigVMStructUpgradeInfo FRigUnit_BoneName::GetUpgradeInfo() const
{
	FRigUnit_Item NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	return Info;
}

FRigUnit_SpaceName_Execute()
{
}

FRigVMStructUpgradeInfo FRigUnit_SpaceName::GetUpgradeInfo() const
{
	FRigUnit_Item NewNode;
	NewNode.Item = FRigElementKey(Space, ERigElementType::Null);

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Space"), TEXT("Item.Name"));
	return Info;
}

FRigUnit_ControlName_Execute()
{
}

FRigVMStructUpgradeInfo FRigUnit_ControlName::GetUpgradeInfo() const
{
	FRigUnit_Item NewNode;
	NewNode.Item = FRigElementKey(Control, ERigElementType::Control);

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Control"), TEXT("Item.Name"));
	return Info;
}

