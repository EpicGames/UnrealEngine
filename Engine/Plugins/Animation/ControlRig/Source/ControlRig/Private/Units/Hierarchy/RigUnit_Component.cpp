// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_Component.h"

#include "Animation/AttributeTypes.h"
#include "Units/RigUnitContext.h"
#include "Rigs/RigHierarchyController.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Component)

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigDispatch_ComponentBase
//////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRigDispatch_ComponentBase::IsTypeSupported(const TRigVMTypeIndex& InTypeIndex, const FRigVMRegistryHandle& InRegistry)
{
	const FRigVMTemplateArgumentType& InType = InRegistry->GetType_NoLock(InTypeIndex);

	if (IsValid(InType.CPPTypeObject))
	{
		if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InType.CPPTypeObject))
		{
			return ScriptStruct != FRigBaseComponent::StaticStruct() &&
				ScriptStruct->IsChildOf(FRigBaseComponent::StaticStruct());
		}
	}
	return false;
}

const TRigVMTypeIndex& FRigDispatch_ComponentBase::GetElementKeyType(FRigVMRegistryHandle& InRegistry)
{
	static TRigVMTypeIndex RigElementKeyType = InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(FRigElementKey::StaticStruct()), false);
	return RigElementKeyType;
}

const TRigVMTypeIndex& FRigDispatch_ComponentBase::GetComponentKeyType(FRigVMRegistryHandle& InRegistry)
{
	static TRigVMTypeIndex RigComponentKeyType = InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(FRigComponentKey::StaticStruct()), false);
	return RigComponentKeyType;
}

void FRigDispatch_ComponentBase::RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	Super::RegisterDependencyTypes_NoLock(InRegistry);

	(void)InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(FRigComponentKey::StaticStruct()), true);
	
	for (TObjectIterator<UScriptStruct> ScriptStructIt; ScriptStructIt; ++ScriptStructIt)
	{
		UScriptStruct* ScriptStruct = *ScriptStructIt;
		if(ScriptStruct != FRigBaseComponent::StaticStruct() && ScriptStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
		{
			(void)InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(ScriptStruct), true);
		}
	}
}

const TArray<FRigVMExecuteArgument>& FRigDispatch_ComponentBase::GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const
{
	if(IsMutable() && ExecuteInfos.IsEmpty())
	{
		ExecuteInfos.Emplace(ExecuteArgName, ERigVMPinDirection::IO);
	}
	return ExecuteInfos;
}

#if WITH_EDITOR

FText FRigDispatch_ComponentBase::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == NameArgName)
	{
		return NSLOCTEXT("FRigDispatch_ComponentBase", "NameArgTooltip", "The name of the component (can be empty)");
	}
	if(InArgumentName == ItemArgName)
	{
		return NSLOCTEXT("FRigDispatch_ComponentBase", "ItemArgTooltip", "The item for this component");
	}
	if(InArgumentName == KeyArgName)
	{
		return NSLOCTEXT("FRigDispatch_ComponentBase", "KeyArgTooltip", "The key of the component");
	}
	if(InArgumentName == ComponentArgName)
	{
		return NSLOCTEXT("FRigDispatch_ComponentBase", "ComponentArgTooltip", "The actual component");
	}
	if(InArgumentName == SuccessArgName)
	{
		return NSLOCTEXT("FRigDispatch_ComponentBase", "SuccessArgTooltip", "Returns true if the operation was successful.");
	}
	return FRigDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);	
}

FString FRigDispatch_ComponentBase::GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == NameArgName)
	{
		return FString();
	}
	if(InArgumentName == ItemArgName)
	{
		const FRigElementKey DefaultItem(NAME_None, ERigElementType::Bone);
		FString DefaultValue;
		FRigElementKey::StaticStruct()->ExportText(DefaultValue, &DefaultItem, &DefaultItem, nullptr, PPF_None, nullptr);
		return DefaultValue;
	}
	return FRigDispatchFactory::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

FRigVMTemplateTypeMap FRigDispatch_ComponentBase::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex,
	FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;

	if (ensure(InArgumentName == ComponentArgName))
	{
		if (IsTypeSupported(InTypeIndex, InRegistry))
		{
			const TArray<FRigVMTemplateArgumentInfo>& Infos = GetArgumentInfos(InRegistry);
			for (const FRigVMTemplateArgumentInfo& Info : Infos)
			{
				if (Info.Name == ComponentArgName)
				{
					Types.Add(Info.Name, InTypeIndex);
				}
				else
				{
					const FRigVMTemplateArgument& Argument = Info.GetArgument_NoLock(InRegistry);
					check(Argument.IsSingleton_NoLock({}, InRegistry));
					Types.Add(Info.Name, Argument.GetTypeIndex_NoLock(0, InRegistry));
				}
			}
		}
	}
	return Types;
}

#endif

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigDispatch_SpawnComponent
//////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_SpawnComponent::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		ItemArgIndex = CachedArgumentInfos.Emplace(ItemArgName, ERigVMPinDirection::Input, GetElementKeyType(InRegistry));
		NameArgIndex = CachedArgumentInfos.Emplace(NameArgName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);
		ComponentArgIndex = CachedArgumentInfos.Add(FRigVMTemplateArgumentInfo(ComponentArgName, ERigVMPinDirection::Input, {FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue}, &FRigDispatch_SpawnComponent::IsTypeSupported));
		KeyArgIndex = CachedArgumentInfos.Emplace(KeyArgName, ERigVMPinDirection::Output, GetComponentKeyType(InRegistry));
		SuccessArgIndex = CachedArgumentInfos.Emplace(SuccessArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	
	return CachedArgumentInfos;
}

#if WITH_EDITOR

FString FRigDispatch_SpawnComponent::GetKeywords() const
{
	return FRigDispatch_ComponentBase::GetKeywords() + TEXT(",AddComponent,CreateComponent");
}

#endif

void FRigDispatch_SpawnComponent::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray /* not needed */)
{
	const FRigDispatch_SpawnComponent* Factory = static_cast<const FRigDispatch_SpawnComponent*>(InContext.Factory);
		
	// unpack the memory
	const FRigElementKey& ElementKey = *(const FRigElementKey*)Handles[Factory->ItemArgIndex].GetInputData();
	const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
	const uint8* SourceComponent = Handles[Factory->ComponentArgIndex].GetInputData();
	FRigComponentKey& ComponentKey = *(FRigComponentKey*)Handles[Factory->KeyArgIndex].GetOutputData();
	bool& bSuccess = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();

	const FStructProperty* StructProperty = CastField<FStructProperty>(Handles[Factory->ComponentArgIndex].GetResolvedProperty());
	check(StructProperty);
	UScriptStruct* ComponentStruct = StructProperty->Struct;
	check(ComponentStruct && ComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()));

	ComponentKey = FRigComponentKey();
	bSuccess = false;

	// extract the component from the hierarchy
	const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();
	if(!Context.Hierarchy->Contains(ElementKey))
	{
#if WITH_EDITOR
		Context.Report(FRigVMLogSettings(EMessageSeverity::Warning), Context.GetFunctionName(), Context.GetInstructionIndex(), FString::Printf(TEXT("Item %s not found."), *ElementKey.ToString()));
#endif
		return;
	}

	if(URigHierarchyController* Controller = Context.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, Context.GetInstructionIndex());
		ComponentKey = Controller->AddComponent(ComponentStruct, Name, ElementKey);
		if(ComponentKey.IsValid())
		{
			FRigBaseComponent* TargetComponent = Context.Hierarchy->FindComponent(ComponentKey);
			check(TargetComponent);
			
			// copy the public content over (members which are not UProperty will be skipped) 
			for (TFieldIterator<FProperty> It(ComponentStruct); It; ++It)
			{
				It->CopyCompleteValue_InContainer(TargetComponent, SourceComponent);
			}

			bSuccess = true;
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigDispatch_GetComponentContent
//////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_GetComponentContent::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		KeyArgIndex = CachedArgumentInfos.Emplace(KeyArgName, ERigVMPinDirection::Input, GetComponentKeyType(InRegistry));
		ComponentArgIndex = CachedArgumentInfos.Add(FRigVMTemplateArgumentInfo(ComponentArgName, ERigVMPinDirection::Output, {FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue}, &FRigDispatch_GetComponentContent::IsTypeSupported));
		SuccessArgIndex = CachedArgumentInfos.Emplace(SuccessArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	
	return CachedArgumentInfos;
}

void FRigDispatch_GetComponentContent::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray)
{
	const FRigDispatch_GetComponentContent* Factory = static_cast<const FRigDispatch_GetComponentContent*>(InContext.Factory);
			
	// unpack the memory
	const FRigComponentKey& ComponentKey = *(const FRigComponentKey*)Handles[Factory->KeyArgIndex].GetInputData();
	uint8* TargetComponent = Handles[Factory->ComponentArgIndex].GetOutputData();
	bool& bSuccess = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();

	const FStructProperty* StructProperty = CastField<FStructProperty>(Handles[Factory->ComponentArgIndex].GetResolvedProperty());
	check(StructProperty);
	const UScriptStruct* ComponentStruct = StructProperty->Struct;
	check(ComponentStruct && ComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()));

	// extract the component from the hierarchy
	const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();
	const FRigBaseComponent* SourceComponent = Context.Hierarchy->FindComponent(ComponentKey);
	if(SourceComponent == nullptr)
	{
		ComponentStruct->InitializeDefaultValue(TargetComponent);
		// Since we return success/failure, this is not an error/warning
		bSuccess = false;
		return;
	}

	if(SourceComponent->GetScriptStruct() != ComponentStruct)
	{
		ComponentStruct->InitializeDefaultValue(TargetComponent);
#if WITH_EDITOR
		const FString Message = FString::Printf(TEXT("Component pin type (%s) doesn't match component in hierarchy (%s)."), *ComponentStruct->GetName(), *SourceComponent->GetScriptStruct()->GetName());
		Context.Report(FRigVMLogSettings(EMessageSeverity::Warning), Context.GetFunctionName(), Context.GetInstructionIndex(), Message);
#endif
		bSuccess = false;
		return;
	}

	ComponentStruct->CopyScriptStruct(TargetComponent, SourceComponent, 1);
	bSuccess = true;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigDispatch_SetComponentContent
//////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_SetComponentContent::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		KeyArgIndex = CachedArgumentInfos.Emplace(KeyArgName, ERigVMPinDirection::Input, GetComponentKeyType(InRegistry));
		ComponentArgIndex = CachedArgumentInfos.Add(FRigVMTemplateArgumentInfo(ComponentArgName, ERigVMPinDirection::Input, {FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue}, &FRigDispatch_SetComponentContent::IsTypeSupported));
		SuccessArgIndex = CachedArgumentInfos.Emplace(SuccessArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	
	return CachedArgumentInfos;
}

void FRigDispatch_SetComponentContent::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray)
{
	const FRigDispatch_SetComponentContent* Factory = static_cast<const FRigDispatch_SetComponentContent*>(InContext.Factory);
			
	// unpack the memory
	const FRigComponentKey& ComponentKey = *(const FRigComponentKey*)Handles[Factory->KeyArgIndex].GetInputData();
	const  uint8* SourceComponent = Handles[Factory->ComponentArgIndex].GetInputData();
	bool& bSuccess = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();

	const FStructProperty* StructProperty = CastField<FStructProperty>(Handles[Factory->ComponentArgIndex].GetResolvedProperty());
	check(StructProperty);
	const UScriptStruct* ComponentStruct = StructProperty->Struct;
	check(ComponentStruct && ComponentStruct->IsChildOf(FRigBaseComponent::StaticStruct()));

	// extract the component from the hierarchy
	const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();
	FRigBaseComponent* TargetComponent = Context.Hierarchy->FindComponent(ComponentKey);
	if(TargetComponent == nullptr)
	{
#if WITH_EDITOR
		Context.Report(FRigVMLogSettings(EMessageSeverity::Warning), Context.GetFunctionName(), Context.GetInstructionIndex(), FString::Printf(TEXT("%s not found."), *ComponentKey.ToString()));
#endif
		bSuccess = false;
		return;
	}

	if(TargetComponent->GetScriptStruct() != ComponentStruct)
	{
#if WITH_EDITOR
		const FString Message = FString::Printf(TEXT("Component pin type (%s) doesn't match component in hierarchy (%s)."), *ComponentStruct->GetName(), *TargetComponent->GetScriptStruct()->GetName());
		Context.Report(FRigVMLogSettings(EMessageSeverity::Warning), Context.GetFunctionName(), Context.GetInstructionIndex(), Message);
#endif
		bSuccess = false;
		return;
	}

	// copy the public content over (members which are not UProperty will be skipped) 
	for (TFieldIterator<FProperty> It(ComponentStruct); It; ++It)
	{
		It->CopyCompleteValue_InContainer(TargetComponent, SourceComponent);
	}
	bSuccess = true;
}
