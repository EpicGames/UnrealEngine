// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_ControlChannel.h"
#include "Units/Hierarchy/RigUnit_ControlChannelFromItem.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ControlChannel)

bool FRigUnit_GetAnimationChannelBase::UpdateCache(const URigHierarchy* InHierarchy, const FName& Control, const FName& Channel, FRigElementKey& Key, int32& Hash)
{
	if (!IsValid(InHierarchy))
	{
		return false;
	}
	
	if(!Key.IsValid())
	{
		Hash = INDEX_NONE;
	}

	const int32 ExpectedHash = (int32)HashCombine(GetTypeHash(InHierarchy->GetTopologyVersion()), HashCombine(GetTypeHash(Control), GetTypeHash(Channel)));
	if(ExpectedHash != Hash)
	{
		if(const FRigControlElement* ControlElement = InHierarchy->Find<FRigControlElement>(FRigElementKey(Control, ERigElementType::Control)))
		{
			FString ChannelName = Channel.ToString();
			(void)FRigHierarchyModulePath(ChannelName).Split(nullptr, &ChannelName);
			
			const FName ChannelFName = *ChannelName;
			for(const FRigBaseElement* Child : InHierarchy->GetChildren(ControlElement))
			{
				if(const FRigControlElement* ChildControl = Cast<FRigControlElement>(Child))
				{
					if(ChildControl->IsAnimationChannel())
					{
						if(ChildControl->GetDisplayName().IsEqual(ChannelFName))
						{
							Key = ChildControl->GetKey();
							Hash = ExpectedHash;
							return true;
						}
					}
				}
			}
		}

		return false;
	}

	return true;
}

FRigUnit_GetBoolAnimationChannel_Execute()
{
	Value = false;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetBoolAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_GetFloatAnimationChannel_Execute()
{
	Value = 0.f;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetFloatAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_GetIntAnimationChannel_Execute()
{
	Value = 0;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetIntAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_GetVector2DAnimationChannel_Execute()
{
	Value = FVector2D::ZeroVector;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetVector2DAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_GetVectorAnimationChannel_Execute()
{
	Value = FVector::ZeroVector;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetVectorAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_GetRotatorAnimationChannel_Execute()
{
	Value = FRotator::ZeroRotator;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetRotatorAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_GetTransformAnimationChannel_Execute()
{
	Value = FTransform::Identity;
	
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_GetTransformAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetBoolAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetBoolAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetFloatAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetFloatAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetIntAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetIntAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetVector2DAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetVector2DAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetVectorAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetVectorAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetRotatorAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetRotatorAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}

FRigUnit_SetTransformAnimationChannel_Execute()
{
	if(!UpdateCache(ExecuteContext.Hierarchy, Control, Channel, CachedChannelKey, CachedChannelHash))
	{
		return;
	}
	FRigUnit_SetTransformAnimationChannelFromItem::StaticExecute(ExecuteContext, Value, CachedChannelKey, bInitial);
}
