// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_ControlChannelFromItem.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ControlChannelFromItem)

FRigUnit_GetBoolAnimationChannelFromItem_Execute()
{
	Value = false;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Bool)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = StoredValue.Get<bool>();
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Bool."), *Item.ToString());
		}
	}
}

FRigUnit_GetFloatAnimationChannelFromItem_Execute()
{
	Value = 0.f;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Float || ChannelElement->Settings.ControlType == ERigControlType::ScaleFloat)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = StoredValue.Get<float>();
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Float."), *Item.ToString());
		}
	}
}

FRigUnit_GetIntAnimationChannelFromItem_Execute()
{
	Value = 0;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Integer)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = StoredValue.Get<int32>();
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not an Integer (or Enum)."), *Item.ToString());
		}
	}
}

FRigUnit_GetVector2DAnimationChannelFromItem_Execute()
{
	Value = FVector2D::ZeroVector;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Vector2D)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = FVector2D(StoredValue.Get<FVector2f>());
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Vector2D."), *Item.ToString());
		}
	}
}

FRigUnit_GetVectorAnimationChannelFromItem_Execute()
{
	Value = FVector::ZeroVector;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Position || ChannelElement->Settings.ControlType == ERigControlType::Scale)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = FVector(StoredValue.Get<FVector3f>());
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Vector (Position or Scale)."), *Item.ToString());
		}
	}
}

FRigUnit_GetRotatorAnimationChannelFromItem_Execute()
{
	Value = FRotator::ZeroRotator;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Rotator)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = FRotator::MakeFromEuler(FVector(StoredValue.Get<FVector3f>()));
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Rotator."), *Item.ToString());
		}
	}
}

FRigUnit_GetTransformAnimationChannelFromItem_Execute()
{
	Value = FTransform::Identity;

	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Transform)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = StoredValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
		}
		else if(ChannelElement->Settings.ControlType == ERigControlType::EulerTransform)
		{
			const FRigControlValue StoredValue = ExecuteContext.Hierarchy->GetControlValueByIndex(ChannelElement->GetIndex(), bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
			Value = StoredValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform().ToFTransform();
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a EulerTransform / Transform."), *Item.ToString());
		}
	}
}

FRigUnit_SetBoolAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Bool)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<bool>(Value);
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Bool."), *Item.ToString());
		}
	}
}

FRigUnit_SetFloatAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Float || ChannelElement->Settings.ControlType == ERigControlType::ScaleFloat)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<float>(Value);
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Float."), *Item.ToString());
		}
	}
}

FRigUnit_SetIntAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Integer)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<int32>(Value);
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not an Integer (or Enum)."), *Item.ToString());
		}
	}
}

FRigUnit_SetVector2DAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Vector2D)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<FVector2f>(FVector2f(Value));
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Vector2D."), *Item.ToString());
		}
	}
}

FRigUnit_SetVectorAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Position || ChannelElement->Settings.ControlType == ERigControlType::Scale)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<FVector3f>(FVector3f(Value));
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Vector (Position or Scale)."), *Item.ToString());
		}
	}
}

FRigUnit_SetRotatorAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Rotator)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<FVector3f>(FVector3f(Value.Euler()));
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a Rotator."), *Item.ToString());
		}
	}
}

FRigUnit_SetTransformAnimationChannelFromItem_Execute()
{
	if (!IsValid(ExecuteContext.Hierarchy))
	{
		return;
	}
	
	if(const FRigControlElement* ChannelElement = ExecuteContext.Hierarchy->Find<FRigControlElement>(Item))
	{
		if(ChannelElement->Settings.ControlType == ERigControlType::Transform)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<FRigControlValue::FTransform_Float>(Value);
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else if(ChannelElement->Settings.ControlType == ERigControlType::EulerTransform)
		{
			const FRigControlValue ValueToStore = FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(FEulerTransform(Value));
			ExecuteContext.Hierarchy->SetControlValueByIndex(ChannelElement->GetIndex(), ValueToStore, bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current);
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Animation Channel %s is not a EulerTransform / Transform."), *Item.ToString());
		}
	}
}
