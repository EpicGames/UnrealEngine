// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_CurveExists.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_CurveExists)

FRigUnit_CurveExists_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Exists = Hierarchy && CachedCurveIndex.UpdateCache(FRigElementKey(Curve, ERigElementType::Curve), Hierarchy);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_CurveExists)
{
	static const FName CurveName("Curve");
	Controller->AddCurve(CurveName, 0.f);
	
	Unit.Curve = CurveName;
	Execute();

	AddErrorIfFalse(Unit.Exists, TEXT("Expected curve to exist"));

	Unit.Curve = TEXT("NonExistentCurve");
	Execute();

	AddErrorIfFalse(!Unit.Exists, TEXT("Expected curve to not exist."));

	Unit.Curve = NAME_None;
	Execute();

	AddErrorIfFalse(!Unit.Exists, TEXT("Expected curve to not exist."));
	
	return true;
}
#endif
