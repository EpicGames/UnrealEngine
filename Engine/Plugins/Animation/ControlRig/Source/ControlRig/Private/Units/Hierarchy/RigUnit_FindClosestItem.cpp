// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_FindClosestItem.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_FindClosestItem)

FRigUnit_FindClosestItem_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Items.Num() == 0)
	{
		static FRigElementKey InvalidKey;
		Item = InvalidKey;
		return;
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	CachedItems.SetNum(Items.Num());
	for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ItemIndex++)
	{
		if (!CachedItems[ItemIndex].UpdateCache(Items[ItemIndex], Hierarchy))
		{
			CachedItems.Reset();
			break;
		}
	}

	if (CachedItems.Num() == 0 && Items.Num() > 0)
	{
		for (FRigElementKey CacheItem : Items)
		{
			CachedItems.Add(FCachedRigElement(CacheItem, Hierarchy));
		}

	}

	float NearestDistance = BIG_NUMBER;
	Item = CachedItems[0].GetKey();
	for (int32 ItemIndex = 0; ItemIndex < CachedItems.Num(); ItemIndex++)
	{
		if (CachedItems[ItemIndex].IsValid())
		{
			const FVector Location = Hierarchy->GetGlobalTransform(CachedItems[ItemIndex]).GetLocation();
			const float Distance = (Location - Point).Size();
			if (Distance < NearestDistance)
			{
				NearestDistance = Distance;
				Item = CachedItems[ItemIndex].GetKey();
			}
		}
	}
}