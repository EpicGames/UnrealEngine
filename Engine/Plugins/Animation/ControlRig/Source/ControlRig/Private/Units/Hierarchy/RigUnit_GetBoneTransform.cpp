// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetBoneTransform.h"
#include "Units/Hierarchy/RigUnit_GetInitialBoneTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetBoneTransform)

FRigUnit_GetBoneTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		// KIARAN AUG - 2023 (HACK)
		// This restores the original behavior of this node where it returns the InitialTransform during the first
		// call to Execute() so that downstream nodes (like TransformConstraint) work correctly. This behavior was lost
		// when 23407131 removed the Context.Init pass which called into FRigUnit_GetInitialBoneTransform
		if (bFirstUpdate)
		{
			FRigUnit_GetInitialBoneTransform::StaticExecute(ExecuteContext, Bone, Space, Transform, CachedBone);
			bFirstUpdate = false;
			return;
		}
		if (!CachedBone.UpdateCache(FRigElementKey(Bone, ERigElementType::Bone), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Transform = Hierarchy->GetGlobalTransform(CachedBone);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Transform = Hierarchy->GetLocalTransform(CachedBone);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_GetBoneTransform::GetUpgradeInfo() const
{
	FRigUnit_GetTransform NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Space = Space;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	return Info;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_GetBoneTransform)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(1.f, 2.f, 3.f)), true, ERigBoneType::User);

	Unit.Bone = TEXT("Unknown");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 0.f, 0.f)), TEXT("unexpected global transform"));
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 0.f, 0.f)), TEXT("unexpected local transform"));

	Unit.Bone = TEXT("Root");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected global transform"));
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected local transform"));

	Unit.Bone = TEXT("BoneA");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(1.f, 2.f, 3.f)), TEXT("unexpected global transform"));
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 2.f, 3.f)), TEXT("unexpected local transform"));

	return true;
}
#endif
