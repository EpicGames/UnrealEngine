// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetControlInitialTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetControlInitialTransform)

FRigUnit_GetControlInitialTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control); 
		if (!CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Transform = Hierarchy->GetInitialGlobalTransform(CachedControlIndex);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Transform = Hierarchy->GetInitialLocalTransform(CachedControlIndex);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_GetControlInitialTransform::GetUpgradeInfo() const
{
	FRigUnit_GetTransform NewNode;
	NewNode.Item = FRigElementKey(Control, ERigElementType::Control);
	NewNode.Space = Space;
	NewNode.bInitial = true;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Control"), TEXT("Item.Name"));
	return Info;
}

