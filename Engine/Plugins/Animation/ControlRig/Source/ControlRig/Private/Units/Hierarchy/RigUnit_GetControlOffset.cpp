// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetControlOffset.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetControlOffset)

FRigUnit_GetControlOffset_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if (!CachedIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(CachedIndex);
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					OffsetTransform = Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialGlobal);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					OffsetTransform = Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialLocal);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}