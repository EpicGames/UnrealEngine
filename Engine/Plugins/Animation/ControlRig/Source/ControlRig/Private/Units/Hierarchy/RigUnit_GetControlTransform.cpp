// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetControlTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetControlTransform)

FRigUnit_GetControlBool_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			BoolValue = Hierarchy->GetControlValue(CachedControlIndex).Get<bool>();
		}
	}
}

FRigUnit_GetControlFloat_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			FloatValue = Hierarchy->GetControlValue(CachedControlIndex).Get<float>();
			Minimum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Minimum).Get<float>();
			Maximum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Maximum).Get<float>();
		}
	}
}


FRigUnit_GetControlInteger_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			IntegerValue = Hierarchy->GetControlValue(CachedControlIndex).Get<int32>();
			Minimum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Minimum).Get<int32>();
			Maximum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Maximum).Get<int32>();
		}
	}
}

FRigUnit_GetControlVector2D_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			const FVector3f TempVector = Hierarchy->GetControlValue(CachedControlIndex).Get<FVector3f>();
			const FVector3f TempMinimum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Minimum).Get<FVector3f>();
			const FVector3f TempMaximum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Maximum).Get<FVector3f>();
			Vector = FVector2D(TempVector.X, TempVector.Y);
			Minimum = FVector2D(TempMinimum.X, TempMinimum.Y);
			Maximum = FVector2D(TempMaximum.X, TempMaximum.Y);
		}
	}
}

FRigUnit_GetControlVector_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			FTransform Transform = FTransform::Identity;
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Transform = Hierarchy->GetGlobalTransform(CachedControlIndex);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Transform = Hierarchy->GetLocalTransform(CachedControlIndex);
					break;
				}
				default:
				{
					break;
				}
			}

			const ERigControlType ControlType = Hierarchy->GetChecked<FRigControlElement>(CachedControlIndex)->Settings.ControlType;
			
			if(ControlType == ERigControlType::Position)
			{
				Vector = Transform.GetLocation();
			}
			else if(ControlType == ERigControlType::Scale)
			{
				Vector = Transform.GetScale3D();
			}

			Minimum = (FVector)Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Minimum).Get<FVector3f>();
			Maximum = (FVector)Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Maximum).Get<FVector3f>();
		}
	}
}

FRigUnit_GetControlRotator_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			FTransform Transform = FTransform::Identity;
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Transform = Hierarchy->GetGlobalTransform(CachedControlIndex);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Transform = Hierarchy->GetLocalTransform(CachedControlIndex);
					break;
				}
				default:
				{
					break;
				}
			}

			Rotator = Transform.GetRotation().Rotator();

			const FVector MinimumVector = (FVector)Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Minimum).Get<FVector3f>();
			const FVector MaximumVector = (FVector)Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Maximum).Get<FVector3f>();
			Minimum = FRotator::MakeFromEuler(MinimumVector); 
			Maximum = FRotator::MakeFromEuler(MaximumVector); 
		}
	}
}

FRigUnit_GetControlPreferredEulerAngles_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	
	Angles = FVector::ZeroVector;
	RotationOrder = FRigPreferredEulerAngles::DefaultRotationOrder;
	
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			RotationOrder = Hierarchy->GetControlPreferredEulerRotationOrderByIndex(CachedControlIndex, false);
			Angles = Hierarchy->GetControlPreferredEulerAnglesByIndex(CachedControlIndex, RotationOrder);
		}
	}
}

FRigUnit_GetControlTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
		}
		else
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Transform = Hierarchy->GetGlobalTransform(CachedControlIndex);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Transform = Hierarchy->GetLocalTransform(CachedControlIndex);
					break;
				}
				default:
				{
					break;
				}
			}
			
			Minimum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Minimum).Get<FRigControlValue::FTransform_Float>().ToTransform();
			Maximum = Hierarchy->GetControlValue(CachedControlIndex, ERigControlValueType::Maximum).Get<FRigControlValue::FTransform_Float>().ToTransform();
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_GetControlTransform)
{
	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Transform;

	const FRigElementKey Root = Controller->AddControl(
		TEXT("Root"),
		FRigElementKey(),
		Settings,
		FRigControlValue::Make(FTransform(FVector(1.f, 0.f, 0.f))),
		FTransform::Identity,
		FTransform::Identity);

	const FRigElementKey ControlA = Controller->AddControl(
	    TEXT("ControlA"),
	    Root,
	    Settings,
	    FRigControlValue::Make(FTransform(FVector(1.f, 2.f, 3.f))),
	    FTransform::Identity,
	    FTransform::Identity);

	Unit.Control = TEXT("Unknown");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 0.f, 0.f)), TEXT("unexpected global transform (0)"));
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 0.f, 0.f)), TEXT("unexpected local transform (0)"));

	Unit.Control = TEXT("Root");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected global transform (1)"));
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected local transform (1)"));

	Unit.Control = TEXT("ControlA");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(2.f, 2.f, 3.f)), TEXT("unexpected global transform (2)"));
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(1.f, 2.f, 3.f)), TEXT("unexpected local transform (2)"));

	return true;
}
#endif
