// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetCurveValue.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetCurveValue)

FRigUnit_GetCurveValue_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    if (const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if(CachedCurveIndex.UpdateCache(FRigElementKey(Curve, ERigElementType::Curve), Hierarchy))
		{
			Valid = Hierarchy->IsCurveValueSetByIndex(CachedCurveIndex);
			Value = Hierarchy->GetCurveValueByIndex(CachedCurveIndex);
			return;
		}
	}
	Valid = false;
	Value = 0.0f;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_GetCurveValue)
{
	static const FName ValidCurveName("ValidCurve");
	static const FName InvalidCurveName("InvalidCurve");
	static const FName UndefinedCurveName(NAME_None);
	
	const FRigElementKey ValidCurve = Controller->AddCurve(ValidCurveName, 1.f);
	const FRigElementKey InvalidCurve = Controller->AddCurve(InvalidCurveName, 1.f);

	Hierarchy->ResetCurveValues();
	Hierarchy->UnsetCurveValue(InvalidCurve);
	
	Unit.Curve = ValidCurveName;
	Execute();
	AddErrorIfFalse(!Unit.Valid, TEXT("Expected curve hold a unset value"));
	Execute();
	AddErrorIfFalse(Unit.Value != 1.0f, TEXT("Expected curve's value to be 1.0"));

	Hierarchy->SetCurveValue({ValidCurveName, ERigElementType::Curve}, 2.0f);
	Execute();
	AddErrorIfFalse(Unit.Valid, TEXT("Expected curve hold a valid value"));
	Execute();
	AddErrorIfFalse(Unit.Value == 2.0f, TEXT("Expected curve's value to be 2.0"));

	Unit.Curve = InvalidCurveName;
	Execute();
	AddErrorIfFalse(!Unit.Valid, TEXT("Expected curve hold an invalid value"));

	Unit.Curve = UndefinedCurveName;
	Execute();
	AddErrorIfFalse(!Unit.Valid, TEXT("Expected undefined curve hold an invalid value"));
	
	return true;
}
#endif
