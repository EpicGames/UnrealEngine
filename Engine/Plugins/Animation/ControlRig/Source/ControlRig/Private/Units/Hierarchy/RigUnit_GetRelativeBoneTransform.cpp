// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetRelativeBoneTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_GetRelativeTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetRelativeBoneTransform)

FRigUnit_GetRelativeBoneTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedBone.UpdateCache(FRigElementKey(Bone, ERigElementType::Bone), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else if (!CachedSpace.UpdateCache(FRigElementKey(Space, ERigElementType::Bone), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Space '%s' is not valid."), *Space.ToString());
		}
		else
		{
			const FTransform SpaceTransform = Hierarchy->GetGlobalTransform(CachedSpace);
			const FTransform BoneTransform = Hierarchy->GetGlobalTransform(CachedBone);
			Transform = BoneTransform.GetRelativeTransform(SpaceTransform);
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_GetRelativeBoneTransform::GetUpgradeInfo() const
{
	FRigUnit_GetRelativeTransformForItem NewNode;
	NewNode.Parent = FRigElementKey(Space, ERigElementType::Bone);
	NewNode.Child = FRigElementKey(Bone, ERigElementType::Bone);

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Space"), TEXT("Parent.Name"));
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Child.Name"));
	Info.AddRemappedPin(TEXT("Transform"), TEXT("RelativeTransform"));
	return Info;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_GetRelativeBoneTransform)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(1.f, 2.f, 3.f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), Root, FTransform(FVector(-4.f, 0.f, 0.f)), true, ERigBoneType::User);

	Unit.Bone = TEXT("Unknown");
	Unit.Space = TEXT("Root");
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 0.f, 0.f)), TEXT("unexpected transform"));

	Unit.Bone = TEXT("BoneA");
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(0.f, 2.f, 3.f)), TEXT("unexpected transform"));

	Unit.Space = TEXT("BoneB");
	Execute();
	AddErrorIfFalse(Unit.Transform.GetTranslation().Equals(FVector(5.f, 2.f, 3.f)), TEXT("unexpected transform"));

	return true;
}
#endif
