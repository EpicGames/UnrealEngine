// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetRelativeTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetRelativeTransform)

FRigUnit_GetRelativeTransformForItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	FTransform ChildTransform = FTransform::Identity;
	FTransform ParentTransform = FTransform::Identity;

	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Child, ERigVMTransformSpace::GlobalSpace, bChildInitial, ChildTransform, CachedChild);
	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Parent, ERigVMTransformSpace::GlobalSpace, bParentInitial, ParentTransform, CachedParent);
	FRigVMFunction_MathTransformMakeRelative::StaticExecute(ExecuteContext, ChildTransform, ParentTransform, RelativeTransform);
}
