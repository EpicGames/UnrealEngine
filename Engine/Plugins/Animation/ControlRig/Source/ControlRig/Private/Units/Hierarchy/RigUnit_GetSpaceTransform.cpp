// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetSpaceTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetSpaceTransform)

FRigUnit_GetSpaceTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedSpaceIndex.UpdateCache(FRigElementKey(Space, ERigElementType::Null), Hierarchy))
		{
			switch (SpaceType)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Transform = Hierarchy->GetGlobalTransform(CachedSpaceIndex);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Transform = Hierarchy->GetLocalTransform(CachedSpaceIndex);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_GetSpaceTransform::GetUpgradeInfo() const
{
	FRigUnit_GetTransform NewNode;
	NewNode.Item = FRigElementKey(Space, ERigElementType::Null);
	NewNode.Space = SpaceType;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Space"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("SpaceType"), TEXT("Space"));
	return Info;
}

