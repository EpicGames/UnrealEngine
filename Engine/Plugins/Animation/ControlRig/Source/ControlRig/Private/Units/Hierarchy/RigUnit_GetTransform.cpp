// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetTransform)

FString FRigUnit_GetTransform::GetUnitLabel() const
{
	FString Initial = bInitial ? TEXT(" Initial") : FString();
	FString Type = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Item.Type).ToString();
	return FString::Printf(TEXT("Get Transform - %s%s"), *Type, *Initial);
}

FRigUnit_GetTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if (!CachedIndex.UpdateCache(Item, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item '%s' is not valid."), *Item.ToString());
		}
		else
		{
			if(bInitial)
			{
				switch (Space)
				{
					case ERigVMTransformSpace::GlobalSpace:
					{
						Transform = Hierarchy->GetInitialGlobalTransform(CachedIndex);
						break;
					}
					case ERigVMTransformSpace::LocalSpace:
					{
						Transform = Hierarchy->GetInitialLocalTransform(CachedIndex);
						break;
					}
					default:
					{
						break;
					}
				}
			}
			else
			{
				switch (Space)
				{
					case ERigVMTransformSpace::GlobalSpace:
					{
						Transform = Hierarchy->GetGlobalTransform(CachedIndex);
						break;
					}
					case ERigVMTransformSpace::LocalSpace:
					{
						Transform = Hierarchy->GetLocalTransform(CachedIndex);
						break;
					}
					default:
					{
						break;
					}
				}
			}
		}
	}
}

FRigUnit_GetTransformArray_Execute()
{
	FRigUnit_GetTransformItemArray::StaticExecute(ExecuteContext, Items.Keys, Space, bInitial, Transforms, CachedIndex);
}

FRigVMStructUpgradeInfo FRigUnit_GetTransformArray::GetUpgradeInfo() const
{
	FRigUnit_GetTransformItemArray NewNode;
	NewNode.Items = Items.GetKeys();
	NewNode.Space = Space;
	NewNode.bInitial = bInitial;
	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_GetTransformItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(CachedIndex.Num() != Items.Num())
	{
		CachedIndex.Reset();
		CachedIndex.SetNum(Items.Num());
	}

	Transforms.SetNum(Items.Num());
	for(int32 Index=0;Index<Items.Num();Index++)
	{
		FRigUnit_GetTransform::StaticExecute(ExecuteContext, Items[Index], Space, bInitial, Transforms[Index], CachedIndex[Index]);

	}
}

