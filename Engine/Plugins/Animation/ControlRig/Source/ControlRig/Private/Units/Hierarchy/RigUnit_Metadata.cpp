// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_Metadata.h"

#include "ModularRig.h"
#include "RigVMTypeUtils.h"
#include "RigVMCore/RigVMStruct.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_Metadata)

#if WITH_EDITOR
#include "RigVMModel/RigVMPin.h"
#endif

#if WITH_EDITOR

FString FRigDispatch_MetadataBase::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	if(const TRigVMTypeIndex* ValueTypeIndexPtr = InTypes.Find(ValueArgName))
	{
		const TRigVMTypeIndex& ValueTypeIndex = *ValueTypeIndexPtr;
		if(ValueTypeIndex != RigVMTypeUtils::TypeIndex::WildCard &&
			ValueTypeIndex != RigVMTypeUtils::TypeIndex::WildCardArray)
		{
			const FRigVMTemplateArgumentType& ValueType = FRigVMRegistry::Get().GetType(ValueTypeIndex);
			FString ValueName;
			if(ValueType.CPPTypeObject == FRigElementKey::StaticStruct())
			{
				ValueName = ItemArgName.ToString();
			}
			else if(ValueType.CPPTypeObject)
			{
				ValueName = ValueType.CPPTypeObject->GetName();
			}
			else if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FName)
			{
				ValueName = NameArgName.ToString();
			}
			else
			{
				ValueName = ValueType.GetBaseCPPType();
				ValueName = ValueName.Left(1).ToUpper() + ValueName.Mid(1);
			}

			if(ValueType.IsArray())
			{
				ValueName += TEXT(" Array");
			}

			return FString::Printf(TEXT("%s %s%s Metadata"), IsSetMetadata() ? TEXT("Set") : TEXT("Get"), *GetNodeTitlePrefix(), *ValueName);
		}
	}
	return FRigDispatchFactory::GetNodeTitle(InTypes);
}

#endif

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_MetadataBase::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		ItemArgIndex = CachedArgumentInfos.Emplace(ItemArgName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<FRigElementKey>());
		NameArgIndex = CachedArgumentInfos.Emplace(NameArgName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);
		NameSpaceArgIndex = CachedArgumentInfos.Emplace(NameSpaceArgName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<ERigMetaDataNameSpace>());
		CacheArgIndex = CachedArgumentInfos.Emplace(CacheArgName, ERigVMPinDirection::Hidden, InRegistry->GetTypeIndex_NoLock<FCachedRigElement>());
	};
	return CachedArgumentInfos;
}

#if WITH_EDITOR

FText FRigDispatch_MetadataBase::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ItemArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "ItemArgTooltip", "The item storing the metadata");
	}
	if(InArgumentName == NameArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "NameArgTooltip", "The name of the metadata");
	}
	if(InArgumentName == NameSpaceArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "NameSpaceArgTooltip", "Defines in which namespace the metadata will be looked up");
	}
	if(InArgumentName == DefaultArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "DefaultArgTooltip", "The default value used as a fallback if the metadata does not exist");
	}
	if(InArgumentName == ValueArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "ValueArgTooltip", "The value to get / set");
	}
	if(InArgumentName == FoundArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "FoundArgTooltip", "Returns true if the metadata exists with the specific type");
	}
	if(InArgumentName == SuccessArgName)
	{
		return NSLOCTEXT("FRigDispatch_MetadataBase", "SuccessArgTooltip", "Returns true if the metadata was successfully stored");
	}
	return FRigDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

FString FRigDispatch_MetadataBase::GetArgumentDefaultValue(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == NameSpaceArgName)
	{
		static const FString SelfString = StaticEnum<ERigMetaDataNameSpace>()->GetDisplayNameTextByValue((int64)ERigMetaDataNameSpace::Self).ToString();
		return SelfString;
	}
	return FRigDispatchFactory::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

FString FRigDispatch_MetadataBase::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == NameArgName)
	{
		if(InMetaDataKey == FRigVMStruct::CustomWidgetMetaName)
		{
			return TEXT("MetadataName");
		}
	}
	return Super::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

#endif

const TArray<TRigVMTypeIndex>& FRigDispatch_MetadataBase::GetValueTypes(const FRigVMRegistryHandle& InRegistry) const
{
	if(ValueTypes.IsEmpty())
	{
		ValueTypes = {
			RigVMTypeUtils::TypeIndex::Bool,
			RigVMTypeUtils::TypeIndex::Float,
			RigVMTypeUtils::TypeIndex::Int32,
			RigVMTypeUtils::TypeIndex::FName,
			InRegistry->GetTypeIndex_NoLock<FVector>(false),
			InRegistry->GetTypeIndex_NoLock<FRotator>(false),
			InRegistry->GetTypeIndex_NoLock<FQuat>(false),
			InRegistry->GetTypeIndex_NoLock<FTransform>(false),
			InRegistry->GetTypeIndex_NoLock<FLinearColor>(false),
			InRegistry->GetTypeIndex_NoLock<FRigElementKey>(false),
			RigVMTypeUtils::TypeIndex::BoolArray,
			RigVMTypeUtils::TypeIndex::FloatArray,
			RigVMTypeUtils::TypeIndex::Int32Array,
			RigVMTypeUtils::TypeIndex::FNameArray,
			InRegistry->GetTypeIndex_NoLock<FVector>(true),
			InRegistry->GetTypeIndex_NoLock<FRotator>(true),
			InRegistry->GetTypeIndex_NoLock<FQuat>(true),
			InRegistry->GetTypeIndex_NoLock<FTransform>(true),
			InRegistry->GetTypeIndex_NoLock<FLinearColor>(true),
			InRegistry->GetTypeIndex_NoLock<FRigElementKey>(true)
		};

		// remove invalid indices
		ValueTypes.Remove(TRigVMTypeIndex());
	}
	return ValueTypes;
}

////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_GetMetadata::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(ValueArgIndex == INDEX_NONE)
	{
		CachedArgumentInfos = Super::GetArgumentInfos(InRegistry); 
		DefaultArgIndex = CachedArgumentInfos.Emplace(DefaultArgName, ERigVMPinDirection::Input, GetValueTypes(InRegistry));
		ValueArgIndex = CachedArgumentInfos.Emplace(ValueArgName, ERigVMPinDirection::Output, GetValueTypes(InRegistry));
		FoundArgIndex = CachedArgumentInfos.Emplace(FoundArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	};
	return CachedArgumentInfos;
}

FRigBaseMetadata* FRigDispatch_GetMetadata::FindMetadata(const FRigVMExtendedExecuteContext& InContext,
                                                         const FRigElementKey& InKey, const FName& InName,
                                                         ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace, FCachedRigElement& Cache)
{
	const FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
	if(Cache.UpdateCache(InKey, ExecuteContext.Hierarchy))
	{
		if(FRigBaseElement* Element = ExecuteContext.Hierarchy->Get(Cache.GetIndex()))
		{
			// first try to find the metadata in the namespace
			const FName Name = ExecuteContext.AdaptMetadataName(InNameSpace, InName);
			return ExecuteContext.Hierarchy->FindMetadataForElement(Element, Name, InType);
		};
	}
	return nullptr;
}

FRigVMFunctionPtr FRigDispatch_GetMetadata::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));
	
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Bool)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<bool, FRigBoolMetadata, ERigMetadataType::Bool>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<float, FRigFloatMetadata, ERigMetadataType::Float>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<int32, FRigInt32Metadata, ERigMetadataType::Int32>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FName, FRigNameMetadata, ERigMetadataType::Name>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(false))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FVector, FRigVectorMetadata, ERigMetadataType::Vector>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(false))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FRotator, FRigRotatorMetadata, ERigMetadataType::Rotator>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(false))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FQuat, FRigQuatMetadata, ERigMetadataType::Quat>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(false))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FTransform, FRigTransformMetadata, ERigMetadataType::Transform>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(false))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FLinearColor, FRigLinearColorMetadata, ERigMetadataType::LinearColor>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(false))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<FRigElementKey, FRigElementKeyMetadata, ERigMetadataType::RigElementKey>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::BoolArray)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<bool>, FRigBoolArrayMetadata, ERigMetadataType::BoolArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FloatArray)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<float>, FRigFloatArrayMetadata, ERigMetadataType::FloatArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32Array)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<int32>, FRigInt32ArrayMetadata, ERigMetadataType::Int32Array>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FNameArray)
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FName>, FRigNameArrayMetadata, ERigMetadataType::NameArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(true))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FVector>, FRigVectorArrayMetadata, ERigMetadataType::VectorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(true))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FRotator>, FRigRotatorArrayMetadata, ERigMetadataType::RotatorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(true))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FQuat>, FRigQuatArrayMetadata, ERigMetadataType::QuatArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(true))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FTransform>, FRigTransformArrayMetadata, ERigMetadataType::TransformArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(true))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FLinearColor>, FRigLinearColorArrayMetadata, ERigMetadataType::LinearColorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(true))
	{
		return &FRigDispatch_GetMetadata::GetMetadataDispatch<TArray<FRigElementKey>, FRigElementKeyArrayMetadata, ERigMetadataType::RigElementKeyArray>;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_SetMetadata::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(ValueArgIndex == INDEX_NONE)
	{
		CachedArgumentInfos = Super::GetArgumentInfos(InRegistry); 
		ValueArgIndex = CachedArgumentInfos.Emplace(ValueArgName, ERigVMPinDirection::Input, GetValueTypes(InRegistry));
		SuccessArgIndex = CachedArgumentInfos.Emplace(SuccessArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	};
	return CachedArgumentInfos;
}

const TArray<FRigVMExecuteArgument>& FRigDispatch_SetMetadata::GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const
{
	static const TArray<FRigVMExecuteArgument> ExecuteArguments = {
		{TEXT("ExecuteContext"), ERigVMPinDirection::IO}
	};
	return ExecuteArguments;
}

FRigBaseMetadata* FRigDispatch_SetMetadata::FindOrAddMetadata(const FControlRigExecuteContext& InContext,
                                                              const FRigElementKey& InKey, const FName& InName, ERigMetadataType InType,
                                                              ERigMetaDataNameSpace NameSpace, FCachedRigElement& Cache)
{
	if(Cache.UpdateCache(InKey, InContext.Hierarchy))
	{
		if(FRigBaseElement* Element = InContext.Hierarchy->Get(Cache.GetIndex()))
		{
			const FName Name = InContext.AdaptMetadataName(NameSpace, InName);
			constexpr bool bNotify = true;
			return InContext.Hierarchy->GetMetadataForElement(Element, Name, InType, bNotify);
		}
	}
	return nullptr;
}

FRigVMFunctionPtr FRigDispatch_SetMetadata::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));
	
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Bool)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<bool, FRigBoolMetadata, ERigMetadataType::Bool>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<float, FRigFloatMetadata, ERigMetadataType::Float>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<int32, FRigInt32Metadata, ERigMetadataType::Int32>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FName, FRigNameMetadata, ERigMetadataType::Name>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(false))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FVector, FRigVectorMetadata, ERigMetadataType::Vector>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(false))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FRotator, FRigRotatorMetadata, ERigMetadataType::Rotator>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(false))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FQuat, FRigQuatMetadata, ERigMetadataType::Quat>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(false))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FTransform, FRigTransformMetadata, ERigMetadataType::Transform>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(false))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FLinearColor, FRigLinearColorMetadata, ERigMetadataType::LinearColor>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(false))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<FRigElementKey, FRigElementKeyMetadata, ERigMetadataType::RigElementKey>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::BoolArray)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<bool>, FRigBoolArrayMetadata, ERigMetadataType::BoolArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FloatArray)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<float>, FRigFloatArrayMetadata, ERigMetadataType::FloatArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32Array)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<int32>, FRigInt32ArrayMetadata, ERigMetadataType::Int32Array>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FNameArray)
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FName>, FRigNameArrayMetadata, ERigMetadataType::NameArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(true))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FVector>, FRigVectorArrayMetadata, ERigMetadataType::VectorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(true))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FRotator>, FRigRotatorArrayMetadata, ERigMetadataType::RotatorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(true))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FQuat>, FRigQuatArrayMetadata, ERigMetadataType::QuatArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(true))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FTransform>, FRigTransformArrayMetadata, ERigMetadataType::TransformArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(true))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FLinearColor>, FRigLinearColorArrayMetadata, ERigMetadataType::LinearColorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(true))
	{
		return &FRigDispatch_SetMetadata::SetMetadataDispatch<TArray<FRigElementKey>, FRigElementKeyArrayMetadata, ERigMetadataType::RigElementKeyArray>;
	}

	return nullptr;
}

FRigUnit_RemoveMetadata_Execute()
{
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Removed = false;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			const FName LocalName = ExecuteContext.AdaptMetadataName(NameSpace, Name);
			Removed = Element->RemoveMetadata(LocalName);
			if(!Removed)
			{
				Removed = Element->RemoveMetadata(Name);
			}
		}
	}
}

FRigUnit_RemoveAllMetadata_Execute()
{
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Removed = false;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(NameSpace != ERigMetaDataNameSpace::None)
			{
				// only remove the metadata within this module / with this namespace
				const TArray<FName> MetadataNames = Hierarchy->GetMetadataNames(Element->GetKey());

				Removed = false;
				const FString ModulePrefix = ExecuteContext.GetElementModulePrefix(NameSpace);
				for(const FName& MetadataName : MetadataNames)
				{
					if(MetadataName.ToString().StartsWith(ModulePrefix, ESearchCase::CaseSensitive))
					{
						if(Element->RemoveMetadata(MetadataName))
						{
							Removed = true;
						}
					}
				}
				return;
			}
			Removed = Element->RemoveAllMetadata();
		}
	}
}

FRigUnit_HasMetadata_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Found = false;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(const FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			const FName LocalName = ExecuteContext.AdaptMetadataName(NameSpace, Name);
			Found = Element->GetMetadata(LocalName, Type) != nullptr;
			if(!Found)
			{
				Found = Element->GetMetadata(Name, Type) != nullptr;
			}
		}
	}
}

FRigUnit_FindItemsWithMetadata_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Items.Reset();
	if (!Hierarchy)
	{
		return;
	}

	const FName LocalName = ExecuteContext.AdaptMetadataName(NameSpace, Name);
	Hierarchy->Traverse([&Items, Name, LocalName, Type](const FRigBaseElement* Element, bool& bContinue)
	{
		if(Element->GetMetadata(LocalName, Type) != nullptr)
		{
			Items.AddUnique(Element->GetKey());
		}
		bContinue = true;
	});
}

FRigUnit_GetMetadataTags_Execute()
{
	Tags.Reset();
	
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(const FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
			{
				Tags = Md->GetValue();
			}
		}
	}
}

FRigUnit_SetMetadataTag_Execute()
{
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->SetupValidMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
			{
				const int32 LastIndex = Md->GetValue().Num(); 
				const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
				if(Md->GetValue().AddUnique(LocalTag) == LastIndex)
				{
					Element->NotifyMetadataTagChanged(LocalTag, true);
				}
			}
		}
	}
}

FRigUnit_SetMetadataTagArray_Execute()
{
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->SetupValidMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
			{
				for(const FName& Tag : Tags)
				{
					const int32 LastIndex = Md->GetValue().Num(); 
					const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
					if(Md->GetValue().AddUnique(LocalTag) == LastIndex)
					{
						Element->NotifyMetadataTagChanged(LocalTag, true);
					}
				}
			}
		}
	}
}

FRigUnit_RemoveMetadataTag_Execute()
{
	Removed = false;
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
			{
				const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
				Removed = Md->GetValue().Remove(LocalTag) > 0;
				if(Removed)
				{
					Element->NotifyMetadataTagChanged(LocalTag, false);
				}
			}
		}
	}
}

FRigUnit_HasMetadataTag_Execute()
{
	Found = false;
	
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(const FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
			{
				const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
				Found = Md->GetValue().Contains(LocalTag);
			}
		}
	}
}

FRigUnit_HasMetadataTagArray_Execute()
{
	Found = false;
	
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	if(CachedIndex.UpdateCache(Item, Hierarchy))
	{
		if(const FRigBaseElement* Element = Hierarchy->Get(CachedIndex))
		{
			if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
			{
				Found = true;
				for(const FName& Tag : Tags)
				{
					const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
					if(!Md->GetValue().Contains(LocalTag))
					{
						Found = false;
						break;
					}
				}
			}
		}
	}
}

FRigUnit_FindItemsWithMetadataTag_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Items.Reset();
	if (!Hierarchy)
	{
		return;
	}

	const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
	Hierarchy->Traverse([&Items, LocalTag](const FRigBaseElement* Element, bool& bContinue)
	{
		if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
		{
			if(Md->GetValue().Contains(LocalTag))
			{
				Items.AddUnique(Element->GetKey());
			}
		}
		bContinue = true;
	});
}

FRigUnit_FindItemsWithMetadataTagArray_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Items.Reset();
	if (!Hierarchy)
	{
		return;
	}

	TArrayView<const FName> LocalTags(Tags);
	TArray<FName> AdaptedTags;

	const bool bUseNameSpace = NameSpace != ERigMetaDataNameSpace::None;
	if(bUseNameSpace && ExecuteContext.IsRigModule())
	{
		AdaptedTags.Reserve(Tags.Num());
		for(const FName& Tag : Tags)
		{
			const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
			AdaptedTags.Add(LocalTag);
		}
		LocalTags = TArrayView<const FName>(AdaptedTags);
	}
	
	Hierarchy->Traverse([&Items, LocalTags](const FRigBaseElement* Element, bool& bContinue)
	{
		if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
		{
			bool bFoundAll = true;
			for(const FName& Tag : LocalTags)
			{
				if(!Md->GetValue().Contains(Tag))
				{
					bFoundAll = false;
					break;
				}
			}

			if(bFoundAll)
			{
				Items.AddUnique(Element->GetKey());
			}
		}
		bContinue = true;
	});
}

FRigUnit_FilterItemsByMetadataTags_Execute()
{
	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;

	Result.Reset();
	if (!Hierarchy)
	{
		return;
	}
	
	if(CachedIndices.Num() != Items.Num())
	{
		CachedIndices.Reset();
		CachedIndices.SetNumZeroed(Items.Num());
	}

	TArrayView<const FName> LocalTags(Tags);
	TArray<FName> AdaptedTags;
	const bool bUseNameSpace = NameSpace != ERigMetaDataNameSpace::None;
	if(bUseNameSpace && ExecuteContext.IsRigModule())
	{
		AdaptedTags.Reserve(Tags.Num());
		for(const FName& Tag : Tags)
		{
			const FName LocalTag = ExecuteContext.AdaptMetadataName(NameSpace, Tag);
			AdaptedTags.Add(LocalTag);
		}
		LocalTags = TArrayView<const FName>(AdaptedTags);
	}

	for(int32 Index = 0; Index < Items.Num(); Index++)
	{
		if(CachedIndices[Index].UpdateCache(Items[Index], Hierarchy))
		{
			if(const FRigBaseElement* Element = Hierarchy->Get(CachedIndices[Index]))
			{
				if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
				{
					if(Inclusive)
					{
						bool bFoundAll = true;
						for(const FName& Tag : LocalTags)
						{
							if(!Md->GetValue().Contains(Tag))
							{
								bFoundAll = false;
								break;
							}
						}
						if(bFoundAll)
						{
							Result.Add(Element->GetKey());
						}
					}
					else
					{
						bool bFoundAny = false;
						for(const FName& Tag : LocalTags)
						{
							if(Md->GetValue().Contains(Tag))
							{
								bFoundAny = true;
								break;
							}
						}
						if(!bFoundAny)
						{
							Result.Add(Element->GetKey());
						}
					}
				}
				else if(!Inclusive)
				{
					Result.Add(Element->GetKey());
				}
			}
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Item '%s' not found"), *Items[Index].ToString());
		}
	}
}

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_GetModuleMetadata::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(ValueArgIndex == INDEX_NONE)
	{
		NameArgIndex = CachedArgumentInfos.Emplace(NameArgName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);
		NameSpaceArgIndex = CachedArgumentInfos.Emplace(NameSpaceArgName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<ERigMetaDataNameSpace>());
		DefaultArgIndex = CachedArgumentInfos.Emplace(DefaultArgName, ERigVMPinDirection::Input, GetValueTypes(InRegistry));
		ValueArgIndex = CachedArgumentInfos.Emplace(ValueArgName, ERigVMPinDirection::Output, GetValueTypes(InRegistry));
		FoundArgIndex = CachedArgumentInfos.Emplace(FoundArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	};
	return CachedArgumentInfos;
}

FRigBaseMetadata* FRigDispatch_GetModuleMetadata::FindMetadata(const FRigVMExtendedExecuteContext& InContext, const FName& InName, ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace)
{
	const FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
	if(const FRigModuleInstance* ModuleInstance = ExecuteContext.GetRigModuleInstance(InNameSpace))
	{
		if(const FRigConnectorElement* PrimaryConnector = ModuleInstance->FindPrimaryConnector())
		{
			// first try to find the metadata in the namespace
			return ExecuteContext.Hierarchy->FindMetadataForElement(PrimaryConnector, InName, InType);
		}
	}
	else if(ExecuteContext.IsRigModule())
	{
		// we are not in a rig module - but we still want to store the metadata for testing.
		const TArray<FRigConnectorElement*> Connectors = ExecuteContext.Hierarchy->GetConnectors();
		for(const FRigConnectorElement* Connector : Connectors)
		{
			if(Connector->IsPrimary())
			{
				const FName Name = ExecuteContext.AdaptMetadataName(InNameSpace, InName);
				return ExecuteContext.Hierarchy->FindMetadataForElement(Connector, Name, InType);
			}
		}
	}
	return nullptr;
}

FRigVMFunctionPtr FRigDispatch_GetModuleMetadata::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));
	
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Bool)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<bool, FRigBoolMetadata, ERigMetadataType::Bool>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<float, FRigFloatMetadata, ERigMetadataType::Float>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<int32, FRigInt32Metadata, ERigMetadataType::Int32>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FName, FRigNameMetadata, ERigMetadataType::Name>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(false))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FVector, FRigVectorMetadata, ERigMetadataType::Vector>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(false))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FRotator, FRigRotatorMetadata, ERigMetadataType::Rotator>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(false))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FQuat, FRigQuatMetadata, ERigMetadataType::Quat>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(false))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FTransform, FRigTransformMetadata, ERigMetadataType::Transform>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(false))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FLinearColor, FRigLinearColorMetadata, ERigMetadataType::LinearColor>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(false))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<FRigElementKey, FRigElementKeyMetadata, ERigMetadataType::RigElementKey>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::BoolArray)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<bool>, FRigBoolArrayMetadata, ERigMetadataType::BoolArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FloatArray)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<float>, FRigFloatArrayMetadata, ERigMetadataType::FloatArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32Array)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<int32>, FRigInt32ArrayMetadata, ERigMetadataType::Int32Array>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FNameArray)
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FName>, FRigNameArrayMetadata, ERigMetadataType::NameArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(true))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FVector>, FRigVectorArrayMetadata, ERigMetadataType::VectorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(true))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FRotator>, FRigRotatorArrayMetadata, ERigMetadataType::RotatorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(true))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FQuat>, FRigQuatArrayMetadata, ERigMetadataType::QuatArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(true))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FTransform>, FRigTransformArrayMetadata, ERigMetadataType::TransformArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(true))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FLinearColor>, FRigLinearColorArrayMetadata, ERigMetadataType::LinearColorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(true))
	{
		return &FRigDispatch_GetModuleMetadata::GetModuleMetadataDispatch<TArray<FRigElementKey>, FRigElementKeyArrayMetadata, ERigMetadataType::RigElementKeyArray>;
	}

	return nullptr;
}

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_SetModuleMetadata::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(ValueArgIndex == INDEX_NONE)
	{
		NameArgIndex = CachedArgumentInfos.Emplace(NameArgName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);
		NameSpaceArgIndex = CachedArgumentInfos.Emplace(NameSpaceArgName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<ERigMetaDataNameSpace>());
		ValueArgIndex = CachedArgumentInfos.Emplace(ValueArgName, ERigVMPinDirection::Input, GetValueTypes(InRegistry));
		SuccessArgIndex = CachedArgumentInfos.Emplace(SuccessArgName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	};
	return CachedArgumentInfos;
}

FRigBaseMetadata* FRigDispatch_SetModuleMetadata::FindOrAddMetadata(const FControlRigExecuteContext& InContext, const FName& InName, ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace)
{
	constexpr bool bNotify = true;
	
	if(const FRigModuleInstance* ModuleInstance = InContext.GetRigModuleInstance(InNameSpace))
	{
		if(const FRigConnectorElement* PrimaryConnector = ModuleInstance->FindPrimaryConnector())
		{
			return InContext.Hierarchy->GetMetadataForElement(const_cast<FRigConnectorElement*>(PrimaryConnector), InName, InType, bNotify);
		}
	}
	else if(InContext.IsRigModule())
	{
		// we are not in a rig module - but we still want to store the metadata for testing.
		const TArray<FRigConnectorElement*> Connectors = InContext.Hierarchy->GetConnectors();
		for(const FRigConnectorElement* Connector : Connectors)
		{
			if(Connector->IsPrimary())
			{
				const FName Name = InContext.AdaptMetadataName(InNameSpace, InName);
				return InContext.Hierarchy->GetMetadataForElement(const_cast<FRigConnectorElement*>(Connector), Name, InType, bNotify);
			}
		}
	}
	return nullptr;
}

FRigVMFunctionPtr FRigDispatch_SetModuleMetadata::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));
	
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Bool)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<bool, FRigBoolMetadata, ERigMetadataType::Bool>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<float, FRigFloatMetadata, ERigMetadataType::Float>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<int32, FRigInt32Metadata, ERigMetadataType::Int32>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FName, FRigNameMetadata, ERigMetadataType::Name>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(false))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FVector, FRigVectorMetadata, ERigMetadataType::Vector>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(false))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FRotator, FRigRotatorMetadata, ERigMetadataType::Rotator>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(false))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FQuat, FRigQuatMetadata, ERigMetadataType::Quat>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(false))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FTransform, FRigTransformMetadata, ERigMetadataType::Transform>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(false))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FLinearColor, FRigLinearColorMetadata, ERigMetadataType::LinearColor>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(false))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<FRigElementKey, FRigElementKeyMetadata, ERigMetadataType::RigElementKey>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::BoolArray)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<bool>, FRigBoolArrayMetadata, ERigMetadataType::BoolArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FloatArray)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<float>, FRigFloatArrayMetadata, ERigMetadataType::FloatArray>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::Int32Array)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<int32>, FRigInt32ArrayMetadata, ERigMetadataType::Int32Array>;
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FNameArray)
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FName>, FRigNameArrayMetadata, ERigMetadataType::NameArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>(true))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FVector>, FRigVectorArrayMetadata, ERigMetadataType::VectorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>(true))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FRotator>, FRigRotatorArrayMetadata, ERigMetadataType::RotatorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>(true))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FQuat>, FRigQuatArrayMetadata, ERigMetadataType::QuatArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>(true))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FTransform>, FRigTransformArrayMetadata, ERigMetadataType::TransformArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>(true))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FLinearColor>, FRigLinearColorArrayMetadata, ERigMetadataType::LinearColorArray>;
	}
	if(ValueTypeIndex == InRegistry->GetTypeIndex_NoLock<FRigElementKey>(true))
	{
		return &FRigDispatch_SetModuleMetadata::SetModuleMetadataDispatch<TArray<FRigElementKey>, FRigElementKeyArrayMetadata, ERigMetadataType::RigElementKeyArray>;
	}

	return nullptr;
}
