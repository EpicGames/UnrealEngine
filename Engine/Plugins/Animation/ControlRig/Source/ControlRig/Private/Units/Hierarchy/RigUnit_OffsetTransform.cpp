// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_OffsetTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_OffsetTransform)

FRigUnit_OffsetTransformForItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Weight < SMALL_NUMBER)
	{
		return;
	}

	FTransform PreviousTransform = FTransform::Identity;
	FTransform GlobalTransform = FTransform::Identity;

	if(!CachedIndex.UpdateCache(Item, ExecuteContext.Hierarchy))
	{
		return;
	}

	const FRigTransformElement* TransformElement = Cast<FRigTransformElement>(CachedIndex.GetElement());
	if(TransformElement == nullptr)
	{
		return;
	}

	// figure out which transform type has already been computed (is clean) to avoid compute
	ERigVMTransformSpace TransformTypeToUse = ERigVMTransformSpace::GlobalSpace;
	if(TransformElement->GetDirtyState().IsDirty(ERigTransformType::CurrentGlobal))
	{
		check(!TransformElement->GetDirtyState().IsDirty(ERigTransformType::CurrentLocal));
		TransformTypeToUse = ERigVMTransformSpace::LocalSpace;
	}

	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Item, TransformTypeToUse, false, PreviousTransform, CachedIndex);
	FRigVMFunction_MathTransformMakeAbsolute::StaticExecute(ExecuteContext, OffsetTransform, PreviousTransform, GlobalTransform);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Item, TransformTypeToUse, false, GlobalTransform, Weight, bPropagateToChildren, CachedIndex);
}

#if WITH_EDITOR

bool FRigUnit_OffsetTransformForItem::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_OffsetTransformForItem, OffsetTransform))
	{
		check(InInfo.IsValid());

		if(!InInfo->bInitialized)
		{
			InInfo->Reset();
			const FTransform ParentTransform = Hierarchy->GetParentTransform(Item, false);
			InInfo->OffsetTransform = OffsetTransform.Inverse() * Hierarchy->GetLocalTransform(Item) * ParentTransform;
		}

		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, InInfo->OffsetTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, OffsetTransform, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, OffsetTransform, true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_OffsetTransformForItem::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_OffsetTransformForItem, OffsetTransform))
	{
		OffsetTransform = Hierarchy->GetLocalTransform(InInfo->ControlKey, false);
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif