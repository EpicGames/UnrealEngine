// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_ParentSwitchConstraint.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ParentSwitchConstraint)

FRigUnit_ParentSwitchConstraint_Execute()
{
	FRigUnit_ParentSwitchConstraintArray::StaticExecute(ExecuteContext, Subject, ParentIndex, Parents.Keys, InitialGlobalTransform, Weight, Transform, Switched, CachedSubject, CachedParent, RelativeOffset);	
}

FRigVMStructUpgradeInfo FRigUnit_ParentSwitchConstraint::GetUpgradeInfo() const
{
	FRigUnit_ParentSwitchConstraintArray NewNode;
	NewNode.Subject = Subject;
	NewNode.ParentIndex = ParentIndex;
	NewNode.Parents = Parents.Keys;
	NewNode.InitialGlobalTransform = InitialGlobalTransform;
	NewNode.Weight = Weight;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_ParentSwitchConstraintArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Switched = false;
	Transform = FTransform::Identity;

	if (URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if (!CachedSubject.UpdateCache(Subject, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Subject '%s' is not valid."), *Subject.ToString());
			return;
		}
		if (ParentIndex < 0 || ParentIndex >= Parents.Num())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent Index is out of bounds."));
			return;
		}

		if (!CachedParent.IsValid())
		{
			if (!CachedParent.UpdateCache(Parents[ParentIndex], Hierarchy))
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent '%s' is not valid."), *Parents[ParentIndex].ToString());
				return;
			}

			FTransform ParentTransform = Hierarchy->GetGlobalTransform(CachedParent);
			RelativeOffset = InitialGlobalTransform.GetRelativeTransform(ParentTransform);
		}

		FTransform ParentTransform = Hierarchy->GetGlobalTransform(CachedParent);
		Transform = RelativeOffset * ParentTransform;

		if (Weight > SMALL_NUMBER)
		{
			if (Weight < 1.0f - SMALL_NUMBER)
			{
				FTransform WeightedTransform = Hierarchy->GetGlobalTransform(CachedSubject);
				WeightedTransform = FControlRigMathLibrary::LerpTransform(WeightedTransform, Transform, Weight);
				Hierarchy->SetGlobalTransform(CachedSubject, WeightedTransform);
			}
			else
			{
				Hierarchy->SetGlobalTransform(CachedSubject, Transform);
			}
		}

		if (CachedParent != Parents[ParentIndex])
		{
			if (!CachedParent.UpdateCache(Parents[ParentIndex], Hierarchy))
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Parent '%s' is not valid."), *Parents[ParentIndex].ToString());
				return;
			}

			ParentTransform = Hierarchy->GetGlobalTransform(CachedParent);
			RelativeOffset = Transform.GetRelativeTransform(ParentTransform);
			Switched = true;
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_ParentSwitchConstraintArray::GetUpgradeInfo() const
{
	// this node is no longer supported.
	return FRigVMStructUpgradeInfo();
}

