// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_PoseAsset.h"
#include "Units/RigUnitContext.h"
#include "Rigs/RigHierarchyController.h"
#include "AnimationRuntime.h"
#include "Engine/SkeletalMesh.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_PoseAsset)

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAsset_ExtractionContext
//////////////////////////////////////////////////////////////////////////////////////////////////////

class FRigUnitPoseAssetRetargetBoneTransformContext : public FAnimationRuntimeRetargetBoneTransformContext
{
public:

	FRigUnitPoseAssetRetargetBoneTransformContext(const USkeleton* InSkeleton, const URigHierarchy* InHierarchy, const TArrayView<const FRigElementKey>& InBoneKeys)
		: TargetSkeleton(InSkeleton)
		, Hierarchy(InHierarchy)
		, BoneKeys(InBoneKeys)
	{
		ReferenceTransforms.SetNumUninitialized(BoneKeys.Num());
	}

	virtual const USkeleton* GetSkeleton(bool bEvenIfUnreachable = false) const override
	{
		return TargetSkeleton;
	}
	
	virtual int32 GetSkeletonBoneIndex(const int32& InContextBoneIndex) const override
	{
		check(TargetSkeleton);
		return TargetSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneKeys[InContextBoneIndex].Name);
	}
	
	virtual int32 GetNumBonesInContext() const override
	{
		return BoneKeys.Num();
	}
	
	virtual bool IsRetargetingDisabled() const override
	{
		return false;
	}
	
	virtual const FTransform& GetRefPoseTransform(const int32& InContextBoneIndex) const override
	{
		check(Hierarchy);
		check(BoneKeys.IsValidIndex(InContextBoneIndex));
		ReferenceTransforms[InContextBoneIndex] = Hierarchy->GetLocalTransform(BoneKeys[InContextBoneIndex]);
		return ReferenceTransforms[InContextBoneIndex];
	}
	
	virtual const FRetargetSourceCachedData& GetRetargetSourceCachedData(const FName& InSourceName, const FSkeletonRemapping& InRemapping, const TArray<FTransform>& InRetargetTransforms) const override
	{
		return RetargetSourceCachedData;
	}

protected:

	const USkeleton* TargetSkeleton = nullptr;
	const URigHierarchy* Hierarchy = nullptr;
	TArrayView<const FRigElementKey> BoneKeys;
	FRetargetSourceCachedData RetargetSourceCachedData;
	mutable TArray<FTransform> ReferenceTransforms;

	friend class FRigUnitPoseAssetExtractionContext;
};

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAsset_ExtractionContext
//////////////////////////////////////////////////////////////////////////////////////////////////////

class FRigUnitPoseAssetExtractionContext : public FPoseAssetExtractionContext
{
public:

	FRigUnitPoseAssetExtractionContext(
		const USkeleton* InSkeleton,
		const URigHierarchy* InHierarchy,
		const TArrayView<const FPoseCurve>& InPoses,
		const TArrayView<const FRigElementKey>& InBoneKeys,
		const TArrayView<FTransform>& OutputTransforms,
		const TArrayView<const FRigElementKey>& InCurveKeys,
		const TArrayView<float>& OutputCurveValues)
		: Poses(InPoses)
		, Transforms(OutputTransforms)
		, CurveKeys(InCurveKeys)
		, CurveValues(OutputCurveValues)
		, RetargetBoneTransformContext(InSkeleton, InHierarchy, InBoneKeys)
	{
		CurveFilter.SetFilterMode(UE::Anim::ECurveFilterMode::DisallowFiltered);
		for (const FPoseCurve& Pose : Poses)
		{
			CurveFilter.Add(Pose.Name);
		}
		for (int32 CurveIndex = 0; CurveIndex < CurveKeys.Num(); ++CurveIndex)
		{
			CurveNameToIndex.Add(CurveKeys[CurveIndex].Name, CurveIndex);
		}
		check(InSkeleton);
		SkeletonBoneToContextPlusOne.AddZeroed(InSkeleton->GetReferenceSkeleton().GetNum());
		for (int32 ContextIndex = 0; ContextIndex < InBoneKeys.Num(); ContextIndex++)
		{
			const int32 SkeletonIndex = InSkeleton->GetReferenceSkeleton().FindBoneIndex(InBoneKeys[ContextIndex].Name);
			if (SkeletonIndex == INDEX_NONE)
			{
				continue;
			}
			SkeletonBoneToContextPlusOne[SkeletonIndex] = ContextIndex + 1;
		}
	}
	
	virtual int32 NumPoses() const override
	{
		return Poses.Num();
	}
	
	virtual const FName& GetPoseName(const int32& InPoseIndex) const override
	{
		return Poses[InPoseIndex].Name;
	}
	
	virtual int32 GetPoseIndexInPoseAsset(const int32& InPoseIndex) const override
	{
		return Poses[InPoseIndex].PoseIndex;
	}
	
	virtual float GetPoseWeight(const int32& InPoseIndex) const override
	{
		return Poses[InPoseIndex].Value;
	}

	virtual const USkeleton* GetTargetSkeleton(bool bEvenIfUnreachable) const override
	{
		return RetargetBoneTransformContext.GetSkeleton();
	}
	
	virtual bool IsOutputBoneRequired(const int32& InOutputBoneIndex) const override
	{
		return true;
	}
	
	virtual int32 GetOutputPoseBoneIndexFromSkeletonPoseBoneIndex(const int32& InSkeletonBoneIndex) const override
	{
		check(SkeletonBoneToContextPlusOne.IsValidIndex(InSkeletonBoneIndex));
		return SkeletonBoneToContextPlusOne[InSkeletonBoneIndex] - 1;
	}
	
	virtual const UE::Anim::FCurveFilter* GetCurveFilter() const override
	{
		return &CurveFilter;
	}
	
	virtual void ResetPoseToAdditiveIdentity() override
	{
		for (FTransform& Transform : Transforms)
		{
			static const FTransform AdditiveIdentity(FQuat::Identity, FVector::ZeroVector, FVector::ZeroVector);
			Transform = AdditiveIdentity;
		}
	}
	
	virtual void ResetPoseToRefPose() override
	{
		check(RetargetBoneTransformContext.BoneKeys.Num() == Transforms.Num());
		for (int32 Index = 0; Index < Transforms.Num(); Index++)
		{
			const FRigElementKey& BoneKey = RetargetBoneTransformContext.BoneKeys[Index]; 
			Transforms[Index] = RetargetBoneTransformContext.Hierarchy->GetLocalTransform(BoneKey, true);
		}
	}
	
	virtual FTransform& GetOutputTransform(const int32& InOutputBoneIndex) override
	{
		return Transforms[InOutputBoneIndex];
	}
		
	virtual const FAnimationRuntimeRetargetBoneTransformContext* GetRetargetBoneTransformContext() const override
	{
		return &RetargetBoneTransformContext;
	}

	virtual int32 NumCurves() const override
	{
		return CurveKeys.Num();
	}

	virtual void BlendPoseCurve(const FBlendedCurve& InCurve, float InWeight) override
	{
		InCurve.ForEachElement([this, InWeight](const UE::Anim::FCurveElement& InCurveElement)
		{
			if (const int32* CurveIndex = CurveNameToIndex.Find(InCurveElement.Name))
			{
				CurveValues[*CurveIndex] += InCurveElement.Value * InWeight;
			}
		});
	}
	
protected:
	
	TArrayView<const FPoseCurve> Poses;
	TArrayView<FTransform> Transforms;
	TArrayView<const FRigElementKey> CurveKeys;
	TArrayView<float> CurveValues;
	TMap<FName,int32> CurveNameToIndex;
	TArray<int32> SkeletonBoneToContextPlusOne;
	FRigUnitPoseAssetRetargetBoneTransformContext RetargetBoneTransformContext;
	UE::Anim::FCurveFilter CurveFilter;
};

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAsset_WorkData
//////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FRigUnit_PoseAsset_WorkData::ValidatePose(const FRigVMExecuteContext& ExecuteContext, const UPoseAsset* InPoseAsset, const FName& InPoseName, bool bReportWarning)
{
	if (!InPoseAsset || InPoseName.IsNone())
	{
		return INDEX_NONE;
	}

	if (InPoseAsset->GetPoseFNames().IsValidIndex(LastPoseIndex))
	{
		if (InPoseAsset->GetPoseFNames()[LastPoseIndex] == InPoseName)
		{
			return LastPoseIndex;
		}
	}
	
	const int32 PoseIndex = InPoseAsset->GetPoseFNames().Find(InPoseName);
	if (PoseIndex == INDEX_NONE && bReportWarning)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Pose %s does not exist."), *InPoseName.ToString())
	}
	LastPoseIndex = PoseIndex;
	return PoseIndex;
}

const USkeleton* FRigUnit_PoseAsset_WorkData::GetTargetSkeleton(const FRigVMExecuteContext& InExecuteContext)
{
	if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InExecuteContext.GetOwningComponent()))
	{
		if (const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshComponent->GetSkinnedAsset()))
		{
			return SkeletalMesh->GetSkeleton();
		}
	}
	return nullptr;
}

void FRigUnit_PoseAsset_WorkData::UpdateBoneHash(const URigHierarchy* InHierarchy, const UPoseAsset* InPoseAsset, const TArrayView<const FPoseCurve>& InPoses, const TArrayView<const FRigElementKey>& InItems)
{
	check(InHierarchy);
	check(InPoseAsset);

	uint32 PosesHash = 0;
	for (const FPoseCurve& Pose : InPoses)
	{
		if (!FAnimWeight::IsRelevant(FMath::Abs(Pose.Value)))
		{
			continue;
		}
		PosesHash = HashCombine(PosesHash, GetTypeHash(Pose.PoseIndex));
	}
	
	const uint32 Hash = HashCombine(
		GetTypeHash(LastPoseIndex),
		InHierarchy->GetTopologyVersion(),
		PosesHash,
		GetTypeHash(InPoseAsset),
		GetTypeHash(InItems)
	);
	
	if (Hash != LastBoneHash)
	{
		TrackIndexPerBone.Reset();
		BoneTransforms.Reset();
		LastBoneHash = Hash;
		
		if (bFilterByInfluences && CachedBoneItems.Num() != InHierarchy->Num(ERigElementType::Bone))
		{
			CachedBoneItems.Reset();
		}
	}
}

void FRigUnit_PoseAsset_WorkData::UpdateCurveHash(const URigHierarchy* InHierarchy, const UPoseAsset* InPoseAsset, const TArrayView<const FPoseCurve>& InPoses, const TArrayView<const FRigElementKey>& InItems)
{
	check(InHierarchy);
	check(InPoseAsset);

	uint32 PosesHash = 0;
	for (const FPoseCurve& Pose : InPoses)
	{
		if (!FAnimWeight::IsRelevant(FMath::Abs(Pose.Value)))
		{
			continue;
		}
		PosesHash = HashCombine(PosesHash, GetTypeHash(Pose.PoseIndex));
	}

	const uint32 Hash = HashCombine(
		GetTypeHash(LastPoseIndex),
		InHierarchy->GetTopologyVersion(),
		PosesHash,
		GetTypeHash(InPoseAsset),
		GetTypeHash(InItems)
	);
	
	if (Hash != LastCurveHash)
	{
		TrackIndexPerCurve.Reset();
		CurveValues.Reset();
		LastCurveHash = Hash;
	}
}

TArrayView<const FRigElementKey> FRigUnit_PoseAsset_WorkData::FilterBoneItems(const UPoseAsset* InPoseAsset, const URigHierarchy* InHierarchy, const TArrayView<const FRigElementKey>& InItems, const TArray<FPoseAssetInfluenceForPose>& InInfluences)
{
	TArrayView<const FRigElementKey> Keys = InItems;

	if (bFilterByInfluences && !InInfluences.IsEmpty())
	{
		if (CachedBoneItems.IsEmpty())
		{
			for (const FPoseAssetInfluenceForPose& Influence : InInfluences)
			{
				const FName& BoneName = InPoseAsset->GetTrackNames()[Influence.TrackIndex];
				const FRigElementKey BoneKey(BoneName, ERigElementType::Bone);
				if (!InItems.IsEmpty() && !InItems.Contains(BoneKey))
				{
					continue;
				}
				if (!InHierarchy->Contains(BoneKey))
				{
					continue;
				}
				CachedBoneItems.Add(BoneKey);
			}
		}
		Keys = TArrayView<const FRigElementKey>(CachedBoneItems.GetData(), CachedBoneItems.Num());
	}
	else if (InItems.IsEmpty())
	{
		if (CachedBoneItems.Num() != InHierarchy->Num(ERigElementType::Bone))
		{
			CachedBoneItems = InHierarchy->GetBoneKeys();
		}
		Keys = TArrayView<const FRigElementKey>(CachedBoneItems.GetData(), CachedBoneItems.Num());
	}
	else 
	{
		if (InItems.ContainsByPredicate([](const FRigElementKey& Key) -> bool
		{
			return Key.Type != ERigElementType::Bone;
		}))
		{
			CachedBoneItems.Reset();
			CachedBoneItems.Reserve(InItems.Num());
			for (const FRigElementKey& Key: InItems)
			{
				if (Key.Type != ERigElementType::Bone)
				{
					continue;
				}
				CachedBoneItems.Add(Key);
			}
			Keys = TArrayView<const FRigElementKey>(CachedBoneItems.GetData(), CachedBoneItems.Num());
		}
	}

	if (Keys.IsEmpty())
	{
		return Keys;
	}
	
	if (TrackIndexPerBone.IsEmpty())
	{
		TrackIndexPerBone.Reserve(Keys.Num());
		
		const TArray<FName>& BoneNames = InPoseAsset->GetTrackNames();
		for (const FRigElementKey& Key : Keys)
		{
			if (Key.Type != ERigElementType::Bone)
			{
				TrackIndexPerBone.Add(INDEX_NONE);
				continue;
			}
			TrackIndexPerBone.Add(BoneNames.Find(Key.Name)); 
		}
	}
	return Keys;
}

TArrayView<const FRigElementKey> FRigUnit_PoseAsset_WorkData::FilterCurveItems(const UPoseAsset* InPoseAsset, const URigHierarchy* InHierarchy, const TArrayView<const FRigElementKey>& InItems)
{
	TArrayView<const FRigElementKey> Keys = InItems;

	if (InItems.IsEmpty())
	{
		if (CachedCurveItems.Num() != InHierarchy->Num(ERigElementType::Curve))
		{
			CachedCurveItems = InHierarchy->GetCurveKeys();
		}
		Keys = TArrayView<const FRigElementKey>(CachedCurveItems.GetData(), CachedCurveItems.Num());
	}
	else 
	{
		if (InItems.ContainsByPredicate([](const FRigElementKey& Key) -> bool
		{
			return Key.Type != ERigElementType::Curve;
		}))
		{
			CachedCurveItems.Reset();
			CachedCurveItems.Reserve(InItems.Num());
			for (const FRigElementKey& Key: InItems)
			{
				if (Key.Type != ERigElementType::Curve)
				{
					continue;
				}
				CachedCurveItems.Add(Key);
			}
			Keys = TArrayView<const FRigElementKey>(CachedCurveItems.GetData(), CachedCurveItems.Num());
		}
	}

	if (Keys.IsEmpty())
	{
		return Keys;
	}

	if (TrackIndexPerCurve.IsEmpty())
	{
		TrackIndexPerCurve.Reserve(Keys.Num());
		
		const TArray<FName>& CurveNames = InPoseAsset->GetCurveFNames();
		for (const FRigElementKey& Key : Keys)
		{
			if (Key.Type != ERigElementType::Curve)
			{
				TrackIndexPerCurve.Add(INDEX_NONE);
				continue;
			}
			TrackIndexPerCurve.Add(CurveNames.Find(Key.Name)); 
		}
	}
	return Keys;
}

bool FRigUnit_PoseAsset_WorkData::GetAnimationPose(const FControlRigExecuteContext& ExecuteContext, const UPoseAsset* InPoseAsset,
	const URigHierarchy* InHierarchy, const TArrayView<const FPoseCurve>& InPoses, const TArrayView<const FRigElementKey>& InItems,
	TArrayView<const FRigElementKey>& OutBoneKeys, TArray<FTransform>& OutBoneTransforms,
	TArrayView<const FRigElementKey>& OutCurveKeys, TArray<float>& OutCurveValues)
{
	OutBoneTransforms.Reset();
	OutCurveValues.Reset();
	
	LastPoseIndex = INDEX_NONE;
	UpdateBoneHash(InHierarchy, InPoseAsset, InPoses, InItems);
	UpdateCurveHash(InHierarchy, InPoseAsset, InPoses, InItems);

	TArray<FPoseAssetInfluenceForPose> LocalSpaceInfluences;
	if (!InPoseAsset->GetTrackInfluencesForPoses(InPoses, LocalSpaceInfluences))
	{
		return false;
	}

	const USkeleton* SourceSkeleton = InPoseAsset->GetSkeleton();
	const USkeleton* TargetSkeleton = GetTargetSkeleton(ExecuteContext);
	if (!SourceSkeleton || !TargetSkeleton)
	{
		return false;
	}
	const FReferenceSkeleton& ReferenceSkeleton = SourceSkeleton->GetReferenceSkeleton();

	OutBoneKeys = FilterBoneItems(InPoseAsset, InHierarchy, InItems, LocalSpaceInfluences);
	OutCurveKeys = FilterCurveItems(InPoseAsset, InHierarchy, InItems);
	if (OutBoneKeys.IsEmpty() && OutCurveKeys.IsEmpty())
	{
		return true;
	}

	OutBoneTransforms.AddDefaulted(OutBoneKeys.Num());
	OutCurveValues.Reserve(OutCurveKeys.Num());
	for (const FRigElementKey& CurveKey : OutCurveKeys)
	{
		OutCurveValues.Add(InHierarchy->GetCurveValue(CurveKey));
	}
	
	FRigUnitPoseAssetExtractionContext Context(TargetSkeleton, InHierarchy, InPoses, OutBoneKeys, OutBoneTransforms, OutCurveKeys, OutCurveValues);
	InPoseAsset->GetAnimationPose(&Context, true, true);

	return OutBoneTransforms.Num() >= OutBoneKeys.Num();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetPoseNames
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetPoseNames_Execute()
{
	PoseNames.Reset();
	if (!PoseAsset)
	{
		return;
	}
	PoseNames = PoseAsset->GetPoseFNames();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetActivePoses
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetActivePoses_Execute()
{
	ActivePoses.Reset();
	if (!PoseAsset)
	{
		return;
	}

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	for (int32 PoseIndex = 0; PoseIndex < PoseAsset->GetNumPoses(); PoseIndex++)
	{
		if (PoseAsset->GetBasePoseIndex() == PoseIndex)
		{
			continue;
		}

		const FName& PoseName = PoseAsset->GetPoseFNames()[PoseIndex];
		const float Weight = Hierarchy->GetCurveValue(FRigElementKey(PoseName, ERigElementType::Curve));
		if (!FAnimWeight::IsRelevant(FMath::Abs(Weight)))
		{
			continue;
		}

		// check if this pose is masked out.
		TArrayView<const FRigElementKey> BoneKeys;
		TArrayView<const FRigElementKey> CurveKeys;
		if (!Items.IsEmpty())
		{
			WorkData.LastPoseIndex = PoseIndex;
			WorkData.bFilterByInfluences = true;
			WorkData.UpdateBoneHash(Hierarchy, PoseAsset, {}, Items);
			WorkData.UpdateCurveHash(Hierarchy, PoseAsset, {}, Items);
			
			TArray<FPoseAssetInfluenceForPose> LocalSpaceInfluences;
			BoneKeys = WorkData.FilterBoneItems(PoseAsset, Hierarchy, Items, LocalSpaceInfluences);
			CurveKeys = WorkData.FilterCurveItems(PoseAsset, Hierarchy, Items);
			if (BoneKeys.IsEmpty() && CurveKeys.IsEmpty())
			{
				continue;
			}
		}

		FRigUnit_PoseAsset_PoseInfo& PoseInfo = ActivePoses.AddDefaulted_GetRef();
		PoseInfo.PoseName = PoseName;
		PoseInfo.PoseIndex = PoseIndex;
		PoseInfo.Weight = Weight;
		PoseInfo.UseWeightFromCurve = false;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetHasRootMotion
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetHasRootMotion_Execute()
{
	Result = false;
	if (!PoseAsset)
	{
		return;
	}
	Result = PoseAsset->HasRootMotion();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetIsAdditive
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetIsAdditive_Execute()
{
	Result = false;
	BasePoseName = NAME_None;
	if (!PoseAsset)
	{
		return;
	}
	Result = PoseAsset->IsValidAdditive();
	if (Result)
	{
		const int32 BasePoseIndex = PoseAsset->GetBasePoseIndex();
		if (PoseAsset->GetPoseFNames().IsValidIndex(BasePoseIndex))
		{
			BasePoseName = PoseAsset->GetPoseFNames()[BasePoseIndex];
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetContainsPose
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetContainsPose_Execute()
{
	Result = WorkData.ValidatePose(ExecuteContext, PoseAsset, PoseName, false) != INDEX_NONE;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetBoneNames
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetBoneNames_Execute()
{
	Success = false;
	BoneNames.Reset();
	if (!PoseAsset)
	{
		return;
	}
	Success = true;
	BoneNames = PoseAsset->GetTrackNames();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetInfluences
//////////////////////////////////////////////////////////////////////////////////////////////////////

// Disable pvs static analysis warnings. The input hierarchy / items isn't visible to the analyzer here
// so it gets to the conclusion
#pragma warning(push)
#pragma warning(disable:4068) // Disable unknown pragma (for non pvs builds)

FRigUnit_PoseAssetGetInfluences_Execute()
{
	Bones.Reset();
	Curves.Reset();
	Success = false;
	if (!PoseAsset)
	{
		return;
	}

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}
	
	const int32 PoseIndex = WorkData.ValidatePose(ExecuteContext, PoseAsset, PoseName, false);
	if (PoseIndex == INDEX_NONE)
	{
		return;
	}

	WorkData.bFilterByInfluences = true;
	WorkData.UpdateBoneHash(Hierarchy, PoseAsset, {}, Items);
	WorkData.UpdateCurveHash(Hierarchy, PoseAsset, {}, Items);
	TArray<FPoseAssetInfluenceForPose> LocalSpaceInfluences;
	PoseAsset->GetTrackInfluencesForPose(PoseIndex, LocalSpaceInfluences);

	TArrayView<const FRigElementKey> BonesView = WorkData.FilterBoneItems(PoseAsset, Hierarchy, Items, LocalSpaceInfluences);
	TArrayView<const FRigElementKey> CurvesView = WorkData.FilterCurveItems(PoseAsset, Hierarchy, Items);
	
	Bones.Reserve(BonesView.Num());
	Curves.Reserve(CurvesView.Num());

	for (int32 KeyIndex = 0; KeyIndex < BonesView.Num(); KeyIndex++) //-V621 //-V654
	{
		if (WorkData.TrackIndexPerBone[KeyIndex] == INDEX_NONE)
		{
			continue;
		}
		Bones.Add(BonesView[KeyIndex]);
	}
	for (int32 KeyIndex = 0; KeyIndex < CurvesView.Num(); KeyIndex++) //-V621 //-V654
	{
		if (WorkData.TrackIndexPerCurve[KeyIndex] == INDEX_NONE)
		{
			continue;
		}
		Curves.Add(CurvesView[KeyIndex]);
	}

	Success = true;
}

// Restore static analysis warnings
#pragma warning(pop)

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetBoneIndex
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetBoneIndex_Execute()
{
	Success = false;
	Index = INDEX_NONE;
	if (!PoseAsset)
	{
		return;
	}

	if (PoseAsset->GetTrackNames().IsValidIndex(LastBoneIndex))
	{
		if (PoseAsset->GetTrackNames()[LastBoneIndex] == BoneName)
		{
			Success = true;
			Index = LastBoneIndex;
			return;
		}
	}

	Index = LastBoneIndex = PoseAsset->GetTrackIndexByName(BoneName);
	Success = Index != INDEX_NONE;
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetBoneTransformsForItems
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetPoseForItems_Execute()
{
	Success = false;
	LocalTransforms.Reset();
	CurveValues.Reset();
	Bones.Reset();
	Curves.Reset();
	
	if (!PoseAsset)
	{
		return;
	}

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	TArray<FPoseCurve> PoseCurves;
	PoseCurves.Reserve(Poses.Num());
	for (const FRigUnit_PoseAsset_PoseInfo& PoseInfo : Poses)
	{
		float Weight = PoseInfo.Weight;
		if (PoseInfo.UseWeightFromCurve)
		{
			Weight = Hierarchy->GetCurveValue(FRigElementKey(PoseInfo.PoseName, ERigElementType::Curve));
		}
		if (!FAnimWeight::IsRelevant(FMath::Abs(Weight)))
		{
			continue;
		}

		int32 PoseIndex = PoseInfo.PoseIndex;
		if (PoseIndex != INDEX_NONE)
		{
			if (!PoseAsset->GetPoseFNames().IsValidIndex(PoseIndex))
			{
				PoseIndex = INDEX_NONE;
			}
			else if (PoseAsset->GetPoseFNames()[PoseIndex] != PoseInfo.PoseName)
			{
				PoseIndex = INDEX_NONE;
			}
		}
		if (PoseIndex == INDEX_NONE)
		{
			PoseIndex = PoseAsset->GetPoseIndexByName(PoseInfo.PoseName);
		}
		if (PoseIndex == INDEX_NONE)
		{
			continue;
		}
		FPoseCurve& PoseCurve = PoseCurves.AddDefaulted_GetRef();
		PoseCurve.Name = PoseInfo.PoseName;
		PoseCurve.PoseIndex = PoseIndex;
		PoseCurve.Value = Weight;
	}
	
	TArrayView<const FRigElementKey> BoneKeys, CurveKeys;
	Success = WorkData.GetAnimationPose(ExecuteContext, PoseAsset, Hierarchy, PoseCurves, Items, BoneKeys, LocalTransforms, CurveKeys, CurveValues);

	if (Success)
	{
		Bones.Append(BoneKeys);
		Curves.Append(CurveKeys);

		const USkeleton* SourceSkeleton = PoseAsset->GetSkeleton();
		if (!SourceSkeleton)
		{
			return;
		}

		// convert the pose to a full pose rather than additive
		if (PoseAsset->IsValidAdditive())
		{
			for (int32 KeyIndex = 0; KeyIndex < BoneKeys.Num(); KeyIndex++)
			{
				if (BoneKeys[KeyIndex].Type != ERigElementType::Bone)
				{
					continue;
				}
				const int32 ElementIndex = Hierarchy->GetIndex(BoneKeys[KeyIndex]);
				if (ElementIndex == INDEX_NONE)
				{
					continue;
				}

				FTransform BaseTransform;
				switch (Mode)
				{
					case EPoseAssetGetPoseForItemsMode::Raw:
					{
						BaseTransform = FTransform::Identity;
						break;
					}
					case EPoseAssetGetPoseForItemsMode::ApplyToCurrent:
					{
						BaseTransform = Hierarchy->GetLocalTransformByIndex(ElementIndex, false /* initial */);
						break;
					}
					case EPoseAssetGetPoseForItemsMode::ApplyToInitial:
					{
						BaseTransform = Hierarchy->GetLocalTransformByIndex(ElementIndex, true /* initial */);
						break;
					}
				}
				
				FTransform AccumulatedBoneTransform = BaseTransform;
				AccumulatedBoneTransform.AccumulateWithAdditiveScale(LocalTransforms[KeyIndex], ScalarRegister(1.f));
				AccumulatedBoneTransform.NormalizeRotation();
				LocalTransforms[KeyIndex] = AccumulatedBoneTransform;
			}

			Success = true;
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetRigPose
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetRigPose_Execute()
{
	RigPose.Reset();
	Success = false;
	if (!PoseAsset)
	{
		return;
	}

	const USkeleton* SourceSkeleton = PoseAsset->GetSkeleton();
	if (!SourceSkeleton)
	{
		return;
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	TArray<FTransform> LocalTransforms;
	TArray<float> CurveValues;
	TArray<FRigElementKey> BoneKeys, CurveKeys;
	FRigUnit_PoseAssetGetPoseForItems::StaticExecute(ExecuteContext, Poses, Items, Mode, BoneKeys, LocalTransforms, CurveKeys, CurveValues, Success, WorkData, PoseAsset);
	if (!Success)
	{
		return;
	}

	RigPose.HierarchyTopologyVersion = Hierarchy->GetTopologyVersion();
	RigPose.PoseHash = RigPose.HierarchyTopologyVersion;
	RigPose.Elements.Reserve(BoneKeys.Num() + CurveKeys.Num());

	TArray<int32> SkeletonIndexToTransformIndex;
	SkeletonIndexToTransformIndex.AddZeroed(SourceSkeleton->GetReferenceSkeleton().GetNum());
	
	for (int32 KeyIndex = 0; KeyIndex < BoneKeys.Num(); KeyIndex++)
	{
		FRigPoseElement& Element = RigPose.Elements.Add_GetRef(FRigPoseElement());
		Element.Index.UpdateCache(BoneKeys[KeyIndex], Hierarchy);
		Element.LocalTransform = LocalTransforms[KeyIndex];
		Element.GlobalTransform = LocalTransforms[KeyIndex];

		const int32 BoneIndex = SourceSkeleton->GetReferenceSkeleton().FindBoneIndex(BoneKeys[KeyIndex].Name);
		if (BoneIndex != INDEX_NONE)
		{
			SkeletonIndexToTransformIndex[KeyIndex] = BoneIndex + 1;
			
			const int32 ParentBoneIndex = SourceSkeleton->GetReferenceSkeleton().GetParentIndex(BoneIndex);
			if (ParentBoneIndex != INDEX_NONE)
			{
				const int32 ParentBoneTransformIndex = SkeletonIndexToTransformIndex[ParentBoneIndex] - 1;
				if (ParentBoneTransformIndex != INDEX_NONE)
				{
					Element.GlobalTransform = Element.LocalTransform * RigPose.Elements[ParentBoneTransformIndex].GlobalTransform;
				}
				else
				{
					const FName ParentBoneName = SourceSkeleton->GetReferenceSkeleton().GetBoneName(ParentBoneIndex);
					Element.ActiveParent = FRigElementKey(ParentBoneName, ERigElementType::Bone);

					const int32 ElementIndex = Hierarchy->GetIndex(Element.ActiveParent);
					if (ElementIndex != INDEX_NONE)
					{
						if (Mode == EPoseAssetGetPoseForItemsMode::ApplyToCurrent)
						{
							Element.GlobalTransform = Element.LocalTransform * Hierarchy->GetGlobalTransformByIndex(ElementIndex, false /* initial */);
						}
						else
						{
							Element.GlobalTransform = Element.LocalTransform * Hierarchy->GetGlobalTransformByIndex(ElementIndex, true /* initial */);
						}
					}
				}
			}
		}
		
		RigPose.PoseHash = HashCombine(RigPose.PoseHash, GetTypeHash(BoneKeys[KeyIndex]));
	}

	for (int32 KeyIndex = 0; KeyIndex < CurveKeys.Num(); KeyIndex++)
	{
		FRigPoseElement& Element = RigPose.Elements.Add_GetRef(FRigPoseElement());
		Element.Index.UpdateCache(CurveKeys[KeyIndex], Hierarchy);
		Element.CurveValue = CurveValues[KeyIndex];
		RigPose.PoseHash = HashCombine(RigPose.PoseHash, GetTypeHash(CurveKeys[KeyIndex]));
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetGetCurveNames
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetGetCurveNames_Execute()
{
	Success = false;
	CurveNames.Reset();
	if (!PoseAsset)
	{
		return;
	}
	Success = true;
	CurveNames = PoseAsset->GetCurveFNames();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetApplyMultiplePoses
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetApplyMultiplePoses_Execute()
{
	Success = false;
	if (!PoseAsset)
	{
		return;
	}

	if (Poses.IsEmpty())
	{
		return;
	}

	if (!ApplyTransforms && !ApplyCurves)
	{
		return;
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	TArray<FTransform> BoneTransforms;
	TArray<float> CurveValues;
	TArrayView<const FRigElementKey> BoneKeys, CurveKeys;
	WorkData.bFilterByInfluences = true;

	TArray<FPoseCurve> PoseCurves;
	PoseCurves.Reserve(Poses.Num());
	for (const FRigUnit_PoseAsset_PoseInfo& PoseInfo : Poses)
	{
		float Weight = PoseInfo.Weight;
		if (PoseInfo.UseWeightFromCurve)
		{
			Weight = Hierarchy->GetCurveValue(FRigElementKey(PoseInfo.PoseName, ERigElementType::Curve));
		}
		if (!FAnimWeight::IsRelevant(FMath::Abs(Weight)))
		{
			continue;
		}

		int32 PoseIndex = PoseInfo.PoseIndex;
		if (PoseIndex != INDEX_NONE)
		{
			if (!PoseAsset->GetPoseFNames().IsValidIndex(PoseIndex))
			{
				PoseIndex = INDEX_NONE;
			}
			else if (PoseAsset->GetPoseFNames()[PoseIndex] != PoseInfo.PoseName)
			{
				PoseIndex = INDEX_NONE;
			}
		}
		if (PoseIndex == INDEX_NONE)
		{
			PoseIndex = PoseAsset->GetPoseIndexByName(PoseInfo.PoseName);
		}
		if (PoseIndex == INDEX_NONE)
		{
			continue;
		}
		FPoseCurve& PoseCurve = PoseCurves.AddDefaulted_GetRef();
		PoseCurve.Name = PoseInfo.PoseName;
		PoseCurve.PoseIndex = PoseIndex;
		PoseCurve.Value = Weight;
		
		UE_LOGF(LogControlRig, Warning, "Curve '%ls' at %.02f", *PoseCurve.Name.ToString(), Weight);
	}

	if (WorkData.GetAnimationPose(ExecuteContext, PoseAsset, Hierarchy, PoseCurves, Items, BoneKeys, BoneTransforms, CurveKeys, CurveValues))
	{
		if (ApplyTransforms)
		{
			FTransform Blended = FTransform::Identity;
			for (int32 KeyIndex = 0; KeyIndex < BoneKeys.Num(); KeyIndex++)
			{
				if (BoneKeys[KeyIndex].Type != ERigElementType::Bone)
				{
					continue;
				}
				const int32 ElementIndex = Hierarchy->GetIndex(BoneKeys[KeyIndex]);
				if (ElementIndex == INDEX_NONE)
				{
					continue;
				}

				if (PoseAsset->IsValidAdditive())
				{
					FTransform AccumulatedBoneTransform = Hierarchy->GetLocalTransformByIndex(ElementIndex, false);
					AccumulatedBoneTransform.AccumulateWithAdditiveScale(BoneTransforms[KeyIndex], ScalarRegister(1.f));
					AccumulatedBoneTransform.NormalizeRotation();
					Hierarchy->SetLocalTransformByIndex(ElementIndex, AccumulatedBoneTransform, false, bPropagateToChildren, false, false);
				}
				else
				{
					Hierarchy->SetLocalTransformByIndex(ElementIndex, BoneTransforms[KeyIndex], false, bPropagateToChildren, false, false);
				}
			}

			Success = true;
		}

		if (ApplyCurves)
		{
			for (int32 KeyIndex = 0; KeyIndex < CurveKeys.Num(); KeyIndex++)
			{
				if (CurveKeys[KeyIndex].Type != ERigElementType::Curve)
				{
					continue;
				}
				const int32 ElementIndex = Hierarchy->GetIndex(CurveKeys[KeyIndex]);
				if (ElementIndex == INDEX_NONE)
				{
					continue;
				}

				Hierarchy->SetCurveValueByIndex(ElementIndex, CurveValues[KeyIndex], false);
			}
			Success = true;
		}
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////
// FRigUnit_PoseAssetApplyAllPoses
//////////////////////////////////////////////////////////////////////////////////////////////////////

FRigUnit_PoseAssetApplyAllPoses_Execute()
{
	Success = false;
	if (!PoseAsset)
	{
		return;
	}

	if (!ApplyTransforms && !ApplyCurves)
	{
		return;
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (!Hierarchy)
	{
		return;
	}

	FRigUnit_PoseAssetGetActivePoses::StaticExecute(ExecuteContext, Items, ActivePoses, GetActivePosesWorkData, PoseAsset);
	if (ActivePoses.IsEmpty())
	{
		return;
	}

	FRigUnit_PoseAssetApplyMultiplePoses::StaticExecute(ExecuteContext, ActivePoses, Items, ApplyTransforms, ApplyCurves, bPropagateToChildren, Success, WorkData, PoseAsset);
}
