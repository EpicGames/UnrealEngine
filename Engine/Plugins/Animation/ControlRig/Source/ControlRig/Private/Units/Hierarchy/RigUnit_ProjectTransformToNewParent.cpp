// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_ProjectTransformToNewParent.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ProjectTransformToNewParent)

FRigUnit_ProjectTransformToNewParent_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	FTransform ChildTransform = FTransform::Identity;
	FTransform OldParentTransform = FTransform::Identity;
	FTransform NewParentTransform = FTransform::Identity;
	FTransform RelativeTransform = FTransform::Identity;

	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Child, ERigVMTransformSpace::GlobalSpace, bChildInitial, ChildTransform, CachedChild);
	FRigUnit_GetTransform::StaticExecute(ExecuteContext, OldParent, ERigVMTransformSpace::GlobalSpace, bOldParentInitial, OldParentTransform, CachedOldParent);
	FRigUnit_GetTransform::StaticExecute(ExecuteContext, NewParent, ERigVMTransformSpace::GlobalSpace, bNewParentInitial, NewParentTransform, CachedNewParent);
	FRigVMFunction_MathTransformMakeRelative::StaticExecute(ExecuteContext, ChildTransform, OldParentTransform, RelativeTransform);
	FRigVMFunction_MathTransformMakeAbsolute::StaticExecute(ExecuteContext, RelativeTransform, NewParentTransform, Transform);
}

