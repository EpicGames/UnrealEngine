// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_PropagateTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_PropagateTransform)

FRigUnit_PropagateTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    if(Item.Type != ERigElementType::Bone)
    {
    	return;
    }

    if (URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
    	/*
    	 * This node doesn't do anything anymore now that the hierarchy is lazy
    	 *
		if (!CachedIndex.UpdateCache(Item, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item '%s' is not valid."), *Item.ToString());
		}
		else
		{
			FRigBoneHierarchy& Bones = Hierarchy->BoneHierarchy;
			int32 BoneIndex = CachedIndex.GetIndex();

			if (bRecomputeGlobal)
			{
				Bones.RecalculateGlobalTransform(BoneIndex);
			}
			if (bApplyToChildren)
			{
				if (bRecursive)
				{
					Bones.PropagateTransform(BoneIndex);
				}
				else
				{
					for (int32 Dependent : Bones[BoneIndex].Dependents)
					{
						Bones.RecalculateGlobalTransform(Dependent);
					}
				}
			}
		}
		*/
	}
}

FRigVMStructUpgradeInfo FRigUnit_PropagateTransform::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

