// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SendEvent.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SendEvent)

FRigUnit_SendEvent_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    if(!bEnable)
    {
    	return;
    }

	if (bOnlyDuringInteraction && !ExecuteContext.UnitContext.IsInteracting())
	{
		return;
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		FRigEventContext EventContext;
		EventContext.Key = Item;
		EventContext.Event = Event;
		EventContext.SourceEventName = ExecuteContext.GetEventName();
		EventContext.LocalTime = ExecuteContext.GetAbsoluteTime() + OffsetInSeconds;
		Hierarchy->SendEvent(EventContext, false /* async */); //needs to be false for sequencer keying to work
	}
}

