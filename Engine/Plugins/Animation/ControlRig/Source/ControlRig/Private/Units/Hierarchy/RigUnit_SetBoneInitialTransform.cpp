// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetBoneInitialTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetBoneInitialTransform)

FRigUnit_SetBoneInitialTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Bone, ERigElementType::Bone);
		if (!CachedBone.UpdateCache(Key, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
			return;
		}

		if (Space == ERigVMTransformSpace::LocalSpace)
		{
			Hierarchy->SetInitialLocalTransform(CachedBone, Transform);
		}
		else
		{
			Hierarchy->SetInitialGlobalTransform(CachedBone, Transform);
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetBoneInitialTransform::GetUpgradeInfo() const
{
	FRigUnit_SetTransform NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Space = Space;
	NewNode.Value = Transform;
	NewNode.bInitial = true;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Transform"), TEXT("Value"));
	return Info;
}

