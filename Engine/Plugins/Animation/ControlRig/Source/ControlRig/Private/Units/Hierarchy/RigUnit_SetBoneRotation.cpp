// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetBoneRotation.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetBoneRotation)

FRigUnit_SetBoneRotation_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Bone, ERigElementType::Bone);
		if (!CachedBone.UpdateCache(Key, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					FTransform Transform = Hierarchy->GetGlobalTransform(CachedBone);

					if (FMath::IsNearlyEqual(Weight, 1.f))
					{
						Transform.SetRotation(Rotation);
					}
					else
					{
						float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
						Transform.SetRotation(FQuat::Slerp(Transform.GetRotation(), Rotation, T));
					}

					Hierarchy->SetGlobalTransform(CachedBone, Transform, bPropagateToChildren);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					FTransform Transform = Hierarchy->GetLocalTransform(CachedBone);

					if (FMath::IsNearlyEqual(Weight, 1.f))
					{
						Transform.SetRotation(Rotation);
					}
					else
					{
						float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
						Transform.SetRotation(FQuat::Slerp(Transform.GetRotation(), Rotation, T));
					}

					Hierarchy->SetLocalTransform(CachedBone, Transform, bPropagateToChildren);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetBoneRotation::GetUpgradeInfo() const
{
	FRigUnit_SetRotation NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Space = Space;
	NewNode.Value = Rotation;
	NewNode.Weight = Weight;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Rotation"), TEXT("Value"));
	return Info;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetBoneRotation)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FQuat(FVector(-1.f, 0.f, 0.f), 0.1f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FQuat(FVector(-1.f, 0.f, 0.f), 0.5f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), BoneA, FTransform(FQuat(FVector(-1.f, 0.f, 0.f), 0.7f)), true, ERigBoneType::User);

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Bone = TEXT("Root");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Unit.Rotation = FQuat(FVector(-1.f, 0.f, 0.f), 0.25f);
	Unit.bPropagateToChildren = false;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle(), 0.25f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle(), 0.5f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle(), 0.7f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle()));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle(), 0.25f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle(), 0.5f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle(), 0.7f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle()));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bPropagateToChildren = true;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle(), 0.25f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle(), 0.65f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle(), 0.85f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle()));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Bone = TEXT("BoneA");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Unit.bPropagateToChildren = false;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle(), 0.1f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle(), 0.25f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle(), 0.7f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle()));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle(), 0.1f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle(), 0.35f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle(), 0.7f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle()));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bPropagateToChildren = true;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle(), 0.1f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(0).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle(), 0.35f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(1).GetRotation().GetAngle()));
	AddErrorIfFalse(FMath::IsNearlyEqual(Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle(), 0.55f, 0.001f), FString::Printf(TEXT("unexpected angle %.04f"), Hierarchy->GetGlobalTransform(2).GetRotation().GetAngle()));

	return true;
}
#endif
