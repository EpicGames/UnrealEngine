// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetBoneTransform.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetBoneTransform)

FRigUnit_SetBoneTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Bone, ERigElementType::Bone);
		if (!CachedBone.UpdateCache(Key, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					if (FMath::IsNearlyEqual(Weight, 1.f))
					{
						Result = Transform;
					}
					else
					{
						float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
						const FTransform PreviousTransform = Hierarchy->GetGlobalTransform(CachedBone);
						Result = FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, T);
					}
					Hierarchy->SetGlobalTransform(CachedBone, Result, bPropagateToChildren);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					if (FMath::IsNearlyEqual(Weight, 1.f))
					{
						Result = Transform;
					}
					else
					{
						float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
						const FTransform PreviousTransform = Hierarchy->GetLocalTransform(CachedBone);
						Result = FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, T);
					}
					Hierarchy->SetLocalTransform(CachedBone, Result, bPropagateToChildren);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetBoneTransform::GetUpgradeInfo() const
{
	FRigUnit_SetTransform NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Space = Space;
	NewNode.Value = Transform;
	NewNode.Weight = Weight;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Transform"), TEXT("Value"));
	return Info;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetBoneTransform)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(1.f, 2.f, 3.f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), BoneA, FTransform(FVector(1.f, 5.f, 3.f)), true, ERigBoneType::User);

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Bone = TEXT("Root");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Unit.Transform = FTransform(FVector(0.f, 0.f, 7.f));
	Unit.bPropagateToChildren = false;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(0.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 2.f, 3.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 5.f, 3.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(0.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 2.f, 3.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 5.f, 3.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bPropagateToChildren = true;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(0.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(0.f, 2.f, 10.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(0.f, 5.f, 10.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Bone = TEXT("BoneA");
	Unit.Space = ERigVMTransformSpace::GlobalSpace;
	Unit.bPropagateToChildren = false;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(0.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 5.f, 3.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Space = ERigVMTransformSpace::LocalSpace;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 5.f, 3.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bPropagateToChildren = true;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 3.f, 7.f)), TEXT("unexpected transform"));

	return true;
}
#endif
