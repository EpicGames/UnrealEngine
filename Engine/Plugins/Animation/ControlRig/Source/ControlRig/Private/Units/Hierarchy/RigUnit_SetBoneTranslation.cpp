// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetBoneTranslation.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetBoneTranslation)

FRigUnit_SetBoneTranslation_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Bone, ERigElementType::Bone); 
		if (!CachedBone.UpdateCache(Key, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					FTransform Transform = Hierarchy->GetGlobalTransform(CachedBone);

					if (FMath::IsNearlyEqual(Weight, 1.f))
					{
						Transform.SetTranslation(Translation);
					}
					else
					{
						float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
						Transform.SetTranslation(FMath::Lerp<FVector>(Transform.GetTranslation(), Translation, T));
					}

					Hierarchy->SetGlobalTransform(CachedBone, Transform, bPropagateToChildren);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					FTransform Transform = Hierarchy->GetLocalTransform(CachedBone);

					if (FMath::IsNearlyEqual(Weight, 1.f))
					{
						Transform.SetTranslation(Translation);
					}
					else
					{
						float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
						Transform.SetTranslation(FMath::Lerp<FVector>(Transform.GetTranslation(), Translation, T));
					}

					Hierarchy->SetLocalTransform(CachedBone, Transform, bPropagateToChildren);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetBoneTranslation::GetUpgradeInfo() const
{
	FRigUnit_SetTranslation NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Space = Space;
	NewNode.Value = Translation;
	NewNode.Weight = Weight;	

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Translation"), TEXT("Value"));
	return Info;
}

