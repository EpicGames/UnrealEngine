// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetControlColor.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetControlColor)

FRigUnit_GetControlColor_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    Color = FLinearColor::Black;

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			const FRigControlElement* ControlElement = Hierarchy->GetChecked<FRigControlElement>(CachedControlIndex);
			Color = ControlElement->Settings.ShapeColor;
		}
	}
}

FRigUnit_SetControlColor_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			FRigControlElement* ControlElement = Hierarchy->GetChecked<FRigControlElement>(CachedControlIndex);
			ControlElement->Settings.ShapeColor = Color;
			Hierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);
		}
	}
}

