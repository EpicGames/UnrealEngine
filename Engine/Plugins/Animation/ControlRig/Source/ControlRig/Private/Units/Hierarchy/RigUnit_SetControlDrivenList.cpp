// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetControlDrivenList.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetControlDrivenList)

FRigUnit_GetControlDrivenList_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    Driven.Reset();

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			const FRigControlElement* ControlElement = Hierarchy->GetChecked<FRigControlElement>(CachedControlIndex);
			Driven = ControlElement->Settings.DrivenControls;
		}
	}
}

FRigUnit_SetControlDrivenList_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			FRigControlElement* ControlElement = Hierarchy->GetChecked<FRigControlElement>(CachedControlIndex);
			if(ControlElement->Settings.DrivenControls != Driven)
			{
				Swap(ControlElement->Settings.DrivenControls, ControlElement->Settings.PreviouslyDrivenControls);
				ControlElement->Settings.DrivenControls = Driven;
				Hierarchy->Notify(ERigHierarchyNotification::ControlDrivenListChanged, ControlElement);
			}
		}
	}
}

