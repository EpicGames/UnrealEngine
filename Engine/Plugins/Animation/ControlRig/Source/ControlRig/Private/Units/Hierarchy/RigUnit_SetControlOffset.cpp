// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetControlOffset.h"
#include "Units/Hierarchy/RigUnit_GetControlOffset.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetControlOffset)

FString FRigUnit_SetControlOffset::GetUnitLabel() const
{
	return FString::Printf(TEXT("Set Control Offset"));
}

FRigUnit_SetControlOffset_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
			return;
		}

		FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(CachedControlIndex);
		if (Space == ERigVMTransformSpace::GlobalSpace)
		{
			Hierarchy->SetControlOffsetTransform(ControlElement, Offset, ERigTransformType::CurrentGlobal, true, false);
			Hierarchy->SetControlOffsetTransform(ControlElement, Offset, ERigTransformType::InitialGlobal, true, false);
		}
		else
		{
			Hierarchy->SetControlOffsetTransform(ControlElement, Offset, ERigTransformType::CurrentLocal, true, false);
			Hierarchy->SetControlOffsetTransform(ControlElement, Offset, ERigTransformType::InitialLocal, true, false);
		}
	}
}

FString FRigUnit_SetControlTranslationOffset::GetUnitLabel() const
{
	return FString::Printf(TEXT("Set Control Translation Offset"));
}

FRigUnit_SetControlTranslationOffset_Execute()
{
	FTransform Transform = FTransform::Identity;
	FRigUnit_GetControlOffset::StaticExecute(ExecuteContext, Control, Space, Transform, CachedControlIndex);
	Transform.SetLocation(Offset);
	FRigUnit_SetControlOffset::StaticExecute(ExecuteContext, Control, Transform, Space, CachedControlIndex);
}

FString FRigUnit_SetControlRotationOffset::GetUnitLabel() const
{
	return FString::Printf(TEXT("Set Control Rotation Offset"));
}

FRigUnit_SetControlRotationOffset_Execute()
{
	FTransform Transform = FTransform::Identity;
	FRigUnit_GetControlOffset::StaticExecute(ExecuteContext, Control, Space, Transform, CachedControlIndex);
	Transform.SetRotation(Offset);
	FRigUnit_SetControlOffset::StaticExecute(ExecuteContext, Control, Transform, Space, CachedControlIndex);
}

FString FRigUnit_SetControlScaleOffset::GetUnitLabel() const
{
	return FString::Printf(TEXT("Set Control Scale Offset"));
}

FRigUnit_SetControlScaleOffset_Execute()
{
	FTransform Transform = FTransform::Identity;
	FRigUnit_GetControlOffset::StaticExecute(ExecuteContext, Control, Space, Transform, CachedControlIndex);
	Transform.SetScale3D(Scale);
	FRigUnit_SetControlOffset::StaticExecute(ExecuteContext, Control, Transform, Space, CachedControlIndex);
}

FRigUnit_GetShapeTransform_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
			return;
		}

		const FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(CachedControlIndex);
		Transform = Hierarchy->GetControlShapeTransform((FRigControlElement*)ControlElement, ERigTransformType::CurrentLocal);
	}
}

FRigUnit_SetShapeTransform_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (!CachedControlIndex.UpdateCache(FRigElementKey(Control, ERigElementType::Control), Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Control '%s' is not valid."), *Control.ToString());
			return;
		}

		FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(CachedControlIndex);
		Hierarchy->SetControlShapeTransform(ControlElement, Transform, ERigTransformType::InitialLocal);
	}
}
