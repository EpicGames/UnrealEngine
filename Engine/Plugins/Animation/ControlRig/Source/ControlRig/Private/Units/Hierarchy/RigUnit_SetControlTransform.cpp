// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetControlTransform.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetControlTransform)

FRigUnit_SetControlBool_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<bool>(BoolValue));
		}
	}
}

FRigUnit_SetMultiControlBool_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		CachedControlIndices.SetNum(Entries.Num());

		for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
		{
			bool BoolValue = Entries[EntryIndex].BoolValue;

			FRigUnit_SetControlBool::StaticExecute(ExecuteContext, Entries[EntryIndex].Control, BoolValue, CachedControlIndices[EntryIndex]);
		}
	}
}

FRigUnit_SetControlFloat_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			if(FMath::IsNearlyEqual((float)Weight, 1.f))
			{
				Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<float>(FloatValue));
			}
			else
			{
				float PreviousValue = Hierarchy->GetControlValue(CachedControlIndex).Get<float>();
				Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<float>(FMath::Lerp<float>(PreviousValue, FloatValue, FMath::Clamp<float>(Weight, 0.f, 1.f))));
			}
		}
	}
}

FRigUnit_SetMultiControlFloat_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{ 
		CachedControlIndices.SetNum(Entries.Num());
	 
		for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
		{
			float FloatValue = Entries[EntryIndex].FloatValue;

			FRigUnit_SetControlFloat::StaticExecute(ExecuteContext, Entries[EntryIndex].Control, Weight, FloatValue, CachedControlIndices[EntryIndex]); 
		}
	}
}

FRigUnit_SetControlInteger_Execute()
{
	float FloatWeight = (float)Weight;
	FRigUnit_SetControlIntegerFloatWeight::StaticExecute(ExecuteContext, Control, FloatWeight, IntegerValue, CachedControlIndex);	
}

FRigVMStructUpgradeInfo FRigUnit_SetControlInteger::GetUpgradeInfo() const
{
	FRigUnit_SetControlIntegerFloatWeight NewNode;
	NewNode.Control = Control;
	NewNode.Weight = (float)Weight;
	NewNode.IntegerValue = IntegerValue;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_SetControlIntegerFloatWeight_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			if(FMath::IsNearlyEqual(Weight, 1.f))
			{
				Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<int32>(IntegerValue));
			}
			else
			{
				int32 PreviousValue = Hierarchy->GetControlValue(CachedControlIndex).Get<int32>();
				Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<int32>((int32)FMath::Lerp<float>((float)PreviousValue, (float)IntegerValue, FMath::Clamp<float>(Weight, 0.f, 1.f))));
			}
		}
	}
}

FRigUnit_SetMultiControlInteger_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		CachedControlIndices.SetNum(Entries.Num());

		for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
		{
			int32 IntegerValue = Entries[EntryIndex].IntegerValue;

			FRigUnit_SetControlInteger::StaticExecute(ExecuteContext, Entries[EntryIndex].Control, Weight, IntegerValue, CachedControlIndices[EntryIndex]);
		}
	}
}

FRigUnit_SetControlVector2D_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			const FVector3f CurrentValue(Vector.X, Vector.Y, 0.f);
			if(FMath::IsNearlyEqual((float)Weight, 1.f))
			{
				Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<FVector3f>(CurrentValue));
			}
			else
			{
				const FVector3f PreviousValue = Hierarchy->GetControlValue(CachedControlIndex).Get<FVector3f>();
				Hierarchy->SetControlValue(CachedControlIndex, FRigControlValue::Make<FVector3f>(FMath::Lerp<FVector3f>(PreviousValue, CurrentValue, FMath::Clamp<float>(Weight, 0.f, 1.f))));
			}
		}
	}
}

FRigUnit_SetMultiControlVector2D_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		CachedControlIndices.SetNum(Entries.Num());

		for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
		{
			FVector2D Vector = Entries[EntryIndex].Vector;

			FRigUnit_SetControlVector2D::StaticExecute(ExecuteContext, Entries[EntryIndex].Control, Weight, Vector, CachedControlIndices[EntryIndex]);
		}
	}
}

FRigUnit_SetControlVector_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			const ERigControlType ControlType = Hierarchy->GetChecked<FRigControlElement>(CachedControlIndex)->Settings.ControlType;
			
			FTransform Transform = FTransform::Identity;
			if (Space == ERigVMTransformSpace::GlobalSpace)
			{
				Transform = Hierarchy->GetGlobalTransform(CachedControlIndex);
			}

			if (ControlType == ERigControlType::Position)
			{
				if(FMath::IsNearlyEqual((float)Weight, 1.f))
				{
					Transform.SetLocation(Vector);
				}
				else
				{
					FVector PreviousValue = Transform.GetLocation();
					Transform.SetLocation(FMath::Lerp<FVector>(PreviousValue, Vector, FMath::Clamp<float>(Weight, 0.f, 1.f)));
				}
			}
			else if (ControlType == ERigControlType::Scale)
			{
				if(FMath::IsNearlyEqual((float)Weight, 1.f))
				{
					Transform.SetScale3D(Vector);
				}
				else
				{
					FVector PreviousValue = Transform.GetScale3D();
					Transform.SetScale3D(FMath::Lerp<FVector>(PreviousValue, Vector, FMath::Clamp<float>(Weight, 0.f, 1.f)));
				}
			}

			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Hierarchy->SetGlobalTransform(CachedControlIndex, Transform);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Hierarchy->SetLocalTransform(CachedControlIndex, Transform);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigUnit_SetControlRotator_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			FTransform Transform = FTransform::Identity;
			if (Space == ERigVMTransformSpace::GlobalSpace)
			{
				Transform = Hierarchy->GetGlobalTransform(CachedControlIndex);
			}

			FQuat Quat = FQuat(Rotator);
			if (FMath::IsNearlyEqual((float)Weight, 1.f))
			{
				Transform.SetRotation(Quat);
			}
			else
			{
				FQuat PreviousValue = Transform.GetRotation();
				Transform.SetRotation(FQuat::Slerp(PreviousValue, Quat, FMath::Clamp<float>(Weight, 0.f, 1.f)));
			}
			Transform.NormalizeRotation();

			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Hierarchy->SetGlobalTransform(CachedControlIndex, Transform);
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Hierarchy->SetLocalTransform(CachedControlIndex, Transform);
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigUnit_SetMultiControlRotator_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		CachedControlIndices.SetNum(Entries.Num());

		for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
		{
			FRotator Rotator = Entries[EntryIndex].Rotator;

			FRigUnit_SetControlRotator::StaticExecute(ExecuteContext, Entries[EntryIndex].Control, Weight, Rotator, Entries[EntryIndex].Space , CachedControlIndices[EntryIndex]);
		}
	}
}

FRigUnit_SetControlTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Control, ERigElementType::Control);
		if (CachedControlIndex.UpdateCache(Key, Hierarchy))
		{
			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					if(FMath::IsNearlyEqual((float)Weight, 1.f))
					{
						Hierarchy->SetGlobalTransform(CachedControlIndex, Transform);
					}
					else
					{
						FTransform PreviousTransform = Hierarchy->GetGlobalTransform(CachedControlIndex);
						Hierarchy->SetGlobalTransform(CachedControlIndex, FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, FMath::Clamp<float>(Weight, 0.f, 1.f)));
					}
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					if(FMath::IsNearlyEqual((float)Weight, 1.f))
					{
						Hierarchy->SetLocalTransform(CachedControlIndex, Transform);
					}
					else
					{
						FTransform PreviousTransform = Hierarchy->GetLocalTransform(CachedControlIndex);
						Hierarchy->SetLocalTransform(CachedControlIndex, FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, FMath::Clamp<float>(Weight, 0.f, 1.f)));
					}
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetControlTransform::GetUpgradeInfo() const
{
	FRigUnit_SetTransform NewNode;
	NewNode.Item = FRigElementKey(Control, ERigElementType::Control);
	NewNode.Space = Space;
	NewNode.Value = Transform;
	NewNode.Weight = Weight;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Control"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Transform"), TEXT("Value"));
	return Info;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetMultiControlBool)
{
	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Bool;
	
	Controller->AddControl(TEXT("Control1"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);
	Controller->AddControl(TEXT("Control2"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);

	// this unit has an empty entry by default, clear it for testing purpose
	Unit.Entries.Reset();

	FRigUnit_SetMultiControlBool_Entry Entry;

	// functional test, whether the unit can set multiple controls
	Entry.Control = TEXT("Control1");
	Entry.BoolValue = true;
	Unit.Entries.Add(Entry);

	Entry.Control = TEXT("Control2");
	Entry.BoolValue = true;
	Unit.Entries.Add(Entry);
	AddErrorIfFalse(Unit.Entries.Num() == 2, TEXT("unexpected number of entries"));

	Execute();

	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control1"), ERigElementType::Control), ERigControlValueType::Current).Get<bool>() == true, TEXT("unexpected control value"));
	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control2"), ERigElementType::Control), ERigControlValueType::Current).Get<bool>() == true, TEXT("unexpected control value")); 

	return true;
}

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetMultiControlFloat)
{
	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Float;
	
	Controller->AddControl(TEXT("Control1"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);
	Controller->AddControl(TEXT("Control2"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);

	// this unit has an empty entry by default, clear it for testing purpose
	Unit.Entries.Reset();

	FRigUnit_SetMultiControlFloat_Entry Entry;

	// functional test, whether the unit can set multiple controls
	Entry.Control = TEXT("Control1");
	Entry.FloatValue = 10.0f;
	Unit.Entries.Add(Entry);

	Entry.Control = TEXT("Control2");
	Entry.FloatValue = 20.0f;
	Unit.Entries.Add(Entry);
	AddErrorIfFalse(Unit.Entries.Num() == 2, TEXT("unexpected number of entries"));

	Execute();

	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control1"), ERigElementType::Control), ERigControlValueType::Current).Get<float>() == 10.0f, TEXT("unexpected control value"));
	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control2"), ERigElementType::Control), ERigControlValueType::Current).Get<float>() == 20.0f, TEXT("unexpected control value"));

	// test if the caches were updated 
	AddErrorIfFalse(Unit.CachedControlIndices.Num() == 2, TEXT("unexpected number of cache entries"));
	AddErrorIfFalse(Unit.CachedControlIndices[0].GetKey().Name == Unit.Entries[0].Control, TEXT("unexpected cached control"));
	AddErrorIfFalse(Unit.CachedControlIndices[1].GetKey().Name == Unit.Entries[1].Control, TEXT("unexpected cached control"));

	// test if we are reallocating cache unnecessarily
	FCachedRigElement* AddressOfCachedElement1 = &Unit.CachedControlIndices[0];
	FCachedRigElement* AddressOfCachedElement2 = &Unit.CachedControlIndices[1];
	Execute();
	AddErrorIfFalse(AddressOfCachedElement1 == &Unit.CachedControlIndices[0], TEXT("unexpected cache reallocation"));
	AddErrorIfFalse(AddressOfCachedElement2 == &Unit.CachedControlIndices[1], TEXT("unexpected cache reallocation"));

	// test if the caches can be updated upon changing the entries
	Unit.Entries.Reset();

	Entry.Control = TEXT("Control2");
	Entry.FloatValue = 30.0f;
	Unit.Entries.Add(Entry);

	AddErrorIfFalse(Unit.Entries.Num() == 1, TEXT("unexpected number of entries"));

	Execute();

	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control2"), ERigElementType::Control), ERigControlValueType::Current).Get<float>() == 30.0f, TEXT("unexpected control value"));

	AddErrorIfFalse(Unit.CachedControlIndices.Num() == 1, TEXT("unexpected number of cache entries"));
	AddErrorIfFalse(Unit.CachedControlIndices[0].GetKey().Name == Unit.Entries[0].Control, TEXT("unexpected cached control"));

	return true;
}

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetMultiControlInteger)
{
	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Integer;
	
	Controller->AddControl(TEXT("Control1"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);
	Controller->AddControl(TEXT("Control2"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);

	// this unit has an empty entry by default, clear it for testing purpose
	Unit.Entries.Reset();

	FRigUnit_SetMultiControlInteger_Entry Entry;

	// functional test, whether the unit can set multiple controls
	Entry.Control = TEXT("Control1");
	Entry.IntegerValue = 10;
	Unit.Entries.Add(Entry);

	Entry.Control = TEXT("Control2");
	Entry.IntegerValue = 20;
	Unit.Entries.Add(Entry);
	AddErrorIfFalse(Unit.Entries.Num() == 2, TEXT("unexpected number of entries"));

	Execute();

	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control1"), ERigElementType::Control), ERigControlValueType::Current).Get<int32>() == 10, TEXT("unexpected control value"));
	AddErrorIfFalse(Hierarchy->GetControlValue(FRigElementKey(TEXT("Control2"), ERigElementType::Control), ERigControlValueType::Current).Get<int32>() == 20, TEXT("unexpected control value"));

	return true;
}

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetMultiControlVector2D)
{
	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Vector2D;
	
	Controller->AddControl(TEXT("Control1"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);
	Controller->AddControl(TEXT("Control2"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);

	// this unit has an empty entry by default, clear it for testing purpose
	Unit.Entries.Reset();

	FRigUnit_SetMultiControlVector2D_Entry Entry;

	// functional test, whether the unit can set multiple controls
	Entry.Control = TEXT("Control1");
	Entry.Vector = FVector2D(1.f, 1.f);
	Unit.Entries.Add(Entry);

	Entry.Control = TEXT("Control2");
	Entry.Vector = FVector2D(2.f, 2.f);
	Unit.Entries.Add(Entry);
	AddErrorIfFalse(Unit.Entries.Num() == 2, TEXT("unexpected number of entries"));

	Execute();

	const FVector3f TempValue1 = Hierarchy->GetControlValue(FRigElementKey(TEXT("Control1"), ERigElementType::Control), ERigControlValueType::Current).Get<FVector3f>();
	const FVector3f TempValue2 = Hierarchy->GetControlValue(FRigElementKey(TEXT("Control2"), ERigElementType::Control), ERigControlValueType::Current).Get<FVector3f>();
	AddErrorIfFalse(FVector2D(TempValue1.X, TempValue1.Y) == FVector2D(1.f, 1.f), TEXT("unexpected control value"));
	AddErrorIfFalse(FVector2D(TempValue2.X, TempValue2.Y) == FVector2D(2.f, 2.f), TEXT("unexpected control value"));

	return true;
}

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetMultiControlRotator)
{
	FRigControlSettings Settings;
	Settings.ControlType = ERigControlType::Rotator;
	
	Controller->AddControl(TEXT("Control1"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);
	Controller->AddControl(TEXT("Control2"), FRigElementKey(), Settings, FRigControlValue(), FTransform::Identity, FTransform::Identity);

	// this unit has an empty entry by default, clear it for testing purpose
	Unit.Entries.Reset();

	FRigUnit_SetMultiControlRotator_Entry Entry;

	// functional test, whether the unit can set multiple controls
	Entry.Control = TEXT("Control1");
	Entry.Rotator = FRotator(90.f,0.f,0.f);
	Entry.Space = ERigVMTransformSpace::LocalSpace;
	Unit.Entries.Add(Entry);

	Entry.Control = TEXT("Control2");
	Entry.Rotator = FRotator(0.f,90.f,0.f);
	Entry.Space = ERigVMTransformSpace::LocalSpace;
	Unit.Entries.Add(Entry);
	AddErrorIfFalse(Unit.Entries.Num() == 2, TEXT("unexpected number of entries"));

	Execute(); 

	const FVector TempValue1 = (FVector)Hierarchy->GetControlValue(FRigElementKey(TEXT("Control1"), ERigElementType::Control), ERigControlValueType::Current).Get<FVector3f>();
	const FVector TempValue2 = (FVector)Hierarchy->GetControlValue(FRigElementKey(TEXT("Control2"), ERigElementType::Control), ERigControlValueType::Current).Get<FVector3f>();
	AddErrorIfFalse(FRotator::MakeFromEuler(TempValue1) != FRotator(0.f, 0.f, 0.f), TEXT("unexpected control value"));
	AddErrorIfFalse(FRotator::MakeFromEuler(TempValue2) != FRotator(0.f, 0.f, 0.f), TEXT("unexpected control value"));

	return true;
}

#endif

