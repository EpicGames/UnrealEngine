// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetControlVisibility.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetControlVisibility)

FRigUnit_GetControlVisibility_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

    bVisible = false;

	const URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedControlIndex.UpdateCache(Item, Hierarchy))
		{
			if(const FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(CachedControlIndex))
			{
				bVisible = ControlElement->Settings.bShapeVisible;
			}
		}
	}
}

FRigUnit_SetControlVisibility_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		auto UpdateVisibility = [&ExecuteContext, Hierarchy, bNewVisibility = bVisible](const FCachedRigElement& CachedControlIndex)
		{
			if (CachedControlIndex.IsValid())
			{
#if WITH_EDITOR
				if (const FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(CachedControlIndex))
				{
					if (ControlElement->Settings.AnimationType == ERigControlAnimationType::ProxyControl &&
						ControlElement->Settings.ShapeVisibility == ERigControlVisibility::BasedOnSelection)
					{
						UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(
						TEXT("SetControlVisibility: visibility of '%s' is based on selection, and cannot be modified through SetControlVisibility. "
						"In order to be able to modify the control's visibility, set the ShapeVisibility to UserDefined."),
						*ControlElement->GetKey().ToString());
					}
				}
#endif
				Hierarchy->SetControlVisibility(CachedControlIndex, bNewVisibility);
			}
		};
		
		if (Item.IsValid())
		{
			if (Item.Type != ERigElementType::Control)
			{
				return;
			}
			
			if (CachedControlIndices.Num() != 1)
			{
				CachedControlIndices.Reset();
				CachedControlIndices.SetNumZeroed(1);
			}

			if (CachedControlIndices[0].UpdateCache(Item, Hierarchy))
			{
				UpdateVisibility(CachedControlIndices[0]);
			} 
			return;
		}

		TArray<FRigElementKey> Keys;
		if (!Pattern.IsEmpty())
		{
			Hierarchy->ForEach<FRigControlElement>([&Keys, Pattern](FRigControlElement* ControlElement) -> bool
			{
				if (ControlElement->GetFName().ToString().Contains(Pattern, ESearchCase::CaseSensitive))
				{
					Keys.Add(ControlElement->GetKey());
				}
				return true;
			});
		}

		if (CachedControlIndices.Num() != Keys.Num())
		{
			CachedControlIndices.Reset();
			CachedControlIndices.SetNumZeroed(Keys.Num());
		}

		if (!CachedControlIndices.IsEmpty())
		{
			for (int32 Index = 0; Index < Keys.Num(); Index++)
			{
				if (CachedControlIndices[Index].UpdateCache(Keys[Index], Hierarchy))
				{
					UpdateVisibility(CachedControlIndices[Index]);
				}
			}
		}
	}
}

