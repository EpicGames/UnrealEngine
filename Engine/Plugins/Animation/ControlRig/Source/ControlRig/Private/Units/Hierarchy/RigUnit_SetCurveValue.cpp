// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetCurveValue.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetCurveValue)

FRigUnit_SetCurveValue_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Curve, ERigElementType::Curve);
		if (CachedCurveIndex.UpdateCache(Key, Hierarchy))
		{
			Hierarchy->SetCurveValueByIndex(CachedCurveIndex, Value);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetCurveValue)
{
	const FRigElementKey CurveA = Controller->AddCurve(TEXT("CurveA"), 0.f);
	const FRigElementKey CurveB = Controller->AddCurve(TEXT("CurveB"), 0.f);
	
	Hierarchy->ResetCurveValues();
	Unit.Curve = TEXT("CurveA");
	Unit.Value = 3.0f;
	Execute();

	AddErrorIfFalse(Hierarchy->GetCurveValue(CurveA) == 3.f, TEXT("unexpected value"));

	Hierarchy->ResetCurveValues();
	Unit.Curve = TEXT("CurveB");
	Unit.Value = 13.0f;
	Execute();

	AddErrorIfFalse(Hierarchy->GetCurveValue(CurveB) == 13.f, TEXT("unexpected value"));

	return true;
}
#endif
