// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetRelativeBoneTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Hierarchy/RigUnit_SetRelativeTransform.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetRelativeBoneTransform)

FRigUnit_SetRelativeBoneTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey BoneKey(Bone, ERigElementType::Bone);
		const FRigElementKey SpaceKey(Space, ERigElementType::Bone);
		if (!CachedBone.UpdateCache(BoneKey, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
		}
		else if (!CachedSpaceIndex.UpdateCache(SpaceKey, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Bone '%s' is not valid."), *Bone.ToString());
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Space '%s' is not valid."), *Space.ToString());
		}
		else
		{
			const FTransform SpaceTransform = Hierarchy->GetGlobalTransform(CachedSpaceIndex);
			FTransform TargetTransform = Transform * SpaceTransform;

			if (!FMath::IsNearlyEqual(Weight, 1.f))
			{
				float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
				const FTransform PreviousTransform = Hierarchy->GetGlobalTransform(CachedBone);
				TargetTransform = FControlRigMathLibrary::LerpTransform(PreviousTransform, TargetTransform, T);
			}

			Hierarchy->SetGlobalTransform(CachedBone, TargetTransform, bPropagateToChildren);
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetRelativeBoneTransform::GetUpgradeInfo() const
{
	FRigUnit_SetRelativeTransformForItem NewNode;
	NewNode.Parent = FRigElementKey(Space, ERigElementType::Bone);
	NewNode.Child = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Value = Transform;
	NewNode.Weight = Weight;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Space"), TEXT("Parent.Name"));
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Child.Name"));
	Info.AddRemappedPin(TEXT("RelativeTransform"), TEXT("Transform"));
	return Info;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_SetRelativeBoneTransform)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(1.f, 2.f, 3.f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), BoneA, FTransform(FVector(1.f, 5.f, 3.f)), true, ERigBoneType::User);
	const FRigElementKey BoneC = Controller->AddBone(TEXT("BoneC"), Root, FTransform(FVector(-4.f, 0.f, 0.f)), true, ERigBoneType::User);

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Bone = TEXT("BoneA");
	Unit.Space = TEXT("Root");
	Unit.Transform = FTransform(FVector(0.f, 0.f, 7.f));
	Unit.bPropagateToChildren = false;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 5.f, 3.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(3).GetTranslation().Equals(FVector(-4.f, 0.f, 0.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bPropagateToChildren = true;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(1.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 3.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(3).GetTranslation().Equals(FVector(-4.f, 0.f, 0.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Space = TEXT("BoneC");
	Unit.bPropagateToChildren = false;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(-4.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(1.f, 5.f, 3.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(3).GetTranslation().Equals(FVector(-4.f, 0.f, 0.f)), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bPropagateToChildren = true;
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(1.f, 0.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(-4.f, 0.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(-4.f, 3.f, 7.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(3).GetTranslation().Equals(FVector(-4.f, 0.f, 0.f)), TEXT("unexpected transform"));

	return true;
}
#endif
