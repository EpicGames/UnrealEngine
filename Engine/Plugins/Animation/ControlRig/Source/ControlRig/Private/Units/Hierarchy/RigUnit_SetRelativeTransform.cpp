// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetRelativeTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"
#include "Units/Hierarchy/RigUnit_GetRelativeTransform.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetRelativeTransform)

FRigUnit_SetRelativeTransformForItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Weight < SMALL_NUMBER)
	{
		return;
	}

	FTransform ParentTransform = FTransform::Identity;
	FTransform GlobalTransform = FTransform::Identity;

	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Parent, ERigVMTransformSpace::GlobalSpace, bParentInitial, ParentTransform, CachedParent);
	FRigVMFunction_MathTransformMakeAbsolute::StaticExecute(ExecuteContext, Value, ParentTransform, GlobalTransform);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Child, ERigVMTransformSpace::GlobalSpace, false, GlobalTransform, Weight, bPropagateToChildren, CachedChild);
}

FRigUnit_SetRelativeTranslationForItem_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Weight < SMALL_NUMBER)
	{
		return;
	}

	FTransform ParentTransform = FTransform::Identity;
	FTransform LocalTransform = FTransform::Identity;
	FTransform GlobalTransform = FTransform::Identity;

	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Parent, ERigVMTransformSpace::GlobalSpace, bParentInitial, ParentTransform, CachedParent);
	FRigUnit_GetRelativeTransformForItem::StaticExecute(ExecuteContext, Child, false, Parent, bParentInitial, LocalTransform, CachedChild, CachedParent);
	LocalTransform.SetTranslation(Value);
	FRigVMFunction_MathTransformMakeAbsolute::StaticExecute(ExecuteContext, LocalTransform, ParentTransform, GlobalTransform);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Child, ERigVMTransformSpace::GlobalSpace, false, GlobalTransform, Weight, bPropagateToChildren, CachedChild);
}

FRigUnit_SetRelativeRotationForItem_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Weight < SMALL_NUMBER)
	{
		return;
	}

	FTransform ParentTransform = FTransform::Identity;
	FTransform LocalTransform = FTransform::Identity;
	FTransform GlobalTransform = FTransform::Identity;

	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Parent, ERigVMTransformSpace::GlobalSpace, bParentInitial, ParentTransform, CachedParent);
	FRigUnit_GetRelativeTransformForItem::StaticExecute(ExecuteContext, Child, false, Parent, bParentInitial, LocalTransform, CachedChild, CachedParent);
	LocalTransform.SetRotation(Value);
	LocalTransform.NormalizeRotation();
	FRigVMFunction_MathTransformMakeAbsolute::StaticExecute(ExecuteContext, LocalTransform, ParentTransform, GlobalTransform);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Child, ERigVMTransformSpace::GlobalSpace, false, GlobalTransform, Weight, bPropagateToChildren, CachedChild);
}

#if WITH_EDITOR

bool FRigUnit_SetRelativeTransformForItem::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRelativeTransformForItem, Value))
	{
		const FTransform ParentTransform = Hierarchy->GetGlobalTransform(Parent, false);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, Value, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, Value, true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetRelativeTransformForItem::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRelativeTransformForItem, Value))
	{
		Value = Hierarchy->GetLocalTransform(InInfo->ControlKey, false);
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetRelativeTranslationForItem::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRelativeTranslationForItem, Value))
	{
		const FTransform ParentTransform = Hierarchy->GetGlobalTransform(Parent, false);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Value), false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Value), true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetRelativeTranslationForItem::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRelativeTranslationForItem, Value))
	{
		Value = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetTranslation();
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetRelativeRotationForItem::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRelativeRotationForItem, Value))
	{
		FTransform ParentTransform = Hierarchy->GetGlobalTransform(Parent, false);
		ParentTransform.SetTranslation(Hierarchy->GetGlobalTransform(Child).GetTranslation());
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Value), false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Value), true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetRelativeRotationForItem::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRelativeRotationForItem, Value))
	{
		Value = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetRotation();
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif
