// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetSpaceInitialTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetSpaceInitialTransform)

FRigUnit_SetSpaceInitialTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey SpaceKey(SpaceName, ERigElementType::Null);
		if (!CachedSpaceIndex.UpdateCache(SpaceKey, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Space '%s' is not valid."), *SpaceName.ToString());
			return;
		}

		FTransform InitialTransform = Transform;
		if (Space == ERigVMTransformSpace::GlobalSpace)
		{
			const FTransform ParentTransform = Hierarchy->GetParentTransformByIndex(CachedSpaceIndex, true);
			InitialTransform = InitialTransform.GetRelativeTransform(ParentTransform);
		}

		Hierarchy->SetInitialLocalTransform(CachedSpaceIndex, InitialTransform);
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetSpaceInitialTransform::GetUpgradeInfo() const
{
	FRigUnit_SetTransform NewNode;
	NewNode.Item = FRigElementKey(SpaceName, ERigElementType::Null);
	NewNode.Space = Space;
	NewNode.Value = Transform;
	NewNode.bInitial = true;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("SpaceName"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Transform"), TEXT("Value"));
	return Info;
}

