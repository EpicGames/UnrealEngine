// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetSpaceTransform.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetSpaceTransform)

FRigUnit_SetSpaceTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey SpaceKey(Space, ERigElementType::Null);

		if (CachedSpaceIndex.UpdateCache(SpaceKey, Hierarchy))
		{
			switch (SpaceType)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					if(FMath::IsNearlyEqual(Weight, 1.f))
					{
						Hierarchy->SetGlobalTransform(CachedSpaceIndex, Transform, true);
					}
					else
					{
						const FTransform PreviousTransform = Hierarchy->GetGlobalTransform(CachedSpaceIndex);
						Hierarchy->SetGlobalTransform(CachedSpaceIndex, FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, FMath::Clamp<float>(Weight, 0.f, 1.f)), true);
					}
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					if(FMath::IsNearlyEqual(Weight, 1.f))
					{
						Hierarchy->SetLocalTransform(CachedSpaceIndex, Transform, true);
					}
					else
					{
						const FTransform PreviousTransform = Hierarchy->GetLocalTransform(CachedSpaceIndex);
						Hierarchy->SetLocalTransform(CachedSpaceIndex, FControlRigMathLibrary::LerpTransform(PreviousTransform, Transform, FMath::Clamp<float>(Weight, 0.f, 1.f)), true);
					}
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_SetSpaceTransform::GetUpgradeInfo() const
{
	FRigUnit_SetTransform NewNode;
	NewNode.Item = FRigElementKey(Space, ERigElementType::Null);
	NewNode.Space = SpaceType;
	NewNode.Value = Transform;
	NewNode.Weight = Weight;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Space"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("SpaceType"), TEXT("Space"));
	Info.AddRemappedPin(TEXT("Transform"), TEXT("Value"));
	return Info;
}

