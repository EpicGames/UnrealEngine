// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_SetTransform.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Hierarchy/RigUnit_SetControlOffset.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SetTransform)

FString FRigUnit_SetTransform::GetUnitLabel() const
{
	FString Initial = bInitial ? TEXT(" Initial") : FString();
	FString Type = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Item.Type).ToString();
	return FString::Printf(TEXT("Set Transform - %s%s"), *Type, *Initial);
}

#if WITH_EDITOR

bool FRigUnit_SetTransform::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetTransform, Value))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			const FTransform ParentTransform = Hierarchy->GetParentTransform(Item, false);
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		}
		else
		{
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, FTransform::Identity, false);
		}
		Hierarchy->SetLocalTransform(InInfo->ControlKey, Value, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, Value, true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetTransform::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetTransform, Value))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			Value = Hierarchy->GetLocalTransform(InInfo->ControlKey, false);
		}
		else
		{
			Value = Hierarchy->GetGlobalTransform(InInfo->ControlKey, false);
		}
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_SetTransform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Weight < SMALL_NUMBER)
	{
		return;
	}

	if (URigHierarchy* Hierarchy = ExecuteContext.Hierarchy)
	{
		if (!CachedIndex.UpdateCache(Item, Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item '%s' is not valid."), *Item.ToString());
		}
		else
		{
#if WITH_EDITOR
			if(bInitial)
			{
				// provide some user feedback when changing initial transforms out of a construction event
				if(!ExecuteContext.IsRunningAConstructionEvent())
				{
					UE_CONTROLRIG_RIGUNIT_LOG_MESSAGE(TEXT("Changing initial transforms during %s is not recommended."), *ExecuteContext.GetEventName().ToString());
				}
			}
#endif

			// for controls - set the control offset transform instead
			if(bInitial && (CachedIndex.GetKey().Type == ERigElementType::Control))
			{
				FTransform TransformMutable = Value;
				FRigUnit_SetControlOffset::StaticExecute(ExecuteContext, CachedIndex.GetKey().Name, TransformMutable, Space, CachedIndex);
				
				if(ExecuteContext.IsRunningAConstructionEvent())
				{
					Hierarchy->SetLocalTransformByIndex(CachedIndex, FTransform::Identity, true, bPropagateToChildren);
					Hierarchy->SetLocalTransformByIndex(CachedIndex, FTransform::Identity, false, bPropagateToChildren);
				}
				return;
			}
			
			FTransform WeightedTransform = Value;
			if (Weight < 1.f - SMALL_NUMBER)
			{
				FTransform PreviousTransform = WeightedTransform;
				switch (Space)
				{
					case ERigVMTransformSpace::GlobalSpace:
					{
						PreviousTransform = Hierarchy->GetGlobalTransformByIndex(CachedIndex, bInitial);
						break;
					}
					case ERigVMTransformSpace::LocalSpace:
					{
						PreviousTransform = Hierarchy->GetLocalTransformByIndex(CachedIndex, bInitial);
						break;
					}
					default:
					{
						break;
					}
				}
				WeightedTransform = FControlRigMathLibrary::LerpTransform(PreviousTransform, WeightedTransform, Weight);
			}

			switch (Space)
			{
				case ERigVMTransformSpace::GlobalSpace:
				{
					Hierarchy->SetGlobalTransformByIndex(CachedIndex, WeightedTransform, bInitial, bPropagateToChildren);

					if (bInitial && ExecuteContext.IsRunningAConstructionEvent())
					{
						Hierarchy->SetGlobalTransformByIndex(CachedIndex, WeightedTransform, false, bPropagateToChildren);
					}
					break;
				}
				case ERigVMTransformSpace::LocalSpace:
				{
					Hierarchy->SetLocalTransformByIndex(CachedIndex, WeightedTransform, bInitial, bPropagateToChildren);

					if (bInitial && ExecuteContext.IsRunningAConstructionEvent())
					{
						Hierarchy->SetLocalTransformByIndex(CachedIndex, WeightedTransform, false, bPropagateToChildren);
					}
					break;
				}
				default:
				{
					break;
				}
			}
		}
	}
}

FString FRigUnit_SetTranslation::GetUnitLabel() const
{
	FString Type = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Item.Type).ToString();
	return FString::Printf(TEXT("Set Translation - %s"), *Type);
}

#if WITH_EDITOR

bool FRigUnit_SetTranslation::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetTranslation, Value))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			const FTransform ParentTransform = Hierarchy->GetParentTransform(Item, false);
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		}
		else
		{
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, FTransform::Identity, false);
		}
		FTransform Transform = FTransform::Identity;
		Transform.SetTranslation(Value);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, Transform, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, Transform, true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetTranslation::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetTranslation, Value))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			Value = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetTranslation();
		}
		else
		{
			Value = Hierarchy->GetGlobalTransform(InInfo->ControlKey, false).GetTranslation();
		}
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_SetTranslation_Execute()
{
	FTransform Transform = FTransform::Identity;
	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Item, Space, bInitial, Transform, CachedIndex);
	Transform.SetLocation(Value);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Item, Space, bInitial, Transform, Weight, bPropagateToChildren, CachedIndex);
}

FString FRigUnit_SetRotation::GetUnitLabel() const
{
	FString Type = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Item.Type).ToString();
	return FString::Printf(TEXT("Set Rotation - %s"), *Type);
}

#if WITH_EDITOR

bool FRigUnit_SetRotation::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRotation, Value))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			const FTransform ParentTransform = Hierarchy->GetParentTransform(Item, false);
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		}
		else
		{
			FTransform OffsetTransform = FTransform::Identity;
			OffsetTransform.SetTranslation(Hierarchy->GetGlobalTransform(Item, false).GetTranslation());
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, false);
		}
		FTransform Transform = FTransform::Identity;
		Transform.SetRotation(Value);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, Transform, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, Transform, true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetRotation::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetRotation, Value))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			Value = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetRotation();
		}
		else
		{
			Value = Hierarchy->GetGlobalTransform(InInfo->ControlKey, false).GetRotation();
		}
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_SetRotation_Execute()
{
	FTransform Transform = FTransform::Identity;
	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Item, Space, bInitial, Transform, CachedIndex);
	Transform.SetRotation(Value);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Item, Space, bInitial, Transform, Weight, bPropagateToChildren, CachedIndex);
}

FString FRigUnit_SetScale::GetUnitLabel() const
{
	FString Type = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Item.Type).ToString();
	return FString::Printf(TEXT("Set Scale - %s"), *Type);
}

#if WITH_EDITOR

bool FRigUnit_SetScale::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetScale, Scale))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			const FTransform ParentTransform = Hierarchy->GetParentTransform(Item, false);
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		}
		else
		{
			FTransform OffsetTransform = FTransform::Identity;
			OffsetTransform.SetTranslation(Hierarchy->GetGlobalTransform(Item, false).GetTranslation());
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, false);
		}
		FTransform Transform = FTransform::Identity;
		Transform.SetScale3D(Scale);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, Transform, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, Transform, true);
		}
		return true;
	}
	return FRigUnitMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SetScale::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetScale, Scale))
	{
		if(Space == ERigVMTransformSpace::LocalSpace)
		{
			Scale = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetScale3D();
		}
		else
		{
			Scale = Hierarchy->GetGlobalTransform(InInfo->ControlKey, false).GetScale3D();
		}
		return true;
	}
	return FRigUnitMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_SetScale_Execute()
{
	FTransform Transform = FTransform::Identity;
	FRigUnit_GetTransform::StaticExecute(ExecuteContext, Item, Space, bInitial, Transform, CachedIndex);
	Transform.SetScale3D(Scale);
	FRigUnit_SetTransform::StaticExecute(ExecuteContext, Item, Space, bInitial, Transform, Weight, bPropagateToChildren, CachedIndex);
}


FRigUnit_SetTransformArray_Execute()
{
	FRigUnit_SetTransformItemArray::StaticExecute(ExecuteContext, Items.Keys, Space, bInitial, Transforms, Weight, bPropagateToChildren, CachedIndex);
}

FRigVMStructUpgradeInfo FRigUnit_SetTransformArray::GetUpgradeInfo() const
{
	FRigUnit_SetTransformItemArray NewNode;
	NewNode.Items = Items.GetKeys();
	NewNode.Space = Space;
	NewNode.bInitial = bInitial;
	NewNode.Transforms = Transforms;
	NewNode.Weight = Weight;
	NewNode.bPropagateToChildren = bPropagateToChildren;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_SetTransformItemArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if(CachedIndex.Num() != Items.Num())
	{
		CachedIndex.Reset();
		CachedIndex.SetNum(Items.Num());
	}

	if(Transforms.Num() != Items.Num())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("The number of transforms (%d) doesn't match the size of the collection (%d)."), Transforms.Num(), Items.Num());
		return;
	}

	for(int32 Index=0;Index<Items.Num();Index++)
	{
		FRigUnit_SetTransform::StaticExecute(ExecuteContext, Items[Index], Space, bInitial, Transforms[Index], Weight, bPropagateToChildren, CachedIndex[Index]);
	}
}
