// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_UnsetCurveValue.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_UnsetCurveValue)

FRigUnit_UnsetCurveValue_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		const FRigElementKey Key(Curve, ERigElementType::Curve);
		if (CachedCurveIndex.UpdateCache(Key, Hierarchy))
		{
			Hierarchy->UnsetCurveValueByIndex(CachedCurveIndex);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_UnsetCurveValue)
{
	const FRigElementKey Curve = Controller->AddCurve(TEXT("Curve"), 0.f);
	
	Hierarchy->ResetCurveValues();
	Unit.Curve = TEXT("Curve");
	Execute();

	AddErrorIfFalse(!Hierarchy->IsCurveValueSet(Curve), TEXT("unexpected value"));

	return true;
}
#endif

