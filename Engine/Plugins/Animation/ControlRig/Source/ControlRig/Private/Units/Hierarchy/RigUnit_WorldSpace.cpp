// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Hierarchy/RigUnit_WorldSpace.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_WorldSpace)

FRigUnit_ToWorldSpace_Transform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    World = ExecuteContext.ToWorldSpace(Value);
}

FRigUnit_ToRigSpace_Transform_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    Global = ExecuteContext.ToVMSpace(Value);
}

FRigUnit_ToWorldSpace_Location_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    World = ExecuteContext.ToWorldSpace(Value);
}

FRigUnit_ToRigSpace_Location_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    Global = ExecuteContext.ToVMSpace(Value);
}

FRigUnit_ToWorldSpace_Rotation_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    World = ExecuteContext.ToWorldSpace(Value);
}

FRigUnit_ToRigSpace_Rotation_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
    Global = ExecuteContext.ToVMSpace(Value);
}
