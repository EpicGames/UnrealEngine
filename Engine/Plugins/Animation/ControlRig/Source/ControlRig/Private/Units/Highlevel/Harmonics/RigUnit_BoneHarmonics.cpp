// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Harmonics/RigUnit_BoneHarmonics.h"
#include "Units/RigUnitContext.h"
#include "AnimationCoreLibrary.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_BoneHarmonics)

FRigUnit_BoneHarmonics_Execute()
{
	TArray<FRigUnit_Harmonics_TargetItem> Targets;
	for(int32 BoneIndex = 0;BoneIndex<Bones.Num();BoneIndex++)
	{
		FRigUnit_Harmonics_TargetItem Target;
		Target.Item = FRigElementKey(Bones[BoneIndex].Bone, ERigElementType::Bone);
		Target.Ratio = Bones[BoneIndex].Ratio;
		Targets.Add(Target);
	}


	FRigUnit_ItemHarmonics::StaticExecute(
		ExecuteContext, 
		Targets,
		WaveSpeed,
		WaveFrequency,
		WaveAmplitude,
		WaveOffset,
		WaveNoise,
		WaveEase,
		WaveMinimum,
		WaveMaximum,
		RotationOrder,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_BoneHarmonics::GetUpgradeInfo() const
{
	FRigUnit_ItemHarmonics NewNode;
	
	for(int32 BoneIndex = 0;BoneIndex<Bones.Num();BoneIndex++)
	{
		FRigUnit_Harmonics_TargetItem Target;
		Target.Item = FRigElementKey(Bones[BoneIndex].Bone, ERigElementType::Bone);
		Target.Ratio = Bones[BoneIndex].Ratio;
		NewNode.Targets.Add(Target);
	}

	NewNode.WaveSpeed = WaveSpeed;
	NewNode.WaveFrequency = WaveFrequency;
	NewNode.WaveAmplitude = WaveAmplitude;
	NewNode.WaveOffset = WaveOffset;
	NewNode.WaveNoise = WaveNoise;
	NewNode.WaveEase = WaveEase;
	NewNode.WaveMinimum = WaveMinimum;
	NewNode.WaveMaximum = WaveMaximum;
	NewNode.RotationOrder = RotationOrder;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	for(int32 BoneIndex = 0;BoneIndex<Bones.Num();BoneIndex++)
	{
		Info.AddRemappedPin(
			FString::Printf(TEXT("Bones.%d.Bone"), BoneIndex),
			FString::Printf(TEXT("Targets.%d.Item.Name"), BoneIndex));
	}
	return Info;
}

FRigUnit_ItemHarmonics_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	FVector& WaveTime = WorkData.WaveTime;

	if (CachedItems.Num() != Targets.Num())
	{
		CachedItems.Reset(Targets.Num());
		
		WaveTime = FVector::ZeroVector;

	if (CachedItems.IsEmpty())
		for (int32 ItemIndex = 0; ItemIndex < Targets.Num(); ItemIndex++)
		{
			FCachedRigElement CachedItem(Targets[ItemIndex].Item, Hierarchy);
			if (!CachedItem.IsValid())
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item not found."));
			}
			CachedItems.Add(CachedItem);
		}
	}

	for (int32 ItemIndex = 0; ItemIndex < CachedItems.Num(); ItemIndex++)
	{
		if (CachedItems[ItemIndex].IsValid())
		{
			float Ease = FMath::Clamp<float>(Targets[ItemIndex].Ratio, 0.f, 1.f);
			Ease = FControlRigMathLibrary::EaseFloat(Ease, WaveEase);
			Ease = FMath::Lerp<float>(WaveMinimum, WaveMaximum, Ease);

			FVector U = WaveTime + WaveFrequency * Targets[ItemIndex].Ratio;

			FVector Noise;
			Noise.X = FMath::PerlinNoise1D(U.X + 132.4f);
			Noise.Y = FMath::PerlinNoise1D(U.Y + 9.2f);
			Noise.Z = FMath::PerlinNoise1D(U.Z + 217.9f);
			Noise = Noise * WaveNoise * 2.f;
			U = U + Noise;

			FVector Angles;
			Angles.X = FMath::Sin(U.X + WaveOffset.X);
			Angles.Y = FMath::Sin(U.Y + WaveOffset.Y);
			Angles.Z = FMath::Sin(U.Z + WaveOffset.Z);
			Angles = Angles * WaveAmplitude * Ease;

			FQuat Rotation = AnimationCore::QuatFromEuler(Angles, RotationOrder);

			FTransform Transform = Hierarchy->GetGlobalTransform(CachedItems[ItemIndex]);
			Transform.SetRotation(Transform.GetRotation() * Rotation);
			Hierarchy->SetGlobalTransform(CachedItems[ItemIndex], Transform);
		}
	}

	WaveTime += WaveSpeed * ExecuteContext.GetDeltaTime();
}

