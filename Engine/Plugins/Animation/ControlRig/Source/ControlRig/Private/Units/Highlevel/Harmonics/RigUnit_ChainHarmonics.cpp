// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Harmonics/RigUnit_ChainHarmonics.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ChainHarmonics)

FRigUnit_ChainHarmonics_Execute()
{
	FRigUnit_ChainHarmonicsPerItem::StaticExecute(
		ExecuteContext, 
		FRigElementKey(ChainRoot, ERigElementType::Bone),
		Speed,
		Reach,
		Wave,
		WaveCurve,
		Pendulum,
		bDrawDebug,
		DrawWorldOffset,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_ChainHarmonics::GetUpgradeInfo() const
{
	FRigUnit_ChainHarmonicsPerItem NewNode;
	NewNode.ChainRoot = FRigElementKey(ChainRoot, ERigElementType::Bone);
	NewNode.Speed = Speed;
	NewNode.Reach = Reach;
	NewNode.Wave = Wave;
	NewNode.WaveCurve = WaveCurve;
	NewNode.Pendulum = Pendulum;
	NewNode.bDrawDebug = bDrawDebug;
	NewNode.DrawWorldOffset = DrawWorldOffset;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("ChainRoot"), TEXT("ChainRoot.Name"));
	return Info;
}

FRigUnit_ChainHarmonicsPerItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	FVector& Time = WorkData.Time;
	TArray<FCachedRigElement>& Items = WorkData.Items;
	TArray<float>& Ratio = WorkData.Ratio;
	TArray<FVector>& LocalTip = WorkData.LocalTip;
	TArray<FVector>& PendulumTip = WorkData.PendulumTip;
	TArray<FVector>& PendulumPosition = WorkData.PendulumPosition;
	TArray<FVector>& PendulumVelocity = WorkData.PendulumVelocity;
	TArray<FVector>& HierarchyLine = WorkData.HierarchyLine;
	TArray<FVector>& VelocityLines = WorkData.VelocityLines;
	
	if(Items.Num() == 0)
	{
		if (!ChainRoot.IsValid())
		{
			return;
		}

		Items.Add(FCachedRigElement(ChainRoot, Hierarchy));

		TArray<FRigElementKey> Children = Hierarchy->GetChildren(Items.Last().GetKey());
		while (Children.Num() > 0)
		{
			Items.Add(FCachedRigElement(Children[0], Hierarchy));
			Children = Hierarchy->GetChildren(Children[0]);
		}

		if (Items.Num() < 2)
		{
			Items.Reset();
			return;
		}

		Ratio.SetNumZeroed(Items.Num());
		LocalTip.SetNumZeroed(Items.Num());
		PendulumTip.SetNumZeroed(Items.Num());
		PendulumPosition.SetNumZeroed(Items.Num());
		PendulumVelocity.SetNumZeroed(Items.Num());
		VelocityLines.SetNumZeroed(Items.Num() * 2);

		for (int32 Index = 0; Index < Items.Num(); Index++)
		{
			Ratio[Index] = float(Index) / float(Items.Num() - 1);
			LocalTip[Index] = Hierarchy->GetLocalTransform(Items[Index]).GetLocation();
			PendulumPosition[Index] = Hierarchy->GetGlobalTransform(Items[Index]).GetLocation();
		}
		
		for (int32 Index = 0; Index < Items.Num() - 1; Index++)
		{
			PendulumTip[Index] = LocalTip[Index + 1];
		}
		PendulumTip[PendulumTip.Num() - 1] = PendulumTip[PendulumTip.Num() - 2];

		for (int32 Index = 0; Index < Items.Num(); Index++)
		{
			PendulumPosition[Index] = Hierarchy->GetGlobalTransform(Items[Index]).TransformPosition(PendulumTip[Index]);
		}
	}
	
	if (Items.Num() < 2)
	{
		return;
	}

	FTransform ParentTransform = FTransform::Identity;
	FRigElementKey ParentKey = Hierarchy->GetFirstParent(Items[0].GetKey());
	if (ParentKey.IsValid())
	{
		ParentTransform = Hierarchy->GetGlobalTransform(ParentKey);
	}

	for (int32 Index = 0;Index < Items.Num(); Index++)
	{
		FTransform GlobalTransform = Hierarchy->GetLocalTransform(Items[Index]) * ParentTransform;
		FQuat Rotation = GlobalTransform.GetRotation();

		if (Reach.bEnabled)
		{
			float Ease = FMath::Lerp<float>(Reach.ReachMinimum, Reach.ReachMaximum, Ratio[Index]);;
			Ease = FControlRigMathLibrary::EaseFloat(Ease, Reach.ReachEase);

			FVector Axis = Reach.ReachAxis;
			Axis = ParentTransform.TransformVectorNoScale(Axis);

			FVector ReachDirection = (Reach.ReachTarget - GlobalTransform.GetLocation()).GetSafeNormal();
			ReachDirection = FMath::Lerp<FVector>(Axis, ReachDirection, Ease);

			FQuat ReachRotation = FControlRigMathLibrary::FindQuatBetweenVectors(Axis, ReachDirection);
			Rotation = (ReachRotation * Rotation).GetNormalized();
		}

		if (Wave.bEnabled)
		{
			float Ease = FMath::Lerp<float>(Wave.WaveMinimum, Wave.WaveMaximum, Ratio[Index]);;
			Ease = FControlRigMathLibrary::EaseFloat(Ease, Wave.WaveEase);

			float Curve = WaveCurve.GetRichCurveConst()->Eval(Ratio[Index], 1.f);

			FVector U = Time + Wave.WaveFrequency * Ratio[Index];

			FVector Noise;
			Noise.X = FMath::PerlinNoise1D(U.X + 132.4f);
			Noise.Y = FMath::PerlinNoise1D(U.Y + 9.2f);
			Noise.Z = FMath::PerlinNoise1D(U.Z + 217.9f);
			Noise = Noise * Wave.WaveNoise * 2.f;
			U = U + Noise;

			FVector Angles;
			Angles.X = FMath::Sin(U.X + Wave.WaveOffset.X);
			Angles.Y = FMath::Sin(U.Y + Wave.WaveOffset.Y);
			Angles.Z = FMath::Sin(U.Z + Wave.WaveOffset.Z);
			Angles = Angles * Wave.WaveAmplitude * Ease * Curve;

			Rotation = Rotation * FQuat(FVector(1.f, 0.f, 0.f), Angles.X);
			Rotation = Rotation * FQuat(FVector(0.f, 1.f, 0.f), Angles.Y);
			Rotation = Rotation * FQuat(FVector(0.f, 0.f, 1.f), Angles.Z);
			Rotation = Rotation.GetNormalized();
		}

		if (Pendulum.bEnabled)
		{
			FQuat NonSimulatedRotation = Rotation;

			float Ease = FMath::Lerp<float>(Pendulum.PendulumMinimum, Pendulum.PendulumMaximum, Ratio[Index]);;
			Ease = FControlRigMathLibrary::EaseFloat(Ease, Pendulum.PendulumEase);

			FVector Stiffness = LocalTip[Index];
			FVector Upvector = Pendulum.UnwindAxis;
			float Length = Stiffness.Size();
			Stiffness = ParentTransform.TransformVectorNoScale(Stiffness);
			Upvector = ParentTransform.TransformVectorNoScale(Upvector);

			FVector Velocity = Pendulum.PendulumGravity;
			Velocity += Stiffness * Pendulum.PendulumStiffness;

			if (ExecuteContext.GetDeltaTime() > 0.f)
			{
				PendulumVelocity[Index] = FMath::Lerp<FVector>(PendulumVelocity[Index], Velocity, FMath::Clamp<float>(Pendulum.PendulumBlend, 0.f, 0.999f));
				PendulumVelocity[Index] = PendulumVelocity[Index] * Pendulum.PendulumDrag;

				FVector PrevPosition = PendulumPosition[Index];
				PendulumPosition[Index] = PendulumPosition[Index] + PendulumVelocity[Index] * ExecuteContext.GetDeltaTime();
				PendulumPosition[Index] = GlobalTransform.GetLocation() + (PendulumPosition[Index] - GlobalTransform.GetLocation()).GetSafeNormal() * Length;
				PendulumVelocity[Index] = (PendulumPosition[Index] - PrevPosition) / ExecuteContext.GetDeltaTime();
			}

			VelocityLines[Index * 2 + 0] = PendulumPosition[Index];
			VelocityLines[Index * 2 + 1] = PendulumPosition[Index] + PendulumVelocity[Index] * 0.1f;

			FQuat PendulumRotation = FControlRigMathLibrary::FindQuatBetweenVectors(Rotation.RotateVector(LocalTip[Index]), PendulumPosition[Index] - GlobalTransform.GetLocation());
			Rotation = (PendulumRotation * Rotation).GetNormalized();

			float Unwind = FMath::Lerp<float>(Pendulum.UnwindMinimum, Pendulum.UnwindMaximum, Ratio[Index]);
			FVector CurrentUpvector = Rotation.RotateVector(Pendulum.UnwindAxis);
			CurrentUpvector = CurrentUpvector - FVector::DotProduct(CurrentUpvector, Rotation.RotateVector(LocalTip[Index]).GetSafeNormal());
			CurrentUpvector = FMath::Lerp<FVector>(Upvector, CurrentUpvector, Unwind);
			FQuat UnwindRotation = FControlRigMathLibrary::FindQuatBetweenVectors(CurrentUpvector, Upvector);
			Rotation = (UnwindRotation * Rotation).GetNormalized();

			Rotation = FQuat::Slerp(NonSimulatedRotation, Rotation, FMath::Clamp<float>(Ease, 0.f, 1.f));
		}

		GlobalTransform.SetRotation(Rotation);
		Hierarchy->SetGlobalTransform(Items[Index], GlobalTransform);
		ParentTransform = GlobalTransform;
	}

	Time = Time + Speed * ExecuteContext.GetDeltaTime();

	if (ExecuteContext.GetDrawInterface() != nullptr && bDrawDebug)
	{
		HierarchyLine.SetNum(Items.Num());
		for (int32 Index = 0; Index < Items.Num(); Index++)
		{
			HierarchyLine[Index] = Hierarchy->GetGlobalTransform(Items[Index]).GetLocation();
		}

		ExecuteContext.GetDrawInterface()->DrawLineStrip(DrawWorldOffset, HierarchyLine, FLinearColor::Yellow, 0.f);
		ExecuteContext.GetDrawInterface()->DrawLines(DrawWorldOffset, VelocityLines, FLinearColor(0.3f, 0.3f, 1.f), 0.f);
		ExecuteContext.GetDrawInterface()->DrawPoints(DrawWorldOffset, PendulumPosition, 3.f, FLinearColor::Blue);
		return;
	}

}
