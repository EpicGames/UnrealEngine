// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_AimBone.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "AnimationCoreLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AimBone)

FRigUnit_AimBoneMath_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	Result = InputTransform;

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!bIsInitialized)
	{
		PrimaryCachedSpace.Reset();
		SecondaryCachedSpace.Reset();
		bIsInitialized = true;
	}

	if ((Weight <= SMALL_NUMBER) || (Primary.Weight <= SMALL_NUMBER && Secondary.Weight <= SMALL_NUMBER))
	{
		return;
	}
	
	if (Primary.Weight > SMALL_NUMBER)
	{
		FVector Target = Primary.Target;

		if (PrimaryCachedSpace.UpdateCache(Primary.Space, Hierarchy))
		{
			FTransform Space = Hierarchy->GetGlobalTransform(PrimaryCachedSpace);
			if (Primary.Kind == EControlRigVectorKind::Direction)
			{
				Target = Space.TransformVectorNoScale(Target);
			}
			else
			{
				Target = Space.TransformPositionNoScale(Target);
			}
		}

		if (ExecuteContext.GetDrawInterface() != nullptr && DebugSettings.bEnabled)
		{
			const FLinearColor Color = FLinearColor(0.f, 1.f, 1.f, 1.f);
			if (Primary.Kind == EControlRigVectorKind::Direction)
			{
				ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, Result.GetLocation(), Result.GetLocation() + Target * DebugSettings.Scale, Color);
			}
			else
			{
				ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, Result.GetLocation(), Target, Color);
				ExecuteContext.GetDrawInterface()->DrawBox(DebugSettings.WorldOffset, FTransform(FQuat::Identity, Target, FVector(1.f, 1.f, 1.f) * DebugSettings.Scale * 0.1f), Color);
			}
		}

		if (Primary.Kind == EControlRigVectorKind::Location)
		{
			Target = Target - Result.GetLocation();
		}

		if (!Target.IsNearlyZero() && !Primary.Axis.IsNearlyZero())
		{
			Target = Target.GetSafeNormal();
			FVector Axis = Result.TransformVectorNoScale(Primary.Axis).GetSafeNormal();
			float T = Primary.Weight * Weight;
			if (T < 1.f - SMALL_NUMBER)
			{
				Target = FMath::Lerp<FVector>(Axis, Target, T).GetSafeNormal();
			}
			FQuat Rotation = FControlRigMathLibrary::FindQuatBetweenVectors(Axis, Target);
			Result.SetRotation((Rotation * Result.GetRotation()).GetNormalized());
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Invalid primary target."));
		}
	}

	if (Secondary.Weight > SMALL_NUMBER)
	{
		FVector Target = Secondary.Target;

		if (SecondaryCachedSpace.UpdateCache(Secondary.Space, Hierarchy))
		{
			FTransform Space = Hierarchy->GetGlobalTransform(SecondaryCachedSpace);
			if (Secondary.Kind == EControlRigVectorKind::Direction)
			{
				Target = Space.TransformVectorNoScale(Target);
			}
			else
			{
				Target = Space.TransformPositionNoScale(Target);
			}
		}

		if (ExecuteContext.GetDrawInterface() != nullptr && DebugSettings.bEnabled)
		{
			const FLinearColor Color = FLinearColor(0.f, 0.2f, 1.f, 1.f);
			if (Secondary.Kind == EControlRigVectorKind::Direction)
			{
				ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, Result.GetLocation(), Result.GetLocation() + Target * DebugSettings.Scale, Color);
			}
			else
			{
				ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, Result.GetLocation(), Target, Color);
				ExecuteContext.GetDrawInterface()->DrawBox(DebugSettings.WorldOffset, FTransform(FQuat::Identity, Target, FVector(1.f, 1.f, 1.f) * DebugSettings.Scale * 0.1f), Color);
			}
		}

		if (Secondary.Kind == EControlRigVectorKind::Location)
		{
			Target = Target - Result.GetLocation();
		}

		FVector PrimaryAxis = Primary.Axis;
		if (!PrimaryAxis.IsNearlyZero())
		{
			PrimaryAxis = Result.TransformVectorNoScale(Primary.Axis).GetSafeNormal();
			Target = Target - FVector::DotProduct(Target, PrimaryAxis) * PrimaryAxis;
		}

		if (!Target.IsNearlyZero() && !Secondary.Axis.IsNearlyZero())
		{
			Target = Target.GetSafeNormal();

			FVector Axis = Result.TransformVectorNoScale(Secondary.Axis).GetSafeNormal();
			float T = Secondary.Weight * Weight;
			if (T < 1.f - SMALL_NUMBER)
			{
				Target = FMath::Lerp<FVector>(Axis, Target, T).GetSafeNormal();
			}
			
			FQuat Rotation;
			if (FVector::DotProduct(Axis,Target) + 1.f < SMALL_NUMBER && !PrimaryAxis.IsNearlyZero())
			{
				// special case, when the axis and target and 180 degrees apart and there is a primary axis
				 Rotation = FQuat(PrimaryAxis, PI);
			}
			else
			{
				Rotation = FControlRigMathLibrary::FindQuatBetweenVectors(Axis, Target);
			}
			Result.SetRotation((Rotation * Result.GetRotation()).GetNormalized());
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Invalid secondary target."));
		}
	}
}

FRigUnit_AimBone_Execute()
{
	FRigUnit_AimItem_Target PrimaryTargetItem;
	PrimaryTargetItem.Weight = Primary.Weight;
	PrimaryTargetItem.Axis = Primary.Axis;
	PrimaryTargetItem.Target = Primary.Target;
	PrimaryTargetItem.Kind = Primary.Kind;
	PrimaryTargetItem.Space = FRigElementKey(Primary.Space, ERigElementType::Bone);

	FRigUnit_AimItem_Target SecondaryTargetItem;
	SecondaryTargetItem.Weight = Secondary.Weight;
	SecondaryTargetItem.Axis = Secondary.Axis;
	SecondaryTargetItem.Target = Secondary.Target;
	SecondaryTargetItem.Kind = Secondary.Kind;
	SecondaryTargetItem.Space = FRigElementKey(Secondary.Space, ERigElementType::Bone);

	FRigUnit_AimItem::StaticExecute(
		ExecuteContext,
		FRigElementKey(Bone, ERigElementType::Bone),
		PrimaryTargetItem,
		SecondaryTargetItem,
		Weight,
		DebugSettings,
		CachedBoneIndex,
		PrimaryCachedSpace,
		SecondaryCachedSpace,
		bIsInitialized);
}

FRigVMStructUpgradeInfo FRigUnit_AimBone::GetUpgradeInfo() const
{
	FRigUnit_AimItem NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.Primary.Weight = Primary.Weight;
	NewNode.Primary.Axis = Primary.Axis;
	NewNode.Primary.Target = Primary.Target;
	NewNode.Primary.Kind = Primary.Kind;
	NewNode.Primary.Space = FRigElementKey(Primary.Space, ERigElementType::Bone);
	NewNode.Secondary.Weight = Secondary.Weight;
	NewNode.Secondary.Axis = Secondary.Axis;
	NewNode.Secondary.Target = Secondary.Target;
	NewNode.Secondary.Kind = Secondary.Kind;
	NewNode.Secondary.Space = FRigElementKey(Secondary.Space, ERigElementType::Bone);
	NewNode.Weight = Weight;
	NewNode.DebugSettings = DebugSettings;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("Primary.Space"), TEXT("Primary.Space.Name"));
	Info.AddRemappedPin(TEXT("Secondary.Space"), TEXT("Secondary.Space.Name"));
	return Info;
}

#if WITH_EDITOR

bool FRigUnit_AimItem::GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const
{
	static TArray<FRigDirectManipulationTarget> Targets = {
		{ TEXT("Primary Target"), ERigControlType::Position },
		{ TEXT("Secondary Target"), ERigControlType::Position },
		{ TEXT("Resulting Transform"), ERigControlType::EulerTransform }
	};
	InOutTargets.Append(Targets);
	return true;
}

TArray<const URigVMPin*> FRigUnit_AimItem::GetPinsForDirectManipulation(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const
{
	if(InTarget.Name == TEXT("Primary Target"))
	{
		return {
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Primary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Kind)),
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Primary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Target)),
		};
	}
	if(InTarget.Name == TEXT("Secondary Target"))
	{
		return {
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Secondary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Kind)),
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Secondary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Target)),
		};
	}
	if(InTarget.Name == TEXT("Resulting Transform"))
	{
		return {
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Primary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Target)),
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Secondary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Target)),
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Primary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Kind)),
			InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem, Secondary))->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AimItem_Target, Kind)),
		};
	}
	return FRigUnit_HighlevelBaseMutable::GetPinsForDirectManipulation(InNode, InTarget);
}

bool FRigUnit_AimItem::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if(Hierarchy == nullptr)
	{
		return false;
	}
	
	if(InInfo->Target.Name == TEXT("Primary Target"))
	{
		const FTransform ParentTransform = Hierarchy->GetGlobalTransform(Primary.Space, false);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Primary.Target), false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Primary.Target), true);
		}
		return true;
	}
	if(InInfo->Target.Name == TEXT("Secondary Target"))
	{
		const FTransform ParentTransform = Hierarchy->GetGlobalTransform(Secondary.Space, false);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Secondary.Target), false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(Primary.Target), true);
		}
		return true;
	}
	if(InInfo->Target.Name == TEXT("Resulting Transform"))
	{
		// execute the aim
		Execute(InContext);
		
		const FTransform ItemTransform = Hierarchy->GetGlobalTransform(CachedItem);
		
		if(!InInfo->bInitialized)
		{
			InInfo->OffsetTransform = ItemTransform;
			
			const FTransform PrimaryParentTransform = Hierarchy->GetGlobalTransform(Primary.Space, false);
			const FTransform SecondaryParentTransform = Hierarchy->GetGlobalTransform(Secondary.Space, false);
			InInfo->Transforms = { FTransform(Primary.Target), FTransform(Secondary.Target) };
			InInfo->Transforms[0] = InInfo->Transforms[0] * PrimaryParentTransform;
			InInfo->Transforms[1] = InInfo->Transforms[1] * SecondaryParentTransform;
			InInfo->Transforms[0] = InInfo->Transforms[0].GetRelativeTransform(ItemTransform);
			InInfo->Transforms[1] = InInfo->Transforms[1].GetRelativeTransform(ItemTransform);
			InInfo->Transforms[0].NormalizeRotation();
			InInfo->Transforms[1].NormalizeRotation();

			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, InInfo->OffsetTransform, false);
			Hierarchy->SetGlobalTransform(InInfo->ControlKey, FTransform::Identity, false);
			Hierarchy->SetGlobalTransform(InInfo->ControlKey, FTransform::Identity, true);
		}
		else
		{
			Hierarchy->SetGlobalTransform(InInfo->ControlKey, ItemTransform, false);
		}
		return true;
	}
	return FRigUnit_HighlevelBaseMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_AimItem::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if(Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == TEXT("Primary Target"))
	{
		Primary.Kind = EControlRigVectorKind::Location;
		Primary.Target = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetTranslation();
		return true;
	}
	if(InInfo->Target.Name == TEXT("Secondary Target"))
	{
		Primary.Kind = EControlRigVectorKind::Location;
		Secondary.Target = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetTranslation();
		return true;
	}
	if(InInfo->Target.Name == TEXT("Resulting Transform"))
	{
		Primary.Kind = EControlRigVectorKind::Location;
		Secondary.Kind = EControlRigVectorKind::Location;
		const FTransform ControlTransform = Hierarchy->GetGlobalTransform(InInfo->ControlKey);
		const FTransform GlobalPrimaryTransform = InInfo->Transforms[0] * ControlTransform;
		const FTransform GlobalSecondaryTransform = InInfo->Transforms[1] * ControlTransform;
		const FTransform LocalPrimaryTransform = GlobalPrimaryTransform.GetRelativeTransform(Hierarchy->GetGlobalTransform(Primary.Space));
		const FTransform LocalSecondaryTransform = GlobalSecondaryTransform.GetRelativeTransform(Hierarchy->GetGlobalTransform(Secondary.Space));
		Primary.Target = LocalPrimaryTransform.GetTranslation();
		Secondary.Target = LocalSecondaryTransform.GetTranslation();
		return true;
	}
	return FRigUnit_HighlevelBaseMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_AimItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!bIsInitialized)
	{
		CachedItem.Reset();
		PrimaryCachedSpace.Reset();
		SecondaryCachedSpace.Reset();
		bIsInitialized = true;
	}

	if (!CachedItem.UpdateCache(Item, Hierarchy))
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item not found '%s'."), *Item.ToString());
		return;
	}

	if ((Weight <= SMALL_NUMBER) || (Primary.Weight <= SMALL_NUMBER && Secondary.Weight <= SMALL_NUMBER))
	{
		return;
	}

	FTransform Transform = Hierarchy->GetGlobalTransform(CachedItem);

	FRigUnit_AimBoneMath::StaticExecute(
		ExecuteContext,
		Transform,
		Primary,
		Secondary,
		Weight,
		Transform,
		DebugSettings,
		PrimaryCachedSpace,
		SecondaryCachedSpace,
		bIsInitialized);

	Hierarchy->SetGlobalTransform(CachedItem, Transform);
}


FRigUnit_AimConstraintLocalSpaceOffset_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	if(!bIsInitialized)
	{
		WorldUpSpaceCache.Reset();
		ChildCache.Reset();
		ParentCaches.Reset();
		bIsInitialized = true;
	}
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		WorldUpSpaceCache.UpdateCache(WorldUp.Space, Hierarchy);
		
		if (!ChildCache.UpdateCache(Child, Hierarchy))
		{
			return;
		}
		
		if(ParentCaches.Num() != Parents.Num())
		{
			ParentCaches.SetNumZeroed(Parents.Num());
		}
		
		float OverallWeight = 0;
		for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
		{
			const FConstraintParent& Parent = Parents[ParentIndex];
			if (!ParentCaches[ParentIndex].UpdateCache(Parent.Item, Hierarchy))
			{
				continue;
			}
			
			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		const float WeightNormalizer = 1.0f / OverallWeight;

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			FTransform AdditionalOffsetTransform;
			
			const bool bChildIsControl = Child.Type == ERigElementType::Control;
			if(bChildIsControl)
			{
				if (FRigControlElement* ChildAsControlElement = Hierarchy->Get<FRigControlElement>(ChildCache))
				{
					AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::InitialLocal);
				}
			}
			
			FQuat OffsetRotation = FQuat::Identity;
			if (bMaintainOffset)
			{
				FVector MixedInitialGlobalPosition = FVector::ZeroVector;

				for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
				{
					if (!ParentCaches[ParentIndex].IsValid())
					{
						continue;
					}
					
					const FConstraintParent& Parent = Parents[ParentIndex];
					
					const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
					if (ClampedWeight < KINDA_SMALL_NUMBER)
					{
						continue;
					}

					const float NormalizedWeight = ClampedWeight * WeightNormalizer;

					FTransform ParentInitialGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], true);

					MixedInitialGlobalPosition += ParentInitialGlobalTransform.GetLocation() * NormalizedWeight;
				}

				// points the aim axis towards the parents
				FRigUnit_AimItem_Target Primary;
				Primary.Axis = AimAxis;
				Primary.Target = MixedInitialGlobalPosition;
				Primary.Kind = EControlRigVectorKind::Location;
				Primary.Space = FRigElementKey();
				FCachedRigElement PrimaryCachedSpace;

				// points the up axis towards the world up target
				FRigUnit_AimItem_Target Secondary;
				Secondary.Axis = UpAxis;

				FVector InitialWorldSpaceUp = WorldUp.Target;

				FTransform SpaceInitialTransform = Hierarchy->GetInitialGlobalTransform(WorldUpSpaceCache);
				if (Secondary.Kind == EControlRigVectorKind::Direction)
				{
					InitialWorldSpaceUp = SpaceInitialTransform.TransformVectorNoScale(InitialWorldSpaceUp);
				}
				else
				{
					InitialWorldSpaceUp = SpaceInitialTransform.TransformPositionNoScale(InitialWorldSpaceUp);
				}	
			
				Secondary.Target = InitialWorldSpaceUp;
				Secondary.Kind = WorldUp.Kind;
				// we don't want to reference the space any more since its transform
				// is already included in InitialWorldSpaceUp 
				Secondary.Space = FRigElementKey();
				FCachedRigElement SecondaryCachedSpace;

			
				FTransform ChildInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(ChildCache);
				FTransform InitialAimResult = ChildInitialGlobalTransform;
				FRigUnit_AimBone_DebugSettings DummyDebugSettings;
				FRigUnit_AimBoneMath::StaticExecute(
					ExecuteContext,
					ChildInitialGlobalTransform, // optional
					Primary,
					Secondary,
					Weight,
					InitialAimResult,
					DummyDebugSettings,
					PrimaryCachedSpace,
					SecondaryCachedSpace,
					bIsInitialized);

				FTransform ChildParentInitialGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, true);
				FQuat MixedInitialLocalRotation = ChildParentInitialGlobalTransform.GetRotation().Inverse() * InitialAimResult.GetRotation();

				FTransform ChildInitialLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, true);

				// Controls need to be handled a bit differently
				if (bChildIsControl)
				{
					ChildInitialLocalTransform *= AdditionalOffsetTransform;
				}
				
				FQuat ChildInitialLocalRotation = ChildInitialLocalTransform.GetRotation();
			
				OffsetRotation = MixedInitialLocalRotation.Inverse() * ChildInitialLocalRotation;
			}

			FVector MixedGlobalPosition = FVector::ZeroVector;

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				if (!ParentCaches[ParentIndex].IsValid())
				{
					continue;
				}
					
				const FConstraintParent& Parent = Parents[ParentIndex];
					
				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;

				FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], false);
				MixedGlobalPosition += ParentCurrentGlobalTransform.GetLocation() * NormalizedWeight;
			}

			// points the aim axis towards the parents
			FRigUnit_AimItem_Target Primary;
			Primary.Axis = AimAxis;
			Primary.Target = MixedGlobalPosition;
			Primary.Kind = EControlRigVectorKind::Location;
			Primary.Space = FRigElementKey();
			FCachedRigElement PrimaryCachedSpace;

		
			// points the up axis towards the world up target
			FRigUnit_AimItem_Target Secondary;
			Secondary.Axis = UpAxis;
			Secondary.Target = WorldUp.Target;
			Secondary.Kind = WorldUp.Kind;
			Secondary.Space = WorldUp.Space;
			
			FTransform ChildGlobalTransform = Hierarchy->GetGlobalTransform(ChildCache);
			FTransform AimResult = ChildGlobalTransform;
			FRigUnit_AimBoneMath::StaticExecute(
				ExecuteContext,
				ChildGlobalTransform, // optional
				Primary,
				Secondary,
				Weight,
				AimResult,
				AdvancedSettings.DebugSettings,
				PrimaryCachedSpace,
				WorldUpSpaceCache,
				bIsInitialized);	

			// handle filtering, performed in local space
			FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, false);
			FQuat MixedLocalRotation = ChildParentGlobalTransform.GetRotation().Inverse() * AimResult.GetRotation();

			if (bMaintainOffset)
			{
				MixedLocalRotation = MixedLocalRotation * OffsetRotation;
			}

			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, false);
				
			// controls need to be handled a bit differently
			if (bChildIsControl)
			{
				ChildCurrentLocalTransform *= AdditionalOffsetTransform;
			}

			FQuat FilteredMixedLocalRotation = MixedLocalRotation;
			
			if(!Filter.HasNoEffect())
			{
				FVector MixedEulerRotation = AnimationCore::EulerFromQuat(MixedLocalRotation, AdvancedSettings.RotationOrderForFilter);

				FVector MixedEulerRotation2 = FControlRigMathLibrary::GetEquivalentEulerAngle(MixedEulerRotation, AdvancedSettings.RotationOrderForFilter);

				FQuat ChildRotation = ChildCurrentLocalTransform.GetRotation();
				FVector ChildEulerRotation = AnimationCore::EulerFromQuat(ChildRotation, AdvancedSettings.RotationOrderForFilter);

				FVector ClosestMixedEulerRotation = FControlRigMathLibrary::ChooseBetterEulerAngleForAxisFilter(ChildEulerRotation, MixedEulerRotation, MixedEulerRotation2	);
				
				FVector FilteredMixedEulerRotation;
				FilteredMixedEulerRotation.X = Filter.bX ? ClosestMixedEulerRotation.X : ChildEulerRotation.X;
            	FilteredMixedEulerRotation.Y = Filter.bY ? ClosestMixedEulerRotation.Y : ChildEulerRotation.Y;
            	FilteredMixedEulerRotation.Z = Filter.bZ ? ClosestMixedEulerRotation.Z : ChildEulerRotation.Z;

				FilteredMixedLocalRotation = AnimationCore::QuatFromEuler(FilteredMixedEulerRotation, AdvancedSettings.RotationOrderForFilter);
			}

			FTransform FilteredMixedLocalTransform = ChildCurrentLocalTransform;

			FilteredMixedLocalTransform.SetRotation(FilteredMixedLocalRotation);

			FTransform FinalLocalTransform = FilteredMixedLocalTransform;

			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				FinalLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, FinalLocalTransform, Weight);
			}

			if (Child.Type == ERigElementType::Control)
			{
				// need to convert back to offset space for the actual control value
				FinalLocalTransform = FinalLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				FinalLocalTransform.NormalizeRotation();
			}
		
			Hierarchy->SetLocalTransform(ChildCache, FinalLocalTransform);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_AimConstraintLocalSpaceOffset)
{
	// multi-parent test
	{
		// use euler rotation here to match other software's rotation representation more easily
    	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
    	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
    	const FRigElementKey ChildTarget = Controller->AddBone(TEXT("ChildTarget"), FRigElementKey(TEXT("Child"), ERigElementType::Bone), FTransform(FVector(0.f, 10.f, 0.f)), false, ERigBoneType::User);
		
    	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform(FVector(10.f, 10.f, 10.f)), true, ERigBoneType::User);
    	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform(FVector(-10.f,10.f, 10.f)), true, ERigBoneType::User);
    	
		Unit.AimAxis = FVector(0,1, 0);
		Unit.UpAxis = FVector(0, 0,1);
    	
    	Unit.Child = Child;
    
    	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
    	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
    
    	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
    	Unit.bMaintainOffset = false;
    	
    	Execute();
    	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(0 , 10.f * FMath::Cos(PI / 4), 10.f * FMath::Sin(PI / 4)), 0.01f),
    		TEXT("unexpected translation for maintain offset off"));
    	
    	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
    	Hierarchy->SetGlobalTransform(2, FTransform(FVector(10.f, 10.f, 20.f)));
    	Hierarchy->SetGlobalTransform(3, FTransform(FVector(-10.f, 10.f, 20.f)));
    	Unit.bMaintainOffset = true;
    	
    	Execute();
    	AddErrorIfFalse(Hierarchy->GetGlobalTransform(1).GetTranslation().Equals(FVector(0 , 9.487f, 3.162f), 0.01f),
    					TEXT("unexpected translation for maintain offset on"));

	}
	
	return true;
}
#endif
