// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_CCDIK.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_CCDIK)

FRigUnit_CCDIK_Execute()
{
	FRigElementKeyCollection Items;
	if(WorkData.CachedItems.Num() == 0)
	{
		Items = FRigElementKeyCollection::MakeFromChain(
			ExecuteContext.Hierarchy,
			FRigElementKey(StartBone, ERigElementType::Bone),
			FRigElementKey(EffectorBone, ERigElementType::Bone),
			false /* reverse */
		);
	}

	// transfer the rotation limits
	TArray<FRigUnit_CCDIK_RotationLimitPerItem> RotationLimitsPerItem;
	for (int32 RotationLimitIndex = 0; RotationLimitIndex < RotationLimits.Num(); RotationLimitIndex++)
	{
		FRigUnit_CCDIK_RotationLimitPerItem RotationLimitPerItem;
		RotationLimitPerItem.Item = FRigElementKey(RotationLimits[RotationLimitIndex].Bone, ERigElementType::Bone);
		RotationLimitPerItem.Limit = RotationLimits[RotationLimitIndex].Limit;
		RotationLimitsPerItem.Add(RotationLimitPerItem);
	}

	FRigUnit_CCDIKPerItem::StaticExecute(
		ExecuteContext, 
		Items,
		EffectorTransform,
		Precision,
		Weight,
		MaxIterations,
		bStartFromTail,
		BaseRotationLimit,
		RotationLimitsPerItem,
		bPropagateToChildren,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_CCDIK::GetUpgradeInfo() const
{
	// this node is no longer supported and the upgrade path is too complex.
	return FRigVMStructUpgradeInfo();
}

FRigUnit_CCDIKPerItem_Execute()
{
	FRigUnit_CCDIKItemArray::StaticExecute(ExecuteContext, Items.Keys, EffectorTransform, Precision, Weight, MaxIterations, bStartFromTail, BaseRotationLimit, RotationLimits, bPropagateToChildren, WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_CCDIKPerItem::GetUpgradeInfo() const
{
	FRigUnit_CCDIKItemArray NewNode;
	NewNode.Items = Items.Keys;
	NewNode.EffectorTransform = EffectorTransform;
	NewNode.Precision = Precision;
	NewNode.Weight = Weight;
	NewNode.MaxIterations = MaxIterations;
	NewNode.bStartFromTail = bStartFromTail;
	NewNode.BaseRotationLimit = BaseRotationLimit;
	NewNode.RotationLimits = RotationLimits;
	NewNode.bPropagateToChildren = bPropagateToChildren;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_CCDIKItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	TArray<FCCDIKChainLink>& Chain = WorkData.Chain;
	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	TArray<int32>& RotationLimitIndex = WorkData.RotationLimitIndex;
	TArray<float>& RotationLimitsPerItem = WorkData.RotationLimitsPerItem;
	FCachedRigElement& CachedEffector = WorkData.CachedEffector;

	if ((RotationLimits.Num() != RotationLimitIndex.Num()) &&
		(RotationLimitIndex.Num() > 0))
	{
		CachedItems.Reset();
		RotationLimitIndex.Reset();
		RotationLimitsPerItem.Reset();
		CachedEffector.Reset();
	}
	
	if (CachedItems.Num() == 0 && Items.Num() > 0)
	{
		CachedItems.Add(FCachedRigElement(Hierarchy->GetFirstParent(Items[0]), Hierarchy));

		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		CachedEffector = CachedItems.Last();
		Chain.Reserve(CachedItems.Num());

		RotationLimitsPerItem.SetNumUninitialized(CachedItems.Num());
		for (const FRigUnit_CCDIK_RotationLimitPerItem& RotationLimit : RotationLimits)
		{
			FCachedRigElement BoneIndex(RotationLimit.Item, Hierarchy);
			int32 BoneIndexInLookup = CachedItems.Find(BoneIndex);
			RotationLimitIndex.Add(BoneIndexInLookup);
		}

		if (CachedItems.Num() < 2)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("No bones found."));
		}
	}

	if (CachedItems.Num() > 1)
	{
		// Gather chain links. These are non zero length bones.
		Chain.Reset();
		
		for (int32 ChainIndex = 0; ChainIndex < CachedItems.Num(); ChainIndex++)
		{
			const FTransform& GlobalTransform = Hierarchy->GetGlobalTransform(CachedItems[ChainIndex]);
			const FTransform& LocalTransform = Hierarchy->GetLocalTransform(CachedItems[ChainIndex]);
			Chain.Add(FCCDIKChainLink(GlobalTransform, LocalTransform, ChainIndex));
		}

		for (float& Limit : RotationLimitsPerItem)
		{
			Limit = BaseRotationLimit;
		}
		
		for (int32 LimitIndex = 0; LimitIndex < RotationLimitIndex.Num(); LimitIndex++)
		{
			if (RotationLimitIndex[LimitIndex] != INDEX_NONE)
			{
				RotationLimitsPerItem[RotationLimitIndex[LimitIndex]] = RotationLimits[LimitIndex].Limit;
			}
		}

		bool bBoneLocationUpdated = AnimationCore::SolveCCDIK(Chain, EffectorTransform.GetLocation(), Precision, MaxIterations, bStartFromTail, RotationLimits.Num() > 0, RotationLimitsPerItem);

		// If we moved some bones, update bone transforms.
		if (bBoneLocationUpdated)
		{
			if (FMath::IsNearlyEqual(Weight, 1.f))
			{
				for (int32 LinkIndex = 0; LinkIndex < CachedItems.Num(); LinkIndex++)
				{
					const FCCDIKChainLink& CurrentLink = Chain[LinkIndex];
					Hierarchy->SetGlobalTransform(CachedItems[LinkIndex], CurrentLink.Transform, bPropagateToChildren);
				}
			}
			else
			{
				float T = FMath::Clamp<float>(Weight, 0.f, 1.f);

				for (int32 LinkIndex = 0; LinkIndex < CachedItems.Num(); LinkIndex++)
				{
					const FCCDIKChainLink& CurrentLink = Chain[LinkIndex];
					FTransform PreviousXfo = Hierarchy->GetGlobalTransform(CachedItems[LinkIndex]);
					FTransform Xfo = FControlRigMathLibrary::LerpTransform(PreviousXfo, CurrentLink.Transform, T);
					Hierarchy->SetGlobalTransform(CachedItems[LinkIndex], Xfo, bPropagateToChildren);
				}
			}
		}

		if (FMath::IsNearlyEqual(Weight, 1.f))
		{
			Hierarchy->SetGlobalTransform(CachedEffector, EffectorTransform, bPropagateToChildren);
		}
		else
		{
			float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
			FTransform PreviousXfo = Hierarchy->GetGlobalTransform(CachedEffector);
			FTransform Xfo = FControlRigMathLibrary::LerpTransform(PreviousXfo, EffectorTransform, T);
			Hierarchy->SetGlobalTransform(CachedEffector, Xfo, bPropagateToChildren);
		}
	}
}

