// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_ChainInfo.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigVMCore/RigVMDebugDrawSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ChainInfo)

FRigUnit_ChainInfo_Execute()
{
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("No hierarchy found in control rig."));
		return;
	}

	if (Items.Num() < 2) 
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Must use at least two items in Items input list."));
		return;
	}

	if(CachedElements.Num() != Items.Num())
	{
		CachedElements.Reset();
		CachedElements.SetNum(Items.Num());
	}
	
	// Run some initialization
	ChainLength = 0.f;
	float InitialChainLength = 0.f;
	TArray<FRigUnit_ChainInfo_Segment> Segments;
	Segments.SetNumZeroed(CachedElements.Num() - 1);

    // Calculate chain length and record segment lengths
	for (int32 Index = 0; Index < Segments.Num(); Index++) 
	{
		FRigUnit_ChainInfo_Segment& Segment = Segments[Index];

		if (Index == 0)
		{
			if (!CachedElements[Index].UpdateCache(Items[Index], Hierarchy))
			{
				UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item '%s' is not valid."), *Items[Index].ToString());
			}
		}
		Segment.StartItem = CachedElements[Index];
		Segment.StartItemIndex = Index;

		if (!CachedElements[Index + 1].UpdateCache(Items[Index + 1], Hierarchy))
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item '%s' is not valid."), *Items[Index + 1].ToString());
		}
		Segment.EndItem = CachedElements[Index + 1];
		Segment.EndItemIndex = Index + 1;

		// Only need to calculate initials if we need stretch info or we have node set to Initial
		if (bCalculateStretch || bInitial)
		{
			const FVector InitialPositionA = Hierarchy->GetInitialGlobalTransform(Segment.EndItem).GetLocation();
        	const FVector InitialPositionB = Hierarchy->GetInitialGlobalTransform(Segment.StartItem).GetLocation();
			const FVector InitialSegmentVector = InitialPositionA - InitialPositionB;
			Segment.InitialLength = InitialSegmentVector.Size();
			Segment.InitialCumLength = InitialChainLength + Segment.InitialLength;
        	InitialChainLength = Segment.InitialCumLength;
		}

        // If node is set to Initial we use the initial calculations as the current
		if (bInitial)
		{
			Segment.Length = Segment.InitialLength;
		}
		else
		{
			const FVector PositionA = Hierarchy->GetGlobalTransform(Segment.EndItem).GetLocation();
			const FVector PositionB = Hierarchy->GetGlobalTransform(Segment.StartItem).GetLocation();
			const FVector SegmentVector = PositionA - PositionB;
			Segment.Length = SegmentVector.Size();
		}

		Segment.CumLength = ChainLength + Segment.Length;
        ChainLength = Segment.CumLength;		
	}

    // Calculate param length
	ParamLength = Param * ChainLength;

	// Binary search for current segment containing param based on recorded lengths
    SegmentInfo.SegmentIndex = 0;
	int StartSegmentIndex = 0;
	int EndSegmentIndex = Segments.Num() - 1;

	while (StartSegmentIndex != EndSegmentIndex) 
	{
		int SearchIndex = StartSegmentIndex + ((EndSegmentIndex - StartSegmentIndex) / 2);
		if (ParamLength < Segments[SearchIndex].CumLength)
		{
			if (SearchIndex == 0)
			{
				SegmentInfo.SegmentIndex = SearchIndex;
				break;
			}
			EndSegmentIndex = SearchIndex;
		}
		else if (ParamLength >= Segments[SearchIndex].CumLength)
		{
			if (Segments[SearchIndex].CumLength == ParamLength || SearchIndex + 1 == EndSegmentIndex)
			{
				SegmentInfo.SegmentIndex = SearchIndex + 1;
				break;
			}
			StartSegmentIndex = SearchIndex;
		}
	}

	const FRigUnit_ChainInfo_Segment& CurrentSegment = Segments[SegmentInfo.SegmentIndex];

	SegmentInfo.SegmentStartItem = CurrentSegment.StartItem.GetKey();
	SegmentInfo.SegmentStartItemIndex = CurrentSegment.StartItemIndex;
	SegmentInfo.SegmentEndItem = CurrentSegment.EndItem.GetKey();
	SegmentInfo.SegmentEndItemIndex = CurrentSegment.EndItemIndex;

    // Calculate segment length and segment param length
	if (SegmentInfo.SegmentIndex == 0) 
	{
		SegmentInfo.SegmentParamLength = ParamLength;
	}
	else 
	{
		SegmentInfo.SegmentParamLength = ParamLength - Segments[SegmentInfo.SegmentIndex - 1].CumLength;
	}
    SegmentInfo.SegmentLength = CurrentSegment.Length;

    // Calculate segment param
	if (SegmentInfo.SegmentLength != 0.f) 
	{
	    SegmentInfo.SegmentParam = SegmentInfo.SegmentParamLength / SegmentInfo.SegmentLength;
	}

	if (bCalculateStretch)
	{
		// Calculate stretch factors
		if (InitialChainLength != 0.f)
		{
			ChainStretchFactor = ChainLength / InitialChainLength;
		}

		if (CurrentSegment.InitialLength != 0.f) 
		{
			SegmentInfo.SegmentStretchFactor = CurrentSegment.Length / CurrentSegment.InitialLength;
		}
	}

    // Lerp a transform from the current start and end items with segment param
	FTransform StartTransform;
	FTransform EndTransform;
	if (bInitial)
	{
		StartTransform = Hierarchy->GetInitialGlobalTransform(SegmentInfo.SegmentStartItem);
		EndTransform = Hierarchy->GetInitialGlobalTransform(SegmentInfo.SegmentEndItem);
	}
	else
	{
		StartTransform = Hierarchy->GetGlobalTransform(SegmentInfo.SegmentStartItem);
		EndTransform = Hierarchy->GetGlobalTransform(SegmentInfo.SegmentEndItem);
	}
	InterpolatedTransform = FControlRigMathLibrary::LerpTransform(StartTransform, EndTransform, SegmentInfo.SegmentParam);
    
	// Debug draw
	if (bDebug) {
		FRigVMDebugDrawSettings DebugDrawSettings;

	    // Draw interpolated output transform
	    ExecuteContext.GetDrawInterface()->DrawAxes(FTransform::Identity, 
			                                        InterpolatedTransform, 
													DebugScale, 
													0.f, 
													DebugDrawSettings.DepthPriority, 
													DebugDrawSettings.Lifetime);
		// Draw transform of start item									
		StartTransform.SetScale3D(StartTransform.GetScale3D() * DebugScale);
		ExecuteContext.GetDrawInterface()->DrawBox(FTransform::Identity, 
			                                       StartTransform, 
												   FLinearColor::Green, 
												   0.f, 
												   DebugDrawSettings.DepthPriority, 
												   DebugDrawSettings.Lifetime);
		// Draw transform of end item
		EndTransform.SetScale3D(EndTransform.GetScale3D() * DebugScale);
		ExecuteContext.GetDrawInterface()->DrawBox(FTransform::Identity, 
			                                       EndTransform, 
												   FLinearColor::Red, 
												   0.f, 
												   DebugDrawSettings.DepthPriority, 
												   DebugDrawSettings.Lifetime);												   
	}
}
