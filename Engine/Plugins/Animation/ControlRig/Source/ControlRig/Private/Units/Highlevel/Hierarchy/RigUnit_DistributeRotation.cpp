// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_DistributeRotation.h"
#include "Units/RigUnitContext.h"
#include "AnimationCoreLibrary.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DistributeRotation)

FRigUnit_DistributeRotation_Execute()
{
	TArray<FRigElementKey> Items;

	if(WorkData.CachedItems.Num() == 0)
	{
		URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		if (Hierarchy == nullptr)
		{
			return;
		}

		int32 EndBoneIndex = Hierarchy->GetIndex(FRigElementKey(EndBone, ERigElementType::Bone));
		if (EndBoneIndex != INDEX_NONE)
		{
			int32 StartBoneIndex = Hierarchy->GetIndex(FRigElementKey(StartBone, ERigElementType::Bone));
			if (StartBoneIndex == EndBoneIndex)
			{
				return;
			}

			while (EndBoneIndex != INDEX_NONE)
			{
				Items.Add(Hierarchy->GetKey(EndBoneIndex));
				if (EndBoneIndex == StartBoneIndex)
				{
					break;
				}
				EndBoneIndex = Hierarchy->GetFirstParent(EndBoneIndex);
			}
		}

		Algo::Reverse(Items);
	}
	else
	{
		for(const FCachedRigElement& CachedItem : WorkData.CachedItems)
		{
			Items.Add(CachedItem.GetKey());
		}
	}

	FRigUnit_DistributeRotationForCollection::StaticExecute(
		ExecuteContext, 
		Items,
		Rotations,
		RotationEaseType,
		Weight,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_DistributeRotation::GetUpgradeInfo() const
{
	// this node is no longer supported and the upgrade path is too complex.
	return FRigVMStructUpgradeInfo();
}

FRigUnit_DistributeRotationForCollection_Execute()
{
	FRigUnit_DistributeRotationForItemArray::StaticExecute(ExecuteContext, Items.Keys, Rotations, RotationEaseType, Weight, WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_DistributeRotationForCollection::GetUpgradeInfo() const
{
	FRigUnit_DistributeRotationForItemArray NewNode;
	NewNode.Items = Items.Keys;
	NewNode.Rotations = Rotations;
	NewNode.RotationEaseType = RotationEaseType;
	NewNode.Weight = Weight;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_DistributeRotationForItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	TArray<int32>& ItemRotationA = WorkData.ItemRotationA;
	TArray<int32>& ItemRotationB = WorkData.ItemRotationB;
	TArray<float>& ItemRotationT = WorkData.ItemRotationT;
	TArray<FTransform>& ItemLocalTransforms = WorkData.ItemLocalTransforms;

	if (CachedItems.Num() == Items.Num())
	{
		for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ItemIndex++)
		{
			if (CachedItems[ItemIndex].GetKey() != Items[ItemIndex])
			{
				CachedItems.Reset();
				break;
			}
		}
	}

	if (CachedItems.Num() > 0 && CachedItems.Num() != Items.Num())
	{
		CachedItems.Reset();
		ItemRotationA.Reset();
		ItemRotationB.Reset();
		ItemRotationT.Reset();
		ItemLocalTransforms.Reset();
	}

	if (CachedItems.Num() == 0)
	{
		if (Items.Num() < 2)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough items. You need at least two!"));
			return;
		}

		for (int32 ItemIndex = 0; ItemIndex < Items.Num(); ItemIndex++)
		{
			CachedItems.Add(FCachedRigElement(Items[ItemIndex], Hierarchy));
		}

		ItemRotationA.SetNumZeroed(CachedItems.Num());
		ItemRotationB.SetNumZeroed(CachedItems.Num());
		ItemRotationT.SetNumZeroed(CachedItems.Num());
		ItemLocalTransforms.SetNumZeroed(CachedItems.Num());

		if (Rotations.Num() < 2)
		{
			return;
		}

		TArray<float> RotationRatios;
		TArray<int32> RotationIndices;

		for (const FRigUnit_DistributeRotation_Rotation& Rotation : Rotations)
		{
			RotationIndices.Add(RotationIndices.Num());
			RotationRatios.Add(FMath::Clamp<float>(Rotation.Ratio, 0.f, 1.f));
		}

		TLess<> Predicate;
		auto Projection = [RotationRatios](int32 Val) -> float
		{
			return RotationRatios[Val];
		};
		Algo::SortBy(RotationIndices, Projection, Predicate);

 		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
 		{
			float T = 0.f;
			if (CachedItems.Num() > 1)
			{
				T = float(Index) / float(CachedItems.Num() - 1);
			}

			if (T <= RotationRatios[RotationIndices[0]])
			{
				ItemRotationA[Index] = ItemRotationB[Index] = RotationIndices[0];
				ItemRotationT[Index] = 0.f;
			}
			else if (T >= RotationRatios[RotationIndices.Last()])
			{
				ItemRotationA[Index] = ItemRotationB[Index] = RotationIndices.Last();
				ItemRotationT[Index] = 0.f;
			}
			else
			{
				for (int32 RotationIndex = 1; RotationIndex < RotationIndices.Num(); RotationIndex++)
				{
					int32 A = RotationIndices[RotationIndex - 1];
					int32 B = RotationIndices[RotationIndex];

					if (FMath::IsNearlyEqual(Rotations[A].Ratio, T))
					{
						ItemRotationA[Index] = ItemRotationB[Index] = A;
						ItemRotationT[Index] = 0.f;
						break;
					}
					else if (FMath::IsNearlyEqual(Rotations[B].Ratio, T))
					{
						ItemRotationA[Index] = ItemRotationB[Index] = B;
						ItemRotationT[Index] = 0.f;
						break;
					}
					else if (Rotations[B].Ratio > T)
					{
						if (FMath::IsNearlyEqual(RotationRatios[A], RotationRatios[B]))
						{
							ItemRotationA[Index] = ItemRotationB[Index] = A;
							ItemRotationT[Index] = 0.f;
						}
						else
						{
							ItemRotationA[Index] = A;
							ItemRotationB[Index] = B;
							ItemRotationT[Index] = (T - RotationRatios[A]) / (RotationRatios[B] - RotationRatios[A]);
							ItemRotationT[Index] = FControlRigMathLibrary::EaseFloat(ItemRotationT[Index], RotationEaseType);
						}
						break;
					}
				}
			}
 		}
	}

	if (CachedItems.Num() < 2 || Rotations.Num() == 0)
	{
		return;
	}

	if (!CachedItems[0].IsValid())
	{
		return;
	}

	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		if (CachedItems[Index].IsValid())
		{
			ItemLocalTransforms[Index] = Hierarchy->GetLocalTransform(CachedItems[Index]);
		}
		else
		{
			ItemLocalTransforms[Index] = FTransform::Identity;
		}
	}

	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		if (ItemRotationA[Index] >= Rotations.Num() ||
			ItemRotationB[Index] >= Rotations.Num())
		{
			continue;
		}

		FQuat Rotation = Rotations[ItemRotationA[Index]].Rotation.GetNormalized();
		FQuat RotationB = Rotations[ItemRotationB[Index]].Rotation.GetNormalized();
		if (ItemRotationA[Index] != ItemRotationB[Index])
		{
			if (ItemRotationT[Index] > 1.f - SMALL_NUMBER)
			{
				Rotation = RotationB;
			}
			else if (ItemRotationT[Index] > SMALL_NUMBER)
			{
				Rotation = FQuat::Slerp(Rotation, RotationB, ItemRotationT[Index]).GetNormalized();
			}
		}

		FTransform Transform = ItemLocalTransforms[Index];

		Rotation = FQuat::Slerp(Transform.GetRotation(), Transform.GetRotation() * Rotation, FMath::Clamp<float>(Weight, 0.f, 1.f));
		Transform.SetRotation(Rotation);

		if (CachedItems[Index].IsValid())
		{
			Hierarchy->SetLocalTransform(CachedItems[Index], Transform);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_DistributeRotation)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(2.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), BoneA, FTransform(FVector(2.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneC = Controller->AddBone(TEXT("BoneC"), BoneB, FTransform(FVector(2.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneD = Controller->AddBone(TEXT("BoneD"), BoneC, FTransform(FVector(2.f, 0.f, 0.f)), true, ERigBoneType::User);

	Unit.StartBone = TEXT("Root");
	Unit.EndBone = TEXT("BoneD");
	FRigUnit_DistributeRotation_Rotation Rotation;
	
	Rotation.Rotation = FQuat::Identity;
	Rotation.Ratio = 0.f;
	Unit.Rotations.Add(Rotation);
	Rotation.Rotation = FQuat::Identity;
	Rotation.Ratio = 1.f;
	Unit.Rotations.Add(Rotation);
	Rotation.Rotation = AnimationCore::QuatFromEuler(FVector(0.f, 90.f, 0.f), EEulerRotationOrder::XYZ);
	Rotation.Ratio = 0.5f;
	Unit.Rotations.Add(Rotation);

	Execute();

	AddErrorIfFalse(Unit.WorkData.ItemRotationA[0] == 0, TEXT("unexpected bone a"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationB[0] == 0, TEXT("unexpected bone b"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.WorkData.ItemRotationT[0], 0.f, 0.001f), TEXT("unexpected bone t"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationA[1] == 0, TEXT("unexpected bone a"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationB[1] == 2, TEXT("unexpected bone b"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.WorkData.ItemRotationT[1], 0.5f, 0.001f), TEXT("unexpected bone t"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationA[2] == 2, TEXT("unexpected bone a"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationB[2] == 2, TEXT("unexpected bone b"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.WorkData.ItemRotationT[2], 0.f, 0.001f), TEXT("unexpected bone t"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationA[3] == 2, TEXT("unexpected bone a"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationB[3] == 1, TEXT("unexpected bone b"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.WorkData.ItemRotationT[3], 0.5f, 0.001f), TEXT("unexpected bone t"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationA[4] == 1, TEXT("unexpected bone a"));
	AddErrorIfFalse(Unit.WorkData.ItemRotationB[4] == 1, TEXT("unexpected bone b"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.WorkData.ItemRotationT[4], 0.0f, 0.001f), TEXT("unexpected bone t"));

	FVector Euler = FVector::ZeroVector;
	Euler = AnimationCore::EulerFromQuat(Hierarchy->GetLocalTransform(0).GetRotation(), EEulerRotationOrder::XYZ);
	AddErrorIfFalse(FMath::IsNearlyEqual((double)Euler.Y, 0.0, 0.1), TEXT("unexpected rotation Y"));
	Euler = AnimationCore::EulerFromQuat(Hierarchy->GetLocalTransform(1).GetRotation(), EEulerRotationOrder::XYZ);
	AddErrorIfFalse(FMath::IsNearlyEqual((double)Euler.Y, 45.0, 0.1), TEXT("unexpected rotation Y"));
	Euler = AnimationCore::EulerFromQuat(Hierarchy->GetLocalTransform(2).GetRotation(), EEulerRotationOrder::XYZ);
	AddErrorIfFalse(FMath::IsNearlyEqual((double)Euler.Y, 90.0, 0.1), TEXT("unexpected rotation Y"));
	Euler = AnimationCore::EulerFromQuat(Hierarchy->GetLocalTransform(3).GetRotation(), EEulerRotationOrder::XYZ);
	AddErrorIfFalse(FMath::IsNearlyEqual((double)Euler.Y, 45.0, 0.1), TEXT("unexpected rotation Y"));
	Euler = AnimationCore::EulerFromQuat(Hierarchy->GetLocalTransform(4).GetRotation(), EEulerRotationOrder::XYZ);
	AddErrorIfFalse(FMath::IsNearlyEqual((double)Euler.Y, 0.0, 0.1), TEXT("unexpected rotation Y"));

	return true;
}
#endif
