// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_FABRIK.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_FABRIK)

FRigUnit_FABRIK_Execute()
{
	FRigElementKeyCollection Items;
	if(WorkData.CachedItems.Num() == 0)
	{
		Items = FRigElementKeyCollection::MakeFromChain(
			ExecuteContext.Hierarchy,
			FRigElementKey(StartBone, ERigElementType::Bone),
			FRigElementKey(EffectorBone, ERigElementType::Bone),
			false /* reverse */
		);
	}

	FRigUnit_FABRIKPerItem::StaticExecute(
		ExecuteContext, 
		Items,
		EffectorTransform,
		Precision,
		Weight,
		bPropagateToChildren,
		MaxIterations,
		WorkData,
		bSetEffectorTransform);
}

FRigVMStructUpgradeInfo FRigUnit_FABRIK::GetUpgradeInfo() const
{
	// this node is no longer supported and the upgrade path is too complex.
	return FRigVMStructUpgradeInfo();
}

FRigUnit_FABRIKPerItem_Execute()
{
	FRigUnit_FABRIKItemArray::StaticExecute(ExecuteContext, Items.Keys, EffectorTransform, Precision, Weight, bPropagateToChildren, MaxIterations, WorkData, bSetEffectorTransform);
}

FRigVMStructUpgradeInfo FRigUnit_FABRIKPerItem::GetUpgradeInfo() const
{
	FRigUnit_FABRIKItemArray NewNode;
	NewNode.Items = Items.Keys;
	NewNode.EffectorTransform = EffectorTransform;
	NewNode.Precision = Precision;
	NewNode.Weight = Weight;
	NewNode.bPropagateToChildren = bPropagateToChildren;
	NewNode.MaxIterations = MaxIterations;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_FABRIKItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	TArray<FFABRIKChainLink>& Chain = WorkData.Chain;
	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	FCachedRigElement& CachedEffector = WorkData.CachedEffector;

	if (CachedItems.Num() == 0 && Items.Num() > 0)
	{
		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		CachedEffector = CachedItems.Last();
	}

	if (CachedItems.Num() > 0)
	{
		// Gather chain links. These are non zero length bones.
		Chain.Reset();
		
		TArray<FTransform> Transforms;
		Transforms.AddDefaulted(CachedItems.Num());

		float MaximumReach = 0.f;
		int32 const NumChainLinks = CachedItems.Num();
		const FCachedRigElement& RootIndex = CachedItems[0];
		Chain.Add(FFABRIKChainLink(Hierarchy->GetGlobalTransform(RootIndex).GetLocation(), 0.f, 0, 0));
		Transforms[0] = Hierarchy->GetGlobalTransform(RootIndex);

		// start from child to up
		for (int32 ChainIndex = 1; ChainIndex < CachedItems.Num() ; ChainIndex++)
		{
			const FTransform& BoneTransform = Hierarchy->GetGlobalTransform(CachedItems[ChainIndex]);
			const FTransform& ParentTransform = Hierarchy->GetGlobalTransform(CachedItems[ChainIndex - 1]);

			// Calculate the combined length of this segment of skeleton
			float const BoneLength = FVector::Dist(BoneTransform.GetLocation(), ParentTransform.GetLocation());

			const int32 TransformIndex = Chain.Num();
			Chain.Add(FFABRIKChainLink(BoneTransform.GetLocation(), BoneLength, ChainIndex, TransformIndex));
			MaximumReach += BoneLength;

			Transforms[TransformIndex] = BoneTransform;
		}


		bool bBoneLocationUpdated = AnimationCore::SolveFabrik(Chain, EffectorTransform.GetLocation(), MaximumReach, Precision, MaxIterations);
		// If we moved some bones, update bone transforms.
		if (bBoneLocationUpdated)
		{
			// FABRIK algorithm - re-orientation of bone local axes after translation calculation
			for (int32 LinkIndex = 0; LinkIndex < NumChainLinks - 1; LinkIndex++)
			{
				const FFABRIKChainLink& CurrentLink = Chain[LinkIndex];
				const FFABRIKChainLink& ChildLink = Chain[LinkIndex + 1];
				const FCachedRigElement& CurrentItem = CachedItems[CurrentLink.BoneIndex];
				const FCachedRigElement& ChildItem = CachedItems[ChildLink.BoneIndex];

				// Calculate pre-translation vector between this bone and child
				FVector const OldDir = (Hierarchy->GetGlobalTransform(ChildItem).GetLocation() - Hierarchy->GetGlobalTransform(CurrentItem).GetLocation()).GetUnsafeNormal();

				// Get vector from the post-translation bone to it's child
				FVector const NewDir = (ChildLink.Position - CurrentLink.Position).GetUnsafeNormal();

				// Calculate axis of rotation from pre-translation vector to post-translation vector
				FVector const RotationAxis = FVector::CrossProduct(OldDir, NewDir).GetSafeNormal();
				float const RotationAngle = FMath::Acos(FVector::DotProduct(OldDir, NewDir));
				FQuat const DeltaRotation = FQuat(RotationAxis, RotationAngle);
				// We're going to multiply it, in order to not have to re-normalize the final quaternion, it has to be a unit quaternion.
				checkSlow(DeltaRotation.IsNormalized());

				// Calculate absolute rotation and set it
				FTransform& CurrentBoneTransform = Transforms[CurrentLink.TransformIndex];
				CurrentBoneTransform.SetRotation(DeltaRotation * CurrentBoneTransform.GetRotation());
				CurrentBoneTransform.NormalizeRotation();
				CurrentBoneTransform.SetTranslation(CurrentLink.Position);
			}

			// fill up the last data transform
			const FFABRIKChainLink & CurrentLink = Chain[NumChainLinks - 1];
			const FCachedRigElement& CurrentItem = CachedItems[CurrentLink.BoneIndex];

			FTransform& CurrentBoneTransform = Transforms[CurrentLink.TransformIndex];
			CurrentBoneTransform.SetTranslation(CurrentLink.Position);
			CurrentBoneTransform.SetRotation(Hierarchy->GetGlobalTransform(CurrentItem).GetRotation());

			if (FMath::IsNearlyEqual(Weight, 1.f))
			{
				for (int32 LinkIndex = 0; LinkIndex < NumChainLinks; LinkIndex++)
				{
					FFABRIKChainLink const & LocalLink = Chain[LinkIndex];
					const FCachedRigElement& LocalItem = CachedItems[LocalLink.BoneIndex];
					Hierarchy->SetGlobalTransform(LocalItem, Transforms[LocalLink.TransformIndex], bPropagateToChildren);
				}
			}
			else
			{
				float T = FMath::Clamp<float>(Weight, 0.f, 1.f);

				for (int32 LinkIndex = 0; LinkIndex < NumChainLinks; LinkIndex++)
				{
					FFABRIKChainLink const & LocalLink = Chain[LinkIndex];
					const FCachedRigElement& LocalItem = CachedItems[LocalLink.BoneIndex];
					FTransform PreviousXfo = Hierarchy->GetGlobalTransform(LocalItem);
					FTransform Xfo = FControlRigMathLibrary::LerpTransform(PreviousXfo, Transforms[LocalLink.TransformIndex], T);
					Hierarchy->SetGlobalTransform(LocalItem, Xfo, bPropagateToChildren);
				}
			}
		}

		if (bSetEffectorTransform)
		{
			if (FMath::IsNearlyEqual(Weight, 1.f))
			{
				Hierarchy->SetGlobalTransform(CachedEffector, EffectorTransform, bPropagateToChildren);
			}
			else
			{
				float T = FMath::Clamp<float>(Weight, 0.f, 1.f);
				FTransform PreviousXfo = Hierarchy->GetGlobalTransform(CachedEffector);
				FTransform Xfo = FControlRigMathLibrary::LerpTransform(PreviousXfo, EffectorTransform, T);
				Hierarchy->SetGlobalTransform(CachedEffector, Xfo, bPropagateToChildren);
			}
		}
	}
}

