// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_FitChainToCurve.h"
#include "Units/Debug/RigUnit_DebugBezier.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_FitChainToCurve)

FRigUnit_FitChainToCurve_Execute()
{
	FRigElementKeyCollection Items;
	if(WorkData.CachedItems.Num() == 0)
	{
		Items = FRigElementKeyCollection::MakeFromChain(
			ExecuteContext.Hierarchy,
			FRigElementKey(StartBone, ERigElementType::Bone),
			FRigElementKey(EndBone, ERigElementType::Bone),
			false /* reverse */
		);
	}

	FRigUnit_FitChainToCurvePerItem::StaticExecute(
		ExecuteContext, 
		Items,
		Bezier,
		Alignment,
		Minimum,
		Maximum,
		SamplingPrecision,
		PrimaryAxis,
		SecondaryAxis,
		PoleVectorPosition,
		Rotations,
		RotationEaseType,
		Weight,
		bPropagateToChildren,
		DebugSettings,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_FitChainToCurve::GetUpgradeInfo() const
{
	// this node is no longer supported and the upgrade path is too complex.
	return FRigVMStructUpgradeInfo();
}

FRigUnit_FitChainToCurvePerItem_Execute()
{
	FRigUnit_FitChainToCurveItemArray::StaticExecute(
		ExecuteContext,
		Items.Keys,
		Bezier,
		Alignment,
		Minimum,
		Maximum,
		SamplingPrecision,
		PrimaryAxis,
		SecondaryAxis,
		PoleVectorPosition,
		Rotations,
		RotationEaseType,
		Weight,
		bPropagateToChildren,
		DebugSettings,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_FitChainToCurvePerItem::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_FitChainToCurveItemArray_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	float& ChainLength = WorkData.ChainLength;
	TArray<FVector>& ItemPositions = WorkData.ItemPositions;
	TArray<float>& ItemSegments = WorkData.ItemSegments;
	TArray<FVector>& CurvePositions = WorkData.CurvePositions;
	TArray<float>& CurveSegments = WorkData.CurveSegments;
	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	TArray<int32>& ItemRotationA = WorkData.ItemRotationA;
	TArray<int32>& ItemRotationB = WorkData.ItemRotationB;
	TArray<float>& ItemRotationT = WorkData.ItemRotationT;
	TArray<FTransform>& ItemLocalTransforms = WorkData.ItemLocalTransforms;

	if(CachedItems.Num() == 0 && Items.Num() > 1)
	{
		CurvePositions.Reset();
		ItemPositions.Reset();
		ItemSegments.Reset();
		CurveSegments.Reset();
		ItemRotationA.Reset();
		ItemRotationB.Reset();
		ItemRotationT.Reset();
		ItemLocalTransforms.Reset();
		ChainLength = 0.f;

		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		if (CachedItems.Num() < 2)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough bones. You need at least two in the chain!"));
			return;
		}

		ItemPositions.SetNumZeroed(CachedItems.Num());
		ItemSegments.SetNumZeroed(CachedItems.Num());
		ItemSegments[0] = 0;
		for (int32 Index = 1; Index < CachedItems.Num(); Index++)
		{
			FVector A = Hierarchy->GetGlobalTransform(CachedItems[Index - 1]).GetLocation();
			FVector B = Hierarchy->GetGlobalTransform(CachedItems[Index]).GetLocation();
			ItemSegments[Index] = (A - B).Size();
			ChainLength += ItemSegments[Index];
		}

		ItemRotationA.SetNumZeroed(CachedItems.Num());
		ItemRotationB.SetNumZeroed(CachedItems.Num());
		ItemRotationT.SetNumZeroed(CachedItems.Num());
		ItemLocalTransforms.SetNumZeroed(CachedItems.Num());

		if (Rotations.Num() > 1)
		{
			TArray<float> RotationRatios;
			TArray<int32> RotationIndices;

			for (const FRigUnit_FitChainToCurve_Rotation& Rotation : Rotations)
			{
				RotationIndices.Add(RotationIndices.Num());
				RotationRatios.Add(FMath::Clamp<float>(Rotation.Ratio, 0.f, 1.f));
			}

			TLess<> Predicate;
			auto Projection = [RotationRatios](int32 Val) -> float
			{
				return RotationRatios[Val];
			};
			Algo::SortBy(RotationIndices, Projection, Predicate);

			for (int32 Index = 0; Index < CachedItems.Num(); Index++)
			{
				float T = 0.f;
				if (CachedItems.Num() > 1)
				{
					T = float(Index) / float(CachedItems.Num() - 1);
				}

				if (T <= RotationRatios[RotationIndices[0]])
				{
					ItemRotationA[Index] = ItemRotationB[Index] = RotationIndices[0];
					ItemRotationT[Index] = 0.f;
				}
				else if (T >= RotationRatios[RotationIndices.Last()])
				{
					ItemRotationA[Index] = ItemRotationB[Index] = RotationIndices.Last();
					ItemRotationT[Index] = 0.f;
				}
				else
				{
					for (int32 RotationIndex = 1; RotationIndex < RotationIndices.Num(); RotationIndex++)
					{
						int32 A = RotationIndices[RotationIndex - 1];
						int32 B = RotationIndices[RotationIndex];

						if (FMath::IsNearlyEqual(Rotations[A].Ratio, T))
						{
							ItemRotationA[Index] = ItemRotationB[Index] = A;
							ItemRotationT[Index] = 0.f;
							break;
						}
						else if (FMath::IsNearlyEqual(Rotations[B].Ratio, T))
						{
							ItemRotationA[Index] = ItemRotationB[Index] = B;
							ItemRotationT[Index] = 0.f;
							break;
						}
						else if (Rotations[B].Ratio > T)
						{
							if (FMath::IsNearlyEqual(RotationRatios[A], RotationRatios[B]))
							{
								ItemRotationA[Index] = ItemRotationB[Index] = A;
								ItemRotationT[Index] = 0.f;
							}
							else
							{
								ItemRotationA[Index] = A;
								ItemRotationB[Index] = B;
								ItemRotationT[Index] = (T - RotationRatios[A]) / (RotationRatios[B] - RotationRatios[A]);
								ItemRotationT[Index] = FControlRigMathLibrary::EaseFloat(ItemRotationT[Index], RotationEaseType);
							}
							break;
						}
					}
				}
			}
 		}
	}

	if (CachedItems.Num() < 2)
	{
		return;
	}

	int32 Samples = FMath::Clamp<int32>(SamplingPrecision, 4, 64);
	if (CurvePositions.Num() != Samples)
	{
		CurvePositions.SetNum(Samples + 1);
		CurveSegments.SetNum(Samples + 1);
	}

	FVector StartTangent = FVector::ZeroVector;
	FVector EndTangent = FVector::ZeroVector;

	float CurveLength = 0.f;
	for (int32 SampleIndex = 0; SampleIndex < Samples; SampleIndex++)
	{
		float T = float(SampleIndex) / float(Samples - 1);
		T = FMath::Lerp<float>(Minimum, Maximum, T);

		FVector Tangent;
		FControlRigMathLibrary::FourPointBezier(Bezier, T, CurvePositions[SampleIndex], Tangent);
		if (SampleIndex == 0)
		{
			StartTangent = Tangent;
		}
		else if (SampleIndex == Samples - 1)
		{
			EndTangent = Tangent;
		}

		if (SampleIndex > 0)
		{
			CurveSegments[SampleIndex] = (CurvePositions[SampleIndex] - CurvePositions[SampleIndex - 1]).Size();
			CurveLength += CurveSegments[SampleIndex];
		}
		else
		{
			CurveSegments[SampleIndex] = 0.f;
		}
	}

	CurvePositions[Samples] = CurvePositions[Samples - 1] + EndTangent * ChainLength;
	CurveSegments[Samples] = ChainLength;

	if (ChainLength < SMALL_NUMBER)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("The chain has no length - all of the bones are in the sample place!"));
		return;
	}

	if (CurveLength < SMALL_NUMBER)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("The curve has no length - all of the points are in the sample place!"));
		return;
	}

	int32 CurveIndex = 1;
	ItemPositions[0] = CurvePositions[0];

	for (int32 Index = 1; Index < CachedItems.Num(); Index++)
	{
		const FVector& LastPosition = ItemPositions[Index - 1];

		float BoneLength = ItemSegments[Index];
		switch (Alignment)
		{
			case EControlRigCurveAlignment::Front:
			{
				break;
			}
			case EControlRigCurveAlignment::Stretched:
			{
				BoneLength = BoneLength * CurveLength / ChainLength;
				break;
			}
		}

		FVector A = FVector::ZeroVector;
		FVector B = FVector::ZeroVector;
		A = CurvePositions[CurveIndex - 1];
		B = CurvePositions[CurveIndex];

		float DistanceA = (LastPosition - A).Size();
		float DistanceB = (LastPosition - B).Size();
		
		if (DistanceB > BoneLength)
		{
			float Ratio = BoneLength / DistanceB;
			ItemPositions[Index] = FMath::Lerp<FVector>(LastPosition, B, Ratio);
			continue;
		}

		while (CurveIndex < CurvePositions.Num() - 1)
		{
			CurveIndex++;
			A = B;
			B = CurvePositions[CurveIndex];
			DistanceA = DistanceB;
			DistanceB = (B - LastPosition).Size();

			if ((DistanceA < BoneLength) != (DistanceB < BoneLength))
			{
				break;
			}
		}

		if (DistanceB < DistanceA)
		{
			FVector TempV = A;
			A = B;
			B = TempV;
			float TempF = DistanceA;
			DistanceA = DistanceB;
			DistanceB = TempF;
		}

		if (FMath::IsNearlyEqual(DistanceA, DistanceB))
		{
			ItemPositions[Index] = A;
			continue;
		}

		float Ratio = (BoneLength - DistanceA) / (DistanceB - DistanceA);
		ItemPositions[Index] = FMath::Lerp<FVector>(A, B, Ratio);
	}

	TArray<FTransform> BoneGlobalTransforms;
	if (Weight < 1.f - SMALL_NUMBER)
	{
		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
		{
			BoneGlobalTransforms.Add(Hierarchy->GetGlobalTransform(CachedItems[Index]));
		}
	}

	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		FTransform Transform = Hierarchy->GetGlobalTransform(CachedItems[Index]);

		Transform.SetTranslation(ItemPositions[Index]);

		FVector Target = FVector::ZeroVector;
		if (Index < CachedItems.Num() - 1)
		{
			Target = ItemPositions[Index + 1] - ItemPositions[Index];
		}
		else
		{
			Target = ItemPositions.Last() - ItemPositions[ItemPositions.Num() - 2];
		}

		if (!Target.IsNearlyZero() && !PrimaryAxis.IsNearlyZero())
		{
			Target = Target.GetSafeNormal();
			FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis).GetSafeNormal();
			FQuat Rotation = FQuat::FindBetweenNormals(Axis, Target);
			Transform.SetRotation((Rotation * Transform.GetRotation()).GetNormalized());
		}

		Target = PoleVectorPosition - ItemPositions[Index];
		if (!SecondaryAxis.IsNearlyZero())
		{
			if (!PrimaryAxis.IsNearlyZero())
			{
				FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis).GetSafeNormal();
				Target = Target - FVector::DotProduct(Target, Axis) * Axis;
			}

			if (!Target.IsNearlyZero() && !SecondaryAxis.IsNearlyZero())
			{
				Target = Target.GetSafeNormal();
				FVector Axis = Transform.TransformVectorNoScale(SecondaryAxis).GetSafeNormal();
				FQuat Rotation = FQuat::FindBetweenNormals(Axis, Target);
				Transform.SetRotation((Rotation * Transform.GetRotation()).GetNormalized());
			}
		}

		if (Weight >= 1.f - SMALL_NUMBER)
		{
			Hierarchy->SetGlobalTransform(CachedItems[Index], Transform, bPropagateToChildren && Rotations.Num() == 0);
		}
		else
		{
			Transform = FControlRigMathLibrary::LerpTransform(BoneGlobalTransforms[Index], Transform, FMath::Clamp<float>(Weight, 0.f, 1.f));
			Hierarchy->SetGlobalTransform(CachedItems[Index], Transform, bPropagateToChildren && Rotations.Num() == 0);
		}
	}

	if (Rotations.Num() > 0)
	{
		FTransform BaseTransform = FTransform::Identity;
		FRigElementKey ParentKey = Hierarchy->GetFirstParent(CachedItems[0].GetKey());
		if (ParentKey.IsValid())
		{
			BaseTransform = Hierarchy->GetGlobalTransform(ParentKey);
		}

		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
		{
			ItemLocalTransforms[Index] = Hierarchy->GetLocalTransform(CachedItems[Index]);
		}

		for (int32 Index = 0; Index < CachedItems.Num(); Index++)
		{
			if (ItemRotationA[Index] >= Rotations.Num() ||
				ItemRotationB[Index] >= Rotations.Num())
			{
				continue;
			}

			FQuat Rotation = Rotations[ItemRotationA[Index]].Rotation;
			FQuat RotationB = Rotations[ItemRotationB[Index]].Rotation;
			if (ItemRotationA[Index] != ItemRotationB[Index])
			{
				if (ItemRotationT[Index] > 1.f - SMALL_NUMBER)
				{
					Rotation = RotationB;
				}
				else if (ItemRotationT[Index] > SMALL_NUMBER)
				{
					Rotation = FQuat::Slerp(Rotation, RotationB, ItemRotationT[Index]).GetNormalized();
				}
			}

			BaseTransform = ItemLocalTransforms[Index] * BaseTransform;
			Rotation = FQuat::Slerp(BaseTransform.GetRotation(), BaseTransform.GetRotation() * Rotation, FMath::Clamp<float>(Weight, 0.f, 1.f));
			BaseTransform.SetRotation(Rotation);
			Hierarchy->SetGlobalTransform(CachedItems[Index], BaseTransform, bPropagateToChildren);
		}
	}

	if (ExecuteContext.GetDrawInterface() != nullptr && DebugSettings.bEnabled)
	{
		FRigUnit_DebugBezierItemSpace::DrawBezier(ExecuteContext, DebugSettings.WorldOffset, Bezier, 0.f, 1.f, DebugSettings.CurveColor, DebugSettings.Scale, 64);
		ExecuteContext.GetDrawInterface()->DrawPoint(DebugSettings.WorldOffset, Bezier.A, DebugSettings.Scale * 6, DebugSettings.CurveColor);
		ExecuteContext.GetDrawInterface()->DrawPoint(DebugSettings.WorldOffset, Bezier.B, DebugSettings.Scale * 6, DebugSettings.CurveColor);
		ExecuteContext.GetDrawInterface()->DrawPoint(DebugSettings.WorldOffset, Bezier.C, DebugSettings.Scale * 6, DebugSettings.CurveColor);
		ExecuteContext.GetDrawInterface()->DrawPoint(DebugSettings.WorldOffset, Bezier.D, DebugSettings.Scale * 6, DebugSettings.CurveColor);
		ExecuteContext.GetDrawInterface()->DrawLineStrip(DebugSettings.WorldOffset, CurvePositions, DebugSettings.SegmentsColor, DebugSettings.Scale);
		ExecuteContext.GetDrawInterface()->DrawPoints(DebugSettings.WorldOffset, CurvePositions, DebugSettings.Scale * 4.f, DebugSettings.SegmentsColor);
		// ExecuteContext.GetDrawInterface()->DrawPoints(DebugSettings.WorldOffset, CurvePositions, DebugSettings.Scale * 3.f, FLinearColor::Blue);
		// ExecuteContext.GetDrawInterface()->DrawPoints(DebugSettings.WorldOffset, ItemPositions, DebugSettings.Scale * 6.f, DebugSettings.SegmentsColor);
	}
}

FRigVMStructUpgradeInfo FRigUnit_FitChainToCurveItemArray::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

