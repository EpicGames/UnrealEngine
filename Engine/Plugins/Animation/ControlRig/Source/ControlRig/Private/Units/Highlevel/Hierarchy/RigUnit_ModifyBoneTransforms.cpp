// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_ModifyBoneTransforms.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ModifyBoneTransforms)

FRigUnit_ModifyBoneTransforms_Execute()
{
	TArray<FRigUnit_ModifyTransforms_PerItem, TInlineAllocator<4>> ItemsToModify;
	for (int32 BoneIndex = 0; BoneIndex < BoneToModify.Num(); BoneIndex++)
	{
		FRigUnit_ModifyTransforms_PerItem ItemToModify;
		ItemToModify.Item = FRigElementKey(BoneToModify[BoneIndex].Bone, ERigElementType::Bone);
		ItemToModify.Transform = BoneToModify[BoneIndex].Transform;
		ItemsToModify.Add(ItemToModify);
	}

	FRigUnit_ModifyTransforms::StaticExecute(
		ExecuteContext,
		ItemsToModify,
		Weight,
		WeightMinimum,
		WeightMaximum,
		Mode,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_ModifyBoneTransforms::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ModifyBoneTransforms)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey BoneA = Controller->AddBone(TEXT("BoneA"), Root, FTransform(FVector(1.f, 2.f, 3.f)), true, ERigBoneType::User);
	const FRigElementKey BoneB = Controller->AddBone(TEXT("BoneB"), Root, FTransform(FVector(5.f, 6.f, 7.f)), true, ERigBoneType::User);

	Unit.BoneToModify.SetNumZeroed(2);
	Unit.BoneToModify[0].Bone = TEXT("BoneA");
	Unit.BoneToModify[1].Bone = TEXT("BoneB");
	Unit.BoneToModify[0].Transform = Unit.BoneToModify[1].Transform = FTransform(FVector(10.f, 11.f, 12.f));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Mode = EControlRigModifyBoneMode::AdditiveLocal;
	Execute();
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(0).GetTranslation() - FVector(1.f, 0.f, 0.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(1).GetTranslation() - FVector(11.f, 13.f, 15.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(2).GetTranslation() - FVector(15.f, 17.f, 19.f)).IsNearlyZero(), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Mode = EControlRigModifyBoneMode::AdditiveGlobal;
	Execute();
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(0).GetTranslation() - FVector(1.f, 0.f, 0.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(1).GetTranslation() - FVector(11.f, 13.f, 15.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(2).GetTranslation() - FVector(15.f, 17.f, 19.f)).IsNearlyZero(), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Mode = EControlRigModifyBoneMode::OverrideLocal;
	Execute();
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(0).GetTranslation() - FVector(1.f, 0.f, 0.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(1).GetTranslation() - FVector(11.f, 11.f, 12.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(2).GetTranslation() - FVector(11.f, 11.f, 12.f)).IsNearlyZero(), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Mode = EControlRigModifyBoneMode::OverrideGlobal;
	Execute();
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(0).GetTranslation() - FVector(1.f, 0.f, 0.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(1).GetTranslation() - FVector(10.f, 11.f, 12.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(2).GetTranslation() - FVector(10.f, 11.f, 12.f)).IsNearlyZero(), TEXT("unexpected transform"));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.Mode = EControlRigModifyBoneMode::AdditiveLocal;
	Unit.Weight = 0.5f;
	Execute();
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(0).GetTranslation() - FVector(1.f, 0.f, 0.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(1).GetTranslation() - FVector(6.f, 7.5f, 9.f)).IsNearlyZero(), TEXT("unexpected transform"));
	AddErrorIfFalse((Hierarchy->GetGlobalTransform(2).GetTranslation() - FVector(10.f, 11.5f, 13.f)).IsNearlyZero(), TEXT("unexpected transform"));


	return true;
}
#endif
