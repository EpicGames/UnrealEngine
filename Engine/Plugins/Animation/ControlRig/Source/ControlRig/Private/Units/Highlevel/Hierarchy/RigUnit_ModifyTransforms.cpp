// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_ModifyTransforms.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ModifyTransforms)

FRigUnit_ModifyTransforms_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if (CachedItems.Num() == 0)
		{
			CachedItems.SetNum(ItemToModify.Num());
		}

		float Minimum = FMath::Min<float>(WeightMinimum, WeightMaximum);
		float Maximum = FMath::Max<float>(WeightMinimum, WeightMaximum);

		if (Weight <= Minimum + SMALL_NUMBER || FMath::IsNearlyEqual(Minimum, Maximum))
		{
			return;
		}

		if (CachedItems.Num() == ItemToModify.Num())
		{
			float T = FMath::Clamp<float>((Weight - Minimum) / (Maximum - Minimum), 0.f, 1.f);
			bool bNeedsBlend = T < 1.f - SMALL_NUMBER;

			int32 EntryIndex = 0;
			for (const FRigUnit_ModifyTransforms_PerItem& Entry : ItemToModify)
			{
				FCachedRigElement& CachedItem = CachedItems[EntryIndex];
				if (!CachedItem.UpdateCache(Entry.Item, Hierarchy))
				{
					continue;
				}

				FTransform Transform = Entry.Transform;

				switch (Mode)
				{
					case EControlRigModifyBoneMode::OverrideLocal:
					{
						if (bNeedsBlend)
						{
							Transform = FControlRigMathLibrary::LerpTransform(Hierarchy->GetLocalTransform(CachedItem), Transform, T);
						}
						Hierarchy->SetLocalTransform(CachedItem, Transform, true);
						break;
					}
					case EControlRigModifyBoneMode::OverrideGlobal:
					{
						if (bNeedsBlend)
						{
							Transform = FControlRigMathLibrary::LerpTransform(Hierarchy->GetGlobalTransform(CachedItem), Transform, T);
						}
						Hierarchy->SetGlobalTransform(CachedItem, Transform, true);
						break;
					}
					case EControlRigModifyBoneMode::AdditiveLocal:
					{
						if (bNeedsBlend)
						{
							Transform = FControlRigMathLibrary::LerpTransform(FTransform::Identity, Transform, T);
						}
							
						FRigTransformElement* TransformElement = const_cast<FRigTransformElement*>(Cast<FRigTransformElement>(CachedItem.GetElement()));
						if(TransformElement == nullptr)
						{
							return;
						}

						// figure out which transform type has already been computed (is clean) to avoid compute
						ERigTransformType::Type TransformTypeToUse = ERigTransformType::CurrentLocal;
						if(TransformElement->GetDirtyState().IsDirty(ERigTransformType::CurrentLocal))
						{
							check(!TransformElement->GetDirtyState().IsDirty(ERigTransformType::CurrentGlobal));
							TransformTypeToUse = ERigTransformType::CurrentGlobal;
						}

						Transform = Transform * Hierarchy->GetTransform(TransformElement, TransformTypeToUse);
						Hierarchy->SetTransform(TransformElement, Transform, TransformTypeToUse, true);
						break;
					}
					case EControlRigModifyBoneMode::AdditiveGlobal:
					{
						if (bNeedsBlend)
						{
							Transform = FControlRigMathLibrary::LerpTransform(FTransform::Identity, Transform, T);
						}
						Transform = Hierarchy->GetGlobalTransform(CachedItem) * Transform;
						Hierarchy->SetGlobalTransform(CachedItem, Transform, true);
						break;
					}
					default:
					{
						break;
					}
				}
				EntryIndex++;
			}
		}
	}
}

#if WITH_EDITOR

bool FRigUnit_ModifyTransforms::GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const
{
	for(const FRigUnit_ModifyTransforms_PerItem& ItemInfo : ItemToModify)
	{
		if(ItemInfo.Item.IsValid())
		{
			static const UEnum* TypeEnum = StaticEnum<ERigElementType>();
			const FString Prefix = TypeEnum->GetDisplayNameTextByValue((int64)ItemInfo.Item.Type).ToString();
			InOutTargets.Emplace(FString::Printf(TEXT("%s %s"), *Prefix, *ItemInfo.Item.Name.ToString()), ERigControlType::EulerTransform);
		}
	}
	return !InOutTargets.IsEmpty();
}

bool FRigUnit_ModifyTransforms::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	const int32 Index = GetIndexFromTarget(InInfo->Target.Name);
	if (ItemToModify.IsValidIndex(Index))
	{
		if(!InInfo->bInitialized)
		{
			InInfo->OffsetTransform = FTransform::Identity;

			const FRigElementKey FirstParent = Hierarchy->GetFirstParent(ItemToModify[Index].Item);

			switch(Mode)
			{
				case EControlRigModifyBoneMode::AdditiveLocal:
				{
					// place the control offset transform where the target is now "without" the change provided by the pin
					InInfo->OffsetTransform = ItemToModify[Index].Transform.Inverse() * Hierarchy->GetGlobalTransform(ItemToModify[Index].Item);
					break;
				}
				case EControlRigModifyBoneMode::OverrideLocal:
				{
					// changing local means let's place the control offset transform where the parent is
					InInfo->OffsetTransform = Hierarchy->GetGlobalTransform(FirstParent);
					break;
				}
				case EControlRigModifyBoneMode::AdditiveGlobal:
				case EControlRigModifyBoneMode::OverrideGlobal:
				default:
				{
					// in this case the value input is a global transform
					// so we'll leave the control offset transform at identity
					break;
				}
			}

		}

		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, InInfo->OffsetTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, ItemToModify[Index].Transform, false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, ItemToModify[Index].Transform, true);
		}
		return true;
	}
	return false;
}

bool FRigUnit_ModifyTransforms::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	const int32 Index = GetIndexFromTarget(InInfo->Target.Name);
	if (ItemToModify.IsValidIndex(Index))
	{
		ItemToModify[Index].Transform = Hierarchy->GetLocalTransform(InInfo->ControlKey);;
		return true;
	}
	return false;
}

TArray<const URigVMPin*> FRigUnit_ModifyTransforms::GetPinsForDirectManipulation(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const
{
	TArray<const URigVMPin*> AffectedPins;
	const int32 Index = GetIndexFromTarget(InTarget.Name);
	if (ItemToModify.IsValidIndex(Index))
	{
		if(const URigVMPin* ItemToModifyArrayPin = InNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_ModifyTransforms, ItemToModify)))
		{
			if(const URigVMPin* ItemToModifyElementPin = ItemToModifyArrayPin->FindSubPin(FString::FromInt(Index)))
			{
				if(const URigVMPin* TransformPin = ItemToModifyElementPin->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_ModifyTransforms_PerItem, Transform)))
				{
					AffectedPins.Add(TransformPin);
				}
			}
		}
	}
	return AffectedPins;
}

int32 FRigUnit_ModifyTransforms::GetIndexFromTarget(const FString& InTarget) const
{
	FString Left, Right;
	if(InTarget.Split(TEXT(" "), &Left, &Right))
	{
		static const UEnum* TypeEnum = StaticEnum<ERigElementType>();
		static TArray<FString> DisplayNames;
		if(DisplayNames.IsEmpty())
		{
			const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();
			for(int64 Index = 0; ; Index++)
			{
				if ((ERigElementType)ElementTypeEnum->GetValueByIndex(Index) == ERigElementType::All)
				{
					break;
				}
				DisplayNames.Add(TypeEnum->GetDisplayNameTextByValue(Index).ToString());
			}
		}

		const int32 TypeIndex = DisplayNames.Find(Left);
		if(TypeIndex != INDEX_NONE)
		{
			const ERigElementType ElementType = (ERigElementType)TypeIndex;
			const FName ElementName(*Right);
			for(int32 Index = 0; Index < ItemToModify.Num(); Index++)
			{
				if(ItemToModify[Index].Item == FRigElementKey(ElementName, ElementType))
				{
					return Index;
				}
			}
		}
	}
	return INDEX_NONE;
}

#endif