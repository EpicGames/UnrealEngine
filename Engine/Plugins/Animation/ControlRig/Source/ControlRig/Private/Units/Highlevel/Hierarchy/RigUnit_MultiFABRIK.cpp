// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_MultiFABRIK.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_MultiFABRIK)

// multi effector utility functions
namespace MultiFABRIK
{
	struct FChainNode
	{
		FChainNode* Parent;
		int32 BoneTreeIndex;
		TArray<FChainNode*> Children;
	};

	// forward reach, focusing on getting to end effector
	bool ForwardReach(TArray<FFABRIKChainLink>& InOutChain, const FVector& TargetPosition, float Precision)
	{
		bool bBoneLocationUpdated = false;
		int32 const NumChainLinks = InOutChain.Num();
		int32 const TipBoneLinkIndex = NumChainLinks - 1;

		// Check distance between tip location and effector location
		float Slop = FVector::DistSquared(InOutChain[TipBoneLinkIndex].Position, TargetPosition);
		if (Slop > Precision*Precision)
		{
			// Set tip bone at end effector location.
			InOutChain[TipBoneLinkIndex].Position = TargetPosition;

			// "Forward Reaching" stage - adjust bones from end effector.
			for (int32 LinkIndex = TipBoneLinkIndex - 1; LinkIndex >= 0; LinkIndex--)
			{
				FFABRIKChainLink & CurrentLink = InOutChain[LinkIndex];
				FFABRIKChainLink const & ChildLink = InOutChain[LinkIndex + 1];

				CurrentLink.Position = ChildLink.Position + (CurrentLink.Position - ChildLink.Position).GetUnsafeNormal() * ChildLink.Length;
			}

			bBoneLocationUpdated = true;
		}

		return bBoneLocationUpdated;
	}

	// backward reach, starting from root to end effector. ensure you always prioritize root
	bool BackwardReach(TArray<FFABRIKChainLink>& InOutChain)
	{
		bool bBoneLocationUpdated = false;
		int32 const NumChainLinks = InOutChain.Num();

		// back reach has to be done all the time because we push back to the correct location from FABRIK, and we give root position from bone position
		// "Backward Reaching" stage - adjust bones from root.
		for (int32 LinkIndex = 1; LinkIndex < NumChainLinks; LinkIndex++)
		{
			FFABRIKChainLink const & ParentLink = InOutChain[LinkIndex - 1];
			FFABRIKChainLink & CurrentLink = InOutChain[LinkIndex];

			CurrentLink.Position = ParentLink.Position + (CurrentLink.Position - ParentLink.Position).GetUnsafeNormal() * CurrentLink.Length;
		}

		return true;
	}

	// update current WIP bone transform back to links
	void UpdateTransfomRecusively(FRigUnit_MultiFABRIK_ChainGroup& InOutCurrentGroup, TArray<FRigUnit_MultiFABRIK_BoneWorkingData>& InBoneTree)
	{
		float MaxLength = 0.f;
		for (int32 LinkIndex = 0; LinkIndex < InOutCurrentGroup.Chain.Link.Num(); ++LinkIndex)
		{
			FFABRIKChainLink& Link = InOutCurrentGroup.Chain.Link[LinkIndex];
			Link.Position = InBoneTree[Link.TransformIndex].Location;
			Link.Length = InBoneTree[Link.TransformIndex].BoneLength;
			MaxLength += Link.Length;
		}

		InOutCurrentGroup.Chain.MaxLength = MaxLength;

		for (int32 ChildIndex = 0; ChildIndex < InOutCurrentGroup.Children.Num(); ++ChildIndex)
		{
			UpdateTransfomRecusively(InOutCurrentGroup.Children[ChildIndex], InBoneTree);
		}

		// if no children, it's good
	}

	// Forward Evaluator
	void SolveForwardRecursively(FRigUnit_MultiFABRIK_ChainGroup& InOutCurrentGroup, TArray<FRigUnit_MultiFABRIK_BoneWorkingData>& InBoneTree, const TArrayView<const FRigUnit_MultiFABRIK_EndEffector>& InEffectors, float InPrecision)
	{
		const int32 ChildNum = InOutCurrentGroup.Children.Num();
		// solve child first, and then follow
		for (int32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			SolveForwardRecursively(InOutCurrentGroup.Children[ChildIndex], InBoneTree, InEffectors, InPrecision);
		}

		FRigUnit_MultiFABRIK_Chain& Chain = InOutCurrentGroup.Chain;

		// update effector locations
		// if i have effector, follow effector
		if (Chain.EffectorArrayIndex != INDEX_NONE)
		{
			// update current effector location
			Chain.EffectorLocation = InEffectors[Chain.EffectorArrayIndex].Location;
		}
		else if (ensure(ChildNum > 1))
		{
			// otherwise, I'll follow my children average root position
			// now find the average root position of my children

			FVector BasePosition = FVector::ZeroVector;
			for (int32 ChildIndex = 0; ChildIndex < InOutCurrentGroup.Children.Num(); ++ChildIndex)
			{
				BasePosition += InOutCurrentGroup.Children[ChildIndex].Chain.Link[0].Position;
			}

			Chain.EffectorLocation = BasePosition / ChildNum;
		}

		// if we moved forward, we also do backward
		ForwardReach(Chain.Link, Chain.EffectorLocation, InPrecision);

		// we copy back the transform except root because root has to be calculated
		for (int32 LinkId = 1; LinkId < Chain.Link.Num(); ++LinkId)
		{
			FFABRIKChainLink& Link = Chain.Link[LinkId];
			InBoneTree[Link.TransformIndex].Location = Link.Position;
		}
	}

	// Backward Evaluator
	void SolveBackwardRecursively(FRigUnit_MultiFABRIK_ChainGroup& InOutCurrentGroup, TArray<FRigUnit_MultiFABRIK_BoneWorkingData>& InBoneTree, float InPrecision)
	{
		FRigUnit_MultiFABRIK_Chain& Chain = InOutCurrentGroup.Chain;

		// we copy back the transform except root because root has to be calculated
		for (int32 LinkId = 0; LinkId < Chain.Link.Num(); ++LinkId)
		{
			FFABRIKChainLink& Link = Chain.Link[LinkId];
			Link.Position = InBoneTree[Link.TransformIndex].Location;
		}

		// solve me first and solve children
		// if we moved forward, we also do backward
		ensure(Chain.Link.Num() >= 1);
		BackwardReach(Chain.Link);

		// we copy back the transform except root because root doesn't change
		for (int32 LinkId = 1; LinkId < Chain.Link.Num(); ++LinkId)
		{
			FFABRIKChainLink& Link = Chain.Link[LinkId];
			InBoneTree[Link.TransformIndex].Location = Link.Position;
		}

		const int32 ChildNum = InOutCurrentGroup.Children.Num();
		// solve child first, and then follow
		for (int32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			SolveBackwardRecursively(InOutCurrentGroup.Children[ChildIndex], InBoneTree, InPrecision);
		}
	}

	// add new links 
	static void AddNewLinkRecursive(FChainNode* Current, FRigUnit_MultiFABRIK_ChainGroup& InOutCurrentGroup, TArray<FRigUnit_MultiFABRIK_BoneWorkingData>& InBoneTree, TArray<int32>& InEffectorIndices)
	{
		if (ensure(Current))
		{
			FFABRIKChainLink NewLink;
			NewLink.BoneIndex = InBoneTree[Current->BoneTreeIndex].BoneIndex;
			NewLink.TransformIndex = Current->BoneTreeIndex;
			InOutCurrentGroup.Chain.Link.Add(NewLink);

			if (Current->Children.Num() > 1)
			{
				InOutCurrentGroup.Children.AddDefaulted(Current->Children.Num());
				for (int32 ChildIndex = 0; ChildIndex < Current->Children.Num(); ++ChildIndex)
				{
					// child needs to add root first
					InOutCurrentGroup.Children[ChildIndex].Chain.Link.Add(NewLink);
					InOutCurrentGroup.Children[ChildIndex].RootBoneTreeIndex = Current->BoneTreeIndex;
					AddNewLinkRecursive(Current->Children[ChildIndex], InOutCurrentGroup.Children[ChildIndex], InBoneTree, InEffectorIndices);
				}
			}
			else if (Current->Children.Num() == 1)
			{
				AddNewLinkRecursive(Current->Children[0], InOutCurrentGroup, InBoneTree, InEffectorIndices);
			}
			else
			{
				// if no children, it's good
				// I'm the effector, add effector array index for later access
				InOutCurrentGroup.Chain.EffectorArrayIndex = InEffectorIndices.Find(NewLink.BoneIndex);
			}
		}
	}

	// Recalculate rotation based on direction of translation change
	void RecalculateRotationBasedOnDirection(const FVector& OldDir, const FVector& NewDir, FTransform& OutCurrentTransform)
	{
		// Calculate axis of rotation from pre-translation vector to post-translation vector
		FVector const RotationAxis = FVector::CrossProduct(OldDir, NewDir).GetSafeNormal();
		float const RotationAngle = FMath::Acos(FVector::DotProduct(OldDir, NewDir));
		FQuat const DeltaRotation = FQuat(RotationAxis, RotationAngle);
		// We're going to multiply it, in order to not have to re-normalize the final quaternion, it has to be a unit quaternion.
		ensure(DeltaRotation.IsNormalized());

		OutCurrentTransform.SetRotation(DeltaRotation * OutCurrentTransform.GetRotation());
		OutCurrentTransform.NormalizeRotation();
	}
};
/////////////////////////////////////////////////////
// AnimNode_MultiFabrik
// Implementation of the FABRIK IK Algorithm
// Please see http://andreasaristidou.com/publications/FABRIK.pdf for more details

FRigUnit_MultiFABRIK_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	// workdata reference
	FRigUnit_MultiFABRIK_ChainGroup&				ChainGroup = WorkData.ChainGroup;
	TArray<FRigUnit_MultiFABRIK_BoneWorkingData>&	BoneTree = WorkData.BoneTree;
	TArray<FCachedRigElement>&				        EffectorIndices = WorkData.EffectorBoneIndices;

	if (!bIsInitialized)
	{
		EffectorIndices.Reset();
		BoneTree.Reset();
		ChainGroup.Reset();
		bIsInitialized = true;
	}
	
	if(BoneTree.Num() == 0)
	{
		// verify the chain
		const FRigElementKey RootKey(RootBone, ERigElementType::Bone);
		const int32 RootBoneIndex = Hierarchy->GetIndex(RootKey);
		if (RootBoneIndex != INDEX_NONE)
		{
			// fill up all effector indices
			for (int32 Index = 0; Index < Effectors.Num(); ++Index)
			{
				const FRigElementKey EffectorKey(Effectors[Index].Bone, ERigElementType::Bone);
				EffectorIndices.Add(FCachedRigElement(EffectorKey, Hierarchy));
			}

			// go up to until you meet root
			struct FLocalBoneData
			{
				FName BoneName;
				int32 BoneIndex;
				FName ParentBoneName;
				int32 ParentIndex;

				FLocalBoneData(const FName& InBoneName, int32 InBoneIndex, const FName& InParentBoneName, int32 InParentIndex)
					: BoneName(InBoneName)
					, BoneIndex(InBoneIndex)
					, ParentBoneName(InParentBoneName)
					, ParentIndex(InParentIndex)
				{}

				FLocalBoneData(URigHierarchy* InHierarchy, int32 InBoneIndex)
				: BoneName(NAME_None)
				, BoneIndex(InBoneIndex)
				, ParentBoneName(NAME_None)
			    , ParentIndex(INDEX_NONE)
				{
					FRigBaseElement* BoneElement = InHierarchy->Get(InBoneIndex);
					BoneName = BoneElement->GetFName();
					BoneIndex = BoneElement->GetIndex();

					if(FRigBaseElement* ParentBoneElement = InHierarchy->GetFirstParent(BoneElement))
					{
						ParentBoneName = ParentBoneElement->GetFName();
						ParentIndex = ParentBoneElement->GetIndex();
					}
				}

			};

			TMap<FName, int32> ExistingBoneList;
			// now fill up data on bonedata_workdata
			// get all list of joints to fill up BoneData array
			for (int32 Index = 0; Index < EffectorIndices.Num(); ++Index)
			{
				const int32 EffectorIndex = EffectorIndices[Index];
				if (EffectorIndex != INDEX_NONE)
				{
					TArray<FLocalBoneData> BoneList;

					int32 CurrentIndex = EffectorIndex;
					do 
					{
						BoneList.Insert(FLocalBoneData(Hierarchy, CurrentIndex), 0);
						CurrentIndex = Hierarchy->GetIndex(Hierarchy->GetFirstParent(Hierarchy->GetKey(CurrentIndex)));
					} while (CurrentIndex != RootBoneIndex && CurrentIndex != INDEX_NONE);

					// if we haven't got to root, this is not valid chain
					if (CurrentIndex == RootBoneIndex)
					{
						// add root
						BoneList.Insert(FLocalBoneData(Hierarchy, RootBoneIndex), 0);

						// go through all bone list collected
						for (const FLocalBoneData& Bone : BoneList)
						{
							// if it's not collected yet, add to the list
							int32* FoundIndex = ExistingBoneList.Find(Bone.BoneName);
							int32 TreeIndex = INDEX_NONE;
							FRigUnit_MultiFABRIK_BoneWorkingData* Data = nullptr;
							if (FoundIndex == nullptr)
							{
								TreeIndex = BoneTree.AddDefaulted();
								Data = &BoneTree[TreeIndex];

								Data->BoneName = Bone.BoneName;
								Data->BoneIndex = FCachedRigElement(FRigElementKey(Bone.BoneName, ERigElementType::Bone), Hierarchy);
								Data->ParentIndex = FCachedRigElement(FRigElementKey(Bone.ParentBoneName, ERigElementType::Bone), Hierarchy);

								// set bone length 
								if (Data->ParentIndex.IsValid())
								{
									// save size - this is initial. we could do this in every frame instead
									Data->BoneLength = Hierarchy->GetLocalTransform(Data->BoneIndex).GetTranslation().Size();
								}
								else
								{
									Data->BoneLength = 0.f;
								}

								// find tree index?
								int32* FoundParent = ExistingBoneList.Find(Bone.ParentBoneName);
								if (FoundParent)
								{
									Data->ParentTreeIndex = *FoundParent;
								}

								ExistingBoneList.Add(Data->BoneName, TreeIndex);
							}
							else
							{
								TreeIndex = *FoundIndex;
								Data = &BoneTree[TreeIndex];
							}

							if (Data->ParentTreeIndex != INDEX_NONE)
							{
								// add my index to my parent
								BoneTree[Data->ParentTreeIndex].ChildrenTreeIndices.AddUnique(TreeIndex);
							}
						}
					}
				}
			}

			// create tree from BoneTree, this way it's easier for us to create chain per group
			TArray<MultiFABRIK::FChainNode> ChainNodes;

			// first create all nodes
			ChainNodes.AddZeroed(BoneTree.Num());
			for (int32 Index = 0; Index < BoneTree.Num(); ++Index)
			{
				const int32 ParentTreeIndex = BoneTree[Index].ParentTreeIndex;
				ChainNodes[Index].Parent = (ParentTreeIndex != INDEX_NONE)? &ChainNodes[ParentTreeIndex] : nullptr;
				ChainNodes[Index].BoneTreeIndex = Index;
			}

			// then now add children
			for (int32 Index = 0; Index < BoneTree.Num(); ++Index)
			{
				if (BoneTree[Index].ChildrenTreeIndices.Num() > 0)
				{
					ChainNodes[Index].Children.AddZeroed(BoneTree[Index].ChildrenTreeIndices.Num());

					for (int32 ChildIndex = 0; ChildIndex < BoneTree[Index].ChildrenTreeIndices.Num(); ++ChildIndex)
					{
						const int32 ChildTreeIndex = BoneTree[Index].ChildrenTreeIndices[ChildIndex];
						ChainNodes[Index].Children[ChildIndex] = (ChildTreeIndex != INDEX_NONE) ? &ChainNodes[ChildTreeIndex] : nullptr;
					}
				}
			}

			// now we have tree 
			// add new link recursively
			// now go from root to child, and create group if you have more children, you can always go back 
			if (ChainNodes.Num() > 0)
			{
				ChainGroup.RootBoneTreeIndex = 0;

				// add new links recursively
				// we group per chain, and sub groups goes next
				// when you have multiple children, we add new groups, and each group gets chain
				TArray<int32> ResolvedEffectorIndices;
				ResolvedEffectorIndices.Reserve(EffectorIndices.Num());
				for (FCachedRigElement& EffectorIndex : EffectorIndices)
				{
					ResolvedEffectorIndices.Add(EffectorIndex);
				}
				MultiFABRIK::AddNewLinkRecursive(&ChainNodes[0], ChainGroup, BoneTree, ResolvedEffectorIndices);
			}
		}
	}

	if (BoneTree.Num() > 0)
	{
		// update bone tree data
		for (int32 BoneIndex = 0; BoneIndex < BoneTree.Num(); ++BoneIndex)
		{
			// update bone transform
			BoneTree[BoneIndex].Location = Hierarchy->GetGlobalTransform(BoneTree[BoneIndex].BoneIndex).GetLocation();
		}

		FVector RootLocation = BoneTree[0].Location;
		// now iterate 
		// we start from end to root group prioritizing getting to effector

		const TArrayView<const FRigUnit_MultiFABRIK_EndEffector> EffectorsView(Effectors.GetData(), Effectors.Num());
		for (int32 IterIndex = 0; IterIndex < MaxIterations; ++IterIndex)
		{
			// update transform first
			MultiFABRIK::UpdateTransfomRecusively(ChainGroup, BoneTree);
			MultiFABRIK::SolveForwardRecursively(ChainGroup, BoneTree, EffectorsView, Precision);

			// root position can change while forward reaching
			BoneTree[0].Location = RootLocation;

			// solve backward
			MultiFABRIK::SolveBackwardRecursively(ChainGroup, BoneTree, Precision);
		}

		TArray<FTransform> NewTransforms;

		// now we should copy bone tree to hierarchy
		const int32 NumBones = BoneTree.Num();
		NewTransforms.AddUninitialized(NumBones);
		NewTransforms[0] = Hierarchy->GetGlobalTransform(BoneTree[0].BoneIndex);

		// we don't modify root bone, and all of these should have parent
		for (int32 BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
		{
			// rotation will be mess
			NewTransforms[BoneIndex] = Hierarchy->GetGlobalTransform(BoneTree[BoneIndex].BoneIndex);
			FTransform ParentTransform = Hierarchy->GetGlobalTransform(BoneTree[BoneIndex].ParentIndex);
			FVector NewParentLocation = BoneTree[BoneTree[BoneIndex].ParentTreeIndex].Location;

			// Calculate pre-translation vector between this bone and child
			const FVector OldDir = (NewTransforms[BoneIndex].GetLocation()-ParentTransform.GetLocation()).GetUnsafeNormal();

			// Get vector from the post-translation bone to it's child
			const FVector  NewDir = (BoneTree[BoneIndex].Location - NewParentLocation).GetUnsafeNormal();

			NewTransforms[BoneIndex].SetLocation(BoneTree[BoneIndex].Location);
			MultiFABRIK::RecalculateRotationBasedOnDirection(OldDir, NewDir, NewTransforms[BoneTree[BoneIndex].ParentTreeIndex]);
		}

		// we don't modify root bone, and all of these should have parent
		for (int32 BoneIndex = 1; BoneIndex < NumBones; ++BoneIndex)
		{
			// only propagate, if you are leaf joints here
			// this means, only the last joint in the test
			Hierarchy->SetGlobalTransform(BoneTree[BoneIndex].BoneIndex, NewTransforms[BoneIndex], bPropagateToChildren);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_MultiFABRIK)
{
	const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(1.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Chain1_0 = Controller->AddBone(TEXT("Chain1_0"), Root, FTransform(FVector(1.f, 2.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Chain1_1 = Controller->AddBone(TEXT("Chain1_1"), Chain1_0, FTransform(FVector(3.f, 2.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Chain2_0 = Controller->AddBone(TEXT("Chain2_0"), Root, FTransform(FVector(-2.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Chain2_1 = Controller->AddBone(TEXT("Chain2_1"), Chain2_0, FTransform(FVector(-2.f, 3.f, 0.f)), true, ERigBoneType::User);

	// first validation test
	// make sure this doesn't crash
	Execute();

	Unit.RootBone = TEXT("Root");
	Unit.Effectors.AddDefaulted(2);

	// second make sure this doesn't crash
	InitAndExecute();

	// now add the data
	Unit.Effectors[0].Bone = TEXT("Chain1_1");
	Unit.Effectors[0].Location = FVector(3.f, 2.f, 0.f);
	Unit.Effectors[1].Bone = TEXT("Chain2_1");
	Unit.Effectors[1].Location = FVector(-2.f, 3.f, 0.f);
	Unit.bPropagateToChildren = true;

	InitAndExecute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(Chain1_1).GetTranslation().Equals(FVector(3.f, 2.f, 0.f)), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(Chain2_1).GetTranslation().Equals(FVector(-2.f, 3.f, 0.f)), TEXT("unexpected transform"));

	// root is (1, 0, 0)
	Unit.Effectors[0].Bone = TEXT("Chain1_1");
	Unit.Effectors[0].Location = FVector(4.f, 0.f, 0.f);
	Unit.Effectors[1].Bone = TEXT("Chain2_1");
	Unit.Effectors[1].Location = FVector(0.f, -5.f, 0.f);
	Unit.bPropagateToChildren = true;

	InitAndExecute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(Chain1_1).GetTranslation().Equals(FVector(4.f, 0.f, 0.f), 0.1f), TEXT("unexpected transform"));
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(Chain2_1).GetTranslation().Equals(FVector(0.f, -5.f, 0.f), 0.1f), TEXT("unexpected transform"));
	return true;
}
#endif
