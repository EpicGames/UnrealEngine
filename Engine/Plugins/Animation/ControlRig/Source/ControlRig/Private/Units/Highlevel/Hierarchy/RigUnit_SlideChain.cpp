// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_SlideChain.h"
#include "Math/ControlRigMathLibrary.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SlideChain)

FRigUnit_SlideChain_Execute()
{
	FRigElementKeyCollection Items;
	if(WorkData.CachedItems.Num() == 0)
	{
		Items = FRigElementKeyCollection::MakeFromChain(
			ExecuteContext.Hierarchy,
			FRigElementKey(StartBone, ERigElementType::Bone),
			FRigElementKey(EndBone, ERigElementType::Bone),
			false /* reverse */
		);
	}

	FRigUnit_SlideChainPerItem::StaticExecute(
		ExecuteContext, 
		Items,
		SlideAmount,
		bPropagateToChildren,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_SlideChain::GetUpgradeInfo() const
{
	// this node is no longer supported and the upgrade path is too complex.
	return FRigVMStructUpgradeInfo();
}

FRigUnit_SlideChainPerItem_Execute()
{
	FRigUnit_SlideChainItemArray::StaticExecute(ExecuteContext, Items.Keys, SlideAmount, bPropagateToChildren, WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_SlideChainPerItem::GetUpgradeInfo() const
{
	FRigUnit_SlideChainItemArray NewNode;
	NewNode.Items = Items.Keys;
	NewNode.SlideAmount = SlideAmount;
	NewNode.bPropagateToChildren = bPropagateToChildren;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigUnit_SlideChainItemArray_Execute()
{
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	float& ChainLength = WorkData.ChainLength;
	TArray<float>& ItemSegments = WorkData.ItemSegments;
	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	TArray<FTransform>& Transforms = WorkData.Transforms;
	TArray<FTransform>& BlendedTransforms = WorkData.BlendedTransforms;

	if(CachedItems.Num() == 0 && Items.Num() > 1)
	{
		ItemSegments.Reset();
		Transforms.Reset();
		BlendedTransforms.Reset();
		ChainLength = 0.f;

		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		if (CachedItems.Num() < 2)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough bones. You need at least two in the chain!"));
			return;
		}

		ItemSegments.SetNumZeroed(CachedItems.Num());
		ItemSegments[0] = 0;
		for (int32 Index = 1; Index < CachedItems.Num(); Index++)
		{
			FVector A = Hierarchy->GetGlobalTransform(CachedItems[Index - 1]).GetLocation();
			FVector B = Hierarchy->GetGlobalTransform(CachedItems[Index]).GetLocation();
			ItemSegments[Index] = (A - B).Size();
			ChainLength += ItemSegments[Index];
		}

		Transforms.SetNum(CachedItems.Num());
		BlendedTransforms.SetNum(CachedItems.Num());
	}

	if (CachedItems.Num() < 2 || ChainLength < SMALL_NUMBER)
	{
		return;
	}

	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		Transforms[Index] = Hierarchy->GetGlobalTransform(CachedItems[Index]);
	}

	for (int32 Index = 0; Index < Transforms.Num(); Index++)
	{
		int32 TargetIndex = Index;
		float Ratio = 0.f;
		float SlidePerBone = -SlideAmount * ChainLength;

		if (SlidePerBone > 0)
		{
			while (SlidePerBone > SMALL_NUMBER && TargetIndex < Transforms.Num() - 1)
			{
				TargetIndex++;
				SlidePerBone -= ItemSegments[TargetIndex];
			}
		}
		else
		{
			while (SlidePerBone < -SMALL_NUMBER && TargetIndex > 0)
			{
				SlidePerBone += ItemSegments[TargetIndex];
				TargetIndex--;
			}
		}

		if (TargetIndex < Transforms.Num() - 1)
		{
			if (ItemSegments[TargetIndex + 1] > SMALL_NUMBER)
			{
				if (SlideAmount < -SMALL_NUMBER)
				{
					Ratio = FMath::Clamp<float>(1.f - FMath::Abs<float>(SlidePerBone / ItemSegments[TargetIndex + 1]), 0.f, 1.f);
				}
				else
				{
					Ratio = FMath::Clamp<float>(SlidePerBone / ItemSegments[TargetIndex + 1], 0.f, 1.f);
				}
			}
		}

		BlendedTransforms[Index] = Transforms[TargetIndex];
		if (TargetIndex < Transforms.Num() - 1 && Ratio > SMALL_NUMBER && Ratio < 1.f - SMALL_NUMBER)
		{
			BlendedTransforms[Index] = FControlRigMathLibrary::LerpTransform(BlendedTransforms[Index], Transforms[TargetIndex + 1], Ratio);
		}
	}

	for (int32 Index = 0; Index < CachedItems.Num(); Index++)
	{
		if (Index < CachedItems.Num() - 1)
		{
			FVector CurrentX = BlendedTransforms[Index].GetRotation().GetAxisX();
			FVector DesiredX = BlendedTransforms[Index + 1].GetLocation() - BlendedTransforms[Index].GetLocation();
			FQuat OffsetQuat = FQuat::FindBetweenVectors(CurrentX, DesiredX);
			BlendedTransforms[Index].SetRotation(OffsetQuat * BlendedTransforms[Index].GetRotation());
		}
		Hierarchy->SetGlobalTransform(CachedItems[Index], BlendedTransforms[Index], bPropagateToChildren);
	}
}

