// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_SphericalPoseReader.h"
#include "Kismet/KismetMathLibrary.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SphericalPoseReader)

FRigUnit_SphericalPoseReader_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!DriverCache.UpdateCache(DriverItem, Hierarchy))
	{
		return;
	}

	// remap/clamp inputs
	RemapAndConvertInputs(
		InnerRegion,
		OuterRegion,
		ActiveRegionSize,
		ActiveRegionScaleFactors,
		FalloffSize,
		FalloffRegionScaleFactors,
		FlipWidthScaling,
		FlipHeightScaling);

	// get parent global space
	OptionalParentCache.UpdateCache(OptionalParentItem, Hierarchy);
	FRigElementKey ParentElementKey = OptionalParentCache.IsValid() ? OptionalParentCache.GetKey() : Hierarchy->GetFirstParent(DriverCache.GetKey());
	FTransform GlobalDriverParentTransform = Hierarchy->GetGlobalTransform(ParentElementKey, false);

	// check if we need to update the LocalDriverTransformInit
	if (!bCachedInitTransforms ||							// haven't cached yet?
		!CachedRotationOffset.Equals(RotationOffset) ||		// rotation offset modified
		CachedParentElementKey != ParentElementKey)			// swapped to different parent
	{
		bCachedInitTransforms = true;
		CachedRotationOffset = RotationOffset;
		CachedParentElementKey = ParentElementKey;

		FTransform GlobalDriverParentTransformInitInverse;
		if (OptionalParentCache.IsValid())
		{
			GlobalDriverParentTransformInitInverse = Hierarchy->GetGlobalTransformByIndex(OptionalParentCache, true).Inverse();
		}else
		{
			// if user does not specify a custom parent space, then simply use the driver's parent
			GlobalDriverParentTransformInitInverse = Hierarchy->GetParentTransformByIndex(DriverCache, true).Inverse();
		}

		// get local rotation space of driver
		const FTransform GlobalDriverTransformInit = Hierarchy->GetGlobalTransformByIndex(DriverCache, true);
		LocalDriverTransformInit = GlobalDriverTransformInit * GlobalDriverParentTransformInitInverse;
		// apply static offset in local space
		FQuat RotationOffsetQuat = FQuat::MakeFromEuler(RotationOffset);
		LocalDriverTransformInit.SetRotation(LocalDriverTransformInit.GetRotation() * RotationOffsetQuat);
	}

	// calculate world transform of sphere
	const FTransform GlobalDriverTransform = Hierarchy->GetGlobalTransformByIndex(DriverCache);
	FTransform WorldOffset = LocalDriverTransformInit * GlobalDriverParentTransform;
	WorldOffset.SetLocation(GlobalDriverTransform.GetLocation());

	// get driver axis in local space of sphere
	const FVector CurrentGlobalDriverAxis = GlobalDriverTransform.GetRotation().RotateVector(DriverAxis);
	DriverNormal = WorldOffset.InverseTransformVectorNoScale(CurrentGlobalDriverAxis).GetSafeNormal();

	//
	// evaluate the interpolation of the output param
	//

	// get angle between the driver direction and the local Z-axis of the sphere
	float DriverDotZAxis = FVector::DotProduct(DriverNormal, FVector::ZAxisVector);
	if (FMath::IsNearlyEqual(DriverDotZAxis, -1.0f, KINDA_SMALL_NUMBER))
	{
		// avoid singularity at negative pole (back of sphere)
		OutputParam = 0.0f;
		Debug.DrawDebug(WorldOffset, ExecuteContext.GetDrawInterface(), InnerRegion, OuterRegion, DriverNormal, Driver2D, OutputParam);
		return;
	}

	// remap angle from 0-PI, to 0-1
	const float AngleFromForward = FMath::Acos(DriverDotZAxis);
	const float Mag = RemapRange(AngleFromForward, 0.0f, PI, 0.0f, 1.0f);
	if (FMath::IsNearlyZero(Mag))
	{
		// avoid singularity at positive pole (guaranteed to be inside inner ellipse)
		// causes NaNs in DistanceToEllipse
		OutputParam = 1.0f;
		Debug.DrawDebug(WorldOffset, ExecuteContext.GetDrawInterface(), InnerRegion, OuterRegion, DriverNormal, Driver2D, OutputParam);
		return;
	}

	// convert to 2d polar coordinates, with magnitude normalized by range 0-PI
	Driver2D = DriverNormal;
	if (Mag > SMALL_NUMBER)
	{
		Driver2D.Z = 0.0f;
		Driver2D.Normalize();
		Driver2D *= Mag;
	}
	Driver2D.Z = -1.0f;
	
	const float PointX = Driver2D.X;
	const float PointY = Driver2D.Y;

	float EllipseWidth;
	float EllipseHeight;

	// query inner ellipse
	InnerRegion.GetEllipseWidthAndHeight(PointX, PointY, EllipseWidth, EllipseHeight);
	FEllipseQuery InnerEllipseResults;
	DistanceToEllipse(PointX, PointY, EllipseWidth, EllipseHeight, InnerEllipseResults);

	// query outer ellipse
	OuterRegion.GetEllipseWidthAndHeight(PointX, PointY, EllipseWidth, EllipseHeight);
	FEllipseQuery OuterEllipseResults;
	DistanceToEllipse(PointX, PointY, EllipseWidth, EllipseHeight, OuterEllipseResults);

	// calc output param
	OutputParam = CalcOutputParam(InnerEllipseResults, OuterEllipseResults);

	// do all debug drawing
	Debug.DrawDebug(WorldOffset, ExecuteContext.GetDrawInterface(), InnerRegion, OuterRegion, DriverNormal, Driver2D, OutputParam);
}

void FRigUnit_SphericalPoseReader::RemapAndConvertInputs(
	FSphericalRegion& InnerRegion,
	FSphericalRegion& OuterRegion,
	const float ActiveRegionSize,
	const FRegionScaleFactors& ActiveRegionScaleFactors,
	const float FalloffSize,
	const FRegionScaleFactors& FalloffRegionScaleFactors,
	const bool FlipWidth,
	const bool FlipHeight)
{
	// remap normalized inputs to angles
	float RegionAngle = ActiveRegionSize * 180.0f;
	RegionAngle = FMath::Min(FMath::Max(0.5f, RegionAngle), 178.0f);

	InnerRegion.RegionAngleRadians = FMath::DegreesToRadians(RegionAngle);
	InnerRegion.ScaleFactors = ActiveRegionScaleFactors;

	// clamp outer falloff angle to always be greater than inner
	float FalloffAngle = FalloffSize * 180.0f;
	FalloffAngle = FMath::Min(FMath::Max(RegionAngle+1.0f, FalloffAngle), 179.0f);
	OuterRegion.RegionAngleRadians = FMath::DegreesToRadians(FalloffAngle);

	// clamp falloff to always be outside inner angle
	const float InvOuterAngleRadians = 1.0f / OuterRegion.RegionAngleRadians;
	//
	const float PosWidthMin = (InnerRegion.RegionAngleRadians * ActiveRegionScaleFactors.PositiveWidth) * InvOuterAngleRadians;
	OuterRegion.ScaleFactors.PositiveWidth = FMath::Lerp(PosWidthMin, 1.0f, FalloffRegionScaleFactors.PositiveWidth);
	//
	const float NegWidthMin = (InnerRegion.RegionAngleRadians * ActiveRegionScaleFactors.NegativeWidth) * InvOuterAngleRadians;
	OuterRegion.ScaleFactors.NegativeWidth = FMath::Lerp(NegWidthMin, 1.0f, FalloffRegionScaleFactors.NegativeWidth);
	//
	const float PosHeightMin = (InnerRegion.RegionAngleRadians * ActiveRegionScaleFactors.PositiveHeight) * InvOuterAngleRadians;
	OuterRegion.ScaleFactors.PositiveHeight = FMath::Lerp(PosHeightMin, 1.0f, FalloffRegionScaleFactors.PositiveHeight);
	//
	const float NegHeightMin = (InnerRegion.RegionAngleRadians * ActiveRegionScaleFactors.NegativeHeight) * InvOuterAngleRadians;
	OuterRegion.ScaleFactors.NegativeHeight = FMath::Lerp(NegHeightMin, 1.0f, FalloffRegionScaleFactors.NegativeHeight);

	// optionally flip the scaling factors to better support mirrored setups
	if (FlipWidth)
	{
		Swap(InnerRegion.ScaleFactors.PositiveWidth, InnerRegion.ScaleFactors.NegativeWidth);
		Swap(OuterRegion.ScaleFactors.PositiveWidth, OuterRegion.ScaleFactors.NegativeWidth);
	}
	if (FlipHeight)
	{
		Swap(InnerRegion.ScaleFactors.PositiveHeight, InnerRegion.ScaleFactors.NegativeHeight);
		Swap(OuterRegion.ScaleFactors.PositiveHeight, OuterRegion.ScaleFactors.NegativeHeight);
	}
}

float FRigUnit_SphericalPoseReader::CalcOutputParam(
	const FEllipseQuery& InnerEllipseResults,
	const FEllipseQuery& OuterEllipseResults)
{
    if (InnerEllipseResults.IsInside)
    {
        return 1.0f; // inside inner ellipse
    }

    if (!OuterEllipseResults.IsInside)
    {
        return 0.0f; // outside outer ellipse
    }

    // we're between outer and inner ellipse, calculate falloff
    const float DistanceToOuter = FMath::Sqrt(OuterEllipseResults.DistSq);
    const float DistanceToInner = FMath::Sqrt(InnerEllipseResults.DistSq);
    const float TotalDistance = DistanceToInner + DistanceToOuter;
    if (TotalDistance < 0.0001f)
    {
        return 0.0f; // don't lerp when outer is VERY close to inner (avoid div by zero)
    }

    return 1.0f - (DistanceToInner / TotalDistance);
}

void FRigUnit_SphericalPoseReader::DistanceToEllipse(
	const float InX,
	const float InY,
	const float SizeX,
	const float SizeY,
	FEllipseQuery& OutEllipseQuery)
{
	// degenerate ellipse
	if (SizeX <= KINDA_SMALL_NUMBER || SizeY <= KINDA_SMALL_NUMBER)
	{
		OutEllipseQuery.ClosestX = 0.0f;
		OutEllipseQuery.ClosestY = 0.0f;
		OutEllipseQuery.DistSq = FMath::Pow(InX, 2.f) + FMath::Pow(InY, 2.f);
		OutEllipseQuery.IsInside = OutEllipseQuery.DistSq <= KINDA_SMALL_NUMBER;
		return;
	}
	
    const float PX = FMath::Abs(InX);
    const float PY = FMath::Abs(InY);

	const float SizeXSq = SizeX * SizeX;
	const float SizeYSq = SizeY * SizeY;

	const float InvSizeX = 1.0f / SizeX;
	const float InvSizeY = 1.0f / SizeY;
	
    float TX = 0.70710678118f;
    float TY = 0.70710678118f;

    const int Iterations = 2; // this could be higher for accuracy, but 2 seems good enough for this use case
    for (int i=0; i<Iterations; ++i)
    {
        const float ScaledX = SizeX * TX;
        const float ScaledY = SizeY * TY;

        const float EX = (SizeXSq - SizeYSq) * (TX * TX * TX) * InvSizeX;
        const float EY = (SizeYSq - SizeXSq) * (TY * TY * TY) * InvSizeY;

        const float RX = ScaledX - EX;
        const float RY = ScaledY - EY;

        const float QX = PX - EX;
        const float QY = PY - EY;

        const float R = FMath::Sqrt(RX * RX + RY * RY);
        const float Q = FMath::Sqrt(QY * QY + QX * QX);

        TX = FMath::Min(1.f, FMath::Max(0.f, (QX * R / Q + EX) * InvSizeX));
        TY = FMath::Min(1.f, FMath::Max(0.f, (QY * R / Q + EY) * InvSizeY));

        const float InvT = 1.0f / FMath::Sqrt(TX * TX + TY * TY);

        TX *= InvT;
        TY *= InvT;
    }

    OutEllipseQuery.ClosestX = SizeX * (InX < 0 ? -TX : TX);
    OutEllipseQuery.ClosestY = SizeY * (InY < 0 ? -TY : TY);
    const float ToClosestX = OutEllipseQuery.ClosestX - InX;
    const float ToClosestY = OutEllipseQuery.ClosestY - InY;
    OutEllipseQuery.DistSq = FMath::Pow(ToClosestX, 2.f) + FMath::Pow(ToClosestY, 2.f);
    const float CenterToClosestDistSq = FMath::Pow(OutEllipseQuery.ClosestX, 2.f) + FMath::Pow(OutEllipseQuery.ClosestY, 2.f);
    const float CenterToInputDistSq = FMath::Pow(InX, 2.f) + FMath::Pow(InY, 2.f);
    OutEllipseQuery.IsInside = CenterToClosestDistSq > CenterToInputDistSq;
}

float FRigUnit_SphericalPoseReader::RemapRange(
	const float T,
	const float AStart,
	const float AEnd,
	const float BStart,
	const float BEnd)
{
	check(FMath::Abs(AEnd - AStart) > 0.0f);
	return BStart + (T - AStart) * (BEnd - BStart) / (AEnd - AStart);
}

#if WITH_EDITOR

// Target name constants for per-axis DM controls.
static const FString SPR_PosWidthTarget      = TEXT("ActiveRegionScaleFactors.PositiveWidth");
static const FString SPR_NegWidthTarget      = TEXT("ActiveRegionScaleFactors.NegativeWidth");
static const FString SPR_PosHeightTarget     = TEXT("ActiveRegionScaleFactors.PositiveHeight");
static const FString SPR_NegHeightTarget     = TEXT("ActiveRegionScaleFactors.NegativeHeight");
static const FString SPR_FallPosWidthTarget  = TEXT("FalloffRegionScaleFactors.PositiveWidth");
static const FString SPR_FallNegWidthTarget  = TEXT("FalloffRegionScaleFactors.NegativeWidth");
static const FString SPR_FallPosHeightTarget = TEXT("FalloffRegionScaleFactors.PositiveHeight");
static const FString SPR_FallNegHeightTarget = TEXT("FalloffRegionScaleFactors.NegativeHeight");

bool FRigUnit_SphericalPoseReader::GetDirectManipulationTargets(
	const URigVMUnitNode* InNode,
	TSharedPtr<FStructOnScope> InInstance,
	URigHierarchy* InHierarchy,
	TArray<FRigDirectManipulationTarget>& InOutTargets,
	FString* OutFailureReason) const
{
	static const FString RotationOffsetName  = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, RotationOffset);
	static const FString ActiveRegionSizeName = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, ActiveRegionSize);
	static const FString FalloffSizeName      = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, FalloffSize);

	// Build the target list in explicit display order rather than relying on auto-detection.
	// DriverAxis is intentionally excluded (not useful to manipulate spatially).
	InOutTargets.Reset();
	InOutTargets.Add({RotationOffsetName,        ERigControlType::Rotator});
	InOutTargets.Add({ActiveRegionSizeName,       ERigControlType::Scale});
	InOutTargets.Add({FalloffSizeName,            ERigControlType::Scale});
	InOutTargets.Add({SPR_PosWidthTarget,         ERigControlType::Float});
	InOutTargets.Add({SPR_PosHeightTarget,        ERigControlType::Float});
	InOutTargets.Add({SPR_NegWidthTarget,         ERigControlType::Float});
	InOutTargets.Add({SPR_NegHeightTarget,        ERigControlType::Float});
	InOutTargets.Add({SPR_FallPosWidthTarget,     ERigControlType::Float});
	InOutTargets.Add({SPR_FallPosHeightTarget,    ERigControlType::Float});
	InOutTargets.Add({SPR_FallNegWidthTarget,     ERigControlType::Float});
	InOutTargets.Add({SPR_FallNegHeightTarget,    ERigControlType::Float});

	return true;
}

void FRigUnit_SphericalPoseReader::ConfigureDirectManipulationControl(
	const URigVMUnitNode* InNode,
	TSharedPtr<FRigDirectManipulationInfo> InInfo,
	FRigControlSettings& InOutSettings,
	FRigControlValue& InOutValue) const
{
	if (InInfo->Target.Name.StartsWith(TEXT("ActiveRegionScaleFactors."), ESearchCase::CaseSensitive) ||
	    InInfo->Target.Name.StartsWith(TEXT("FalloffRegionScaleFactors."), ESearchCase::CaseSensitive))
	{
		InOutSettings.ControlType = ERigControlType::Float;
		InOutSettings.PrimaryAxis = ERigControlAxis::X;
		InOutValue = FRigControlValue::Make(1.0f);
		return;
	}

	static const FString ActiveRegionSizeName = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, ActiveRegionSize);
	static const FString FalloffSizeName      = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, FalloffSize);
	if (InInfo->Target.Name.Equals(ActiveRegionSizeName, ESearchCase::CaseSensitive) ||
	    InInfo->Target.Name.Equals(FalloffSizeName, ESearchCase::CaseSensitive))
	{
		InOutSettings.ControlType = ERigControlType::Scale;
		InOutValue = FRigControlValue::Make(FVector::OneVector);
		return;
	}

	FRigUnit::ConfigureDirectManipulationControl(InNode, InInfo, InOutSettings, InOutValue);
}

bool FRigUnit_SphericalPoseReader::UpdateHierarchyForDirectManipulation(
	const URigVMUnitNode* InNode,
	TSharedPtr<FStructOnScope> InInstance,
	FControlRigExecuteContext& InContext,
	TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	const FRigUnit_SphericalPoseReader* PoseReader = reinterpret_cast<const FRigUnit_SphericalPoseReader*>(InInstance->GetStructMemory());

	// Guard: if the driver cache hasn't been populated yet (node hasn't executed, or DriverItem is None),
	// GetFirstParent would receive an invalid key and crash inside GetIndex. Fall back to base.
	if (!PoseReader->DriverCache.IsValid())
	{
		return FRigUnit::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
	}

	static const FString RotationOffsetName = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, RotationOffset);
	if (InInfo->Target.Name.Equals(RotationOffsetName, ESearchCase::CaseSensitive))
	{
		const FQuat CurrentOffsetQuat = FQuat::MakeFromEuler(PoseReader->RotationOffset);

		// Build the gizmo's offset transform at the sphere pivot, oriented to the sphere's base space.
		// LocalDriverTransformInit has RotationOffset baked in; back it out to get the base orientation.
		FTransform BaseLocalTransform = PoseReader->LocalDriverTransformInit;
		BaseLocalTransform.SetRotation(PoseReader->LocalDriverTransformInit.GetRotation() * CurrentOffsetQuat.Inverse());

		const FRigElementKey ParentKey = PoseReader->OptionalParentCache.IsValid()
			? PoseReader->OptionalParentCache.GetKey()
			: Hierarchy->GetFirstParent(PoseReader->DriverCache.GetKey());
		FTransform OffsetTransform = BaseLocalTransform * Hierarchy->GetGlobalTransform(ParentKey, false);
		OffsetTransform.SetLocation(Hierarchy->GetGlobalTransformByIndex(PoseReader->DriverCache).GetLocation());

		// Offset = sphere base world transform; local rotation = RotationOffset in sphere space
		const FTransform LocalRotTransform(CurrentOffsetQuat);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalRotTransform, false);
		if (!InInfo->bInitialized)
		{
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, true);
			Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalRotTransform, true);
		}
		return true;
	}

	if (InInfo->Target.Name.StartsWith(TEXT("ActiveRegionScaleFactors."), ESearchCase::CaseSensitive))
	{
		// Orient the scale gizmo to the sphere's coordinate space (parent space + RotationOffset),
		// then rotate it so the X handle faces the target quadrant:
		//   PositiveWidth  — identity (X = +width)
		//   NegativeWidth  — 180° around Z  (X = -width)
		//   PositiveHeight — +90° around Z  (X = +height)
		//   NegativeHeight — -90° around Z  (X = -height)
		// LocalDriverTransformInit already has RotationOffset baked in.
		const FRigElementKey ParentKey = PoseReader->OptionalParentCache.IsValid()
			? PoseReader->OptionalParentCache.GetKey()
			: Hierarchy->GetFirstParent(PoseReader->DriverCache.GetKey());
		FTransform OffsetTransform = PoseReader->LocalDriverTransformInit * Hierarchy->GetGlobalTransform(ParentKey, false);
		OffsetTransform.SetLocation(Hierarchy->GetGlobalTransformByIndex(PoseReader->DriverCache).GetLocation());

		// When FlipWidthScaling/FlipHeightScaling is active, RemapAndConvertInputs swaps the
		// Positive/Negative scale factors for that axis, so the gizmo for each field must face
		// the opposite direction to stay visually aligned with the side it actually controls.
		const FQuat PosWidthRot  = PoseReader->FlipWidthScaling  ? FQuat(FVector::ZAxisVector,  PI)        : FQuat::Identity;
		const FQuat NegWidthRot  = PoseReader->FlipWidthScaling  ? FQuat::Identity                         : FQuat(FVector::ZAxisVector,  PI);
		const FQuat PosHeightRot = PoseReader->FlipHeightScaling ? FQuat(FVector::ZAxisVector, -PI * 0.5f) : FQuat(FVector::ZAxisVector,  PI * 0.5f);
		const FQuat NegHeightRot = PoseReader->FlipHeightScaling ? FQuat(FVector::ZAxisVector,  PI * 0.5f) : FQuat(FVector::ZAxisVector, -PI * 0.5f);

		float FieldValue = 1.0f;
		FQuat QuadrantRot = FQuat::Identity;
		const FString& TargetName = InInfo->Target.Name;
		if (TargetName.Equals(SPR_PosWidthTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->ActiveRegionScaleFactors.PositiveWidth;
			QuadrantRot = PosWidthRot;
		}
		else if (TargetName.Equals(SPR_NegWidthTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->ActiveRegionScaleFactors.NegativeWidth;
			QuadrantRot = NegWidthRot;
		}
		else if (TargetName.Equals(SPR_PosHeightTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->ActiveRegionScaleFactors.PositiveHeight;
			QuadrantRot = PosHeightRot;
		}
		else if (TargetName.Equals(SPR_NegHeightTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->ActiveRegionScaleFactors.NegativeHeight;
			QuadrantRot = NegHeightRot;
		}

		// Place the gizmo at the visual pole of the inner region in this quadrant so it sits
		// on the sphere surface. Use effective (flip-adjusted) scale factors so the pole
		// matches what the debug draw shows.
		FRegionScaleFactors EffectiveScaleFactors = PoseReader->ActiveRegionScaleFactors;
		if (PoseReader->FlipWidthScaling)
			Swap(EffectiveScaleFactors.PositiveWidth, EffectiveScaleFactors.NegativeWidth);
		if (PoseReader->FlipHeightScaling)
			Swap(EffectiveScaleFactors.PositiveHeight, EffectiveScaleFactors.NegativeHeight);

		const float RegionAngleRadians = FMath::DegreesToRadians(FMath::Clamp(PoseReader->ActiveRegionSize * 180.0f, 0.5f, 178.0f));
		FSphericalRegion TempRegion;
		TempRegion.RegionAngleRadians = RegionAngleRadians;
		TempRegion.ScaleFactors = EffectiveScaleFactors;
		// Clamp each scale factor to a small minimum so the pole never reaches the north-pole
		// singularity (where the tangent degenerates). Without this, dragging a scale factor to 0
		// causes the gizmo's local X axis to flip sign, producing a large positive pop on readback.
		// The actual pin value is still written as 0 via LocalTranslateTransform.X = FieldValue * SS.
		TempRegion.ScaleFactors.PositiveWidth  = FMath::Max(TempRegion.ScaleFactors.PositiveWidth,  0.001f);
		TempRegion.ScaleFactors.NegativeWidth  = FMath::Max(TempRegion.ScaleFactors.NegativeWidth,  0.001f);
		TempRegion.ScaleFactors.PositiveHeight = FMath::Max(TempRegion.ScaleFactors.PositiveHeight, 0.001f);
		TempRegion.ScaleFactors.NegativeHeight = FMath::Max(TempRegion.ScaleFactors.NegativeHeight, 0.001f);

		// Derive the spin angle for this quadrant (angle of the gizmo's X axis in sphere 2D space)
		const FVector SpinDir = QuadrantRot * FVector::XAxisVector;
		const float SpinAngle = FMath::Atan2(SpinDir.Y, SpinDir.X);

		const FVector Pole3D = FSphericalPoseReaderDebugSettings::SphericalCoordinatesToNormal(
			FSphericalPoseReaderDebugSettings::CalcSphericalCoordinates(SpinAngle, TempRegion));
		const FVector PoleWorld = OffsetTransform.TransformPosition(Pole3D * PoseReader->Debug.DebugScale);

		// Orient X axis along the sphere surface tangent at the pole in the outward direction.
		// Tangent = d/dθ [SphericalCoordinatesToNormal] = normalize(Pole3D.Z * Pole3D - ZAxis).
		// This naturally points toward +/- width or height depending on the quadrant.
		const FVector TangentInSphereSpace = (Pole3D * Pole3D.Z - FVector::ZAxisVector).GetSafeNormal();
		// World-space tangent, computed before TangentRot is applied to OffsetTransform.
		const FVector TangentWorldDir = OffsetTransform.GetRotation().RotateVector(TangentInSphereSpace);
		const FQuat TangentRot = FQuat::FindBetweenNormals(FVector::XAxisVector, TangentInSphereSpace);
		OffsetTransform.SetRotation(OffsetTransform.GetRotation() * TangentRot);

		// Sensitivity: 1 world-unit drag ≈ 1/(RegionAngle * DebugScale) field value change,
		// matching the rate at which the pole moves along the sphere surface.
		const float SensitivityScale = RegionAngleRadians * PoseReader->Debug.DebugScale;

		// Float control value lives in LocalTransform.Location.X = FieldValue * SensitivityScale.
		// Back the OffsetTransform location away from the pole by the same amount so the gizmo
		// world position lands exactly on the pole regardless of the current FieldValue.
		OffsetTransform.SetLocation(PoleWorld - FieldValue * SensitivityScale * TangentWorldDir);

		const FTransform LocalTranslateTransform(FQuat::Identity, FVector(FieldValue * SensitivityScale, 0.0f, 0.0f));

		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalTranslateTransform, false);
		if (!InInfo->bInitialized)
		{
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, true);
			Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalTranslateTransform, true);
		}
		return true;
	}

	if (InInfo->Target.Name.StartsWith(TEXT("FalloffRegionScaleFactors."), ESearchCase::CaseSensitive))
	{
		const FRigElementKey ParentKey = PoseReader->OptionalParentCache.IsValid()
			? PoseReader->OptionalParentCache.GetKey()
			: Hierarchy->GetFirstParent(PoseReader->DriverCache.GetKey());
		FTransform OffsetTransform = PoseReader->LocalDriverTransformInit * Hierarchy->GetGlobalTransform(ParentKey, false);
		OffsetTransform.SetLocation(Hierarchy->GetGlobalTransformByIndex(PoseReader->DriverCache).GetLocation());

		const FQuat PosWidthRot  = PoseReader->FlipWidthScaling  ? FQuat(FVector::ZAxisVector,  PI)        : FQuat::Identity;
		const FQuat NegWidthRot  = PoseReader->FlipWidthScaling  ? FQuat::Identity                         : FQuat(FVector::ZAxisVector,  PI);
		const FQuat PosHeightRot = PoseReader->FlipHeightScaling ? FQuat(FVector::ZAxisVector, -PI * 0.5f) : FQuat(FVector::ZAxisVector,  PI * 0.5f);
		const FQuat NegHeightRot = PoseReader->FlipHeightScaling ? FQuat(FVector::ZAxisVector,  PI * 0.5f) : FQuat(FVector::ZAxisVector, -PI * 0.5f);

		float FieldValue = 1.0f;
		FQuat QuadrantRot = FQuat::Identity;
		const FString& TargetName = InInfo->Target.Name;
		if (TargetName.Equals(SPR_FallPosWidthTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->FalloffRegionScaleFactors.PositiveWidth;
			QuadrantRot = PosWidthRot;
		}
		else if (TargetName.Equals(SPR_FallNegWidthTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->FalloffRegionScaleFactors.NegativeWidth;
			QuadrantRot = NegWidthRot;
		}
		else if (TargetName.Equals(SPR_FallPosHeightTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->FalloffRegionScaleFactors.PositiveHeight;
			QuadrantRot = PosHeightRot;
		}
		else if (TargetName.Equals(SPR_FallNegHeightTarget, ESearchCase::CaseSensitive))
		{
			FieldValue  = PoseReader->FalloffRegionScaleFactors.NegativeHeight;
			QuadrantRot = NegHeightRot;
		}

		// Mirror RemapAndConvertInputs clamping exactly: falloff angle lower-bound is RegionAngle+1.
		const float RegionAngleDeg    = FMath::Clamp(PoseReader->ActiveRegionSize * 180.0f, 0.5f, 178.0f);
		const float RegionAngleRadians = FMath::DegreesToRadians(RegionAngleDeg);
		const float FalloffAngleRadians = FMath::DegreesToRadians(
			FMath::Clamp(FMath::Max(PoseReader->FalloffSize * 180.0f, RegionAngleDeg + 1.0f), 0.5f, 179.0f));

		// Remap raw FalloffRegionScaleFactors the same way RemapAndConvertInputs does to match OuterRegion.ScaleFactors
		// (what the debug draw actually renders). Raw falloff values are lerped between PosMin and 1.0;
		// at default (1.0) they coincide, but at any other value the gizmo would land off the ellipse without this.
		const float InvFalloffAngle = 1.0f / FalloffAngleRadians;
		FRegionScaleFactors EffectiveScaleFactors;
		EffectiveScaleFactors.PositiveWidth  = FMath::Lerp((RegionAngleRadians * PoseReader->ActiveRegionScaleFactors.PositiveWidth)  * InvFalloffAngle, 1.0f, PoseReader->FalloffRegionScaleFactors.PositiveWidth);
		EffectiveScaleFactors.NegativeWidth  = FMath::Lerp((RegionAngleRadians * PoseReader->ActiveRegionScaleFactors.NegativeWidth)  * InvFalloffAngle, 1.0f, PoseReader->FalloffRegionScaleFactors.NegativeWidth);
		EffectiveScaleFactors.PositiveHeight = FMath::Lerp((RegionAngleRadians * PoseReader->ActiveRegionScaleFactors.PositiveHeight) * InvFalloffAngle, 1.0f, PoseReader->FalloffRegionScaleFactors.PositiveHeight);
		EffectiveScaleFactors.NegativeHeight = FMath::Lerp((RegionAngleRadians * PoseReader->ActiveRegionScaleFactors.NegativeHeight) * InvFalloffAngle, 1.0f, PoseReader->FalloffRegionScaleFactors.NegativeHeight);
		if (PoseReader->FlipWidthScaling)
			Swap(EffectiveScaleFactors.PositiveWidth, EffectiveScaleFactors.NegativeWidth);
		if (PoseReader->FlipHeightScaling)
			Swap(EffectiveScaleFactors.PositiveHeight, EffectiveScaleFactors.NegativeHeight);

		FSphericalRegion TempRegion;
		TempRegion.RegionAngleRadians = FalloffAngleRadians;
		TempRegion.ScaleFactors = EffectiveScaleFactors;
		// Same minimum clamp as the active region block — prevents north-pole singularity.
		TempRegion.ScaleFactors.PositiveWidth  = FMath::Max(TempRegion.ScaleFactors.PositiveWidth,  0.001f);
		TempRegion.ScaleFactors.NegativeWidth  = FMath::Max(TempRegion.ScaleFactors.NegativeWidth,  0.001f);
		TempRegion.ScaleFactors.PositiveHeight = FMath::Max(TempRegion.ScaleFactors.PositiveHeight, 0.001f);
		TempRegion.ScaleFactors.NegativeHeight = FMath::Max(TempRegion.ScaleFactors.NegativeHeight, 0.001f);

		const FVector SpinDir = QuadrantRot * FVector::XAxisVector;
		const float SpinAngle = FMath::Atan2(SpinDir.Y, SpinDir.X);
		const FVector Pole3D = FSphericalPoseReaderDebugSettings::SphericalCoordinatesToNormal(
			FSphericalPoseReaderDebugSettings::CalcSphericalCoordinates(SpinAngle, TempRegion));
		const FVector PoleWorld = OffsetTransform.TransformPosition(Pole3D * PoseReader->Debug.DebugScale);

		const FVector TangentInSphereSpace = (Pole3D * Pole3D.Z - FVector::ZAxisVector).GetSafeNormal();
		const FVector TangentWorldDir = OffsetTransform.GetRotation().RotateVector(TangentInSphereSpace);
		const FQuat TangentRot = FQuat::FindBetweenNormals(FVector::XAxisVector, TangentInSphereSpace);
		OffsetTransform.SetRotation(OffsetTransform.GetRotation() * TangentRot);

		const float SensitivityScale = FalloffAngleRadians * PoseReader->Debug.DebugScale;
		OffsetTransform.SetLocation(PoleWorld - FieldValue * SensitivityScale * TangentWorldDir);

		const FTransform LocalTranslateTransform(FQuat::Identity, FVector(FieldValue * SensitivityScale, 0.0f, 0.0f));
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalTranslateTransform, false);
		if (!InInfo->bInitialized)
		{
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, true);
			Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalTranslateTransform, true);
		}
		return true;
	}

	static const FString ActiveRegionSizeName = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, ActiveRegionSize);
	static const FString FalloffSizeName      = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, FalloffSize);
	const bool bIsActiveSize  = InInfo->Target.Name.Equals(ActiveRegionSizeName, ESearchCase::CaseSensitive);
	const bool bIsFalloffSize = InInfo->Target.Name.Equals(FalloffSizeName,      ESearchCase::CaseSensitive);
	if (bIsActiveSize || bIsFalloffSize)
	{
		const FRigElementKey ParentKey = PoseReader->OptionalParentCache.IsValid()
			? PoseReader->OptionalParentCache.GetKey()
			: Hierarchy->GetFirstParent(PoseReader->DriverCache.GetKey());
		FTransform OffsetTransform = PoseReader->LocalDriverTransformInit * Hierarchy->GetGlobalTransform(ParentKey, false);
		OffsetTransform.SetLocation(Hierarchy->GetGlobalTransformByIndex(PoseReader->DriverCache).GetLocation());

		// Fixed at the north pole (Z-axis tip) of the sphere — does not move as the size changes.
		const float FieldValue = bIsActiveSize ? PoseReader->ActiveRegionSize : PoseReader->FalloffSize;
		const FVector NorthPoleWorld = OffsetTransform.TransformPosition(FVector::ZAxisVector * PoseReader->Debug.DebugScale);
		OffsetTransform.SetLocation(NorthPoleWorld);
		// No tangent rotation: the north pole is a singularity, keep the sphere's base orientation.

		FTransform LocalScaleTransform = FTransform::Identity;
		LocalScaleTransform.SetScale3D(FVector(FieldValue, 1.0f, 1.0f));
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalScaleTransform, false);
		if (!InInfo->bInitialized)
		{
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, OffsetTransform, true);
			Hierarchy->SetLocalTransform(InInfo->ControlKey, LocalScaleTransform, true);
		}
		return true;
	}

	return FRigUnit::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SphericalPoseReader::UpdateDirectManipulationFromHierarchy(
	const URigVMUnitNode* InNode,
	TSharedPtr<FStructOnScope> InInstance,
	FControlRigExecuteContext& InContext,
	TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	FRigUnit_SphericalPoseReader* PoseReader = reinterpret_cast<FRigUnit_SphericalPoseReader*>(InInstance->GetStructMemory());

	static const FString RotationOffsetName = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, RotationOffset);
	if (InInfo->Target.Name.Equals(RotationOffsetName, ESearchCase::CaseSensitive))
	{
		PoseReader->RotationOffset = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetRotation().Euler();
		return true;
	}

	// Per-axis Float controls: Location.X = FieldValue * SensitivityScale (= AngleRadians * DebugScale).
	const FString& TargetName = InInfo->Target.Name;
	if (TargetName.StartsWith(TEXT("ActiveRegionScaleFactors."), ESearchCase::CaseSensitive))
	{
		const float RegionAngleRadians = FMath::DegreesToRadians(FMath::Clamp(PoseReader->ActiveRegionSize * 180.0f, 0.5f, 178.0f));
		const float SensitivityScale = RegionAngleRadians * PoseReader->Debug.DebugScale;
		const float InvSensitivity = (SensitivityScale > SMALL_NUMBER) ? (1.0f / SensitivityScale) : 1.0f;
		const float NewValue = FMath::Clamp(Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetLocation().X * InvSensitivity, 0.0f, 1.0f);

		if (TargetName.Equals(SPR_PosWidthTarget, ESearchCase::CaseSensitive))
			PoseReader->ActiveRegionScaleFactors.PositiveWidth = NewValue;
		else if (TargetName.Equals(SPR_NegWidthTarget, ESearchCase::CaseSensitive))
			PoseReader->ActiveRegionScaleFactors.NegativeWidth = NewValue;
		else if (TargetName.Equals(SPR_PosHeightTarget, ESearchCase::CaseSensitive))
			PoseReader->ActiveRegionScaleFactors.PositiveHeight = NewValue;
		else if (TargetName.Equals(SPR_NegHeightTarget, ESearchCase::CaseSensitive))
			PoseReader->ActiveRegionScaleFactors.NegativeHeight = NewValue;
		return true;
	}
	if (TargetName.StartsWith(TEXT("FalloffRegionScaleFactors."), ESearchCase::CaseSensitive))
	{
		const float RegionAngleDeg      = FMath::Clamp(PoseReader->ActiveRegionSize * 180.0f, 0.5f, 178.0f);
		const float FalloffAngleRadians = FMath::DegreesToRadians(
			FMath::Clamp(FMath::Max(PoseReader->FalloffSize * 180.0f, RegionAngleDeg + 1.0f), 0.5f, 179.0f));
		const float SensitivityScale = FalloffAngleRadians * PoseReader->Debug.DebugScale;
		const float InvSensitivity = (SensitivityScale > SMALL_NUMBER) ? (1.0f / SensitivityScale) : 1.0f;
		const float NewValue = FMath::Clamp(Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetLocation().X * InvSensitivity, 0.0f, 1.0f);

		if (TargetName.Equals(SPR_FallPosWidthTarget, ESearchCase::CaseSensitive))
			PoseReader->FalloffRegionScaleFactors.PositiveWidth = NewValue;
		else if (TargetName.Equals(SPR_FallNegWidthTarget, ESearchCase::CaseSensitive))
			PoseReader->FalloffRegionScaleFactors.NegativeWidth = NewValue;
		else if (TargetName.Equals(SPR_FallPosHeightTarget, ESearchCase::CaseSensitive))
			PoseReader->FalloffRegionScaleFactors.PositiveHeight = NewValue;
		else if (TargetName.Equals(SPR_FallNegHeightTarget, ESearchCase::CaseSensitive))
			PoseReader->FalloffRegionScaleFactors.NegativeHeight = NewValue;
		return true;
	}

	// Region size Scale controls: Scale.X = FieldValue directly.
	static const FString ActiveRegionSizeName = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, ActiveRegionSize);
	static const FString FalloffSizeName      = GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SphericalPoseReader, FalloffSize);
	if (TargetName.Equals(ActiveRegionSizeName, ESearchCase::CaseSensitive))
	{
		PoseReader->ActiveRegionSize = FMath::Clamp(Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetScale3D().X, 0.0f, 1.0f);
		return true;
	}
	if (TargetName.Equals(FalloffSizeName, ESearchCase::CaseSensitive))
	{
		PoseReader->FalloffSize = FMath::Clamp(Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetScale3D().X, 0.0f, 1.0f);
		return true;
	}

	return FRigUnit::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif
