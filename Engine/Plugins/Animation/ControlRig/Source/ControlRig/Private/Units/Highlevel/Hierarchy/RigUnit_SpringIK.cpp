// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_SpringIK.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SpringIK)

#if WITH_EDITOR

bool FRigUnit_SpringIK::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SpringIK, PoleVector))
	{
		const FRigElementKey PoleVectorSpaceKey(PoleVectorSpace, ERigElementType::Bone);
		const FTransform ParentTransform = Hierarchy->GetGlobalTransform(PoleVectorSpaceKey, false);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(PoleVector), false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(PoleVector), true);
		}
		return true;
	}
	return FRigUnit_HighlevelBaseMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_SpringIK::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SpringIK, PoleVector))
	{
		PoleVector = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetTranslation();
		return true;
	}
	return FRigUnit_HighlevelBaseMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_SpringIK_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	TArray<FCachedRigElement>& CachedBones = WorkData.CachedBones;
	FCachedRigElement& CachedPoleVector = WorkData.CachedPoleVector;
	TArray<FTransform>& Transforms = WorkData.Transforms;
	FCRSimPointContainer& Simulation = WorkData.Simulation;

	if(CachedBones.Num() == 0)
	{
		Simulation.Reset();
		Simulation.TimeStep = 1.f / 60.f;

		int32 EndBoneIndex = Hierarchy->GetIndex(FRigElementKey(EndBone, ERigElementType::Bone));
		if (EndBoneIndex != INDEX_NONE)
		{
			int32 StartBoneIndex = Hierarchy->GetIndex(FRigElementKey(StartBone, ERigElementType::Bone));
			if (StartBoneIndex == EndBoneIndex)
			{
				return;
			}

			while (EndBoneIndex != INDEX_NONE)
			{
				const FRigElementKey Key = Hierarchy->GetKey(EndBoneIndex);
				CachedBones.Add(FCachedRigElement(Key, Hierarchy));
				if (EndBoneIndex == StartBoneIndex)
				{
					break;
				}
				EndBoneIndex = Hierarchy->GetIndex(Hierarchy->GetFirstParent(Key));
			}
		}

		if (CachedBones.Num() < 3)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough bones. You need at least three in the chain!"));
			return;
		}

		Algo::Reverse(CachedBones);

		for (int32 PointIndex = 0; PointIndex < CachedBones.Num() - 1; PointIndex++)
		{
			Simulation.Points.Add(FRigVMSimPoint());

			FTransform A = Hierarchy->GetInitialGlobalTransform(CachedBones[PointIndex]);
			FTransform B = Hierarchy->GetInitialGlobalTransform(CachedBones[PointIndex + 1]);

			FCRSimLinearSpring Spring;
			Spring.SubjectA = PointIndex;
			Spring.SubjectB = PointIndex + 1;
			Spring.Equilibrium = (A.GetLocation() - B.GetLocation()).Size();
			Spring.Coefficient = HierarchyStrength;
			Simulation.Springs.Add(Spring);

			// also add the root based springs
			if (PointIndex > 1 && RootStrength > SMALL_NUMBER && HierarchyStrength > SMALL_NUMBER)
			{
				B = Hierarchy->GetInitialGlobalTransform(CachedBones[0]);
				Spring.SubjectA = PointIndex;
				Spring.SubjectB = 0;
				Spring.Equilibrium = FMath::Lerp<float>(0.f, (A.GetLocation() - B.GetLocation()).Size(), FMath::Clamp<float>(RootRatio, 0.f, 1.f));
				Spring.Coefficient = RootStrength;
				Simulation.Springs.Add(Spring);
			}

			// also add the effector based springs
			if(PointIndex > 0 && PointIndex < CachedBones.Num() - 2 && EffectorStrength > SMALL_NUMBER && HierarchyStrength > SMALL_NUMBER)
			{
				B = Hierarchy->GetInitialGlobalTransform(CachedBones[CachedBones.Num() - 1]);
				Spring.SubjectA = PointIndex;
				Spring.SubjectB = CachedBones.Num() - 1;
				Spring.Equilibrium = FMath::Lerp<float>(0.f, (A.GetLocation() - B.GetLocation()).Size(), FMath::Clamp<float>(EffectorRatio, 0.f, 1.f));
				Spring.Coefficient = EffectorStrength;
				Simulation.Springs.Add(Spring);
			}
		}
		Simulation.Points.Add(FRigVMSimPoint());

		Simulation.Points[0].Mass = 0.f;
		Simulation.Points.Last().Mass = 0.f;

		// plane constraints
		for (int32 PointIndex = 0; PointIndex < CachedBones.Num(); PointIndex++)
		{
			FTransform Transform = Hierarchy->GetGlobalTransform(CachedBones[PointIndex]);
			Simulation.Points[PointIndex].LinearDamping = Damping;
			Simulation.Points[PointIndex].Position = Transform.GetLocation();

			if (Simulation.Points[PointIndex].Mass > SMALL_NUMBER)
			{
				FCRSimPointConstraint Constraint;
				Constraint.Type = ECRSimConstraintType::Plane;
				Constraint.SubjectA = Constraint.SubjectB = PointIndex;
				Simulation.Constraints.Add(Constraint);
			}
		}

		const FRigElementKey CachedPoleVectorKey(PoleVectorSpace, ERigElementType::Bone);
		CachedPoleVector = FCachedRigElement(CachedPoleVectorKey, Hierarchy);
	}

	if (CachedBones.Num() < 3)
	{
		return;
	}

	if (!bLiveSimulation)
	{
		Simulation.ResetTime();
	}

	Transforms.Reset();
	for (int32 PointIndex = 0; PointIndex < CachedBones.Num(); PointIndex++)
	{
		FTransform Transform = Hierarchy->GetGlobalTransform(CachedBones[PointIndex]);
		Transforms.Add(Transform);
		if (Simulation.Points[PointIndex].Mass < SMALL_NUMBER || !bLiveSimulation)
		{
			Simulation.Points[PointIndex].LinearDamping = Damping;
			Simulation.Points[PointIndex].Position = Transform.GetLocation();
		}
	}

	FVector PoleTarget = PoleVector;
	if (CachedPoleVector.IsValid())
	{
		const FTransform PoleVectorSpaceTransform = Hierarchy->GetGlobalTransform(CachedPoleVector);
		if (PoleVectorKind == EControlRigVectorKind::Direction)
		{
			PoleTarget = PoleVectorSpaceTransform.TransformVectorNoScale(PoleTarget);
		}
		else
		{
			PoleTarget = PoleVectorSpaceTransform.TransformPositionNoScale(PoleTarget);
		}
	}

	const FVector FirstPoint = Simulation.Points[0].Position;
	const FVector SecondPoint = Simulation.Points[1].Position;
	const FVector ThirdPoint = Simulation.Points[2].Position;
	const FVector LastPoint = Simulation.Points.Last().Position;
	FVector CenterPoint = (FirstPoint + LastPoint) * 0.5f;

	if (PoleVectorKind == EControlRigVectorKind::Direction)
	{
		PoleTarget = PoleTarget + CenterPoint;
	}

	FQuat PreRotation = FQuat::Identity;
	FVector PlaneNormal = FVector::CrossProduct(PoleTarget - LastPoint, PoleTarget - FirstPoint);
	if (!PlaneNormal.IsNearlyZero())
	{
		// apply the normal to all constraints
		PlaneNormal = PlaneNormal.GetSafeNormal();
		for (FCRSimPointConstraint& Constraint : Simulation.Constraints)
		{
			if (Constraint.Type == ECRSimConstraintType::Plane)
			{
				Constraint.DataA = PoleTarget;
				Constraint.DataB = PlaneNormal;
			}
		}

		// pre-rotate all of the points to already sit on the pole triangle
		FVector RotationAxis = FirstPoint - LastPoint;
		if (!RotationAxis.IsNearlyZero())
		{
			RotationAxis = RotationAxis.GetSafeNormal();
			FVector CurrentPole = SecondPoint - (FirstPoint + ThirdPoint) * 0.5f;
			FVector DesiredPole = PoleTarget - CenterPoint;

			if (bFlipPolePlane)
			{
				CurrentPole = -CurrentPole;
			}

			CurrentPole -= FVector::DotProduct(CurrentPole, RotationAxis) * RotationAxis;
			DesiredPole -= FVector::DotProduct(DesiredPole, RotationAxis) * RotationAxis;
			CurrentPole = CurrentPole.GetSafeNormal();
			DesiredPole = DesiredPole.GetSafeNormal();

			if (!CurrentPole.IsNearlyZero() && !DesiredPole.IsNearlyZero())
			{
				PreRotation = FQuat::FindBetweenNormals(CurrentPole, DesiredPole);
				for (int32 PointIndex = 1; PointIndex < CachedBones.Num() - 1; PointIndex++)
				{
					if (Simulation.Points[PointIndex].Mass > SMALL_NUMBER)
					{
						Simulation.Points[PointIndex].Position = CenterPoint + PreRotation.RotateVector(Simulation.Points[PointIndex].Position - CenterPoint);
					}
				}
			}
		}
	}

	Simulation.StepSemiExplicitEuler((bLiveSimulation ? ExecuteContext.GetDeltaTime() : Simulation.TimeStep) * (float)FMath::Clamp<int32>(Iterations, 1, 32));

	FVector AccumulatedTarget = FVector::ZeroVector;
	FVector LastPrimaryTarget = FVector::ZeroVector;
	for (int32 PointIndex = 0; PointIndex < CachedBones.Num(); PointIndex++)
	{
		FTransform& Transform = Transforms[PointIndex];

		if (Simulation.Points[PointIndex].Mass > SMALL_NUMBER)
		{
			Transform.SetLocation(Simulation.GetPointInterpolated(PointIndex).Position);
		}

		if (PointIndex != CachedBones.Num() - 1) // skip the effector
		{
			Transform.SetRotation(PreRotation * Transform.GetRotation());

			FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis);
			FVector Target1 = Simulation.GetPointInterpolated(PointIndex + 1).Position - Transform.GetLocation();
			if (!Target1.IsNearlyZero() && !Axis.IsNearlyZero())
			{
				Target1 = Target1.GetSafeNormal();
				FQuat Rotation1 = FQuat::FindBetweenNormals(Axis, Target1);
				Transform.SetRotation((Rotation1 * Transform.GetRotation()).GetNormalized());

				if (PointIndex == 0)
				{
					AccumulatedTarget = PoleTarget - Transform.GetLocation();
				}
				else
				{
					FQuat TargetRotation = FQuat::FindBetweenNormals(LastPrimaryTarget, Target1);
					AccumulatedTarget = TargetRotation.RotateVector(AccumulatedTarget);
				}
				Axis = Transform.TransformVectorNoScale(SecondaryAxis);
				if (!AccumulatedTarget.IsNearlyZero() && !Axis.IsNearlyZero())
				{
					AccumulatedTarget = AccumulatedTarget - FVector::DotProduct(AccumulatedTarget, Target1) * Target1;
					AccumulatedTarget = AccumulatedTarget.GetSafeNormal();

					FQuat Rotation2 = FQuat::FindBetweenNormals(Axis, AccumulatedTarget);
					Transform.SetRotation((Rotation2 * Transform.GetRotation()).GetNormalized());
				}

				LastPrimaryTarget = Target1;
			}
		}

		if(bLimitLocalPosition)
		{
			int32 ParentIndex = Hierarchy->GetIndex(Hierarchy->GetFirstParent(CachedBones[PointIndex].GetKey()));
			if (ParentIndex != INDEX_NONE)
			{
				FTransform InitialTransform = Hierarchy->GetInitialGlobalTransform(CachedBones[PointIndex]);
				FTransform ParentInitialTransform = Hierarchy->GetInitialGlobalTransform(ParentIndex);
				FTransform ParentTransform = Hierarchy->GetGlobalTransform(ParentIndex);
				float ExpectedDistance = (InitialTransform.GetLocation() - ParentInitialTransform.GetLocation()).Size();
				if (ExpectedDistance > SMALL_NUMBER && PointIndex < CachedBones.Num() - 1)
				{
					FVector Direction = Transform.GetLocation() - ParentTransform.GetLocation();
					if (!Direction.IsNearlyZero())
					{
						Transform.SetLocation(ParentTransform.GetLocation() + Direction.GetSafeNormal() * ExpectedDistance);
					}
				}

				// correct the rotation on the last bone
				if (PointIndex == CachedBones.Num() - 2)
				{
					FVector Axis = Transform.TransformVectorNoScale(PrimaryAxis);
					FVector Target = Simulation.GetPointInterpolated(PointIndex + 1).Position - Transform.GetLocation();
					if (!Target.IsNearlyZero() && !Axis.IsNearlyZero())
					{
						Target = Target.GetSafeNormal();
						FQuat Rotation = FQuat::FindBetweenNormals(Axis, Target);
						Transform.SetRotation((Rotation * Transform.GetRotation()).GetNormalized());
					}
				}

				if (ExpectedDistance > SMALL_NUMBER && PointIndex == CachedBones.Num() - 1)
				{
					FVector Direction = Transform.GetLocation() - ParentTransform.GetLocation();
					if (!Direction.IsNearlyZero())
					{
						Transform.SetLocation(ParentTransform.GetLocation() + Direction.GetSafeNormal() * ExpectedDistance);
					}
				}
			}
		}

		Hierarchy->SetGlobalTransform(CachedBones[PointIndex], Transform, bPropagateToChildren);
	}

	if (ExecuteContext.GetDrawInterface() != nullptr && DebugSettings.bEnabled)
	{
		ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, PoleTarget, FirstPoint, DebugSettings.Color, 0.f);
		ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, PoleTarget, LastPoint, DebugSettings.Color, 0.f);
		ExecuteContext.GetDrawInterface()->DrawBox(DebugSettings.WorldOffset, FTransform(FQuat::Identity, PoleTarget, FVector::OneVector * DebugSettings.Scale * 10.f), DebugSettings.Color);
	}
}
