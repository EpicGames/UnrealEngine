// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_TransformConstraint.h"
#include "Units/RigUnitContext.h"
#include "HelperUtil.h"
#include "AnimationCoreLibrary.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_TransformConstraint)

FRigUnit_TransformConstraint_Execute()
{
	FRigUnit_TransformConstraintPerItem::StaticExecute(
		ExecuteContext, 
		FRigElementKey(Bone, ERigElementType::Bone),
		BaseTransformSpace,
		BaseTransform,
		FRigElementKey(BaseBone, ERigElementType::Bone),
		Targets,
		bUseInitialTransforms,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_TransformConstraint::GetUpgradeInfo() const
{
	FRigUnit_TransformConstraintPerItem NewNode;
	NewNode.Item = FRigElementKey(Bone, ERigElementType::Bone);
	NewNode.BaseTransformSpace = BaseTransformSpace;
	NewNode.BaseTransform = BaseTransform;
	NewNode.BaseItem = FRigElementKey(BaseBone, ERigElementType::Bone);
	NewNode.Targets = Targets;
	NewNode.bUseInitialTransforms = bUseInitialTransforms;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Bone"), TEXT("Item.Name"));
	Info.AddRemappedPin(TEXT("BaseBone"), TEXT("BaseItem.Name"));
	return Info;
}

FRigUnit_TransformConstraintPerItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	TArray<FConstraintData>&	ConstraintData = WorkData.ConstraintData;
	TMap<int32, int32>& ConstraintDataToTargets = WorkData.ConstraintDataToTargets;

	auto SetupConstraintData = [&]()
	{
		ConstraintData.Reset();
		ConstraintDataToTargets.Reset();

		URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
		if(Hierarchy)
		{
			if (Item.IsValid())
			{
				const int32 TargetNum = Targets.Num();
				if (TargetNum > 0)
				{
					const FTransform SourceTransform = bUseInitialTransforms ? Hierarchy->GetGlobalTransform(Item, true) : Hierarchy->GetGlobalTransform(Item, false);
					FTransform InputBaseTransform =
						bUseInitialTransforms ?
						UtilityHelpers::GetBaseTransformByMode(
							BaseTransformSpace,
							[Hierarchy](const FRigElementKey& Item) { return Hierarchy->GetGlobalTransform(Item, true); },
							Hierarchy->GetFirstParent(Item),
							BaseItem,
							BaseTransform
						) :
						UtilityHelpers::GetBaseTransformByMode(
							BaseTransformSpace,
							[Hierarchy](const FRigElementKey& Item) { return Hierarchy->GetGlobalTransform(Item, false); },
							Hierarchy->GetFirstParent(Item),
							BaseItem,
							BaseTransform
						);


					const TArrayView<const FConstraintTarget> TargetsView(Targets.GetData(), Targets.Num());
					for (int32 TargetIndex = 0; TargetIndex < TargetNum; ++TargetIndex)
					{
						// talk to Rob about the implication of support both of this
						const bool bTranslationFilterValid = Targets[TargetIndex].Filter.TranslationFilter.IsValid();
						const bool bRotationFilterValid = Targets[TargetIndex].Filter.RotationFilter.IsValid();
						const bool bScaleFilterValid = Targets[TargetIndex].Filter.ScaleFilter.IsValid();

						if (bTranslationFilterValid && bRotationFilterValid && bScaleFilterValid)
						{
							AddConstraintData(TargetsView, ETransformConstraintType::Parent, TargetIndex, SourceTransform, InputBaseTransform, ConstraintData, ConstraintDataToTargets);
						}
						else
						{
							if (bTranslationFilterValid)
							{
								AddConstraintData(TargetsView, ETransformConstraintType::Translation, TargetIndex, SourceTransform, InputBaseTransform, ConstraintData, ConstraintDataToTargets);
							}

							if (bRotationFilterValid)
							{
								AddConstraintData(TargetsView, ETransformConstraintType::Rotation, TargetIndex, SourceTransform, InputBaseTransform, ConstraintData, ConstraintDataToTargets);
							}

							if (bScaleFilterValid)
							{
								AddConstraintData(TargetsView, ETransformConstraintType::Scale, TargetIndex, SourceTransform, InputBaseTransform, ConstraintData, ConstraintDataToTargets);
							}
						}
					}
				}
			}
		}
	};

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if ((ConstraintData.Num() != Targets.Num()))
		{
			SetupConstraintData();
		}

		if (Item.IsValid())
		{
			const int32 TargetNum = Targets.Num();
			if (TargetNum > 0 && ConstraintData.Num() > 0)
			{
				for (int32 ConstraintIndex= 0; ConstraintIndex< ConstraintData.Num(); ++ConstraintIndex)
				{
					// for now just try translate
					const int32* TargetIndexPtr = ConstraintDataToTargets.Find(ConstraintIndex);
					if (TargetIndexPtr)
					{
						const int32 TargetIndex = *TargetIndexPtr;
						ConstraintData[ConstraintIndex].CurrentTransform = Targets[TargetIndex].Transform;
						ConstraintData[ConstraintIndex].Weight = Targets[TargetIndex].Weight;
					}
				}

				FTransform InputBaseTransform = UtilityHelpers::GetBaseTransformByMode(BaseTransformSpace, [Hierarchy](const FRigElementKey& Item) { return Hierarchy->GetGlobalTransform(Item); },
						Hierarchy->GetFirstParent(Item), BaseItem, BaseTransform);

				FTransform SourceTransform = Hierarchy->GetGlobalTransform(Item);

				// @todo: ignore maintain offset for now
				FTransform ConstrainedTransform = AnimationCore::SolveConstraints(SourceTransform, InputBaseTransform, ConstraintData);

				Hierarchy->SetGlobalTransform(Item, ConstrainedTransform);
			}
		}
	}
}

void FRigUnit_TransformConstraintPerItem::AddConstraintData(const TArrayView<const FConstraintTarget>& Targets, ETransformConstraintType ConstraintType, const int32 TargetIndex, const FTransform& SourceTransform, const FTransform& InBaseTransform, TArray<FConstraintData>& OutConstraintData, TMap<int32, int32>& OutConstraintDataToTargets)
{
	const FConstraintTarget& Target = Targets[TargetIndex];

	int32 NewIndex = OutConstraintData.AddDefaulted();
	check(NewIndex != INDEX_NONE);
	FConstraintData& NewData = OutConstraintData[NewIndex];
	NewData.Constraint = FTransformConstraintDescription(ConstraintType);
	NewData.bMaintainOffset = Target.bMaintainOffset;
	NewData.Weight = Target.Weight;

	if (Target.bMaintainOffset)
	{
		NewData.SaveInverseOffset(SourceTransform, Target.Transform, InBaseTransform);
	}

	OutConstraintDataToTargets.Add(NewIndex, TargetIndex);
}

FRigVMStructUpgradeInfo FRigUnit_TransformConstraintPerItem::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_ParentConstraint_Execute()
 {
	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}

 	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!ChildCache.UpdateCache(Child, Hierarchy))
		{
			return;
		}

		if(ParentCaches.Num() != Parents.Num())
		{
			ParentCaches.SetNumZeroed(Parents.Num());
		}

		const bool bChildIsControl = Child.Type == ERigElementType::Control;

		// calculate total weight
		float OverallWeight = 0;
		for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
		{
			const FConstraintParent& Parent = Parents[ParentIndex];
			if (!ParentCaches[ParentIndex].UpdateCache(Parent.Item, Hierarchy))
			{
				continue;
			}

			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			const float WeightNormalizer = 1.0f / OverallWeight;

			FTransform MixedGlobalTransform;

			// initial rotation needs to be (0,0,0,0) instead of (0,0,0,1) due to Quaternion Blending Math
			MixedGlobalTransform.SetLocation(FVector::ZeroVector);
			MixedGlobalTransform.SetRotation(FQuat(0.f, 0.f, 0.f, 0.f));
			MixedGlobalTransform.SetScale3D(FVector::ZeroVector);

			float AccumulatedWeight = 0.0f;

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				if (!ParentCaches[ParentIndex].IsValid())
				{
					continue;
				}

				const FConstraintParent& Parent = Parents[ParentIndex];

				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;
				AccumulatedWeight += NormalizedWeight;

				FTransform OffsetParentTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], false);

				if (bMaintainOffset)
				{
					const FTransform ChildInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(ChildCache);
					const FTransform ParentInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(ParentCaches[ParentIndex]);
					
					// offset transform is a transform that transforms parent to child
					const FTransform OffsetTransform = ChildInitialGlobalTransform.GetRelativeTransform(ParentInitialGlobalTransform);

					OffsetParentTransform = OffsetTransform * OffsetParentTransform;
					OffsetParentTransform.NormalizeRotation();
				}

				// deal with different interpolation types
				if (AdvancedSettings.InterpolationType == EConstraintInterpType::Average)
				{
					// component-wise average
					MixedGlobalTransform.AccumulateWithShortestRotation(OffsetParentTransform, ScalarRegister(NormalizedWeight));
				}
				else if (AdvancedSettings.InterpolationType == EConstraintInterpType::Shortest)
				{
					FQuat MixedGlobalQuat = MixedGlobalTransform.GetRotation();

					if (MixedGlobalQuat == FQuat(0.0f, 0.0f, 0.0f, 0.0f))
					{
						MixedGlobalTransform = OffsetParentTransform;
					}
					else
					{
						const float Alpha = NormalizedWeight / AccumulatedWeight;
						const FQuat OffsetParentQuat = OffsetParentTransform.GetRotation();

						MixedGlobalTransform.LerpTranslationScale3D(MixedGlobalTransform, OffsetParentTransform, ScalarRegister(Alpha));
						MixedGlobalQuat = FQuat::Slerp(MixedGlobalQuat, OffsetParentQuat, NormalizedWeight);
						MixedGlobalTransform.SetRotation(MixedGlobalQuat);
					}
				}
				else
				{
					// invalid interpolation type
					ensure(false);
					MixedGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ChildCache, false);
					break;
				}
			}

			MixedGlobalTransform.NormalizeRotation();

			// handle filtering, performed in local(parent) space
			const FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, false);
			FTransform MixedLocalTransform = MixedGlobalTransform.GetRelativeTransform(ChildParentGlobalTransform);
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, false);
			
			// Controls have an offset transform built-in and thus need to be handled a bit differently
			FTransform AdditionalOffsetTransform;
			
			if (bChildIsControl)
			{
				if (FRigControlElement* ChildAsControlElement = Hierarchy->Get<FRigControlElement>(ChildCache))
				{
					AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::CurrentLocal);
					// Control's local(parent) space transform = control local value * offset
					ChildCurrentLocalTransform *= AdditionalOffsetTransform;
				}
			}

			if(!Filter.TranslationFilter.HasNoEffect())
			{
				const FVector ChildTranslation = ChildCurrentLocalTransform.GetTranslation();
				FVector MixedTranslation = MixedLocalTransform.GetTranslation();
				MixedTranslation.X = Filter.TranslationFilter.bX ? MixedTranslation.X : ChildTranslation.X;
				MixedTranslation.Y = Filter.TranslationFilter.bY ? MixedTranslation.Y : ChildTranslation.Y;
				MixedTranslation.Z = Filter.TranslationFilter.bZ ? MixedTranslation.Z : ChildTranslation.Z;
				MixedLocalTransform.SetTranslation(MixedTranslation);
			}

			if(!Filter.RotationFilter.HasNoEffect())
			{
				const FQuat ChildRotation = ChildCurrentLocalTransform.GetRotation();
				const FVector ChildEulerRotation = AnimationCore::EulerFromQuat(ChildRotation, AdvancedSettings.RotationOrderForFilter);
				const FQuat MixedRotation = MixedLocalTransform.GetRotation();
				FVector MixedEulerRotation = AnimationCore::EulerFromQuat(MixedRotation, AdvancedSettings.RotationOrderForFilter);
				MixedEulerRotation.X = Filter.RotationFilter.bX ? MixedEulerRotation.X: ChildEulerRotation.X;
				MixedEulerRotation.Y = Filter.RotationFilter.bY ? MixedEulerRotation.Y : ChildEulerRotation.Y;
				MixedEulerRotation.Z = Filter.RotationFilter.bZ ? MixedEulerRotation.Z : ChildEulerRotation.Z;
				MixedLocalTransform.SetRotation(AnimationCore::QuatFromEuler(MixedEulerRotation, AdvancedSettings.RotationOrderForFilter));
				MixedLocalTransform.NormalizeRotation();
			}

			if(!Filter.ScaleFilter.HasNoEffect())
			{
				const FVector ChildScale = ChildCurrentLocalTransform.GetScale3D();
				FVector MixedScale = MixedLocalTransform.GetScale3D();
				MixedScale.X = Filter.ScaleFilter.bX ? MixedScale.X : ChildScale.X;
				MixedScale.Y = Filter.ScaleFilter.bY ? MixedScale.Y : ChildScale.Y;
				MixedScale.Z = Filter.ScaleFilter.bZ ? MixedScale.Z : ChildScale.Z;
				MixedLocalTransform.SetScale3D(MixedScale);
			}

			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				MixedLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, MixedLocalTransform, Weight);
				if(!bChildIsControl)
				{
					MixedLocalTransform.NormalizeRotation();
				}
			}
			
			if (bChildIsControl)
			{
				// need to convert back to offset space for the actual control value
				MixedLocalTransform = MixedLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				MixedLocalTransform.NormalizeRotation();
			}

			Hierarchy->SetLocalTransformByIndex(ChildCache, MixedLocalTransform);
		}
	} 
 }

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

 IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ParentConstraint)
 {
	// use euler rotation here to match other software's rotation representation more easily
	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-10, -10, -10), Order), FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(30, -30, -30), Order), FVector(20.f, 20.f, 20.f)), true, ERigBoneType::User);
	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-40, -40, 40), Order), FVector(40.f, 40.f, 40.f)), true, ERigBoneType::User);
	const FRigElementKey Parent3 = Controller->AddBone(TEXT("Parent3"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-50, 50, -50), Order), FVector(60.f, 60.f, 60.f)), true, ERigBoneType::User);
	const FRigElementKey Parent4 = Controller->AddBone(TEXT("Parent4"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(60, 60, 60), Order), FVector(80.f, 80.f, 80.f)), true, ERigBoneType::User);
	
	Unit.Child = Child;

	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent3, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent4, 1.0));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bMaintainOffset = false;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
	
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(50.f, 50.f, 50.f)), TEXT("unexpected translation for average interpolation type"));

	FQuat Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	FQuat Expected = AnimationCore::QuatFromEuler( FVector(-0.852f, 15.189f, -0.572f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for average interpolation type"));
	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bMaintainOffset = false;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
	Execute();
	
	Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(16.74f, 8.865f, -5.562f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for shortest interpolation type"));

	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100, 100, -100), Order), FVector(-40.f, -40.f, -40.f)));
	Unit.bMaintainOffset = true;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
	
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(-8.66f, 7.01f, -13.0f), 0.02f),
                    TEXT("unexpected translation for maintain offset and average interpolation type"));
	
	Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(5.408f, -5.679f, -34.44f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for maintain offset and average interpolation type"));

	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100.0f, 100.0f, -100.0f), Order), FVector(-40.f, -40.f, -40.f)));
	Unit.bMaintainOffset = true;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
	
	Execute();
	Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(-1.209f, -8.332f, -25.022f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for maintain offset and shortest interpolation type"));
	
	return true;
 }

#endif

FRigUnit_ParentConstraintMath_Execute()
 {
 	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{

		if(ParentCaches.Num() != Parents.Num())
		{
			ParentCaches.SetNumZeroed(Parents.Num());
		}

		// calculate total weight
		float OverallWeight = 0;
		for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
		{
			const FConstraintParent& Parent = Parents[ParentIndex];
			if (!ParentCaches[ParentIndex].UpdateCache(Parent.Item, Hierarchy))
			{
				continue;
			}

			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			const float WeightNormalizer = 1.0f / OverallWeight;

			FTransform MixedGlobalTransform;

			// initial rotation needs to be (0,0,0,0) instead of (0,0,0,1) due to Quaternion Blending Math
			MixedGlobalTransform.SetLocation(FVector::ZeroVector);
			MixedGlobalTransform.SetRotation(FQuat(0.f, 0.f, 0.f, 0.f));
			MixedGlobalTransform.SetScale3D(FVector::ZeroVector);

			float AccumulatedWeight = 0.0f;

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				if (!ParentCaches[ParentIndex].IsValid())
				{
					continue;
				}

				const FConstraintParent& Parent = Parents[ParentIndex];

				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;
				AccumulatedWeight += NormalizedWeight;

				FTransform OffsetParentTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], false);

				// Input initial
				const FTransform ChildInitialGlobalTransform = Input;
				const FTransform ParentInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(ParentCaches[ParentIndex]);
				
				// offset transform is a transform that transforms parent to child
				const FTransform OffsetTransform = ChildInitialGlobalTransform.GetRelativeTransform(ParentInitialGlobalTransform);

				OffsetParentTransform = OffsetTransform * OffsetParentTransform;
				OffsetParentTransform.NormalizeRotation();

				// deal with different interpolation types
				if (AdvancedSettings.InterpolationType == EConstraintInterpType::Average)
				{
					// component-wise average
					MixedGlobalTransform.AccumulateWithShortestRotation(OffsetParentTransform, ScalarRegister(NormalizedWeight));
				}
				else if (AdvancedSettings.InterpolationType == EConstraintInterpType::Shortest)
				{
					FQuat MixedGlobalQuat = MixedGlobalTransform.GetRotation();

					if (MixedGlobalQuat == FQuat(0.0f, 0.0f, 0.0f, 0.0f))
					{
						MixedGlobalTransform = OffsetParentTransform;
					}
					else
					{
						const float Alpha = NormalizedWeight / AccumulatedWeight;
						const FQuat OffsetParentQuat = OffsetParentTransform.GetRotation();

						MixedGlobalTransform.LerpTranslationScale3D(MixedGlobalTransform, OffsetParentTransform, ScalarRegister(Alpha));
						MixedGlobalQuat = FQuat::Slerp(MixedGlobalQuat, OffsetParentQuat, NormalizedWeight);
						MixedGlobalTransform.SetRotation(MixedGlobalQuat);
					}
				}
				else
				{
					// invalid interpolation type
					ensure(false);
					MixedGlobalTransform = Input;
					break;
				}
			}

			MixedGlobalTransform.NormalizeRotation();

			Output = MixedGlobalTransform;
		}
	} 
 }

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

 IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ParentConstraintMath)
 {
	// use euler rotation here to match other software's rotation representation more easily
	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-10, -10, -10), Order), FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(30, -30, -30), Order), FVector(20.f, 20.f, 20.f)), true, ERigBoneType::User);
	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-40, -40, 40), Order), FVector(40.f, 40.f, 40.f)), true, ERigBoneType::User);
	const FRigElementKey Parent3 = Controller->AddBone(TEXT("Parent3"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-50, 50, -50), Order), FVector(60.f, 60.f, 60.f)), true, ERigBoneType::User);
	const FRigElementKey Parent4 = Controller->AddBone(TEXT("Parent4"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(60, 60, 60), Order), FVector(80.f, 80.f, 80.f)), true, ERigBoneType::User);
	
	ExecuteContext.Hierarchy = Hierarchy.Get();
	Unit.Input = Hierarchy->GetInitialGlobalTransform(Child);

	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent3, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent4, 1.0));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100, 100, -100), Order), FVector(-40.f, -40.f, -40.f)));
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
	
	Execute();
	AddErrorIfFalse(Unit.Output.GetTranslation().Equals(FVector(-8.66f, 7.01f, -13.0f), 0.02f),
                    TEXT("unexpected translation for average interpolation type"));
	
	FQuat Result = Unit.Output.GetRotation();
	FQuat Expected = AnimationCore::QuatFromEuler( FVector(5.408f, -5.679f, -34.44f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for average interpolation type"));

	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100.0f, 100.0f, -100.0f), Order), FVector(-40.f, -40.f, -40.f)));
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
	
	Execute();
	Result = Unit.Output.GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(-1.209f, -8.332f, -25.022f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for shortest interpolation type"));
	
	return true;
 }

#endif

FRigUnit_PositionConstraint_Execute()
{
	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!Child.IsValid())
		{
			return;
		}
		
		FTransform ChildInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(Child);

		float OverallWeight = 0;
		for (const FConstraintParent& Parent : Parents)
		{
			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);

			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			if (!Parent.Item.IsValid())
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			const float WeightNormalizer = 1.0f / OverallWeight;

			FVector MixedGlobalPosition = FVector::ZeroVector;

			for (const FConstraintParent& Parent : Parents)
			{
				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);

				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				if (!Parent.Item.IsValid())
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;

				FVector OffsetPosition = FVector::ZeroVector;
				FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransform(Parent.Item, false);

				if (bMaintainOffset)
				{
					FTransform ParentInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(Parent.Item);
					OffsetPosition = ChildInitialGlobalTransform.GetLocation() - ParentInitialGlobalTransform.GetLocation();
				}

				FVector OffsetParentPosition = OffsetPosition + ParentCurrentGlobalTransform.GetLocation();

				MixedGlobalPosition += OffsetParentPosition * NormalizedWeight;
			}

			// handle filtering, performed in local space
			FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransform(Child, false);
			FVector MixedPosition = ChildParentGlobalTransform.Inverse().TransformPosition(MixedGlobalPosition);
			
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransform(Child);
			
			// Controls need to be handled a bit differently
			FTransform AdditionalOffsetTransform = FTransform::Identity;
			
			if (Child.Type == ERigElementType::Control)
			{
				if (FRigControlElement* ChildAsControlElement = Hierarchy->Find<FRigControlElement>(Child))
				{
					AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::CurrentLocal);
					// Control's local(parent) space position = local * offset
					ChildCurrentLocalTransform *= AdditionalOffsetTransform;
				}
			}
			
			FVector ChildPosition = ChildCurrentLocalTransform.GetTranslation();

			FVector FilteredPosition;
			FilteredPosition.X = Filter.bX ? MixedPosition.X : ChildPosition.X;
			FilteredPosition.Y = Filter.bY ? MixedPosition.Y : ChildPosition.Y;
			FilteredPosition.Z = Filter.bZ ? MixedPosition.Z : ChildPosition.Z;

			FTransform FilteredMixedLocalTransform = ChildCurrentLocalTransform;

			FilteredMixedLocalTransform.SetTranslation(FilteredPosition); 

			FTransform FinalLocalTransform = FilteredMixedLocalTransform;
			
			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				FinalLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, FinalLocalTransform, Weight);
			}
			
			if (Child.Type == ERigElementType::Control)
			{
				// need to convert back to offset space for the actual control value
				FinalLocalTransform = FinalLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				FinalLocalTransform.NormalizeRotation();
			}

			Hierarchy->SetLocalTransform(Child, FinalLocalTransform);
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_PositionConstraint::GetUpgradeInfo() const
{
	FRigUnit_PositionConstraintLocalSpaceOffset NewNode;
	NewNode.Child = Child;
	NewNode.bMaintainOffset = bMaintainOffset;
	NewNode.Filter = Filter;
	NewNode.Parents = Parents;
	NewNode.Weight = Weight;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

 IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_PositionConstraint)
 {
	// use euler rotation here to match other software's rotation representation more easily
	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform(FVector(20.f, 20.f, 20.f)), true, ERigBoneType::User);
	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform(FVector(40.f, 40.f, 40.f)), true, ERigBoneType::User);
	const FRigElementKey Parent3 = Controller->AddBone(TEXT("Parent3"), FRigElementKey(), FTransform(FVector(60.f, 60.f, 60.f)), true, ERigBoneType::User);
	
	Unit.Child = Child;

	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent3, 1.0));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bMaintainOffset = false;
	
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(40.f, 40.f, 40.f)), TEXT("unexpected translation for maintain offset off"));

	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(FVector(-40.f, -40.f, -40.f)));
	Unit.bMaintainOffset = true;
	
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(-26.67f, -26.67f, -26.67f), 0.01f),
                    TEXT("unexpected translation for maintain offset on"));

	return true;
 }

#endif

FRigUnit_PositionConstraintLocalSpaceOffset_Execute()
{
	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!ChildCache.UpdateCache(Child, Hierarchy))
		{
			return;
		}

		if(ParentCaches.Num() != Parents.Num())
		{
			ParentCaches.SetNumZeroed(Parents.Num());
		}
		
		float OverallWeight = 0;
		for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
		{
			const FConstraintParent& Parent = Parents[ParentIndex];
			if (!ParentCaches[ParentIndex].UpdateCache(Parent.Item, Hierarchy))
			{
				continue;
			}

			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		const float WeightNormalizer = 1.0f / OverallWeight;

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			FTransform AdditionalOffsetTransform;
			
			const bool bChildIsControl = Child.Type == ERigElementType::Control;
			if(bChildIsControl)
			{
				if (FRigControlElement* ChildAsControlElement = Hierarchy->Get<FRigControlElement>(ChildCache))
				{
					AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::InitialLocal);
				}
			}

			FVector OffsetPosition = FVector::ZeroVector;
			if (bMaintainOffset)
			{
				FVector MixedInitialGlobalPosition = FVector::ZeroVector;

				for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
				{
					if (!ParentCaches[ParentIndex].IsValid())
					{
						continue;
					}

					const FConstraintParent& Parent = Parents[ParentIndex];

					const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
					if (ClampedWeight < KINDA_SMALL_NUMBER)
					{
						continue;
					}

					const float NormalizedWeight = ClampedWeight * WeightNormalizer;

					FTransform ParentInitialGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], true);

					MixedInitialGlobalPosition += ParentInitialGlobalTransform.GetLocation() * NormalizedWeight;
				}
				
				const FTransform ChildParentInitialGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, true);
				FVector MixedInitialLocalPosition = ChildParentInitialGlobalTransform.Inverse().TransformPosition(MixedInitialGlobalPosition);

				FTransform ChildInitialLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, true);

				// Controls need to be handled a bit differently
				if (bChildIsControl)
				{
					ChildInitialLocalTransform *= AdditionalOffsetTransform;
				}

				// use initial transforms to calculate the offset
				// offset is applied in local space, at the end of constraint solve
				OffsetPosition = ChildInitialLocalTransform.GetLocation() - MixedInitialLocalPosition;
			}
			
			FVector MixedGlobalPosition = FVector::ZeroVector;

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				if (!ParentCaches[ParentIndex].IsValid())
				{
					continue;
				}

				const FConstraintParent& Parent = Parents[ParentIndex];
				
				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;

				const FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], false);
				MixedGlobalPosition += ParentCurrentGlobalTransform.GetLocation() * NormalizedWeight;
			}
			
			// handle filtering, performed in local space
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransform(Child);
			
			// Controls need to be handled a bit differently
			if (bChildIsControl)
			{
				ChildCurrentLocalTransform *= AdditionalOffsetTransform;
			}
			
			FTransform MixedTransform = ChildCurrentLocalTransform;

			const FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, false);
			FVector MixedPosition = ChildParentGlobalTransform.InverseTransformPosition(MixedGlobalPosition);
			if (bMaintainOffset)
			{
				MixedPosition = MixedPosition + OffsetPosition;	
			}
			
			if(!Filter.HasNoEffect())
			{
				FVector ChildPosition = ChildCurrentLocalTransform.GetTranslation();
				MixedPosition.X = Filter.bX ? MixedPosition.X : ChildPosition.X;
				MixedPosition.Y = Filter.bY ? MixedPosition.Y : ChildPosition.Y;
				MixedPosition.Z = Filter.bZ ? MixedPosition.Z : ChildPosition.Z;
			}

			MixedTransform.SetTranslation(MixedPosition);
			
			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				MixedTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, MixedTransform, Weight);
			}
			
			if (Child.Type == ERigElementType::Control)
			{
				// need to convert back to offset space for the actual control value
				MixedTransform = MixedTransform.GetRelativeTransform(AdditionalOffsetTransform);
				MixedTransform.NormalizeRotation();
			}

			Hierarchy->SetLocalTransformByIndex(ChildCache, MixedTransform);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_PositionConstraintLocalSpaceOffset)
{
	// multi-parent test
	{
		// use euler rotation here to match other software's rotation representation more easily
    	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
    	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
    	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform(FVector(20.f, 20.f, 20.f)), true, ERigBoneType::User);
    	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform(FVector(40.f, 40.f, 40.f)), true, ERigBoneType::User);
    	const FRigElementKey Parent3 = Controller->AddBone(TEXT("Parent3"), FRigElementKey(), FTransform(FVector(60.f, 60.f, 60.f)), true, ERigBoneType::User);
    	
    	Unit.Child = Child;
    
    	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
    	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
    	Unit.Parents.Add(FConstraintParent(Parent3, 1.0));
    
    	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
    	Unit.bMaintainOffset = false;
    	
    	Execute();
    	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(40.f, 40.f, 40.f)), TEXT("unexpected translation for maintain offset off"));
    
    	
    	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
    	Hierarchy->SetGlobalTransform(2, FTransform(FVector(-40.f, -40.f, -40.f)));
    	Unit.bMaintainOffset = true;
    	
    	Execute();
    	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetTranslation().Equals(FVector(-26.67f, -26.67f, -26.67f), 0.01f),
    					TEXT("unexpected translation for maintain offset on"));

	}

	// local offset test
	{
		Unit.Parents.Reset();
		
		Controller->GetHierarchy()->Reset();
		const FRigElementKey Thigh = Controller->AddBone(TEXT("Thigh"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
		const FRigElementKey Knee = Controller->AddBone(TEXT("Knee"), FRigElementKey(TEXT("Thigh"), ERigElementType::Bone), FTransform(FVector(10.f, 0.f, -50.f)), true, ERigBoneType::User);
		const FRigElementKey Corrective = Controller->AddBone(TEXT("Corrective"), FRigElementKey(TEXT("Knee"), ERigElementType::Bone), FTransform(FVector(-50.f, 0.f, 0.f)), false, ERigBoneType::User);

		Unit.Child = Corrective;
    
		Unit.Parents.Add(FConstraintParent(Knee, 1.0));
		Unit.Parents.Add(FConstraintParent(Thigh, 1.0));
		Unit.bMaintainOffset = true;

		// use euler rotation here to match other software's rotation representation more easily
		EEulerRotationOrder Order = EEulerRotationOrder::XZY;	
		Hierarchy->SetLocalTransform(1,
			FTransform(AnimationCore::QuatFromEuler( FVector(0, 45, 0), Order), 
			FVector(10.f,0.f,-50.f)));
		
		Execute();
		AddErrorIfFalse(Hierarchy->GetGlobalTransform(2).GetTranslation().Equals(FVector(-44.5f, 0.f, -10.86f), 0.1f), TEXT("unexpected translation for local space offset test"));	
	}
	
	return true;
}
#endif

FRigUnit_RotationConstraint_Execute()
{
	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!Child.IsValid())
		{
			return;
		}

		FTransform ChildInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(Child);
		FTransform ChildCurrentGlobalTransform = Hierarchy->GetGlobalTransform(Child, false);

		float OverallWeight = 0;
		for (const FConstraintParent& Parent : Parents)
		{
			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);

			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			if (!Parent.Item.IsValid())
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			const float WeightNormalizer = 1.0f / OverallWeight;

			FQuat MixedGlobalRotation = FQuat(0,0,0,0);

			for (const FConstraintParent& Parent : Parents)
			{
				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);

				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				if (!Parent.Item.IsValid())
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;

				FQuat OffsetRotation = FQuat::Identity;
				FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransform(Parent.Item, false);

				if (bMaintainOffset)
				{
					FTransform ParentInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(Parent.Item);
					OffsetRotation = ParentInitialGlobalTransform.GetRotation().Inverse() * ChildInitialGlobalTransform.GetRotation();
					OffsetRotation.Normalize();
				}

				FQuat OffsetParentRotation = ParentCurrentGlobalTransform.GetRotation() * OffsetRotation;

				// deal with different interpolation types
				if (AdvancedSettings.InterpolationType == EConstraintInterpType::Average)
				{
					// component-wise average
					FQuat WeightedOffsetParentRotation = OffsetParentRotation * NormalizedWeight; 
                    
                    // To ensure the 'shortest route', we make sure the dot product between the accumulator and the incoming child Quat is positive.
					if ((WeightedOffsetParentRotation | MixedGlobalRotation) < 0.f)
					{
						MixedGlobalRotation -= WeightedOffsetParentRotation;	
					}
					else
					{
						MixedGlobalRotation += WeightedOffsetParentRotation;	
					}
				}
				else if (AdvancedSettings.InterpolationType == EConstraintInterpType::Shortest)
				{
					if (MixedGlobalRotation == FQuat(0.0f, 0.0f, 0.0f, 0.0f))
					{
						MixedGlobalRotation = OffsetParentRotation;
					}
					else
					{
						MixedGlobalRotation = FQuat::Slerp(MixedGlobalRotation, OffsetParentRotation, NormalizedWeight);
					}
				}
				else
				{
					// invalid interpolation type
					ensure(false);
					MixedGlobalRotation = ChildCurrentGlobalTransform.GetRotation();
					break;
				}	
			}

			MixedGlobalRotation.Normalize();
			
			// handle filtering, performed in local space
			FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransform(Child, false);
			FQuat MixedLocalRotation = ChildParentGlobalTransform.GetRotation().Inverse() * MixedGlobalRotation;
			FVector MixedEulerRotation = AnimationCore::EulerFromQuat(MixedLocalRotation, AdvancedSettings.RotationOrderForFilter);
			
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransform(Child, false);
			
			// Controls need to be handled a bit differently
			FTransform AdditionalOffsetTransform = FTransform::Identity;

			if (Child.Type == ERigElementType::Control)
			{
				if (FRigControlElement* ChildAsControlElement = Hierarchy->Find<FRigControlElement>(Child))
				{
					AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::CurrentLocal);
					// Control's local(parent) space = local * offset
					ChildCurrentLocalTransform *= AdditionalOffsetTransform;
				}
			}

			FQuat ChildRotation = ChildCurrentLocalTransform.GetRotation();
			
			FVector ChildEulerRotation = AnimationCore::EulerFromQuat(ChildRotation, AdvancedSettings.RotationOrderForFilter);	
			
			FVector FilteredEulerRotation;
			FilteredEulerRotation.X = Filter.bX ? MixedEulerRotation.X : ChildEulerRotation.X;
			FilteredEulerRotation.Y = Filter.bY ? MixedEulerRotation.Y : ChildEulerRotation.Y;
			FilteredEulerRotation.Z = Filter.bZ ? MixedEulerRotation.Z : ChildEulerRotation.Z; 

			FTransform FilteredMixedLocalTransform = ChildCurrentLocalTransform;

			FilteredMixedLocalTransform.SetRotation(AnimationCore::QuatFromEuler(FilteredEulerRotation, AdvancedSettings.RotationOrderForFilter));

			FTransform FinalLocalTransform = FilteredMixedLocalTransform;

			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				FinalLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, FinalLocalTransform, Weight);
			}

			if (Child.Type == ERigElementType::Control)
			{
				// need to convert back to offset space for the actual control value
				FinalLocalTransform = FinalLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				FinalLocalTransform.NormalizeRotation();
			}
			
			Hierarchy->SetLocalTransform(Child, FinalLocalTransform);
		}
	} 	
}

FRigVMStructUpgradeInfo FRigUnit_RotationConstraint::GetUpgradeInfo() const
{
	FRigUnit_RotationConstraintLocalSpaceOffset NewNode;
	NewNode.Child = Child;
	NewNode.bMaintainOffset = bMaintainOffset;
	NewNode.Filter = Filter;
	NewNode.Parents = Parents;
	NewNode.AdvancedSettings = AdvancedSettings;
	NewNode.Weight = Weight;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_RotationConstraint)
{
	// the rotation constraint is expected to behave similarly to parent constraint with translation filter turned off.

	// use euler rotation here to match other software's rotation representation more easily
	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-10, -10, -10), Order), FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(30, -30, -30), Order), FVector(20.f, 20.f, 20.f)), true, ERigBoneType::User);
	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-40, -40, 40), Order), FVector(40.f, 40.f, 40.f)), true, ERigBoneType::User);
	const FRigElementKey Parent3 = Controller->AddBone(TEXT("Parent3"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-50, 50, -50), Order), FVector(60.f, 60.f, 60.f)), true, ERigBoneType::User);
	const FRigElementKey Parent4 = Controller->AddBone(TEXT("Parent4"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(60, 60, 60), Order), FVector(80.f, 80.f, 80.f)), true, ERigBoneType::User);
	
	Unit.Child = Child;

	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent3, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent4, 1.0));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bMaintainOffset = false;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
	
	Execute();

	FQuat Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	FQuat Expected = AnimationCore::QuatFromEuler( FVector(-0.853f, 15.189f, -0.572f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for average interpolation type"));
	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bMaintainOffset = false;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
	
	Execute();
	
	Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(16.74f, 8.865f, -5.562f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for shortest interpolation type"));

	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100, 100, -100), Order), FVector(-40.f, -40.f, -40.f)));
	Unit.bMaintainOffset = true;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
	
	Execute();
	
	Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(5.408f, -5.679f, -34.44f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for maintain offset and average interpolation type"));

	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100.0f, 100.0f, -100.0f), Order), FVector(-40.f, -40.f, -40.f)));
	Unit.bMaintainOffset = true;
	Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
	
	Execute();
	
	Result = Hierarchy->GetGlobalTransform(0).GetRotation();
	Expected = AnimationCore::QuatFromEuler( FVector(-1.209f, -8.332f, -25.022f), Order);
	AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for maintain offset and shortest interpolation type"));
	
	return true;
}
#endif

FRigUnit_RotationConstraintLocalSpaceOffset_Execute()
{
	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!ChildCache.UpdateCache(Child, Hierarchy))
		{
			return;
		}

		if(ParentCaches.Num() != Parents.Num())
		{
			ParentCaches.SetNumZeroed(Parents.Num());
		}

		FTransform ChildInitialLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, true);

		// Controls need to be handled a bit differently
		FTransform AdditionalOffsetTransform;

		const bool bChildIsControl = Child.Type == ERigElementType::Control;
		if (bChildIsControl)
		{
			if (FRigControlElement* ChildAsControlElement = Hierarchy->Get<FRigControlElement>(ChildCache))
			{
				AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::CurrentLocal);
				
				// Control's local(parent) space position = local * offset
				ChildInitialLocalTransform *= AdditionalOffsetTransform;
			}
		}
		
		float OverallWeight = 0;
		for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
		{
			const FConstraintParent& Parent = Parents[ParentIndex];
			if (!ParentCaches[ParentIndex].UpdateCache(Parent.Item, Hierarchy))
			{
				continue;
			}
			
			const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		const float WeightNormalizer = 1.0f / OverallWeight;
		
		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			FQuat OffsetRotation = FQuat::Identity;
			
			if (bMaintainOffset)
			{
				FQuat MixedInitialGlobalRotation = FQuat(0,0,0,0);

				for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
				{
					if(!ParentCaches[ParentIndex].IsValid())
					{
						continue;
					}
					
					const FConstraintParent& Parent = Parents[ParentIndex];
					
					const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
					if (ClampedWeight < KINDA_SMALL_NUMBER)
					{
						continue;
					}

					const float NormalizedWeight = ClampedWeight * WeightNormalizer;

					FTransform ParentInitialGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], true);
					FQuat ParentInitialGlobalRotation = ParentInitialGlobalTransform.GetRotation();

					// deal with different interpolation types
					if (AdvancedSettings.InterpolationType == EConstraintInterpType::Average)
					{
						// component-wise average
						FQuat WeightedInitialParentRotation = ParentInitialGlobalRotation * NormalizedWeight; 
						
						// To ensure the 'shortest route', we make sure the dot product between the accumulator and the incoming child Quat is positive.
						if ((WeightedInitialParentRotation | MixedInitialGlobalRotation) < 0.f)
						{
							MixedInitialGlobalRotation -= WeightedInitialParentRotation;	
						}
						else
						{
							MixedInitialGlobalRotation += WeightedInitialParentRotation;	
						}
					}
					else if (AdvancedSettings.InterpolationType == EConstraintInterpType::Shortest)
					{
						if (MixedInitialGlobalRotation == FQuat(0.0f, 0.0f, 0.0f, 0.0f))
						{
							MixedInitialGlobalRotation = ParentInitialGlobalRotation;
						}
						else
						{
							MixedInitialGlobalRotation = FQuat::Slerp(MixedInitialGlobalRotation, ParentInitialGlobalRotation, NormalizedWeight);
						}
					}
					else
					{
						// invalid interpolation type
						ensure(false);
						FTransform ChildInitialGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ChildCache, true);
						MixedInitialGlobalRotation = ChildInitialGlobalTransform.GetRotation();
						break;
					}
				}

				MixedInitialGlobalRotation.Normalize();
				
				// calculate the offset rotation that rotates the initial blended result
				// back to the initial orientation of the child
				const FTransform ChildParentInitialGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, true);
				const FQuat MixedInitialLocalRotation = ChildParentInitialGlobalTransform.GetRotation().Inverse() * MixedInitialGlobalRotation;
				const FQuat ChildInitialLocalRotation = ChildInitialLocalTransform.GetRotation();
					
				OffsetRotation = MixedInitialLocalRotation.Inverse() * ChildInitialLocalRotation;
			}
			
			FQuat MixedGlobalRotation = FQuat(0,0,0,0);

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				if(!ParentCaches[ParentIndex].IsValid())
				{
					continue;
				}
					
				const FConstraintParent& Parent = Parents[ParentIndex];
				
				const float ClampedWeight = FMath::Max(Parent.Weight, 0.f);
				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				const float NormalizedWeight = ClampedWeight * WeightNormalizer;

				FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], false);
				FQuat ParentCurrentGlobalRotation = ParentCurrentGlobalTransform.GetRotation();

				// deal with different interpolation types
				if (AdvancedSettings.InterpolationType == EConstraintInterpType::Average)
				{
					// component-wise average
					FQuat WeightedParentRotation = ParentCurrentGlobalRotation * NormalizedWeight; 
                    
                    // To ensure the 'shortest route', we make sure the dot product between the accumulator and the incoming child Quat is positive.
					if ((WeightedParentRotation | MixedGlobalRotation) < 0.f)
					{
						MixedGlobalRotation -= WeightedParentRotation;	
					}
					else
					{
						MixedGlobalRotation += WeightedParentRotation;	
					}
				}
				else if (AdvancedSettings.InterpolationType == EConstraintInterpType::Shortest)
				{
					if (MixedGlobalRotation == FQuat(0.0f, 0.0f, 0.0f, 0.0f))
					{
						MixedGlobalRotation = ParentCurrentGlobalRotation;
					}
					else
					{
						MixedGlobalRotation = FQuat::Slerp(MixedGlobalRotation, ParentCurrentGlobalRotation, NormalizedWeight);
					}
				}
				else
				{
					// invalid interpolation type
					ensure(false);
					FTransform ChildCurrentGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ChildCache, false);
					MixedGlobalRotation = ChildCurrentGlobalTransform.GetRotation();
					break;
				}	
			}
				
			MixedGlobalRotation.Normalize();
			
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, false);
			
			// Controls need to be handled a bit differently
			if (bChildIsControl)
			{
				// Control's local(parent) space = local * offset
				ChildCurrentLocalTransform *= AdditionalOffsetTransform;
			}

			// handle filtering, performed in local space
			FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, false);
			FQuat MixedLocalRotation = ChildParentGlobalTransform.GetRotation().Inverse() * MixedGlobalRotation;
			FTransform MixedLocalTransform = ChildCurrentLocalTransform;

			if (bMaintainOffset)
			{
				MixedLocalRotation = MixedLocalRotation * OffsetRotation;
			}

			if(!Filter.HasNoEffect())
			{
				FVector MixedEulerRotation = AnimationCore::EulerFromQuat(MixedLocalRotation, AdvancedSettings.RotationOrderForFilter);

				const FQuat ChildRotation = ChildCurrentLocalTransform.GetRotation();
				const FVector ChildEulerRotation = AnimationCore::EulerFromQuat(ChildRotation, AdvancedSettings.RotationOrderForFilter);
				
				MixedEulerRotation.X = Filter.bX ? MixedEulerRotation.X : ChildEulerRotation.X;
				MixedEulerRotation.Y = Filter.bY ? MixedEulerRotation.Y : ChildEulerRotation.Y;
				MixedEulerRotation.Z = Filter.bZ ? MixedEulerRotation.Z : ChildEulerRotation.Z; 
				MixedLocalTransform.SetRotation(AnimationCore::QuatFromEuler(MixedEulerRotation, AdvancedSettings.RotationOrderForFilter));
			}
			else
			{
				MixedLocalTransform.SetRotation(MixedLocalRotation);
			}

			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				MixedLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, MixedLocalTransform, Weight);
			}

			if (bChildIsControl)
			{
				// need to convert back to offset space for the actual control value
				MixedLocalTransform = MixedLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				MixedLocalTransform.NormalizeRotation();
			}
			
			Hierarchy->SetLocalTransformByIndex(ChildCache, MixedLocalTransform);
		}
	} 	
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_RotationConstraintLocalSpaceOffset)
{
	// the rotation constraint is expected to behave similarly to parent constraint with translation filter turned off.

	// multi parent test
	{
		// use euler rotation here to match other software's rotation representation more easily
		EEulerRotationOrder Order = EEulerRotationOrder::XZY;
		const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-10, -10, -10), Order), FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
		const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(30, -30, -30), Order), FVector(20.f, 20.f, 20.f)), true, ERigBoneType::User);
		const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-40, -40, 40), Order), FVector(40.f, 40.f, 40.f)), true, ERigBoneType::User);
		const FRigElementKey Parent3 = Controller->AddBone(TEXT("Parent3"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(-50, 50, -50), Order), FVector(60.f, 60.f, 60.f)), true, ERigBoneType::User);
		const FRigElementKey Parent4 = Controller->AddBone(TEXT("Parent4"), FRigElementKey(), FTransform( AnimationCore::QuatFromEuler( FVector(60, 60, 60), Order), FVector(80.f, 80.f, 80.f)), true, ERigBoneType::User);
		
		Unit.Child = Child;

		Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
		Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
		Unit.Parents.Add(FConstraintParent(Parent3, 1.0));
		Unit.Parents.Add(FConstraintParent(Parent4, 1.0));

		Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		Unit.bMaintainOffset = false;
		Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
		
		Execute();

		FQuat Result = Hierarchy->GetGlobalTransform(0).GetRotation();
		FQuat Expected = AnimationCore::QuatFromEuler( FVector(-0.853f, 15.189f, -0.572f), Order);
		AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for average interpolation type"));
		
		Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		Unit.bMaintainOffset = false;
		Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
		
		Execute();
		
		Result = Hierarchy->GetGlobalTransform(0).GetRotation();
		Expected = AnimationCore::QuatFromEuler( FVector(16.74f, 8.865f, -5.562f), Order);
		AddErrorIfFalse(Result.Equals(Expected, 0.001f), TEXT("unexpected rotation for shortest interpolation type"));

		
		Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100, 100, -100), Order), FVector(-40.f, -40.f, -40.f)));
		Unit.bMaintainOffset = true;
		Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Average;
		
		Execute();
		
		Result = Hierarchy->GetGlobalTransform(0).GetRotation();
		Expected = AnimationCore::QuatFromEuler( FVector(-0.61f, 8.524f, -45.755f), Order);
		AddErrorIfFalse(Result.Equals(Expected, 0.1f), TEXT("unexpected rotation for maintain offset and average interpolation type"));

		
		Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		Hierarchy->SetGlobalTransform(2, FTransform(AnimationCore::QuatFromEuler( FVector(100.0f, 100.0f, -100.0f), Order), FVector(-40.f, -40.f, -40.f)));
		Unit.bMaintainOffset = true;
		Unit.AdvancedSettings.InterpolationType = EConstraintInterpType::Shortest;
		
		Execute();
		
		Result = Hierarchy->GetGlobalTransform(0).GetRotation();
		Expected = AnimationCore::QuatFromEuler( FVector(-9.143, -4.997f, -38.156f), Order);
		AddErrorIfFalse(Result.Equals(Expected, 0.1f), TEXT("unexpected rotation for maintain offset and shortest interpolation type"));
	}

	// local space offset test
	{
		Unit.Parents.Reset();
		
		Controller->GetHierarchy()->Reset();
		
		// use euler rotation here to match other software's rotation representation more easily
		EEulerRotationOrder Order = EEulerRotationOrder::XZY;
		
		const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
		const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(TEXT("Root"), ERigElementType::Bone),
			FTransform(AnimationCore::QuatFromEuler(FVector(90,0,0), Order), FVector(0.f, 0.f, 50.f)), false, ERigBoneType::User);
		const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(TEXT("Root"), ERigElementType::Bone),
			FTransform(AnimationCore::QuatFromEuler(FVector(0,0,-45), Order), FVector(0.f, 100.f, 0.f)), false, ERigBoneType::User);
		const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(TEXT("Parent2"), ERigElementType::Bone),
			FTransform(AnimationCore::QuatFromEuler(FVector(0,0,-45), Order), FVector(0.f, 0.f, 50.f)), false, ERigBoneType::User);

		Unit.Child = Child;
    
		Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
		Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
		
		{
			Unit.bMaintainOffset = false;
			Execute();
			FQuat Rotation = Hierarchy->GetLocalTransform(3).GetRotation();
			FVector EulerRotation = AnimationCore::EulerFromQuat(Rotation, Order);
			AddErrorIfFalse(EulerRotation.Equals(FVector(41.12f, 10.18f, 22.18f), 0.1f), TEXT("unexpected rotation for local space offset test 1"));
		}
		
		{
			Unit.bMaintainOffset = true;
			Execute();
			FQuat Rotation = Hierarchy->GetLocalTransform(3).GetRotation();
			FVector EulerRotation = AnimationCore::EulerFromQuat(Rotation, Order);
			AddErrorIfFalse(EulerRotation.Equals(FVector(0.f, 0.f, -45.f), 0.1f), TEXT("unexpected rotation for local space offset test 2"));
		}

		{
			Hierarchy->SetLocalTransform(1,
				FTransform(AnimationCore::QuatFromEuler( FVector(120, 40, 20), Order), 
				FVector(0.f,0.f,50.f)));
			
			Unit.bMaintainOffset = true;
			Execute();
			FQuat Rotation = Hierarchy->GetLocalTransform(3).GetRotation();
			FVector EulerRotation = AnimationCore::EulerFromQuat(Rotation, Order);
			AddErrorIfFalse(EulerRotation.Equals(FVector(0.583f, 30.506f, -53.959f), 0.1f), TEXT("unexpected rotation for local space offset test 3"));
		}
	}
	return true;
}
#endif

FRigUnit_ScaleConstraint_Execute()
{
	TFunction<FVector (const FVector&)> GetNonZeroScale([ExecuteContext](const FVector& InScale)
	{
		FVector NonZeroScale = InScale;
       
       	bool bDetectedZeroScale = false;
       	if (FMath::Abs(NonZeroScale.X) < KINDA_SMALL_NUMBER)
       	{
       		NonZeroScale.X = KINDA_SMALL_NUMBER * FMath::Sign(NonZeroScale.X);
       		bDetectedZeroScale = true;
       	}
       	if (FMath::Abs(NonZeroScale.Y) < KINDA_SMALL_NUMBER)
       	{
       		NonZeroScale.Y = KINDA_SMALL_NUMBER * FMath::Sign(NonZeroScale.Y);
       		bDetectedZeroScale = true;
       	}
       	if (FMath::Abs(NonZeroScale.Z) < KINDA_SMALL_NUMBER)
       	{
       		NonZeroScale.Z = KINDA_SMALL_NUMBER * FMath::Sign(NonZeroScale.Z);
       		bDetectedZeroScale = true;
       	}
       
       	if (bDetectedZeroScale)
       	{
       		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Scale value: (%f, %f, %f) contains value too close to 0 to use with scale constraint."), InScale.X, InScale.Y, InScale.Z);		
       	}
       	
       	return NonZeroScale;	
	});

	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!Child.IsValid())
		{
			return;
		}
		
		FTransform ChildInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(Child);

		FVector::FReal OverallWeight = 0;
		for (const FConstraintParent& Parent : Parents)
		{
			const FVector::FReal ClampedWeight = FMath::Max<FVector::FReal>(Parent.Weight, 0.f);

			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			if (!Parent.Item.IsValid())
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			const FVector::FReal WeightNormalizer = 1.0f / OverallWeight;

			FVector MixedGlobalScale = FVector::OneVector;

			for (const FConstraintParent& Parent : Parents)
			{
				const FVector::FReal ClampedWeight = FMath::Max<FVector::FReal>(Parent.Weight, 0.f);

				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				if (!Parent.Item.IsValid())
				{
					continue;
				}

				const FVector::FReal NormalizedWeight = ClampedWeight * WeightNormalizer;

				FVector OffsetScale = FVector::OneVector;
				FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransform(Parent.Item, false);

				if (bMaintainOffset)
				{
					FTransform ParentInitialGlobalTransform = Hierarchy->GetInitialGlobalTransform(Parent.Item);

					FVector ParentInitialGlobalScale = ParentInitialGlobalTransform.GetScale3D();
					
					OffsetScale = ChildInitialGlobalTransform.GetScale3D() / GetNonZeroScale(ParentInitialGlobalScale);
				}

				FVector OffsetParentScale = ParentCurrentGlobalTransform.GetScale3D() * OffsetScale;

				FVector WeightedOffsetParentScale;
				WeightedOffsetParentScale.X = FMath::Pow(OffsetParentScale.X, NormalizedWeight);
				WeightedOffsetParentScale.Y = FMath::Pow(OffsetParentScale.Y, NormalizedWeight);
				WeightedOffsetParentScale.Z = FMath::Pow(OffsetParentScale.Z, NormalizedWeight);

				MixedGlobalScale *= WeightedOffsetParentScale;
			}

			// handle filtering, performed in local space
			FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransform(Child, false);
			FVector ChildParentGlobalScale = ChildParentGlobalTransform.GetScale3D();
			FVector MixedLocalScale = MixedGlobalScale / GetNonZeroScale(ChildParentGlobalScale);
			
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransform(Child, false);

			// Controls need to be handled a bit differently
			FTransform AdditionalOffsetTransform = FTransform::Identity;

			if (Child.Type == ERigElementType::Control)
			{
				if (FRigControlElement* ChildAsControlElement = Hierarchy->Find<FRigControlElement>(Child))
				{
					AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::CurrentLocal);
					// Control's local(parent) space = local * offset
					ChildCurrentLocalTransform *= AdditionalOffsetTransform;
				}
			}
			
			FVector ChildLocalScale = ChildCurrentLocalTransform.GetScale3D();

			FVector FilteredLocalScale;
			FilteredLocalScale.X = Filter.bX ? MixedLocalScale.X : ChildLocalScale.X;
			FilteredLocalScale.Y = Filter.bY ? MixedLocalScale.Y : ChildLocalScale.Y;
			FilteredLocalScale.Z = Filter.bZ ? MixedLocalScale.Z : ChildLocalScale.Z;

			FTransform FilteredMixedLocalTransform = ChildCurrentLocalTransform;

			FilteredMixedLocalTransform.SetScale3D(FilteredLocalScale);

			FTransform FinalLocalTransform = FilteredMixedLocalTransform;

			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				FinalLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, FinalLocalTransform, Weight);
			}
			
			if (Child.Type == ERigElementType::Control)
			{
				// need to convert back to offset space for the actual control value
				FinalLocalTransform = FinalLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				FinalLocalTransform.NormalizeRotation();
			}

			Hierarchy->SetLocalTransform(Child, FinalLocalTransform);
		}
	}
}

FRigVMStructUpgradeInfo FRigUnit_ScaleConstraint::GetUpgradeInfo() const
{
	FRigUnit_ScaleConstraintLocalSpaceOffset NewNode;
	NewNode.Child = Child;
	NewNode.bMaintainOffset = bMaintainOffset;
	NewNode.Filter = Filter;
	NewNode.Parents = Parents;
	NewNode.Weight = Weight;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ScaleConstraint)
{
	// use euler rotation here to match other software's rotation representation more easily
	EEulerRotationOrder Order = EEulerRotationOrder::XZY;
	const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform(FQuat::Identity, FVector::ZeroVector, FVector::OneVector), true, ERigBoneType::User);
	const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform(FQuat::Identity, FVector::ZeroVector, FVector(4,4,4)), true, ERigBoneType::User);
	const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform(FQuat::Identity, FVector::ZeroVector, FVector(1,1,1)), true, ERigBoneType::User);
	
	Unit.Child = Child;

	Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
	Unit.Parents.Add(FConstraintParent(Parent2, 1.0));

	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Unit.bMaintainOffset = false;
	
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetScale3D().Equals(FVector(2,2,2)), TEXT("unexpected scale for maintain offset off"));
	
	Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
	Hierarchy->SetGlobalTransform(2, FTransform(FQuat::Identity, FVector::ZeroVector, FVector(0.5, 0.5, 0.5)));
	Unit.bMaintainOffset = true;
	
	Execute();
	AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetScale3D().Equals(FVector(0.707, 0.707f, 0.707f), 0.001f),
                    TEXT("unexpected scale for maintain offset on"));

	return true;
}

#endif

FRigUnit_ScaleConstraintLocalSpaceOffset_Execute()
{
	TFunction<FVector (const FVector&)> GetNonZeroScale([ExecuteContext](const FVector& InScale)
	{
		FVector NonZeroScale = InScale;
       
       	bool bDetectedZeroScale = false;
       	if (FMath::Abs(NonZeroScale.X) < KINDA_SMALL_NUMBER)
       	{
       		NonZeroScale.X = KINDA_SMALL_NUMBER * FMath::Sign(NonZeroScale.X);
       		bDetectedZeroScale = true;
       	}
       	if (FMath::Abs(NonZeroScale.Y) < KINDA_SMALL_NUMBER)
       	{
       		NonZeroScale.Y = KINDA_SMALL_NUMBER * FMath::Sign(NonZeroScale.Y);
       		bDetectedZeroScale = true;
       	}
       	if (FMath::Abs(NonZeroScale.Z) < KINDA_SMALL_NUMBER)
       	{
       		NonZeroScale.Z = KINDA_SMALL_NUMBER * FMath::Sign(NonZeroScale.Z);
       		bDetectedZeroScale = true;
       	}
       
       	if (bDetectedZeroScale)
       	{
       		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Scale value: (%f, %f, %f) contains value too close to 0 to use with scale constraint."), InScale.X, InScale.Y, InScale.Z);		
       	}
       	
       	return NonZeroScale;	
	});

	if (Weight < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;	
	if (Hierarchy)
	{
		if (!ChildCache.UpdateCache(Child, Hierarchy))
		{
			return;
		}

		if(ParentCaches.Num() != Parents.Num())
		{
			ParentCaches.SetNumZeroed(Parents.Num());
		}
		
		FTransform ChildInitialLocalTransform = Hierarchy->GetLocalTransformByIndex(ChildCache, true);
		FTransform AdditionalOffsetTransform;
		
		// Controls need to be handled a bit differently
		const bool bChildIsControl = Child.Type == ERigElementType::Control;
		if (Child.Type == ERigElementType::Control)
		{
			if (FRigControlElement* ChildAsControlElement = Hierarchy->Get<FRigControlElement>(ChildCache))
			{
				AdditionalOffsetTransform = Hierarchy->GetControlOffsetTransform(ChildAsControlElement, ERigTransformType::CurrentLocal);

				// Control's local(parent) space position = local * offset
				ChildInitialLocalTransform *= AdditionalOffsetTransform;
			}
		}
		FVector::FReal OverallWeight = 0;
		for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
		{
			const FConstraintParent& Parent = Parents[ParentIndex];
			if (!ParentCaches[ParentIndex].UpdateCache(Parent.Item, Hierarchy))
			{
				continue;
			}
			
			const FVector::FReal ClampedWeight = FMath::Max<FVector::FReal>(Parent.Weight, 0.f);
			if (ClampedWeight < KINDA_SMALL_NUMBER)
			{
				continue;
			}

			OverallWeight += ClampedWeight;
		}

		const FVector::FReal WeightNormalizer = 1.0f / OverallWeight;
		
		if (OverallWeight > KINDA_SMALL_NUMBER)
		{
			FVector OffsetScale = FVector::OneVector;
			if (bMaintainOffset)
			{
				FVector MixedInitialGlobalScale = FVector::OneVector;

				for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
				{
					if (!ParentCaches[ParentIndex].IsValid())
					{
						continue;
					}
					
					const FConstraintParent& Parent = Parents[ParentIndex];

					const FVector::FReal ClampedWeight = FMath::Max<FVector::FReal>(Parent.Weight, 0.f);
					if (ClampedWeight < KINDA_SMALL_NUMBER)
					{
						continue;
					}

					if (!Parent.Item.IsValid())
					{
						continue;
					}

					const FVector::FReal NormalizedWeight = ClampedWeight * WeightNormalizer;

					const FTransform ParentInitialGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], true);
					const FVector ParentInitialGlobalScale = ParentInitialGlobalTransform.GetScale3D();
					
					FVector WeightedInitialParentGlobalScale;
					WeightedInitialParentGlobalScale.X = FMath::Pow(ParentInitialGlobalScale.X, NormalizedWeight);
					WeightedInitialParentGlobalScale.Y = FMath::Pow(ParentInitialGlobalScale.Y, NormalizedWeight);
					WeightedInitialParentGlobalScale.Z = FMath::Pow(ParentInitialGlobalScale.Z, NormalizedWeight);

					MixedInitialGlobalScale *= WeightedInitialParentGlobalScale;
				}
				
				// handle filtering, performed in local space
				const FTransform ChildParentInitialGlobalTransform = Hierarchy->GetParentTransformByIndex(ChildCache, true);
				const FVector ChildParentInitialGlobalScale = ChildParentInitialGlobalTransform.GetScale3D();
				const FVector MixedInitialLocalScale = MixedInitialGlobalScale / GetNonZeroScale(ChildParentInitialGlobalScale);
			
				OffsetScale = ChildInitialLocalTransform.GetScale3D() / GetNonZeroScale(MixedInitialLocalScale);
			}

			
			FVector MixedGlobalScale = FVector::OneVector;

			for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				if (!ParentCaches[ParentIndex].IsValid())
				{
					continue;
				}
					
				const FConstraintParent& Parent = Parents[ParentIndex];
				
				const FVector::FReal ClampedWeight = FMath::Max<FVector::FReal>(Parent.Weight, 0.f);
				if (ClampedWeight < KINDA_SMALL_NUMBER)
				{
					continue;
				}

				const FVector::FReal NormalizedWeight = ClampedWeight * WeightNormalizer;

				FTransform ParentCurrentGlobalTransform = Hierarchy->GetGlobalTransformByIndex(ParentCaches[ParentIndex], false);
				FVector ParentCurrentGlobalScale = ParentCurrentGlobalTransform.GetScale3D();

				FVector WeightedParentGlobalScale;
				WeightedParentGlobalScale.X = FMath::Pow(ParentCurrentGlobalScale.X, NormalizedWeight);
				WeightedParentGlobalScale.Y = FMath::Pow(ParentCurrentGlobalScale.Y, NormalizedWeight);
				WeightedParentGlobalScale.Z = FMath::Pow(ParentCurrentGlobalScale.Z, NormalizedWeight);

				MixedGlobalScale *= WeightedParentGlobalScale;
			}

			// handle filtering, performed in local space
			FTransform ChildParentGlobalTransform = Hierarchy->GetParentTransform(Child, false);
			FVector ChildParentGlobalScale = ChildParentGlobalTransform.GetScale3D();
			FVector MixedLocalScale = MixedGlobalScale / GetNonZeroScale(ChildParentGlobalScale);
			if (bMaintainOffset)
			{
				MixedLocalScale = MixedLocalScale * OffsetScale;
			}
			
			FTransform ChildCurrentLocalTransform = Hierarchy->GetLocalTransform(Child, false);

			// Controls need to be handled a bit differently
			if (bChildIsControl)
			{
				// Control's local(parent) space = local * offset
				ChildCurrentLocalTransform *= AdditionalOffsetTransform;
			}

			FTransform MixedLocalTransform = ChildCurrentLocalTransform;
			if(!Filter.HasNoEffect())
			{
				const FVector ChildLocalScale = ChildCurrentLocalTransform.GetScale3D();
				MixedLocalScale.X = Filter.bX ? MixedLocalScale.X : ChildLocalScale.X;
				MixedLocalScale.Y = Filter.bY ? MixedLocalScale.Y : ChildLocalScale.Y;
				MixedLocalScale.Z = Filter.bZ ? MixedLocalScale.Z : ChildLocalScale.Z;
			}
			MixedLocalTransform.SetScale3D(MixedLocalScale);

			if (Weight < 1.0f - KINDA_SMALL_NUMBER)
			{
				MixedLocalTransform = FControlRigMathLibrary::LerpTransform(ChildCurrentLocalTransform, MixedLocalTransform, Weight);
			}
			
			if (bChildIsControl)
			{
				// need to convert back to offset space for the actual control value
				MixedLocalTransform = MixedLocalTransform.GetRelativeTransform(AdditionalOffsetTransform);
				MixedLocalTransform.NormalizeRotation();
			}

			Hierarchy->SetLocalTransform(Child, MixedLocalTransform);
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "Units/RigUnitTest.h"

IMPLEMENT_RIGUNIT_AUTOMATION_TEST(FRigUnit_ScaleConstraintLocalSpaceOffset)
{
	// multi parent test
	{
		// use euler rotation here to match other software's rotation representation more easily
		EEulerRotationOrder Order = EEulerRotationOrder::XZY;
		const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(), FTransform(FQuat::Identity, FVector::ZeroVector, FVector::OneVector), true, ERigBoneType::User);
		const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(), FTransform(FQuat::Identity, FVector::ZeroVector, FVector(4,4,4)), true, ERigBoneType::User);
		const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(), FTransform(FQuat::Identity, FVector::ZeroVector, FVector(1,1,1)), true, ERigBoneType::User);
	
		Unit.Child = Child;

		Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
		Unit.Parents.Add(FConstraintParent(Parent2, 1.0));

		Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		Unit.bMaintainOffset = false;
	
		Execute();
		AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetScale3D().Equals(FVector(2,2,2)), TEXT("unexpected scale for maintain offset off"));
	
		Hierarchy->ResetPoseToInitial(ERigElementType::Bone);
		Hierarchy->SetGlobalTransform(2, FTransform(FQuat::Identity, FVector::ZeroVector, FVector(0.5, 0.5, 0.5)));
		Unit.bMaintainOffset = true;
	
		Execute();
		AddErrorIfFalse(Hierarchy->GetGlobalTransform(0).GetScale3D().Equals(FVector(0.707, 0.707f, 0.707f), 0.001f),
						TEXT("unexpected scale for maintain offset on"));	
	}

	// local offset test
	{
		Unit.Parents.Reset();
		
		Controller->GetHierarchy()->Reset();
		
		// use euler rotation here to match other software's rotation representation more easily
		EEulerRotationOrder Order = EEulerRotationOrder::XZY;
		
		const FRigElementKey Root = Controller->AddBone(TEXT("Root"), FRigElementKey(), FTransform(FVector(0.f, 0.f, 0.f)), true, ERigBoneType::User);
		const FRigElementKey Parent1 = Controller->AddBone(TEXT("Parent1"), FRigElementKey(TEXT("Root"), ERigElementType::Bone),
			FTransform(AnimationCore::QuatFromEuler(FVector(0,0,0), Order), FVector(0.f, 0.f, 50.f), FVector(4,5,6)),
			false, ERigBoneType::User);
		const FRigElementKey Parent2 = Controller->AddBone(TEXT("Parent2"), FRigElementKey(TEXT("Root"), ERigElementType::Bone),
			FTransform(AnimationCore::QuatFromEuler(FVector(0,0,0), Order), FVector(0.f, 100.f, 0.f), FVector(1,2,3)),
			false, ERigBoneType::User);
		const FRigElementKey Child = Controller->AddBone(TEXT("Child"), FRigElementKey(TEXT("Parent2"), ERigElementType::Bone),
			FTransform(AnimationCore::QuatFromEuler(FVector(0,0,0), Order), FVector(0.f, 0.f, 50.f)),
			false, ERigBoneType::User);

		Unit.Child = Child;
    
		Unit.Parents.Add(FConstraintParent(Parent1, 1.0));
		Unit.Parents.Add(FConstraintParent(Parent2, 1.0));
		
		{
			Unit.bMaintainOffset = false;
			Execute();
			FVector Scale = Hierarchy->GetLocalTransform(3).GetScale3D();
			AddErrorIfFalse(Scale.Equals(FVector(2.f, 1.58f, 1.414f), 0.1f), TEXT("unexpected scale for local space offset test 1"));
		}
		
		{
			Unit.bMaintainOffset = true;
			Execute();
			FVector Scale = Hierarchy->GetLocalTransform(3).GetScale3D();
			AddErrorIfFalse(Scale.Equals(FVector(1.f, 1.f, 1.f), 0.1f), TEXT("unexpected scale for local space offset test 2"));
		}

		{
			Hierarchy->SetLocalTransform(1,
				FTransform(AnimationCore::QuatFromEuler( FVector(0, 0, 0), Order), 
				FVector(0.f,0.f,50.f),
				FVector(4,10,6)));
			
			Unit.bMaintainOffset = true;
			Execute();
			FVector Scale = Hierarchy->GetLocalTransform(3).GetScale3D();
			AddErrorIfFalse(Scale.Equals(FVector(1.0f, 1.414f, 1.0f), 0.1f), TEXT("unexpected scale for local space offset test 3"));
		}
		
	}
	

	
	return true;
}

#endif
