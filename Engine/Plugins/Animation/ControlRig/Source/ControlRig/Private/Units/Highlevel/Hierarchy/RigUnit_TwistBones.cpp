// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_TwistBones.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_TwistBones)

FRigUnit_TwistBones_Execute()
{
	FRigElementKeyCollection Items;
	if(WorkData.CachedItems.Num() == 0)
	{
		Items = FRigElementKeyCollection::MakeFromChain(
			ExecuteContext.Hierarchy,
			FRigElementKey(StartBone, ERigElementType::Bone),
			FRigElementKey(EndBone, ERigElementType::Bone),
			false /* reverse */
		);
	}

	FRigUnit_TwistBonesPerItem::StaticExecute(
		ExecuteContext, 
		Items,
		TwistAxis,
		PoleAxis,
		TwistEaseType,
		Weight,
		bPropagateToChildren,
		WorkData);
}

FRigVMStructUpgradeInfo FRigUnit_TwistBones::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigUnit_TwistBonesPerItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	TArray<FCachedRigElement>& CachedItems = WorkData.CachedItems;
	TArray<float>& ItemRatios = WorkData.ItemRatios;
	TArray<FTransform>& ItemTransforms = WorkData.ItemTransforms;

	if (CachedItems.Num() == 0 && Items.Num() > 0)
	{
		ItemRatios.Reset();
		ItemTransforms.Reset();

		for (FRigElementKey Item : Items)
		{
			CachedItems.Add(FCachedRigElement(Item, Hierarchy));
		}

		if (CachedItems.Num() < 3)
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Didn't find enough items. You need at least three in the chain!"));
			return;
		}

		ItemRatios.AddZeroed(CachedItems.Num());
		ItemTransforms.AddUninitialized(CachedItems.Num());
	}

	if (CachedItems.Num() < 3 || ItemRatios.Num() == 0 || ItemTransforms.Num() == 0)
	{
		return;
	}

	if (TwistAxis.IsNearlyZero())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Twist Axis is not set."));
		return;
	}

	if (PoleAxis.IsNearlyZero())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Pole Axis is not set."));
		return;
	}

	ItemTransforms[0] = Hierarchy->GetGlobalTransform(CachedItems[0]);
	ItemRatios[0] = 0.f;

	FVector LastPosition = ItemTransforms[0].GetLocation();
	for (int32 Index = 1; Index < CachedItems.Num(); Index++)
	{
		ItemTransforms[Index] = Hierarchy->GetGlobalTransform(CachedItems[Index]);
		FVector Position = ItemTransforms[Index].GetLocation();
		float Distance = (LastPosition - Position).Size();
		ItemRatios[Index] = ItemRatios[Index - 1] + Distance;
		LastPosition = Position;
	}

	if (ItemRatios.Last() <= SMALL_NUMBER)
	{
		return;
	}

	for (int32 Index = 1; Index < ItemRatios.Num(); Index++)
	{
		ItemRatios[Index] = ItemRatios[Index] / ItemRatios.Last();
		ItemRatios[Index] = FControlRigMathLibrary::EaseFloat(ItemRatios[Index], TwistEaseType);
	}

	FVector TwistAxisN = TwistAxis.GetSafeNormal();
	FVector PoleAxisN = PoleAxis.GetSafeNormal();
	FQuat DiffTwistQuat = ItemTransforms.Last().GetRelativeTransform(ItemTransforms[0]).GetRotation();

	for (int32 Index = 1; Index < ItemRatios.Num() - 1; Index++)
	{
		FVector CurrentTwistVector = ItemTransforms[Index].TransformVectorNoScale(TwistAxisN);
		FVector CurrentPoleVector = ItemTransforms[Index].TransformVectorNoScale(PoleAxisN);

		FQuat TwistQuat = FQuat::Slerp(FQuat::Identity, DiffTwistQuat, ItemRatios[Index]);
		TwistQuat = ItemTransforms[0].GetRotation() * TwistQuat;
		FVector DesiredPoleVector = TwistQuat.GetNormalized().RotateVector(PoleAxisN);

		DesiredPoleVector = DesiredPoleVector - FVector::DotProduct(CurrentTwistVector, DesiredPoleVector) * CurrentTwistVector;

		FQuat OffsetQuat = FQuat::FindBetweenVectors(CurrentPoleVector, DesiredPoleVector).GetNormalized();

		OffsetQuat = FQuat::Slerp(ItemTransforms[Index].GetRotation(), OffsetQuat * ItemTransforms[Index].GetRotation(), FMath::Clamp<float>(Weight, 0.f, 1.f));
		ItemTransforms[Index].SetRotation(OffsetQuat);
		Hierarchy->SetGlobalTransform(CachedItems[Index], ItemTransforms[Index], bPropagateToChildren);
	}
}

FRigVMStructUpgradeInfo FRigUnit_TwistBonesPerItem::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

