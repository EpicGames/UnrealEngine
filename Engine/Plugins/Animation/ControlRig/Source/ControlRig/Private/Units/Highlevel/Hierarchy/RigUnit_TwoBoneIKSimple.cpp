// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Highlevel/Hierarchy/RigUnit_TwoBoneIKSimple.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"
#include "TwoBoneIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_TwoBoneIKSimple)

FRigUnit_TwoBoneIKSimple_Execute()
{
	FRigUnit_TwoBoneIKSimplePerItem::StaticExecute(
		ExecuteContext, 
		FRigElementKey(BoneA, ERigElementType::Bone),
		FRigElementKey(BoneB, ERigElementType::Bone),
		FRigElementKey(EffectorBone, ERigElementType::Bone),
		Effector,
		PrimaryAxis,
		SecondaryAxis,
		SecondaryAxisWeight,
		PoleVector,
		PoleVectorKind,
		FRigElementKey(PoleVectorSpace, ERigElementType::Bone),
		bEnableStretch,
		StretchStartRatio,
		StretchMaximumRatio,
		Weight,
		BoneALength,
		BoneBLength,
		bPropagateToChildren,
		DebugSettings,
		CachedBoneAIndex,
		CachedBoneBIndex,
		CachedEffectorBoneIndex,
		CachedPoleVectorSpaceIndex);
}

FRigVMStructUpgradeInfo FRigUnit_TwoBoneIKSimple::GetUpgradeInfo() const
{
	FRigUnit_TwoBoneIKSimplePerItem NewNode;
	NewNode.ItemA = FRigElementKey(BoneA, ERigElementType::Bone);
	NewNode.ItemB = FRigElementKey(BoneB, ERigElementType::Bone);
	NewNode.EffectorItem = FRigElementKey(EffectorBone, ERigElementType::Bone);
	NewNode.Effector = Effector;
	NewNode.PrimaryAxis = PrimaryAxis;
	NewNode.SecondaryAxis = SecondaryAxis;
	NewNode.SecondaryAxisWeight = SecondaryAxisWeight;
	NewNode.PoleVector = PoleVector;
	NewNode.PoleVectorKind = PoleVectorKind;
	NewNode.PoleVectorSpace = FRigElementKey(PoleVectorSpace, ERigElementType::Bone);
	NewNode.bEnableStretch = bEnableStretch;
	NewNode.StretchStartRatio = StretchStartRatio;
	NewNode.StretchMaximumRatio = StretchMaximumRatio;
	NewNode.Weight = Weight;
	NewNode.ItemALength = BoneALength;
	NewNode.ItemBLength = BoneBLength;
	NewNode.bPropagateToChildren = bPropagateToChildren;
	NewNode.DebugSettings = DebugSettings;
	
	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("BoneA"), TEXT("ItemA.Name"));
	Info.AddRemappedPin(TEXT("BoneB"), TEXT("ItemB.Name"));
	Info.AddRemappedPin(TEXT("EffectorBone"), TEXT("EffectorItem.Name"));
	Info.AddRemappedPin(TEXT("BoneALength"), TEXT("ItemALength"));
	Info.AddRemappedPin(TEXT("BoneBLength"), TEXT("ItemBLength"));
	return Info;
}

#if WITH_EDITOR

bool FRigUnit_TwoBoneIKSimplePerItem::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_TwoBoneIKSimplePerItem, PoleVector))
	{
		const FTransform ParentTransform = Hierarchy->GetParentTransform(PoleVectorSpace, false);
		Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, ParentTransform, false);
		Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(PoleVector), false);
		if(!InInfo->bInitialized)
		{
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(PoleVector), true);
		}
		return true;
	}
	return FRigUnit_HighlevelBaseMutable::UpdateHierarchyForDirectManipulation(InNode, InInstance, InContext, InInfo);
}

bool FRigUnit_TwoBoneIKSimplePerItem::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if(InInfo->Target.Name == GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_TwoBoneIKSimplePerItem, PoleVector))
	{
		PoleVector = Hierarchy->GetLocalTransform(InInfo->ControlKey, false).GetTranslation();
		return true;
	}
	return FRigUnit_HighlevelBaseMutable::UpdateDirectManipulationFromHierarchy(InNode, InInstance, InContext, InInfo);
}

#endif

FRigUnit_TwoBoneIKSimplePerItem_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	if (!CachedItemAIndex.UpdateCache(ItemA, Hierarchy) ||
		!CachedItemBIndex.UpdateCache(ItemB, Hierarchy))
	{
		return;
	}

	CachedEffectorItemIndex.UpdateCache(EffectorItem, Hierarchy);
	CachedPoleVectorSpaceIndex.UpdateCache(PoleVectorSpace, Hierarchy);

	if (Weight <= SMALL_NUMBER)
	{
		return;
	}

	FVector PoleTarget = PoleVector;
	if (CachedPoleVectorSpaceIndex.IsValid())
	{
		const FTransform PoleVectorSpaceTransform = Hierarchy->GetGlobalTransform(CachedPoleVectorSpaceIndex);
		if (PoleVectorKind == EControlRigVectorKind::Direction)
		{
			PoleTarget = PoleVectorSpaceTransform.TransformVectorNoScale(PoleTarget);
		}
		else
		{
			PoleTarget = PoleVectorSpaceTransform.TransformPositionNoScale(PoleTarget);
		}
	}

	FTransform TransformA = Hierarchy->GetGlobalTransform(CachedItemAIndex);
	FTransform TransformB = TransformA;
	TransformB.SetLocation(Hierarchy->GetGlobalTransform(CachedItemBIndex).GetLocation());
	FTransform TransformC = Effector;

	float LengthA = ItemALength;
	float LengthB = ItemBLength;

	if (LengthA < SMALL_NUMBER)
	{
		FTransform InitialTransformA = Hierarchy->GetInitialGlobalTransform(CachedItemAIndex);
		FVector Scale = FVector::OneVector;
		if (InitialTransformA.GetScale3D().SizeSquared() > SMALL_NUMBER)
		{
			Scale = TransformA.GetScale3D() / InitialTransformA.GetScale3D();
		}
		FVector Diff = InitialTransformA.GetLocation() - Hierarchy->GetInitialGlobalTransform(CachedItemBIndex).GetLocation();
		Diff = Diff * Scale;
		LengthA = Diff.Size();
	}

	if (LengthB < SMALL_NUMBER && CachedEffectorItemIndex != INDEX_NONE)
	{
		FTransform InitialTransformB = Hierarchy->GetInitialGlobalTransform(CachedItemBIndex);
		FVector Scale = FVector::OneVector;
		if (InitialTransformB.GetScale3D().SizeSquared() > SMALL_NUMBER)
		{
			Scale = TransformB.GetScale3D() / InitialTransformB.GetScale3D();
		}
		FVector Diff = InitialTransformB.GetLocation() - Hierarchy->GetInitialGlobalTransform(CachedEffectorItemIndex).GetLocation();
		Diff = Diff * Scale;
		LengthB = Diff.Size();
	}

	if (LengthA < SMALL_NUMBER || LengthB < SMALL_NUMBER)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("Item Lengths are not provided.\nEither set item length(s) or set effector item."));
		return;
	}

	FControlRigMathLibrary::SolveBasicTwoBoneIK(TransformA, TransformB, TransformC, PoleTarget, PrimaryAxis, SecondaryAxis, SecondaryAxisWeight, LengthA, LengthB, bEnableStretch, StretchStartRatio, StretchMaximumRatio);

	if (ExecuteContext.GetDrawInterface() != nullptr && DebugSettings.bEnabled)
	{
		const FLinearColor Dark = FLinearColor(0.f, 0.2f, 1.f, 1.f);
		const FLinearColor Bright = FLinearColor(0.f, 1.f, 1.f, 1.f);
		ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, TransformA.GetLocation(), TransformB.GetLocation(), Dark);
		ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, TransformB.GetLocation(), TransformC.GetLocation(), Dark);
		ExecuteContext.GetDrawInterface()->DrawLine(DebugSettings.WorldOffset, TransformB.GetLocation(), PoleTarget, Bright);
		ExecuteContext.GetDrawInterface()->DrawBox(DebugSettings.WorldOffset, FTransform(FQuat::Identity, PoleTarget, FVector(1.f, 1.f, 1.f) * DebugSettings.Scale * 0.1f), Bright);
	}

	if (Weight < 1.0f - SMALL_NUMBER)
	{
		FVector PositionB = TransformA.InverseTransformPosition(TransformB.GetLocation());
		FVector PositionC = TransformB.InverseTransformPosition(TransformC.GetLocation());
		TransformA.SetRotation(FQuat::Slerp(Hierarchy->GetGlobalTransform(CachedItemAIndex).GetRotation(), TransformA.GetRotation(), Weight));
		TransformB.SetRotation(FQuat::Slerp(Hierarchy->GetGlobalTransform(CachedItemBIndex).GetRotation(), TransformB.GetRotation(), Weight));
		if(CachedEffectorItemIndex != INDEX_NONE)
		{
			TransformC.SetRotation(FQuat::Slerp(Hierarchy->GetGlobalTransform(CachedEffectorItemIndex).GetRotation(), TransformC.GetRotation(), Weight));
		}
		TransformB.SetLocation(TransformA.TransformPosition(PositionB));
		TransformC.SetLocation(TransformB.TransformPosition(PositionC));
	}

	Hierarchy->SetGlobalTransform(CachedItemAIndex, TransformA, bPropagateToChildren);
	Hierarchy->SetGlobalTransform(CachedItemBIndex, TransformB, bPropagateToChildren);
	if(CachedEffectorItemIndex != INDEX_NONE)
	{
		Hierarchy->SetGlobalTransform(CachedEffectorItemIndex, TransformC, bPropagateToChildren);
	}
}

FRigUnit_TwoBoneIKSimpleVectors_Execute()
{
	AnimationCore::SolveTwoBoneIK(Root, Elbow, Effector, PoleVector, Effector, Elbow, Effector, BoneALength, BoneBLength, bEnableStretch, StretchStartRatio, StretchMaximumRatio);
}

FRigUnit_TwoBoneIKSimpleTransforms_Execute()
{
	FControlRigMathLibrary::SolveBasicTwoBoneIK(Root, Elbow, Effector, PoleVector, PrimaryAxis, SecondaryAxis, SecondaryAxisWeight, BoneALength, BoneBLength, bEnableStretch, StretchStartRatio, StretchMaximumRatio);
}
