// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Modules/RigUnit_ConnectionCandidates.h"
#include "Units/Modules/RigUnit_ConnectorExecution.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ConnectionCandidates)

FRigUnit_GetCandidates_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// only allow these nodes during connector resolval event
	if(ExecuteContext.GetEventName() != FRigUnit_ConnectorExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Can only use GetCandidates during %s event."), *FRigUnit_ConnectorExecution::EventName.ToString());
		return;
	}

	Candidates.Reset();
	Algo::Transform(ExecuteContext.UnitContext.ConnectionResolve.Matches, Candidates, [](const FRigElementResolveResult& Match)
	{
		return Match.GetKey();
	});
	Connector = ExecuteContext.UnitContext.ConnectionResolve.Connector;
}

FRigUnit_DiscardMatches_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// only allow these nodes during connector resolval event
	if(ExecuteContext.GetEventName() != FRigUnit_ConnectorExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Can only use DiscardMatches during %s event."), *FRigUnit_ConnectorExecution::EventName.ToString());
		return;
	}

	// Filter any items that are already excluded
	TArray<FRigElementKey> FilterExcluded = Excluded.FilterByPredicate([ExecuteContext](const FRigElementKey& Exclude)
	{
		return !ExecuteContext.UnitContext.ConnectionResolve.Excluded.ContainsByPredicate([Exclude](const FRigElementResolveResult& Existing)
		{
			return Existing.GetKey() == Exclude;
		});
	});

	// Remove elements from matches, and add to excluded
	for (FRigElementKey& Exclude : FilterExcluded)
	{
		int32 Index = ExecuteContext.UnitContext.ConnectionResolve.Matches.IndexOfByPredicate([Exclude](const FRigElementResolveResult& Match)
		{
			return Exclude == Match.GetKey();
		});
		if (Index != INDEX_NONE)
		{
			ExecuteContext.UnitContext.ConnectionResolve.Matches.RemoveAt(Index);
		}

		ExecuteContext.UnitContext.ConnectionResolve.Excluded.Add(FRigElementResolveResult(Exclude, ERigElementResolveState::InvalidTarget, FText::FromString(Message)));
	}
}

FRigUnit_SetDefaultMatch_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// only allow these nodes during connector resolval event
	if(ExecuteContext.GetEventName() != FRigUnit_ConnectorExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Can only use SetDefaultMatch during %s event."), *FRigUnit_ConnectorExecution::EventName.ToString());
		return;
	}

	FModularRigResolveResult& Resolve = ExecuteContext.UnitContext.ConnectionResolve;

	// Remove the old default if it exists (set to just possible target)
	FRigElementResolveResult* OldDefault = const_cast<FRigElementResolveResult*>(Resolve.GetDefaultMatch());
	if (OldDefault)
	{
		OldDefault->SetPossibleTarget();
	}

	// Make sure the Default element is contained in the matches array
	FRigElementResolveResult* NewDefault = const_cast<FRigElementResolveResult*>(Resolve.FindMatch(Default));
	if (NewDefault)
	{
		NewDefault->SetDefaultTarget();
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Cannot set default match %s because it is not part of %s matches."), *Default.ToString(), *Resolve.GetConnectorKey().ToString());
	}
}

FRigUnit_SetDefaultMatchArray_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// only allow these nodes during connector resolval event
	if (ExecuteContext.GetEventName() != FRigUnit_ConnectorExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Can only use SetDefaultMatchForArray during %s event."), *FRigUnit_ConnectorExecution::EventName.ToString());
		return;
	}

	FModularRigResolveResult& Resolve = ExecuteContext.UnitContext.ConnectionResolve;

	// Make sure the Default element is contained in the matches array
	TArray<FRigElementResolveResult*> NewDefaults;
	for (const FRigElementKey& DefaultEntry : Default)
	{
		if (FRigElementResolveResult* OldDefault = const_cast<FRigElementResolveResult*>(Resolve.GetDefaultMatch()))
		{
			OldDefault->SetPossibleTarget();
		}

		if (FRigElementResolveResult* NewDefault = const_cast<FRigElementResolveResult*>(Resolve.FindMatch(DefaultEntry)))
		{
			NewDefault->SetDefaultTarget();
		}
		else
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Cannot set default match for %s because it is not part of %s matches."), *DefaultEntry.ToString(), *Resolve.GetConnectorKey().ToString());
			return;
		}
	}
}
