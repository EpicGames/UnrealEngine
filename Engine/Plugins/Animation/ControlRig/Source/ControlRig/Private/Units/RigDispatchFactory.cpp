// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/RigDispatchFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDispatchFactory)

#if WITH_EDITOR

FString FRigDispatchFactory::GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InTypeIndex == FRigVMRegistry::Get().GetTypeIndex<FEulerTransform>())
	{
		static FString DefaultValueString = GetDefaultValueForStruct(FEulerTransform::Identity); 
		return DefaultValueString;
	}
	if(InTypeIndex == FRigVMRegistry::Get().GetTypeIndex<FRigElementKey>())
	{
		static FString DefaultValueString = GetDefaultValueForStruct(FRigElementKey(NAME_None, ERigElementType::Bone)); 
		return DefaultValueString;
	}
	return FRigVMDispatchFactory::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

#endif
