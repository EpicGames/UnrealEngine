// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/RigUnit.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathVector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit)

DEFINE_LOG_CATEGORY_STATIC(LogRigUnit, Log, All);

#if WITH_EDITOR

bool FRigUnit::GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const
{
	const UScriptStruct* ScriptStruct = InNode->GetScriptStruct();
	if(ScriptStruct == nullptr)
	{
		if(OutFailureReason)
		{
			*OutFailureReason = TEXT("Node is not resolved yet.");
		}
		return false;
	}
	
	for(const URigVMPin* Pin : InNode->GetPins())
	{
		(void)AddDirectManipulationTarget_Internal(InOutTargets, Pin, ScriptStruct);
	}

	return true;
}

bool FRigUnit::AddDirectManipulationTarget_Internal(TArray<FRigDirectManipulationTarget>& InOutTargets, const URigVMPin* InPin, const UScriptStruct* InScriptStruct)
{
	check(InPin);

	if(InPin->GetDirection() == ERigVMPinDirection::Input ||
		InPin->GetDirection() == ERigVMPinDirection::IO ||
		InPin->GetDirection() == ERigVMPinDirection::Visible)
	{
		if(!InPin->IsArray() && (
			InPin->GetCPPTypeObject() == TBaseStructure<FTransform>::Get() ||
			InPin->GetCPPTypeObject() == TBaseStructure<FEulerTransform>::Get() ||
			InPin->GetCPPTypeObject() == TBaseStructure<FVector>::Get() ||
			InPin->GetCPPTypeObject() == TBaseStructure<FQuat>::Get()
		))
		{
			if(const URigVMPin* ParentPin = InPin->GetParentPin())
			{
				if(ParentPin->GetCPPTypeObject() == TBaseStructure<FTransform>::Get() ||
					ParentPin->GetCPPTypeObject() == TBaseStructure<FEulerTransform>::Get())
				{
					return false;
				}
			}

			ERigControlType ControlType = ERigControlType::EulerTransform;
			if(InPin->GetCPPTypeObject() == TBaseStructure<FVector>::Get())
			{
				ControlType = ERigControlType::Position;
			}
			else if(InPin->GetCPPTypeObject() == TBaseStructure<FQuat>::Get())
			{
				ControlType = ERigControlType::Rotator;
			}
			InOutTargets.AddUnique({InPin->GetSegmentPath(true), ControlType});
		}

		for(const URigVMPin* SubPin : InPin->GetSubPins())
		{
			AddDirectManipulationTarget_Internal(InOutTargets, SubPin, InScriptStruct);
		}
	}

	return false;
}

TTuple<const FStructProperty*, uint8*> FRigUnit::FindStructPropertyAndTargetMemory(
	TSharedPtr<FStructOnScope> InInstance, const UScriptStruct* InStruct, const FString& InPinPath)
{
	FString PinPath = InPinPath;
	PinPath = PinPath.Replace(TEXT("["), TEXT("."));
	PinPath = PinPath.Replace(TEXT("]"), TEXT("."));
	PinPath = PinPath.Replace(TEXT(".."), TEXT("."));
	PinPath.TrimCharInline('.', nullptr);

	FString Left, Right;
	if(!PinPath.Split(TEXT("."), &Left, &Right))
	{
		Left = PinPath;
		Right.Reset();
	}

	const FProperty* Property = InStruct->FindPropertyByName(*Left);
	if(Property == nullptr)
	{
		return {nullptr, nullptr};
	}

	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	if(StructProperty == nullptr)
	{
		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			StructProperty = CastField<FStructProperty>(ArrayProperty->Inner);
			// if we are in an array we expect the path to exist since we
			// need to exist a sub element.
			check(!Right.IsEmpty());
		}
	}
	if(StructProperty == nullptr)
	{
		return {nullptr, nullptr};
	}

	uint8* Memory = Property->ContainerPtrToValuePtr<uint8>(InInstance->GetStructMemory());
	if(!Right.IsEmpty())
	{
		const FRigVMPropertyPath PropertyPath(Property, Right);
		Memory = PropertyPath.GetData<uint8>(Memory, Property);
	}

	return {StructProperty, Memory};
}

void FRigUnit::ConfigureDirectManipulationControl(const URigVMUnitNode* InNode, TSharedPtr<FRigDirectManipulationInfo> InInfo, FRigControlSettings& InOutSettings, FRigControlValue& InOutValue) const
{
	if(InInfo->Target.ControlType == ERigControlType::Transform)
	{
		InOutSettings.ControlType = ERigControlType::EulerTransform;
		InOutValue = FRigControlValue::Make<FEulerTransform>(FEulerTransform::Identity);
	}
	else if(InInfo->Target.ControlType == ERigControlType::Position)
	{
		InOutSettings.ControlType = ERigControlType::Position;
		InOutValue = FRigControlValue::Make<FVector>(FVector::ZeroVector);
	}
	else if(InInfo->Target.ControlType == ERigControlType::Rotator)
	{
		InOutSettings.ControlType = ERigControlType::Rotator;
		InOutValue = FRigControlValue::Make<FRotator>(FRotator::ZeroRotator);
	}
}

bool FRigUnit::UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	const UScriptStruct* Struct = InNode->GetScriptStruct();
	check(Struct);
	check(InInstance.IsValid() && InInstance->IsValid());
	check(InInstance->GetStruct() == Struct);

	if(!InInfo.IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	// deal with special cases here
	if(Struct == FRigVMFunction_MathTransformMakeAbsolute::StaticStruct())
	{
		if(InInfo->Target.Name.Equals(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_MathTransformMakeAbsolute, Local), ESearchCase::CaseSensitive))
		{
			const FRigVMFunction_MathTransformMakeAbsolute* MakeAbsolute =
				reinterpret_cast<const FRigVMFunction_MathTransformMakeAbsolute*>(InInstance->GetStructMemory());
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, MakeAbsolute->Parent, false);
			Hierarchy->SetLocalTransform(InInfo->ControlKey, MakeAbsolute->Local, false);

			if(!InInfo->bInitialized)
			{
				Hierarchy->SetLocalTransform(InInfo->ControlKey, MakeAbsolute->Local, true);
			}
			return true;
		}
	}
	else if(Struct == FRigVMFunction_MathVectorMakeAbsolute::StaticStruct())
	{
		if(InInfo->Target.Name.Equals(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_MathVectorMakeAbsolute, Global), ESearchCase::CaseSensitive))
		{
			const FRigVMFunction_MathVectorMakeAbsolute* MakeAbsolute =
				reinterpret_cast<const FRigVMFunction_MathVectorMakeAbsolute*>(InInstance->GetStructMemory());
			Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, FTransform(MakeAbsolute->Parent), false);
			Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(MakeAbsolute->Local), false);

			if(!InInfo->bInitialized)
			{
				Hierarchy->SetLocalTransform(InInfo->ControlKey, FTransform(MakeAbsolute->Local), true);
			}
			return true;
		}
	}

	TTuple<const FStructProperty*, uint8*> StructPropertyAndTargetMemory =
		FindStructPropertyAndTargetMemory(InInstance, Struct, InInfo->Target.Name);

	const FStructProperty* StructProperty = StructPropertyAndTargetMemory.Get<0>();
	const uint8* Memory = StructPropertyAndTargetMemory.Get<1>();
	if(StructProperty == nullptr || Memory == nullptr)
	{
		return false;
	}

	FTransform Transform = Hierarchy->GetGlobalTransform(InInfo->ControlKey, false);

	if(StructProperty->Struct == TBaseStructure<FTransform>::Get())
	{
		const FTransform& Result = *(const FTransform*)Memory;
		Transform = Result;
	}
	else if(StructProperty->Struct == TBaseStructure<FEulerTransform>::Get())
	{
		const FEulerTransform& Result = *(const FEulerTransform*)Memory;
		Transform = Result.ToFTransform();
	}
	else if(StructProperty->Struct == TBaseStructure<FQuat>::Get())
	{
		const FQuat& Result = *(const FQuat*)Memory;
		Transform.SetRotation(Result);
	}
	else if(StructProperty->Struct == TBaseStructure<FVector>::Get())
	{
		const FVector& Result = *(const FVector*)Memory;
		Transform.SetTranslation(Result);
	}
	else
	{
		return false;
	}

	Hierarchy->SetControlOffsetTransform(InInfo->ControlKey, FTransform::Identity, false);
	Hierarchy->SetGlobalTransform(InInfo->ControlKey, Transform, false);

	if(!InInfo->bInitialized)
	{
		Hierarchy->SetGlobalTransform(InInfo->ControlKey, Transform, true);
	}
	return true;
}

bool FRigUnit::UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo)
{
	const UScriptStruct* Struct = InNode->GetScriptStruct();
	check(Struct);
	check(InInstance.IsValid() && InInstance->IsValid());
	check(InInstance->GetStruct() == Struct);

	if(!InInfo.IsValid())
	{
		return false;
	}

	URigHierarchy* Hierarchy = InContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return false;
	}

	const FTransform Transform = Hierarchy->GetLocalTransform(InInfo->ControlKey, false);

	// deal with special cases here
	if(Struct == FRigVMFunction_MathTransformMakeAbsolute::StaticStruct())
	{
		if(InInfo->Target.Name.Equals(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_MathTransformMakeAbsolute, Local), ESearchCase::CaseSensitive))
		{
			FRigVMFunction_MathTransformMakeAbsolute* MakeAbsolute =
				reinterpret_cast<FRigVMFunction_MathTransformMakeAbsolute*>(InInstance->GetStructMemory());
			MakeAbsolute->Local = Transform;
			return true;
		}
	}
	if(Struct == FRigVMFunction_MathVectorMakeAbsolute::StaticStruct())
	{
		if(InInfo->Target.Name.Equals(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_MathVectorMakeAbsolute, Local), ESearchCase::CaseSensitive))
		{
			FRigVMFunction_MathVectorMakeAbsolute* MakeAbsolute =
				reinterpret_cast<FRigVMFunction_MathVectorMakeAbsolute*>(InInstance->GetStructMemory());
			MakeAbsolute->Local = Transform.GetTranslation();
			return true;
		}
	}
	
	TTuple<const FStructProperty*, uint8*> StructPropertyAndTargetMemory =
		FindStructPropertyAndTargetMemory(InInstance, Struct, InInfo->Target.Name);

	const FStructProperty* StructProperty = StructPropertyAndTargetMemory.Get<0>();
	uint8* Memory = StructPropertyAndTargetMemory.Get<1>();
	if(StructProperty == nullptr || Memory == nullptr)
	{
		return false;
	}

	if(StructProperty->Struct == TBaseStructure<FTransform>::Get())
	{
		FTransform& Result = *(FTransform*)Memory;
		Result = Transform;
	}
	else if(StructProperty->Struct == TBaseStructure<FEulerTransform>::Get())
	{
		FEulerTransform& Result = *(FEulerTransform*)Memory;
		Result = FEulerTransform(Transform);
	}
	else if(StructProperty->Struct == TBaseStructure<FQuat>::Get())
	{
		FQuat& Result = *(FQuat*)Memory;
		Result = Transform.GetRotation();
	}
	else if(StructProperty->Struct == TBaseStructure<FVector>::Get())
	{
		FVector& Result = *(FVector*)Memory;
		Result = Transform.GetTranslation();
	}
	else
	{
		return false;
	}
	
	return true;
}

TArray<const URigVMPin*> FRigUnit::GetPinsForDirectManipulation(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const
{
	// the default is to return the pin matching the target
	TArray<const URigVMPin*> Pins;
	if(const URigVMPin* Pin = InNode->FindPin(InTarget.Name))
	{
		Pins.Add(Pin);
	}
	return Pins;
}

void FRigUnit::PerformDebugDrawingForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) const
{
	FRigVMDrawInterface* DrawInterface = InContext.GetDrawInterface();
	if(DrawInterface == nullptr)
	{
		return;
	}
	
	if(FRigControlElement* Control = InContext.Hierarchy->Find<FRigControlElement>(InInfo->ControlKey))
	{
		const FTransform Transform = InContext.Hierarchy->GetLocalTransform(InInfo->ControlKey, false);
		const FVector Location = Transform.GetTranslation();
		if(!Location.IsNearlyZero())
		{
			const FTransform OffsetTransform = InContext.Hierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentGlobal);
			DrawInterface->DrawLine(OffsetTransform, FVector::ZeroVector, Location, FLinearColor::Green);
		}
	}	
}

#endif
