// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/RigUnitContext.h"
#include "ControlRig.h"
#include "ModularRig.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnitContext)

bool FControlRigExecuteContext::IsRunningAConstructionEvent() const
{
	return GetEventName() == FRigUnit_PrepareForExecution::EventName ||
		GetEventName() == FRigUnit_PostPrepareForExecution::EventName;
}

const FString& FControlRigExecuteContext::GetElementModulePrefix(ERigMetaDataNameSpace InNameSpaceType) const
{
	if(IsRigModule())
	{
		// prefix the meta data name with the namespace to allow modules to store their
		// metadata in a way that doesn't collide with other modules' metadata.
		switch(InNameSpaceType)
		{
			case ERigMetaDataNameSpace::Self:
			{
				return GetRigModulePrefix();
			}
			case ERigMetaDataNameSpace::Parent:
			{
				return GetRigParentModulePrefix();
			}
			case ERigMetaDataNameSpace::Root:
			{
				return GetRigRootModulePrefix();
			}
			default:
			{
				break;
			}
		}
	}
	else
	{
		// prefix the meta data with some mockup namespaces
		// so we can test this even without a module present.
		switch(InNameSpaceType)
		{
			case ERigMetaDataNameSpace::Self:
			{
				// if we are storing on self and this is not a modular
				// rig let's just not use a namespace.
				break;
			}
			case ERigMetaDataNameSpace::Parent:
			{
				static const FString ParentNameSpace = TEXT("Parent/");
				return ParentNameSpace;
			}
			case ERigMetaDataNameSpace::Root:
			{
				static const FString RootNameSpace = TEXT("Root/");
				return RootNameSpace;
			}
			default:
			{
				break;
			}
		}
	}

	static const FString EmptyPrefix;
	return EmptyPrefix;
}

const FRigModuleInstance* FControlRigExecuteContext::GetRigModuleInstance(ERigMetaDataNameSpace InNameSpaceType) const
{
	if(RigModuleInstance)
	{
		switch(InNameSpaceType)
		{
			case ERigMetaDataNameSpace::Self:
			{
				return RigModuleInstance;
			}
			case ERigMetaDataNameSpace::Parent:
			{
				return RigModuleInstance->GetParentModule();
			}
			case ERigMetaDataNameSpace::Root:
			{
				return RigModuleInstance->GetRootModule();
			}
			case ERigMetaDataNameSpace::None:
			default:
			{
				break;
			}
		}
		
	}
	return nullptr;
}

FName FControlRigExecuteContext::AdaptMetadataName(ERigMetaDataNameSpace InNameSpaceType, const FName& InMetadataName) const
{
	// only if we are within a rig module let's adapt the meta data name
	const bool bUseNameSpace = InNameSpaceType != ERigMetaDataNameSpace::None;
	if(bUseNameSpace && !InMetadataName.IsNone())
	{
		// if the metadata name already contains a namespace - we are just going
		// to use it as is. this means that modules have access to other module's metadata,
		// and that's ok. the user will require the full path to it anyway so it is a
		// conscious user decision.
		const FString MetadataNameString = InMetadataName.ToString();
		int32 Index = INDEX_NONE;
		if(MetadataNameString.FindChar(FRigHierarchyModulePath::ModuleNameSuffixChar, Index))
		{
			return InMetadataName;
		}

		if(IsRigModule())
		{
			// prefix the meta data name with the namespace to allow modules to store their
			// metadata in a way that doesn't collide with other modules' metadata.
			switch(InNameSpaceType)
			{
				case ERigMetaDataNameSpace::Self:
				case ERigMetaDataNameSpace::Parent:
				case ERigMetaDataNameSpace::Root:
				{
					return *(GetElementModulePrefix(InNameSpaceType) + MetadataNameString);
				}
				default:
				{
					break;
				}
			}
		}
		else
		{
			// prefix the meta data with some mockup namespaces
			// so we can test this even without a module present.
			switch(InNameSpaceType)
			{
				case ERigMetaDataNameSpace::Self:
				{
					// if we are storing on self and this is not a modular
					// rig let's just not use a namespace.
					break;
				}
				case ERigMetaDataNameSpace::Parent:
				case ERigMetaDataNameSpace::Root:
				{
					return *(GetElementModulePrefix(InNameSpaceType) + MetadataNameString);
				}
				default:
				{
					break;
				}
			}
		}
	}
	return InMetadataName;
}

FControlRigExecuteContextRigModuleGuard::FControlRigExecuteContextRigModuleGuard(FControlRigExecuteContext& InContext, const UControlRig* InControlRig)
	: Context(InContext)
	, PreviousRigModulePrefix(InContext.RigModulePrefix)
	, PreviousRigParentModulePrefix(InContext.RigParentModulePrefix)
	, PreviousRigRootModulePrefix(InContext.RigRootModulePrefix)
	, PreviousRigModulePrefixHash(InContext.RigModulePrefixHash)
{
	Context.RigModulePrefix = InControlRig->GetRigModulePrefix();
	Context.RigParentModulePrefix = Context.RigModulePrefix;
	Context.RigRootModulePrefix = Context.RigModulePrefix;
	if(UModularRig* ModularRig = Cast<UModularRig>(InControlRig->GetParentRig()))
	{
		if(const FRigModuleInstance* Module = ModularRig->FindModule(InControlRig->GetFName()))
		{
			if(const FRigModuleInstance* ParentModule = Module->GetParentModule())
			{
				Context.RigParentModulePrefix = ParentModule->GetModulePrefix();
			}
			if(const FRigModuleInstance* RootModule = Module->GetRootModule())
			{
				Context.RigRootModulePrefix = RootModule->GetModulePrefix();
			}
		}
	}
	Context.RigModulePrefixHash = GetTypeHash(Context.RigModulePrefix);
}

FControlRigExecuteContextRigModuleGuard::FControlRigExecuteContextRigModuleGuard(FControlRigExecuteContext& InContext, const FString& InNewModulePrefix, const FString& InNewParentModulePrefix, const FString& InNewRootModulePrefix)
	: Context(InContext)
	, PreviousRigModulePrefix(InContext.RigModulePrefix)
	, PreviousRigParentModulePrefix(InContext.RigParentModulePrefix)
	, PreviousRigRootModulePrefix(InContext.RigRootModulePrefix)
	, PreviousRigModulePrefixHash(InContext.RigModulePrefixHash)
{
	Context.RigModulePrefix = InNewModulePrefix;
	Context.RigParentModulePrefix = InNewParentModulePrefix;
	Context.RigRootModulePrefix = InNewRootModulePrefix;
	Context.RigModulePrefixHash = GetTypeHash(Context.RigModulePrefix);
}

FControlRigExecuteContextRigModuleGuard::~FControlRigExecuteContextRigModuleGuard()
{
	Context.RigModulePrefix = PreviousRigModulePrefix;
	Context.RigParentModulePrefix = PreviousRigParentModulePrefix;
	Context.RigRootModulePrefix = PreviousRigRootModulePrefix;
	Context.RigModulePrefixHash = PreviousRigModulePrefixHash; 
}
