// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Simulation/RigUnit_PointSimulation.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_PointSimulation)

FRigUnit_PointSimulation_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
}

FRigVMStructUpgradeInfo FRigUnit_PointSimulation::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

