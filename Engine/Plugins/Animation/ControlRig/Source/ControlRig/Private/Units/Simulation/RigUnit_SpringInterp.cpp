// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/Simulation/RigUnit_SpringInterp.h"
#include "ControlRigDefines.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_SpringInterp)

namespace RigUnitSpringInterpConstants
{
	static const float FixedTimeStep = 1.0f / 60.0f;
	static const float MaxTimeStep = 0.1f;
	static const float Mass = 1.0f;
}

FRigUnit_SpringInterp_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
 
	// Clamp to avoid large time deltas.
	float RemainingTime = FMath::Min(ExecuteContext.GetDeltaTime(), RigUnitSpringInterpConstants::MaxTimeStep);

	Result = Current;
	while (RemainingTime >= RigUnitSpringInterpConstants::FixedTimeStep)
	{
		Result = UKismetMathLibrary::FloatSpringInterp(Result, Target, SpringState, Stiffness, CriticalDamping, RigUnitSpringInterpConstants::FixedTimeStep, Mass);
		RemainingTime -= RigUnitSpringInterpConstants::FixedTimeStep;
	}

	Result = UKismetMathLibrary::FloatSpringInterp(Result, Target, SpringState, Stiffness, CriticalDamping, RemainingTime, Mass);
}

FRigVMStructUpgradeInfo FRigUnit_SpringInterp::GetUpgradeInfo() const
{
	// this node is no longer supported and the new implementation is vastly different
	return FRigVMStructUpgradeInfo();
}
 
FRigUnit_SpringInterpVector_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
 
	// Clamp to avoid large time deltas.
	float RemainingTime = FMath::Min(ExecuteContext.GetDeltaTime(), RigUnitSpringInterpConstants::MaxTimeStep);

	Result = Current;
	while (RemainingTime >= RigUnitSpringInterpConstants::FixedTimeStep)
	{
		Result = UKismetMathLibrary::VectorSpringInterp(Result, Target, SpringState, Stiffness, CriticalDamping, RigUnitSpringInterpConstants::FixedTimeStep, Mass);
		RemainingTime -= RigUnitSpringInterpConstants::FixedTimeStep;
	}

	Result = UKismetMathLibrary::VectorSpringInterp(Result, Target, SpringState, Stiffness, CriticalDamping, RemainingTime, Mass);
}

FRigVMStructUpgradeInfo FRigUnit_SpringInterpVector::GetUpgradeInfo() const
{
	// this node is no longer supported and the new implementation is vastly different
	return FRigVMStructUpgradeInfo();
}

FRigUnit_SpringInterpV2_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// Treat the input as a frequency in Hz
	float AngularFrequency = Strength * 2.0f * PI;
	float Stiffness = AngularFrequency * AngularFrequency;
	float AdjustedTarget = Target;
	if (!FMath::IsNearlyZero(Stiffness))
	{
		AdjustedTarget += Force / (Stiffness * RigUnitSpringInterpConstants::Mass);
	}
	else
	{
		SpringState.Velocity += Force * (ExecuteContext.GetDeltaTime() / RigUnitSpringInterpConstants::Mass);
	}
	SimulatedResult = UKismetMathLibrary::FloatSpringInterp(
		bUseCurrentInput ? Current : SimulatedResult, AdjustedTarget, SpringState, Stiffness, CriticalDamping,
		ExecuteContext.GetDeltaTime(), RigUnitSpringInterpConstants::Mass, TargetVelocityAmount, 
		false, 0.0f, 0.0f, !bUseCurrentInput || bInitializeFromTarget);

	Result = SimulatedResult;
	Velocity = SpringState.Velocity;
}

FRigUnit_SpringInterpVectorV2_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// Treat the input as a frequency in Hz
	float AngularFrequency = Strength * 2.0f * PI;
	float Stiffness = AngularFrequency * AngularFrequency;
	FVector AdjustedTarget = Target;
	if (!FMath::IsNearlyZero(Stiffness))
	{
		AdjustedTarget += Force / (Stiffness * RigUnitSpringInterpConstants::Mass);
	}
	else
	{
		SpringState.Velocity += Force * (ExecuteContext.GetDeltaTime() / RigUnitSpringInterpConstants::Mass);
	}
	SimulatedResult = UKismetMathLibrary::VectorSpringInterp(
		bUseCurrentInput ? Current : SimulatedResult, AdjustedTarget, SpringState, Stiffness, CriticalDamping,
		ExecuteContext.GetDeltaTime(), RigUnitSpringInterpConstants::Mass, TargetVelocityAmount,
		false, FVector(), FVector(), !bUseCurrentInput || bInitializeFromTarget);
	Result = SimulatedResult;
}

FRigUnit_SpringInterpQuaternionV2_Execute()
{
    DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// Treat the input as a frequency in Hz
	float AngularFrequency = Strength * 2.0f * PI;
	float Stiffness = AngularFrequency * AngularFrequency;
	SpringState.AngularVelocity += Torque * (ExecuteContext.GetDeltaTime() / RigUnitSpringInterpConstants::Mass);
	SimulatedResult = UKismetMathLibrary::QuaternionSpringInterp(
		bUseCurrentInput ? Current : SimulatedResult, Target, SpringState, Stiffness, CriticalDamping,
		ExecuteContext.GetDeltaTime(), RigUnitSpringInterpConstants::Mass, TargetVelocityAmount, 
		!bUseCurrentInput || bInitializeFromTarget);
	Result = SimulatedResult;
	AngularVelocity = SpringState.AngularVelocity;
}


