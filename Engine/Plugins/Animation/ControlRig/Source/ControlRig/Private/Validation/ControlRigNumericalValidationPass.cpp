// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigNumericalValidationPass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigNumericalValidationPass)

////////////////////////////////////////////////////////////////////////////////
// UControlRigNumericalValidationPass
////////////////////////////////////////////////////////////////////////////////

UControlRigNumericalValidationPass::UControlRigNumericalValidationPass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bCheckControls(true)
	, bCheckBones(true)
	, bCheckCurves(true)
	, TranslationPrecision(0.01f)
	, RotationPrecision(0.01f)
	, ScalePrecision(0.001f)
	, CurvePrecision(0.01f)
{
}

void UControlRigNumericalValidationPass::OnSubjectChanged(UControlRig* InControlRig, FControlRigValidationContext* InContext)
{
	Pose.Reset();

	TArray<FName> EventNames = InControlRig->GetEventQueue();
	if (EventNames.Num() == 2)
	{
		EventNameA = EventNames[0];
		EventNameB = EventNames[1];
	}
	else
	{
		EventNameA = EventNameB = NAME_None;
	}
}

void UControlRigNumericalValidationPass::OnInitialize(UControlRig* InControlRig, FControlRigValidationContext* InContext)
{
	OnSubjectChanged(InControlRig, InContext);
}

void UControlRigNumericalValidationPass::OnEvent(UControlRig* InControlRig, const FName& InEventName, FControlRigValidationContext* InContext)
{
	if (InEventName == EventNameA)
	{
		Pose = InControlRig->GetHierarchy()->GetPose();
	}
	else if (InEventName == EventNameB)
	{
		for(FRigPoseElement& PoseElement : Pose)
		{
			if (!PoseElement.Index.UpdateCache(InControlRig->GetHierarchy()))
			{
				continue;
			}

			FRigElementKey Key = PoseElement.Index.GetKey();
			int32 ElementIndex = PoseElement.Index.GetIndex();

			if(Key.Type == ERigElementType::Control)
			{
				if (!bCheckControls)
				{
					continue;
				}
			}
			else if(Key.Type == ERigElementType::Bone)
			{
				if (!bCheckBones)
				{
					continue;
				}
			}
			else if(Key.Type != ERigElementType::Curve)
			{
				continue;
			}
			
			if(Key.Type == ERigElementType::Curve)
			{
				float A = PoseElement.CurveValue;
				float B = InControlRig->GetHierarchy()->GetCurveValue(ElementIndex);

				if (FMath::Abs(A - B) > CurvePrecision + SMALL_NUMBER)
				{
					FString EventNameADisplayString = InContext->GetDisplayNameForEvent(EventNameA);
					FString EventNameBDisplayString = InContext->GetDisplayNameForEvent(EventNameB);
					FString Message = FString::Printf(TEXT("Values don't match between %s and %s."), *EventNameADisplayString, *EventNameBDisplayString);
					InContext->Report(EMessageSeverity::Warning, Key, Message);
				}
				continue;
			}

			FTransform A = PoseElement.GlobalTransform;
			FTransform B = InControlRig->GetHierarchy()->GetGlobalTransform(PoseElement.Index);

			FQuat RotA = A.GetRotation().GetNormalized();
			FQuat RotB = B.GetRotation().GetNormalized();

			bool bPosesMatch = true;
			if (FMath::Abs(A.GetLocation().X - B.GetLocation().X) > TranslationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}
			if (FMath::Abs(A.GetLocation().Y - B.GetLocation().Y) > TranslationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}
			if (FMath::Abs(A.GetLocation().Z - B.GetLocation().Z) > TranslationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}
			if (FMath::Abs(RotA.GetAngle() - RotB.GetAngle()) > RotationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}
			if (FMath::Abs(RotA.GetAngle()) > SMALL_NUMBER || FMath::Abs(RotB.GetAngle()) > SMALL_NUMBER)
			{
				if (FMath::Abs(RotA.GetRotationAxis().X - RotB.GetRotationAxis().X) > RotationPrecision + SMALL_NUMBER)
				{
					bPosesMatch = false;
				}
				if (FMath::Abs(RotA.GetRotationAxis().Y - RotB.GetRotationAxis().Y) > RotationPrecision + SMALL_NUMBER)
				{
					bPosesMatch = false;
				}
				if (FMath::Abs(RotA.GetRotationAxis().Z - RotB.GetRotationAxis().Z) > RotationPrecision + SMALL_NUMBER)
				{
					bPosesMatch = false;
				}
			}
			if (FMath::Abs(A.GetScale3D().X - B.GetScale3D().X) > TranslationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}
			if (FMath::Abs(A.GetScale3D().Y - B.GetScale3D().Y) > TranslationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}
			if (FMath::Abs(A.GetScale3D().Z - B.GetScale3D().Z) > TranslationPrecision + SMALL_NUMBER)
			{
				bPosesMatch = false;
			}

			if (!bPosesMatch)
			{
				FString EventNameADisplayString = InContext->GetDisplayNameForEvent(EventNameA);
				FString EventNameBDisplayString = InContext->GetDisplayNameForEvent(EventNameB);
				FString Message = FString::Printf(TEXT("Poses don't match between %s and %s."), *EventNameADisplayString, *EventNameBDisplayString);
				InContext->Report(EMessageSeverity::Warning, Key, Message);

				if (InControlRig->GetHierarchy()->IsSelected(Key))
				{
					if (FRigVMDrawInterface* DrawInterface = InContext->GetDrawInterface())
					{
						DrawInterface->DrawAxes(FTransform::Identity, A, 50.f);
						DrawInterface->DrawAxes(FTransform::Identity, B, 50.f);
					}
				}
			}
		}
	}
	else
	{
		InContext->Report(EMessageSeverity::Info, TEXT("Numerical validation only works when running 'Backwards and Forwards'"));
	}
}

