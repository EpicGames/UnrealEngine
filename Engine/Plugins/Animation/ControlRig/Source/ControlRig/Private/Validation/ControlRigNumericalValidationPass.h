// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ControlRigValidationPass.h"

#include "ControlRigNumericalValidationPass.generated.h"

#define UE_API CONTROLRIG_API


/** Used to perform a numerical comparison of the poses */
UCLASS(MinimalAPI, DisplayName="Numerical Comparison")
class UControlRigNumericalValidationPass : public UControlRigValidationPass
{
	GENERATED_UCLASS_BODY()

public:

	// UControlRigValidationPass interface
	UE_API virtual void OnSubjectChanged(UControlRig* InControlRig, FControlRigValidationContext* InContext) override;
	UE_API virtual void OnInitialize(UControlRig* InControlRig, FControlRigValidationContext* InContext) override;
	UE_API virtual void OnEvent(UControlRig* InControlRig, const FName& InEventName, FControlRigValidationContext* InContext) override;

	// If set to true the pass will validate the poses of all bones
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bCheckControls;

	// If set to true the pass will validate the poses of all bones
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bCheckBones;

	// If set to true the pass will validate the values of all curves
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bCheckCurves;

	// The threshold under which we'll ignore a precision issue in the pass
	UPROPERTY(EditAnywhere, Category = "Settings")
	float TranslationPrecision;

	// The threshold under which we'll ignore a precision issue in the pass (in degrees)
	UPROPERTY(EditAnywhere, Category = "Settings")
	float RotationPrecision;

	// The threshold under which we'll ignore a precision issue in the pass
	UPROPERTY(EditAnywhere, Category = "Settings")
	float ScalePrecision;

	// The threshold under which we'll ignore a precision issue in the pass
	UPROPERTY(EditAnywhere, Category = "Settings")
	float CurvePrecision;

private:

	UPROPERTY(transient)
	FName EventNameA;
	
	UPROPERTY(transient)
	FName EventNameB;
	
	UPROPERTY(transient)
	FRigPose Pose;
};

#undef UE_API
