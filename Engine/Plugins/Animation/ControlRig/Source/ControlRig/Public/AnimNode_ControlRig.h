// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRig.h"
#include "Animation/InputScaleBias.h"
#include "AnimNode_ControlRigBase.h"
#include "ControlRigAssetReference.h"
#include "Animation/AnimBulkCurves.h"
#include "Tools/ControlRigVariableMappings.h"
#include "AnimNode_ControlRig.generated.h"

#define UE_API CONTROLRIG_API

class UNodeMappingContainer;

/**
 * Animation node that allows animation ControlRig output to be used in an animation graph
 */
USTRUCT()
struct FAnimNode_ControlRig : public FAnimNode_ControlRigBase
{
	GENERATED_BODY()

public:

	UE_API FAnimNode_ControlRig();
	UE_API ~FAnimNode_ControlRig();

	virtual UControlRig* GetControlRig() const override { return ControlRig; }
	
	UE_DEPRECATED(5.8, "Use GetControlRigAssetReference instead")
	virtual TSubclassOf<UControlRig> GetControlRigClass() const override { return ControlRigAssetReference.GetBlueprintClass(); }
	virtual FControlRigAssetStrongReference GetControlRigAssetReference() const override { return ControlRigAssetReference; }

	// FAnimNode_Base interface
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext & Output) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }

	//void SetIOMapping(bool bInput, const FName& SourceProperty, const FName& TargetCurve);
	//FName GetIOMapping(bool bInput, const FName& SourceProperty) const;

	UE_API virtual void InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass) override;
	UE_API virtual void InitializeProperties(const UObject* InSourceInstance, FControlRigAssetStrongReference InTargetReference);
	UE_API virtual void PropagateInputProperties(const UObject* InSourceInstance) override;
	UE_API void SetControlRigClass(TSubclassOf<UControlRig> InControlRigClass);

private:
	UE_API void HandleOnInitialized_AnyThread(URigVMHost*, const FName&);
#if WITH_EDITOR
	UE_API virtual void HandleObjectsReinstanced_Impl(UObject* InSourceObject, UObject* InTargetObject, const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
#endif
private:

	// The class to use for the rig. 
	UPROPERTY(EditAnywhere, Category = ControlRig, meta = (DisallowCreateNew))
	FControlRigAssetStrongReference ControlRigAssetReference;

	// The default class to use for the rig. This is needed
	// only if the Control Rig Class is exposed as a pin.
	UPROPERTY()
	FControlRigAssetStrongReference DefaultControlRigAssetReference;
	
	// The class to use for the rig. 
	UPROPERTY(meta=(DeprecatedProperty))
	TSubclassOf<UControlRig> ControlRigClass_DEPRECATED;

	// The default class to use for the rig. This is needed
	// only if the Control Rig Class is exposed as a pin.
	UPROPERTY(meta=(DeprecatedProperty))
	TSubclassOf<UControlRig> DefaultControlRigClass_DEPRECATED;

	/** Cached ControlRig */
	UPROPERTY(transient)
	TObjectPtr<UControlRig> ControlRig;

	/** Cached ControlRigs per class */
	UPROPERTY(transient)
	TMap<FControlRigAssetStrongReference, TObjectPtr<UControlRig>> ControlRigPerClass;

	// alpha value handler
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault))
	float Alpha;

	UPROPERTY(EditAnywhere, Category = Settings)
	EAnimAlphaInputType AlphaInputType;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault, DisplayName = "bEnabled", DisplayAfter = "AlphaScaleBias"))
	uint8 bAlphaBoolEnabled : 1;

	// Override the initial transforms with those taken from the mesh component
	UPROPERTY(EditAnywhere, Category=Settings, meta = (DisplayName = "Set Initial Transforms From Mesh"))
	uint8 bSetRefPoseFromSkeleton : 1;

	UPROPERTY(EditAnywhere, Category=Settings)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "Blend Settings"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinShownByDefault))
	FName AlphaCurveName;

	UPROPERTY(EditAnywhere, Category = Settings)
	FInputScaleBiasClamp AlphaScaleBiasClamp;

	// we only save mapping, 
	// we have to query control rig when runtime 
	// to ensure type and everything is still valid or not
	UPROPERTY()
	TMap<FName, FName> InputMapping;

	UPROPERTY()
	TMap<FName, FName> OutputMapping;

	/*
	 * Max LOD that this node is allowed to run
	 * For example if you have LODThreshold to be 2, it will run until LOD 2 (based on 0 index)
	 * when the component LOD becomes 3, it will stop update/evaluate
	 * currently transition would be issue and that has to be re-visited
	 */
	UPROPERTY(EditAnywhere, Category = Performance, meta = (PinHiddenByDefault, DisplayName = "LOD Threshold"))
	int32 LODThreshold;

protected:
	UE_DEPRECATED(5.8, "Use GetTargetAssetReference instead")
	UE_API virtual UClass* GetTargetClass() const override;
	UE_API virtual FControlRigAssetStrongReference GetTargetAssetReference() const override;
	UE_API virtual void UpdateInput(UControlRig* InControlRig, FPoseContext& InOutput) override;
	UE_API virtual void UpdateOutput(UControlRig* InControlRig, FPoseContext& InOutput) override;
	
	UE_API bool UpdateControlRigIfNeeded(const UAnimInstance* InAnimInstance, const FBoneContainer& InRequiredBones);

	FControlRigVariableMappings ControlRigVariableMappings;

public:

	UE_API void PostSerialize(const FArchive& Ar);

	friend class UAnimGraphNode_ControlRig;
	friend class UAnimNodeControlRigLibrary;
};

template<>
struct TStructOpsTypeTraits<FAnimNode_ControlRig> : public TStructOpsTypeTraitsBase2<FAnimNode_ControlRig>
{
	enum
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
