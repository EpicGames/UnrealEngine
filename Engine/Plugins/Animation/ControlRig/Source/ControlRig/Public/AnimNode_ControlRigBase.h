// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigAssetReference.h"
#include "Animation/AnimBulkCurves.h"
#include "Animation/AnimNode_CustomProperty.h"
#include "Rigs/RigHierarchyPoseAdapter.h"
#include "Rigs/RigHierarchy.h"
#include "Tools/ControlRigHierarchyMappings.h"
#include "Tools/ControlRigIOSettings.h"
#include "AnimNode_ControlRigBase.generated.h"

#define UE_API CONTROLRIG_API

class UNodeMappingContainer;
class UControlRig;
class FControlRigPoseAdapter;

#if ENABLE_ANIM_DEBUG
extern CONTROLRIG_API TAutoConsoleVariable<int32> CVarAnimNodeControlRigDebug;
#endif

USTRUCT()
struct FControlRigAnimNodeEventName
{
	GENERATED_BODY()

	FControlRigAnimNodeEventName()
	: EventName(NAME_None)
	{}

	UPROPERTY(EditAnywhere, Category = Links)
	FName EventName;	
};


/**
 * Animation node that allows animation ControlRig output to be used in an animation graph
 */
USTRUCT()
struct FAnimNode_ControlRigBase : public FAnimNode_CustomProperty
{
	GENERATED_BODY()

	UE_API FAnimNode_ControlRigBase();

	/* return Control Rig of current object */
	virtual UControlRig* GetControlRig() const PURE_VIRTUAL(FAnimNode_ControlRigBase::GetControlRig, return nullptr; );
	
	UE_DEPRECATED(5.8, "Use GetControlRigAssetReference instead")
	virtual TSubclassOf<UControlRig> GetControlRigClass() const PURE_VIRTUAL(FAnimNode_ControlRigBase::GetControlRigClass, return nullptr; );
	virtual FControlRigAssetStrongReference GetControlRigAssetReference() const PURE_VIRTUAL(FAnimNode_ControlRigBase::GetControlRigAssetReference, return FControlRigAssetStrongReference(); );
	
	// FAnimNode_Base interface
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;

protected:

	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

	/**
	 * If this is checked the rig's pose needs to be reset to its initial
	 * prior to evaluating the rig.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	bool bResetInputPoseToInitial;

	/**
	 * If this is checked the bone pose coming from the AnimBP will be
	 * transferred into the Control Rig.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	bool bTransferInputPose;

	/**
	 * If this is checked the curves coming from the AnimBP will be
	 * transferred into the Control Rig.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	bool bTransferInputCurves;

	/**
	 * Transferring the pose in global space guarantees a global pose match,
	 * while transferring in local space ensures a match of the local transforms.
	 * In general transforms only differ if the hierarchy topology differs
	 * between the Control Rig and the skeleton used in the AnimBP.
	 * Note: Turning this off can potentially improve performance.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	bool bTransferPoseInGlobalSpace;

	/**
	 * An inclusive list of bones to transfer as part
	 * of the input pose transfer phase.
	 * If this list is empty all bones will be transferred.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	TArray<FBoneReference> InputBonesToTransfer;

	/**
	 * An inclusive list of bones to transfer as part
	 * of the output pose transfer phase.
	 * If this list is empty all bones will be transferred.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	TArray<FBoneReference> OutputBonesToTransfer;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	/** Node Mapping Container */
	UPROPERTY(transient)
	TWeakObjectPtr<UNodeMappingContainer> NodeMappingContainer;

	UPROPERTY(transient)
	FControlRigIOSettings InputSettings;

	UPROPERTY(transient)
	FControlRigIOSettings OutputSettings;

	UPROPERTY(transient)
	bool bExecute;

	// The below is alpha value support for control rig
	float InternalBlendAlpha;

	// The customized event queue to run
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Settings)
	TArray<FControlRigAnimNodeEventName> EventQueue;

	bool bClearEventQueueRequired = false;

	UE_API virtual bool CanExecute();

	// update input/output to control rig
	UE_API virtual void UpdateInput(UControlRig* ControlRig, FPoseContext& InOutput);
	UE_API virtual void UpdateOutput(UControlRig* ControlRig, FPoseContext& InOutput);
	
	UE_API virtual UClass* GetTargetClass() const override;
	UE_API virtual FControlRigAssetStrongReference GetTargetAssetReference() const;
	
	// execute control rig on the input pose and outputs the result
	UE_API void ExecuteControlRig(FPoseContext& InOutput);
	UE_API void ExecuteConstructionIfNeeded();

	UE_API void QueueControlRigDrawInstructions(UControlRig* ControlRig, FAnimInstanceProxy* Proxy) const;

	TArray<TObjectPtr<UAssetUserData>> GetAssetUserData() const { return AssetUserData; }
	UE_API void UpdateGetAssetUserDataDelegate(UControlRig* InControlRig) const;

	bool bControlRigRequiresInitialization;
	uint16 LastBonesSerialNumberForCacheBones;

	FControlRigHierarchyMappings ControlRigHierarchyMappings;

	TWeakObjectPtr<const UAnimInstance> WeakAnimInstanceObject;

	friend struct FControlRigSequencerAnimInstanceProxy;
	friend struct FControlRigLayerInstanceProxy;
};
template<>
struct TStructOpsTypeTraits<FAnimNode_ControlRigBase> : public TStructOpsTypeTraitsBase2<FAnimNode_ControlRigBase>
{
	enum
	{
		WithPureVirtual = true,
	};
};

#undef UE_API
