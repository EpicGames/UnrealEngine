// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_ControlRigBase.h"
#include "AnimNode_ControlRig_ExternalSource.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Animation node that allows animation ControlRig output to be used in an animation graph
 */
USTRUCT()
struct FAnimNode_ControlRig_ExternalSource : public FAnimNode_ControlRigBase
{
	GENERATED_BODY()

	UE_API FAnimNode_ControlRig_ExternalSource();

	UE_API void SetControlRig(UControlRig* InControlRig);
	UE_API virtual UControlRig* GetControlRig() const override;
	UE_DEPRECATED(5.8, "Use GetControlRigAssetReference instead")
	UE_API virtual TSubclassOf<UControlRig> GetControlRigClass() const override;
	UE_API virtual FControlRigAssetStrongReference GetControlRigAssetReference() const override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;

private:
	UPROPERTY(transient)
	TWeakObjectPtr<UControlRig> ControlRig;
};

#undef UE_API
