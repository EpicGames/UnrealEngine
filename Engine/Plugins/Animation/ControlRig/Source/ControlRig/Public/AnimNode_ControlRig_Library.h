// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Animation/AnimNodeReference.h"
#include "AnimNode_ControlRig.h"
#include "AnimNode_ControlRig_Library.generated.h"

USTRUCT(BlueprintType)
struct FControlRigReference : public FAnimNodeReference
{
	GENERATED_BODY()

	typedef FAnimNode_ControlRig FInternalNodeType;
};

// Exposes operations to be performed on a control rig anim node
UCLASS(Experimental, MinimalAPI)
class UAnimNodeControlRigLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get a control rig context from an anim node context */
	UFUNCTION(BlueprintCallable, Category = "Animation|ControlRig", meta = (BlueprintThreadSafe, ExpandEnumAsExecs = "Result"))
	static CONTROLRIG_API FControlRigReference ConvertToControlRig(const FAnimNodeReference& Node, EAnimNodeReferenceConversionResult& Result);

	/** Get a control rig context from an anim node context (pure) */
	UFUNCTION(BlueprintPure, Category = "Animation|ControlRig", meta = (BlueprintThreadSafe, DisplayName = "Convert to Sequence Player"))
	static void ConvertToControlRigPure(const FAnimNodeReference& Node, FControlRigReference& ControlRig, bool& Result)
	{
		EAnimNodeReferenceConversionResult ConversionResult;
		ControlRig = ConvertToControlRig(Node, ConversionResult);
		Result = (ConversionResult == EAnimNodeReferenceConversionResult::Succeeded);
	}

	/** Set the control rig class on the node */
	UFUNCTION(BlueprintCallable, Category = "Animation|ControlRig", meta = (BlueprintThreadSafe))
	static CONTROLRIG_API FControlRigReference SetControlRigClass(const FControlRigReference& Node, TSubclassOf<UControlRig> ControlRigClass);
};
