// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transform/TransformableHandle.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Transform/AnimationEvaluation.h"

#include "ControlRigTransformableHandle.generated.h"

#define UE_API CONTROLRIG_API

struct FRigBaseElement;
struct FRigControlElement;

class UControlRig;
class USkeletalMeshComponent;
class UControlRigComponent;
class URigHierarchy;

/**
 * FControlEvaluationGraphBinding
 */
struct FControlEvaluationGraphBinding
{
	UE_API void HandleControlModified(
		UControlRig* InControlRig,
		FRigControlElement* InControl,
		const FRigControlModifiedContext& InContext);
	bool bPendingFlush = false;
};

/**
 * UTransformableControlHandle
 */

UCLASS(MinimalAPI, Blueprintable)
class UTransformableControlHandle : public UTransformableHandle 
{
	GENERATED_BODY()
	
public:
	
	UE_API virtual ~UTransformableControlHandle();

	UE_API virtual void PostLoad() override;
	
	/** Sanity check to ensure that ControlRig and ControlName are safe to use. */
	UE_API virtual bool IsValid(const bool bDeepCheck = true) const override;

	/** Sets the global transform of the control. */
	UE_API virtual void SetGlobalTransform(const FTransform& InGlobal) const override;
	/** Sets the local transform of the control. */
	UE_API virtual void SetLocalTransform(const FTransform& InLocal) const override;
	/** Gets the global transform of the control. */
	UE_API virtual FTransform GetGlobalTransform() const  override;
	/** Sets the local transform of the control. */
	UE_API virtual FTransform GetLocalTransform() const  override;

	/** Returns the target object containing the tick function (e.i. SkeletalComponent bound to ControlRig). */
	UE_API virtual UObject* GetPrerequisiteObject() const override;
	/** Returns ths SkeletalComponent tick function. */
	UE_API virtual FTickFunction* GetTickFunction() const override;

	/** Generates a hash value based on ControlRig and ControlName. */
	static UE_API uint32 ComputeHash(const UControlRig* InControlRig, const FName& InControlName);
	UE_API virtual uint32 GetHash() const override;
	
	/** Returns the underlying targeted object. */
	UE_API virtual TWeakObjectPtr<UObject> GetTarget() const override;

	/** Get the array of float channels for the specified section*/
	UE_API virtual TArrayView<FMovieSceneFloatChannel*>  GetFloatChannels(const UMovieSceneSection* InSection) const override;
	/** Get the array of double channels for the specified section*/
	UE_API virtual TArrayView<FMovieSceneDoubleChannel*>  GetDoubleChannels(const UMovieSceneSection* InSection) const override;
	UE_API virtual bool AddTransformKeys(const TArray<FFrameNumber>& InFrames,
		const TArray<FTransform>& InTransforms,
		const EMovieSceneTransformChannel& InChannels,
		const FFrameRate& InTickResolution,
		UMovieSceneSection* InSection,
		const bool bLocal = true) const override;

	/** Resolve the bound objects so that any object it references are resolved and correctly set up*/
	UE_API virtual void ResolveBoundObjects(FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, UObject* SubObject = nullptr) override;

	/** Make a duplicate of myself with this outer*/
	UE_API virtual UTransformableHandle* Duplicate(UObject* NewOuter) const override;

	/** Tick any skeletal mesh related to the bound component. */ 
	UE_API virtual void TickTarget() const override;

	/**
	 * Perform any pre-evaluation of the handle to ensure that the transform data is up to date.
	 * @param bTick to force any pre-evaluation ticking. The rig will still be pre-evaluated even
	 * if bTick is false (it just won't tick the bound skeletal meshes) Default is false.
	*/
	UE_API virtual void PreEvaluate(const bool bTick = false) const override;

	/** Registers/Unregisters useful delegates to track changes in the control's transform. */
	UE_API void UnregisterDelegates() const;
	UE_API void RegisterDelegates();

	/** Check for direct dependencies (ie hierarchy + skeletal mesh) with InOther. */
	UE_API virtual bool HasDirectDependencyWith(const UTransformableHandle& InOther) const override;

	/** Check for indirect dependencies with InOther. */
	UE_API virtual bool HasIndirectDependencyWith(const UTransformableHandle& InOther) const override;

	/** Look for a possible tick function that can be used as a prerequisite. */
	UE_API virtual FTickPrerequisite GetPrimaryPrerequisite(const bool bAllowThis = true) const override;

#if WITH_EDITOR
	/** Returns labels used for UI. */
	UE_API virtual FString GetLabel() const override;
	UE_API virtual FString GetFullLabel() const override;
#endif

	/** The ControlRig that this handle is pointing at. */
	UPROPERTY(BlueprintReadOnly, Category = "Object")
	TSoftObjectPtr<UControlRig> ControlRig;

	/** The ControlName of the control that this handle is pointing at. */
	UPROPERTY(BlueprintReadOnly, Category = "Object")
	FName ControlName;

	/** @todo document */
	UE_API void OnControlModified(
		UControlRig* InControlRig,
		FRigControlElement* InControl,
		const FRigControlModifiedContext& InContext);

	/** Updates the control rig to InControlRig if different and updates the delegates bindings accordingly. */
	UE_API void UpdateControlRig(UControlRig* InControlRig);
	
private:

	/** Returns the component bounded to ControlRig. */
	USceneComponent* GetBoundComponent() const;
	USkeletalMeshComponent* GetSkeletalMesh() const;
	UControlRigComponent* GetControlRigComponent() const;
	
	/** Handles notifications coming from the ControlRig's hierarchy */
	void OnHierarchyModified(
		ERigHierarchyNotification InNotif,
		URigHierarchy* InHierarchy,
		const FRigNotificationSubject& InSubject);

	void OnControlRigBound(UControlRig* InControlRig);
	void OnObjectBoundToControlRig(UObject* InObject);
	
	static FControlEvaluationGraphBinding& GetEvaluationBinding();

#if WITH_EDITOR
	/** Tracks if the control rig that this handle wraps has been replaced. (see FCoreUObjectDelegates::FOnObjectsReplaced for more info). */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InOldToNewInstances);
#endif

	/** Returns the control that this handle wraps. */
	FRigControlElement* GetControlElement() const;

	/** Returns an post-evaluation task that can be added to the animation evaluator. */
	const UE::Anim::FAnimationEvaluationTask& GetEvaluationTask() const;
	mutable UE::Anim::FAnimationEvaluationTask EvaluationTask;
	mutable FTransform Cache = FTransform::Identity;
};

#undef UE_API
