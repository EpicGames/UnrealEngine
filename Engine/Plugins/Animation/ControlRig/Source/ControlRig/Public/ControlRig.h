// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/SparseDelegate.h"
#include "Engine/EngineBaseTypes.h"
#include "Templates/SubclassOf.h"
#include "ControlRigDefines.h"
#include "ControlRigGizmoLibrary.h"
#include "ControlRigOverride.h"
#include "ControlRigAssetReference.h"
#include "Rigs/RigHierarchy.h"
#include "Units/RigUnitContext.h"
#include "Animation/NodeMappingProviderInterface.h"
#include "Units/RigUnit.h"
#include "Units/Control/RigUnit_Control.h"
#include "RigVMCore/RigVM.h"
#include "RigVMHost.h"
#include "Components/SceneComponent.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/MeshDeformerInstance.h"
#include "Animation/MeshDeformerProducer.h"
#include "Rigs/RigModuleDefines.h"
#include "UObject/OverriddenPropertySet.h"

#if WITH_EDITOR
#include "RigVMModel/RigVMPin.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMTypeUtils.h"
#endif

#if WITH_EDITOR
#include "AnimPreviewInstance.h"
#endif 

#include "ControlRig.generated.h"

class UControlRigRuntimeAsset;
class UControlRig;
class IControlRigObjectBinding;
class UScriptStruct;
class USkeletalMesh;
class USkeletalMeshComponent;
class AActor;
class UTransformableControlHandle;
class UControlRigReplay;
class UControlRigOverrideAsset;

struct FReferenceSkeleton;
struct FRigUnit;
struct FRigControl;
struct FRigPhysicsSimulationBase;
struct FControlRigOverrideContainer;

CONTROLRIG_API DECLARE_LOG_CATEGORY_EXTERN(LogControlRig, Log, All);

DECLARE_MULTICAST_DELEGATE_OneParam(FControlRigOverrideAssetsChanged, UControlRig*);

/** Runs logic for mapping input data to transforms (the "Rig") */
UCLASS(MinimalAPI, Blueprintable, editinlinenew)
class UControlRig : public URigVMHost, public INodeMappingProviderInterface, public IRigHierarchyProvider, public IMeshDeformerProducer
{
	GENERATED_UCLASS_BODY()

	friend class UControlRigComponent;
	friend class SControlRigStackView;

public:

	/** Bindable event for external objects to contribute to / filter a control value */
	DECLARE_EVENT_ThreeParams(UControlRig, FFilterControlEvent, UControlRig*, FRigControlElement*, FRigControlValue&);

	/** Bindable event for external objects to be notified of Control changes */
	DECLARE_EVENT_ThreeParams(UControlRig, FControlModifiedEvent, UControlRig*, FRigControlElement*, const FRigControlModifiedContext&);

	/** Bindable event for external objects to be notified that a Control is Selected */
	DECLARE_EVENT_ThreeParams(UControlRig, FControlSelectedEvent, UControlRig*, FRigControlElement*, bool);

	/** Bindable event to manage undo / redo brackets in the client */
	DECLARE_EVENT_TwoParams(UControlRig, FControlUndoBracketEvent, UControlRig*, bool /* bOpen */);

	// To support Blueprints/scripting, we need a different delegate type (a 'Dynamic' delegate) which supports looser style UFunction binding (using names).
	DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_ThreeParams(FOnControlSelectedBP, UControlRig, OnControlSelected_BP, UControlRig*, Rig, const FRigControlElement&, Control, bool, bSelected);

	/** Bindable event to notify object binding change. */
	DECLARE_EVENT_OneParam(UControlRig, FControlRigBoundEvent, UControlRig*);

	static CONTROLRIG_API const FName OwnerComponent;

	UFUNCTION(BlueprintCallable, Category = ControlRig)
	static CONTROLRIG_API TArray<UControlRig*> FindControlRigs(UObject* Outer, TSubclassOf<UControlRig> OptionalClass);

public:
	
	CONTROLRIG_API virtual UWorld* GetWorld() const override;
	CONTROLRIG_API virtual void Serialize(FArchive& Ar) override;
	CONTROLRIG_API virtual void PostLoad() override;
	virtual UScriptStruct* GetPublicContextStruct() const override { return FControlRigExecuteContext::StaticStruct(); }
	
	CONTROLRIG_API FControlRigAssetStrongReference GetAssetReference() const;

	// Returns the settings of the module this instance belongs to
	CONTROLRIG_API const FRigModuleSettings& GetRigModuleSettings() const;

	// Returns true if the rig is defined as a rig module
	CONTROLRIG_API bool IsRigModule() const;

	// Returns true if this rig is an instance module. Rigs may be a module but not instance
	// when being interacted with the asset editor
	CONTROLRIG_API bool IsRigModuleInstance() const;

	// Returns true if this rig is a modular rig (of class UModularRig)
	CONTROLRIG_API bool IsModularRig() const;

	// Returns true if this is a standalone rig (of class UControlRig and not modular)
	CONTROLRIG_API bool IsStandaloneRig() const;

	// Returns true if this is a native rig (implemented in C++)
	CONTROLRIG_API bool IsNativeRig() const;

	// Returns the parent rig hosting this module instance
	CONTROLRIG_API UControlRig* GetParentRig() const;

	// Returns the module prefix of this module (for example Arm/ )
	CONTROLRIG_API const FString& GetRigModulePrefix() const;

	// Returns the redirector from key to key for this rig
	CONTROLRIG_API virtual FRigElementKeyRedirector& GetElementKeyRedirector();
	virtual FRigElementKeyRedirector GetElementKeyRedirector() const { return ElementKeyRedirector; }
	
	// Returns the redirector from key to key for this rig
	CONTROLRIG_API virtual void SetElementKeyRedirector(const FRigElementKeyRedirector InElementRedirector);

	/** Creates a transformable control handle for the specified control to be used by the constraints system. Should use the UObject from 
	ConstraintsScriptingLibrary::GetManager(UWorld* InWorld)*/
	UFUNCTION(BlueprintCallable, Category = "Control Rig | Constraints")
	CONTROLRIG_API UTransformableControlHandle* CreateTransformableControlHandle(const FName& ControlName) const;


#if WITH_EDITOR
	/** Get the category of this ControlRig (for display in menus) */
	CONTROLRIG_API virtual FText GetCategory() const;

	/** Get the tooltip text to display for this node (displayed in graphs and from context menus) */
	CONTROLRIG_API virtual FText GetToolTipText() const;
#endif

	/** Initialize things for the ControlRig */
	CONTROLRIG_API virtual void Initialize(bool bInitRigUnits = true) override;

	/** Initialize the VM */
	CONTROLRIG_API virtual bool InitializeVM(const FName& InEventName) override;

	virtual void InitializeVMs(bool bInitRigUnits = true) { Super::Initialize(bInitRigUnits); }
	virtual bool InitializeVMs(const FName& InEventName) { return Super::InitializeVM(InEventName); }

#if WITH_EDITOR
protected:
	bool bIsRunningInPIE;
#endif
	
public:

	/** Evaluates the ControlRig */
	CONTROLRIG_API virtual void Evaluate_AnyThread() override;

	/** Ticks animation of the skeletal mesh component bound to this control rig */
	CONTROLRIG_API bool EvaluateSkeletalMeshComponent(double InDeltaTime);

	/** Removes any stored additive control values */
	CONTROLRIG_API void ResetControlValues();

	/** Resets the stored pose coming from the anim sequence.
	 * This usually indicates a new pose should be stored. */
	CONTROLRIG_API void ClearPoseBeforeBackwardsSolve();

	/* For additive rigs, will set control values by inverting the pose found after the backwards solve */
	/* Returns the array of control elements that were modified*/
	CONTROLRIG_API TArray<FRigControlElement*> InvertInputPose(const TArray<FRigElementKey>& InElements = TArray<FRigElementKey>(), EControlRigSetKey InSetKey = EControlRigSetKey::Never);

	/** Get cached control pose after backwards solve*/
	const FRigPose& GetControlsAfterBackwardsSolve() const { return  ControlsAfterBackwardsSolve; }

	/** Setup bindings to a runtime object (or clear by passing in nullptr). */
	void SetObjectBinding(TSharedPtr<IControlRigObjectBinding> InObjectBinding)
	{
		ObjectBinding = InObjectBinding;
		OnControlRigBound.Broadcast(this);
	}

	TSharedPtr<IControlRigObjectBinding> GetObjectBinding() const
	{
		return ObjectBinding;
	}

	/** Find the actor the rig is bound to, if any */
	UFUNCTION(BlueprintPure, Category = "Control Rig")
	CONTROLRIG_API AActor* GetHostingActor() const;

	UFUNCTION(BlueprintPure, Category = "Control Rig")
	URigHierarchy* GetHierarchy()
	{
		return DynamicHierarchy;
	}
	
	virtual URigHierarchy* GetHierarchy() const override
	{
		return DynamicHierarchy;
	}

#if WITH_EDITOR

	// called after post reinstance when compilng blueprint by Sequencer
	CONTROLRIG_API void PostReinstanceCallback(const UControlRig* Old);

	// resets the recorded transform changes
	CONTROLRIG_API void ResetRecordedTransforms(const FName& InEventName);

#endif // WITH_EDITOR
	
	// BEGIN UObject interface
	CONTROLRIG_API virtual void BeginDestroy() override;
	// END UObject interface

	using FControlRigBeginDestroyEvent = FMeshDeformerBeginDestroyEvent;

	//~ Begin IMeshDeformerProducer interface
	virtual FMeshDeformerBeginDestroyEvent& OnBeginDestroy() override {return BeginDestroyEvent;};
	//~ End IMeshDeformerProducer interface

	CONTROLRIG_API int32 NumOverrideAssets() const;
	
	/**
	 * The active override asset is the last one linked to this control rig,
	 * which can be used to record changes.
	 * It needs to be the last once since they get applied in order, and if two
	 * override assets override the same property the last one is the one that will win.
	 */
	CONTROLRIG_API const UControlRigOverrideAsset* GetLastOverrideAsset() const;
	CONTROLRIG_API UControlRigOverrideAsset* GetLastOverrideAsset();
	CONTROLRIG_API const UControlRigOverrideAsset* GetOverrideAsset(int32 InIndex) const;
	CONTROLRIG_API UControlRigOverrideAsset* GetOverrideAsset(int32 InIndex);
	CONTROLRIG_API int32 LinkOverrideAsset(UControlRigOverrideAsset* InOverrideAsset);
	CONTROLRIG_API bool UnlinkOverrideAsset(UControlRigOverrideAsset* InContainer);
	CONTROLRIG_API bool UnlinkAllOverrideAssets();
	CONTROLRIG_API bool IsLinkedToOverrideAsset(const UControlRigOverrideAsset* InOverrideAsset) const;
	FControlRigOverrideAssetsChanged& OnOverrideAssetsChanged() { return OverrideAssetsChangedDelegate; }
	void SetSuspendOverrideAssetChangedDelegate(bool bSuspended)
	{
		bSuspendOverrideAssetChangedDelegate = bSuspended;
	}

private:

	void ApplyElementOverrides();
	void HandleOverrideChanged(const UControlRigOverrideAsset* InOverrideAsset);

	/** Broadcasts a notification just before the controlrig is destroyed. */
	FMeshDeformerBeginDestroyEvent BeginDestroyEvent;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UControlRigOverrideAsset>> OverrideAssets;

	bool bSuspendOverrideAssetChangedDelegate = false;
	
public:	
	
	UPROPERTY(transient)
	ERigExecutionType ExecutionType;

	UPROPERTY()
	FRigHierarchySettings HierarchySettings;

	CONTROLRIG_API virtual bool CanExecute() const override;
	CONTROLRIG_API virtual bool Execute(const FName& InEventName) override;
	CONTROLRIG_API virtual bool Execute_Internal(const FName& InEventName) override;
	CONTROLRIG_API virtual void RequestInit() override;
	virtual void RequestInitVMs()  { Super::RequestInit(); }

	/**
	 * Returns true if this rig instance needs the sequencer to re-run its instance-level
	 * initialization (re-binding, re-transferring bones from the skeletal mesh, etc.) on
	 * the next bind. Set by InitializeFromCDO when the rig is backed by a Runtime Asset —
	 * since those re-initialize the same UObject in place, the sequencer's track-map check
	 * (which is keyed by GetUniqueID()) would otherwise skip the refresh. BP rigs don't
	 * need this flag because BP reinstancing produces a new UObject with a new UniqueID.
	 */
	bool NeedsInstanceInitialization() const { return bNeedsInstanceInitialization; }
	void ClearInstanceInitializationPending() { bNeedsInstanceInitialization = false; }

	virtual bool SupportsEvent(const FName& InEventName) const override { return Super::SupportsEvent(InEventName); }
	virtual const TArray<FName>& GetSupportedEvents() const override{ return Super::GetSupportedEvents(); }

	template<class T>
	bool SupportsEvent() const
	{
		return SupportsEvent(T::EventName);
	}

	CONTROLRIG_API bool AllConnectorsAreResolved(FString* OutFailureReason = nullptr, FRigElementKey* OutConnector = nullptr) const;

	/** Requests to perform construction during the next execution */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	CONTROLRIG_API void RequestConstruction();

	CONTROLRIG_API bool IsConstructionRequired() const;

	/** Contains a backwards solve event */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	CONTROLRIG_API bool SupportsBackwardsSolve() const;

	CONTROLRIG_API virtual void AdaptEventQueueForEvaluate(TArray<FName>& InOutEventQueueToRun) override;

	/** INodeMappingInterface implementation */
	CONTROLRIG_API virtual void GetMappableNodeData(TArray<FName>& OutNames, TArray<FNodeItem>& OutNodeItems) const override;

	/** Data Source Registry Getter */
	CONTROLRIG_API UAnimationDataSourceRegistry* GetDataSourceRegistry();

	CONTROLRIG_API virtual TArray<FRigControlElement*> AvailableControls() const;
	CONTROLRIG_API virtual FRigControlElement* FindControl(const FName& InControlName) const;
	virtual bool ShouldApplyLimits() const { return !IsConstructionModeEnabled(); }
	CONTROLRIG_API virtual bool IsConstructionModeEnabled() const;
	CONTROLRIG_API virtual FTransform SetupControlFromGlobalTransform(const FName& InControlName, const FTransform& InGlobalTransform);
	CONTROLRIG_API virtual FTransform GetControlGlobalTransform(const FName& InControlName) const;

	// Sets the relative value of a Control
	template<class T>
	void SetControlValue(const FName& InControlName, T InValue, bool bNotify = true,
		const FRigControlModifiedContext& Context = FRigControlModifiedContext(), bool bSetupUndo = true, bool bPrintPythonCommnds = false, bool bFixEulerFlips = false)
	{
		SetControlValueImpl(InControlName, FRigControlValue::Make<T>(InValue), bNotify, Context, bSetupUndo, bPrintPythonCommnds, bFixEulerFlips);
	}

	// Returns the value of a Control
	FRigControlValue GetControlValue(const FName& InControlName) const
	{
		const FRigElementKey Key(InControlName, ERigElementType::Control);
		if (FRigBaseElement* Element = DynamicHierarchy->Find(Key))
		{
			if (FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
			{
				return GetControlValue(ControlElement, ERigControlValueType::Current);
			}
		}
		return DynamicHierarchy->GetControlValue(Key);
	}

	CONTROLRIG_API FRigControlValue GetControlValue(FRigControlElement* InControl, const ERigControlValueType& InValueType) const;

	// Sets the relative value of a Control
	CONTROLRIG_API virtual void SetControlValueImpl(const FName& InControlName, const FRigControlValue& InValue, bool bNotify = true,
		const FRigControlModifiedContext& Context = FRigControlModifiedContext(), bool bSetupUndo = true, bool bPrintPythonCommnds = false, bool bFixEulerFlips = false);

	CONTROLRIG_API void SwitchToParent(const FRigElementKey& InElementKey, const FRigElementKey& InNewParentKey, bool bInitial, bool bAffectChildren);

	FTransform GetInitialLocalTransform(const FRigElementKey &InKey)
	{
		if (bIsAdditive)
		{
			// The initial value of all additive controls is always Identity
			return FTransform::Identity;
		}
		return GetHierarchy()->GetInitialLocalTransform(InKey);
	}

	CONTROLRIG_API bool SetControlGlobalTransform(const FName& InControlName, const FTransform& InGlobalTransform, bool bNotify = true, const FRigControlModifiedContext& Context = FRigControlModifiedContext(), bool bSetupUndo = true, bool bPrintPythonCommands = false, bool bFixEulerFlips = false);

	CONTROLRIG_API virtual FRigControlValue GetControlValueFromGlobalTransform(const FName& InControlName, const FTransform& InGlobalTransform, ERigTransformType::Type InTransformType);

	CONTROLRIG_API virtual void SetControlLocalTransform(const FName& InControlName, const FTransform& InLocalTransform, bool bNotify = true, const FRigControlModifiedContext& Context = FRigControlModifiedContext(), bool bSetupUndo = true, bool bFixEulerFlips = false);
	CONTROLRIG_API virtual FTransform GetControlLocalTransform(const FName& InControlName) ;

	CONTROLRIG_API FVector GetControlSpecifiedEulerAngle(const FRigControlElement* InControlElement, bool bIsInitial = false) const;

	CONTROLRIG_API virtual const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() const;
	const TMap<FString, FString>& GetShapeLibraryNameMap() const { return ShapeLibraryNameMap; }
	CONTROLRIG_API virtual void CreateRigControlsForCurveContainer();
	CONTROLRIG_API virtual void GetControlsInOrder(TArray<FRigControlElement*>& SortedControls) const;

	/**
	 * Selects or deselects an element in the hierarchy
	 * @param InControlName The key of the element to select
	 * @param bSelect If set to false the element will be deselected
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	CONTROLRIG_API virtual void SelectControl(const FName& InControlName, bool bSelect = true, bool bSetupUndo = false);
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	/** @param bSetupUndo If set to true the stack will record the change for undo / redo */
	CONTROLRIG_API virtual bool ClearControlSelection(bool bSetupUndo = false);
	UFUNCTION(BlueprintPure, Category = "Control Rig")
	CONTROLRIG_API virtual TArray<FName> CurrentControlSelection() const;
	UFUNCTION(BlueprintPure, Category = "Control Rig")
	CONTROLRIG_API virtual bool IsControlSelected(const FName& InControlName)const;
	UFUNCTION(BlueprintPure, Category = "Control Rig")
	CONTROLRIG_API virtual TArray<FName> GetSortedControls() const;

	// Returns true if this manipulatable subject is currently
	// available for manipulation / is enabled.
	virtual bool ManipulationEnabled() const
	{
		return bManipulationEnabled;
	}

	// Sets the manipulatable subject to enabled or disabled
	virtual bool SetManipulationEnabled(bool Enabled = true)
	{
		if (bManipulationEnabled == Enabled)
		{
			return false;
		}
		bManipulationEnabled = Enabled;
		return true;
	}

	// Returns a event that can be used to subscribe to
	// filtering control data when needed
	FFilterControlEvent& ControlFilter() { return OnFilterControl; }

	// Returns a event that can be used to subscribe to
	// change notifications coming from the manipulated subject.
	FControlModifiedEvent& ControlModified() { return OnControlModified; }

	// Returns a event that can be used to subscribe to
	// selection changes coming from the manipulated subject.
	FControlSelectedEvent& ControlSelected() { return OnControlSelected; }

	// Returns an event that can be used to subscribe to
	// Undo Bracket requests such as Open and Close.
	FControlUndoBracketEvent & ControlUndoBracket() { return OnControlUndoBracket; }
	
	FControlRigBoundEvent& ControlRigBound() { return OnControlRigBound; };

	CONTROLRIG_API bool IsCurveControl(const FRigControlElement* InControlElement) const;

	DECLARE_EVENT_TwoParams(UControlRig, FControlRigExecuteEvent, class UControlRig*, const FName&);
#if WITH_EDITOR
	FControlRigExecuteEvent& OnPreConstructionForUI_AnyThread() { return PreConstructionForUIEvent; }
#endif
	FControlRigExecuteEvent& OnPreConstruction_AnyThread() { return PreConstructionEvent; }
	FControlRigExecuteEvent& OnPostConstruction_AnyThread() { return PostConstructionEvent; }
	FControlRigExecuteEvent& OnPreForwardsSolve_AnyThread() { return PreForwardsSolveEvent; }
	FControlRigExecuteEvent& OnPostForwardsSolve_AnyThread() { return PostForwardsSolveEvent; }
	FControlRigExecuteEvent& OnPreAdditiveValuesApplication_AnyThread() { return PreAdditiveValuesApplicationEvent; }
	FRigEventDelegate& OnRigEvent_AnyThread() { return RigEventDelegate; }

	// Setup the initial transform / ref pose of the bones based upon an anim instance
	// This uses the current refpose instead of the RefSkeleton pose.
	CONTROLRIG_API virtual void SetBoneInitialTransformsFromAnimInstance(UAnimInstance* InAnimInstance);

	// Setup the initial transform / ref pose of the bones based upon an anim instance proxy
	// This uses the current refpose instead of the RefSkeleton pose.
	CONTROLRIG_API virtual void SetBoneInitialTransformsFromAnimInstanceProxy(const FAnimInstanceProxy* InAnimInstanceProxy);

	// Setup the initial transform / ref pose of the bones based upon skeletal mesh component (ref skeleton)
	// This uses the RefSkeleton pose instead of the current refpose (or vice versae is bUseAnimInstance == true)
	CONTROLRIG_API virtual void SetBoneInitialTransformsFromSkeletalMeshComponent(USkeletalMeshComponent* InSkelMeshComp, bool bUseAnimInstance = false);

	// Setup the initial transforms / ref pose of the bones based on a skeletal mesh
	// This uses the RefSkeleton pose instead of the current refpose.
	CONTROLRIG_API virtual void SetBoneInitialTransformsFromSkeletalMesh(USkeletalMesh* InSkeletalMesh);

	// Setup the initial transforms / ref pose of the bones based on a reference skeleton
	// This uses the RefSkeleton pose instead of the current refpose.
	CONTROLRIG_API virtual void SetBoneInitialTransformsFromRefSkeleton(const FReferenceSkeleton& InReferenceSkeleton);

private:

	CONTROLRIG_API void SetBoneInitialTransformsFromCompactPose(FCompactPose* InCompactPose);

public:
	
	CONTROLRIG_API const FRigControlElementCustomization* GetControlCustomization(const FRigElementKey& InControl) const;
	CONTROLRIG_API void SetControlCustomization(const FRigElementKey& InControl, const FRigControlElementCustomization& InCustomization);

	CONTROLRIG_API virtual void PostInitInstanceIfRequired() override;
#if WITH_EDITORONLY_DATA
	static CONTROLRIG_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	CONTROLRIG_API virtual USceneComponent* GetOwningSceneComponent() override;

	CONTROLRIG_API void SetDynamicHierarchy(TObjectPtr<URigHierarchy> InHierarchy);

protected:

	CONTROLRIG_API virtual void PostInitInstance(URigVMHost* InCDO) override;
	
	UPROPERTY()
	TMap<FRigElementKey, FRigControlElementCustomization> ControlCustomizations;

	UPROPERTY()
	TObjectPtr<URigHierarchy> DynamicHierarchy;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSoftObjectPtr<UControlRigShapeLibrary> GizmoLibrary_DEPRECATED;
#endif

	UPROPERTY()
	TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries;

	UPROPERTY(transient)
	TMap<FString, FString> ShapeLibraryNameMap;

	/** Runtime object binding */
	TSharedPtr<IControlRigObjectBinding> ObjectBinding;

#if WITH_EDITORONLY_DATA
	// you either go Input or Output, currently if you put it in both place, Output will override
	UPROPERTY()
	TMap<FName, FCachedPropertyPath> InputProperties_DEPRECATED;

	UPROPERTY()
	TMap<FName, FCachedPropertyPath> OutputProperties_DEPRECATED;
#endif

private:
	
	CONTROLRIG_API void HandleOnControlModified(UControlRig* Subject, FRigControlElement* Control, const FRigControlModifiedContext& Context);

public:
	
	class FAnimAttributeContainerPtrScope
	{
	public:
		CONTROLRIG_API FAnimAttributeContainerPtrScope(UControlRig* InControlRig, UE::Anim::FMeshAttributeContainer& InExternalContainer);
		CONTROLRIG_API ~FAnimAttributeContainerPtrScope();

		UControlRig* ControlRig;
	};
	
private:
	UPROPERTY(Transient)
	FRigVMExtendedExecuteContext RigVMExtendedExecuteContext;

	UE::Anim::FMeshAttributeContainer* ExternalAnimAttributeContainer;

#if WITH_EDITOR
	void SetEnableAnimAttributeTrace(bool bInEnable)
	{
		bEnableAnimAttributeTrace = bInEnable;
	};
	
	bool bEnableAnimAttributeTrace;
	
	UE::Anim::FMeshAttributeContainer InputAnimAttributeSnapshot;
	UE::Anim::FMeshAttributeContainer OutputAnimAttributeSnapshot;
#endif
	
	/** The registry to access data source */
	UPROPERTY(Transient)
	TObjectPtr<UAnimationDataSourceRegistry> DataSourceRegistry;

	/** Broadcasts a notification when launching the construction event */
	FControlRigExecuteEvent PreConstructionForUIEvent;

	/** Broadcasts a notification just before the controlrig is setup. */
	FControlRigExecuteEvent PreConstructionEvent;

	/** Broadcasts a notification whenever the controlrig has been setup. */
	FControlRigExecuteEvent PostConstructionEvent;

	/** Broadcasts a notification before a forward solve has been initiated */
	FControlRigExecuteEvent PreForwardsSolveEvent;
	
	/** Broadcasts a notification after a forward solve has been initiated */
	FControlRigExecuteEvent PostForwardsSolveEvent;

	/** Broadcasts a notification before additive controls have been applied */
	FControlRigExecuteEvent PreAdditiveValuesApplicationEvent;

	/** Handle changes within the hierarchy */
	CONTROLRIG_API void HandleHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);

protected:
	
	CONTROLRIG_API virtual void EmitPostConstructionEventFinished();
	CONTROLRIG_API void SendSelectionBeforeInitNotifications();

	/**
	 * Session-local flag set by InitializeFromCDO when the rig is backed by a Runtime Asset.
	 * Consumed by BindToSequencerInstance (and similar paths) to force a re-bind/refresh
	 * even though the rig's UniqueID is unchanged. Cleared after the refresh completes.
	 */
	bool bNeedsInstanceInitialization = false;

#if RIGVM_TRACE_ENABLED
public:
	CONTROLRIG_API virtual UObject* GetBoundOuterForTrace() const override;
	
protected:
	CONTROLRIG_API virtual void TraceExecuteData(FRigVMTraceArchiveWriter& InArchive) const override;
	CONTROLRIG_API virtual void LoadTracedExecuteData(FRigVMTraceArchiveReader& InArchive) override;
#endif

private:
#if WITH_EDITOR
	/** Add a transient / temporary control used to interact with a node */
	CONTROLRIG_API FName AddTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget);

	/** Sets the value of a transient control based on a node */
	CONTROLRIG_API bool SetTransientControlValue(const URigVMUnitNode* InNode, TSharedPtr<FRigDirectManipulationInfo> InInfo);

	/** Remove a transient / temporary control used to interact with a node */
	CONTROLRIG_API FName RemoveTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget);

	CONTROLRIG_API FName AddTransientControl(const FRigElementKey& InElement);

	/** Sets the value of a transient control based on a bone */
	CONTROLRIG_API bool SetTransientControlValue(const FRigElementKey& InElement);

	/** Remove a transient / temporary control used to interact with a bone */
	CONTROLRIG_API FName RemoveTransientControl(const FRigElementKey& InElement);

	static CONTROLRIG_API FName GetNameForTransientControl(const FRigElementKey& InElement);
	CONTROLRIG_API FName GetNameForTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const;
	static CONTROLRIG_API FString GetNodeNameFromTransientControl(const FRigElementKey& InKey);
	static CONTROLRIG_API FString GetTargetFromTransientControl(const FRigElementKey& InKey);
	CONTROLRIG_API TSharedPtr<FRigDirectManipulationInfo> GetRigUnitManipulationInfoForTransientControl(const FRigElementKey& InKey);
	
	static CONTROLRIG_API FRigElementKey GetElementKeyFromTransientControl(const FRigElementKey& InKey);
	CONTROLRIG_API bool CanAddTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget, FString* OutFailureReason);

	/** Removes all  transient / temporary control used to interact with pins */
	CONTROLRIG_API void ClearTransientControls();

	UAnimPreviewInstance* PreviewInstance;

	// this is needed because PreviewInstance->ModifyBone(...) cannot modify user created bones,
	TMap<FName, FTransform> TransformOverrideForUserCreatedBones;
	
public:
	
	CONTROLRIG_API void ApplyTransformOverrideForUserCreatedBones();
	CONTROLRIG_API void ApplySelectionPoseForConstructionMode(const FName& InEventName);
	
#endif

protected:

	CONTROLRIG_API void HandleHierarchyEvent(URigHierarchy* InHierarchy, const FRigEventContext& InEvent);
	FRigEventDelegate RigEventDelegate;

	CONTROLRIG_API void RestoreShapeLibrariesFromCDO();
	CONTROLRIG_API void OnAddShapeLibrary(const FControlRigExecuteContext* InContext, const FString& InLibraryName, UControlRigShapeLibrary* InShapeLibrary, bool bLogResults);
	CONTROLRIG_API bool OnShapeExists(const FName& InShapeName) const;
	virtual void InitializeVMsFromCDO() { Super::InitializeFromCDO(); }
	CONTROLRIG_API virtual void InitializeFromCDO() override;


	UPROPERTY()
	FRigInfluenceMapPerEvent Influences;

	CONTROLRIG_API const FRigInfluenceMap* FindInfluenceMap(const FName& InEventName);


	mutable FRigElementKeyRedirector ElementKeyRedirector;
	
	TArray<FRigHierarchyKey> SelectionBeforeInit;

public:

	// UObject interface
#if WITH_EDITOR
	CONTROLRIG_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	CONTROLRIG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	float GetDebugBoneRadiusMultiplier() const { return DebugBoneRadiusMultiplier; }
	static CONTROLRIG_API FRigUnit* GetRigUnitInstanceFromScope(TSharedPtr<FStructOnScope> InScope);

public:
	//~ Begin IInterface_AssetUserData Interface
	CONTROLRIG_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface
protected:
	mutable TArray<TObjectPtr<UAssetUserData>> CombinedAssetUserData;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TMap<FName, TObjectPtr<UDataAssetLink>> ExternalVariableDataAssetLinks;

	DECLARE_DELEGATE_RetVal(TArray<TObjectPtr<UAssetUserData>>, FGetExternalAssetUserData);
	FGetExternalAssetUserData GetExternalAssetUserDataDelegate;

private:

	CONTROLRIG_API void CopyPoseFromOtherRig(UControlRig* Subject);

public:
	TGuardValue<bool> GetResetCurrentTransformsAfterConstructionGuard(bool bNewValue)
	{
		return TGuardValue<bool> (bResetCurrentTransformsAfterConstruction, bNewValue);
	}

protected:
	bool bCopyHierarchyBeforeConstruction;
	bool bResetInitialTransformsBeforeConstruction;
	bool bHierarchyInitializedThisFrame;
	bool bResetCurrentTransformsAfterConstruction;
	bool bManipulationEnabled;

	int32 PreConstructionBracket;
	int32 PostConstructionBracket;
	int32 PreForwardsSolveBracket;
	int32 PostForwardsSolveBracket;
	int32 PreAdditiveValuesApplicationBracket;
	int32 InteractionBracket;
	int32 InterRigSyncBracket;
	int32 ControlUndoBracketIndex;
	uint8 InteractionType;
	bool bEvaluationTriggeredFromInteraction;
	TArray<FRigElementKey> ElementsBeingInteracted;
#if WITH_EDITOR
	TArray<TSharedPtr<FRigDirectManipulationInfo>> RigUnitManipulationInfos;
#endif
	bool bInteractionJustBegan;

	TWeakObjectPtr<USceneComponent> OuterSceneComponent;

	bool IsRunningPreConstruction() const
	{
		return PreConstructionBracket > 0;
	}

	bool IsRunningPostConstruction() const
	{
		return PostConstructionBracket > 0;
	}

	bool IsInteracting() const
	{
		return InteractionBracket > 0;
	}

	uint8 GetInteractionType() const
	{
		return InteractionType;
	}

	bool IsSyncingWithOtherRig() const
	{
		return InterRigSyncBracket > 0;
	}


#if WITH_EDITOR
	static void OnHierarchyTransformUndoRedoWeak(URigHierarchy* InHierarchy, const FRigElementKey& InKey, ERigTransformType::Type InTransformType, const FTransform& InTransform, bool bIsUndo, TWeakObjectPtr<UControlRig> WeakThis)
	{
		if(WeakThis.IsValid() && InHierarchy != nullptr)
		{
			WeakThis->OnHierarchyTransformUndoRedo(InHierarchy, InKey, InTransformType, InTransform, bIsUndo);
		}
	}
#endif
	
	CONTROLRIG_API void OnHierarchyTransformUndoRedo(URigHierarchy* InHierarchy, const FRigElementKey& InKey, ERigTransformType::Type InTransformType, const FTransform& InTransform, bool bIsUndo);

	FFilterControlEvent OnFilterControl;
	FControlModifiedEvent OnControlModified;
	FControlSelectedEvent OnControlSelected;
	FControlUndoBracketEvent OnControlUndoBracket;
	FControlRigBoundEvent OnControlRigBound;

	UPROPERTY(BlueprintAssignable, Category = ControlRig, meta=(DisplayName="OnControlSelected"))
	FOnControlSelectedBP OnControlSelected_BP;

	TArray<FRigElementKey> QueuedModifiedControls;

	FControlRigOverrideAssetsChanged OverrideAssetsChangedDelegate;

 	CONTROLRIG_API virtual void GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& Context) override;
	
private:

#if WITH_EDITORONLY_DATA

	/** Whether controls are visible */
	UPROPERTY(transient)
	bool bControlsVisible = true;

#endif


protected:
	/** An additive control rig runs a backwards solve before applying additive control values
	 * and running the forward solve
	 */
	UPROPERTY()
	bool bIsAdditive;

	struct FRigSetControlValueInfo
	{
		FRigControlValue Value;
		bool bNotify;
		FRigControlModifiedContext Context;
		bool bSetupUndo;
		bool bPrintPythonCommnds;
		bool bFixEulerFlips;
	};
	struct FRigSwitchParentInfo
	{
		FRigElementKey NewParent;
		bool bInitial;
		bool bAffectChildren;
	};
	FRigPose PoseBeforeBackwardsSolve;
	FRigPose ControlsAfterBackwardsSolve;
	bool bOnlyRunConstruction = false;
	TMap<FRigElementKey, FRigSetControlValueInfo> ControlValues; // Layered Rigs: Additive values in local space (to add after backwards solve)
	TMap<FRigElementKey, FRigSwitchParentInfo> SwitchParentValues; // Layered Rigs: Parent switching values to perform after backwards solve


private:
	float DebugBoneRadiusMultiplier;

public:

	// We no longer support the idea of ControlRig hosting simulations - that can be done by adding
	// them to elements or components (etc). 
	UE_DEPRECATED(5.8, "Control Rig core no longer supports owning simulations")
	FRigPhysicsSimulationBase* GetPhysicsSimulation(const FRigComponentKey& InComponentKey) { return nullptr ;}

	UE_DEPRECATED(5.8, "Control Rig core no longer supports owning simulations")
	const FRigPhysicsSimulationBase* GetPhysicsSimulation(const FRigComponentKey& InComponentKey) const { return nullptr; }

	UE_DEPRECATED(5.8, "Control Rig core no longer supports owning simulations")
	bool RegisterPhysicsSimulation(
		TSharedPtr<FRigPhysicsSimulationBase> PhysicsSimulation, const FRigComponentKey& InComponentKey) { return false; }

#if WITH_EDITOR	

	void ToggleControlsVisible() { bControlsVisible = !bControlsVisible; }
	void SetControlsVisible(const bool bIsVisible) { bControlsVisible = bIsVisible; }
	bool GetControlsVisible()const { return bControlsVisible;}

#endif
	
	virtual bool IsAdditive() const { return bIsAdditive; }
	void SetIsAdditive(const bool bInIsAdditive)
	{
		bIsAdditive = bInIsAdditive;
		if (URigHierarchy* Hierarchy = GetHierarchy())
		{
			Hierarchy->bUsePreferredEulerAngles = !bIsAdditive;
		}
	}

private:

	// Class used to temporarily cache current pose of the hierarchy
	// restore it on destruction, similar to UControlRigBlueprint::FControlValueScope
	class FPoseScope
	{
	public:
		FPoseScope(UControlRig* InControlRig, ERigElementType InFilter = ERigElementType::All,
			const TArray<FRigElementKey>& InElements = TArray<FRigElementKey>(), const ERigTransformType::Type InTransformType = ERigTransformType::CurrentLocal);
		~FPoseScope();

	private:

		UControlRig* ControlRig;
		ERigElementType Filter;
		FRigPose CachedPose;
		ERigTransformType::Type TransformType;
	};

	UPROPERTY()
	FRigModuleSettings RigModuleSettings;

	mutable TPair<FName, FString> RigModulePrefix = {NAME_None, FString()};

public:

#if WITH_EDITOR

	// Class used to temporarily cache current transient controls
	// restore them after a CopyHierarchy call
	class FTransientControlScope
	{
	public:
		FTransientControlScope(TObjectPtr<URigHierarchy> InHierarchy);
		~FTransientControlScope();
	
	private:
		// used to match URigHierarchyController::AddControl(...)
		struct FTransientControlInfo
		{
			FName Name;
			// transient control should only have 1 parent, with weight = 1.0
			FRigElementKey Parent;
			FRigControlSettings Settings;
			FRigControlValue Value;
			FTransform OffsetTransform;
			FTransform ShapeTransform;
		};
		
		TArray<FTransientControlInfo> SavedTransientControls;
		TObjectPtr<URigHierarchy> Hierarchy;
	};

	// Class used to temporarily cache current pose of transient controls
	// restore them after a ResetPoseToInitial call,
	// which allows user to move bones in construction mode
	class FTransientControlPoseScope
	{
	public:
		FTransientControlPoseScope(TObjectPtr<UControlRig> InControlRig)
		{
			ControlRig = InControlRig;

			TArray<FRigControlElement*> TransientControls = ControlRig->GetHierarchy()->GetTransientControls();
			TArray<FRigElementKey> Keys;
			for(FRigControlElement* TransientControl : TransientControls)
			{
				Keys.Add(TransientControl->GetKey());
			}

			if(Keys.Num() > 0)
			{
				CachedPose = ControlRig->GetHierarchy()->GetPose(false, ERigElementType::Control, TArrayView<FRigElementKey>(Keys));
			}
		}
		~FTransientControlPoseScope()
		{
			check(ControlRig);

			if(CachedPose.Num() > 0)
			{
				ControlRig->GetHierarchy()->SetPose(CachedPose);
			}
		}
	
	private:
		
		UControlRig* ControlRig;
		FRigPose CachedPose;	
	};	

	bool bRecordSelectionPoseForConstructionMode;
	TMap<FRigElementKey, FTransform> SelectionPoseForConstructionMode;
	bool bIsClearingTransientControls;

	FRigPose InputPoseOnDebuggedRig;
	
#endif

public:
	UE_DEPRECATED(5.4, "InteractionRig is no longer used") UFUNCTION(BlueprintGetter, meta = (DeprecatedFunction, DeprecationMessage = "InteractionRig is no longer used"))
	UControlRig* GetInteractionRig() const
	{
#if WITH_EDITORONLY_DATA
		return InteractionRig_DEPRECATED;
#else
		return nullptr;
#endif
	}

	UE_DEPRECATED(5.4, "InteractionRig is no longer used")
	UFUNCTION(BlueprintSetter, meta = (DeprecatedFunction, DeprecationMessage = "InteractionRig is no longer used"))
	void SetInteractionRig(UControlRig* InInteractionRig) {}

	UE_DEPRECATED(5.4, "InteractionRig is no longer used")
	UFUNCTION(BlueprintGetter, meta = (DeprecatedFunction, DeprecationMessage = "InteractionRig is no longer used"))
	TSubclassOf<UControlRig> GetInteractionRigClass() const
	{
#if WITH_EDITORONLY_DATA
		return InteractionRigClass_DEPRECATED;
#else
		return nullptr;
#endif
	}

	UE_DEPRECATED(5.4, "InteractionRig is no longer used")
	UFUNCTION(BlueprintSetter, meta = (DeprecatedFunction, DeprecationMessage = "InteractionRig is no longer used"))
	void SetInteractionRigClass(TSubclassOf<UControlRig> InInteractionRigClass) {}

	CONTROLRIG_API uint32 GetShapeLibraryHash() const;
	
private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UControlRig> InteractionRig_DEPRECATED;

	UPROPERTY()
	TSubclassOf<UControlRig> InteractionRigClass_DEPRECATED;
#endif

public:
	CONTROLRIG_API int32 GetReplayTimeIndex() const;
	CONTROLRIG_API void SetReplayTimeIndex(int32 InReplayTimeIndex);
	CONTROLRIG_API void DisableReplay();
	CONTROLRIG_API bool IsReplayEnabled() const;
	CONTROLRIG_API void SetReplay(UControlRigReplay* InReplay);

	// Nodes/components etc that contain "simulation" (i.e. contain significant amounts of state,
	// and won't behave nicely when given extra "updates") should set this on the control rig during
	// construction or during execution. It will be used to prevent additional updates of the
	// control rig that are there primarily to make it feel responsive when editing. Note that the
	// flag will be reset automatically before construction etc.
	void SetContainsSimulation() { bContainsSimulation = true; }
	bool ContainsSimulation() const { return bContainsSimulation; }

private:
	bool bContainsSimulation = false;

	int32 ReplayTimeIndex;
	TObjectPtr<UControlRigReplay> Replay;

	friend class FControlRigBlueprintCompilerContext;
	friend struct FRigHierarchyRef;
	friend class FControlRigBaseEditor;
	friend class SRigCurveContainer;
	friend class SRigHierarchy;
	friend class SControlRigAnimAttributeView;
	friend class UEngineTestControlRig;
 	friend class FControlRigEditMode;
	friend class IControlRigEditorAssetInterface;
	friend class UControlRigBlueprint;
	friend class UControlRigRuntimeAsset;
	friend class UControlRigComponent;
	friend class UControlRigBlueprintGeneratedClass;
	friend class FControlRigInteractionScope;
	friend class UControlRigValidator;
	friend struct FAnimNode_ControlRig;
	friend struct FAnimNode_ControlRigBase;
	friend class URigHierarchy;
	friend class UFKControlRig;
	friend class UControlRigGraph;
	friend class AControlRigControlActor;
	friend class AControlRigShapeActor;
	friend class FRigTransformElementDetails;
	friend class FControlRigEditorModule;
	friend class UModularRig;
	friend class UModularRigController;
	friend struct FControlRigHierarchyMappings;
	friend class UControlRigReplay;
	friend class FControlRigEditModeShapeSettingsDetails;
};

class FControlRigBracketScope
{
public:

	FControlRigBracketScope(int32& InBracket)
		: Bracket(InBracket)
	{
		Bracket++;
	}

	~FControlRigBracketScope()
	{
		Bracket--;
	}

private:

	int32& Bracket;
};

class FControlRigInteractionScope
{
public:

	FControlRigInteractionScope(UControlRig* InControlRig)
		: ControlRig(InControlRig)
		, InteractionBracketScope(InControlRig->InteractionBracket)
		, SyncBracketScope(InControlRig->InterRigSyncBracket)
	{
		InControlRig->GetHierarchy()->StartInteraction();
	}

	FControlRigInteractionScope(
		UControlRig* InControlRig,
		const FRigElementKey& InKey,
		EControlRigInteractionType InInteractionType = EControlRigInteractionType::All
	)
		: ControlRig(InControlRig)
		, InteractionBracketScope(InControlRig->InteractionBracket)
		, SyncBracketScope(InControlRig->InterRigSyncBracket)
	{
		InControlRig->ElementsBeingInteracted = { InKey };
		InControlRig->InteractionType = (uint8)InInteractionType;
		InControlRig->bInteractionJustBegan = true;
		InControlRig->GetHierarchy()->StartInteraction();
	}

	FControlRigInteractionScope(
		UControlRig* InControlRig,
		TArray<FRigElementKey> InKeys,
		EControlRigInteractionType InInteractionType = EControlRigInteractionType::All
	)
		: ControlRig(InControlRig)
		, InteractionBracketScope(InControlRig->InteractionBracket)
		, SyncBracketScope(InControlRig->InterRigSyncBracket)
	{
		InControlRig->ElementsBeingInteracted = MoveTemp(InKeys);
		InControlRig->InteractionType = (uint8)InInteractionType;
		InControlRig->bInteractionJustBegan = true;
		InControlRig->GetHierarchy()->StartInteraction();
	}

	~FControlRigInteractionScope()
	{
		if(ensure(ControlRig.IsValid()))
		{
			ControlRig->GetHierarchy()->EndInteraction();
			ControlRig->InteractionType = (uint8)EControlRigInteractionType::None;
			ControlRig->bInteractionJustBegan = false;
			ControlRig->ElementsBeingInteracted.Reset();
		}
	}

	const TArray<FRigElementKey>& GetElementsBeingInteracted() const
	{
		if(ensure(ControlRig.IsValid()))
		{
			return ControlRig->ElementsBeingInteracted;
		}

		static const TArray<FRigElementKey> DummyElements;
		return DummyElements;
	}

	UControlRig* GetControlRig() const
	{
		return ControlRig.Get();
	}

private:

	TWeakObjectPtr<UControlRig> ControlRig;
	FControlRigBracketScope InteractionBracketScope;
	FControlRigBracketScope SyncBracketScope;
};

