// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AttributesRuntime.h"
#include "ControlRigAnimInstance.generated.h"

#define UE_API CONTROLRIG_API

struct FMeshPoseBoneIndex;

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FControlRigAnimInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

public:
	FControlRigAnimInstanceProxy()
	{
	}

	FControlRigAnimInstanceProxy(UAnimInstance* InAnimInstance)
		: FAnimInstanceProxy(InAnimInstance)
	{
	}

	UE_API virtual ~FControlRigAnimInstanceProxy() override;

	// FAnimInstanceProxy interface
	UE_API virtual void Initialize(UAnimInstance* InAnimInstance) override;
	UE_API virtual bool Evaluate(FPoseContext& Output) override;
	UE_API virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;

	TMap<FMeshPoseBoneIndex, FTransform> StoredTransforms;
	TMap<FName, float> StoredCurves;
	UE::Anim::FMeshAttributeContainer StoredAttributes;
};

UCLASS(MinimalAPI, transient, NotBlueprintable)
class UControlRigAnimInstance : public UAnimInstance
{
	GENERATED_UCLASS_BODY()

		FControlRigAnimInstanceProxy* GetControlRigProxyOnGameThread() { return &GetProxyOnGameThread <FControlRigAnimInstanceProxy>(); }

protected:

	// UAnimInstance interface
	UE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
};

#undef UE_API
