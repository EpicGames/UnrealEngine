// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "RigVMCore/RigVMExecuteContext.h"

#include "ControlRigAssetReference.generated.h"

#define UE_API CONTROLRIG_API

struct FControlRigAssetSoftReference;
struct FModularRigModel;
struct FRigVMExternalVariable;
class UControlRigRuntimeAsset;
class UControlRig;
class URigHierarchy;
struct FRigModuleSettings;
struct FRigVMVariant;

USTRUCT(BlueprintType)
struct FControlRigAssetStrongReference 
{
	GENERATED_BODY()
	
private:
	// Legacy
	UPROPERTY()
	TSubclassOf<UControlRig> BlueprintRigClass;
	
	UPROPERTY()
	TObjectPtr<UControlRigRuntimeAsset> ControlRigAsset;
	
public:
	
	FControlRigAssetStrongReference()
		: BlueprintRigClass(nullptr), ControlRigAsset(nullptr)
	{}
	
	UE_API FControlRigAssetStrongReference(UObject* InAsset);
	UE_API FControlRigAssetStrongReference(TSubclassOf<UControlRig> InClass);
	UE_API FControlRigAssetStrongReference(TObjectPtr<UControlRigRuntimeAsset> InRuntimeAsset);
	
	void Set(TSubclassOf<UControlRig> InClass) { BlueprintRigClass = InClass; ControlRigAsset = nullptr; }
	void Set(TObjectPtr<UControlRigRuntimeAsset> InAsset) { ControlRigAsset = InAsset; BlueprintRigClass= nullptr; }
	
	UE_API void Reset();
	
	TSubclassOf<UControlRig> GetBlueprintClass() const { return BlueprintRigClass; }
	TObjectPtr<UControlRigRuntimeAsset> GetControlRigAsset() const { return ControlRigAsset; }
	
	UE_API bool operator==(const FControlRigAssetStrongReference& InOtherValue) const;

	UE_API UControlRig* CreateInstance(UObject* InOuter, FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags) const;
	
	UE_API bool IsValid() const;
	UE_API FString GetName() const;
	UE_API FString GetPathName() const;
	UE_API UObject* Get() const;
	UE_API UStruct* GetVariablesStruct() const;
	UE_API uint8* GetVariablesMemory() const;
#if WITH_EDITORONLY_DATA
	UE_API UObject* GetEditorAsset() const;
#endif
	UE_API bool IsSourceOf(UObject* InObject) const;

	UE_API bool IsRigModule() const;
	UE_API bool IsModularRig() const;
	UE_API UClass* GetRigClass() const;
	UE_API const FRigModuleSettings& GetRigModuleSettings() const;
	UE_API const TArray<FName>& GetSupportedEvents() const;
	UE_API TArray<FRigVMExternalVariable> GetExternalVariables() const;
	UE_API FRigVMExtendedExecuteContext& GetRigVMExtendedExecuteContext() const;
	UE_API const FModularRigModel& GetModularRigModel() const;
	UE_API URigHierarchy* GetDefaultHierarchy() const;
	UE_API bool IsNative() const;
	UE_API void ForEachVariableProperty(TFunction<bool(const FProperty*)> PerVariableProperty);
	UE_API const FProperty* FindVariablePropertyByName(const FName& InName) const;
	UE_API FRigVMVariant GetVariant() const;
	
	friend uint32 GetTypeHash(const FControlRigAssetStrongReference& InSource);
};


USTRUCT(BlueprintType)
struct FControlRigAssetSoftReference 
{
	GENERATED_BODY()
	
private:
	// Legacy
	UPROPERTY()
	TSoftClassPtr<UControlRig> BlueprintRigClass;
	
	UPROPERTY()
	TSoftObjectPtr<UControlRigRuntimeAsset> ControlRigAsset;
	
public:
	
	FControlRigAssetSoftReference()	{}
	
	UE_API FControlRigAssetSoftReference(UObject* InAsset);
	UE_API FControlRigAssetSoftReference(TSoftClassPtr<UControlRig> InClass);
	UE_API FControlRigAssetSoftReference(TSoftObjectPtr<UControlRigRuntimeAsset> InRuntimeAsset);
	
	void Set(TSoftClassPtr<UControlRig> InClass) { BlueprintRigClass = InClass; ControlRigAsset.Reset(); }
	void Set(TSoftObjectPtr<UControlRigRuntimeAsset> InAsset) { ControlRigAsset = InAsset; BlueprintRigClass.Reset(); }
	
	UE_API bool operator==(const FControlRigAssetSoftReference& InOtherValue) const;
	
	UE_API bool IsValid() const;
	UE_API UObject* Get() const;
	UE_API UObject* LoadSynchronous(bool bForceLoad = false) const;
	UE_API FString GetName() const;
	UE_API FString GetPathName() const;
	UE_API FSoftObjectPath ToSoftObjectPath() const;
	UE_API FRigVMVariant GetVariant() const;
	
	UE_API FControlRigAssetStrongReference LoadStrongReference(bool bForceLoad = false) const;
	
	friend uint32 GetTypeHash(const FControlRigAssetSoftReference& InSource);
};



#undef UE_API