// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMAssetUserData.h"
#include "ControlRigGizmoLibrary.h"
#include "ControlRigAssetUserData.generated.h"

#define UE_API CONTROLRIG_API

/**
* Namespaced user data which provides access to a linked shape library
*/
UCLASS(MinimalAPI, BlueprintType)
class UControlRigShapeLibraryLink : public UNameSpacedUserData
{
	GENERATED_BODY()

public:

	/** If assigned, the data asset link will provide access to the data asset's content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, BlueprintGetter = GetShapeLibrary, BlueprintSetter = SetShapeLibrary, Meta = (DisplayAfter="NameSpace"))
	TSoftObjectPtr<UControlRigShapeLibrary> ShapeLibrary;

	UFUNCTION(BlueprintGetter)
	TSoftObjectPtr<UControlRigShapeLibrary> GetShapeLibrary() const { return ShapeLibrary; }

	UFUNCTION(BlueprintSetter)
	UE_API void SetShapeLibrary(TSoftObjectPtr<UControlRigShapeLibrary> InShapeLibrary);

	UE_API virtual const FUserData* GetUserData(const FString& InPath, FString* OutErrorMessage = nullptr) const override;
	UE_API virtual const TArray<const FUserData*>& GetUserDataArray(const FString& InParentPath = FString(), FString* OutErrorMessage = nullptr) const override;

	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual bool IsPostLoadThreadSafe() const override { return false; }
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	UE_API virtual void UpdateShapeLibraryCache() const;

protected:

	UPROPERTY(transient)
	TArray<FName> ShapeNames;

	UPROPERTY()
	mutable TObjectPtr<UControlRigShapeLibrary> ShapeLibraryCached;
	

	static inline constexpr TCHAR DefaultShapePath[] = TEXT("DefaultShape");
	static inline constexpr TCHAR ShapeNamesPath[] = TEXT("ShapeNames");
	static inline constexpr TCHAR ShapeLibraryNullFormat[] = TEXT("User data path '%s' could not be found (ShapeLibrary not provided)");
};

#undef UE_API
