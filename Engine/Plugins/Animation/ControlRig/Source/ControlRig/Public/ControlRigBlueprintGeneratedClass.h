// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigRuntimeAsset.h"
#include "UObject/ObjectMacros.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Engine/SkeletalMesh.h"
#include "Rigs/RigModuleDefines.h"
#include "ControlRigBlueprintGeneratedClass.generated.h"

#define UE_API CONTROLRIG_API

UCLASS(MinimalAPI)
class UControlRigBlueprintGeneratedClass : public URigVMBlueprintGeneratedClass, public IControlRigRuntimeAssetInterface
{
	GENERATED_UCLASS_BODY()

public:

	// UObject interface
	UE_API void Serialize(FArchive& Ar);

	UPROPERTY(AssetRegistrySearchable)
	FSoftObjectPath PreviewSkeletalMesh;

	UPROPERTY(AssetRegistrySearchable)
	bool bExposesAnimatableControls;

	UPROPERTY()
	bool bAllowMultipleInstances;

	UPROPERTY(AssetRegistrySearchable)
	EControlRigType ControlRigType;

	UPROPERTY(AssetRegistrySearchable)
	FName ItemTypeDisplayName = TEXT("Control Rig");

	UPROPERTY(AssetRegistrySearchable)
	FRigModuleSettings RigModuleSettings;

	/** Asset searchable information module references in this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FModuleReferenceData> ModuleReferenceData;

	// This relates to FAssetThumbnailPool::CustomThumbnailTagName and allows
	// the thumbnail pool to show the thumbnail of the icon rather than the
	// rig itself to avoid deploying the 3D renderer.
	UPROPERTY(AssetRegistrySearchable)
	FString CustomThumbnail;

	/** Whether or not this rig has an Inversion Event */
	UPROPERTY(AssetRegistrySearchable)
	bool bSupportsInversion;

	/** Whether or not this rig has Controls on It */
	UPROPERTY(AssetRegistrySearchable)
	bool bSupportsControls;

	UE_API bool IsControlRigModule() const;
	
	// IControlRigRuntimeAssetInterface implementation
	UE_API virtual FControlRigAssetStrongReference GetControlRigAssetReference() override;
	UE_API virtual UClass* GetRigVMHostClass() const override;
	UE_API virtual URigHierarchy* GetHierarchy() override;
	UE_API virtual void SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy) override;
	UE_API virtual FModularRigSettings& GetModularRigSettings() override;
	UE_API virtual FRigHierarchySettings& GetHierarchySettings() override;
	virtual FRigModuleSettings& GetRigModuleSettings() override { return RigModuleSettings; }
	virtual TArray<FModuleReferenceData>& GetModuleReferenceData() override { return ModuleReferenceData; }
	UE_API virtual FRigInfluenceMapPerEvent& GetInfluences() override;
	UE_API virtual FModularRigModel& GetModularRigModel() override;
	virtual void SetControlRigType(EControlRigType& InType) override { ControlRigType = InType;}
	virtual EControlRigType GetControlRigType() const override { return ControlRigType; }
	virtual void SetItemTypeDisplayName(FName& InName) override { ItemTypeDisplayName = InName; }
	virtual FName GetItemTypeDisplayName() const override { return ItemTypeDisplayName; }
	virtual bool& GetSupportsInversion() override { return bSupportsInversion; }
	virtual bool& GetSupportsControls() override { return bSupportsControls; }
	virtual bool& GetExposesAnimatableControls() override { return bExposesAnimatableControls; }
	virtual bool& GetAllowMultipleInstances() override { return bAllowMultipleInstances; }
	UE_API virtual FRigElementKeyRedirector& GetElementKeyRedirector() override;
	UE_API virtual TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() override;
	UE_API virtual TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() override;
	UE_API virtual TSoftObjectPtr<UObject>& GetSourceCurveImport() override;
	UE_API virtual TSoftObjectPtr<UObject>& GetSourceHierarchyImport() override;
	virtual TSoftObjectPtr<USkeletalMesh> GetPreviewSkeletalMesh() const override { return Cast<USkeletalMesh>(PreviewSkeletalMesh.ResolveObject()); }
	virtual void SetPreviewSkeletalMesh(TSoftObjectPtr<USkeletalMesh> InMesh) override { PreviewSkeletalMesh = InMesh.Get();}

private:
	// Helper to get CDO
	UControlRig* GetCDO() const;
};

#undef UE_API
