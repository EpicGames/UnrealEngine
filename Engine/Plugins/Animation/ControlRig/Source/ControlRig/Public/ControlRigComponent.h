// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "ControlRig.h"
#include "ControlRigAnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "PrimitiveSceneProxy.h"

#if WITH_EDITOR
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#endif

#include "ControlRigComponent.generated.h"

class UControlRigComponent;

/** Enum for controlling which space a transform is applied in. */
UENUM()
enum class EControlRigComponentSpace : uint8
{
	/** World space transform */
	WorldSpace,
	/** The space below the actor's root transform */
	ActorSpace,
	/** The space defined by the Control Rig Component */
	ComponentSpace,
	/**
	* The space within the rig. Currently the same as Component Space.
	* Inside of control rig this is called 'Global Space'.
	*/
	RigSpace,
	/** The space defined by each element's parent (bone, control etc) */
	LocalSpace,
	Max UMETA(Hidden),
};

/** Enum for controlling how an element should be mapped. */
UENUM()
enum class EControlRigComponentMapDirection : uint8
{
	Input, // An input driving a rig element
	Output, // An output driven by a rig element
};


USTRUCT(BlueprintType, DisplayName = "Mapped Element")
struct FControlRigComponentMappedElement
{
	GENERATED_BODY()

		FControlRigComponentMappedElement()
		: ComponentReference(FSoftComponentReference())
		, TransformIndex(INDEX_NONE)
		, TransformName(NAME_None)
		, ElementType(ERigElementType::Bone)
		, ElementName(NAME_None)
		, Direction(EControlRigComponentMapDirection::Output)
		, Offset(FTransform::Identity)
		, Weight(1.f)
		, Space(EControlRigComponentSpace::WorldSpace)
		, SceneComponent(nullptr)
		, ElementIndex(INDEX_NONE)
		, SubIndex(INDEX_NONE)
	{
	}

	// The component to map to the Control Rig
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FSoftComponentReference ComponentReference;

	// An optional index that can be used with components
	// with multiple transforms (for example the InstancedStaticMeshComponent)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	int32 TransformIndex;

	// An optional name that can be used with components
	// that have sockets (for example the SkeletalMeshComponent)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName TransformName;

	// The type of element this is mapped to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	ERigElementType ElementType; 

	// The name of the element to map to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName ElementName;

	// The direction (input / output) to be used for mapping an element
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	EControlRigComponentMapDirection Direction;

	// The offset transform to apply
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FTransform Offset;

	// defines how much the mapped element should be driven
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	float Weight;

	// space in which the mapping happens
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	EControlRigComponentSpace Space;

	UPROPERTY(transient)
	TObjectPtr<USceneComponent> SceneComponent;

	UPROPERTY(transient)
	int32 ElementIndex;

	UPROPERTY(transient)
	int32 SubIndex;

	FControlRigAnimInstanceProxy* GetAnimProxyOnGameThread() const;
};

USTRUCT(BlueprintType, DisplayName = "Mapped Component")
struct FControlRigComponentMappedComponent
{
	GENERATED_BODY()

	FControlRigComponentMappedComponent()
		: Component(nullptr)
		, ElementName(NAME_None)
		, ElementType(ERigElementType::Bone)
		, Direction(EControlRigComponentMapDirection::Output)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	TObjectPtr<USceneComponent> Component;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName ElementName;

	// The type of element this is mapped to
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	ERigElementType ElementType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	EControlRigComponentMapDirection Direction;
};

USTRUCT(BlueprintType, DisplayName = "Mapped Bone")
struct FControlRigComponentMappedBone
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName Source;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName Target;
};

USTRUCT(BlueprintType, DisplayName = "Mapped Curve")
struct FControlRigComponentMappedCurve
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName Source;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Mapping")
	FName Target;
};

/** Bindable event for external objects to hook into ControlRig-level execution */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FControlRigComponentDelegate, UControlRigComponent*, Component);

/** A component that hosts an animation ControlRig, manages control components and marshals data between the two */
UCLASS(MinimalAPI, Blueprintable, ClassGroup = "Animation", meta = (BlueprintSpawnableComponent))
class UControlRigComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

public:

	/** The class of control rig to instantiate */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ControlRig, meta = (DisallowCreateNew), BlueprintSetter = SetControlRigAssetReference)
	FControlRigAssetStrongReference ControlRigAssetReference;
	
	UPROPERTY(meta=(DeprecatedProperty))
	TSubclassOf<UControlRig> ControlRigClass_DEPRECATED;

	/** Event fired just before this component's ControlRig is initialized */
	UPROPERTY(BlueprintAssignable, Category = "ControlRig", meta = (DisplayName = "On Pre Initialize"))
	FControlRigComponentDelegate OnPreInitializeDelegate;

	/** Event fired after this component's ControlRig is initialized */
	UPROPERTY(BlueprintAssignable, Category = "ControlRig", meta = (DisplayName = "On Post Initialize"))
	FControlRigComponentDelegate OnPostInitializeDelegate;

	/** Event fired before this component's ControlRig is setup */
	UPROPERTY(BlueprintAssignable, Category = "ControlRig", meta = (DisplayName = "On Pre Construction"))
	FControlRigComponentDelegate OnPreConstructionDelegate;

	/** Event fired after this component's ControlRig is setup */
	UPROPERTY(BlueprintAssignable, Category = "ControlRig", meta = (DisplayName = "On Post Construction"))
	FControlRigComponentDelegate OnPostConstructionDelegate;

	/** Event fired before this component's ControlRig's forwards solve */
	UPROPERTY(BlueprintAssignable, Category = "ControlRig", meta = (DisplayName = "On Pre Forwards Solve"))
	FControlRigComponentDelegate OnPreForwardsSolveDelegate;

	/** Event fired after this component's ControlRig's forwards solve */
	UPROPERTY(BlueprintAssignable, Category = "ControlRig", meta = (DisplayName = "On Post Forwards Solve"))
	FControlRigComponentDelegate OnPostForwardsSolveDelegate;

	//~ Begin UObject interface
#if WITH_EDITOR
	CONTROLRIG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CONTROLRIG_API virtual void PostLoad() override;
#endif

	CONTROLRIG_API virtual void BeginDestroy() override;

	CONTROLRIG_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface
	
	//~ Begin UActorComponent Interface
	CONTROLRIG_API virtual void OnRegister() override;
	CONTROLRIG_API virtual void OnUnregister() override;
	CONTROLRIG_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	CONTROLRIG_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	CONTROLRIG_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End UPrimitiveComponent Interface.

	/** Get the ControlRig hosted by this component */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API UControlRig* GetControlRig();

	/** Returns true if the Component can execute its Control Rig */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API bool CanExecute();

	/** Get the ControlRig's local time in seconds since its last initialize */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API float GetAbsoluteTime() const;

	UFUNCTION(BlueprintNativeEvent, Category = "ControlRig", CallInEditor)
	CONTROLRIG_API void OnPreInitialize(UControlRigComponent* Component);

	UFUNCTION(BlueprintNativeEvent, Category = "ControlRig", CallInEditor)
	CONTROLRIG_API void OnPostInitialize(UControlRigComponent* Component);

	UFUNCTION(BlueprintNativeEvent, Category = "ControlRig", CallInEditor)
	CONTROLRIG_API void OnPreConstruction(UControlRigComponent* Component);

	UFUNCTION(BlueprintNativeEvent, Category = "ControlRig", CallInEditor)
	CONTROLRIG_API void OnPostConstruction(UControlRigComponent* Component);

	UFUNCTION(BlueprintNativeEvent, Category = "ControlRig", CallInEditor, meta = (DisplayName = "On Pre Forwards Solve"))
	CONTROLRIG_API void OnPreForwardsSolve(UControlRigComponent* Component);

	UFUNCTION(BlueprintNativeEvent, Category = "ControlRig", CallInEditor, meta = (DisplayName = "On Post Forwards Solve"))
	CONTROLRIG_API void OnPostForwardsSolve(UControlRigComponent* Component);

	/** Initializes the rig's memory and calls the construction event */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void Initialize();

	/** Updates and ticks the rig. */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void Update(float DeltaTime = 0.f);

	UPROPERTY(EditAnywhere, Category = "Animation")
	TArray<FControlRigComponentMappedElement> UserDefinedElements;

	UPROPERTY(VisibleAnywhere, Category = "Animation")
	TArray<FControlRigComponentMappedElement> MappedElements;

	/** Removes all mapped elements from the component */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void ClearMappedElements();

	/** Replaces the mapped elements on the component with the provided array, should not be used before OnPreInitialize Event */
	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta = (UnsafeDuringActorConstruction = "true"))
	CONTROLRIG_API void SetMappedElements(TArray<FControlRigComponentMappedElement> NewMappedElements);

	/** Adds the provided mapped elements to the component, should not be used before OnPreInitialize Event*/
	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta = (UnsafeDuringActorConstruction = "true"))
	CONTROLRIG_API void AddMappedElements(TArray<FControlRigComponentMappedElement> NewMappedElements);

	/** Adds a series of mapped bones to the rig, should not be used before OnPreInitialize Event */
	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta = (UnsafeDuringActorConstruction = "true"))
	CONTROLRIG_API void AddMappedComponents(TArray<FControlRigComponentMappedComponent> Components);

	/** Adds a series of mapped bones to the rig, should not be used before OnPreInitialize Event */
	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta = (DisplayName = "Add Mapped Skeletal Mesh Bone Array", UnsafeDuringActorConstruction = "true"))
	CONTROLRIG_API void AddMappedSkeletalMesh(USkeletalMeshComponent* SkeletalMeshComponent, TArray<FControlRigComponentMappedBone> Bones, TArray<FControlRigComponentMappedCurve> Curves, const EControlRigComponentMapDirection InDirection = EControlRigComponentMapDirection::Output);

	/** Adds all matching bones to the rig, should not be used before OnPreInitialize Event */
	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta = (DisplayName = "Add Mapped Skeletal Mesh", UnsafeDuringActorConstruction = "true"))
	CONTROLRIG_API void AddMappedCompleteSkeletalMesh(USkeletalMeshComponent* SkeletalMeshComponent, const EControlRigComponentMapDirection InDirection = EControlRigComponentMapDirection::Output);

	/** Setup the initial transforms / ref pose of the bones based on a skeletal mesh */
	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta = (DisplayName = "Set Bone Initial Transforms From Skeletal Mesh"))
	CONTROLRIG_API void SetBoneInitialTransformsFromSkeletalMesh(USkeletalMesh* InSkeletalMesh);

	/** When checked the rig will only run if any of the mapped inputs has changed */
	UPROPERTY(EditAnywhere, Category = "Lazy Evaluation")
	bool bEnableLazyEvaluation;

	// The delta threshold for a translation / position difference. 0.0 disables position differences.
	UPROPERTY(EditAnywhere, Category = "Lazy Evaluation", meta = (DisplayName = "Position Threshold"))
	float LazyEvaluationPositionThreshold;

	// The delta threshold for a rotation difference (in degrees). 0.0 disables rotation differences.
	UPROPERTY(EditAnywhere, Category = "Lazy Evaluation", meta = (DisplayName = "Rotation Threshold"))
	float LazyEvaluationRotationThreshold;

	// The delta threshold for a scale difference. 0.0 disables scale differences.
	UPROPERTY(EditAnywhere, Category = "Lazy Evaluation", meta = (DisplayName = "Scale Threshold"))
	float LazyEvaluationScaleThreshold;

	/** When checked the transforms are reset before a tick / update of the rig */
	UPROPERTY(EditAnywhere, Category = "Animation")
	bool bResetTransformBeforeTick;

	/** When checked the initial transforms on bones, nulls and controls are reset prior to a construction event */
	UPROPERTY(EditAnywhere, Category = "Animation")
	bool bResetInitialsBeforeConstruction;

	/** When checked this ensures to run the rig's update on the component's tick automatically */
	UPROPERTY(EditAnywhere, Category = "Animation")
	bool bUpdateRigOnTick;

	/** When checked the rig is run in the editor viewport without running / simulation the game */
	UPROPERTY(EditAnywhere, Category = "Animation")
	bool bUpdateInEditor;

	/** When checked the rig's bones are drawn using debug drawing similar to the animation editor viewport */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	bool bDrawBones;

	/** When checked the rig's debug drawing instructions are drawn in the viewport */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	bool bShowDebugDrawing;

	/**
	 * Returns all of the names for a given element type (Bone, Control, etc)
	 * @param ElementType The type of elements to return the names for
	 *
	 * @return all of the names for a given element type (Bone, Control, etc)
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API TArray<FName> GetElementNames(ERigElementType ElementType = ERigElementType::Bone);

	/**
	 * Returns true if an element given a type and name exists in the rig
	 * @param Name The name for the element to look up
	 * @param ElementType The type of element to look up
	 *
	 * @return true if the element exists
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API bool DoesElementExist(FName Name, ERigElementType ElementType = ERigElementType::Bone);

	/** 
	 * Returns the transform of the bone in the requested space 
	 * @param BoneName The name of the bone to retrieve the transform for
	 * @param Space The space to retrieve the transform in
	 *
	 * @return the transform of the bone in the requested space
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FTransform GetBoneTransform(FName BoneName, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Returns the initial transform of the bone in the requested space 
	 * @param BoneName The name of the bone to retrieve the transform for
	 * @param Space The space to retrieve the transform in
	 *
	 * @return the initial transform of the bone in the requested space
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FTransform GetInitialBoneTransform(FName BoneName, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Sets the transform of the bone in the requested space 
	 * @param BoneName The name of the bone to set the transform for
	 * @param Space The space to set the transform in
	 * @param Weight The weight of how much the change should be applied (0.0 to 1.0)
	 * @param bPropagateToChildren If true the child bones will be moved together with the affected bone
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetBoneTransform(FName BoneName, FTransform Transform, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace, float Weight = 1.f, bool bPropagateToChildren = true);

	/** 
	 * Sets the initial transform of the bone in the requested space 
	 * @param BoneName The name of the bone to set the transform for
	 * @param InitialTransform The initial transform to set for the bone
	 * @param Space The space to set the transform in
	 * @param bPropagateToChildren If true the child bones will be moved together with the affected bone
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetInitialBoneTransform(FName BoneName, FTransform InitialTransform, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace, bool bPropagateToChildren = false);

	/** 
	 * Returns the value of a bool control 
	 * @param ControlName The name of the control to retrieve the value for
	 *
	 * @return The bool value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API bool GetControlBool(FName ControlName);

	/** 
	 * Returns the value of a float control 
	 * @param ControlName The name of the control to retrieve the value for
	 *
	 * @return The float value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API float GetControlFloat(FName ControlName);

	/** 
	 * Returns the value of an integer control 
	 * @param ControlName The name of the control to retrieve the value for
	 *
	 * @return The int32 value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API int32 GetControlInt(FName ControlName);

	/** 
	 * Returns the value of a Vector3D control 
	 * @param ControlName The name of the control to retrieve the value for
	 *
	 * @return The Vector3D value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FVector2D GetControlVector2D(FName ControlName);

	/** 
	 * Returns the value of a position control 
	 * @param ControlName The name of the control to retrieve the value for
	 * @param Space The space to retrieve the control's value in
	 *
	 * @return The position value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FVector GetControlPosition(FName ControlName, EControlRigComponentSpace Space = EControlRigComponentSpace::LocalSpace);

	/** 
	 * Returns the value of a rotator control 
	 * @param ControlName The name of the control to retrieve the value for
	 * @param Space The space to retrieve the control's value in
	 *
	 * @return The rotator value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FRotator GetControlRotator(FName ControlName, EControlRigComponentSpace Space = EControlRigComponentSpace::LocalSpace);

	/** 
	 * Returns the value of a scale control 
	 * @param ControlName The name of the control to retrieve the value for
	 * @param Space The space to retrieve the control's value in
	 *
	 * @return The scale value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FVector GetControlScale(FName ControlName, EControlRigComponentSpace Space = EControlRigComponentSpace::LocalSpace);

	/** 
	 * Returns the value of a transform control 
	 * @param ControlName The name of the control to retrieve the value for
	 * @param Space The space to retrieve the control's value in
	 *
	 * @return The transform value of the control
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FTransform GetControlTransform(FName ControlName, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Sets the value of a bool control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlBool(FName ControlName, bool Value);

	/** 
	 * Sets the value of a float control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlFloat(FName ControlName, float Value);

	/** 
	 * Sets the value of an integer control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlInt(FName ControlName, int32 Value);

	/** 
	 * Sets the value of a vector2D control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlVector2D(FName ControlName, FVector2D Value);

	/** 
	 * Sets the value of a position control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 * @param Space The space to set the value in
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlPosition(FName ControlName, FVector Value, EControlRigComponentSpace Space = EControlRigComponentSpace::LocalSpace);

	/** 
	 * Sets the value of a rotator control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 * @param Space The space to set the value in
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlRotator(FName ControlName, FRotator Value, EControlRigComponentSpace Space = EControlRigComponentSpace::LocalSpace);

	/** 
	 * Sets the value of a scale control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 * @param Space The space to set the value in
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlScale(FName ControlName, FVector Value, EControlRigComponentSpace Space = EControlRigComponentSpace::LocalSpace);

	/** 
	 * Sets the value of a transform control 
	 * @param ControlName The name of the control to set
	 * @param Value The new value for the control
	 * @param Space The space to set the value in
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlTransform(FName ControlName, FTransform Value, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);


	/** 
	 * Returns the offset transform of a control 
	 * @param ControlName The name of the control to retrieve the offset transform for
	 * @param Space The space to retrieve the offset transform in
	 *
	 * @return The offset transform of a control 
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API FTransform GetControlOffset(FName ControlName, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Sets the offset transform of a control 
	 * @param ControlName The name of the control to set
	 * @param OffsetTransform The new offset trasnform for the control
	 * @param Space The space to set the offset transform in
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlOffset(FName ControlName, FTransform OffsetTransform, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Returns the transform of the space in the requested space 
	 * @param SpaceName The name of the space to retrieve the transform for
	 * @param Space The space to retrieve the transform in
	 *
	 * @return the transform of the space in the requested space
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FTransform GetSpaceTransform(FName SpaceName, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Returns the initial transform of the space in the requested space 
	 * @param SpaceName The name of the space to retrieve the transform for
	 * @param Space The space to retrieve the transform in
	 *
	 * @return the initial transform of the space in the requested space
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig")
	CONTROLRIG_API FTransform GetInitialSpaceTransform(FName SpaceName, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	/** 
	 * Sets the transform of the space in the requested space 
	 * @param SpaceName The name of the space to set the transform for
	 * @param Space The space to set the transform in
	 */
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetInitialSpaceTransform(FName SpaceName, FTransform InitialTransform, EControlRigComponentSpace Space = EControlRigComponentSpace::RigSpace);

	DECLARE_EVENT_OneParam(UControlRigComponent, FControlRigComponentEvent, class UControlRigComponent*);
	FControlRigComponentEvent& OnControlRigCreated() { return ControlRigCreatedEvent; }
	public:
		
	CONTROLRIG_API void SetControlRig(UControlRig* ControlRig);

	UFUNCTION(BlueprintCallable, Category = "ControlRig", meta=(DeprecatedFunction, DeprecationMessage="Use SetControlRigAssetReference instead."))
	CONTROLRIG_API void SetControlRigClass(TSubclassOf<UControlRig> InControlRigClass);
	
	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetControlRigAssetReference(FControlRigAssetStrongReference InControlRigAssetReference);

	UFUNCTION(BlueprintCallable, Category = "ControlRig")
	CONTROLRIG_API void SetObjectBinding(UObject* InObjectToBind);

private:

	struct FCachedSkeletalMeshComponentSettings
	{
		UClass* AnimInstanceClass;
		UClass* AnimClass;
		bool bCanEverTick;

		FCachedSkeletalMeshComponentSettings()
		{
			AnimInstanceClass = AnimClass = nullptr;
			bCanEverTick = false;
		}

		FCachedSkeletalMeshComponentSettings(USkeletalMeshComponent* InComponent)
		{
			AnimInstanceClass = nullptr;
			if (UAnimInstance* AnimInstance = InComponent->GetAnimInstance())
			{
				AnimInstanceClass = AnimInstance->GetClass();
			}
			AnimClass = InComponent->GetAnimClass();
			bCanEverTick = InComponent->PrimaryComponentTick.bCanEverTick;
		}

		void Apply(USkeletalMeshComponent* InComponent)
		{
				InComponent->SetAnimInstanceClass(AnimInstanceClass);
				InComponent->PrimaryComponentTick.bCanEverTick = bCanEverTick;
		}
	};

	CONTROLRIG_API UControlRig* SetupControlRigIfRequired();
	CONTROLRIG_API void ValidateMappingData();
	CONTROLRIG_API void TransferInputs();
	CONTROLRIG_API void TransferOutputs();
	static CONTROLRIG_API FName GetComponentNameWithinActor(UActorComponent* InComponent);

	CONTROLRIG_API void HandleControlRigInitializedEvent(URigVMHost* InControlRig, const FName& InEventName);
	CONTROLRIG_API void HandleControlRigPreConstructionEvent(UControlRig* InControlRig, const FName& InEventName);
	CONTROLRIG_API void HandleControlRigPostConstructionEvent(UControlRig* InControlRig, const FName& InEventName);
	CONTROLRIG_API void HandleControlRigPreForwardsSolveEvent(UControlRig* InControlRig, const FName& InEventName);
	CONTROLRIG_API void HandleControlRigPostForwardsSolveEvent(UControlRig* InControlRig, const FName& InEventName);
	CONTROLRIG_API void HandleControlRigExecutedEvent(URigVMHost* InControlRig, const FName& InEventName);

	CONTROLRIG_API void ConvertTransformToRigSpace(FTransform& InOutTransform, EControlRigComponentSpace FromSpace);
	CONTROLRIG_API void ConvertTransformFromRigSpace(FTransform& InOutTransform, EControlRigComponentSpace ToSpace);

	CONTROLRIG_API bool EnsureCalledOutsideOfBracket(const TCHAR* InCallingFunctionName = nullptr);
	CONTROLRIG_API void ReportError(const FString& InMessage);

public:
	
	static CONTROLRIG_API bool HasTickDependency(const UObject* InDependent, const UObject* InDependency, FString* OutErrorMessage = nullptr);
	static CONTROLRIG_API bool AllowsTickDependency(const UObject* InDependent, const UObject* InDependency, FString* OutErrorMessage = nullptr);

private:
	
	static CONTROLRIG_API bool HasTickDependency_Impl(const UObject* InDependent, const UObject* InDependency, TSet<const UObject*>& InOutVisited, FString* OutErrorMessage);

#if WITH_EDITOR
	static CONTROLRIG_API TMap<FString, TSharedPtr<SNotificationItem>> EditorNotifications;
	TSet<FString> TickDependencyErrorMessages;
#endif

	UPROPERTY(transient)
	TObjectPtr<UControlRig> ControlRig;
	
	UE::Anim::FMeshAttributeContainer TempAttributeContainer;

	TMap<USkeletalMeshComponent*, FCachedSkeletalMeshComponentSettings> CachedSkeletalMeshComponentSettings;

	FControlRigComponentEvent ControlRigCreatedEvent;
	bool bIsInsideInitializeBracket;
	bool bNeedsEvaluation;

	TArray<int32> InputElementIndices;
	TArray<FTransform> InputTransforms;
	TArray<FTransform> LastInputTransforms;

	TSharedPtr<IControlRigObjectBinding> ObjectBinding;

	bool bNeedToInitialize;

	friend class FControlRigSceneProxy;
};

class FControlRigSceneProxy : public FPrimitiveSceneProxy
{
public:

	CONTROLRIG_API SIZE_T GetTypeHash() const override;
	CONTROLRIG_API FControlRigSceneProxy(const UControlRigComponent* InComponent);

	CONTROLRIG_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	/**
	*  Returns a struct that describes to the renderer when to draw this proxy.
	*	@param		Scene view to use to determine our relevence.
	*  @return		View relevance struct
	*/
	CONTROLRIG_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	CONTROLRIG_API virtual uint32 GetMemoryFootprint(void) const override;
	CONTROLRIG_API uint32 GetAllocatedSize(void) const;

private:

	const UControlRigComponent* ControlRigComponent;
};
