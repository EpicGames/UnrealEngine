// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ControlRig.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Rigs/RigControlHierarchy.h"

#include "ControlRigControlActor.generated.h"

#define UE_API CONTROLRIG_API

UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Control Display Actor"))
class AControlRigControlActor : public AActor
{
	GENERATED_BODY()

public:

	UE_API AControlRigControlActor(const FObjectInitializer& ObjectInitializer);
	UE_API virtual ~AControlRigControlActor() override;
	// AACtor overrides
	UE_API virtual void Tick(float DeltaSeconds) override;
	virtual void BeginPlay() override { Clear(); }
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }
	virtual bool IsSelectable() const override { return bIsSelectable; }
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Control Actor")
	TObjectPtr<class AActor> ActorToTrack;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Control Actor", meta = (DisallowCreateNew))
	TSubclassOf<UControlRig> ControlRigClass;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Control Actor")
	bool bRefreshOnTick;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Control Actor")
	bool bIsSelectable;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Materials")
	TObjectPtr<UMaterialInterface> MaterialOverride;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Materials")
	FString ColorParameter;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Interp, Category = "Materials")
	bool bCastShadows;

	UFUNCTION(BlueprintCallable, Category = "Control Actor", DisplayName="Reset")
	UE_API void ResetControlActor();
	
	UFUNCTION(BlueprintCallable, Category = "Control Actor")
	UE_API void Clear();

	UFUNCTION(BlueprintCallable, Category = "Control Actor")
	UE_API void Refresh();

private:

	UPROPERTY()
	TObjectPtr<class USceneComponent> ActorRootComponent;

	UPROPERTY(transient)
	TSoftObjectPtr<UControlRig>  ControlRig;

	UPROPERTY(transient)
	TArray<FName> ControlNames;

	UPROPERTY(transient)
	TArray<FTransform> ShapeTransforms;

	UPROPERTY(transient)
	TArray<TObjectPtr<USceneComponent>> Components;

	UPROPERTY(transient)
	TArray<TObjectPtr<UMaterialInstanceDynamic>> Materials;

	UPROPERTY(transient)
	FName ColorParameterName;

private:
	UE_API void RemoveUnbindDelegate() const;
	
	UE_API void HandleControlRigUnbind();
};

#undef UE_API
