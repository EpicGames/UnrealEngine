// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyPathHelpers.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyCache.h"
#include "Stats/StatsHierarchical.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "ControlRigDefines.generated.h"

#define DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
//#define DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT() \
//	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#define UE_CONTROLRIG_LLM_SCOPE_BYNAME(InLabel)
//#define UE_CONTROLRIG_LLM_SCOPE_BYNAME(InLabel) \
//	LLM_SCOPE_BYNAME(TEXT(InLabel));

UENUM()
enum class ETransformSpaceMode : uint8
{
	/** Apply in parent space */
	LocalSpace,

	/** Apply in rig space*/
	GlobalSpace,

	/** Apply in Base space */
	BaseSpace,

	/** Apply in base bone */
	BaseJoint,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

UENUM()
enum class ETransformGetterType : uint8
{
	Initial,
	Current,
	Max UMETA(Hidden),
};

// thought of mixing this with execution on
// the problem is execution on is transient state, and 
// this execution type is something to be set per rig
UENUM()
enum class ERigExecutionType : uint8
{
	Runtime,
	Editing, // editing time
	Max UMETA(Hidden),
};
