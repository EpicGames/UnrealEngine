// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Engine/StaticMesh.h"
#include "IViewportSelectableObject.h"
#include "Components/PrimitiveComponent.h"
#include "Materials/Material.h"
#include "Rigs/RigControlHierarchy.h"
#include "ControlRigGizmoActor.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
struct FActorSpawnParameters;

USTRUCT()
struct FControlShapeActorCreationParam
{
	GENERATED_BODY()

	FControlShapeActorCreationParam()
		: ManipObj(nullptr)
		, ControlRigIndex(INDEX_NONE)
		, ControlRig(nullptr)
		, ControlName(NAME_None)
		, ShapeName(NAME_None)
		, SpawnTransform(FTransform::Identity)
		, ShapeTransform(FTransform::Identity)
		, MeshTransform(FTransform::Identity)
		, Material(nullptr)
		, ColorParameterName(NAME_None)
		, Color(FLinearColor::Red)
		, bSelectable(true)
		, HoveredParameterName(NAME_None)
		, HoveredColorParameterName(NAME_None)
		, HoveredColor(1.f,1.f,0.f, 0.25f)
	{
	}

	UObject*	ManipObj;
	int32		ControlRigIndex;
	UControlRig* ControlRig;
	FName		ControlName;
	FName		ShapeName;
	FTransform	SpawnTransform;
	FTransform  ShapeTransform;
	FTransform  MeshTransform;
	TSoftObjectPtr<UObject> Proxy = nullptr;
	TSoftObjectPtr<UMaterial> Material;
	FName ColorParameterName;
	FLinearColor Color;
	bool bSelectable;
	FName HoveredParameterName;
	FName HoveredColorParameterName;
	FLinearColor HoveredColor;
	TFunction<void(AControlRigShapeActor*)> PostSetupFunction;
};

/** Defines the proxy component update type when calling GetProxyComponent. */
enum class EProxyUpdateType : uint8
{
	None, // no update happened
	Created, // a new proxy component has been created
	Modified, // the proxy component has been modified 
	Removed // the proxy component has been removed 
};

/** An actor used to represent a rig control */
UCLASS(MinimalAPI, NotPlaceable, Transient)
class AControlRigShapeActor : public AActor, public IViewportSelectableObject
{
	GENERATED_BODY()

public:
	UE_API AControlRigShapeActor(const FObjectInitializer& ObjectInitializer);

	// this is the one holding transform for the controls
	UPROPERTY()
	TObjectPtr<class USceneComponent> ActorRootComponent;

	// this is visual representation of the transform
	UPROPERTY(Category = Proxy, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UPrimitiveComponent> ProxyComponent = nullptr;

	// the name of the control this actor is referencing
	UPROPERTY()
	uint32 ControlRigIndex;

	//  control rig this actor is referencing we can have multiple control rig's visible
	UPROPERTY()
	TWeakObjectPtr<UControlRig> ControlRig;

	// the name of the control this actor is referencing
	UPROPERTY()
	FName ControlName;

	// the name of the shape to use on this actor
   	UPROPERTY()
   	FName ShapeName;

	// the name of the color parameter on the material
	UPROPERTY()
	FName ColorParameterName;

	FLinearColor OverrideColor;

	FTransform OffsetTransform;

	UFUNCTION(BlueprintSetter)
	/** Set the control to be enabled/disabled */
	virtual void SetEnabled(bool bInEnabled) { SetSelectable(bInEnabled); }

	UFUNCTION(BlueprintGetter)
	/** Get whether the control is enabled/disabled */
	virtual bool IsEnabled() const { return IsSelectable(); }

	UFUNCTION(BlueprintSetter)
	/** Set the control to be selected/unselected */
	UE_API virtual void SetSelected(bool bInSelected);

	/** Get whether the control is selected/unselected */
	UFUNCTION(BlueprintGetter)
	UE_API virtual bool IsSelectedInEditor() const;

	/** Get wether the control is selectable/unselectable */
	UE_API virtual bool IsSelectable() const;

	UFUNCTION(BlueprintSetter)
	/** Set the control to be selected/unselected */
	UE_API virtual void SetSelectable(bool bInSelectable);

	UFUNCTION(BlueprintSetter)
	/** Set the control to be hovered */
	UE_API virtual void SetHovered(bool bInHovered);

	UFUNCTION(BlueprintGetter)
	/** Get whether the control is hovered */
	UE_API virtual bool IsHovered() const;
	
	/** changes the hovered color */
	UE_API virtual void SetHoveredColor(const FLinearColor& InColor, const bool bForce = false);

	/* Returns the key of the control element this shape actor represents */
	FRigElementKey GetElementKey() const { return FRigElementKey(ControlName, ERigElementType::Control); }

	/** Called from the edit mode each tick */
	virtual void TickControl() {};

	/** changes the shape color */
	UE_API virtual void SetShapeColor(const FLinearColor& InColor, const bool bForce = false);

	/** Event called when the transform of this control has changed */
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnTransformChanged(const FTransform& NewTransform);

	/** Event called when the enabled state of this control has changed */
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnEnabledChanged(bool bIsEnabled);

	/** Event called when the selection state of this control has changed */
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnSelectionChanged(bool bIsSelected);

	/** Event called when the hovered state of this control has changed */
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnHoveredChanged(bool bIsSelected);

	/** Event called when the manipulating state of this control has changed */
	UFUNCTION(BlueprintImplementableEvent)
	UE_API void OnManipulatingChanged(bool bIsManipulating);

	// this returns root component transform based on attach
	// when there is no attach, it is based on 0
	UFUNCTION(BlueprintCallable, Category = "ControlRig|Shape")
	UE_API void SetGlobalTransform(const FTransform& InTransform);

	UFUNCTION(BlueprintPure, Category = "ControlRig|Shape")
	UE_API FTransform GetGlobalTransform() const;

	// try to update the actor with the latest settings
	UE_API bool UpdateControlSettings(
		ERigHierarchyNotification InNotif,
		UControlRig* InControlRig,
		const FRigControlElement* InControlElement,
		bool bHideManipulators,
		bool bIsInLevelEditor);

	UPROPERTY()
	FCachedRigElement CachedIndex;

	// the name of the hovered parameter on the material
	UPROPERTY()
	FName HoveredParameterName = NAME_None;
	
	// the name of the hovered color parameter on the material
	UPROPERTY()
	FName HoveredColorParameterName = NAME_None;

	// Returns the proxy component for the form proxy object provided.
	// The component will be either created, updated, or removed depending on current component's type/state. (use OutUpdate to track changes)
	template<typename ComponentType = UPrimitiveComponent>
	ComponentType* GetProxyComponent(UObject* InShapeProxy, EProxyUpdateType* OutUpdate = nullptr)
	{
		const EProxyUpdateType& ProxyUpdate = EnsureProxyComponent(InShapeProxy);
		if (OutUpdate)
		{
			(*OutUpdate) = ProxyUpdate;
		}
		return Cast<ComponentType>(ProxyComponent.Get());
	}
	
private:

	/** Whether this control is selected */
	UPROPERTY(BlueprintGetter = IsSelectedInEditor, BlueprintSetter = SetSelected, Category = "ControlRig|Shape")
	uint8 bSelected : 1;

	/** Whether this control is hovered */
	UPROPERTY(BlueprintGetter = IsHovered, BlueprintSetter = SetHovered, Category = "ControlRig|Shape")
	uint8 bHovered : 1;

	/** Whether or not this control is selectable*/
	uint8 bSelectable : 1;

	/** Current color used to avoid color updates when unnecessary */
	FLinearColor CurrentColor;

	/** Current hovered color used to avoid updates when unnecessary */
	FLinearColor HoveredColor = FLinearColor(1.f,1.f,0.f, 0.25f);
	
	// Create or update a shape proxy component given the provided shape proxy object.
	// If the shape proxy object is not supported, the current proxy component will be removed.
	UE_API EProxyUpdateType EnsureProxyComponent(UObject* InShapeProxy);
	
	// Removes the current shape proxy component.
	EProxyUpdateType RemoveProxyComponent();

	// Allocate a new shape proxy component.
	EProxyUpdateType AllocateProxyComponent(UObject* InShapeProxy);
	
	// Updates the current shape proxy component.
	EProxyUpdateType UpdateProxyComponent(UObject* InShapeProxy) const;
	
	// Returns the first material instance of the proxy component.
	UMaterialInstanceDynamic* GetProxyMaterial() const;
};

/**
 * Creating Shape Param helper functions
 */
namespace FControlRigShapeHelper
{
	extern CONTROLRIG_API AControlRigShapeActor* CreateShapeActor(UWorld* InWorld, UStaticMesh* InStaticMesh, const FControlShapeActorCreationParam& CreationParam);
	AControlRigShapeActor* CreateShapeActor(UWorld* InWorld, TSubclassOf<AControlRigShapeActor> InClass, const FControlShapeActorCreationParam& CreationParam);
	extern CONTROLRIG_API AControlRigShapeActor* CreateDefaultShapeActor(UWorld* InWorld, const FControlShapeActorCreationParam& CreationParam);

	const FActorSpawnParameters& GetDefaultSpawnParameter();
}

#undef UE_API
