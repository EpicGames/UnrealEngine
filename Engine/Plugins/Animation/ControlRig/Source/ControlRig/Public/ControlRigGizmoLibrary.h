// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Engine/StaticMesh.h"

#include "ControlRigGizmoLibrary.generated.h"

class AControlRigShapeActor;
class UControlRigShapeLibrary;
class UMaterial;

USTRUCT(BlueprintType, meta = (DisplayName = "Shape"))
struct FControlRigShapeDefinition
{
	GENERATED_USTRUCT_BODY()

	FControlRigShapeDefinition()
	{
		ShapeName = TEXT("Default");
		Transform = FTransform::Identity;
	}

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Shape")
	FName ShapeName;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use ShapeProxy instead."))
	TSoftObjectPtr<UStaticMesh> StaticMesh_DEPRECATED;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Shape")
	FTransform Transform;
	
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Shape",  meta = (AllowedClasses = "/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh"))
	TSoftObjectPtr<UObject> ShapeProxy = nullptr;
	
	mutable TWeakObjectPtr<UControlRigShapeLibrary> Library;

	// Optional function to support an external setup post-process 
	TFunction<void(AControlRigShapeActor*)> PostSetupFunction;
  
	// Returns the shape proxy object. (forcing the loading of the soft object pointer if bLoadPending is true)
	template<typename ProxyType = UObject>
	ProxyType* GetShapeProxy(const bool bLoadPending = true) const
	{
		return Cast<ProxyType>(GetShapeProxyObject(bLoadPending));
	}
	
private:
	
	// Returns the shape proxy object. (forcing the loading of the soft object pointer if bLoadPending is true)
	CONTROLRIG_API UObject* GetShapeProxyObject(const bool bLoadPending = true) const;
};

UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Control Rig Shape Library"))
class UControlRigShapeLibrary : public UObject
{
	GENERATED_BODY()

public:

	CONTROLRIG_API UControlRigShapeLibrary();

	UPROPERTY(EditAnywhere, Category = "ShapeLibrary")
	FControlRigShapeDefinition DefaultShape;

	UPROPERTY(EditAnywhere, Category = "ShapeLibrary", meta = (DisplayName = "Override Material"))
	TSoftObjectPtr<UMaterial> DefaultMaterial;

	UPROPERTY(EditAnywhere, Category = "ShapeLibrary")
	TSoftObjectPtr<UMaterial> XRayMaterial;

	UPROPERTY(EditAnywhere, Category = "ShapeLibrary")
	FName MaterialColorParameter;

	UPROPERTY(EditAnywhere, Category = "ShapeLibrary")
	FName MaterialHoveredParameter = NAME_None;
	
	UPROPERTY(EditAnywhere, Category = "ShapeLibrary")
	FName MaterialHoveredColorParameter = NAME_None;
	
	UPROPERTY(EditAnywhere, Category = "ShapeLibrary")
	TArray<FControlRigShapeDefinition> Shapes;

	CONTROLRIG_API const FControlRigShapeDefinition* GetShapeByName(const FName& InName, bool bUseDefaultIfNotFound = false) const;
	static CONTROLRIG_API const FControlRigShapeDefinition* GetShapeByName(const FName& InName, const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& InShapeLibraries, const TMap<FString, FString>& InLibraryNameMap, bool bUseDefaultIfNotFound = true);
	static CONTROLRIG_API const FString GetShapeName(const UControlRigShapeLibrary* InShapeLibrary, bool bUseNameSpace, const TMap<FString, FString>& InLibraryNameMap, const FControlRigShapeDefinition& InShape);

#if WITH_EDITOR

	// UObject interface
	CONTROLRIG_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

#endif
	
	// UObject interface
	virtual void PostLoad() override; 

private:

	const TArray<FName> GetUpdatedNameList(bool bReset = false);

	TArray<FName> NameList;
};

/**
 * FControlRigShapeProxyComponentProvider contains object classes that support primitive components to represent a control shape
 * This allows other types than UStaticMesh and USkeletalMesh to be used a control shapes.
 */

class FShapeProxyComponentProviderRegistry
{
public:

	~FShapeProxyComponentProviderRegistry() = default;
	
	// Get the singleton registry object.
	static CONTROLRIG_API FShapeProxyComponentProviderRegistry& Get();
	
	// Creation function returning a new component given a proxy object
	using FCreateFunction = TUniqueFunction<UPrimitiveComponent* (UObject*, UObject*)>;
	
	// Update function updating a component given a proxy object
	using FUpdateFunction = TUniqueFunction<bool (UObject*, UObject*)>;
	
	// Registers a particular ProxyType object subclass to provide a ComponentType primitive component subclass.
	template<typename ProxyType, typename ComponentType>
	void RegisterProvider(FCreateFunction&& CreateFunction, FUpdateFunction&& UpdateFunction)
	{
		static_assert(TIsDerivedFrom<ProxyType, UObject>::Value,
			"The template class ProxyType must be a subclass of UObject.");
		
		static_assert(TIsDerivedFrom<ComponentType, UPrimitiveComponent>::Value,
			"The template class ComponentType must be a subclass of UPrimitiveComponent.");
		
		ShapeProxyToComponent.Emplace(ProxyType::StaticClass(), ComponentType::StaticClass());
		ProxyComponentFunctions.Emplace(ProxyType::StaticClass(), {MoveTemp(CreateFunction), MoveTemp(UpdateFunction)});
	}
	
	// Whether this proxy class is supported to represent a control shape.
	bool IsShapeProxySupported(const UClass* InProxyClass) const;
	
	// Get the component class associated with this proxy class if any.
	const UClass* GetProxyComponentClass(const UClass* InProxyClass) const;
	
	// Create a new component given the proxy object if supported.
	UPrimitiveComponent* GetNewProxyComponent(UObject* InProxy, UObject* InOuter) const;
	
	// Update the component with the proxy object if supported.
	bool UpdateProxyComponent(UObject* InProxy, UObject* InComponent) const;

	// Registers proxy objects supported by default (UStaticMesh and USkeletalMesh for now).
	static void RegisterDefaultProviders();

private:
	FShapeProxyComponentProviderRegistry() = default;
	
	// Functions associated with a specific proxy object.
	struct FProxyFunctions
	{
		FCreateFunction CreateFunction;
		FUpdateFunction UpdateFunction;
	};
	TMap<UClass*, FProxyFunctions> ProxyComponentFunctions;
	
	// Components associated with a specific proxy object.
	TMap<UClass*, UClass*> ShapeProxyToComponent;

	// Track proxy classes that have been queried but are not currently supported.
	mutable TSet<const UClass*> QueriedUnregisteredClass;
	void LogOnce(const UClass* InClass) const;
	static void LogNull();
};

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Materials/Material.h"
#endif
