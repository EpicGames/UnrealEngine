// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Metadata used by the control rig system
namespace ControlRigMetadata
{
	// Metadata usable in USTRUCT for customizing the behavior when displaying the property in a property panel or graph node
	enum
	{
		/// [StructMetaData] Indicates that a control rig unit struct cannot be instantiated
		Abstract,

		/// [StructMetaData] The name used for a control rig node's title
		DisplayName,

		/// [StructMetaData] Whether to show the variable name in the node's title bar
		ShowVariableNameInTitle,
	};

	// Metadata usable in UPROPERTY for customizing the behavior when displaying the property in a property panel or graph node
	enum
	{
		/// [PropertyMetadata] Indicates that the property should be exposed as an input for a control rig
		Input,

		/// [PropertyMetadata] Indicates that the property should be exposed as an output for a control rig
		Output,

		/// [PropertyMetadata] !!!EDITORONLY!!! Indicates that the property should have access to source property (backward chain)
		// when this is set, you can chase back and override the source property if it's normal property or control rig unit
		// for example if you have property A link to your rig unit's input B (A->B), if B has this meta data, you can modify A 
		AllowSourceAccess,
	};
}