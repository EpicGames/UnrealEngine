// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "IControlRigObjectBinding.h"

#define UE_API CONTROLRIG_API

class USceneComponent;

class FControlRigObjectBinding : public IControlRigObjectBinding
{
public:
	
	UE_API virtual ~FControlRigObjectBinding();

	// IControlRigObjectBinding interface
	UE_API virtual void BindToObject(UObject* InObject) override;
	UE_API virtual void UnbindFromObject() override;
	virtual FControlRigBind& OnControlRigBind() override { return ControlRigBind; }
	virtual FControlRigUnbind& OnControlRigUnbind() override { return ControlRigUnbind; }
	UE_API virtual bool IsBoundToObject(UObject* InObject) const override;
	UE_API virtual UObject* GetBoundObject() const override;
	UE_API virtual AActor* GetHostingActor() const override;

	static UE_API UObject* GetBindableObject(UObject* InObject);
private:
	/** The scene component or USkeleton we are bound to */
	TWeakObjectPtr<UObject> BoundObject;

	FControlRigBind ControlRigBind;
	FControlRigUnbind ControlRigUnbind;
};

#undef UE_API
