// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"

#define UE_API CONTROLRIG_API

// Custom serialization version for changes made in Dev-Anim stream
struct FControlRigObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,

		// Added execution pins and removed hierarchy ref pins
		RemovalOfHierarchyRefPins,

		// Refactored operators to store FCachedPropertyPath instead of string
		OperatorsStoringPropertyPaths,

		// Introduced new RigVM as a backend
		SwitchedToRigVM,

		// Added a new transform as part of the control
		ControlOffsetTransform,

		// Using a cache data structure for key indices now
		RigElementKeyCache,

		// Full variable support
		BlueprintVariableSupport,

		// Hierarchy V2.0
		RigHierarchyV2,

		// RigHierarchy to support multi component parent constraints
		RigHierarchyMultiParentConstraints,

		// RigHierarchy now supports space favorites per control
		RigHierarchyControlSpaceFavorites,

		// RigHierarchy now stores min and max values as float storages 
		StorageMinMaxValuesAsFloatStorage,

		// RenameGizmoToShape 
		RenameGizmoToShape,

		// BoundVariableWithInjectionNode 
		BoundVariableWithInjectionNode,

		// Switch limit control over to per channel limits 
		PerChannelLimits,

		// Removed the parent cache for multi parent elements 
		RemovedMultiParentParentCache,

		// Deprecation of parameters
		RemoveParameters,

		// Added rig curve element value state flag
		CurveElementValueStateFlag,

		// Added the notion of a per control animation type
		ControlAnimationType,

		// Added preferred permutation for templates
		TemplatesPreferredPermutatation,

		// Added preferred euler angles to controls
		PreferredEulerAnglesForControls,

		// Added rig hierarchy element metadata
		HierarchyElementMetadata,

		// Converted library nodes to templates
		LibraryNodeTemplates,

		// Controls to be able specify space switch targets
		RestrictSpaceSwitchingForControls,

		// Controls to be able specify which channels should be visible in sequencer
		ControlTransformChannelFiltering,

		// Store function information (and compilation data) in blueprint generated class
		StoreFunctionsInGeneratedClass,

		// Hierarchy storing previous names
		RigHierarchyStoringPreviousNames,

		// Control supporting preferred rotation order
		RigHierarchyControlPreferredRotationOrder,

		// Last bit required for Control supporting preferred rotation order
		RigHierarchyControlPreferredRotationOrderFlag,

		// Element metadata is now stored on URigHierarchy, rather than FRigBaseElement
		RigHierarchyStoresElementMetadata,

		// Add type (primary, secondary) and optional bool to FRigConnectorSettings
		ConnectorsWithType,

		// Add parent key to control rig pose
		RigPoseWithParentKey,

		// Physics solvers stored on hierarchy
		ControlRigStoresPhysicsSolvers,

		// Moved the element storage into separate buffers
		RigHierarchyIndirectElementStorage,

		// Compress the rig hierarchy when storing to disk
		RigHierarchyCompressElements,

		// Added the notion of components to the rig hierarchy
		RigHierarchyStoresComponents,

		// Improve transform compactness when serializing the hierarchy
		RigHierarchyCompactTransformSerialization,

		// Connectors to support arrays
		RigHierarchyArrayConnectors,

		// Parent constraints offering a display label
		RigHierarchyParentContraintWithLabel,

		// Previous name and parent maps serialized as FRigHierarchyKey
		RigHierarchyPreviousNameAndParentMapUsingHierarchyKey,

		// New setting for connectors to optionally specify their use only during post construction
		RigHierarchyPostConstructionConnectors,

		// Overrides store TOC data for properties to solidify loading of data when the definition has changed
		OverridesStoreTOCDataForProperties,

		// Overrides the skip offset as int64 - previous versions stored it as int32
		OverridesStoreDatSkipOffsetAsInt64,

		// Overrides now only store the path to the leaf as well as the details about the leaf property
		OverridesStorePathAndLeafPropertyOnly,

		// Overrides now only store the hash for validation, not the size since size can change without changing the payload (containers), also starts including hash for maps and sets
		OverridesStoreLeafPropertyHashOnly,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FControlRigObjectVersion() {}
};

#undef UE_API
