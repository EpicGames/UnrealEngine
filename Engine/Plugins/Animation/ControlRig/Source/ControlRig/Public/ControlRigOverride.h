// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/CityHash.h"
#include "UObject/WeakFieldPtr.h"
#include "RigVMStringUtils.h"
#include "HAL/IConsoleManager.h"
#include "Misc/OutputDevice.h"
#include "Logging/LogVerbosity.h"
#include "ControlRigOverride.generated.h"

#define UE_API CONTROLRIG_API

#if WITH_EDITOR
class FPropertyPath;
#endif

class UControlRig;

extern CONTROLRIG_API TAutoConsoleVariable<bool> CVarControlRigEnableOverrides;

class FControlRigOverrideValueErrorPipe : public FOutputDevice
{
public:

	typedef  TFunction<void(const TCHAR*, ELogVerbosity::Type)> TReportFunction;

	FControlRigOverrideValueErrorPipe( ELogVerbosity::Type InMaxVerbosity = ELogVerbosity::Warning, TReportFunction InReportFunction = nullptr )
		: FOutputDevice()
		, NumErrors(0)
		, MaxVerbosity(InMaxVerbosity)
		, ReportFunction(InReportFunction)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if(Verbosity <= MaxVerbosity)
		{
			NumErrors++;
			if(ReportFunction)
			{
				ReportFunction(V, Verbosity);
			}
		}
	}

	ELogVerbosity::Type GetMaxVerbosity() const { return MaxVerbosity; }
	int32 GetNumErrors() const { return NumErrors; }

private:
	int32 NumErrors;
	ELogVerbosity::Type MaxVerbosity;
	TReportFunction ReportFunction;
};

/**
 * A single value used to represent an override on a subject.
 * The value serialized the data using binary serialization based on the last
 * property in the property chain.
 * The value can be copied on to the subject or from the subject
 * as well as copied from and to string.
 * This data-structure is not thread-safe for writing.
 */
USTRUCT(BlueprintType)
struct FControlRigOverrideValue
{
	GENERATED_BODY()

public:

	static inline const FString PathSeparator = TEXT("->");
	static inline constexpr int32 PathSeparatorLength = 2;
	static inline constexpr TCHAR ArraySeparator = TEXT('[');

	FControlRigOverrideValue()
	{}

	// constructor given the path, the owning structure as well as the container
	UE_API FControlRigOverrideValue(const FString& InPath, const UStruct* InOwnerStruct, const void* InSubjectPtr, const FName& InSubjectKey = NAME_None);

	// constructor given the path, the owning structure as well as the value as string
	UE_API FControlRigOverrideValue(const FString& InPath, const UStruct* InOwnerStruct, const FString& InValueAsString, const FName& InSubjectKey = NAME_None, const FControlRigOverrideValueErrorPipe::TReportFunction& InReportFunction = nullptr);

	UE_API FControlRigOverrideValue(const FControlRigOverrideValue& InOther);

	UE_API ~FControlRigOverrideValue();

	UE_API FControlRigOverrideValue& operator=(const FControlRigOverrideValue& InOther);

	UE_API bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FControlRigOverrideValue& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	// returns true if this value is valid - meaning if the property chain is valid
	// and the value has been set
	UE_API bool IsValid() const;

	// resets the contents of this value
	UE_API void Reset();

	// returns the name of the subject
	UE_API const FName& GetSubjectKey() const;

	// returns the path this value represents
	UE_API const FString& GetPath() const;

	// returns the path this value was serialized with
	UE_API const FString& GetSerializedPath() const;
	
	// returns the raw data of this value
	UE_API void* GetData();
	
	// returns the raw data of this value
	UE_API const void* GetData() const;

	// returns the raw data of this value
	template<typename T>
	T* GetData()
	{
		return reinterpret_cast<T*>(GetData());
	}

	// returns the raw data of this value
	template<typename T>
	const T* GetData() const
	{
		return reinterpret_cast<const T*>(GetData());
	}

	// returns the value of this override as a string (or an empty string)  
	UE_API const FString& ToString() const;

	// sets the contents of this value from string
	UE_API bool SetFromString(const FString& InValue, const FControlRigOverrideValueErrorPipe::TReportFunction& InReportFunction = nullptr);

	// copies the override value onto a container
	[[nodiscard]] UE_API bool CopyToSubject(void* InSubjectPtr, const UStruct* InSubjectStruct = nullptr) const;

	// copies the value from a container into this override value
	[[nodiscard]] UE_API bool SetFromSubject(const void* InSubjectPtr, const UStruct* InSubjectStruct = nullptr);

	UE_API bool operator==(const FControlRigOverrideValue& InOtherValue) const;

	// returns true if the stored value matches the provided value
	UE_API bool Identical(const FControlRigOverrideValue& InOtherValue) const;

	// returns true if the stored value matches the provided value memory
	UE_API bool IdenticalValue(const void* InValuePtr) const;

	// returns true if the stored value matches the provided value memory
	UE_API bool IdenticalValueInSubject(const void* InSubjectPtr) const;

#if WITH_EDITOR
	// helper conversion used mainly for user interface code
	UE_API TSharedPtr<FPropertyPath> ToPropertyPath() const;
#endif

	// returns the root property represented by this value
	UE_API const FProperty* GetRootProperty() const;
	
	// returns the leaf property represented by this value (potentially leaf == root property)
	UE_API const FProperty* GetLeafProperty() const;
	UE_API FProperty* GetLeafProperty();

	// number of properties in the override
	int32 GetNumProperties() const;

	// returns a given property
	const FProperty* GetProperty(int32 InIndex) const;

	// returns the array index for a given property
	int32 GetArrayIndex(int32 InIndex) const;

	bool ContainsProperty(const FProperty* InProperty) const;

	/**
	 * converts a memory pointer from the subject to the leaf property.
	 * @param bResizeArrays If set to true the subject memory will be resized to fit the property chain
	 */
	UE_API uint8* SubjectPtrToValuePtr(const void* InSubjectPtr, bool bResizeArrays) const;

	// helper method to look up a property under a structure
	static UE_API FProperty* FindProperty(const UStruct* InStruct, const FString& InNameOrDisplayName);

	UE_API friend uint32 GetTypeHash(const FControlRigOverrideValue& InOverride);

	void AddStructReferencedObjects(FReferenceCollector& Collector) const;

	UE_API void GetPreloadDependencies(TArray<UObject*>& OutDeps);

private:

	static UE_API FProperty* FindProperty(const UStruct* InStruct, const FString& InNameOrDisplayName, bool& bWasRedirected);

	UE_API bool SetPropertiesFromPath(const FString& InPath, const UStruct* InOwnerStruct);
	UE_API FString GetPathFromProperties() const;
	UE_API void* AllocateDataIfRequired();
	UE_API void FreeDataIfRequired();
	UE_API void* AllocateDataIfRequired(TArray<uint8, TAlignedHeapAllocator<16>>& InOutDataArray, uint32& InOutPropertyHash) const;
	UE_API void FreeDataIfRequired(TArray<uint8, TAlignedHeapAllocator<16>>& InOutDataArray, uint32& InOutPropertyHash) const;
	UE_API void CopyValue(void* InDestPtr, const void* InSourcePtr) const;
	UE_API void UpdateHash();
	UE_API bool IsDataValid() const;
	UE_API bool IsDataValid(const TArray<uint8, TAlignedHeapAllocator<16>>& InDataArray, const uint32& InOutPropertyHash) const;
	UE_API uint32 GetPropertyHash() const;
	UE_API uint32 GetPropertyHash(const FProperty* InProperty) const;

	template<typename T>
	bool InitFromLegacyString(const FString& InLegacyString, const T& InDefault)
	{
		T Value = InDefault;
		if(Value.InitFromString(InLegacyString))
		{
			void* DataPtr = AllocateDataIfRequired(); 
			CopyValue(DataPtr, &Value);
			CachedStringValue = InLegacyString;
			UpdateHash();
			return true;
		}
		return false;
	}

	TOptional<FString> Path;
	TOptional<FString> SerializedPath;
	TOptional<FString> CachedStringValue;
	TOptional<FName> SubjectKey;
	uint32 Hash;

	struct FPropertyInfo
	{
		TWeakFieldPtr<FProperty> Property;
		int32 ArrayIndex = INDEX_NONE;
	};

	UE_API static FString GetPathFromProperties(const TArray<FPropertyInfo>& InProperties);
	
	TArray<FPropertyInfo> Properties;
	TArray<uint8, TAlignedHeapAllocator<16>> DataArray;
	uint32 DataPropertyHash;
	
	friend class UModularRigController;
};

template<>
struct TStructOpsTypeTraits<FControlRigOverrideValue> : public TStructOpsTypeTraitsBase2<FControlRigOverrideValue>
{
	enum 
	{
		WithSerializer = true,
		WithAddStructReferencedObjects = true,
		WithGetPreloadDependencies = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Soft;
};

/**
 * A container to represent a set of override values for one or more subjects.
 * The bUsesKeyForSubject setting will control if the subject name is respected when
 * adding / looking up / applying overrides.
 * This data-structure is not thread-safe for writing.
 */
USTRUCT(BlueprintType)
struct FControlRigOverrideContainer
{
	GENERATED_BODY()

public:

	FControlRigOverrideContainer()
		: bUsesKeyForSubject(true)
	{}

	bool IsEmpty() const { return Values.IsEmpty(); }
	int32 Num() const { return Values.Num(); }
	bool IsValidIndex(int32 InIndex) const { return Values.IsValidIndex(InIndex); }
	FControlRigOverrideValue& operator[](int32 InIndex) { return Values[InIndex]; }
	const FControlRigOverrideValue& operator[](int32 InIndex) const { return Values[InIndex]; }
	TArray<FControlRigOverrideValue>::RangedForIteratorType begin() { return Values.begin(); }
	TArray<FControlRigOverrideValue>::RangedForIteratorType end() { return Values.end(); }
	TArray<FControlRigOverrideValue>::RangedForConstIteratorType begin() const { return Values.begin(); }
	TArray<FControlRigOverrideValue>::RangedForConstIteratorType end() const { return Values.end(); }

	UE_API void Reset();
	UE_API void Empty();
	UE_API void Reserve(int32 InNum);
	UE_API bool UsesKeyForSubject() const;
	UE_API void SetUsesKeyForSubject(bool InUsesKeyForSubject);

	UE_API int32 Add(const FControlRigOverrideValue& InValue);
	UE_API const FControlRigOverrideValue* FindOrAdd(const FControlRigOverrideValue& InValue);
	UE_API bool Remove(const FControlRigOverrideValue& InValue);
	UE_API bool Remove(const FString& InPath, const FName& InSubjectKey = NAME_None);
	UE_API bool RemoveAll(const FName& InSubjectKey = NAME_None);

	int32 Emplace(const FString& InPath, const UStruct* InOwnerStruct, const void* InSubjectPtr, const FName& InSubjectKey = NAME_None)
	{
		const FControlRigOverrideValue Value(InPath, InOwnerStruct, InSubjectPtr, InSubjectKey);
		return Add(Value);
	}
	
	int32 Emplace(const FString& InPath, const UStruct* InOwnerStruct, const FString& InValueAsString, const FName& InSubjectKey = NAME_None)
	{
		const FControlRigOverrideValue Value(InPath, InOwnerStruct, InValueAsString, InSubjectKey);
		return Add(Value);
	}

	UE_API int32 GetIndex(const FString& InPath, const FName& InSubjectKey = NAME_None) const;
	UE_API const TArray<int32>* GetIndicesForSubject(const FName& InSubjectKey) const;
	UE_API const FControlRigOverrideValue* Find(const FString& InPath, const FName& InSubjectKey = NAME_None) const;
	UE_API FControlRigOverrideValue* Find(const FString& InPath, const FName& InSubjectKey = NAME_None);
	UE_API const FControlRigOverrideValue& FindChecked(const FString& InPath, const FName& InSubjectKey = NAME_None) const;
	UE_API FControlRigOverrideValue& FindChecked(const FString& InPath, const FName& InSubjectKey = NAME_None);

	UE_API TArray<FName> GenerateSubjectArray() const;

	UE_API bool Contains(const FString& InPath, const FName& InSubjectKey = NAME_None) const;
	UE_API bool ContainsParentPathOf(const FString& InChildPath, const FName& InSubjectKey = NAME_None) const;
	UE_API bool ContainsChildPathOf(const FString& InParentPath, const FName& InSubjectKey = NAME_None) const;
	UE_API bool ContainsAnyPathForSubject(const FName& InSubjectKey) const;
	UE_API bool ContainsPathForAnySubject(const FString& InPath) const;

	UE_API bool Contains(const FControlRigOverrideValue& InOverrideValue) const;
	UE_API bool ContainsParentPathOf(const FControlRigOverrideValue& InOverrideValue) const;
	UE_API bool ContainsChildPathOf(const FControlRigOverrideValue& InOverrideValue) const;

	// copies the override values onto a subject.
	UE_API void CopyToSubject(void* InSubjectPtr, const UStruct* InSubjectStruct = nullptr, const FName& InSubjectKey = NAME_None) const;

	// copies the value from the subject into this container
	UE_API void SetFromSubject(const void* InSubjectPtr, const UStruct* InSubjectStruct = nullptr, const FName& InSubjectKey = NAME_None);

	UE_API bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FControlRigOverrideContainer& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	void AddStructReferencedObjects(FReferenceCollector& Collector);

	UE_API void GetPreloadDependencies(TArray<UObject*>& OutDeps);

	bool Identical(const FControlRigOverrideContainer* Other, uint32 PortFlags) const;

	// returns a parent path of a given child path, separators used are -> and [,
	// so parent of 'Values[1]->Flag' is 'Values[1]',
	// and parent of 'Values[1]' is 'Values'.
	static UE_API FString GetParentPath(const FString& InChildPath);

	// returns true if a given child path is a child of a given parent path 
	static UE_API bool IsChildPathOf(const FString& InChildPath, const FString& InParentPath);

	UE_API friend uint32 GetTypeHash(const FControlRigOverrideContainer& InContainer);

private:

	UE_API void RebuildLookup();
	UE_API void InvalidateCache() const;
	
	inline uint32 GetLookupHash(const FString& InPath, const FName& InSubjectKey) const
	{
		const uint32 PathHash = CityHash32(
			(const char*)InPath.GetCharArray().GetData(),
			sizeof(InPath[0]) * InPath.GetCharArray().Num());
		return HashCombineFast(GetTypeHash(InSubjectKey), PathHash);
	}

	TArray<FControlRigOverrideValue> Values;

	bool bUsesKeyForSubject;
	TMap<uint32, int32> HashIndexLookup;
	TMap<FName, TArray<int32>> SubjectIndexLookup;
	mutable TMap<uint32, bool> ContainsParentPathCache; 
	mutable TMap<uint32, bool> ContainsChildPathCache; 
};

template<>
struct TStructOpsTypeTraits<FControlRigOverrideContainer> : public TStructOpsTypeTraitsBase2<FControlRigOverrideContainer>
{
	enum 
	{
		WithSerializer = true,
		WithIdentical = true,
		WithAddStructReferencedObjects = true,
		WithGetPreloadDependencies = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Soft;
};

class UControlRigOverrideAsset;
DECLARE_EVENT_OneParam(UControlRigOverrideAsset, FControlRigOverrideChanged, const UControlRigOverrideAsset*);

UCLASS(MinimalAPI, BlueprintType)
class UControlRigOverrideAsset : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Overrides")
	FControlRigOverrideContainer Overrides;

#if WITH_EDITOR
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif

	UE_API static UControlRigOverrideAsset* CreateOverrideAsset(const FString& InLongName);
	UE_API static UControlRigOverrideAsset* CreateOverrideAssetInDeveloperFolder(const UControlRig* InSubject);

	FControlRigOverrideChanged& OnChanged() { return OverrideChangedDelegate; }
	UE_API void BroadcastChanged();

private:
	FControlRigOverrideChanged OverrideChangedDelegate;
};

template<typename T>
class TControlRigOverrideHandle : public TSharedFromThis<TControlRigOverrideHandle<T>>
{
public:
	
	TControlRigOverrideHandle(UControlRigOverrideAsset* InOverrideAsset, int32 InIndex)
		: WeakOverrideAsset(InOverrideAsset)
		, OverrideIndex(InIndex)
		, LeafProperty(nullptr)
	{
		if(UControlRigOverrideAsset* OverrideAsset = GetOverrideAsset())
		{
			if(OverrideAsset->Overrides.IsValidIndex(OverrideIndex))
			{
				LeafProperty = OverrideAsset->Overrides[OverrideIndex].GetLeafProperty();
			}
		}
	}

	bool IsValid() const
	{
		if(UControlRigOverrideAsset* OverrideAsset = GetOverrideAsset())
		{
			if(OverrideAsset->Overrides.IsValidIndex(OverrideIndex))
			{
				return OverrideAsset->Overrides[OverrideIndex].GetLeafProperty() == LeafProperty;
			}
		}
		return false;
	}

	UControlRigOverrideAsset* GetOverrideAsset() const
	{
		if(WeakOverrideAsset.IsValid())
		{
			return WeakOverrideAsset.Get();
		}
		return nullptr;
	}

	const FProperty* GetLeafProperty() const
	{
		return LeafProperty;
	}

	const T* GetData() const
	{
		if(!IsValid())
		{
			return nullptr;
		}
		return GetOverrideAsset()->Overrides[OverrideIndex].template GetData<T>();
	}

	T* GetData()
    {
    	if(!IsValid())
    	{
    		return nullptr;
    	}
    	return GetOverrideAsset()->Overrides[OverrideIndex].template GetData<T>();
    }

private:

	TWeakObjectPtr<UControlRigOverrideAsset> WeakOverrideAsset;
	int32 OverrideIndex;
	const FProperty* LeafProperty;
};;

#undef UE_API
