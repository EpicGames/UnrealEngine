// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/StatsMisc.h"

#define UE_API CONTROLRIG_API

struct FControlRigStats
{
	bool Enabled;
	TArray<FName> Stack;
	// Values are heap-allocated so that `double&` returned from RetainCounter remains valid
	// when later inserts cause the TMap to rehash. A scoped counter may outlive subsequent
	// FindOrAdd calls; storing the double inline would dangle on rehash.
	TMap<FName, TUniquePtr<double>> Counters;

	FControlRigStats()
	: Enabled(false)
	{

	}

	static UE_API FControlRigStats& Get();
	UE_API void Clear();
	UE_API double& RetainCounter(const TCHAR* Key);
	UE_API double& RetainCounter(const FName& Key);
	UE_API void ReleaseCounter();
	UE_API void Dump();
};

struct FControlRigSimpleScopeSecondsCounter : public FSimpleScopeSecondsCounter
{
public:
	UE_API FControlRigSimpleScopeSecondsCounter(const TCHAR* InName);
	UE_API ~FControlRigSimpleScopeSecondsCounter();
};

#if STATS
#if WITH_EDITOR
#define CONTROLRIG_SCOPE_SECONDS_COUNTER(Name) \
	FControlRigSimpleScopeSecondsCounter ANONYMOUS_VARIABLE(ControlRigScopeSecondsCounter)(TEXT(#Name))
#else
#define CONTROLRIG_SCOPE_SECONDS_COUNTER(Name)
#endif
#endif

#undef UE_API
