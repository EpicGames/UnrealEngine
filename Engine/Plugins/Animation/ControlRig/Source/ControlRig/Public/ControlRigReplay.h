// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyPose.h"
#include "Logging/TokenizedMessage.h"
#include "Tracks/SampleTrackContainer.h"

#include "ControlRigReplay.generated.h"

#define UE_API CONTROLRIG_API

class UControlRigReplay;
class UControlRig;

USTRUCT(BlueprintType)
struct FControlRigReplayVariable
{
	GENERATED_USTRUCT_BODY()

	FControlRigReplayVariable()
	{
		Name = NAME_None;
		CPPType = NAME_None;
		Value = FString();
	}

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigReplayVariable")
	FName Name;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigReplayVariable")
	FName CPPType;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigReplayVariable")
	FString Value;
};

UENUM()
enum class EControlRigReplayPlaybackMode : uint8
{
	Live,
	ReplayInputs,
	GroundTruth,
	Max UMETA(Hidden),
};

USTRUCT()
struct FControlRigReplayTracks : public FSampleTrackHost
{
public:
	GENERATED_BODY()

	typedef TFunction<void(EMessageSeverity::Type,const FName&,const FString&)> TReportFunction;
	
	virtual ~FControlRigReplayTracks() override = default;

	UE_API virtual bool Serialize(FArchive& InArchive) override;

	UE_API virtual void Reset() override;
	UE_API bool IsEmpty() const;

	UE_API void StoreRigVMEvent(const FName& InEvent);
	UE_API FName GetRigVMEvent(int32 InTimeIndex) const;
	UE_API void StoreInteraction(uint8 InInteractionMode, const TArray<FRigElementKey>& InElementsBeingInteracted);
	UE_API TTuple<uint8, TArray<FRigElementKey>> GetInteraction(int32 InTimeIndex) const;

	UE_API void StoreHierarchy(
		URigHierarchy* InHierarchy,
		const TArrayView<FRigElementKey>& InKeys = TArrayView<FRigElementKey>(),
		bool bStorePose = true,
		bool bStoreComponents = true,
		bool bStoreMetadata = true);
	UE_API bool RestoreHierarchy(
		int32 InTimeIndex,
		URigHierarchy* InHierarchy,
		const TArrayView<FRigElementKey>& InKeys = TArrayView<FRigElementKey>(),
		TReportFunction InReportFunction = nullptr,
		bool bRestorePose = true,
		bool bRestoreComponents = true,
		bool bRestoreMetadata = true) const;

	UE_API void StorePose(
		URigHierarchy* InHierarchy,
		const TArrayView<FRigElementKey>& InKeys = TArrayView<FRigElementKey>());
	UE_API bool RestorePose(
		int32 InTimeIndex,
		URigHierarchy* InHierarchy,
		const TArrayView<FRigElementKey>& InKeys = TArrayView<FRigElementKey>(),
		TReportFunction InReportFunction = nullptr) const;

	UE_API void StoreComponents(
		URigHierarchy* InHierarchy,
		const TArrayView<FRigComponentKey>& InKeys = TArrayView<FRigComponentKey>());
	UE_API bool RestoreComponents(
		int32 InTimeIndex,
		URigHierarchy* InHierarchy,
		const TArrayView<FRigComponentKey>& InKeys = TArrayView<FRigComponentKey>(),
		TReportFunction InReportFunction = nullptr) const;

	UE_API void StoreMetaData(
		URigHierarchy* InHierarchy,
		const TArrayView<FRigElementKey>& InKeys = TArrayView<FRigElementKey>());
	UE_API bool RestoreMetaData(
		int32 InTimeIndex,
		URigHierarchy* InHierarchy,
		const TArrayView<FRigElementKey>& InKeys = TArrayView<FRigElementKey>(),
		TReportFunction InReportFunction = nullptr) const;

	UE_API void StoreVariables(URigVMHost* InHost);
	UE_API bool RestoreVariables(int32 InTimeIndex, URigVMHost* InHost, TReportFunction InReportFunction = nullptr) const;

private:
	UE_API const FName& GetTrackName(const FRigElementKey& InElementKey) const;
	UE_API const FName& GetTrackName(const FRigComponentKey& InComponentKey) const;
	UE_API const FName& GetTrackName(const FRigElementKey& InElementKey, const FName& InMetadataName) const;
	UE_API const FName& GetTrackName(const FProperty* InProperty) const;
	static UE_API FSampleTrackBase::ETrackType GetTrackTypeFromMetadataType(ERigMetadataType InMetadataType);
	static UE_API ERigMetadataType GetMetadataTypeFromTrackType(FSampleTrackBase::ETrackType InTrackType);
	static UE_API FSampleTrackBase::ETrackType GetTrackTypeFromProperty(const FProperty* InProperty) ;

	UE_API const TArray<FRigElementKey> GetElementKeys() const;

	UE_API void StorePose(URigHierarchy* InHierarchy, FRigBaseElement* InElement);
	UE_API bool RestorePose(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, TReportFunction InReportFunction) const;

	UE_API void StoreComponents(URigHierarchy* InHierarchy, FRigBaseElement* InElement);
	UE_API bool RestoreComponents(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, TReportFunction InReportFunction) const;

	UE_API void StoreComponent(URigHierarchy* InHierarchy, FRigBaseComponent* InComponent);
	UE_API bool RestoreComponent(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseComponent* InComponent, TReportFunction InReportFunction) const;

	UE_API void StoreMetaData(URigHierarchy* InHierarchy, FRigBaseElement* InElement);
	UE_API bool RestoreMetaData(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, TReportFunction InReportFunction) const;
	UE_API const TArray<FName> GetMetadataNames(int32 InTimeIndex, FSampleTrackIndex& OutSampleTrackIndex, const FRigElementKey& InElementKey, TReportFunction InReportFunction = nullptr) const;

	UE_API void StoreMetaData(URigHierarchy* InHierarchy, FRigBaseElement* InElement, FRigBaseMetadata* InMetadata);
	UE_API bool RestoreMetaData(int32 InTimeIndex, URigHierarchy* InHierarchy, FRigBaseElement* InElement, const FName& InMetadataName, TReportFunction InReportFunction) const;

	static UE_API bool ForEachElement(URigHierarchy* InHierarchy, const TArrayView<FRigElementKey>& InKeys, TFunction<void(FRigBaseElement*, bool&)> InFunction);
	static UE_API bool ForEachComponent(URigHierarchy* InHierarchy, const TArrayView<FRigComponentKey>& InKeys, TFunction<void(FRigBaseComponent*, bool&)> InFunction);

	static UE_API void FilterElementKeys(TArray<FRigElementKey>& InOutElementKeys);

	static inline FLazyName HierarchyTopologyHashName = FLazyName(TEXT("HierarchyTopologyHash"));
	static inline FLazyName HierarchyTopologyParentIndicesName = FLazyName(TEXT("HierarchyTopologyParentIndices"));
	mutable TMap<FRigElementKey, FName> ElementKeyToTrackName;
	mutable TMap<FRigComponentKey, FName> ComponentKeyToTrackName;
	mutable TMap<TTuple<FRigElementKey, FName>, FName> MetadataToTrackName;
	mutable TMap<FName, FName> PropertyNameToTrackName;
	mutable FSampleTrackIndex SampleTrackIndex;
	bool bIsInput = true;

	static inline const FLazyName RigVMEventName = FLazyName(TEXT("RigVMEvent"));
	static inline const FLazyName InteractionTypeName = FLazyName(TEXT("InteractionType"));
	static inline const FLazyName ElementsBeingInteractedName = FLazyName(TEXT("ElementsBeingInteracted"));
	static inline const FLazyName TopologyHashName = FLazyName(TEXT("TopologyHash"));
	static inline const FLazyName MetadataVersionName = FLazyName(TEXT("MetadataVersion"));
	static inline const FLazyName ElementKeysName = FLazyName(TEXT("ElementKeys"));
	static inline const FLazyName ParentIndicesName = FLazyName(TEXT("ParentIndices"));
	static inline const FLazyName VariableNamesName = FLazyName(TEXT("VariableNames"));

	friend class UControlRigReplay;
};

template<>
struct TStructOpsTypeTraits<FControlRigReplayTracks> : public TStructOpsTypeTraitsBase2<FControlRigReplayTracks>
{
	enum 
	{
		WithSerializer = true, // struct has a Serialize function for serializing its state to an FArchive.
		WithCopy = true, // struct can be copied via its copy assignment operator.
		WithIdentical = true, // struct can be compared via an Identical(const T* Other, uint32 PortFlags) function.
	};
};

UCLASS(MinimalAPI, BlueprintType)
class UControlRigReplay : public UObject
{
	GENERATED_BODY()

public:

	UControlRigReplay()
		: Tolerance(0.001)
		, bValidateHierarchyTopology(true)
		, bValidatePose(true)
		, bValidateMetadata(true)
		, bValidateVariables(true)
		, EnableTest(true)
		, TimeAtStartOfRecording(0.0)
		, TimeOfLastFrame(0.0)
		, DesiredRecordingDuration(-1.0)
	{
		InputTracks.bIsInput = true;
		OutputTracks.bIsInput = false;
	}

	UE_API virtual void BeginDestroy() override;

	UE_API virtual void Serialize(FArchive& Ar) override;

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	static UE_API UControlRigReplay* CreateNewAsset(FString InDesiredPackagePath, FString InBlueprintPathName, UClass* InAssetClass);

	UFUNCTION(BlueprintPure, Category = "ControlRigReplay")
	UE_API virtual FVector2D GetTimeRange() const;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay", meta=(MultiLine=true))
   	FText Description;

	UPROPERTY(AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	FSoftObjectPath ControlRigObjectPath;

	UPROPERTY(AssetRegistrySearchable, VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	FSoftObjectPath PreviewSkeletalMeshObjectPath;

	UPROPERTY(EditAnywhere, Category = "ControlRigReplay")
	FControlRigReplayTracks InputTracks;

	UPROPERTY(EditAnywhere, Category = "ControlRigReplay")
	FControlRigReplayTracks OutputTracks;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	double Tolerance;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	bool bValidateHierarchyTopology;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	bool bValidatePose;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	bool bValidateMetadata;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	bool bValidateVariables;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay")
	TArray<int32> FramesToSkip;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigReplay", AssetRegistrySearchable)
	bool EnableTest;

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	UE_API virtual bool StartRecording(UControlRig* InControlRig);

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	UE_API virtual bool StopRecording();

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	UE_API virtual bool StartReplay(UControlRig* InControlRig, EControlRigReplayPlaybackMode InMode = EControlRigReplayPlaybackMode::ReplayInputs);

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	UE_API virtual bool StopReplay();

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	UE_API virtual bool PauseReplay();

	UFUNCTION(BlueprintPure, Category = "ControlRigReplay")
	UE_API virtual bool IsReplaying() const;

	UFUNCTION(BlueprintPure, Category = "ControlRigReplay")
	UE_API virtual bool IsPaused() const;

	UFUNCTION(BlueprintPure, Category = "ControlRigReplay")
	UE_API virtual bool IsRecording() const;

	UFUNCTION(BlueprintPure, Category = "ControlRigReplay")
	UE_API EControlRigReplayPlaybackMode GetPlaybackMode() const;

	UFUNCTION(BlueprintCallable, Category = "ControlRigReplay")
	UE_API void SetPlaybackMode(EControlRigReplayPlaybackMode InMode);

	bool IsTestEnabled() const
	{
		return EnableTest;
	}

	UE_API virtual bool IsValidForTesting() const;
	UE_API virtual bool HasValidationErrors() const;
	UE_API virtual const TArray<FString>& GetValidationErrors() const;

	UE_API virtual bool PerformTest(UControlRig* InSubject, TFunction<void(EMessageSeverity::Type, const FString&)> InLogFunction) const;
	UE_API virtual bool ValidateExpectedResults(
		int32 InPlaybackTimeIndex,
		FSampleTrackIndex& OutSampleTrackIndex,
		UControlRig* InSubject,
		TFunction<void(EMessageSeverity::Type, const FString&)> InLogFunction) const;

protected:

	UE_API void ClearDelegates(UControlRig* InControlRig);

	UE_API void HandlePreconstructionForTest(UControlRig* InRig, const FName& InEventName) const;

	TWeakObjectPtr<UControlRig> ReplayControlRig;
	TWeakObjectPtr<UControlRig> RecordControlRig;
	FDelegateHandle PreConstructionHandle;
	FDelegateHandle PreEventHandle;
	FDelegateHandle PostEventHandle;
	double TimeAtStartOfRecording;
	double TimeOfLastFrame;
	double DesiredRecordingDuration;
	bool bStoreVariablesDuringPreEvent = false;
	bool bReplayPaused = false;
	EControlRigReplayPlaybackMode PlaybackMode = EControlRigReplayPlaybackMode::Live;
	mutable TArray<FString> LastValidationWarningsAndErrors;

	static UE_API const FText LiveStatus; 
	static UE_API const FText LiveStatusTooltip; 
	static UE_API const FText ReplayInputsStatus;
	static UE_API const FText ReplayInputsStatusTooltip;
	static UE_API const FText GroundTruthStatus;
	static UE_API const FText GroundTruthStatusTooltip;

	friend class FControlRigBaseEditor;
};

#undef UE_API
