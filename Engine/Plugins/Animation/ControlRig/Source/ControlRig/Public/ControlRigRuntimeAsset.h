// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRig.h"
#include "ModularRigModel.h"
#include "RigVMRuntimeAsset.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigModuleDefines.h"

#include "ControlRigRuntimeAsset.generated.h"

#define UE_API CONTROLRIG_API


UINTERFACE(MinimalAPI, BlueprintType)
class UControlRigRuntimeAssetInterface : public UInterface
{
	GENERATED_BODY()
};

class IControlRigRuntimeAssetInterface
{
	GENERATED_BODY()
	
public:
	UE_API static TScriptInterface<IControlRigRuntimeAssetInterface> GetInterfaceOuter(const UObject* InObject);
	
	virtual FControlRigAssetStrongReference GetControlRigAssetReference() = 0;
	virtual URigHierarchy* GetHierarchy() = 0;
	virtual void SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy) = 0;
	virtual FModularRigSettings& GetModularRigSettings() = 0;
	virtual FRigHierarchySettings& GetHierarchySettings() = 0;
	virtual FRigModuleSettings& GetRigModuleSettings() = 0;
	virtual TArray<FModuleReferenceData>& GetModuleReferenceData() = 0;
	virtual TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() = 0;
	const TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() const { return const_cast<IControlRigRuntimeAssetInterface*>(this)->GetArrayConnectionMap(); }
	virtual FRigInfluenceMapPerEvent& GetInfluences() = 0;
	virtual FModularRigModel& GetModularRigModel() = 0;
	const FModularRigModel& GetModularRigModel() const { return const_cast<IControlRigRuntimeAssetInterface*>(this)->GetModularRigModel(); }
	virtual void SetControlRigType(EControlRigType& InType) = 0;
	virtual EControlRigType GetControlRigType() const = 0;
	virtual void SetItemTypeDisplayName(FName& InName) = 0;
	virtual FName GetItemTypeDisplayName() const = 0;
	virtual bool& GetSupportsInversion() = 0;
	virtual bool& GetSupportsControls() = 0;
	virtual TSoftObjectPtr<UObject>& GetSourceHierarchyImport() = 0;
	virtual TSoftObjectPtr<UObject>& GetSourceCurveImport() = 0;
	virtual bool& GetExposesAnimatableControls() = 0;
	virtual bool& GetAllowMultipleInstances() = 0;
	virtual FRigElementKeyRedirector& GetElementKeyRedirector() = 0;
	virtual TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() = 0;
	virtual TSoftObjectPtr<USkeletalMesh> GetPreviewSkeletalMesh() const = 0;
	virtual void SetPreviewSkeletalMesh(TSoftObjectPtr<USkeletalMesh> InMesh) = 0;
};

UCLASS(MinimalAPI, BlueprintType, meta=(IgnoreClassThumbnail))
class UControlRigRuntimeAsset : public URigVMRuntimeAsset, public IControlRigRuntimeAssetInterface
{
	GENERATED_BODY()
	
	
public:
	

	UE_API virtual FControlRigAssetStrongReference GetControlRigAssetReference() override;
	virtual UClass* GetRigVMHostClass() const override;
	bool IsRigModule() const { return ControlRigType == EControlRigType::RigModule; }
	bool IsModularRig() const { return ControlRigType == EControlRigType::ModularRig; }
	virtual FRigModuleSettings& GetRigModuleSettings() override { return RigModuleSettings; }
	virtual URigHierarchy* GetHierarchy() override { return Hierarchy; }
	virtual void SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy) override { Hierarchy = InHierarchy; }
	virtual FModularRigSettings& GetModularRigSettings() override { return ModularRigSettings; }
	virtual FRigHierarchySettings& GetHierarchySettings() override { return HierarchySettings; }
	virtual TArray<FModuleReferenceData>& GetModuleReferenceData() override { return ModuleReferenceData; }
	virtual TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() override { return ArrayConnectionMap; }
	virtual FRigInfluenceMapPerEvent& GetInfluences() override { return Influences; }
	virtual FModularRigModel& GetModularRigModel() override { return ModularRigModel; }
	virtual void SetControlRigType(EControlRigType& InType) override { ControlRigType = InType; }
	virtual EControlRigType GetControlRigType() const override { return ControlRigType; }
	virtual void SetItemTypeDisplayName(FName& InName) override { ItemTypeDisplayName = InName; }
	virtual FName GetItemTypeDisplayName() const override { return ItemTypeDisplayName; }
	virtual bool& GetSupportsControls() override { return bSupportsControls; }
	virtual bool& GetSupportsInversion() override { return bSupportsInversion; }
	virtual TSoftObjectPtr<UObject>& GetSourceCurveImport() override { return SourceCurveImport; }
	virtual TSoftObjectPtr<UObject>& GetSourceHierarchyImport() override { return SourceHierarchyImport; }
	virtual bool& GetExposesAnimatableControls() override { return bExposesAnimatableControls; }
	virtual bool& GetAllowMultipleInstances() override { return bAllowMultipleInstances; }
	virtual FRigElementKeyRedirector& GetElementKeyRedirector() override { return ElementKeyRedirector; }
	virtual TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() override { return ShapeLibraries; }
	virtual TSoftObjectPtr<USkeletalMesh> GetPreviewSkeletalMesh() const override { return PreviewSkeletalMesh; }
	virtual void SetPreviewSkeletalMesh(TSoftObjectPtr<USkeletalMesh> InMesh) override { PreviewSkeletalMesh = InMesh; }
	UE_API virtual void UpdateSupportedEventNames() override;
	
	virtual void SetCustomThumbnail(FString InThumbnail) { CustomThumbnail = InThumbnail; }
	virtual FString GetCustomThumbnail() const { return CustomThumbnail; }
	
	UE_API virtual void PostLoad() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Initialize(UClass* InEditorOnlyDataClass) override;
	UE_API virtual URigVMHost* InstantiateObject(UObject* InOuter, FName InName, EObjectFlags InFlags) override;
	UE_API virtual bool InitializeInstance(URigVMHost* InInstance) override;
	
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
#if WITH_EDITOR
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif
	
protected:
	UPROPERTY(BlueprintReadOnly, Category = "Hierarchy")
	TObjectPtr<URigHierarchy> Hierarchy;
	
	UPROPERTY(EditAnywhere, Category = "Hierarchy")
	FRigHierarchySettings HierarchySettings;
	
	UPROPERTY(BlueprintReadOnly, Category = "Modules")
	FModularRigModel ModularRigModel;
	
	UPROPERTY(EditAnywhere, Category = "Modules")
	FModularRigSettings ModularRigSettings;
	
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = "Modules")
	FRigModuleSettings RigModuleSettings;
	
	UPROPERTY(AssetRegistrySearchable)
	EControlRigType ControlRigType;
	
	UPROPERTY(AssetRegistrySearchable)
	FName ItemTypeDisplayName = TEXT("Control Rig");
	
	/** The skeleton from import into a hierarchy */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable, EditAnywhere, Category="Control Rig")
	TSoftObjectPtr<UObject> SourceHierarchyImport;

	/** The skeleton from import into a curve */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable, EditAnywhere, Category="Control Rig")
	TSoftObjectPtr<UObject> SourceCurveImport;
	
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category="Control Rig")
	TSoftObjectPtr<USkeletalMesh> PreviewSkeletalMesh;
	
	/** Asset searchable information module references in this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FModuleReferenceData> ModuleReferenceData;
	
	UPROPERTY()
	TMap<FRigElementKey, FRigElementKeyCollection> ArrayConnectionMap;
	
	UPROPERTY(BlueprintReadOnly, Category = "Influence Map")
	FRigInfluenceMapPerEvent Influences;
	
	/** Whether or not this rig has an Inversion Event */
	UPROPERTY(AssetRegistrySearchable)
	bool bSupportsInversion;

	/** Whether or not this rig has Controls on It */
	UPROPERTY(AssetRegistrySearchable)
	bool bSupportsControls;
	
	/** If set to true, this control rig has animatable controls */
	UPROPERTY(AssetRegistrySearchable)
	bool bExposesAnimatableControls;
	
	/** If set to true, multiple control rig tracks can be created for the same rig in sequencer*/
	UPROPERTY(EditAnywhere, Category="Sequencer", AssetRegistrySearchable)
	bool bAllowMultipleInstances = false;
	
	// This relates to FAssetThumbnailPool::CustomThumbnailTagName and allows
	// the thumbnail pool to show the thumbnail of the icon rather than the
	// rig itself to avoid deploying the 3D renderer.
	UPROPERTY(EditAnywhere, Category = "Hierarchy", AssetRegistrySearchable)
	FString CustomThumbnail;
	
	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries;
	
	FRigElementKeyRedirector ElementKeyRedirector;
	
	friend class UControlRigAssetFactory;
	friend class UControlRigBlueprint;
	friend class FControlRigAssetActions;
	friend class FControlRigBasicsConvertToAssetTest;
};


#undef UE_API