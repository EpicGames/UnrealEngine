// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigReplay.h"

#include "ControlRigTestData.generated.h"

#define UE_API CONTROLRIG_API

class UControlRigTestData;
class UControlRig;

USTRUCT(BlueprintType)
struct FControlRigTestDataFrame
{
	GENERATED_USTRUCT_BODY()

	FControlRigTestDataFrame()
	{
		AbsoluteTime = 0.0;
		DeltaTime = 0.0;
		Variables.Reset();
		Pose.Reset();
		bTestMetadata = true;
	}

	UE_API bool Store(UControlRig* InControlRig, bool bInitial = false);
	UE_API bool Restore(UControlRig* InControlRig, bool bInitial = false) const;
	UE_API bool RestoreVariables(UControlRig* InControlRig) const;
	UE_API bool RestoreMetadata(UControlRig* InControlRig) const;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestDataFrame")
	double AbsoluteTime;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestDataFrame")
	double DeltaTime;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestDataFrame")
	TArray<FControlRigReplayVariable> Variables;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestDataFrame")
	FRigPose Pose;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestDataFrame")
	TArray<uint8> Metadata;

	FCustomVersionContainer ArchiveCustomVersions;
	mutable TMap<FRigElementKey, URigHierarchy::FMetadataStorage> MetadataMap;
	mutable bool bTestMetadata;
};

UCLASS(MinimalAPI, BlueprintType)
class UControlRigTestData : public UControlRigReplay
{
	GENERATED_BODY()

public:

	UControlRigTestData()
		: LastFrameIndex(INDEX_NONE)
		, bIsApplyingOutputs(false)
	{}

	UE_API virtual void BeginDestroy() override;

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual FVector2D GetTimeRange() const override;

	UFUNCTION(BlueprintPure, Category = "ControlRigTestData")
	UE_API int32 GetFrameIndexForTime(double InSeconds, bool bInput = false) const;

	UE_API virtual bool StartRecording(UControlRig* InControlRig) override;
	UE_API virtual bool StopRecording() override;
	UE_API virtual bool StartReplay(UControlRig* InControlRig, EControlRigReplayPlaybackMode InMode = EControlRigReplayPlaybackMode::ReplayInputs) override;
	UE_API virtual bool StopReplay() override;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestData")
	FControlRigTestDataFrame Initial;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestData")
	TArray<FControlRigTestDataFrame> InputFrames;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "ControlRigTestData")
	TArray<FControlRigTestDataFrame> OutputFrames;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ControlRigTestData")
	TArray<FName> EventQueue;

	UE_API virtual bool IsValidForTesting() const override;
	
	UE_API virtual bool PerformTest(UControlRig* InSubject, TFunction<void(EMessageSeverity::Type, const FString&)> InLogFunction) const override;

protected:

	mutable int32 LastFrameIndex;
	bool bIsApplyingOutputs;
};

#undef UE_API
