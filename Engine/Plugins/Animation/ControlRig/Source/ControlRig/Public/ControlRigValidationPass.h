// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ControlRig.h"
#include "Logging/TokenizedMessage.h"

#include "ControlRigValidationPass.generated.h"

#define UE_API CONTROLRIG_API

class UControlRigValidationPass;

DECLARE_DELEGATE(FControlRigValidationClearDelegate);
DECLARE_DELEGATE_FourParams(FControlRigValidationReportDelegate, EMessageSeverity::Type, const FRigElementKey&, float, const FString&);
// todo DECLARE_DELEGATE_TwoParams(FControlRigValidationControlRigChangedDelegate, UControlRigValidator*, UControlRig*);

USTRUCT()
struct FControlRigValidationContext
{
	GENERATED_BODY()

public:

	FControlRigValidationContext();

	void Clear();
	void Report(EMessageSeverity::Type InSeverity, const FString& InMessage);
	void Report(EMessageSeverity::Type InSeverity, const FRigElementKey& InKey, const FString& InMessage);
	void Report(EMessageSeverity::Type InSeverity, const FRigElementKey& InKey, float InQuality, const FString& InMessage);
	
	FControlRigValidationClearDelegate& OnClear() { return ClearDelegate;  }
	FControlRigValidationReportDelegate& OnReport() { return ReportDelegate; }

	FRigVMDrawInterface* GetDrawInterface() { return DrawInterface; }

	FString GetDisplayNameForEvent(const FName& InEventName) const;

private:

	FControlRigValidationClearDelegate ClearDelegate;;
	FControlRigValidationReportDelegate ReportDelegate;

	FRigVMDrawInterface* DrawInterface;

	friend class UControlRigValidator;
};

/** Used to perform validation on a debugged Control Rig */
UCLASS(MinimalAPI)
class UControlRigValidator : public UObject
{
	GENERATED_UCLASS_BODY()

	UE_API UControlRigValidationPass* FindPass(UClass* InClass) const;
	UE_API UControlRigValidationPass* AddPass(UClass* InClass);
	UE_API void RemovePass(UClass* InClass);

	UControlRig* GetControlRig() const { return WeakControlRig.Get(); }
	UE_API void SetControlRig(UControlRig* InControlRig);

	FControlRigValidationClearDelegate& OnClear() { return ValidationContext.OnClear(); }
	FControlRigValidationReportDelegate& OnReport() { return ValidationContext.OnReport(); }
	// todo FControlRigValidationControlRigChangedDelegate& OnControlRigChanged() { return ControlRigChangedDelegate;  }

private:

	UPROPERTY()
	TArray<TObjectPtr<UControlRigValidationPass>> Passes;

	UE_API void OnControlRigInitialized(URigVMHost* Subject, const FName& EventName);
	UE_API void OnControlRigExecuted(URigVMHost* Subject, const FName& EventName);

	FControlRigValidationContext ValidationContext;
	TWeakObjectPtr<UControlRig> WeakControlRig;
	// todo FControlRigValidationControlRigChangedDelegate ControlRigChangedDelegate;
};


/** Used to perform validation on a debugged Control Rig */
UCLASS(MinimalAPI, Abstract)
class UControlRigValidationPass : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	// Called whenever the rig being validated question is changed
	virtual void OnSubjectChanged(UControlRig* InControlRig, FControlRigValidationContext* InContext) {}

	// Called whenever the rig in question is initialized
	virtual void OnInitialize(UControlRig* InControlRig, FControlRigValidationContext* InContext) {}

	// Called whenever the rig is running an event
	virtual void OnEvent(UControlRig* InControlRig, const FName& InEventName, FControlRigValidationContext* InContext) {}
};

#undef UE_API
