// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#if WITH_EDITOR
#include "PropertyPath.h"
#endif
#include "ControlRigDefines.h"
namespace UtilityHelpers
{
	template <typename Predicate>
	FName CreateUniqueName(const FName& InBaseName, Predicate IsUnique)
	{
		FName CurrentName = InBaseName;
		int32 CurrentIndex = 0;

		while (!IsUnique(CurrentName))
		{
			FString PossibleName = InBaseName.ToString() + TEXT("_") + FString::FromInt(CurrentIndex++);
			CurrentName = FName(*PossibleName);
		}

		return CurrentName;
	}

	template <typename Predicate>
	FTransform GetBaseTransformByMode(ETransformSpaceMode TransformSpaceMode, Predicate TransformGetter, const FRigElementKey& ParentKey, const FRigElementKey& BaseKey, const FTransform& BaseTransform)
	{
		switch (TransformSpaceMode)
		{
		case ETransformSpaceMode::LocalSpace:
		{
			return TransformGetter(ParentKey);
		}
		case ETransformSpaceMode::BaseSpace:
		{
			return BaseTransform;
		}
		case ETransformSpaceMode::BaseJoint:
		{
			return TransformGetter(BaseKey);
		}
		case ETransformSpaceMode::GlobalSpace:
		default:
		{
			return FTransform::Identity;
		}
		}
	}
}
