// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
/** Interface to allow control rigs to bind to external objects */
class IControlRigObjectBinding
{
public:
	/** Bind to a runtime object */
	virtual void BindToObject(UObject* InObject) = 0;

	/** Unbind from the current bound runtime object */
	virtual void UnbindFromObject() = 0;

	/** Bindable event for external objects to be notified that a Control has been bound */
	DECLARE_EVENT_OneParam(IControlRigObjectBinding, FControlRigBind, UObject*);
	virtual FControlRigBind& OnControlRigBind() = 0;

	/** Bindable event for external objects to be notified that a Control has been unbound */
	DECLARE_EVENT(IControlRigObjectBinding, FControlRigUnbind);
	virtual FControlRigUnbind& OnControlRigUnbind() = 0;

	/** 
	 * Check whether we are bound to the supplied object. 
	 * This can be distinct from a direct pointer comparison (e.g. in the case of an actor passed
	 * to BindToObject, we may actually bind to one of its components).
	 */
	virtual bool IsBoundToObject(UObject* InObject) const = 0;

	/** Get the current object we are bound to */
	virtual UObject* GetBoundObject() const = 0;

	/** Find the actor we are bound to, if any */
	virtual AActor* GetHostingActor() const = 0;
};