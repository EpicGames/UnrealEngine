// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigDefines.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"

#define UE_API CONTROLRIG_API

class FControlRigMathLibrary : public FRigVMMathLibrary
{
public:
	static UE_API void SolveBasicTwoBoneIK(FTransform& BoneA, FTransform& BoneB, FTransform& Effector, const FVector& PoleVector, const FVector& PrimaryAxis, const FVector& SecondaryAxis, float SecondaryAxisWeight, float BoneALength, float BoneBLength, bool bEnableStretch, float StretchStartRatio, float StretchMaxRatio);
};

#undef UE_API
