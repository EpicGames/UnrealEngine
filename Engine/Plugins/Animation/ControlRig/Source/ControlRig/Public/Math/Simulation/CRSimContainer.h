// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CRSimContainer.generated.h"

USTRUCT(meta = (Abstract))
struct FCRSimContainer
{
	GENERATED_BODY()

	FCRSimContainer()
	{
		TimeStep = 1.f / 60.f;
		AccumulatedTime = 0.f;
		TimeLeftForStep = 0.f;
	}
	virtual ~FCRSimContainer() = default;

	/**
	 * The time step used by this container
	 */
	UPROPERTY()
	float TimeStep;

	/**
	 * The time step used by this container
	 */
	UPROPERTY()
	float AccumulatedTime;

	/**
	 * The time left until the next step
	 */
	UPROPERTY()
	float TimeLeftForStep;

	virtual void Reset();
	virtual void ResetTime();
	virtual void StepVerlet(float InDeltaTime, float InBlend);
	virtual void StepSemiExplicitEuler(float InDeltaTime);

protected:
	virtual void CachePreviousStep() {};
	virtual void IntegrateVerlet(float InBlend) {};
	virtual void IntegrateSemiExplicitEuler() {};
};
