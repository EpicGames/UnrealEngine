// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "CRSimLinearSpring.generated.h"

USTRUCT(BlueprintType)
struct FCRSimLinearSpring
{
	GENERATED_BODY()

	FCRSimLinearSpring()
	{
		SubjectA = SubjectB = INDEX_NONE;
		Coefficient = 32.f;
		Equilibrium = -1.f;
	}

	/**
	 * The first point affected by this spring
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	int32 SubjectA;

	/**
	 * The second point affected by this spring
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	int32 SubjectB;

	/**
	 * The power of this spring
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float Coefficient;

	/**
	 * The rest length of this spring.
	 * A value of lower than zero indicates that the equilibrium
	 * should be based on the current distance of the two subjects.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float Equilibrium;

	void CalculateForPoints(const FRigVMSimPoint& InPointA, const FRigVMSimPoint& InPointB, FVector& ForceA, FVector& ForceB) const;
};
