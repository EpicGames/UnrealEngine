// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "CRSimPointConstraint.generated.h"

UENUM()
enum class ECRSimConstraintType : uint8
{
	Distance,
	DistanceFromA,
	DistanceFromB,
	Plane
};

USTRUCT()
struct FCRSimPointConstraint
{
	GENERATED_BODY()

	FCRSimPointConstraint()
	{
		Type = ECRSimConstraintType::Distance;
		SubjectA = SubjectB = INDEX_NONE;
		DataA = DataB = FVector::ZeroVector;
	}

	/**
	 * The type of the constraint
	 */
	UPROPERTY()
	ECRSimConstraintType Type;

	/**
	 * The first point affected by this constraint
	 */
	UPROPERTY()
	int32 SubjectA;

	/**
	 * The (optional) second point affected by this constraint
	 * This is currently only used for the distance constraint
	 */
	UPROPERTY()
	int32 SubjectB;

	/**
	 * The first data member for the constraint.
	 */
	UPROPERTY()
	FVector DataA;

	/**
	 * The second data member for the constraint.
	 */
	UPROPERTY()
	FVector DataB;

	void Apply(FRigVMSimPoint& OutPointA, FRigVMSimPoint& OutPointB) const;
};
