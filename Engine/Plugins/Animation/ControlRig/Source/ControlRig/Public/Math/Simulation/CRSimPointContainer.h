// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "CRSimLinearSpring.h"
#include "CRSimPointForce.h"
#include "CRSimPointConstraint.h"
#include "CRSimSoftCollision.h"
#include "CRSimContainer.h"
#include "CRSimPointContainer.generated.h"

USTRUCT()
struct FCRSimPointContainer : public FCRSimContainer
{
	GENERATED_BODY()

	FCRSimPointContainer()
	{
	}

	/**
	 * The points within the simulation
	 */
	UPROPERTY()
	TArray<FRigVMSimPoint> Points;

	/**
	 * The springs within the simulation
	 */
	UPROPERTY()
	TArray<FCRSimLinearSpring> Springs;

	/**
	 * The forces to apply to the points
	 */
	UPROPERTY()
	TArray<FCRSimPointForce> Forces;

	/**
     * The collision volumes for the simulation
     */
	UPROPERTY()
	TArray<FCRSimSoftCollision> CollisionVolumes;

	/**
	 * The constraints within the simulation
	 */
	UPROPERTY()
	TArray<FCRSimPointConstraint> Constraints;

	FRigVMSimPoint GetPointInterpolated(int32 InIndex) const;

	virtual void Reset() override;
	virtual void ResetTime() override;

protected:

	UPROPERTY()
	TArray<FRigVMSimPoint> PreviousStep;

	virtual void CachePreviousStep() override;
	virtual void IntegrateVerlet(float InBlend) override;
	virtual void IntegrateSemiExplicitEuler() override;
	void IntegrateSprings();
	void IntegrateForcesAndVolumes();
	void IntegrateVelocityVerlet(float InBlend);
	void IntegrateVelocitySemiExplicitEuler();
	void ApplyConstraints();
};
