// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "CRSimPointForce.generated.h"

UENUM()
enum class ECRSimPointForceType : uint8
{
	Direction
};

USTRUCT(BlueprintType)
struct FCRSimPointForce
{
	GENERATED_BODY()

	FCRSimPointForce()
	{
		ForceType = ECRSimPointForceType::Direction;
		Vector = FVector::ZeroVector;
		Coefficient = 1.f;
		bNormalize = false;
	}

	/**
	 * The type of force.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	ECRSimPointForceType ForceType;

	/**
	 * The point / direction to use for the force.
	 * This is a direction for direction based forces,
	 * while this is a position for attractor / repel based forces.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	FVector Vector;

	/**
	 * The strength of the force (a multiplier for direction based forces)
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float Coefficient;

	/**
	 * If set to true the input vector will be normalized.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	bool bNormalize;

	FVector Calculate(const FRigVMSimPoint& InPoint, float InDeltaTime) const;
};
