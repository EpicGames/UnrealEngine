// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "CRSimSoftCollision.generated.h"

UENUM()
enum class ECRSimSoftCollisionType : uint8
{
	Plane,
	Sphere,
	Cone
};

USTRUCT(BlueprintType)
struct FCRSimSoftCollision
{
	GENERATED_BODY()

	FCRSimSoftCollision()
	{
		Transform = FTransform::Identity;
		ShapeType = ECRSimSoftCollisionType::Sphere;
		MinimumDistance = 10.f;
		MaximumDistance = 20.f;
		FalloffType = ERigVMAnimEasingType::CubicEaseIn;
		Coefficient = 64.f;
		bInverted = false;
	}

	/**
	 * The world / global transform of the collisoin
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	FTransform Transform;

	/**
	 * The type of collision shape
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	ECRSimSoftCollisionType ShapeType;

	/**
	 * The minimum distance for the collision.
	 * If this is equal or higher than the maximum there's no falloff.
	 * For a cone shape this represents the minimum angle in degrees.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float MinimumDistance;

	/**
	 * The maximum distance for the collision.
	 * If this is equal or lower than the minimum there's no falloff.
	 * For a cone shape this represents the maximum angle in degrees.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float MaximumDistance;

	/**
	 * The type of falloff to use
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	ERigVMAnimEasingType FalloffType;

	/**
	 * The strength of the collision force
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float Coefficient;

	/**
	 * If set to true the collision volume will be inverted
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	bool bInverted;

	static float CalculateFalloff(const FCRSimSoftCollision& InCollision, const FVector& InPosition, float InSize, FVector& OutDirection);
	FVector CalculateForPoint(const FRigVMSimPoint& InPoint, float InDeltaTime) const;
};
