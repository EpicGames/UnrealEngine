// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CONTROLRIG_API

class FCRSimUtils
{
public:

	static UE_API void ComputeWeightsFromMass(float MassA, float MassB, float& OutWeightA, float& OutWeightB);
};

#undef UE_API
