// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ControlRig.h"
#include "ModularRigModel.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "ModularRig.generated.h"

#define UE_API CONTROLRIG_API

struct FRigModuleInstance;

struct FModuleInstanceHandle
{
public:

	FModuleInstanceHandle()
		: ModularRig(nullptr)
		, ModuleName(NAME_None)
	{}

	UE_API FModuleInstanceHandle(const UModularRig* InModularRig, const FName& InModuleName);
	UE_API FModuleInstanceHandle(const UModularRig* InModularRig, const FRigModuleInstance* InElement);

	bool IsValid() const { return Get() != nullptr; }
	operator bool() const { return IsValid(); }
	
	const UModularRig* GetModularRig() const
	{
		if (ModularRig.IsValid())
		{
			return ModularRig.Get();
		}
		return nullptr;
	}
	const FName& GetModuleName() const { return ModuleName; }

	UE_API const FRigModuleInstance* Get() const;

private:

	mutable TSoftObjectPtr<UModularRig> ModularRig;
	FName ModuleName;
};

USTRUCT(BlueprintType)
struct FRigModuleInstance 
{
	GENERATED_USTRUCT_BODY()

public:
	
	FRigModuleInstance()
	: Name(NAME_None)
	, RigPtr(nullptr)
	, ParentModuleName(NAME_None)
	, CachedParentModule(nullptr)
	, ConstructionSpawnStartIndex(INDEX_NONE)
	, PostConstructionSpawnStartIndex(INDEX_NONE)
	{
	}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	TSubclassOf<UControlRig> RigClass_DEPRECATED;
	
	UPROPERTY()
	FControlRigAssetStrongReference RigAssetReference;

private:

	UPROPERTY(transient)
	mutable TObjectPtr<UControlRig> RigPtr;

public:
	
	UPROPERTY()
	FString ParentPath_DEPRECATED;

	UPROPERTY()
	FName ParentModuleName;

	UPROPERTY()
	TMap<FName, FRigVMExternalVariable> VariableBindings;

	FRigModuleInstance* CachedParentModule;
	TArray<FRigModuleInstance*> CachedChildren;
	mutable FCachedRigElement PrimaryConnector;
	int32 ConstructionSpawnStartIndex;
	int32 PostConstructionSpawnStartIndex;

	UE_API UControlRig* GetRig() const;
	UE_API void SetRig(UControlRig* InRig);
	UE_API bool ContainsRig(const UControlRig* InRig) const;
	UE_API const FRigModuleReference* GetModuleReference() const;
	UE_API const FRigModuleInstance* GetParentModule() const;
	UE_API const FRigModuleInstance* GetRootModule() const;
	UE_API const FRigConnectorElement* FindPrimaryConnector() const;
	UE_API TArray<const FRigConnectorElement*> FindConnectors() const;
	UE_API bool IsRootModule() const;
	UE_API FString GetModulePrefix() const;
	UE_API FString GetModulePath_Deprecated() const;
	UE_API bool HasChildModule(const FName& InModuleName, bool bRecursive = true) const;
	UE_API void PostSerialize(const FArchive& Ar); 
};

template<>
struct TStructOpsTypeTraits<FRigModuleInstance> : public TStructOpsTypeTraitsBase2<FRigModuleInstance>
{
	enum
	{
		WithPostSerialize = true,
	};
};

USTRUCT(BlueprintType)
struct FRigModuleExecutionElement
{
	GENERATED_USTRUCT_BODY()

	FRigModuleExecutionElement()
		: ModuleName(NAME_None)
		, ModuleInstance(nullptr)
		, EventName(NAME_None)
		, bExecuted(false)
		, DurationInMicroSeconds(0.0)
	{}

	FRigModuleExecutionElement(FRigModuleInstance* InModule, FName InEvent, double InDurationMicroSeconds = 0.0)
		: ModuleName(InModule->Name)
		, ModuleInstance(InModule)
		, EventName(InEvent)
		, bExecuted(false)
		, DurationInMicroSeconds(InDurationMicroSeconds)
	{}

	UPROPERTY()
	FName ModuleName;
	FRigModuleInstance* ModuleInstance;

	UPROPERTY()
	FName EventName;

	UPROPERTY()
	bool bExecuted;

	UPROPERTY()
	double DurationInMicroSeconds;

	UE_API friend uint32 GetTypeHash(const FRigModuleExecutionElement& InElement);
};

USTRUCT(BlueprintType)
struct FRigModuleExecutionQueue
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FRigModuleExecutionElement> Elements;

	UE_API friend uint32 GetTypeHash(const FRigModuleExecutionQueue& InQueue);
};

/** Runs logic for mapping input data to transforms (the "Rig") */
UCLASS(MinimalAPI, Blueprintable, editinlinenew)
class UModularRig : public UControlRig
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FRigModuleInstance> Modules;
	TArray<FRigModuleInstance*> RootModules;

	mutable TArray<FName> SupportedEvents;

public:

	UE_API virtual void PostInitProperties() override;
	virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;

	// BEGIN ControlRig
	UE_API virtual void PostInitInstance(URigVMHost* InCDO) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void InitializeVMs(bool bRequestInit = true) override;
	UE_API virtual bool InitializeVMs(const FName& InEventName) override;
	virtual void InitializeVMsFromCDO() override { URigVMHost::InitializeFromCDO(); }
	UE_API virtual void InitializeFromCDO() override;
	virtual void RequestInitVMs() override { URigVMHost::RequestInit(); }
	UE_API virtual bool Execute_Internal(const FName& InEventName) override;
	UE_API virtual void Evaluate_AnyThread() override;
	virtual FRigElementKeyRedirector& GetElementKeyRedirector() override { return ElementKeyRedirector; }
	virtual FRigElementKeyRedirector GetElementKeyRedirector() const override { return ElementKeyRedirector; }
	UE_API virtual bool SupportsEvent(const FName& InEventName) const override;
	UE_API virtual const TArray<FName>& GetSupportedEvents() const override;
	virtual void GetControlsInOrder(TArray<FRigControlElement*>& SortedControls) const override;
	// END ControlRig

	UPROPERTY()
	FModularRigSettings ModularRigSettings;

	// Returns the settings of the modular rig
	UE_API const FModularRigSettings& GetModularRigSettings() const;

	UPROPERTY()
	TArray<FRigModuleExecutionElement> ExecutionQueue;
	int32 ExecutionQueueFront = 0;
	UE_API void ExecuteQueue();
	UE_API void ResetExecutionQueue();

#if WITH_EDITOR
	UE_API FRigModuleExecutionQueue GetLastExecutionQueue() const;
#endif

	// BEGIN UObject
	UE_API virtual void BeginDestroy() override;
	// END UObject
	
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	UE_API void ResetModules(bool bDestroyModuleRigs = true);

	UE_API const FModularRigModel& GetModularRigModel() const;
	UE_API void UpdateModuleHierarchyFromCDO();
	UE_API void UpdateCachedChildren();
	UE_API void UpdateSupportedEvents() const;

	/**
	 * @return Returns all of the module paths within this rig
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetModuleNames instead."))
	UE_API TArray<FString> GetModulePaths() const;

	/**
	 * @return Returns all of the module paths within this rig
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig)
	UE_API TArray<FName> GetModuleNames() const;

	UE_API const FRigModuleInstance* FindModule(const FName& InModuleName) const;
	UE_API const FRigModuleInstance* FindModule_Deprecated(const FString& InModulePath) const;
	UE_API const FRigModuleInstance* FindModule(const UControlRig* InModuleInstance) const;
	UE_API const FRigModuleInstance* FindModule(const FRigBaseElement* InElement) const;
	UE_API const FRigModuleInstance* FindModule(const FRigElementKey& InElementKey) const;

	UE_API FRigModuleInstance* FindModule(const FName& InModuleName);
	UE_API FRigModuleInstance* FindModule_Deprecated(const FString& InModulePath);
	UE_API FRigModuleInstance* FindModule(const UControlRig* InModuleInstance);
	UE_API FRigModuleInstance* FindModule(const FRigBaseElement* InElement);
	UE_API FRigModuleInstance* FindModule(const FRigElementKey& InElementKey);

	/**
	 * @param InModulePath The path of the module to retrieve the rig for
	 * @return Returns the rig instance for a given module name
	 */
	UFUNCTION(BlueprintCallable, Category=ModularRig, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetModuleRigByName instead."))
	UE_API UControlRig* GetModuleRig(FString InModulePath);
	UE_API const UControlRig* GetModuleRig_Deprecated(FString InModulePath) const;

	/**
	 * @param InModuleName The name of the module to retrieve the rig for
	 * @return Returns the rig instance for a given module name
	 */
	UFUNCTION(BlueprintCallable, Category=ModularRig)
	UE_API UControlRig* GetModuleRigByName(FName InModuleName);
	UE_API const UControlRig* GetModuleRigByName(FName InModuleName) const;

	/**
	 * @param InModulePath The path of the module to receive the parent path for
	 * @return Returns the parent path for a given module path (or an empty string)
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig, meta=(ScriptName="GetParentPath", DisplayName="Get Parent Path", DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetParentModuleName instead."))
	UE_API FString GetParentPathForBP(FString InModulePath) const;

	/**
	 * @param InModuleName The name of the module to receive the parent module name for
	 * @return Returns the parent name for a given module name (or an empty string)
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig, meta=(ScriptName="GetParentModuleName", DisplayName="Get Parent Module Name"))
	UE_API FName GetParentModuleNameForBP(FName InModuleName) const;
	UE_API FName GetParentModuleName(const FName& InModuleName) const;

	UE_API void ForEachModule(TFunctionRef<bool(FRigModuleInstance*)> PerModuleFunction, bool bDepthFirst = true);
	UE_API void ForEachModule(TFunctionRef<bool(const FRigModuleInstance*)> PerModuleFunction, bool bDepthFirst = true) const;

	UE_API void ExecuteConnectorEvent(const FRigElementKey& InConnector, const FRigModuleInstance* InModuleInstance, const FRigElementKeyRedirector* InRedirector, TArray<FRigElementResolveResult>& InOutCandidates);

	/**
	 * @return Returns all of the events supported by the modules in this modular rig
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig)
	UE_API TArray<FName> GetEventsForAllModules() const;

	/**
	 * @param InModulePath The path of the module to receive the events for
	 * @return Returns the names of all supported events for a given module path
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetEventsForModuleByName instead."))
	UE_API TArray<FName> GetEventsForModule(FString InModulePath) const;

	/**
	 * @param InModuleName The name of the module to receive the events for
	 * @return Returns the names of all supported events for a given module name
	 */
	UFUNCTION(BlueprintPure, Category=ModularRig)
	UE_API TArray<FName> GetEventsForModuleByName(FName InModuleName) const;

	/**
	 * @param InEvent The name of the event to run
	 * @return Returns the paths of all modules which ran the event successfully
	 */
	UFUNCTION(BlueprintCallable, Category=ModularRig)
	UE_API TArray<FName> ExecuteEventOnAllModules(FName InEvent);

	/**
	 * @param InEvent The name of the event to run
	 * @param InModulePath The name of the module to run the event on
	 * @return Returns true if the event was run successfully
	 */
	UFUNCTION(BlueprintCallable, Category=ModularRig, meta=(ScriptName="ExecuteEventOnModule", DisplayName="Execute Event On Module", DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on ExecuteEventOnModuleByName instead."))
	UE_API bool ExecuteEventOnModuleForBP(FName InEvent, FString InModulePath);

	/**
	 * @param InEvent The name of the event to run
	 * @param InModuleName The name of the module to run the event on
	 * @return Returns true if the event was run successfully
	 */
	UFUNCTION(BlueprintCallable, Category=ModularRig, meta=(ScriptName="ExecuteEventOnModuleByName", DisplayName="Execute Event On Module By Name"))
	UE_API bool ExecuteEventOnModuleByNameForBP(FName InEvent, FName InModuleName);
	UE_API bool ExecuteEventOnModule(const FName& InEvent, FRigModuleInstance* InModule);

	/**
	 * Returns a handle to an existing module
	 * @param InModuleName The name of the module to retrieve a handle for.
	 * @return The retrieved handle (may be invalid)
	 */
	FModuleInstanceHandle GetHandle(const FName& InModuleName) const
	{
		if(FindModule(InModuleName))
		{
			return FModuleInstanceHandle((UModularRig*)this, InModuleName);
		}
		return FModuleInstanceHandle();
	}
	
protected:

	CONTROLRIG_API virtual void GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& Context) override;

private:

	/** Adds a module to the rig*/
	UE_API FRigModuleInstance* AddModuleInstance(const FName& InModuleName, FControlRigAssetStrongReference InModuleSource, const FRigModuleInstance* InParent, const FRigElementKeyRedirector::FKeyMap& InConnectionMap, const FControlRigOverrideContainer& InConfigValues);

	/** Updates the module's variable bindings */
	UE_API bool SetModuleVariableBindings(const FName& InModuleName, const TMap<FName, FString>& InVariableBindings);

	/** Updates the module's variable values based on its binding */
	UE_API void UpdateModuleVariables(const FRigModuleInstance* InModule);

	/** Destroys / discards a module instance rig */
	static UE_API void DiscardModuleRig(UControlRig* InControlRig);

	static bool TraverseModules(FRigModuleInstance* InModuleInstance, TFunctionRef<bool(FRigModuleInstance*)> PerModule);

	UPROPERTY()
	FModularRigModel ModularRigModel;

	TMap<FName, UControlRig*> PreviousModuleRigs;

#if WITH_EDITOR
	TArray<FRigModuleExecutionElement> LastExecutedElements;
	FRigModuleExecutionQueue LastExecutionQueue;
	mutable FTransactionallySafeRWLock LastExecutionQueueLock;
#endif

	friend struct FRigModuleInstance;
	friend class IControlRigEditorAssetInterface;
	friend class UControlRigRuntimeAsset;
};

#undef UE_API
