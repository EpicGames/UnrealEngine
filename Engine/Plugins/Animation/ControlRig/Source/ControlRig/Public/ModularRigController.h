// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModularRigModel.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "ModularRigController.generated.h"

#define UE_API CONTROLRIG_API

struct FRigModuleReference;
class UModularRig;

DECLARE_MULTICAST_DELEGATE_TwoParams(FModularRigModifiedEvent, EModularRigNotification /* type */, const FRigModuleReference* /* element */);

UCLASS(MinimalAPI, BlueprintType)
class UModularRigController : public UObject
{
	GENERATED_UCLASS_BODY()

	UModularRigController()
		: Model(nullptr)
		, bSuspendNotifications(false)
	{
	}

	FModularRigModel* Model;
	FModularRigModifiedEvent ModifiedEvent;
	bool bSuspendNotifications;

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules", meta=(DeprecatedFunction, DeprecationMessage="Use AddModuleFromAssetReference"))
	UE_API FName AddModule(const FName& InModuleName, TSubclassOf<UControlRig> InClass, const FName& InParentModuleName, bool bSetupUndo = true);
	UE_API FName AddModuleFromAssetReference(const FName& InModuleName, FControlRigAssetStrongReference InHandle, const FName& InParentModuleName, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool CanConnectConnectorToElement(const FRigElementKey& InConnectorKey, const FRigElementKey& InTargetKey, FText& OutErrorMessage);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool CanConnectConnectorToElements(const FRigElementKey& InConnectorKey, const TArray<FRigElementKey>& InTargetKeys, FText& OutErrorMessage);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool ConnectConnectorToElement(const FRigElementKey& InConnectorKey, const FRigElementKey& InTargetKey, bool bSetupUndo = true, bool bAutoResolveOtherConnectors = true, bool bCheckValidConnection = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool ConnectConnectorToElements(const FRigElementKey& InConnectorKey, const TArray<FRigElementKey>& InTargetKeys, bool bSetupUndo = true, bool bAutoResolveOtherConnectors = true, bool bCheckValidConnection = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool AddTargetToArrayConnector(const FRigElementKey& InConnectorKey, const FRigElementKey& InTargetKey, bool bSetupUndo = true, bool bAutoResolveOtherConnectors = true, bool bCheckValidConnection = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool DisconnectConnector(const FRigElementKey& InConnectorKey, bool bDisconnectSubModules = false, bool bSetupUndo = true);
	UE_API bool DisconnectConnector_Internal(const FRigElementKey& InConnectorKey, bool bDisconnectSubModules = false, TMap<FRigElementKey, TArray<FRigElementKey>>*OutRemovedConnections = nullptr, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API TArray<FRigElementKey> DisconnectCyclicConnectors(bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool AutoConnectSecondaryConnectors(const TArray<FRigElementKey>& InConnectorKeys, bool bReplaceExistingConnections, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool AutoConnectModules(const TArray<FName>& InModuleNames, bool bReplaceExistingConnections, bool bSetupUndo = true);

	UE_DEPRECATED(5.6, "Please rely on the overload taking an FControlRigOverrideValue")
	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool SetConfigValueInModule(const FName& InModuleName, const FName& InVariableName, const FString& InValue, bool bSetupUndo = true);
	UE_API bool SetConfigValueInModule(const FName& InModuleName, const FControlRigOverrideValue& InValue, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "Control Rig | Modules")
	UE_API bool ResetConfigValueInModule(const FName& InModuleName, const FString& InPath, bool bClearOverride = true, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool BindModuleVariable(const FName& InModuleName, const FName& InVariableName, const FString& InSourcePath, bool bSetupUndo = true);
	UE_API bool CanBindModuleVariable(const FName& InModuleName, const FName& InVariableName, const FString& InSourcePath, FText& OutErrorMessage);
	UE_API TArray<FString> GetPossibleBindings(const FName& InModuleName, const FName& InVariableName);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool UnBindModuleVariable(const FName& InModuleName, const FName& InVariableName, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool DeleteModule(const FName& InModuleName, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API FName RenameModule(const FName& InModuleName, const FName& InNewName, bool bSetupUndo = true);
	UE_API bool CanRenameModule(const FName& InModuleName, const FName& InNewName, FText& OutErrorMessage) const;

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool ReparentModule(const FName& InModuleName, const FName& InNewParentModuleName, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool ReorderModule(const FName& InModuleName, int32 InModuleIndex, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API FName MirrorModule(const FName& InModuleName, const FRigVMMirrorSettings& InSettings, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules", meta=(DeprecatedFunction, DeprecationMessage="Use SwapModuleSource"))
	UE_API bool SwapModuleClass(const FName& InModuleName, TSubclassOf<UControlRig> InNewClass, bool bSetupUndo = true);
	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool SwapModuleSource(const FName& InModuleName, FControlRigAssetStrongReference InNewSource, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules", meta=(DeprecatedFunction, DeprecationMessage="Use SwapModulesWithSource"))
	UE_API bool SwapModulesOfClass(TSubclassOf<UControlRig> InOldClass, TSubclassOf<UControlRig> InNewClass, bool bSetupUndo = true);
	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool SwapModulesWithSource(FControlRigAssetStrongReference InOldClass, FControlRigAssetStrongReference InNewClass, bool bSetupUndo = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool SelectModule(const FName& InModuleName, const bool InSelected = true);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool DeselectModule(const FName& InModuleName);

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool SetModuleSelection(const TArray<FName>& InModuleNames);

	UFUNCTION(BlueprintPure, Category = "ControlRig | Modules")
	UE_API TArray<FName> GetAllModules() const;

	UFUNCTION(BlueprintPure, Category = "ControlRig | Modules")
	UE_API TArray<FName> GetSelectedModules() const;

	UE_API void RefreshModuleVariables(bool bSetupUndo = true);
	UE_API void RefreshModuleVariables(const FRigModuleReference* InModule, bool bSetupUndo = true);
	
	UFUNCTION(BlueprintPure, Category = "ControlRig | Modules")
	UE_API FString ExportModuleSettingsToString(TArray<FName> InModuleNames) const;

	UFUNCTION(BlueprintCallable, Category = "ControlRig | Modules")
	UE_API bool ImportModuleSettingsFromString(FString InContent, TArray<FName> InOptionalModuleNames, bool bSetupUndo = true);

	static int32 GetMaxNameLength() { return 100; }
	static UE_API void SanitizeName(FRigName& InOutName, bool bAllowNameSpaces);
	static UE_API FRigName GetSanitizedName(const FRigName& InName, bool bAllowNameSpaces);
	UE_API bool IsNameAvailable(const FRigName& InDesiredName, FString* OutErrorMessage = nullptr, const FRigModuleReference* InModuleToSkip = nullptr) const;
	UE_API FRigName GetSafeNewName(const FRigName& InDesiredName, const FRigModuleReference* InModuleToSkip = nullptr) const;
	UE_API FRigModuleReference* FindModule(const FName& InModuleName);
	UE_API const FRigModuleReference* FindModule(const FName& InModuleName) const;
	FModularRigModifiedEvent& OnModified() { return ModifiedEvent; }

	/**
	 * @param InModuleName The name of the module reference to return
	 * @return Returns the module for the given name
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig | Modules")
	UE_API FRigModuleReference GetModuleReference(FName InModuleName) const;

	/**
	 * @param InModuleName The name of the module reference to return
	 * @return Returns all of associated connectors for a given module name
	 */
	UFUNCTION(BlueprintPure, Category = "ControlRig | Modules")
	UE_API TArray<FRigElementKey> GetConnectorsForModule(FName InModuleName) const;

private:

	void SetModel(FModularRigModel* InModel) { Model = InModel; }
	UE_API void Notify(const EModularRigNotification& InNotification, const FRigModuleReference* InElement);
	UE_API UModularRig* GetDebuggedModularRig();
	UE_API const UModularRig* GetDebuggedModularRig() const;

	bool bAutomaticReparenting;

	friend struct FModularRigModel;
	friend class FModularRigControllerCompileBracketScope;
	friend class UControlRigBlueprint;
};

class FModularRigControllerCompileBracketScope
{
public:
   
	UE_API FModularRigControllerCompileBracketScope(UModularRigController *InController);

	UE_API ~FModularRigControllerCompileBracketScope();

private:
	UModularRigController* Controller;
	bool bSuspendNotifications;
};

#undef UE_API
