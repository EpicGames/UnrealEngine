// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigAssetReference.h"
#include "ControlRigOverride.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyCache.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ModularRigModel.generated.h"

#define UE_API CONTROLRIG_API

struct FRigElementKey;
class UModularRigController;
struct FModularRigModel;

#if WITH_EDITOR
class FPropertyPath;
#endif

UENUM()
enum class EModularRigNotification : uint8
{
	ModuleAdded,

	ModuleRenamed,

	ModuleRemoved,

	ModuleReparented,

	ModuleReordered,

	ConnectionChanged,

	ModuleConfigValueChanged,

	ModuleShortNameChanged,

	InteractionBracketOpened, // A bracket has been opened
	
	InteractionBracketClosed, // A bracket has been opened
	
	InteractionBracketCanceled, // A bracket has been canceled

	ModuleClassChanged,

	ModuleSelected,

	ModuleDeselected,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

USTRUCT(BlueprintType)
struct FRigModuleReference
{
	GENERATED_BODY()

public:

	FRigModuleReference()
		: Name(NAME_None)
		, bShortNameBasedOnPath_DEPRECATED(true)
		, PreviousName(NAME_None)
		, PreviousParentName(NAME_None)
	{
		ConfigOverrides.SetUsesKeyForSubject(false);
	}
	
	FRigModuleReference(const FName& InName, FControlRigAssetSoftReference InClass, const FName& InParentModuleName, const FModularRigModel* InModel)
		: Name(InName)
		, bShortNameBasedOnPath_DEPRECATED(true)
		, ParentModuleName(InParentModuleName)
		, ControlRigAssetReference(InClass)
		, PreviousName(NAME_None)
		, PreviousParentName(NAME_None)
		, Model(InModel)
	{
		ConfigOverrides.SetUsesKeyForSubject(false);
	}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FString ShortName_DEPRECATED;

	UPROPERTY()
	bool bShortNameBasedOnPath_DEPRECATED;

	UPROPERTY()
	FString ParentPath_DEPRECATED;

	UPROPERTY()
	FName ParentModuleName;

	UPROPERTY(meta=(DeprecatedProperty))
	TSoftClassPtr<UControlRig> Class_DEPRECATED;
	
	UPROPERTY()
	FControlRigAssetSoftReference ControlRigAssetReference;

	UPROPERTY(meta = (DeprecatedProperty))
	TMap<FRigElementKey, FRigElementKey> Connections_DEPRECATED; // Connectors to Connection element

	UPROPERTY()
	TMap<FName, FString> ConfigValues_DEPRECATED;

	UPROPERTY()
	FControlRigOverrideContainer ConfigOverrides;

	UPROPERTY()
	TMap<FName, FString> Bindings; // ExternalVariableName (current module) -> SourceExternalVariableNamespacedPath (root rig or other module)

	UPROPERTY(transient)
	FName PreviousName;

	UPROPERTY(transient)
	FName PreviousParentName;

	TArray<FRigModuleReference*> CachedChildren;

	FName GetFName() const
	{
		return Name;
	}

	FString GetName() const
	{
		return Name.ToString();
	}

	FName GetElementPrefixFName() const
	{
		return *GetElementPrefix();
	}

	UE_API FString GetElementPrefix() const;

	UE_API FRigHierarchyModulePath GetModulePath() const;

	bool HasParentModule() const { return !ParentModuleName.IsNone() || !ParentPath_DEPRECATED.IsEmpty(); }

	bool IsRootModule() const { return !HasParentModule(); }

	UE_API const FRigModuleReference* GetParentModule() const;
	UE_API const FRigModuleReference* GetRootModule() const;

	friend bool operator==(const FRigModuleReference& A, const FRigModuleReference& B)
	{
		return A.ParentModuleName == B.ParentModuleName &&
			A.Name == B.Name;
	}

	UE_API const FRigConnectorElement* FindPrimaryConnector(const URigHierarchy* InHierarchy) const;
	UE_API TArray<const FRigConnectorElement*> FindConnectors(const URigHierarchy* InHierarchy) const;

	UE_API void PatchModelsOnLoad();

private:
	const FModularRigModel* Model = nullptr;

	friend class UModularRigController;
	friend struct FModularRigModel;
};

USTRUCT(BlueprintType)
struct FModularRigSingleConnection
{
	GENERATED_BODY()

	FModularRigSingleConnection()
		: Connector(FRigElementKey()), Targets({FRigElementKey()}) {}
	
	FModularRigSingleConnection(const FRigElementKey& InConnector, const FRigElementKey& InTarget)
		: Connector(InConnector), Targets({InTarget}) {}

	FModularRigSingleConnection(const FRigElementKey& InConnector, const FRigElementKeyRedirector::FKeyArray& InTargets)
		: Connector(InConnector), Targets(InTargets) {}
	
	bool operator==(const FModularRigSingleConnection& InOther) const
	{
		return Connector == InOther.Connector && Targets == InOther.Targets;
	}

	UPROPERTY()
	FRigElementKey Connector;

	UPROPERTY()
	FRigElementKey Target_DEPRECATED;

	UPROPERTY()
	TArray<FRigElementKey> Targets;

	FRigElementKeyRedirector::FKeyArray GetTargetArray() const;
};

USTRUCT(BlueprintType)
struct FModularRigConnections
{
public:
	
	GENERATED_BODY()
	/** Connections sorted by creation order */

private:
	
	UPROPERTY()
	TArray<FModularRigSingleConnection> ConnectionList;

	/** Target key to connector array */
	TMap<FRigElementKey, TArray<FRigElementKey>> ReverseConnectionMap;

public:
	
	bool operator==(const FModularRigConnections& InOther) const
	{
		return ConnectionList == InOther.ConnectionList;
	}

	const TArray<FModularRigSingleConnection>& GetConnectionList() const { return ConnectionList; }

	bool IsEmpty() const { return ConnectionList.IsEmpty(); }
	int32 Num() const { return ConnectionList.Num(); }
	const FModularRigSingleConnection& operator[](int32 InIndex) const { return ConnectionList[InIndex]; }
	FModularRigSingleConnection& operator[](int32 InIndex) { return ConnectionList[InIndex]; }
	TArray<FModularRigSingleConnection>::RangedForIteratorType begin() { return ConnectionList.begin(); }
	TArray<FModularRigSingleConnection>::RangedForIteratorType end() { return ConnectionList.end(); }
	TArray<FModularRigSingleConnection>::RangedForConstIteratorType begin() const { return ConnectionList.begin(); }
	TArray<FModularRigSingleConnection>::RangedForConstIteratorType end() const { return ConnectionList.end(); }

	void UpdateFromConnectionList()
	{
		ReverseConnectionMap.Reset();
		for (const FModularRigSingleConnection& Connection : ConnectionList)
		{
			for(const FRigElementKey& Target : Connection.Targets)
			{
				TArray<FRigElementKey>& Connectors = ReverseConnectionMap.FindOrAdd(Target);
				Connectors.AddUnique(Connection.Connector);
			}
		}
	}

	void AddConnection(const FRigElementKey& Connector, const FRigElementKeyRedirector::FKeyArray& Targets)
	{
		// Remove any existing connection
		RemoveConnection(Connector);

		ConnectionList.Add(FModularRigSingleConnection(Connector, Targets));
		for(const FRigElementKey& Target : Targets)
		{
			ReverseConnectionMap.FindOrAdd(Target).AddUnique(Connector);
		}
	}

	void RemoveConnection(const FRigElementKey& Connector)
	{
		int32 ExistingIndex = FindConnectionIndex(Connector);
		if (ConnectionList.IsValidIndex(ExistingIndex))
		{
			const TArray<FRigElementKey>& Targets = ConnectionList[ExistingIndex].Targets;
			for(const FRigElementKey& Target : Targets)
			{
				if (TArray<FRigElementKey>* Connectors = ReverseConnectionMap.Find(Target))
				{
					*Connectors = Connectors->FilterByPredicate([Connector](const FRigElementKey& TargetConnector)
					{
						return TargetConnector != Connector;
					});
					if (Connectors->IsEmpty())
					{
						ReverseConnectionMap.Remove(Target);
					}
				}
			}
			ConnectionList.RemoveAt(ExistingIndex);
		}
	}

	int32 FindConnectionIndex(const FRigElementKey& InConnectorKey) const
	{
		return ConnectionList.IndexOfByPredicate([InConnectorKey](const FModularRigSingleConnection& Connection)
		{
			return InConnectorKey == Connection.Connector;
		});
	}

	FRigElementKey FindTargetFromConnector(const FRigElementKey& InConnectorKey) const
	{
		int32 Index = FindConnectionIndex(InConnectorKey);
		if (ConnectionList.IsValidIndex(Index))
		{
			check(ConnectionList[Index].Targets.Num() >= 1);
			return ConnectionList[Index].Targets[0];
		}
		static const FRigElementKey EmptyKey;
		return EmptyKey;
	}

	TArray<FRigElementKey> FindTargetsFromConnector(const FRigElementKey& InConnectorKey) const
	{
		int32 Index = FindConnectionIndex(InConnectorKey);
		if (ConnectionList.IsValidIndex(Index))
		{
			check(!ConnectionList[Index].Targets.IsEmpty());
			return ConnectionList[Index].Targets;
		}
		static const TArray<FRigElementKey> EmptyKeys;
		return EmptyKeys;
	}

	const TArray<FRigElementKey>& FindConnectorsFromTarget(const FRigElementKey& InTargetKey) const
	{
		if(const TArray<FRigElementKey>* Connectors = ReverseConnectionMap.Find(InTargetKey))
		{
			return *Connectors;
		}
		static const TArray<FRigElementKey> EmptyList;
		return EmptyList;
	}

	bool HasConnection(const FRigElementKey& InConnectorKey) const
	{
		return ConnectionList.IsValidIndex(FindConnectionIndex(InConnectorKey));
	}

	bool HasConnection(const FRigElementKey& InConnectorKey, const URigHierarchy* InHierarchy) const
	{
		if(HasConnection(InConnectorKey))
		{
			const FRigElementKey Target = FindTargetFromConnector(InConnectorKey);
			return InHierarchy->Contains(Target);
		}
		return false;
	}

	/** Gets the connection map for a single module, where the connectors are identified without its namespace*/
	UE_API FRigElementKeyRedirector::FKeyMap GetModuleConnectionMap(const FName& InModuleName) const;

	UE_API void PatchOnLoad(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr);
};

// A management struct containing all modules in the rig
USTRUCT(BlueprintType)
struct FModularRigModel
{
public:

	GENERATED_BODY()

	FModularRigModel()
	{}
	
	FModularRigModel(const FModularRigModel& Other)
	{
		Modules = Other.Modules;
		PreviousModulePaths = Other.PreviousModulePaths;
		Connections = Other.Connections;
		for (FRigModuleReference& Module : Modules)
		{
			Module.Model = this;
		}
		UpdateCachedChildren();
		Connections.UpdateFromConnectionList();
	}

	FModularRigModel& operator=(const FModularRigModel& Other)
	{
		Modules = Other.Modules;
		PreviousModulePaths = Other.PreviousModulePaths;
		Connections = Other.Connections;
		for (FRigModuleReference& Module : Modules)
		{
			Module.Model = this;
		}
		UpdateCachedChildren();
		Connections.UpdateFromConnectionList();
		return *this;
	}
	FModularRigModel(FModularRigModel&&) = delete;
	FModularRigModel& operator=(FModularRigModel&&) = delete;
	
	bool operator==(const FModularRigModel& InOther) const
	{
		return Modules == InOther.Modules
			&& Connections == InOther.Connections;
	}

	UPROPERTY()
	TArray<FRigModuleReference> Modules;
	TArray<FRigModuleReference*> RootModules;
	TArray<FRigModuleReference> DeletedModules;

	UPROPERTY()
	FModularRigConnections Connections;

	UPROPERTY(transient)
	TObjectPtr<UObject> Controller;

	// remember what modules were called so we can recover.
	UPROPERTY()
	TMap<FRigHierarchyModulePath, FName> PreviousModulePaths;

	UE_API void PatchModelsOnLoad();

	UE_API UModularRigController* GetController(bool bCreateIfNeeded = true);

	UObject* GetOuter() const { return OuterModelHost.IsValid() ? OuterModelHost.Get() : nullptr; }

	UE_DEPRECATED(5.8, "Use SetOuterModelHost instead")
	UE_API void SetOuterClientHost(UObject* InOuterClientHost);
	UE_API void SetOuterModelHost(UObject* InOuterModelHost);

	UE_API void UpdateCachedChildren();

	UE_API FRigModuleReference* FindModule(const FName& InModuleName);
	UE_API const FRigModuleReference* FindModule(const FName& InModuleName) const;
	UE_API const FRigModuleReference* FindModuleByPath(const FString& InModulePath) const;

	UE_API FRigModuleReference* GetParentModule(const FName& InModuleName);
	UE_API const FRigModuleReference* GetParentModule(const FName& InModuleName) const;

	UE_API FRigModuleReference* GetParentModule(const FRigModuleReference* InChildModule);
	UE_API const FRigModuleReference* GetParentModule(const FRigModuleReference* InChildModule) const;

	UE_API void ForEachModule(TFunction<bool(const FRigModuleReference*)> PerModule, bool bDepthFirst = true) const;

	UE_API TArray<FName> SortModuleNames(const TArray<FName>& InModuleNames) const;

	UE_API bool IsModuleParentedTo(const FName& InChildModuleName, const FName& InParentModuleName) const;
	
	UE_API bool IsModuleParentedTo(const FRigModuleReference* InChildModule, const FRigModuleReference* InParentModule) const;

	UE_API TArray<const FRigModuleReference*> FindModuleInstancesOfClass(const FString& InModuleClassPath) const;
	UE_API TArray<const FRigModuleReference*> FindModuleInstancesOfClass(const FAssetData& InModuleAsset) const;
	UE_API TArray<const FRigModuleReference*> FindModuleInstancesOfClass(TSoftClassPtr<UControlRig> InClass) const;
	
	UE_API TArray<FName> GetSupportedEvents() const;

private:

	static bool TraverseModules(const FRigModuleReference* InModuleReference, TFunction<bool(const FRigModuleReference*)> PerModule);

	TWeakObjectPtr<UObject> OuterModelHost;
	TArray<FName> SelectedModuleNames;

#if WITH_EDITOR
	mutable TArray<FName> ReceivedOldModulePaths;
#endif

	friend class UModularRigController;
	friend struct FRigModuleReference;
};

// A transient struct used for copy & paste of module settings
USTRUCT(BlueprintType)
struct FModularRigModuleSettingsForClipboard
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Module")
	FSoftObjectPath ModuleClass;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Module")
   	TMap<FString, FString> Defaults;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Module")
	TMap<FString, FString> Overrides;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Module")
	TMap<FName, FString> Bindings;
};

// A transient struct used for copy & paste of module settings
USTRUCT(BlueprintType)
struct FModularRigModuleSettingsSetForClipboard
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Module")
	TMap<FName, FModularRigModuleSettingsForClipboard> Settings;
};

#undef UE_API
