// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModularRig.h"
#include "ModularRigRuleManager.generated.h"

#define UE_API CONTROLRIG_API

// A management class to validate rules and pattern match
UCLASS(MinimalAPI, BlueprintType)
class UModularRigRuleManager : public UObject
{
public:

	GENERATED_BODY()

	/***
	 * Returns the possible targets for the given connector in the current resolve stage
	 * Note: This method is thread-safe. 
	 * @param InConnector The connector to resolve
	 * @param InModule The module the connector belongs to
	 * @param InResolvedConnectors A redirect map of the already resolved connectors
	 * @return The resolve result including a list of matches
	 */
	UE_API FModularRigResolveResult FindMatches(
		const FRigConnectorElement* InConnector,
		const FRigModuleInstance* InModule,
		const FRigElementKeyRedirector& InResolvedConnectors = FRigElementKeyRedirector()
	) const;

	/***
	 * Returns the possible targets for the given external module connector in the current resolve stage
	 * Note: This method is thread-safe. 
	 * @param InConnector The connector to resolve
	 * @return The resolve result including a list of matches
	 */
	UE_API FModularRigResolveResult FindMatches(
		const FRigModuleConnector* InConnector
	) const;

	/***
	 * Returns the possible targets for the primary connector in the current resolve stage
	 * Note: This method is thread-safe. 
	 * @param InModule The module the connector belongs to
	 * @return The resolve result including a list of matches
	 */
	UE_API FModularRigResolveResult FindMatchesForPrimaryConnector(
		const FRigModuleInstance* InModule
	) const;

	/***
	 * Returns the possible targets for each secondary connector
	 * Note: This method is thread-safe. 
	 * @param InModule The module the secondary connectors belongs to
	 * @param InResolvedConnectors A redirect map of the already resolved connectors
	 * @return The resolve result including a list of matches for each connector
	 */
	UE_API TArray<FModularRigResolveResult> FindMatchesForSecondaryConnectors(
		const FRigModuleInstance* InModule,
		const FRigElementKeyRedirector& InResolvedConnectors = FRigElementKeyRedirector()
	) const;

	/***
	 * Returns the possible targets for each optional connector
	 * Note: This method is thread-safe. 
	 * @param InModule The module the optional connectors belongs to
	 * @param InResolvedConnectors A redirect map of the already resolved connectors
	 * @return The resolve result including a list of matches for each connector
	 */
	UE_API TArray<FModularRigResolveResult> FindMatchesForOptionalConnectors(
		const FRigModuleInstance* InModule,
		const FRigElementKeyRedirector& InResolvedConnectors = FRigElementKeyRedirector()
	) const;

private:

	struct FWorkData
	{
		FWorkData()
		: Hierarchy(nullptr)
		, Connector(nullptr)
		, ModuleConnector(nullptr)
		, Module(nullptr)
		, ResolvedConnectors(nullptr)
		, Result(nullptr)
		{
		}

		void Filter(TFunction<void(FRigElementResolveResult&)> PerMatchFunction);

		const URigHierarchy* Hierarchy;
		const FRigConnectorElement* Connector;
		const FRigModuleConnector* ModuleConnector;
		const FRigModuleInstance* Module;
		const FRigElementKeyRedirector* ResolvedConnectors;
		FModularRigResolveResult* Result;
	};

	UE_API FModularRigResolveResult FindMatches(FWorkData& InWorkData) const;

	UE_API void SetHierarchy(const URigHierarchy* InHierarchy);
	static UE_API void ResolveConnector(FWorkData& InOutWorkData);
	static UE_API void FilterIncompatibleTypes(FWorkData& InOutWorkData);
	static UE_API void FilterInvalidModules(FWorkData& InOutWorkData);
	static UE_API void FilterByConnectorRules(FWorkData& InOutWorkData);
	static UE_API void FilterByConnectorEvent(FWorkData& InOutWorkData);

	TWeakObjectPtr<const URigHierarchy> Hierarchy;

	friend class URigHierarchy;
};

#undef UE_API
