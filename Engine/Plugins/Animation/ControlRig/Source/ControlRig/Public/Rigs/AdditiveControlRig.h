// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ControlRig.h"
#include "Units/Hierarchy/RigUnit_AddBoneTransform.h"
#include "AdditiveControlRig.generated.h"

#define UE_API CONTROLRIG_API

class USkeletalMesh;
struct FReferenceSkeleton;
struct FSmartNameMapping;

/** Rig that allows additive layer editing per joint */
UCLASS(MinimalAPI, NotBlueprintable)
class UAdditiveControlRig : public UControlRig
{
	GENERATED_UCLASS_BODY()

public: 
	// BEGIN ControlRig
	UE_API virtual void Initialize(bool bInitRigUnits = true) override;
	virtual void InitializeVMs(bool bRequestInit = true) override { URigVMHost::Initialize(bRequestInit); }
	virtual bool InitializeVMs(const FName& InEventName) override { return URigVMHost::InitializeVM(InEventName); }
	virtual void InitializeVMsFromCDO() override { URigVMHost::InitializeFromCDO(); }
	virtual void RequestInitVMs() override { URigVMHost::RequestInit(); }
	UE_API virtual bool Execute_Internal(const FName& InEventName) override;
	// END ControlRig

	// utility function to 
	static UE_API FName GetControlName(const FName& InBoneName);
	static UE_API FName GetNullName(const FName& InBoneName);

private:
	// custom units that are used to execute this rig
	TArray<FRigUnit_AddBoneTransform> AddBoneRigUnits;

	/** Create RigElements - bone hierarchy and curves - from incoming skeleton */
	UE_API void CreateRigElements(const USkeletalMesh* InReferenceMesh);
	UE_API void CreateRigElements(const FReferenceSkeleton& InReferenceSkeleton, const USkeleton* InSkeleton);
};

#undef UE_API
