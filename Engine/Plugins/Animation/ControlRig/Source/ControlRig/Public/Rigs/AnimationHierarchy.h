// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NodeHierarchy.h"
#include "AnimationHierarchy.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FConstraintNodeData
{
	GENERATED_BODY()

	UPROPERTY()
	FTransform RelativeParent;

	UPROPERTY()
	FConstraintOffset ConstraintOffset;

	UPROPERTY()
	FName	LinkedNode;

private:
	UPROPERTY()
	TArray<FTransformConstraint> Constraints;

public:
	// returns constraint ptr if found
	const TArray<FTransformConstraint> GetConstraints() const { return Constraints;  }
	UE_API FTransformConstraint* FindConstraint(const FName& TargetNode);
	UE_API void AddConstraint(const FTransformConstraint& TransformConstraint);
	UE_API void DeleteConstraint(const FName& TargetNode);
	bool DoesHaveConstraint() const { return Constraints.Num() > 0; }
};

USTRUCT()
struct FAnimationHierarchy : public FNodeHierarchyWithUserData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FConstraintNodeData> UserData;

	virtual ~FAnimationHierarchy() = default;

	virtual const void* GetUserDataImpl(int32 Index) const { return &UserData[Index]; }
	virtual void* GetUserDataImpl(int32 Index) { return &UserData[Index]; }
	virtual int32 AddUserDataImpl(const void* InData) { return UserData.Add(*reinterpret_cast<const FConstraintNodeData*>(InData)); }
	virtual int32 GetNumUserData() const { return UserData.Num(); }
	virtual void EmptyUserData(int32 Size = 0) { UserData.Empty(Size); }
	virtual void RemoveUserData(int32 Index) { UserData.RemoveAt(Index); }
	virtual bool HasUserData() const { return true; }

	virtual const FTransform& GetLocalTransform(int32 Index) const override
	{
		check(IsValidIndex(Index));
		return GetNodeData<FConstraintNodeData>(Index).RelativeParent;
	}

	virtual FTransform& GetLocalTransform(int32 Index) override
	{
		check(IsValidIndex(Index));
		return GetNodeData<FConstraintNodeData>(Index).RelativeParent;
	}

	virtual const FTransform& GetGlobalTransform(int32 Index) const override
	{
		check(IsValidIndex(Index));
		return Hierarchy.GetTransform(Index);
	}

	virtual FTransform& GetGlobalTransform(int32 Index) override
	{
		check(IsValidIndex(Index));
		return Hierarchy.GetTransform(Index);
	}

	virtual void SetLocalTransform(int32 Index, const FTransform& NewTransform) override
	{
		if (IsValidIndex(Index))
		{
			FTransform Transform = Hierarchy.GetTransform(Index);
			FConstraintNodeData& NodeData = GetNodeData<FConstraintNodeData>(Index);
			NodeData.RelativeParent = NewTransform;

			// recalculate global transform
			FTransform GlobalTransform;
			int32 ParentIndex = GetParentIndex(Index);
			if (IsValidIndex(ParentIndex))
			{
				FTransform ParentTransform = GetGlobalTransform(ParentIndex);
				GlobalTransform = NewTransform * ParentTransform;
			}
			else
			{
				GlobalTransform = NewTransform;
			}

			GlobalTransform.NormalizeRotation();
			Hierarchy.SetTransform(Index, GlobalTransform);
		}
	}

	virtual void SetGlobalTransform(int32 Index, const FTransform& NewTransform) override
	{
		if (IsValidIndex(Index))
		{
			Hierarchy.SetTransform(Index, NewTransform);
			// recalculate local transform
			FTransform LocalTransform;
			int32 ParentIndex = GetParentIndex(Index);
			if (IsValidIndex(ParentIndex))
			{
				FTransform ParentTransform = GetGlobalTransform(ParentIndex);
				LocalTransform = NewTransform.GetRelativeTransform(ParentTransform);
			}
			else
			{
				LocalTransform = NewTransform;
			}

			LocalTransform.NormalizeRotation();

			GetNodeData<FConstraintNodeData>(Index).RelativeParent = LocalTransform;
		}
	}
};

#undef UE_API
