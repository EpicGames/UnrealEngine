// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ControlRig.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"
#include "Units/Hierarchy/RigUnit_SetCurveValue.h"
#include "FKControlRig.generated.h"

#define UE_API CONTROLRIG_API

class USkeletalMesh;
struct FReferenceSkeleton;
struct FSmartNameMapping;
/** Structs used to specify which bones/curves/controls we should have active, since if all controls or active we can't passthrough some previous bone transform*/
struct FFKBoneCheckInfo
{
	FName BoneName;
	int32 BoneID;
	bool  bActive;
};

UENUM(Blueprintable)
enum class EControlRigFKRigExecuteMode: uint8
{
	/** Replaces the current pose */
	Replace,

	/** Applies the authored pose as an additive layer */
	Additive,

	/** Sets the current pose without the use of offset transforms */
	Direct,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

/** Rig that allows override editing per joint */
UCLASS(MinimalAPI, NotBlueprintable, Meta = (DisplayName = "FK Control Rig"))
class UFKControlRig : public UControlRig
{
	GENERATED_UCLASS_BODY()

public: 

	// BEGIN ControlRig
	UE_API virtual void Initialize(bool bInitRigUnits = true) override;
	virtual void InitializeVMs(bool bRequestInit = true) override { URigVMHost::Initialize(bRequestInit); }
	virtual bool InitializeVMs(const FName& InEventName) override { return URigVMHost::InitializeVM(InEventName); }
	virtual void InitializeVMsFromCDO() override { URigVMHost::InitializeFromCDO(); }
	virtual void RequestInitVMs() override { URigVMHost::RequestInit(); }
	UE_API virtual bool Execute_Internal(const FName& InEventName) override;
	UE_API virtual void SetBoneInitialTransformsFromSkeletalMeshComponent(USkeletalMeshComponent* InSkelMeshComp, bool bUseAnimInstance = false) override;
	// END ControlRig

	// utility function to generate a valid control element name
	static UE_API FName GetControlName(const FName& InName, const ERigElementType& InType);
	// utility function to generate a target element name for control
	static UE_API FName GetControlTargetName(const FName& InName, const ERigElementType& InType);

	UE_API TArray<FName> GetControlNames();
	UE_API bool GetControlActive(int32 Index) const;
	UE_API void SetControlActive(int32 Index, bool bActive);
	UE_API void SetControlActive(const TArray<FFKBoneCheckInfo>& InBoneChecks);

	UE_API void SetApplyMode(EControlRigFKRigExecuteMode InMode);
	UE_API void ToggleApplyMode();
	bool CanToggleApplyMode() const { return true; }
	bool IsAdditive() const override { return ApplyMode == EControlRigFKRigExecuteMode::Additive; }
	EControlRigFKRigExecuteMode GetApplyMode() const { return ApplyMode; }

	// Ensures that controls mask is updated according to contained ControlRig (control) elements
	UE_API void RefreshActiveControls();

	struct FRigElementInitializationOptions
	{	
		// Flag whether or not to generate a transform control for bones
		bool bGenerateBoneControls = true;
		// Flag whether or not to generate a float control for all curves in the hierarchy
		bool bGenerateCurveControls = true;
		
		// Flag whether or not to import all curves from SmartNameMapping
		bool bImportCurves = true;

		// Set of bone names to generate a transform control for
		TArray<FName> BoneNames;
		// Set of curve names to generate a float control for (requires bImportCurves to be false)
		TArray<FName> CurveNames;
	};
	void SetInitializationOptions(const FRigElementInitializationOptions& Options) { InitializationOptions = Options; }
	
private:

	/** Create RigElements - bone hierarchy and curves - from incoming skeleton */
	UE_API void CreateRigElements(const USkeletalMesh* InReferenceMesh);
	UE_API void CreateRigElements(const FReferenceSkeleton& InReferenceSkeleton, const USkeleton* InSkeleton);
	UE_API void SetControlOffsetsFromBoneInitials();

	UPROPERTY()
	TArray<bool> IsControlActive;

	UPROPERTY()
	EControlRigFKRigExecuteMode ApplyMode;
	EControlRigFKRigExecuteMode CachedToggleApplyMode;

	FRigElementInitializationOptions InitializationOptions;
	friend class FControlRigInteractionTest;
};

#undef UE_API
