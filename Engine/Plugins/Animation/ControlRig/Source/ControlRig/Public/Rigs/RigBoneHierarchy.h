// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyPose.h"
#include "ReferenceSkeleton.h"
#include "RigBoneHierarchy.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
struct FRigHierarchyContainer;

USTRUCT()
struct FRigBone: public FRigElement
{
	GENERATED_BODY()

	FRigBone()
		: FRigElement()
		, ParentName(NAME_None)
		, ParentIndex(INDEX_NONE)
		, InitialTransform(FTransform::Identity)
		, GlobalTransform(FTransform::Identity)
		, LocalTransform(FTransform::Identity)
		, Dependents()
		, Type(ERigBoneType::Imported)
	{
	}
	virtual ~FRigBone() = default;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = FRigElement)
	FName ParentName;

	UPROPERTY(transient)
	int32 ParentIndex;

	/* Initial global transform that is saved in this rig */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = FRigElement)
	FTransform InitialTransform;

	UPROPERTY(BlueprintReadOnly, transient, EditAnywhere, Category = FRigElement)
	FTransform GlobalTransform;

	UPROPERTY(BlueprintReadOnly, transient, EditAnywhere, Category = FRigElement)
	FTransform LocalTransform;

	/** dependent list - direct dependent for child or anything that needs to update due to this */
	UPROPERTY(transient)
	TArray<int32> Dependents;

	/** the source of the bone to differentiate procedurally generated, imported etc */
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = FRigElement)
	ERigBoneType Type;

	virtual ERigElementType GetElementType() const override
	{
		return ERigElementType::Bone;
	}

	virtual FRigElementKey GetParentElementKey(bool bForce = false) const
	{
		return FRigElementKey(ParentName, GetElementType());
	}
};

USTRUCT()
struct FRigBoneHierarchy
{
	GENERATED_BODY()

	UE_API FRigBoneHierarchy();

	TArray<FRigBone>::RangedForIteratorType      begin()       { return Bones.begin(); }
	TArray<FRigBone>::RangedForConstIteratorType begin() const { return Bones.begin(); }
	TArray<FRigBone>::RangedForIteratorType      end()         { return Bones.end();   }
	TArray<FRigBone>::RangedForConstIteratorType end() const   { return Bones.end();   }

	UE_API FRigBone& Add(const FName& InNewName, const FName& InParentName, ERigBoneType InType, const FTransform& InInitTransform, const FTransform& InLocalTransform, const FTransform& InGlobalTransform);

	// Pretty weird that this type is copy/move assignable (needed for USTRUCTs) but not copy/move constructible
	FRigBoneHierarchy(FRigBoneHierarchy&& InOther) = delete;
	FRigBoneHierarchy(const FRigBoneHierarchy& InOther) = delete;
	FRigBoneHierarchy& operator=(FRigBoneHierarchy&& InOther) = default;
	FRigBoneHierarchy& operator=(const FRigBoneHierarchy& InOther) = default;

private:
	UPROPERTY(EditAnywhere, Category = FRigBoneHierarchy)
	TArray<FRigBone> Bones;

	friend struct FRigHierarchyContainer;
	friend struct FCachedRigElement;
	friend class UControlRigHierarchyModifier;
	friend class UControlRig;
};

#undef UE_API
