// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "UObject/StructOnScope.h"
#include "RigConnectionRules.generated.h"

#define UE_API CONTROLRIG_API

struct FRigBaseElement;
struct FRigConnectionRule;
class FRigElementKeyRedirector;
class URigHierarchy;
struct FRigModuleInstance;
struct FRigBaseElement;
struct FRigTransformElement;
struct FRigConnectorElement;

USTRUCT(BlueprintType)
struct FRigConnectionRuleStash
{
	GENERATED_BODY()

	UE_API FRigConnectionRuleStash();
	UE_API FRigConnectionRuleStash(const FRigConnectionRule* InRule);

	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	
	friend uint32 GetTypeHash(const FRigConnectionRuleStash& InRuleStash);

	UE_API bool IsValid() const;
	UE_API UScriptStruct* GetScriptStruct() const;
	UE_API TSharedPtr<FStructOnScope> Get() const;
	UE_API const FRigConnectionRule* Get(TSharedPtr<FStructOnScope>& InOutStorage) const;

	UE_API bool operator == (const FRigConnectionRuleStash& InOther) const;

	bool operator != (const FRigConnectionRuleStash& InOther) const
	{
		return !(*this == InOther);
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Rule)
	FString ScriptStructPath;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Rule)
	FString ExportedText;
};

struct FRigConnectionRuleInput
{
public:
	
	FRigConnectionRuleInput()
	: Hierarchy(nullptr)
	, Module(nullptr)
	, Redirector(nullptr)
	{
	}

	const URigHierarchy* GetHierarchy() const
	{
		return Hierarchy;
	}
	
	const FRigModuleInstance* GetModule() const
	{
		return Module;
	}
	
	const FRigElementKeyRedirector* GetRedirector() const
	{
		return Redirector;
	}

	UE_API const FRigConnectorElement* FindPrimaryConnector(FText* OutErrorMessage = nullptr) const;
   	UE_API TArray<const FRigConnectorElement*> FindSecondaryConnectors(bool bOptional, FText* OutErrorMessage = nullptr) const;

	UE_API const FRigTransformElement* ResolveConnector(const FRigConnectorElement* InConnector, FText* OutErrorMessage) const;
	UE_API const FRigTransformElement* ResolvePrimaryConnector(FText* OutErrorMessage = nullptr) const;

private:

	const URigHierarchy* Hierarchy;

	/** The specific module if the rule is used in a Modular Rig, or nullptr if it is used in a Control Rig Module */
	const FRigModuleInstance* Module;

	const FRigElementKeyRedirector* Redirector;

	friend class UModularRigRuleManager;
};

USTRUCT(meta=(Hidden))
struct FRigConnectionRule
{
	GENERATED_BODY()

public:

	FRigConnectionRule() {}
	virtual ~FRigConnectionRule() = default;

	virtual UScriptStruct* GetScriptStruct() const { return FRigConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const;
};

USTRUCT(BlueprintType, DisplayName="And Rule")
struct FRigAndConnectionRule : public FRigConnectionRule
{
	GENERATED_BODY()

public:

	FRigAndConnectionRule()
	{}

	template<typename TypeA, typename TypeB>
	FRigAndConnectionRule(const TypeA& InA, const TypeB& InB)
	{
		ChildRules.Emplace(&InA);
		ChildRules.Emplace(&InB);
	}

	virtual ~FRigAndConnectionRule() override = default;

	virtual UScriptStruct* GetScriptStruct() const override { return FRigAndConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const override;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings)
	TArray<FRigConnectionRuleStash> ChildRules;
};

USTRUCT(BlueprintType, DisplayName="Or Rule")
struct FRigOrConnectionRule : public FRigConnectionRule
{
	GENERATED_BODY()

public:

	FRigOrConnectionRule()
	{}

	template<typename TypeA, typename TypeB>
	FRigOrConnectionRule(const TypeA& InA, const TypeB& InB)
	{
		ChildRules.Emplace(&InA);
		ChildRules.Emplace(&InB);
	}

	virtual ~FRigOrConnectionRule() override = default;

	virtual UScriptStruct* GetScriptStruct() const override { return FRigOrConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const override;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings)
	TArray<FRigConnectionRuleStash> ChildRules;
};

USTRUCT(BlueprintType, DisplayName="Type Rule")
struct FRigTypeConnectionRule : public FRigConnectionRule
{
	GENERATED_BODY()

public:

	FRigTypeConnectionRule()
		: ElementType(ERigElementType::Socket)
	{}

	FRigTypeConnectionRule(ERigElementType InElementType)
	: ElementType(InElementType)
	{}

	virtual ~FRigTypeConnectionRule() override = default;

	virtual UScriptStruct* GetScriptStruct() const override { return FRigTypeConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const override;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings)
	ERigElementType ElementType;
};

USTRUCT(BlueprintType, DisplayName="Tag Rule")
struct FRigTagConnectionRule : public FRigConnectionRule
{
	GENERATED_BODY()

public:

	FRigTagConnectionRule()
		: Tag(NAME_None)
	{}

	FRigTagConnectionRule(const FName& InTag)
	: Tag(InTag)
	{}

	virtual ~FRigTagConnectionRule() override = default;

	virtual UScriptStruct* GetScriptStruct() const override { return FRigTagConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const override;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Settings)
	FName Tag;
};

USTRUCT(BlueprintType, DisplayName="Child of Primary")
struct FRigChildOfPrimaryConnectionRule : public FRigConnectionRule
{
	GENERATED_BODY()

public:

	FRigChildOfPrimaryConnectionRule()
	{}

	virtual ~FRigChildOfPrimaryConnectionRule() override = default;

	virtual UScriptStruct* GetScriptStruct() const override { return FRigChildOfPrimaryConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const override;
};

/** A rule that defines how many Connections an Array Connector can have */
USTRUCT(BlueprintType, DisplayName = "Array Size Rule")
struct FRigArraySizeConnectionRule : public FRigConnectionRule
{
	GENERATED_BODY()

public:
	virtual UScriptStruct* GetScriptStruct() const override { return FRigArraySizeConnectionRule::StaticStruct(); }
	UE_API virtual FRigElementResolveResult Resolve(const FRigBaseElement* InTarget, const FRigConnectionRuleInput& InRuleInput) const override;

	/** Returns the min num elments the connections array should have */
	UE_API TOptional<int32> GetMinNumConnections() const;

	/** Returns the max num elments the connections array should have */
	UE_API TOptional<int32> GetMaxNumConnections() const;

private:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = bMinEnabled, ClampMin = 0, AllowPrivateAccess = true))
	int32 MinNumConnections = 1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	bool bMinEnabled = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (EditCondition = bMaxEnabled, ClampMin = 0, AllowPrivateAccess = true))
	int32 MaxNumConnections = 2;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle, AllowPrivateAccess = true))
	bool bMaxEnabled = true;
};

#undef UE_API
