// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyPose.h"
#include "TransformNoScale.h"
#include "EulerTransform.h"
#include "RigControlHierarchy.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class UStaticMesh;
struct FRigHierarchyContainer;

USTRUCT()
struct FRigControl : public FRigElement
{
	GENERATED_BODY()

		FRigControl()
		: FRigElement()
		, ControlType(ERigControlType::Transform)
		, DisplayName(NAME_None)
		, ParentName(NAME_None)
		, ParentIndex(INDEX_NONE)
		, SpaceName(NAME_None)
		, SpaceIndex(INDEX_NONE)
		, OffsetTransform(FTransform::Identity)
		, InitialValue()
		, Value()
		, PrimaryAxis(ERigControlAxis::X)
		, bIsCurve(false)
		, bAnimatable(true)
		, bLimitTranslation(false)
		, bLimitRotation(false)
		, bLimitScale(false)
		, bDrawLimits(true)
		, MinimumValue()
		, MaximumValue()
		, bGizmoEnabled(true)
		, bGizmoVisible(true)
		, GizmoName(TEXT("Gizmo"))
		, GizmoTransform(FTransform::Identity)
		, GizmoColor(FLinearColor::Red)
		, Dependents()
		, bIsTransientControl(false)
		, ControlEnum(nullptr)
	{
	}
	virtual ~FRigControl() = default;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	ERigControlType ControlType;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	FName DisplayName;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Control)
	FName ParentName;

	UPROPERTY(BlueprintReadOnly, transient, Category = Control)
	int32 ParentIndex;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Control)
	FName SpaceName;

	UPROPERTY(BlueprintReadOnly, transient, Category = Control)
	int32 SpaceIndex;

	/**
	 * Used to offset a control in global space. This can be useful
	 * to offset a float control by rotating it or translating it.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control, meta = (DisplayAfter = "ControlType"))
	FTransform OffsetTransform;

	/**
	 * The value that a control is reset to during begin play or when the
	 * control rig is instantiated.
	 */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Control)
	FRigControlValue InitialValue;

	/**
	 * The current value of the control.
	 */
	UPROPERTY(BlueprintReadOnly, transient, VisibleAnywhere, Category = Control)
	FRigControlValue Value;

	/** the primary axis to use for float controls */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	ERigControlAxis PrimaryAxis;

	/** If Created from a Curve  Container*/
	UPROPERTY(transient)
	bool bIsCurve;

	/** If the control is animatable in sequencer */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	bool bAnimatable;

	/** True if the control has to obey translation limits. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	bool bLimitTranslation;

	/** True if the control has to obey rotation limits. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	bool bLimitRotation;

	/** True if the control has to obey scale limits. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	bool bLimitScale;

	/** True if the limits should be drawn in debug. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (EditCondition = "bLimitTranslation || bLimitRotation || bLimitScale"))
	bool bDrawLimits;

	/** The minimum limit of the control's value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (EditCondition = "bLimitTranslation || bLimitRotation || bLimitScale"))
	FRigControlValue MinimumValue;

	/** The maximum limit of the control's value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (EditCondition = "bLimitTranslation || bLimitRotation || bLimitScale"))
	FRigControlValue MaximumValue;

	/** Set to true if the gizmo is enabled in 3d */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Gizmo)
	bool bGizmoEnabled;

	/** Set to true if the gizmo is currently visible in 3d */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Gizmo, meta = (EditCondition = "bGizmoEnabled"))
	bool bGizmoVisible;

	/* This is optional UI setting - this doesn't mean this is always used, but it is optional for manipulation layer to use this*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Gizmo, meta = (EditCondition = "bGizmoEnabled"))
	FName GizmoName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Gizmo, meta = (EditCondition = "bGizmoEnabled"))
	FTransform GizmoTransform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Gizmo, meta = (EditCondition = "bGizmoEnabled"))
	FLinearColor GizmoColor;

	/** dependent list - direct dependent for child or anything that needs to update due to this */
	UPROPERTY(transient)
	TArray<int32> Dependents;

	/** If the control is transient and only visible in the control rig editor */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	bool bIsTransientControl;

	/** If the control is transient and only visible in the control rig editor */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	TObjectPtr<UEnum> ControlEnum;

	virtual ERigElementType GetElementType() const override
	{
		return ERigElementType::Control;
	}

	const FName& GetDisplayName() const
	{
		return DisplayName.IsNone() ? Name : DisplayName;
	}

	virtual FRigElementKey GetParentElementKey() const
	{
		return FRigElementKey(ParentName, GetElementType());
	}

	virtual FRigElementKey GetSpaceElementKey() const
	{
		return FRigElementKey(SpaceName, ERigElementType::Null);
	}

	const FRigControlValue& GetValue(ERigControlValueType InValueType = ERigControlValueType::Current) const
	{
		switch(InValueType)
		{
			case ERigControlValueType::Initial:
			{
				return InitialValue;
			}
			case ERigControlValueType::Minimum:
			{
				return MinimumValue;
			}
			case ERigControlValueType::Maximum:
			{
				return MaximumValue;
			}
			default:
			{
				break;
			}
		}
		return Value;
	}

	FRigControlValue& GetValue(ERigControlValueType InValueType = ERigControlValueType::Current)
	{
		switch(InValueType)
		{
			case ERigControlValueType::Initial:
			{
				return InitialValue;
			}
			case ERigControlValueType::Minimum:
			{
				return MinimumValue;
			}
			case ERigControlValueType::Maximum:
			{
				return MaximumValue;
			}
			default:
			{
				break;
			}
		}
		return Value;
	}

	UE_API void ApplyLimits(FRigControlValue& InOutValue) const;

	static FProperty* FindPropertyForValueType(ERigControlValueType InValueType)
	{
		switch (InValueType)
		{
			case ERigControlValueType::Current:
			{
				return FRigControl::StaticStruct()->FindPropertyByName(TEXT("Value"));
			}
			case ERigControlValueType::Initial:
			{
				return FRigControl::StaticStruct()->FindPropertyByName(TEXT("InitialValue"));
			}
			case ERigControlValueType::Minimum:
			{
				return FRigControl::StaticStruct()->FindPropertyByName(TEXT("MinimumValue"));
			}
			case ERigControlValueType::Maximum:
			{
				return FRigControl::StaticStruct()->FindPropertyByName(TEXT("MaximumValue"));
			}
		}
		return nullptr;
	}

	UE_API FTransform GetTransformFromValue(ERigControlValueType InValueType = ERigControlValueType::Current) const;
	UE_API void SetValueFromTransform(const FTransform& InTransform, ERigControlValueType InValueType = ERigControlValueType::Current);

};

USTRUCT()
struct FRigControlHierarchy
{
	GENERATED_BODY()

	UE_API FRigControlHierarchy();

	int32 Num() const { return Controls.Num(); }
	TArray<FRigControl>::RangedForIteratorType      begin()       { return Controls.begin(); }
	TArray<FRigControl>::RangedForConstIteratorType begin() const { return Controls.begin(); }
	TArray<FRigControl>::RangedForIteratorType      end()         { return Controls.end();   }
	TArray<FRigControl>::RangedForConstIteratorType end() const   { return Controls.end();   }

	UE_API FRigControl& Add(
	    const FName& InNewName,
	    ERigControlType InControlType,
	    const FName& InParentName,
	    const FName& InSpaceName,
	    const FTransform& InOffsetTransform,
	    const FRigControlValue& InValue,
	    const FName& InGizmoName,
	    const FTransform& InGizmoTransform,
	    const FLinearColor& InGizmoColor
	);

	UE_API void PostLoad();

	// Pretty weird that this type is copy/move assignable (needed for USTRUCTs) but not copy/move constructible
	FRigControlHierarchy(FRigControlHierarchy&& InOther) = delete;
	FRigControlHierarchy(const FRigControlHierarchy& InOther) = delete;
	FRigControlHierarchy& operator=(FRigControlHierarchy&& InOther) = default;
	FRigControlHierarchy& operator=(const FRigControlHierarchy& InOther) = default;

private:
	UPROPERTY(EditAnywhere, Category = FRigControlHierarchy)
	TArray<FRigControl> Controls;

	friend struct FRigHierarchyContainer;
	friend struct FCachedRigElement;
	friend class UControlRigHierarchyModifier;
	friend class UControlRig;
	friend class UControlRigBlueprint;
};

#undef UE_API
