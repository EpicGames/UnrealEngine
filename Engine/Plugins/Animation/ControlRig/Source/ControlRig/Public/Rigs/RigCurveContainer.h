// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/StatsHierarchical.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyPose.h"
#include "RigCurveContainer.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class USkeleton;
struct FRigHierarchyContainer;

USTRUCT()
struct FRigCurve : public FRigElement
{
	GENERATED_BODY()

	FRigCurve()
		: FRigElement()
		, Value(0.f)
	{
	}
	virtual ~FRigCurve() = default;

	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = FRigElement)
	float Value;

	virtual ERigElementType GetElementType() const override
	{
		return ERigElementType::Curve;
	}
};

USTRUCT()
struct FRigCurveContainer
{
	GENERATED_BODY()

public:

	UE_API FRigCurveContainer();

	TArray<FRigCurve>::RangedForIteratorType      begin()       { return Curves.begin(); }
	TArray<FRigCurve>::RangedForConstIteratorType begin() const { return Curves.begin(); }
	TArray<FRigCurve>::RangedForIteratorType      end()         { return Curves.end();   }
	TArray<FRigCurve>::RangedForConstIteratorType end() const   { return Curves.end();   }

	UE_API FRigCurve& Add(const FName& InNewName, float InValue);

	// Pretty weird that this type is copy/move assignable (needed for USTRUCTs) but not copy/move constructible
	FRigCurveContainer(FRigCurveContainer&& InOther) = delete;
	FRigCurveContainer(const FRigCurveContainer& InOther) = delete;
	FRigCurveContainer& operator=(FRigCurveContainer&& InOther) = default;
	FRigCurveContainer& operator=(const FRigCurveContainer& InOther) = default;

private:
	UPROPERTY(EditAnywhere, Category = FRigCurveContainer)
	TArray<FRigCurve> Curves;

	friend struct FRigHierarchyContainer;
	friend struct FCachedRigElement;
	friend class UControlRigHierarchyModifier;
};

#undef UE_API
