// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "RigHierarchyDefines.h"
#include "RigDependencyRecords.h"
#include "Units/RigUnitContext.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class URigVM;

/**
 * IRigDependenciesProvider provides an interface for constructing and passing elements dependencies to functions that need to perform dependency tests.
 * This is currently mainly used to ensure that certain rig elements are not dependent on each other (parent switching, constraints ordering, etc.) 
 */

struct IRigDependenciesProvider
{
	virtual ~IRigDependenciesProvider() = default;
	virtual const TRigHierarchyDependencyMap& GetRigHierarchyDependencies() const = 0;
	virtual const TRigHierarchyDependencyMap& GetReverseRigHierarchyDependencies() const = 0;
	virtual void InvalidateCache() = 0;
	virtual uint32 GetHash() const = 0;

	bool IsInteractiveDialogEnabled() const { return bInteractiveDialogEnabled; } 
	void SetInteractiveDialogEnabled(bool InEnabled) { bInteractiveDialogEnabled = InEnabled; }

private:

	bool bInteractiveDialogEnabled = false;
};

/**
 * FEmptyRigDependenciesProvider is the default dependency provider (no dependency at all) 
 */

struct FEmptyRigDependenciesProvider : public IRigDependenciesProvider
{
	virtual ~FEmptyRigDependenciesProvider() override = default;
	UE_API virtual const TRigHierarchyDependencyMap& GetRigHierarchyDependencies() const override;
	UE_API virtual const TRigHierarchyDependencyMap& GetReverseRigHierarchyDependencies() const;
	virtual void InvalidateCache() {}
	virtual uint32 GetHash() const { return 0; }
};

/**
 * FRigDependenciesProviderForVM builds a dependency map based on the provided RigVM's instructions
 */

struct FRigDependenciesProviderForVM : public IRigDependenciesProvider
{
	FRigDependenciesProviderForVM() = default;
	UE_API FRigDependenciesProviderForVM(UControlRig* InControlRig, const FName& InEventName = NAME_None, bool InFollowVariables = false);
	virtual ~FRigDependenciesProviderForVM() override = default;
	
	UE_API virtual const TRigHierarchyDependencyMap& GetRigHierarchyDependencies() const override;
	UE_API virtual const TRigHierarchyDependencyMap& GetReverseRigHierarchyDependencies() const override;
	UE_API virtual void InvalidateCache() override;

	UE_API UControlRig* GetControlRig() const;
	virtual uint32 GetHash() const { return RecordsHash; }

protected:

	void ComputeDependencies(const FControlRigExecuteContext& InContext, const URigVM* InVM, const FName& InVMName, TRigHierarchyDependencyMap& OutDependencies) const;

	TWeakObjectPtr<UControlRig> WeakControlRig = nullptr;
	FName EventName = NAME_None;
	bool bFollowVariables = false;
	
	mutable TRigHierarchyDependencyMap CachedDependencies;
	mutable TRigHierarchyDependencyMap CachedReverseDependencies;
	mutable uint32 RecordsHash = 0;
	mutable uint32 ReverseRecordsHash = 0;
};

/**
 * FRigDependenciesProviderForControlRig builds a dependency map based on the provided rig.
 * This works both for graph based control rigs or modular control rigs.
 */

struct FRigDependenciesProviderForControlRig : public FRigDependenciesProviderForVM
{
public:
	
	FRigDependenciesProviderForControlRig() = default;
	UE_API FRigDependenciesProviderForControlRig(UControlRig* InControlRig, const FName& InEventName = NAME_None, bool InFollowVariables = false);
	virtual ~FRigDependenciesProviderForControlRig() override = default;
	
	UE_API virtual const TRigHierarchyDependencyMap& GetRigHierarchyDependencies() const override;
};

#undef UE_API
