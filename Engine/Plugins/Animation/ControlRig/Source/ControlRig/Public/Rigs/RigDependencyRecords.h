// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "RigHierarchyElements.h"

#define UE_API CONTROLRIG_API

struct FRigHierarchyRecord
{
	enum EType
	{
		EType_RigElement,
		EType_Metadata,
		EType_Instruction,
		EType_Variable,
		EType_Invalid
	};

	FRigHierarchyRecord()
		: Type(EType_Invalid)
		, Index(INDEX_NONE)
		, Name(NAME_None)
	{
	}

	FRigHierarchyRecord(EType InType, int32 InIndex, const FName& InName)
		: Type(InType)
		, Index(InIndex)
		, Name(InName)
	{
	}
	
	bool IsValid() const
	{
		return Type != EType_Invalid;
	}

	bool IsElement() const
	{
		return Type == EType_RigElement;
	}

	bool IsMetadata() const
	{
		return Type == EType_Metadata;
	}

	bool IsInstruction() const
	{
		return Type == EType_Instruction;
	}

	bool IsVariable() const
	{
		return Type == EType_Variable;
	}

	bool operator==(const FRigHierarchyRecord& InRecord) const
	{
		return Type == InRecord.Type && Index == InRecord.Index && Name == InRecord.Name;
	}

	bool operator <(const FRigHierarchyRecord& InRecord) const
	{
		if (Type < InRecord.Type)
		{
			return true;
		}
		if (Type > InRecord.Type)
		{
			return false;
		}
		if (Index < InRecord.Index)
		{
			return true;
		}
		if (Index > InRecord.Index)
		{
			return false;
		}
		return Name.Compare(InRecord.Name) < 0;
	}

	bool operator >(const FRigHierarchyRecord& InRecord) const
	{
		if (Type > InRecord.Type)
		{
			return true;
		}
		if (Type < InRecord.Type)
		{
			return false;
		}
		if (Index > InRecord.Index)
		{
			return true;
		}
		if (Index < InRecord.Index)
		{
			return false;
		}
		return Name.Compare(InRecord.Name) > 0;
	}

	EType Type;
	int32 Index;
	FName Name;
};

typedef TMap<FRigHierarchyRecord, TSet<FRigHierarchyRecord>> TRigHierarchyDependencyMap;
typedef TPair<FRigHierarchyRecord, TSet<FRigHierarchyRecord>> TRigHierarchyDependencyMapPair;
typedef TArray<TPair<FRigHierarchyRecord, FRigHierarchyRecord>> FRigHierarchyDependencyChain;

inline uint32 GetTypeHash(const FRigHierarchyRecord& InRecord)
{
	return HashCombine(GetTypeHash(static_cast<int32>(InRecord.Type)),
		HashCombine(GetTypeHash(InRecord.Index), GetTypeHash(InRecord.Name)));
}

struct FInstructionRecord
{
	FInstructionRecord()
		: InstructionIndex(INDEX_NONE)
		, SliceIndex(INDEX_NONE)
		, ElementIndex(INDEX_NONE)
		, VM(NAME_None)
		, MetadataName(NAME_None)
	{
	}

	FInstructionRecord(int32 InInstructionIndex, int32 InSliceIndex, int32 InElementIndex, const FName& InVM = NAME_None, const FName& InMetadataName = NAME_None)
		:InstructionIndex(InInstructionIndex)
		, SliceIndex(InSliceIndex)
		, ElementIndex(InElementIndex)
		, VM(InVM)
		, MetadataName(InMetadataName)
	{
	}

	bool operator==(const FInstructionRecord& InRecord) const
	{
		return InstructionIndex == InRecord.InstructionIndex &&
			SliceIndex == InRecord.SliceIndex &&
			ElementIndex == InRecord.ElementIndex &&
			VM == InRecord.VM &&
			MetadataName == InRecord.MetadataName;
	}

	bool operator <(const FInstructionRecord& InRecord) const
	{
		if (InstructionIndex < InRecord.InstructionIndex)
		{
			return true;
		}
		if (InstructionIndex > InRecord.InstructionIndex)
		{
			return false;
		}
		if (SliceIndex < InRecord.SliceIndex)
		{
			return true;
		}
		if (SliceIndex > InRecord.SliceIndex)
		{
			return false;
		}
		if (ElementIndex < InRecord.ElementIndex)
		{
			return true;
		}
		if (ElementIndex > InRecord.ElementIndex)
		{
			return false;
		}
		const int32 VMComp = VM.Compare(InRecord.VM);
		if (VMComp < 0)
		{
			return true;
		}
		if (VMComp > 0)
		{
			return false;
		}
		return MetadataName.Compare(InRecord.MetadataName) < 0;
	}

	bool operator >(const FInstructionRecord& InRecord) const
	{
		if (InstructionIndex > InRecord.InstructionIndex)
		{
			return true;
		}
		if (InstructionIndex < InRecord.InstructionIndex)
		{
			return false;
		}
		if (SliceIndex > InRecord.SliceIndex)
		{
			return true;
		}
		if (SliceIndex < InRecord.SliceIndex)
		{
			return false;
		}
		if (ElementIndex > InRecord.ElementIndex)
		{
			return true;
		}
		if (ElementIndex < InRecord.ElementIndex)
		{
			return false;
		}
		const int32 VMComp = VM.Compare(InRecord.VM);
		if (VMComp > 0)
		{
			return true;
		}
		if (VMComp < 0)
		{
			return false;
		}
		return MetadataName.Compare(InRecord.MetadataName) > 0;
	}

	int32 InstructionIndex;
	int32 SliceIndex;
	int32 ElementIndex;
	FName VM;
	FName MetadataName;
};

inline uint32 GetTypeHash(const FInstructionRecord& InRecord)
{
	uint32 Hash = HashCombine(GetTypeHash(InRecord.InstructionIndex), GetTypeHash(InRecord.SliceIndex));
	Hash = HashCombine(Hash, GetTypeHash(InRecord.ElementIndex));
	Hash = HashCombine(Hash, GetTypeHash(InRecord.VM));
	Hash = HashCombine(Hash, GetTypeHash(InRecord.MetadataName));
	return Hash;
}

typedef TMap<FInstructionRecord, TArray<FInstructionRecord>> TInstructionRecordMap;
typedef TPair<FInstructionRecord, TArray<FInstructionRecord>> TInstructionRecordMapPair;

struct FInstructionRecordContainer
{
	FInstructionRecordContainer()
		: ReadRecordsHash(UINT32_MAX)
		, WrittenRecordsHash(UINT32_MAX)
	{
	}
	
	mutable TArray<FInstructionRecord> ReadRecords;
	mutable TArray<FInstructionRecord> WrittenRecords;
	mutable uint32 ReadRecordsHash;
	mutable uint32 WrittenRecordsHash;
};

#undef UE_API
