// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVM.h"
#include "RigName.h"
#include "RigHierarchyElements.h"
#include "RigHierarchyCache.h"
#include "RigHierarchyPose.h"
#include "RigHierarchyPoseAdapter.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "EdGraph/EdGraphPin.h"
#include "RigHierarchyDefines.h"
#if WITH_EDITOR
#include "RigVMPythonUtils.h"
#endif
#include "RigDependency.h"
#include "Containers/Queue.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "RigHierarchy.generated.h"

class UControlRig;
class URigHierarchy;
class URigHierarchyController;
class UModularRigRuleManager; 

DECLARE_MULTICAST_DELEGATE_ThreeParams(FRigHierarchyModifiedEvent, ERigHierarchyNotification /* type */, URigHierarchy* /* hierarchy */, const FRigNotificationSubject& /* element or component */);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FRigHierarchyModifiedDynamicEvent, ERigHierarchyNotification, NotifType, URigHierarchy*, Hierarchy, FRigElementKey, Subject);
DECLARE_EVENT_FiveParams(URigHierarchy, FRigHierarchyUndoRedoTransformEvent, URigHierarchy*, const FRigElementKey&, ERigTransformType::Type, const FTransform&, bool /* bUndo */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FRigHierarchyMetadataChangedDelegate, const FRigElementKey& /* Key */, const FName& /* Name */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FRigHierarchyMetadataTagChangedDelegate, const FRigElementKey& /* Key */, const FName& /* Tag */, bool /* AddedOrRemoved */);
DECLARE_DELEGATE_RetVal_FourParams(bool, FRigHierarchyDismissDependencyDelegate, const URigHierarchy* /* InHierarchy */, const FRigElementKey& /* InChild */, const FRigElementKey& /* InParent */, const FRigHierarchyDependencyChain& /* InDependencyChain */);

extern CONTROLRIG_API TAutoConsoleVariable<bool> CVarControlRigHierarchyEnableRotationOrder;
extern CONTROLRIG_API TAutoConsoleVariable<bool> CVarControlRigHierarchyEnableModules;

#define UE_RIGHIERARCHY_DEFAULT_MAX_NAME_LENGTH 100 
#define UE_RIGHIERARCHY_NUMERIC_SUFFIX_LEN 4 

UENUM()
enum ERigTransformStackEntryType : int
{
	TransformPose,
	ControlOffset,
	ControlShape,
	CurveValue
};

USTRUCT()
struct FRigTransformStackEntry
{
	GENERATED_BODY()

	FRigTransformStackEntry()
		: Key()
		, EntryType(ERigTransformStackEntryType::TransformPose)
		, TransformType(ERigTransformType::CurrentLocal)
		, OldTransform(FTransform::Identity)
		, NewTransform(FTransform::Identity)
		, bAffectChildren(true)
		, Callstack() 
	{}

	FRigTransformStackEntry(
		const FRigElementKey& InKey,
		ERigTransformStackEntryType InEntryType,
		ERigTransformType::Type InTransformType,
		const FTransform& InOldTransform,
		const FTransform& InNewTransform,
		bool bInAffectChildren,
		const TArray<FString>& InCallstack =  TArray<FString>())
		: Key(InKey)
		, EntryType(InEntryType)
		, TransformType(InTransformType)
		, OldTransform(InOldTransform)
		, NewTransform(InNewTransform)
		, bAffectChildren(bInAffectChildren)
		, Callstack(InCallstack)
	{}

	UPROPERTY()
	FRigElementKey Key;

	UPROPERTY()
	TEnumAsByte<ERigTransformStackEntryType> EntryType;

	UPROPERTY()
	TEnumAsByte<ERigTransformType::Type> TransformType;
	
	UPROPERTY()
	FTransform OldTransform;

	UPROPERTY()
	FTransform NewTransform;

	UPROPERTY()
	bool bAffectChildren;

	UPROPERTY()
	TArray<FString> Callstack;
};

template<typename T>
class THierarchyCache
{
public:

	THierarchyCache()
		: TopologyVersion(0)
	{}

	THierarchyCache(const T& InValue, uint32 InTopologyVersion)
		: THierarchyCache()
	{
		Value = InValue;
		TopologyVersion = InTopologyVersion;
	}

	bool IsValid(uint32 InTopologyVersion) const
	{
		return (TopologyVersion == InTopologyVersion) && Value.IsSet();
	}

	void Reset()
	{
		TopologyVersion = 0;
		Value = TOptional<T>();
	}

	const T& Get() const
	{
		return Value.GetValue();
	}

	T& Get()
	{
		if(!Value.IsSet())
		{
			Value = T();
		}
		return Value.GetValue();
	}

	void Set(uint32 InTopologyVersion)
	{
		check(Value.IsSet());
		TopologyVersion = InTopologyVersion;
	}

	void Set(const T& InValue, uint32 InTopologyVersion)
	{
		Value = InValue;
		TopologyVersion = InTopologyVersion;
	}

private:

	uint32 TopologyVersion;
	TOptional<T> Value;
};

UCLASS(MinimalAPI, BlueprintType)
class URigHierarchy : public UObject
{
	GENERATED_BODY()

public:

	inline static const FLazyName TagMetadataName = TEXT("Tags");
	inline static const FLazyName ShortModuleNameMetadataName_Deprecated = TEXT("ShortModuleName");
	inline static const FLazyName DesiredNameMetadataName = TEXT("DesiredName");
	inline static const FLazyName DesiredKeyMetadataName = TEXT("DesiredKey");
	inline static const FLazyName ModuleMetadataName = TEXT("Module");
	inline static const FLazyName NameSpaceMetadataName_Deprecated = TEXT("NameSpace");
	inline static const FLazyName ShortNameMetadataName_Deprecated = TEXT("ShortName");

	CONTROLRIG_API URigHierarchy();

	// UObject interface
	CONTROLRIG_API virtual void BeginDestroy() override;
	CONTROLRIG_API virtual void Serialize(FArchive& Ar) override;
	static CONTROLRIG_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	CONTROLRIG_API void Save(FArchive& Ar);
	CONTROLRIG_API void Load(FArchive& Ar);
	CONTROLRIG_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static CONTROLRIG_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
#if WITH_EDITOR
	virtual void PreEditUndo() override;
	CONTROLRIG_API virtual void PostEditUndo() override;
#endif

	/**
	 * Clears the whole hierarchy and removes all elements.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void Reset();

	/**
	 * Resets the hierarchy to the state of its default. This refers to the
	 * hierarchy on the default object.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void ResetToDefault();

	/**
	 * Copies the contents of a hierarchy onto this one
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void CopyHierarchy(URigHierarchy* InHierarchy);

	bool IsCopyingHierarchy() const { return bIsCopyingHierarchy; }

	/**
	 * Returns true if the hierarchy currently has an execute context / the rig is running
	 */
	bool HasExecuteContext() const { return ExecuteContext != nullptr; }

	/**
	 * Returns a hash for the hierarchy representing all names
	 * as well as the topology version.
	 */
	CONTROLRIG_API uint32 GetNameHash() const;

	/**
	 * Returns a hash representing the topological state of the hierarchy
	 */
	CONTROLRIG_API uint32 GetTopologyHash(bool bIncludeTopologyVersion = true, bool bIncludeTransientControls = false) const;

#if WITH_EDITOR
	/**
	 * Returns true if this hierarchy has unique short names for a given element type.
	 */
	CONTROLRIG_API bool HasOnlyUniqueShortNames(ERigElementType InElementType) const;

	/**
	 * Returns true if in this hierarchy a given short name is unique
	 */
	CONTROLRIG_API bool HasUniqueShortName(ERigElementType InElementType, const FRigName& InName) const;
	CONTROLRIG_API bool HasUniqueShortName(ERigElementType InElementType, const FString& InName) const;
	CONTROLRIG_API bool HasUniqueShortName(const FRigBaseElement* InElement) const;
	
	/**
	* Add dependent hierarchies that listens to changes made to this hierarchy
	* Note: By default, only changes to the initial states of this hierarchy is mirrored to the listening hierarchies
	*/	
	CONTROLRIG_API void RegisterListeningHierarchy(URigHierarchy* InHierarchy);
	
	/**
	* Remove dependent hierarchies that listens to changes made to this hierarchy
	*/	
	CONTROLRIG_API void UnregisterListeningHierarchy(URigHierarchy* InHierarchy);
	
	/**
	* Remove all dependent hierarchies
	*/
	CONTROLRIG_API void ClearListeningHierarchy();
	
	/**
	* Cleanup dependent hierarchies that are marked as garbage
	*/
	CONTROLRIG_API void CleanupListeningHierarchies();

#endif

	/**
	 * Returns the default hierarchy for this hierarchy (or nullptr)
	 */
	URigHierarchy* GetDefaultHierarchy() { return DefaultHierarchyPtr.Get(); }

public:
	/**
	 * Copies the contents of a hierarchy onto this one
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void CopyPose(URigHierarchy* InHierarchy, bool bCurrent, bool bInitial, bool bWeights, bool bMatchPoseInGlobalIfNeeded = false);

	/**
	 * Update all elements that depend on external references
	 */
	CONTROLRIG_API void UpdateReferences(const FRigVMExecuteContext* InContext);

	/**
	 * Resets the current pose of a filtered list of elements to the initial / ref pose.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void ResetPoseToInitial(ERigElementType InTypeFilter);

	/**
	 * Resets the current pose of all elements to the initial / ref pose.
	 */
	void ResetPoseToInitial()
	{
		ResetPoseToInitial(ERigElementType::All);
	}

	/**
	 * Resets all curves to 0.0
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void ResetCurveValues();

	/**
	 * Resets all curves to be unset (defaulting to 0.0)
	 */
	CONTROLRIG_API void UnsetCurveValues(bool bSetupUndo = false);

	/**
	 * Returns all changed curve values
	 */
	CONTROLRIG_API const TArray<int32>& GetChangedCurveIndices() const;

	/**
	 * Returns all changed curve values
	 */
	CONTROLRIG_API void ResetChangedCurveIndices();

	/**
	 * Returns the flag used decide if we should be recording curve changes
	 */
	bool& GetRecordCurveChangesFlag() { return bRecordCurveChanges; }
	
	/**
	 * Returns the number of elements in the Hierarchy.
	 * @return The number of elements in the Hierarchy
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	int32 Num() const
	{
		return Elements.Num();
	}

	/**
	 * Returns the number of elements in the Hierarchy.
	 * @param InElementType The type filter to apply
	 * @return The number of elements in the Hierarchy
	 */
	CONTROLRIG_API int32 Num(ERigElementType InElementType) const;

	// iterators
	TArray<FRigBaseElement*>::RangedForIteratorType      begin() { return Elements.begin(); }
	TArray<FRigBaseElement*>::RangedForIteratorType      end() { return Elements.end(); }

	/**
	 * Iterator function to invoke a lambda / TFunctionRef for each element
	 * @param PerElementFunction The function to invoke for each element
	 */
	void ForEach(TFunctionRef<bool(FRigBaseElement*)> PerElementFunction) const
	{
		for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
		{
			if(!PerElementFunction(Elements[ElementIndex]))
			{
				return;
			}
		}
	}
	
	/**
	 * Filtered template Iterator function to invoke a lambda / TFunctionRef for each element of a given type.
	 * @param PerElementFunction The function to invoke for each element of a given type
	 */
	template<typename T>
	void ForEach(TFunctionRef<bool(T*)> PerElementFunction) const
	{
		constexpr int32 FlatIndex = ElementToFlatIndex<T>();
		for (FRigBaseElement* Element: ElementsPerType.IsValidIndex(FlatIndex) ? ElementsPerType[FlatIndex] : Elements)
		{
			if(T* CastElement = Cast<T>(Element))
			{
				if(!PerElementFunction(CastElement))
				{
					return;
				}
			}
		}
	}

	/**
	 * Returns true if the provided element index is valid
	 * @param InElementIndex The index to validate
	 * @return Returns true if the provided element index is valid
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsValidIndex(int32 InElementIndex) const
	{
		return Elements.IsValidIndex(InElementIndex);
	}

	/**
	 * Returns true if the provided element key is valid
	 * @param InKey The key to validate
	 * @return Returns true if the provided element key is valid
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Contains", ScriptName = "Contains"))
	bool Contains_ForBlueprint(FRigElementKey InKey) const
	{
		return Contains(InKey);
	}

	/**
	 * Returns true if the provided element key is valid
	 * @param InKey The key to validate
	 * @return Returns true if the provided element key is valid
	 */
	bool Contains(const FRigElementKey& InKey) const
	{
		return GetIndex(InKey) != INDEX_NONE;
	}

	/**
	 * Returns true if the provided element key is valid as a certain typename
	 * @param InKey The key to validate
	 * @return Returns true if the provided element key is valid
	 */
	template<typename T>
	bool Contains(const FRigElementKey& InKey) const
	{
		return Find<T>(InKey) != nullptr;
	}

	/**
	 * Returns true if the provided element is procedural.
	 * @param InKey The key to validate
	 * @return Returns true if the element is procedural
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API bool IsProcedural(const FRigElementKey& InKey) const;

	/**
	 * Returns true if the provided element is procedural.
	 * @param InElement The element to check
	 * @return Returns true if the element is procedural
	 */
	CONTROLRIG_API bool IsProcedural(const FRigBaseElement* InElement) const;

	/**
	 * Returns true if the provided component is procedural.
	 * @param InKey The key to validate
	 * @return Returns true if the component is procedural
	 */
	CONTROLRIG_API bool IsProcedural(const FRigComponentKey& InKey) const;

	/**
	 * Returns true if the provided component is procedural.
	 * @param InComponent The component to check
	 * @return Returns true if the component is procedural
	 */
	CONTROLRIG_API bool IsProcedural(const FRigBaseComponent* InComponent) const;

	/**
	 * Returns true if the provided component or element is procedural.
	 * @param InKey The key to validate
	 * @return Returns true if the component or element is procedural
	 */
	CONTROLRIG_API bool IsProcedural(const FRigHierarchyKey& InKey) const;

	/**
	 * Returns the index of an element given its key
	 * @param InKey The key of the element to retrieve the index for
	 * @return The index of the element or INDEX_NONE
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Index", ScriptName = "GetIndex"))
	int32 GetIndex_ForBlueprint(FRigElementKey InKey) const
	{
		return GetIndex(InKey);
	}

	/**
	 * Returns the index of an element given its key
	 * @param InKey The key of the element to retrieve the index for
	 * @param bFollowRedirector false if we want to retrieve a connector itself
	 * @return The index of the element or INDEX_NONE
	 */
	CONTROLRIG_API int32 GetIndex(const FRigElementKey& InKey, bool bFollowRedirector = true) const;

	/**
	 * Returns the spawn index of an element / component given its key.
	 * The spawn index indicates the spawning order of elements.
	 * @param InKey The key of the element to retrieve the spawn index for
	 * @return The spawn index of the element or INDEX_NONE
	 */
	CONTROLRIG_API int32 GetSpawnIndex(const FRigHierarchyKey& InKey) const;

	/**
	 * Returns the key and index pair of an element given its key
	 * @param InKey The key of the element to retrieve the information for
	 * @return The key and index pair of the element
	 */
	FRigElementKeyAndIndex GetKeyAndIndex(const FRigElementKey& InKey) const
	{
		return GetKeyAndIndex(GetIndex(InKey));
	};

	/**
	 * Returns the key and index pair of an element given its index
	 * @param InIndex The index of the element to retrieve the information for
	 * @return The key and index pair of the element
	 */
	FRigElementKeyAndIndex GetKeyAndIndex(int32 InIndex) const
	{
		if(const FRigBaseElement* Element = Get(InIndex))
		{
			return Element->GetKeyAndIndex();
		}
		return FRigElementKeyAndIndex();
	};

	/**
	 * Returns the index of an element given its key within its default parent (or root)
	 * @param InKey The key of the element to retrieve the index for
	 * @return The index of the element or INDEX_NONE
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Local Index", ScriptName = "GetLocalIndex"))
	int32 GetLocalIndex_ForBlueprint(FRigElementKey InKey) const
	{
		return GetLocalIndex(InKey);
	}

	/**
	 * Returns the index of an element given its key within its default parent (or root)
	 * @param InKey The key of the element to retrieve the index for
	 * @return The index of the element or INDEX_NONE
	 */
	int32 GetLocalIndex(const FRigElementKey& InKey) const
	{
		return GetLocalIndex(Find(InKey));
	}

	/**
	 * Returns the indices of an array of keys
	 * @param InKeys The keys of the elements to retrieve the indices for
	 * @return The indices of the elements or INDEX_NONE
	 */
	TArray<int32> GetIndices(const TArray<FRigElementKey>& InKeys) const
	{
		TArray<int32> Indices;
		for(const FRigElementKey& Key : InKeys)
		{
			Indices.Add(GetIndex(Key));
		}
		return Indices;
	}

	/**
	 * Returns the key of an element given its index
	 * @param InElementIndex The index of the element to retrieve the key for
	 * @return The key of an element given its index
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FRigElementKey GetKey(int32 InElementIndex) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			return Elements[InElementIndex]->Key;
		}
		return FRigElementKey();
	}

	/**
	 * Returns the keys of an array of indices
	 * @param InElementIndices The indices to retrieve the keys for
	 * @return The keys of the elements given the indices
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	TArray<FRigElementKey> GetKeys(const TArray<int32> InElementIndices) const
	{
		TArray<FRigElementKey> Keys;
		for(int32 Index : InElementIndices)
		{
			Keys.Add(GetKey(Index));
		}
		return Keys;
	}

	/**
	 * Returns an element at a given index or nullptr.
	 * @param InIndex The index of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	const FRigBaseElement* Get(int32 InIndex) const
	{
		if(Elements.IsValidIndex(InIndex))
		{
			return Elements[InIndex];
		}
		return nullptr;
	}

	/**
	 * Returns an element at a given index or nullptr.
	 * @param InIndex The index of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	FRigBaseElement* Get(int32 InIndex)
	{
		if(Elements.IsValidIndex(InIndex))
		{
			return Elements[InIndex];
		}
		return nullptr;
	}

	/**
	 * Returns an element at a given index or nullptr.
	 * This templated method also casts to the chosen
	 * element type but does not guarantee a valid result.
	 * @param InIndex The index of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	const T* Get(int32 InIndex) const
	{
		return Cast<T>(Get(InIndex));
	}

	/**
	 * Returns an element at a given index or nullptr.
	 * This templated method also casts to the chosen
	 * element type but does not guarantee a valid result.
	 * @param InIndex The index of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	T* Get(int32 InIndex)
	{
		return Cast<T>(Get(InIndex));
	}

	/**
	 * Returns an element at a given index.
	 * This templated method also casts to the chosen
	 * element type and checks for a the valid result.
	 * @param InIndex The index of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	const T* GetChecked(int32 InIndex) const
	{
		return CastChecked<T>(Get(InIndex));
	}

	/**
	 * Returns an element at a given index.
	 * This templated method also casts to the chosen
	 * element type and checks for a the valid result.
	 * @param InIndex The index of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	T* GetChecked(int32 InIndex)
	{
		return CastChecked<T>(Get(InIndex));
	}

	/**
	 * Returns a handle to an existing element
	 * @param InKey The key of the handle to retrieve.
	 * @return The retrieved handle (may be invalid)
	 */
	FRigElementHandle GetHandle(const FRigElementKey& InKey) const
	{
		if(Contains(InKey))
		{
			return FRigElementHandle((URigHierarchy*)this, InKey);
		}
		return FRigElementHandle();
	}

	/**
	 * Returns an element for a given key or nullptr.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	const FRigBaseElement* Find(const FRigElementKey& InKey) const
	{
		return Get(GetIndex(InKey));
	}

	/**
	 * Returns an element for a given key or nullptr.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	FRigBaseElement* Find(const FRigElementKey& InKey)
	{
		return Get(GetIndex(InKey));
	}

	/**
	 * Returns an element for a given key and raises for invalid results.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	const FRigBaseElement* FindChecked(const FRigElementKey& InKey) const
	{
		const FRigBaseElement* Element = Get(GetIndex(InKey));
		check(Element);
		return Element;
	}

	/**
	 * Returns an element for a given key and raises for invalid results.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	FRigBaseElement* FindChecked(const FRigElementKey& InKey)
	{
		FRigBaseElement* Element = Get(GetIndex(InKey));
		check(Element);
		return Element;
	}

	/**
	 * Returns an element for a given key or nullptr.
	 * This templated method also casts to the chosen
	 * element type but does not guarantee a valid result.
	 * @param InKey The key of the element to retrieve.
	 * @param bFollowRedirector false if we want to retrieve a connector itself
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	const T* Find(const FRigElementKey& InKey, bool bFollowRedirector = true) const
	{
		return Get<T>(GetIndex(InKey, bFollowRedirector));
	}

	/**
	 * Returns an element for a given key or nullptr.
	 * This templated method also casts to the chosen
	 * element type but does not guarantee a valid result.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	T* Find(const FRigElementKey& InKey)
	{
		return Get<T>(GetIndex(InKey));
	}

	/**
	 * Returns a component for a given key or nullptr.
	 * @param InKey The key of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	CONTROLRIG_API const FRigBaseComponent* FindComponent(const FRigComponentKey& InKey) const;

	/**
	 * Returns a component for a given key or nullptr.
	 * @param InKey The key of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	CONTROLRIG_API FRigBaseComponent* FindComponent(const FRigComponentKey& InKey);

	/**
	 * Returns the index of a component or INDEX_NONE
	 * @param InComponentKey The key of the component to look up
	 * @param bFollowRedirector If True the element key will be resolved using the element redirector
	 * @return The index of the component
	 */
	CONTROLRIG_API int32 GetComponentIndex(const FRigComponentKey& InComponentKey, bool bFollowRedirector = true) const;

	/**
	 * Returns a component at a given index or nullptr.
	 * @param InIndex The index of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	CONTROLRIG_API const FRigBaseComponent* GetComponent(int32 InIndex) const;

	/**
	 * Returns a component at a given index or nullptr.
	 * @param InIndex The index of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	CONTROLRIG_API FRigBaseComponent* GetComponent(int32 InIndex);

	/**
	 * Returns a component at a given index or nullptr.
	 * This templated method also casts to the chosen
	 * component type but does not guarantee a valid result.
	 * @param InIndex The index of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	template<typename T>
	const T* GetComponent(int32 InIndex) const
	{
		return Cast<T>(GetComponent(InIndex));
	}

	/**
	 * Returns a component at a given index or nullptr.
	 * This templated method also casts to the chosen
	 * component type but does not guarantee a valid result.
	 * @param InIndex The index of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	template<typename T>
	T* GetComponent(int32 InIndex)
	{
		return Cast<T>(GetComponent(InIndex));
	}

	/**
	 * Returns a component at a given index.
	 * This templated method also casts to the chosen
	 * component type and checks for a the valid result.
	 * @param InIndex The index of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	template<typename T>
	const T* GetComponentChecked(int32 InIndex) const
	{
		return CastChecked<T>(GetComponent(InIndex));
	}

	/**
	 * Returns a component at a given index.
	 * This templated method also casts to the chosen
	 * component type and checks for a the valid result.
	 * @param InIndex The index of the component to retrieve.
	 * @return The retrieved component or nullptr.
	 */
	template<typename T>
	T* GetComponentChecked(int32 InIndex)
	{
		return CastChecked<T>(GetComponent(InIndex));
	}

	/**
	 * Returns all components of a given type
	 * @param InComponentStruct The component type to look up
	 * @return All components of type InComponentStruct
	 */
	CONTROLRIG_API TArray<const FRigBaseComponent*> GetComponents(const UScriptStruct* InComponentStruct) const;

	/**
	 * Returns all components of the template type
	 * @return All components of the template type
	 */
	template<typename T>
	TArray<const T*> GetComponents() const
	{
		TArray<const FRigBaseComponent*> Components = GetComponents(T::StaticStruct());
		TArray<const T*> Result;
		Result.SetNumZeroed(Components.Num());
		FMemory::Memcpy(Result.GetData(), Components.GetData(), Components.GetTypeSize() * Components.Num());
		return Result;
	}

	/**
	 * Returns the count of all components
	 * @return The count of all components
	 */
	CONTROLRIG_API int32 NumComponents() const;

	/**
	 * Returns the count of all components of a given type
	 * @param InComponentStruct The component type to look up
	 * @return The count of all components of type InComponentStruct
	 */
	CONTROLRIG_API int32 NumComponents(const UScriptStruct* InComponentStruct) const;

	/**
	 * Returns the count of all components of the template type
	 * @return The count of all components of the template type
	 */
	template<typename T>
	int32 NumComponents() const
	{
		return NumComponents(T::StaticStruct());
	}

	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API int32 NumComponents(FRigElementKey InElement) const;

	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigComponentKey> GetAllComponentKeys() const;

	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigComponentKey> GetComponentKeys(FRigElementKey InElement) const;

	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API FRigComponentKey GetComponentKey(FRigElementKey InElement, int32 InComponentIndex) const;

	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API FName GetComponentName(FRigElementKey InElement, int32 InComponentIndex) const;
	
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API UScriptStruct* GetComponentType(FRigElementKey InElement, int32 InComponentIndex) const;

	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API FString GetComponentContent(FRigElementKey InElement, int32 InComponentIndex) const;

	CONTROLRIG_API bool CanAddComponent(FRigElementKey InElementKey, const UScriptStruct* InComponentStruct, FString* OutFailureReason = nullptr) const;
	CONTROLRIG_API bool CanAddComponent(FRigElementKey InElementKey, const FRigBaseComponent* InComponent, FString* OutFailureReason = nullptr) const;

	CONTROLRIG_API int32 GetNextSpawnIndex() const;

private:
	/**
	* Returns bone element for a given key, for scripting purpose only, for cpp usage, use Find<FRigBoneElement>()
	* @param InKey The key of the bone element to retrieve. 
	*/
	UFUNCTION(BlueprintCallable, Category = URigHierarchy,
		meta = (DisplayName = "Find Bone", ScriptName = "FindBone",
			DeprecatedFunction, DeprecationMessage = "Use Contains to check if the element exists, GetBoneType for the bone type, IsSelected for selection state, or GetIndex to retrieve the element index."))
	FRigBoneElement FindBone_ForBlueprintOnly(const FRigElementKey& InKey) const
	{
		if (const FRigBoneElement* Bone = Find<FRigBoneElement>(InKey))
		{
			return *Bone;
		}
		return FRigBoneElement{};
	}

	/**
	 * Returns the bone type for a bone identified by key.
	 * @param InKey The element key of the bone
	 * @return The ERigBoneType, or ERigBoneType::User if the key is invalid
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy,
		meta = (DisplayName = "Get Bone Type", ScriptName = "GetBoneType"))
	ERigBoneType GetBoneType_ForBlueprintOnly(const FRigElementKey& InKey) const
	{
		if (const FRigBoneElement* Bone = Find<FRigBoneElement>(InKey))
		{
			return Bone->BoneType;
		}
		return ERigBoneType::User;
	}

	/**
	* Returns control element for a given key, for scripting purpose only, for cpp usage, use Find<FRigControlElement>()
	* @param InKey The key of the control element to retrieve.
	*/
	UFUNCTION(BlueprintCallable, Category = URigHierarchy,
		meta = (DisplayName = "Find Control", ScriptName = "FindControl",
			DeprecatedFunction, DeprecationMessage = "Use Contains to check if the element exists, GetControlSettings for settings, IsSelected for selection state, or GetIndex to retrieve the element index."))
	FRigControlElement FindControl_ForBlueprintOnly(const FRigElementKey& InKey) const
	{
		if (const FRigControlElement* Control = Find<FRigControlElement>(InKey))
		{
			return *Control;
		}
		return FRigControlElement{};
	}

	/**
	 * Returns the settings for a control identified by key.
	 * @param InKey The element key of the control
	 * @return The FRigControlSettings, or a default-constructed one if the key is invalid
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy,
		meta = (DisplayName = "Get Control Settings", ScriptName = "GetControlSettings"))
	FRigControlSettings GetControlSettings_ForBlueprintOnly(const FRigElementKey& InKey) const
	{
		if (const FRigControlElement* Control = Find<FRigControlElement>(InKey))
		{
			return Control->Settings;
		}
		return FRigControlSettings();
	}

	/**
	* Returns null element for a given key, for scripting purpose only, for cpp usage, use Find<FRigControlElement>()
	* @param InKey The key of the null element to retrieve. 
	*/
	UFUNCTION(BlueprintCallable, Category = URigHierarchy,
		meta = (DisplayName = "Find Null", ScriptName = "FindNull",
			DeprecatedFunction, DeprecationMessage = "Use Contains to check if the element exists, IsSelected for selection state, or GetIndex to retrieve the element index."))
	FRigNullElement FindNull_ForBlueprintOnly(const FRigElementKey& InKey) const
	{
		if (const FRigNullElement* Null = Find<FRigNullElement>(InKey))
		{
			return *Null;
		}
		return FRigNullElement{};
	}
	
public:	
	/**
	 * Returns an element for a given key.
	 * This templated method also casts to the chosen
	 * element type and checks for a the valid result.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	const T* FindChecked(const FRigElementKey& InKey) const
	{
		return GetChecked<T>(GetIndex(InKey));
	}

	/**
	 * Returns an element for a given key.
	 * This templated method also casts to the chosen
	 * element type and checks for a the valid result.
	 * @param InKey The key of the element to retrieve.
	 * @return The retrieved element or nullptr.
	 */
	template<typename T>
	T* FindChecked(const FRigElementKey& InKey)
	{
		return GetChecked<T>(GetIndex(InKey));
	}

	/**
	 * Filtered accessor to retrieve all elements of a given type
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 * @param bIgnoreInactiveRelations Ignores zero-weight parent-child relationships
	 */
	template<typename T>
	TArray<T*> GetElementsOfType(bool bTraverse = false, bool bIgnoreInactiveRelations = false) const
	{
		TArray<T*> Results;

		if(bTraverse)
		{
			TArray<bool> ElementVisited;
			ElementVisited.AddZeroed(Elements.Num());

			Traverse([&ElementVisited, &Results](FRigBaseElement* InElement, bool& bContinue)
			{
				bContinue = false;
				if (ElementVisited.IsValidIndex(InElement->GetIndex()))
				{
					bContinue = !ElementVisited[InElement->GetIndex()];

				   if(bContinue)
				   {
					   if(T* CastElement = Cast<T>(InElement))
					   {
						   Results.Add(CastElement);
					   }
					   ElementVisited[InElement->GetIndex()] = true;
				   }
				}
			}, true, bIgnoreInactiveRelations);
		}
		else
		{
			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
			{
				if(T* CastElement = Cast<T>(Elements[ElementIndex]))
				{
					Results.Add(CastElement);
				}
			}
		}
		return Results;
	}

	/**
	 * Filtered accessor to retrieve all element keys of a given type
	 * @param bTraverse Returns the element keys in order of a depth first traversal
	 */
	template<typename T>
	TArray<FRigElementKey> GetKeysOfType(bool bTraverse = false) const
	{
		TArray<FRigElementKey> Keys;
		TArray<T*> Results = GetElementsOfType<T>(bTraverse);
		for(T* Element : Results)
		{
			Keys.Add(Element->GetKey());
		}
		return Keys;
	}

	/**
	 * Filtered accessor to retrieve all elements of a given type
	 * @param InKeepElementFunction A function to return true if an element is to be keep
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	template<typename T>
	TArray<T*> GetFilteredElements(TFunction<bool(T*)> InKeepElementFunction, bool bTraverse = false) const
	{
		TArray<T*> Results;

		if(bTraverse)
		{
			TArray<bool> ElementVisited;
			ElementVisited.AddZeroed(Elements.Num());
		
			Traverse([&ElementVisited, &Results, InKeepElementFunction](FRigBaseElement* InElement, bool& bContinue)
			{
				bContinue = !ElementVisited[InElement->GetIndex()];

				if(bContinue)
				{
					if(T* CastElement = Cast<T>(InElement))
					{
						if(InKeepElementFunction(CastElement))
						{
							Results.Add(CastElement);
						}
					}
					ElementVisited[InElement->GetIndex()] = true;
				}
			});
		}
		else
		{
			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
			{
				if(T* CastElement = Cast<T>(Elements[ElementIndex]))
				{
					if(InKeepElementFunction(CastElement))
					{
						Results.Add(CastElement);
					}
				}
			}
		}
		return Results;
	}

	/**
	 * Returns all Bone elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	TArray<FRigBoneElement*> GetBones(bool bTraverse = false) const
	{
		return GetElementsOfType<FRigBoneElement>(bTraverse);
	}
	
	/**
		* Returns all Bone elements without traversing the hierarchy
		*/
	TArray<FRigBaseElement*>& GetBonesFast() const
	{
		return ElementsPerType[RigElementTypeToFlatIndex(ERigElementType::Bone)];
	}

	/**
	 * Returns all Bone elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Bones", ScriptName = "GetBones"))
	TArray<FRigElementKey> GetBoneKeys(bool bTraverse = true) const
	{
		return GetKeysOfType<FRigBoneElement>(bTraverse);
	}

	/**
	 * Returns all Null elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	TArray<FRigNullElement*> GetNulls(bool bTraverse = false) const
	{
		return GetElementsOfType<FRigNullElement>(bTraverse);
	}

	/**
	* Returns all Null elements
	* @param bTraverse Returns the elements in order of a depth first traversal
	*/
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Nulls", ScriptName = "GetNulls"))
	TArray<FRigElementKey> GetNullKeys(bool bTraverse = true) const
	{
		return GetKeysOfType<FRigNullElement>(bTraverse);
	}

	/**
	 * Returns all Control elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	TArray<FRigControlElement*> GetControls(bool bTraverse = false) const
	{
		return GetElementsOfType<FRigControlElement>(bTraverse);
	}

	/**
	 * Returns all Control elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Controls", ScriptName = "GetControls"))
	TArray<FRigElementKey> GetControlKeys(bool bTraverse = true) const
	{
		return GetKeysOfType<FRigControlElement>(bTraverse);
	}

	/**
	 * Returns all transient Control elements
	 */
	CONTROLRIG_API TArray<FRigControlElement*> GetTransientControls() const;

	/**
	 * Returns all Curve elements
	 */
	TArray<FRigCurveElement*> GetCurves() const
	{
		return GetElementsOfType<FRigCurveElement>();
	}

	/**
	 * Returns all Curve elements without traversing the hierarchy
	 */
	TArray<FRigBaseElement*>& GetCurvesFast() const
	{
		return ElementsPerType[RigElementTypeToFlatIndex(ERigElementType::Curve)];
	}

	/**
	 * Returns all Curve elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Curves", ScriptName = "GetCurves"))
	TArray<FRigElementKey> GetCurveKeys() const
	{
		return GetKeysOfType<FRigCurveElement>(false);
	}

	/**
	 * Returns all references
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	TArray<FRigReferenceElement*> GetReferences(bool bTraverse = false) const
	{
		return GetElementsOfType<FRigReferenceElement>(bTraverse);
	}

	/**
	 * Returns all references
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get References", ScriptName = "GetReferences"))
	TArray<FRigElementKey> GetReferenceKeys(bool bTraverse = true) const
	{
		return GetKeysOfType<FRigReferenceElement>(bTraverse);
	}

	/**
	 * Returns all Connector elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	TArray<FRigConnectorElement*> GetConnectors(bool bTraverse = false) const
	{
		return GetElementsOfType<FRigConnectorElement>(bTraverse);
	}

	/**
	 * Returns all Connector elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Connectors", ScriptName = "GetConnectors"))
	TArray<FRigElementKey> GetConnectorKeys(bool bTraverse = true) const
	{
		return GetKeysOfType<FRigConnectorElement>(bTraverse);
	}

	/**
	 * Returns all of the sockets' state
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigSocketState> GetSocketStates() const;

	/**
	 * Try to restore the sockets from the state structs
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigElementKey> RestoreSocketsFromStates(TArray<FRigSocketState> InStates, bool bSetupUndoRedo = false);

	/**
	 * Returns all of the connectors' state
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigConnectorState> GetConnectorStates() const;

	/**
	 * Try to restore the connectors from the state structs
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigElementKey> RestoreConnectorsFromStates(TArray<FRigConnectorState> InStates, bool bSetupUndoRedo = false);

	/**
	 * Returns all Socket elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	TArray<FRigSocketElement*> GetSockets(bool bTraverse = false) const
	{
		return GetElementsOfType<FRigSocketElement>(bTraverse);
	}

	/**
	 * Returns all Socket elements
	 * @param bTraverse Returns the elements in order of a depth first traversal
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Sockets", ScriptName = "GetSockets"))
	TArray<FRigElementKey> GetSocketKeys(bool bTraverse = true) const
	{
		return GetKeysOfType<FRigSocketElement>(bTraverse);
	}

	/**
	 * Returns all root elements
	 */
	TArray<FRigBaseElement*> GetRootElements() const
	{
		return GetFilteredElements<FRigBaseElement>([this](FRigBaseElement* Element) -> bool
		{
			return GetNumberOfParents(Element) == 0;
		});
	}

	/**
	 * Returns all root element keys
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Root Elements", ScriptName = "GetRootElements"))
	TArray<FRigElementKey> GetRootElementKeys() const
	{
		return GetKeysByPredicate([this](const FRigBaseElement& Element) -> bool
		{
			return GetNumberOfParents(Element.Index) == 0;
		}, false);
	}

	/**
	 * Returns the name of metadata for a given element
	 * @param InItem The element key to return the metadata keys for
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API TArray<FName> GetMetadataNames(FRigElementKey InItem) const;

	/**
	 * Returns the type of metadata given its name the item it is stored under
	 * @param InItem The element key to return the metadata type for
	 * @param InMetadataName The name of the metadata to return the type for
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API ERigMetadataType GetMetadataType(FRigElementKey InItem, FName InMetadataName) const;

	/**
	 * Removes the metadata under a given element 
	 * @param InItem The element key to search under
	 * @param InMetadataName The name of the metadata to remove
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API bool RemoveMetadata(FRigElementKey InItem, FName InMetadataName);

	/**
	 * Removes all of the metadata under a given item 
	 * @param InItem The element key to search under
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API bool RemoveAllMetadata(FRigElementKey InItem);

	/**
	 * Queries and returns the value of bool metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	bool GetBoolMetadata(FRigElementKey InItem, FName InMetadataName, bool DefaultValue) const
	{
		return GetMetadata<bool>(InItem, ERigMetadataType::Bool, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of bool array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<bool> GetBoolArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<bool>(InItem, ERigMetadataType::BoolArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a bool value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetBoolMetadata(FRigElementKey InItem, FName InMetadataName, bool InValue)
	{
		return SetMetadata<bool>(InItem, ERigMetadataType::Bool, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a bool array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetBoolArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<bool> InValue)
	{
		return SetArrayMetadata<bool>(InItem, ERigMetadataType::BoolArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of float metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	float GetFloatMetadata(FRigElementKey InItem, FName InMetadataName, float DefaultValue) const
	{
		return GetMetadata<float>(InItem, ERigMetadataType::Float, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of float array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<float> GetFloatArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<float>(InItem, ERigMetadataType::FloatArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a float value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetFloatMetadata(FRigElementKey InItem, FName InMetadataName, float InValue)
	{
		return SetMetadata<float>(InItem, ERigMetadataType::Float, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a float array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetFloatArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<float> InValue)
	{
		return SetArrayMetadata<float>(InItem, ERigMetadataType::FloatArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of int32 metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	int32 GetInt32Metadata(FRigElementKey InItem, FName InMetadataName, int32 DefaultValue) const
	{
		return GetMetadata<int32>(InItem, ERigMetadataType::Int32, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of int32 array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<int32> GetInt32ArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<int32>(InItem, ERigMetadataType::Int32Array, InMetadataName);
	}

	/**
	 * Sets the metadata to a int32 value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetInt32Metadata(FRigElementKey InItem, FName InMetadataName, int32 InValue)
	{
		return SetMetadata<int32>(InItem, ERigMetadataType::Int32, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a int32 array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetInt32ArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<int32> InValue)
	{
		return SetArrayMetadata<int32>(InItem, ERigMetadataType::Int32Array, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of FName metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FName GetNameMetadata(FRigElementKey InItem, FName InMetadataName, FName DefaultValue) const
	{
		return GetMetadata<FName>(InItem, ERigMetadataType::Name, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FName array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FName> GetNameArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FName>(InItem, ERigMetadataType::NameArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FName value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetNameMetadata(FRigElementKey InItem, FName InMetadataName, FName InValue)
	{
		return SetMetadata<FName>(InItem, ERigMetadataType::Name, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FName array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetNameArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FName> InValue)
	{
		return SetArrayMetadata<FName>(InItem, ERigMetadataType::NameArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of FVector metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FVector GetVectorMetadata(FRigElementKey InItem, FName InMetadataName, FVector DefaultValue) const
	{
		return GetMetadata<FVector>(InItem, ERigMetadataType::Vector, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FVector array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FVector> GetVectorArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FVector>(InItem, ERigMetadataType::VectorArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FVector value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetVectorMetadata(FRigElementKey InItem, FName InMetadataName, FVector InValue)
	{
		return SetMetadata<FVector>(InItem, ERigMetadataType::Vector, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FVector array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetVectorArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FVector> InValue)
	{
		return SetArrayMetadata<FVector>(InItem, ERigMetadataType::VectorArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of FRotator metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FRotator GetRotatorMetadata(FRigElementKey InItem, FName InMetadataName, FRotator DefaultValue) const
	{
		return GetMetadata<FRotator>(InItem, ERigMetadataType::Rotator, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FRotator array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FRotator> GetRotatorArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FRotator>(InItem, ERigMetadataType::RotatorArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FRotator value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetRotatorMetadata(FRigElementKey InItem, FName InMetadataName, FRotator InValue)
	{
		return SetMetadata<FRotator>(InItem, ERigMetadataType::Rotator, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FRotator array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetRotatorArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FRotator> InValue)
	{
		return SetArrayMetadata<FRotator>(InItem, ERigMetadataType::RotatorArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of FQuat metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FQuat GetQuatMetadata(FRigElementKey InItem, FName InMetadataName, FQuat DefaultValue) const
	{
		return GetMetadata<FQuat>(InItem, ERigMetadataType::Quat, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FQuat array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FQuat> GetQuatArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FQuat>(InItem, ERigMetadataType::QuatArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FQuat value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetQuatMetadata(FRigElementKey InItem, FName InMetadataName, FQuat InValue)
	{
		return SetMetadata<FQuat>(InItem, ERigMetadataType::Quat, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FQuat array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetQuatArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FQuat> InValue)
	{
		return SetArrayMetadata<FQuat>(InItem, ERigMetadataType::QuatArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of FTransform metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FTransform GetTransformMetadata(FRigElementKey InItem, FName InMetadataName, FTransform DefaultValue) const
	{
		return GetMetadata<FTransform>(InItem, ERigMetadataType::Transform, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FTransform array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FTransform> GetTransformArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FTransform>(InItem, ERigMetadataType::TransformArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FTransform value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetTransformMetadata(FRigElementKey InItem, FName InMetadataName, FTransform InValue)
	{
		return SetMetadata<FTransform>(InItem, ERigMetadataType::Transform, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FTransform array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetTransformArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FTransform> InValue)
	{
		return SetArrayMetadata<FTransform>(InItem, ERigMetadataType::TransformArray, InMetadataName, InValue);
	}

	/**
 * Queries and returns the value of FLinearColor metadata
 * @param InItem The element key to return the metadata for
 * @param InMetadataName The name of the metadata to query
 * @param DefaultValue The default value to fall back on
 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FLinearColor GetLinearColorMetadata(FRigElementKey InItem, FName InMetadataName, FLinearColor DefaultValue) const
	{
		return GetMetadata<FLinearColor>(InItem, ERigMetadataType::LinearColor, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FLinearColor array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FLinearColor> GetLinearColorArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FLinearColor>(InItem, ERigMetadataType::LinearColorArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FLinearColor value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetLinearColorMetadata(FRigElementKey InItem, FName InMetadataName, FLinearColor InValue)
	{
		return SetMetadata<FLinearColor>(InItem, ERigMetadataType::LinearColor, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FLinearColor array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetLinearColorArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FLinearColor> InValue)
	{
		return SetArrayMetadata<FLinearColor>(InItem, ERigMetadataType::LinearColorArray, InMetadataName, InValue);
	}

	/**
	 * Queries and returns the value of FRigElementKey metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 * @param DefaultValue The default value to fall back on
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	FRigElementKey GetRigElementKeyMetadata(FRigElementKey InItem, FName InMetadataName, FRigElementKey DefaultValue) const
	{
		return GetMetadata<FRigElementKey>(InItem, ERigMetadataType::RigElementKey, InMetadataName, DefaultValue);
	}

	/**
	 * Queries and returns the value of FRigElementKey array metadata
	 * @param InItem The element key to return the metadata for
	 * @param InMetadataName The name of the metadata to query
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FRigElementKey> GetRigElementKeyArrayMetadata(FRigElementKey InItem, FName InMetadataName) const
	{
		return GetArrayMetadata<FRigElementKey>(InItem, ERigMetadataType::RigElementKeyArray, InMetadataName);
	}

	/**
	 * Sets the metadata to a FRigElementKey value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetRigElementKeyMetadata(FRigElementKey InItem, FName InMetadataName, FRigElementKey InValue)
	{
		return SetMetadata<FRigElementKey>(InItem, ERigMetadataType::RigElementKey, InMetadataName, InValue);
	}

	/**
	 * Sets the metadata to a FRigElementKey array value
	 * @param InItem The element key to set the metadata for
	 * @param InMetadataName The name of the metadata to set
	 * @param InValue The value to set
	 * @return Returns true if setting the metadata was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetRigElementKeyArrayMetadata(FRigElementKey InItem, FName InMetadataName, TArray<FRigElementKey> InValue)
	{
		return SetArrayMetadata<FRigElementKey>(InItem, ERigMetadataType::RigElementKeyArray, InMetadataName, InValue);
	}

	/**
	 * Returns the path of the module an element belong to (or NAME_None in case the element doesn't belong to a module)
	 * @return The path the element belongs to (or NAME_None)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated. Please use GetModuleFName() instead."))
	CONTROLRIG_API FName GetModulePathFName(FRigElementKey InItem) const;
	
	/**
	 * Returns the path of the module an element belong to (or an empty string in case the element doesn't belong to a module)
	 * @return The path the element belongs to (or empty string)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated. Please use GetModuleName() instead."))
	CONTROLRIG_API FString GetModulePath(FRigElementKey InItem) const;

	/**
	 * Returns the name of the module an element belong to (or NAME_None in case the element doesn't belong to a module)
	 * @return The name the element belongs to (or NAME_None)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API FName GetModuleFName(FRigElementKey InItem) const;
	
	/**
	 * Returns the name of the module an element belong to (or an empty string in case the element doesn't belong to a module)
	 * @return The name the element belongs to (or empty string)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API FString GetModuleName(FRigElementKey InItem) const;

	/**
	 * Returns the prefix of the module an element belong to (or an empty string in case the element doesn't belong to a module)
	 * @return The prefix the element belongs to (or empty string)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API FString GetModulePrefix(FRigElementKey InItem) const;

	/**
	 * Returns the namespace of an element belong to (or NAME_None in case the element doesn't belong to a module / namespace)
	 * @return The namespace the element belongs to (or NAME_None)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated. Please use GetModuleFName() instead."))
	CONTROLRIG_API FName GetNameSpaceFName(FRigElementKey InItem) const;
	
	/**
	 * Returns the namespace of an element belong to (or an empty string in case the element doesn't belong to a module / namespace)
	 * @return The namespace the element belongs to (or empty string)
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated. Please use GetModuleName() instead."))
	CONTROLRIG_API FString GetNameSpace(FRigElementKey InItem) const;

	/*
	 * Returns the tags for a given item
	 * @param InItem The item to return the tags for
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	TArray<FName> GetTags(FRigElementKey InItem) const
	{
		return GetNameArrayMetadata(InItem, TagMetadataName);
	}

	/*
	 * Returns true if a given item has a certain tag
	 * @param InItem The item to return the tags for
	 * @param InTag The tag to check
	 */
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	bool HasTag(FRigElementKey InItem, FName InTag) const
	{
		return GetTags(InItem).Contains(InTag);
	}

	/*
	 * Sets a tag on an element in the hierarchy
	 * @param InItem The item to set the tag for
	 * @param InTag The tag to set
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SetTag(FRigElementKey InItem, FName InTag)
	{
		TArray<FName> Tags = GetTags(InItem);
		Tags.AddUnique(InTag);
		return SetNameArrayMetadata(InItem, TagMetadataName, Tags);
	}

	/**
	 * Returns the selected elements
	 * @InTypeFilter The types to retrieve the selection for
	 * @return An array of the currently selected elements
	 */
	CONTROLRIG_API TArray<const FRigBaseElement*> GetSelectedElements(ERigElementType InTypeFilter = ERigElementType::All) const;

	/**
	 * Returns the selected components
	 * @return An array of the currently selected components
	 */
	CONTROLRIG_API TArray<const FRigBaseComponent*> GetSelectedComponents() const;

	/**
	 * Returns the keys of selected elements
	 * @InTypeFilter The types to retrieve the selection for
	 * @return An array of the currently selected elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigElementKey> GetSelectedKeys(ERigElementType InTypeFilter = ERigElementType::All) const;

	/**
	 * Returns the keys of selected elements and components
	 * @return An array of the currently selected elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get Selected Hierarchy Keys", ScriptName = "GetSelectedHierarchyKeys"))
	TArray<FRigHierarchyKey> GetSelectedHierarchyKeys_ForBlueprint() const
	{
		return GetSelectedHierarchyKeys();
	}

	CONTROLRIG_API const TArray<FRigHierarchyKey>& GetSelectedHierarchyKeys() const;

	/**
	 * Returns true if any element is selected and respects the provided predicate 
	 * @InPredicate A function that should return true when the element is desirable
	 * @return true if any element is selected and respects the provided predicate
	 */
	CONTROLRIG_API bool HasAnythingSelectedByPredicate(TFunctionRef<bool(const FRigElementKey&)> InPredicate) const;

	/**
	 * Returns the keys of selected elements respecting the provided predicate 
	 * @InPredicate A function that should return true when the element is desirable
	 * @return An array of any selected element respecting the provided predicate
	 */
	CONTROLRIG_API TArray<FRigElementKey> GetSelectedKeysByPredicate(TFunctionRef<bool(const FRigElementKey&)> InPredicate) const;

	/**
	 * Returns true if a given element is selected
	 * @param InKey The key to check
	 * @return true if a given element is selected
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsSelected(FRigElementKey InKey) const
	{
		return IsSelected(Find(InKey));
	}

	/**
	 * Returns true if a given component is selected
	 * @param InKey The key to check
	 * @return true if a given component is selected
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsComponentSelected(FRigComponentKey InKey) const
	{
		return IsComponentSelected(FindComponent(InKey));
	}

	bool IsHierarchyKeySelected(FRigHierarchyKey InKey) const
	{
		if(InKey.IsElement())
		{
			return IsSelected(InKey.GetElement());
		}
		if(InKey.IsComponent())
		{
			return IsComponentSelected(InKey.GetComponent());
		}
		return false;
	}

	/**
	 * Returns true if a given element is selected
	 * @param InIndex The index to check
	 * @return true if a given element is selected
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsSelectedByIndex(int32 InIndex) const
	{
		return IsSelected(Get(InIndex));
	}

	bool IsSelected(int32 InIndex) const
	{
		return IsSelectedByIndex(InIndex);
	}

	/**
	 * Sorts the input key list by traversing the hierarchy
	 * @param InKeys The keys to sort
	 * @return The sorted keys
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	TArray<FRigElementKey> SortKeys(const TArray<FRigElementKey>& InKeys) const
	{
		TArray<FRigElementKey> Result;
		Traverse([InKeys, &Result](FRigBaseElement* Element, bool& bContinue)
		{
			const FRigElementKey& Key = Element->GetKey();
			if(InKeys.Contains(Key))
			{
				Result.AddUnique(Key);
			}
		});
		return Result;
	}

	/**
	 * Returns the two name sections with the right namespace separator
	 */
	static CONTROLRIG_API FString JoinNameSpace_Deprecated(const FString& InLeft, const FString& InRight);
	static CONTROLRIG_API FRigName JoinNameSpace_Deprecated(const FRigName& InLeft, const FRigName& InRight);

	/**
	 * Returns the two name sections with the right namespace separator
	 */
	static CONTROLRIG_API TPair<FString, FString> SplitNameSpace_Deprecated(const FString& InNameSpacedPath, bool bFromEnd = true);
	static CONTROLRIG_API TPair<FRigName, FRigName> SplitNameSpace_Deprecated(const FRigName& InNameSpacedPath, bool bFromEnd = true);
	static CONTROLRIG_API bool SplitNameSpace_Deprecated(const FString& InNameSpacedPath, FString* OutNameSpace, FString* OutName, bool bFromEnd = true);
	static CONTROLRIG_API bool SplitNameSpace_Deprecated(const FRigName& InNameSpacedPath, FRigName* OutNameSpace, FRigName* OutName, bool bFromEnd = true);

	/**
	 * Returns the max allowed length for a name within the hierarchy.
	 * @return Returns the max allowed length for a name within the hierarchy.
	 */
	static int32 GetDefaultMaxNameLength() { return UE_RIGHIERARCHY_DEFAULT_MAX_NAME_LENGTH; }

	/**
	 * Returns the max allowed length for a name within the hierarchy.
	 * @return Returns the max allowed length for a name within the hierarchy.
	 */
	int32 GetMaxNameLength() const { return MaxNameLength; }

	/**
	 * Sets the maximum allowed length for a name within the hierarchy.
	 * This value is corrected to the upperbound between desired max and current max.
	 * @return Returns the max allowed length for a name within the hierarchy.
	 */
	CONTROLRIG_API int32 SetMaxNameLength(int32 InMaxNameLength);

	/**
	 * Sanitizes a name by removing invalid characters.
	 * @param InOutName The name to sanitize in place.
	 */
	static CONTROLRIG_API void SanitizeName(FRigName& InOutName, bool bAllowNameSpaces = true, int32 InMaxNameLength = UE_RIGHIERARCHY_DEFAULT_MAX_NAME_LENGTH);

	/**
	 * Sanitizes a name by removing invalid characters.
	 * @param InName The name to sanitize.
	 * @return The sanitized name.
	  */
	static CONTROLRIG_API FRigName GetSanitizedName(const FRigName& InName, bool bAllowNameSpaces = true, int32 InMaxNameLength = UE_RIGHIERARCHY_DEFAULT_MAX_NAME_LENGTH);

	/**
	 * Returns true if a given name is available.
	 * @param InPotentialNewName The name to test for availability
	 * @param InType The type of the to-be-added element
	 * @param OutErrorMessage An optional pointer to return a potential error message 
	 * @return Returns true if the name is available.
	 */
	CONTROLRIG_API bool IsNameAvailable(const FRigName& InPotentialNewName, ERigElementType InType, FString* OutErrorMessage = nullptr) const;

	/**
	 * Returns true if a given display name is available.
	 * @param InParentElement The element to check the display name under
	 * @param InPotentialNewDisplayName The name to test for availability
	 * @param OutErrorMessage An optional pointer to return a potential error message 
	 * @return Returns true if the name is available.
	 */
	CONTROLRIG_API bool IsDisplayNameAvailable(const FRigElementKey& InParentElement, const FRigName& InPotentialNewDisplayName, FString* OutErrorMessage = nullptr) const;

	/**
	 * Returns true if a given component name is available.
	 * @param InElementKey The element to check the component name for
	 * @param InPotentialNewName The name to test for availability
	 * @param OutErrorMessage An optional pointer to return a potential error message 
	 * @return Returns true if the name is available.
	 */
	CONTROLRIG_API bool IsComponentNameAvailable(const FRigElementKey& InElementKey, const FRigName& InPotentialNewName, FString* OutErrorMessage = nullptr) const;

	/**
	 * Returns a valid new name for a to-be-added element.
	 * @param InPotentialNewName The name to be sanitized and adjusted for availability
	 * @param InType The type of the to-be-added element
	 * @param bAllowNameSpace If true the name will be allowed to contain namespaces
	 * @return Returns the name to use for the to-be-added element.
	 */
	CONTROLRIG_API FRigName GetSafeNewName(const FRigName& InPotentialNewName, ERigElementType InType, bool bAllowNameSpace = false) const;

	/**
	 * Returns a valid new display name for a control
	 * @param InParentElement The element to check the display name under
	 * @param InPotentialNewDisplayName The name to be sanitized and adjusted for availability
	 * @return Returns the name to use for the to-be-added element.
	 */
	CONTROLRIG_API FRigName GetSafeNewDisplayName(const FRigElementKey& InParentElement, const FRigName& InPotentialNewDisplayName) const;

	/**
	 * Returns a valid new name for a to-be-added element.
	 * @param InElementKey The element to check the component name for
	 * @param InPotentialNewName The name to be sanitized and adjusted for availability
	 * @return Returns the name to use for the to-be-added element.
	 */
	CONTROLRIG_API FRigName GetSafeNewComponentName(const FRigElementKey& InElementKey, const FRigName& InPotentialNewName) const;

	/**
	 * Returns the display label for an element to be used for the UI
	 */
	CONTROLRIG_API FText GetDisplayNameForUI(const FRigBaseElement* InElement, EElementNameDisplayMode InNameMode = EElementNameDisplayMode::AssetDefault) const;
	CONTROLRIG_API FText GetDisplayNameForUI(const FRigElementKey& InKey, EElementNameDisplayMode InNameMode = EElementNameDisplayMode::AssetDefault) const;

	/**
	 * Returns the modified event, which can be used to 
	 * subscribe to topological changes happening within the hierarchy.
	 * @return The event used for subscription.
	 */
	FRigHierarchyModifiedEvent& OnModified() { return ModifiedEvent; }

	/**
	 * Returns the MetadataChanged event, which can be used to track metadata changes
	 * Note: This notification has a very high volume - so the consequences of subscribing
	 * to it may cause performance slowdowns.
	 */
	FRigHierarchyMetadataChangedDelegate& OnMetadataChanged() { return MetadataChangedDelegate; }

	/**
	 * Returns the MetadataTagChanged event, which can be used to track metadata tag changes
	 * Note: This notification has a very high volume - so the consequences of subscribing
	 * to it may cause performance slowdowns.
	 */
	FRigHierarchyMetadataTagChangedDelegate& OnMetadataTagChanged() { return MetadataTagChangedDelegate; }

	/**
	 * Returns the DependencyDismissed delegate. This is used by editors to dismiss dependencies
	 * which may be blocked certain workflows such as space switching or establishing constraints.
	 */
	FRigHierarchyDismissDependencyDelegate& OnDependencyDismissed() { return DismissDependencyDelegate; }

	/**
	 * Returns the local current or initial value for a given key.
	 * If the key is invalid FTransform::Identity will be returned.
	 * @param InKey The key to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The local current or initial transform's value.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetLocalTransform(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetLocalTransformByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns the local current or initial value for a element index.
	 * If the index is invalid FTransform::Identity will be returned.
	 * @param InElementIndex The index to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The local current or initial transform's value.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FTransform GetLocalTransformByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Elements[InElementIndex]))
			{
				return GetTransform(TransformElement, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal);
			}
		}
		return FTransform::Identity;
	}

	FTransform GetLocalTransform(int32 InElementIndex) const
	{
		return GetLocalTransformByIndex(InElementIndex, false);
	}
	FTransform GetInitialLocalTransform(int32 InElementIndex) const
	{
		return GetLocalTransformByIndex(InElementIndex, true);
	}

	FTransform GetInitialLocalTransform(const FRigElementKey &InKey) const
	{
		return GetLocalTransform(InKey, true);
	}

	/**
	 * Sets the local current or initial transform for a given key.
	 * @param InKey The key to set the transform for
	 * @param InTransform The new transform value to set
	 * @param bInitial If true the initial transform will be used
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetLocalTransform(FRigElementKey InKey, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		SetLocalTransformByIndex(GetIndex(InKey), InTransform, bInitial, bAffectChildren, bSetupUndo, bPrintPythonCommands);
	}

	/**
	 * Sets the local current or initial transform for a given element index.
	 * @param InElementIndex The index of the element to set the transform for
	 * @param InTransform The new transform value to set
	 * @param bInitial If true the initial transform will be used
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetLocalTransformByIndex(int32 InElementIndex, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Elements[InElementIndex]))
			{
				SetTransform(TransformElement, InTransform, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, false, bPrintPythonCommands);
			}
		}
	}

	void SetLocalTransform(int32 InElementIndex, const FTransform& InTransform, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		SetLocalTransformByIndex(InElementIndex, InTransform, false, bAffectChildren, bSetupUndo, bPrintPythonCommands);
	}

	void SetInitialLocalTransform(int32 InElementIndex, const FTransform& InTransform, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		SetLocalTransformByIndex(InElementIndex, InTransform, true, bAffectChildren, bSetupUndo, bPrintPythonCommands);
    }

	void SetInitialLocalTransform(const FRigElementKey& InKey, const FTransform& InTransform, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		SetLocalTransform(InKey, InTransform, true, bAffectChildren, bSetupUndo, bPrintPythonCommands);
	}

	/**
	 * Returns the global current or initial value for a given key.
	 * If the key is invalid FTransform::Identity will be returned.
	 * @param InKey The key to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The global current or initial transform's value.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetGlobalTransform(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetGlobalTransformByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns the global current or initial value for a element index.
	 * If the index is invalid FTransform::Identity will be returned.
	 * @param InElementIndex The index to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The global current or initial transform's value.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetGlobalTransformByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Elements[InElementIndex]))
			{
				return GetTransform(TransformElement, bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal);
			}
		}
		return FTransform::Identity;
	}

	FTransform GetGlobalTransform(int32 InElementIndex) const
	{
		return GetGlobalTransformByIndex(InElementIndex, false);
	}
	FTransform GetInitialGlobalTransform(int32 InElementIndex) const
	{
		return GetGlobalTransformByIndex(InElementIndex, true);
	}

	FTransform GetInitialGlobalTransform(const FRigElementKey &InKey) const
	{
		return GetGlobalTransform(InKey, true);
	}

	/**
	 * Sets the global current or initial transform for a given key.
	 * @param InKey The key to set the transform for
	 * @param InTransform The new transform value to set
	 * @param bInitial If true the initial transform will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetGlobalTransform(FRigElementKey InKey, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommand = false)
	{
		SetGlobalTransformByIndex(GetIndex(InKey), InTransform, bInitial, bAffectChildren, bSetupUndo, bPrintPythonCommand);
	}

	/**
	 * Sets the global current or initial transform for a given element index.
	 * @param InElementIndex The index of the element to set the transform for
	 * @param InTransform The new transform value to set
	 * @param bInitial If true the initial transform will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetGlobalTransformByIndex(int32 InElementIndex, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommand = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Elements[InElementIndex]))
			{
				SetTransform(TransformElement, InTransform, bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal, bAffectChildren, bSetupUndo, false, bPrintPythonCommand);
			}
		}
	}

	void SetGlobalTransform(int32 InElementIndex, const FTransform& InTransform, bool bAffectChildren = true, bool bSetupUndo = false)
	{
		SetGlobalTransformByIndex(InElementIndex, InTransform, false, bAffectChildren, bSetupUndo);
	}

	void SetInitialGlobalTransform(int32 InElementIndex, const FTransform& InTransform, bool bAffectChildren = true, bool bSetupUndo = false)
	{
		SetGlobalTransformByIndex(InElementIndex, InTransform, true, bAffectChildren, bSetupUndo);
	}

	void SetInitialGlobalTransform(const FRigElementKey& InKey, const FTransform& InTransform, bool bAffectChildren = true, bool bSetupUndo = false)
	{
		SetGlobalTransform(InKey, InTransform, true, bAffectChildren, bSetupUndo);
	}

	/**
	 * Returns the version of the transform / pose on the element given its key.
	 * Versions are incremented with every change occured to the transform. You
	 * can use this to compare your previous "knowledge" of the pose - and see
	 * if anybody has changed it during your last access.
	 */
	CONTROLRIG_API int32 GetPoseVersion(const FRigElementKey& InKey) const;

	/**
	 * Returns the version of the transform / pose on the given element.
	 * see int32 GetPoseVersion(const FRigElementKey& InKey) const for more info.
	 */
	CONTROLRIG_API int32 GetPoseVersion(const FRigTransformElement* InTransformElement) const;

	/**
	 * Returns the global offset transform for a given control element.
	 * @param InKey The key of the control to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The global offset transform
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetGlobalControlOffsetTransform(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetGlobalControlOffsetTransformByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns the global offset transform for a given control element.
	 * @param InElementIndex The index of the control to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The global offset transform
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetGlobalControlOffsetTransformByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlOffsetTransform(ControlElement, bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal);
			}
		}
		return FTransform::Identity;
	}

	/**
	 * Returns the local shape transform for a given control element.
	 * @param InKey The key of the control to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The local shape transform
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FTransform GetLocalControlShapeTransform(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetLocalControlShapeTransformByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns the local shape transform for a given control element.
	 * @param InElementIndex The index of the control to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The local shape transform
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FTransform GetLocalControlShapeTransformByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlShapeTransform(ControlElement, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal);
			}
		}
		return FTransform::Identity;
	}

	/**
	 * Returns the global shape transform for a given control element.
	 * @param InKey The key of the control to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The global shape transform
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetGlobalControlShapeTransform(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetGlobalControlShapeTransformByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns the global shape transform for a given control element.
	 * @param InElementIndex The index of the control to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The global shape transform
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetGlobalControlShapeTransformByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlShapeTransform(ControlElement, bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal);
			}
		}
		return FTransform::Identity;
	}

	/**
	 * Returns a control's current value given its key
	 * @param InKey The key of the element to retrieve the current value for
	 * @param InValueType The type of value to return
	 * @return Returns the current value of the control
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FRigControlValue GetControlValue(FRigElementKey InKey, ERigControlValueType InValueType = ERigControlValueType::Current) const
	{
		return GetControlValueByIndex(GetIndex(InKey), InValueType);
	}

	/**
	 * Returns a control's current value given its key
	 * @param InKey The key of the element to retrieve the current value for
	 * @param InValueType The type of value to return
	 * @return Returns the current value of the control
	 */
	template<typename T>
	T GetControlValue(FRigElementKey InKey, ERigControlValueType InValueType = ERigControlValueType::Current) const
	{
		return GetControlValueByIndex(GetIndex(InKey), InValueType).Get<T>();
	}

	/**
	 * Returns a control's current value given its index
	 * @param InElementIndex The index of the element to retrieve the current value for
	 * @param InValueType The type of value to return
	 * @return Returns the current value of the control
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FRigControlValue GetControlValueByIndex(int32 InElementIndex, ERigControlValueType InValueType = ERigControlValueType::Current) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlValue(ControlElement, InValueType, bUsePreferredEulerAngles);
			}
		}
		return FRigControlValue();
	}

	/**
	 * Returns a control's current value given its index
	 * @param InElementIndex The index of the element to retrieve the current value for
	 * @param InValueType The type of value to return
	 * @return Returns the current value of the control
	 */
	FRigControlValue GetControlValue(int32 InElementIndex, ERigControlValueType InValueType = ERigControlValueType::Current) const
	{
		return GetControlValueByIndex(InElementIndex, InValueType);
	}

	/**
	 * Returns a control's current value given its index
	 * @param InElementIndex The index of the element to retrieve the current value for
	 * @param InValueType The type of value to return
	 * @return Returns the current value of the control
	 */
	template<typename T>
	T GetControlValue(int32 InElementIndex, ERigControlValueType InValueType = ERigControlValueType::Current) const
	{
		return GetControlValueByIndex(InElementIndex, InValueType).Get<T>();
	}

	/**
	 * Returns a control's initial value given its index
	 * @param InElementIndex The index of the element to retrieve the initial value for
	 * @return Returns the current value of the control
	 */
	FRigControlValue GetInitialControlValue(int32 InElementIndex) const
	{
		return GetControlValueByIndex(InElementIndex, ERigControlValueType::Initial);
	}

	/**
	 * Returns a control's initial value given its index
	 * @param InElementIndex The index of the element to retrieve the current value for
	 * @return Returns the current value of the control
	 */
	template<typename T>
	T GetInitialControlValue(int32 InElementIndex) const
	{
		return GetInitialControlValue(InElementIndex).Get<T>();
	}

	/**
	 * Returns a control's preferred rotator (local transform rotation)
	 * @param InKey The key of the element to retrieve the current value for
	 * @param bInitial If true we'll return the preferred rotator for the initial - otherwise current transform
	 * @return Returns the current preferred rotator
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FRotator GetControlPreferredRotator(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetControlPreferredRotatorByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns a control's preferred rotator (local transform rotation)
	 * @param InElementIndex The element index to look up
	 * @param bInitial If true we'll return the preferred rotator for the initial - otherwise current transform
	 * @return Returns the current preferred rotator
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FRotator GetControlPreferredRotatorByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlPreferredRotator(ControlElement, bInitial);
			}
		}
		return FRotator::ZeroRotator;
	}

	/**
	 * Returns a control's preferred rotator (local transform rotation)
	 * @param InControlElement The element to look up
	 * @param bInitial If true we'll return the preferred rotator for the initial - otherwise current transform
	 * @return Returns the current preferred rotator
	 */
	FRotator GetControlPreferredRotator(const FRigControlElement* InControlElement, bool bInitial = false) const
	{
		if(InControlElement)
		{
			if (bUsePreferredEulerAngles)
			{
				return InControlElement->PreferredEulerAngles.GetRotator(bInitial);
			}
			const ERigTransformType::Type Type = bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal;
			return GetControlValue(InControlElement->GetKey()).GetAsTransform(InControlElement->Settings.ControlType, InControlElement->Settings.PrimaryAxis).Rotator();
		}
		return FRotator::ZeroRotator;
	}

	/**
	 * Sets a control's preferred rotator (local transform rotation)
	 * @param InKey The key of the element to retrieve the current value for
	 * @param InRotator The new preferred rotator to set
	 * @param bInitial If true we'll return the preferred rotator for the initial - otherwise current transform
	 * @param bFixEulerFlips If true the new rotator value will use the shortest path
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlPreferredRotator(FRigElementKey InKey, const FRotator& InRotator, bool bInitial = false, bool bFixEulerFlips = false)
	{
		SetControlPreferredRotatorByIndex(GetIndex(InKey), InRotator, bInitial, bFixEulerFlips);
	}
	

	/**
	 * Sets a control's preferred rotator (local transform rotation)
	 * @param InElementIndex The element index to look up
	 * @param InRotator The new preferred rotator to set
	 * @param bInitial If true we'll return the preferred rotator for the initial - otherwise current transform
	 * @param bFixEulerFlips If true the new rotator value will use the shortest path
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlPreferredRotatorByIndex(int32 InElementIndex, const FRotator& InRotator, bool bInitial = false, bool bFixEulerFlips = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlPreferredRotator(ControlElement, InRotator, bInitial, bFixEulerFlips);
			}
		}
	}

	/**
	 * Sets a control's preferred rotator (local transform rotation)
	 * @param InControlElement The element to look up
	 * @param InRotator The new preferred rotator to set
	 * @param bInitial If true we'll return the preferred rotator for the initial - otherwise current transform
	 * @param bFixEulerFlips If true the new rotator value will use the shortest path
	 */
	void SetControlPreferredRotator(FRigControlElement* InControlElement, const FRotator& InRotator, bool bInitial = false, bool bFixEulerFlips = false)
	{
		if(InControlElement)
		{
			InControlElement->PreferredEulerAngles.SetRotator(InRotator, bInitial, bFixEulerFlips);
		}
	}

	/**
	 * Returns a control's preferred euler angles (local transform rotation)
	 * @param InKey The key of the element to retrieve the current value for
	 * @param InRotationOrder The rotation order to use when retrieving the euler angles
	 * @param bInitial If true we'll return the preferred euler angles for the initial - otherwise current transform
	 * @return Returns the current preferred euler angles
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FVector GetControlPreferredEulerAngles(FRigElementKey InKey, EEulerRotationOrder InRotationOrder, bool bInitial = false) const
	{
		return GetControlPreferredEulerAnglesByIndex(GetIndex(InKey), InRotationOrder, bInitial);
	}

	/**
	 * Returns a control's preferred euler angles (local transform rotation)
	 * @param InElementIndex The element index to look up
	 * @param InRotationOrder The rotation order to use when retrieving the euler angles
	 * @param bInitial If true we'll return the preferred euler angles for the initial - otherwise current transform
	 * @return Returns the current preferred euler angles
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FVector GetControlPreferredEulerAnglesByIndex(int32 InElementIndex, EEulerRotationOrder InRotationOrder, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlPreferredEulerAngles(ControlElement, InRotationOrder, bInitial);
			}
		}
		return FVector::ZeroVector;
	}

	/**
	 * Returns a control's preferred euler angles (local transform rotation)
	 * @param InControlElement The element to look up
	 * @param InRotationOrder The rotation order to use when retrieving the euler angles
	 * @param bInitial If true we'll return the preferred euler angles for the initial - otherwise current transform
	 * @return Returns the current preferred euler angles
	 */
	FVector GetControlPreferredEulerAngles(const FRigControlElement* InControlElement, EEulerRotationOrder InRotationOrder, bool bInitial = false) const
	{
		if(InControlElement)
		{
			return InControlElement->PreferredEulerAngles.GetAngles(bInitial, InRotationOrder);
		}
		return FVector::ZeroVector;
	}

	/**
	 * Sets a control's preferred euler angles (local transform rotation)
	 * @param InKey The key of the element to retrieve the current value for
	 * @param InEulerAngles The new preferred euler angles to set
	 * @param InRotationOrder The rotation order to use when setting the euler angles
	 * @param bInitial If true we'll return the preferred euler angles for the initial - otherwise current transform
	 * @param bFixEulerFlips If true the new euler angles value will use the shortest path
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlPreferredEulerAngles(FRigElementKey InKey, const FVector& InEulerAngles, EEulerRotationOrder InRotationOrder, bool bInitial = false, bool bFixEulerFlips = false)
	{
		SetControlPreferredEulerAnglesByIndex(GetIndex(InKey), InEulerAngles, InRotationOrder, bInitial, bFixEulerFlips);
	}
	

	/**
	 * Sets a control's preferred euler angles (local transform rotation)
	 * @param InElementIndex The element index to look up
	 * @param InEulerAngles The new preferred euler angles to set
	 * @param InRotationOrder The rotation order to use when setting the euler angles
	 * @param bInitial If true we'll return the preferred euler angles for the initial - otherwise current transform
	 * @param bFixEulerFlips If true the new euler angles value will use the shortest path
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlPreferredEulerAnglesByIndex(int32 InElementIndex, const FVector& InEulerAngles, EEulerRotationOrder InRotationOrder, bool bInitial = false, bool bFixEulerFlips = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlPreferredEulerAngles(ControlElement, InEulerAngles, InRotationOrder, bInitial, bFixEulerFlips);
			}
		}
	}

	/**
	 * Sets a control's preferred euler angles (local transform rotation)
	 * @param InControlElement The element to look up
	 * @param InEulerAngles The new preferred euler angles to set
	 * @param InRotationOrder The rotation order to use when setting the euler angles
	 * @param bInitial If true we'll return the preferred euler angles for the initial - otherwise current transform
	 * @param bFixEulerFlips If true the new euler angles value will use the shortest path
	 */
	void SetControlPreferredEulerAngles(FRigControlElement* InControlElement, const FVector& InEulerAngles, EEulerRotationOrder InRotationOrder, bool bInitial = false, bool bFixEulerFlips = false)
	{
		if(InControlElement)
		{
			InControlElement->PreferredEulerAngles.SetRotationOrder(InRotationOrder);
			InControlElement->PreferredEulerAngles.SetAngles(InEulerAngles, bInitial, InRotationOrder, bFixEulerFlips);
		}
	}

	/**
	 * Returns a control's preferred euler rotation order
	 * @param InKey The key of the element to retrieve the current value for
	 * @param bFromSettings If true the rotation order will be looked up from the control's settings, otherwise from the currently set animation value
	 * @return Returns the current preferred euler rotation order
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	EEulerRotationOrder GetControlPreferredEulerRotationOrder(FRigElementKey InKey, bool bFromSettings = true) const
	{
		return GetControlPreferredEulerRotationOrderByIndex(GetIndex(InKey), bFromSettings);
	}

	/**
	 * Returns a control's preferred euler rotation order
	 * @param InElementIndex The element index to look up
	 * @param bFromSettings If true the rotation order will be looked up from the control's settings, otherwise from the currently set animation value
	 * @return Returns the current preferred euler rotation order
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	EEulerRotationOrder GetControlPreferredEulerRotationOrderByIndex(int32 InElementIndex, bool bFromSettings = true) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				return GetControlPreferredEulerRotationOrder(ControlElement, bFromSettings);
			}
		}
		return FRigPreferredEulerAngles::DefaultRotationOrder;
	}

	/**
	 * Returns a control's preferred euler rotation order
	 * @param InControlElement The element to look up
	 * @param bFromSettings If true the rotation order will be looked up from the control's settings, otherwise from the currently set animation value
	 * @return Returns the current preferred euler rotation order
	 */
	EEulerRotationOrder GetControlPreferredEulerRotationOrder(const FRigControlElement* InControlElement, bool bFromSettings = true) const
	{
		if(InControlElement)
		{
			if(bFromSettings)
			{
				return InControlElement->Settings.PreferredRotationOrder;
			}
			return InControlElement->PreferredEulerAngles.RotationOrder;
		}
		return FRigPreferredEulerAngles::DefaultRotationOrder;
	}
	
	/**
	 * Sets a control's preferred euler rotation order
	 * @param InKey The key of the element to retrieve the current value for
	 * @param InRotationOrder The rotation order to use when setting the euler angles
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlPreferredRotationOrder(FRigElementKey InKey, EEulerRotationOrder InRotationOrder)
	{
		SetControlPreferredRotationOrderByIndex(GetIndex(InKey), InRotationOrder);
	}
	

	/**
	 * Sets a control's preferred euler rotation order
	 * @param InElementIndex The element index to look up
	 * @param InRotationOrder The rotation order to use when setting the euler angles
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlPreferredRotationOrderByIndex(int32 InElementIndex, EEulerRotationOrder InRotationOrder)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlPreferredRotationOrder(ControlElement, InRotationOrder);
			}
		}
	}

	/**
	 * Sets a control's preferred euler rotation order
	 * @param InControlElement The element to look up
	 * @param InRotationOrder The rotation order to use when setting the euler angles
	 */
	void SetControlPreferredRotationOrder(FRigControlElement* InControlElement, EEulerRotationOrder InRotationOrder)
	{
		if(InControlElement)
		{
			InControlElement->PreferredEulerAngles.SetRotationOrder(InRotationOrder);
		}
	}

	bool GetUsePreferredRotationOrder(const FRigControlElement* InControlElement) const
	{
		if (InControlElement && bUsePreferredEulerAngles)
		{
			return InControlElement->Settings.bUsePreferredRotationOrder;
		}
		return false;
	}

	void  SetUsePreferredRotationOrder(FRigControlElement* InControlElement, bool bVal)
	{
		if (InControlElement)
		{
			InControlElement->Settings.bUsePreferredRotationOrder = bVal;
		}
	}

	FVector GetControlSpecifiedEulerAngle(const FRigControlElement* InControlElement, bool bIsInitial = false) const
	{
		FVector EulerAngle = FVector::ZeroVector;
		if (InControlElement)
		{
			if (bIsInitial == false && GetUsePreferredRotationOrder(InControlElement))
			{
				EEulerRotationOrder RotationOrder = GetControlPreferredEulerRotationOrder(InControlElement);
				EulerAngle = InControlElement->PreferredEulerAngles.GetAngles(false, RotationOrder);
			}
			else
			{
				FRotator Rotator = GetControlPreferredRotator(InControlElement, bIsInitial);
				EulerAngle = FVector(Rotator.Roll, Rotator.Pitch, Rotator.Yaw);
			}
		}
		return EulerAngle;
	}
	
	void SetControlSpecifiedEulerAngle(FRigControlElement* InControlElement, const FVector& InEulerAngle, bool bIsInitial = false)
	{
		if (InControlElement)
		{
			if (GetUsePreferredRotationOrder(InControlElement))
			{
				EEulerRotationOrder RotationOrder = GetControlPreferredEulerRotationOrder(InControlElement);
				SetControlPreferredEulerAngles(InControlElement, InEulerAngle, RotationOrder, bIsInitial);
			}
			else
			{
				FRotator Rotator(InEulerAngle[1], InEulerAngle[2], InEulerAngle[0]);
				SetControlPreferredRotator(InControlElement, Rotator, bIsInitial, false /* fix euler flips*/); //test fix todo mikez
			}
		}
	}

	CONTROLRIG_API void SetControlPreferredEulerAngles(FRigControlElement* InControlElement, const FTransform& InTransform, bool bIsInitial = false);

	FQuat GetControlQuaternion(const FRigControlElement* InControlElement, const FVector& InEulerAngle) const
	{
		if (InControlElement)
		{
			FRotator Rotator(InEulerAngle[1], InEulerAngle[2], InEulerAngle[0]);

			if (GetUsePreferredRotationOrder(InControlElement))
			{
				return InControlElement->PreferredEulerAngles.GetQuatFromRotator(Rotator);
			}
			else
			{
				return Rotator.Quaternion();
			}
		}
		return FQuat();
	}

	FVector GetControlAnglesFromQuat(const FRigControlElement* InControlElement, const FQuat& InQuat, bool bUseRotationOrder) const
	{
		FVector Angle(0, 0, 0);
		if (InControlElement)
		{

			if (bUseRotationOrder && InControlElement->Settings.bUsePreferredRotationOrder)
			{
				FRotator Rotator = InControlElement->PreferredEulerAngles.GetRotatorFromQuat(InQuat);
				Angle = Rotator.Euler();
			}
			else
			{
				FRotator Rotator(InQuat);
				Angle = Rotator.Euler();
			}
		}
		return Angle;
	}
	/**
	 * Returns the pin type to use for a control
	 * @param InControlElement The control to return the pin type for
	 * @return The pin type
	 */
	CONTROLRIG_API FEdGraphPinType GetControlPinType(FRigControlElement* InControlElement) const;

	/**
	 * Returns the pin type to use for a control type
	 * @param InControlElement The control to return the pin type for
	 * @return The pin type
	 */
	CONTROLRIG_API static FEdGraphPinType GetControlPinType(ERigControlType ControlType);

	/**
	 * Returns the default value to use for a pin for a control
	 * @param InControlElement The control to return the pin default value for
	 * @param bForEdGraph If this is true to the math types' ::ToString will be used rather than text export
	 * @param InValueType The type of value to return
	 * @return The pin default value
	 */
	CONTROLRIG_API FString GetControlPinDefaultValue(FRigControlElement* InControlElement, bool bForEdGraph, ERigControlValueType InValueType = ERigControlValueType::Initial) const;

	/**
	 * Sets a control's current value given its key
	 * @param InKey The key of the element to set the current value for
	 * @param InValue The value to set on the control
	 * @param InValueType The type of value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlValue(FRigElementKey InKey, FRigControlValue InValue, ERigControlValueType InValueType = ERigControlValueType::Current, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		SetControlValueByIndex(GetIndex(InKey), InValue, InValueType, bSetupUndo, bPrintPythonCommands);
	}

	/**
	 * Sets a control's current value given its key
	 * @param InKey The key of the element to set the current value for
	 * @param InValue The value to set on the control
	 * @param InValueType The type of value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	template<typename T>
	void SetControlValue(FRigElementKey InKey, const T& InValue, ERigControlValueType InValueType = ERigControlValueType::Current, bool bSetupUndo = false, bool bPrintPythonCommands = false) const
	{
		return SetControlValue(InKey, FRigControlValue::Make<T>(InValue), InValueType, bSetupUndo, bPrintPythonCommands);
	}

	/**
	 * Sets a control's current value given its index
	 * @param InElementIndex The index of the element to set the current value for
	 * @param InValue The value to set on the control
	 * @param InValueType The type of value to set
  	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlValueByIndex(int32 InElementIndex, FRigControlValue InValue, ERigControlValueType InValueType = ERigControlValueType::Current, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlValue(ControlElement, InValue, InValueType, bSetupUndo, false, bPrintPythonCommands);
			}
		}
	}

	/**
	 * Sets a control's current value given its index
	 * @param InElementIndex The index of the element to set the current value for
	 * @param InValue The value to set on the control
	 * @param InValueType The type of value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	void SetControlValue(int32 InElementIndex, const FRigControlValue& InValue, ERigControlValueType InValueType = ERigControlValueType::Current, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		SetControlValueByIndex(InElementIndex, InValue, InValueType, bSetupUndo, bPrintPythonCommands);
	}

	/**
	 * Sets a control's current value given its index
	 * @param InElementIndex The index of the element to set the current value for
	 * @param InValue The value to set on the control
	 * @param InValueType The type of value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	template<typename T>
	void SetControlValue(int32 InElementIndex, const T& InValue, ERigControlValueType InValueType = ERigControlValueType::Current, bool bSetupUndo = false)
	{
		return SetControlValueByIndex(InElementIndex, FRigControlValue::Make<T>(InValue), InValueType, bSetupUndo, false);
	}

	/**
	 * Sets a control's initial value given its index
	 * @param InElementIndex The index of the element to set the initial value for
	 * @param InValue The value to set on the control
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	void SetInitialControlValue(int32 InElementIndex, const FRigControlValue& InValue, bool bSetupUndo = false)
	{
		SetControlValueByIndex(InElementIndex, InValue, ERigControlValueType::Initial, bSetupUndo);
	}

	/**
	 * Sets a control's initial value given its index
	 * @param InElementIndex The index of the element to set the initial value for
	 * @param InValue The value to set on the control
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	template<typename T>
	void SetInitialControlValue(int32 InElementIndex, const T& InValue, bool bSetupUndo = false) const
	{
		return SetInitialControlValue(InElementIndex, FRigControlValue::Make<T>(InValue), bSetupUndo);
	}

	/**
	 * Sets a control's current visibility based on a key
	 * @param InKey The key of the element to set the visibility for
	 * @param bVisibility The visibility to set on the control
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlVisibility(FRigElementKey InKey, bool bVisibility)
	{
		SetControlVisibilityByIndex(GetIndex(InKey), bVisibility);
	}

	/**
	 * Sets a control's current visibility based on a key
	 * @param InElementIndex The index of the element to set the visibility for
	 * @param bVisibility The visibility to set on the control
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlVisibilityByIndex(int32 InElementIndex, bool bVisibility)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlVisibility(ControlElement, bVisibility);
			}
		}
	}

	void SetControlVisibility(int32 InElementIndex, bool bVisibility)
	{
		SetControlVisibilityByIndex(InElementIndex, bVisibility);
	}

	/**
	 * Returns a curve's value given its key
	 * @param InKey The key of the element to retrieve the value for
	 * @return Returns the value of the curve
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    float GetCurveValue(FRigElementKey InKey) const
	{
		return GetCurveValueByIndex(GetIndex(InKey));
	}

	/**
	 * Returns a curve's value given its index
	 * @param InElementIndex The index of the element to retrieve the value for
	 * @return Returns the value of the curve
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    float GetCurveValueByIndex(int32 InElementIndex) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Elements[InElementIndex]))
			{
				return GetCurveValue(CurveElement);
			}
		}
		return 0.f;
	}

	// TODO: Deprecate?
	float GetCurveValue(int32 InElementIndex) const
	{
		return GetCurveValueByIndex(InElementIndex);
	}

	/**
	 * Returns whether a curve's value is set, given its key
	 * @param InKey The key of the element to retrieve the value for
	 * @return Returns true if the value is set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsCurveValueSet(FRigElementKey InKey) const
	{
		return IsCurveValueSetByIndex(GetIndex(InKey));
	}

	/**
	 * Returns a curve's value given its index
	 * @param InElementIndex The index of the element to retrieve the value for
	 * @return Returns true if the value is set, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsCurveValueSetByIndex(int32 InElementIndex) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Elements[InElementIndex]))
			{
				return IsCurveValueSet(CurveElement);
			}
		}
		return false;
	}
	
	/**
	 * Sets a curve's value given its key
	 * @param InKey The key of the element to set the value for
	 * @param InValue The value to set on the curve
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetCurveValue(FRigElementKey InKey, float InValue, bool bSetupUndo = false)
	{
		SetCurveValueByIndex(GetIndex(InKey), InValue, bSetupUndo);
	}

	/**
	 * Sets a curve's value given its index
	 * @param InElementIndex The index of the element to set the value for
	 * @param InValue The value to set on the curve
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetCurveValueByIndex(int32 InElementIndex, float InValue, bool bSetupUndo = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Elements[InElementIndex]))
			{
				SetCurveValue(CurveElement, InValue, bSetupUndo);
			}
		}
	}

	// TODO: Deprecate?
	void SetCurveValue(int32 InElementIndex, float InValue, bool bSetupUndo = false)
	{
		SetCurveValueByIndex(InElementIndex, InValue, bSetupUndo);
	}

	/**
	 * Sets a curve's value given its key
	 * @param InKey The key of the element to set the value for
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void UnsetCurveValue(FRigElementKey InKey, bool bSetupUndo = false)
	{
		UnsetCurveValueByIndex(GetIndex(InKey), bSetupUndo);
	}

	/**
	 * Sets a curve's value given its index
	 * @param InElementIndex The index of the element to set the value for
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void UnsetCurveValueByIndex(int32 InElementIndex, bool bSetupUndo = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigCurveElement* CurveElement = Cast<FRigCurveElement>(Elements[InElementIndex]))
			{
				UnsetCurveValue(CurveElement, bSetupUndo);
			}
		}
	}

	/**
	 * Sets the offset transform for a given control element by key
	 * @param InKey The key of the control element to set the offset transform for
	 * @param InTransform The new offset transform value to set
	 * @param bInitial If true the initial value will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlOffsetTransform(FRigElementKey InKey, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		return SetControlOffsetTransformByIndex(GetIndex(InKey), InTransform, bInitial, bAffectChildren, bSetupUndo, bPrintPythonCommands);
	}

	/**
	 * Sets the local offset transform for a given control element by index
	 * @param InElementIndex The index of the control element to set the offset transform for
 	 * @param InTransform The new local offset transform value to set
	 * @param bInitial If true the initial value will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlOffsetTransformByIndex(int32 InElementIndex, FTransform InTransform, bool bInitial = false, bool bAffectChildren = true, bool bSetupUndo = false, bool bPrintPythonCommands = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlOffsetTransform(ControlElement, InTransform, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, bPrintPythonCommands);
			}
		}
	}

	/**
	 * Sets the shape transform for a given control element by key
	 * @param InKey The key of the control element to set the shape transform for
	 * @param InTransform The new shape transform value to set
	 * @param bInitial If true the initial value will be used
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlShapeTransform(FRigElementKey InKey, FTransform InTransform, bool bInitial = false, bool bSetupUndo = false)
	{
		return SetControlShapeTransformByIndex(GetIndex(InKey), InTransform, bInitial, bSetupUndo);
	}

	/**
	 * Sets the local shape transform for a given control element by index
	 * @param InElementIndex The index of the control element to set the shape transform for
	 * @param InTransform The new local shape transform value to set
	 * @param bInitial If true the initial value will be used
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    void SetControlShapeTransformByIndex(int32 InElementIndex, FTransform InTransform, bool bInitial = false, bool bSetupUndo = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlShapeTransform(ControlElement, InTransform, bInitial ? ERigTransformType::InitialLocal : ERigTransformType::CurrentLocal, bSetupUndo);
			}
		}
	}

	/**
	 * Sets the control settings for a given control element by key
	 * @param InKey The key of the control element to set the settings for
	 * @param InSettings The new control settings value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlSettings(FRigElementKey InKey, FRigControlSettings InSettings, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false)
	{
		return SetControlSettingsByIndex(GetIndex(InKey), InSettings, bSetupUndo, bForce, bPrintPythonCommands);
	}

	/**
	 * Sets the control settings for a given control element by index
	 * @param InElementIndex The index of the control element to set the settings for
	 * @param InSettings The new control settings value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetControlSettingsByIndex(int32 InElementIndex, FRigControlSettings InSettings, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Elements[InElementIndex]))
			{
				SetControlSettings(ControlElement, InSettings, bSetupUndo, bForce, bPrintPythonCommands);
			}
		}
	}

	/**
	 * Sets the connector settings for a given connector element by key
	 * @param InKey The key of the connector element to set the settings for
	 * @param InSettings The new connector settings value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetConnectorSettings(FRigElementKey InKey, FRigConnectorSettings InSettings, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false)
	{
		return SetConnectorSettingsByIndex(GetIndex(InKey), InSettings, bSetupUndo, bForce, bPrintPythonCommands);
	}

	/**
	 * Sets the connector settings for a given connector element by index
	 * @param InElementIndex The index of the connector element to set the settings for
	 * @param InSettings The new connector settings value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	void SetConnectorSettingsByIndex(int32 InElementIndex, FRigConnectorSettings InSettings, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false)
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			if(FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Elements[InElementIndex]))
			{
				SetConnectorSettings(ConnectorElement, InSettings, bSetupUndo, bForce, bPrintPythonCommands);
			}
		}
	}

	/**
	 * Returns the global current or initial value for a given key.
	 * If the element does not have a parent FTransform::Identity will be returned.
	 * @param InKey The key of the element to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The element's parent's global current or initial transform's value.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetParentTransform(FRigElementKey InKey, bool bInitial = false) const
	{
		return GetParentTransformByIndex(GetIndex(InKey), bInitial);
	}

	/**
	 * Returns the global current or initial value for a given element index.
	 * If the element does not have a parent FTransform::Identity will be returned.
	 * @param InElementIndex The index of the element to retrieve the transform for
	 * @param bInitial If true the initial transform will be used
	 * @return The element's parent's global current or initial transform's value.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    FTransform GetParentTransformByIndex(int32 InElementIndex, bool bInitial = false) const
	{
		if(Elements.IsValidIndex(InElementIndex))
		{
			return GetParentTransform(Elements[InElementIndex], bInitial ? ERigTransformType::InitialGlobal : ERigTransformType::CurrentGlobal);
		}
		return FTransform::Identity;
	}

	/**
	 * Returns the child elements of a given element key
	 * @param InKey The key of the element to retrieve the children for
	 * @param bRecursive If set to true grand-children will also be returned etc
	 * @return Returns the child elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigElementKey> GetChildren(FRigElementKey InKey, bool bRecursive = false) const;

	/**
	 * Returns the child elements of a given element key
	 * @param InElement The element to retrieve the children for
	 * @param bRecursive If set to true grand-children will also be returned etc
	 * @return Returns the child elements
	 */
	CONTROLRIG_API FRigBaseElementChildrenArray GetActiveChildren(const FRigBaseElement* InElement, bool bRecursive = false) const;

	/**
	 * Returns the child elements of a given element index
	 * @param InIndex The index of the element to retrieve the children for
	 * @param bRecursive If set to true grand-children will also be returned etc
	 * @return Returns the child elements' indices
	 */
    CONTROLRIG_API TArray<int32> GetChildren(int32 InIndex, bool bRecursive = false) const;

	/**
	 * Returns the child elements of a given element
	 * @param InElement The element to retrieve the children for
	 * @return Returns the child elements
	 */
	CONTROLRIG_API TConstArrayView<FRigBaseElement*> GetChildren(const FRigBaseElement* InElement) const;
	CONTROLRIG_API TArrayView<FRigBaseElement*> GetChildren(const FRigBaseElement* InElement);

	/**
	 * Returns the child elements of a given element
	 * @param InElement The element to retrieve the children for
	 * @param bRecursive If set to true grand-children will also be returned etc
	 * @return Returns the child elements
	 */
	CONTROLRIG_API FRigBaseElementChildrenArray GetChildren(const FRigBaseElement* InElement, bool bRecursive) const;

	/**
	 * Returns the parent elements of a given element key
	 * @param InKey The key of the element to retrieve the parents for
	 * @param bRecursive If set to true parents of parents will also be returned
	 * @return Returns the parent elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    CONTROLRIG_API TArray<FRigElementKey> GetParents(FRigElementKey InKey, bool bRecursive = false) const;

	/**
	 * Returns the parent elements of a given element index
	 * @param InIndex The index of the element to retrieve the parents for
	 * @param bRecursive If set to true parents of parents will also be returned
	 * @return Returns the parent elements' indices
	 */
    CONTROLRIG_API TArray<int32> GetParents(int32 InIndex, bool bRecursive = false) const;

	/**
	 * Returns the parent elements of a given element
	 * @param InElement The element to retrieve the parents for
	 * @param bRecursive If set to true parents of parents will also be returned
	 * @return Returns the parent elements
	 */
	CONTROLRIG_API FRigBaseElementParentArray GetParents(const FRigBaseElement* InElement, bool bRecursive = false) const;

	/**
	 * Returns the default parent element's key of a given child key
	 * @param InKey The key of the element to retrieve the parent for
	 * @return Returns the default parent element key
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API FRigElementKey GetDefaultParent(FRigElementKey InKey) const;

	/**
	 * Returns the first parent element of a given element key
	 * @param InKey The key of the element to retrieve the parents for
	 * @return Returns the first parent element
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    CONTROLRIG_API FRigElementKey GetFirstParent(FRigElementKey InKey) const;

	/**
	 * Returns the first parent element of a given element index
	 * @param InIndex The index of the element to retrieve the parent for
	 * @return Returns the first parent index (or INDEX_NONE)
	 */
    CONTROLRIG_API int32 GetFirstParent(int32 InIndex) const;

	/**
	 * Returns the first parent element of a given element
	 * @param InElement The element to retrieve the parents for
	 * @return Returns the first parent element
	 */
	CONTROLRIG_API FRigBaseElement* GetFirstParent(const FRigBaseElement* InElement) const;

	/**
	 * Returns the number of parents of an element
	 * @param InKey The key of the element to retrieve the number of parents for
	 * @return Returns the number of parents of an element
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    CONTROLRIG_API int32 GetNumberOfParents(FRigElementKey InKey) const;

	/**
	 * Returns the number of parents of an element
	 * @param InIndex The index of the element to retrieve the number of parents for
	 * @return Returns the number of parents of an element
	 */
    CONTROLRIG_API int32 GetNumberOfParents(int32 InIndex) const;

	/**
	 * Returns the number of parents of an element
	 * @param InElement The element to retrieve the number of parents for
	 * @return Returns the number of parents of an element
	 */
	CONTROLRIG_API int32 GetNumberOfParents(const FRigBaseElement* InElement) const;

	/**
	 * Returns the weight of a parent below a multi parent element
	 * @param InChild The key of the multi parented element
	 * @param InParent The key of the parent to look up the weight for
	 * @param bInitial If true the initial weights will be used
	 * @return Returns the weight of a parent below a multi parent element, or FLT_MAX if the parent is invalid
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    CONTROLRIG_API FRigElementWeight GetParentWeight(FRigElementKey InChild, FRigElementKey InParent, bool bInitial = false) const;

	/**
	 * Returns the weight of a parent below a multi parent element
	 * @param InChild The multi parented element
	 * @param InParent The parent to look up the weight for
	 * @param bInitial If true the initial weights will be used
	 * @return Returns the weight of a parent below a multi parent element, or FLT_MAX if the parent is invalid
	 */
    CONTROLRIG_API FRigElementWeight GetParentWeight(const FRigBaseElement* InChild, const FRigBaseElement* InParent, bool bInitial = false) const;

	/**
	 * Returns the weight of a parent below a multi parent element
	 * @param InChild The multi parented element
	 * @param InParentIndex The index of the parent inside of the multi parent element
	 * @param bInitial If true the initial weights will be used
	 * @return Returns the weight of a parent below a multi parent element, or FLT_MAX if the parent is invalid
	 */
	CONTROLRIG_API FRigElementWeight GetParentWeight(const FRigBaseElement* InChild, int32 InParentIndex, bool bInitial = false) const;

	/**
	 * Returns the weights of all parents below a multi parent element
	 * @param InChild The key of the multi parented element
	 * @param bInitial If true the initial weights will be used
	 * @return Returns the weight of a parent below a multi parent element, or FLT_MAX if the parent is invalid
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API TArray<FRigElementWeight> GetParentWeightArray(FRigElementKey InChild, bool bInitial = false) const;

	/**
	 * Returns the weights of all parents below a multi parent element
	 * @param InChild The multi parented element
	 * @param bInitial If true the initial weights will be used
	 * @return Returns the weight of a parent below a multi parent element, or FLT_MAX if the parent is invalid
	 */
	CONTROLRIG_API TArray<FRigElementWeight> GetParentWeightArray(const FRigBaseElement* InChild, bool bInitial = false) const;

	/**
	 * Get the current active parent for the passed in key. This is only valid when only one parent has a weight value and the other parents have zero weights
	 * @param InKey The multi parented element
	 * @param bReferenceKey Whether or not to return a reference key
	 * @param bInitial Whether or not to rely on initial weights
	 * @return Returns the first parent with a non-zero weight
	 */
	CONTROLRIG_API FRigElementKey GetActiveParent(const FRigElementKey& InKey, bool bReferenceKey = true, bool bInitial = false) const;

	/**
	 * Get the current active parent for a given element index. This is only valid when only one parent has a weight value and the other parents have zero weights
	 * @param InIndex The index of the element to retrieve the parent for
	 * @param bInitial Whether or not to rely on initial weights
	 * @return Returns the first parent index (or INDEX_NONE) with a non-zero weight
	 */
	CONTROLRIG_API int32 GetActiveParent(int32 InIndex, bool bInitial = false) const;

	/**
	 * Get the current active parent for the passed in key. This is only valid when only one parent has a weight value and the other parents have zero weights
	 * @param InElement The element to retrieve the parents for
	 * @param bInitial Whether or not to rely on initial weights
	 * @return Returns the first parent with a non-zero weight
	 */
	CONTROLRIG_API FRigBaseElement* GetActiveParent(const FRigBaseElement* InElement, bool bInitial = false) const;
	
	/**
	 * Get the current active parents for the passed in key. 
	 * @param InElement The element to retrieve the parents for
	 * @param bInitial Whether or not to rely on initial weights
	 * @return Returns all parents with a non-zero weight
	 */
	CONTROLRIG_API FRigBaseElementParentArray GetActiveParents(const FRigBaseElement* InElement, bool bInitial = false) const;

	/**
	 * Returns the display label to use for the space given a parent key
	 * @param InChildKey The key of the child element to look up the parent for
	 * @param InParentKey The key of the parent to look up the display label for
	 * @return The display label to use for the user interface for a given parent space
	 */
	CONTROLRIG_API FName GetDisplayLabelForParent(const FRigElementKey& InChildKey, const FRigElementKey& InParentKey) const;

	/**
	 * Sets the weight of a parent below a multi parent element
	 * @param InChild The key of the multi parented element
	 * @param InParent The key of the parent to look up the weight for
	 * @param InWeight The new weight to set for the parent
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    CONTROLRIG_API bool SetParentWeight(FRigElementKey InChild, FRigElementKey InParent, FRigElementWeight InWeight, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Sets the weight of a parent below a multi parent element
	 * @param InChild The multi parented element
	 * @param InParent The parent to look up the weight for
	 * @param InWeight The new weight to set for the parent
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	CONTROLRIG_API bool SetParentWeight(FRigBaseElement* InChild, const FRigBaseElement* InParent, FRigElementWeight InWeight, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Sets the weight of a parent below a multi parent element
	 * @param InChild The multi parented element
	 * @param InParentIndex The index of the parent inside of the multi parent element
	 * @param InWeight The new weight to set for the parent
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	CONTROLRIG_API bool SetParentWeight(FRigBaseElement* InChild, int32 InParentIndex, FRigElementWeight InWeight, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Sets the all of the weights of the parents of a multi parent element
	 * @param InChild The key of the multi parented element
	 * @param InWeights The new weights to set for the parents
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API bool SetParentWeightArray(FRigElementKey InChild, TArray<FRigElementWeight> InWeights, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Sets the all of the weights of the parents of a multi parent element
	 * @param InChild The multi parented element
	 * @param InWeights The new weights to set for the parents
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	CONTROLRIG_API bool SetParentWeightArray(FRigBaseElement* InChild,  const TArray<FRigElementWeight>& InWeights, bool bInitial = false, bool bAffectChildren = true);

	/**
	* Sets the all of the weights of the parents of a multi parent element
	* @param InChild The multi parented element
	* @param InWeights The new weights to set for the parents
	* @param bInitial If true the initial weights will be used
	* @param bAffectChildren If set to false children will not move (maintain global).
	* @return Returns true if changing the weight was successful
	*/
	CONTROLRIG_API bool SetParentWeightArray(FRigBaseElement* InChild,  const TArrayView<const FRigElementWeight>& InWeights, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Determines if relating the child element to the given parent would cause a cycle 
	 * @param InChild The key of the child for the new potential relationship
	 * @param InParent The key of the parent for the new potential relationship
	 * @param InDependencyProvider An additional object providing dependencies to respect
	 * @param OutFailureReason An optional pointer to retrieve the reason for failure
	 * @return Returns true if adding a new relationship from child to parent would cause a cycle
	 */
	CONTROLRIG_API bool CanCauseCycle(
		FRigElementKey InChild,
		FRigElementKey InParent,
		const IRigDependenciesProvider& InDependencyProvider = FEmptyRigDependenciesProvider(),
		FString* OutFailureReason = nullptr) const;

	/**
	 * Determines if relating the child element to the given parent would cause a cycle 
	 * @param InChild The key of the child for the new potential relationship
	 * @param InParent The key of the parent for the new potential relationship
	 * @param InDependencyProvider An additional object providing dependencies to respect
	 * @param OutFailureReason An optional pointer to retrieve the reason for failure
	 * @return Returns true if adding a new relationship from child to parent would cause a cycle
	 */
	CONTROLRIG_API bool CanCauseCycle(
		const FRigTransformElement* InChild,
		const FRigTransformElement* InParent,
		const IRigDependenciesProvider& InDependencyProvider = FEmptyRigDependenciesProvider(),
		FString* OutFailureReason = nullptr) const;
	
	/**
	* Determines if the element can be switched to a provided parent
	* @param InChild The key of the multi parented element
	* @param InParent The key of the parent to look up the weight for
    * @param InDependencyProvider An additional object providing dependencies to respect
	* @param OutFailureReason An optional pointer to retrieve the reason for failure
	* @return Returns true if the child can be switched to the desired parent
	*/
	CONTROLRIG_API bool CanSwitchToParent(
		FRigElementKey InChild,
		FRigElementKey InParent,
		const IRigDependenciesProvider& InDependencyProvider = FEmptyRigDependenciesProvider(),
		FString* OutFailureReason = nullptr) const;

	/**
	 * Switches a multi parent element to a single parent.
	 * This sets the new parent's weight to 1.0 and disables
	 * weights for all other potential parents.
	 * @param InChild The key of the multi parented element
	 * @param InParent The key of the parent to look up the weight for
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool SwitchToParent(FRigElementKey InChild, FRigElementKey InParent, bool bInitial = false, bool bAffectChildren = true)
	{
		return SwitchToParent(InChild, InParent, bInitial, bAffectChildren, FEmptyRigDependenciesProvider(), nullptr);
	}

	CONTROLRIG_API bool SwitchToParent(FRigElementKey InChild,
		FRigElementKey InParent,
		bool bInitial,
		bool bAffectChildren,
		const IRigDependenciesProvider& InDependencyProvider,
		FString* OutFailureReason);

	/**
	 * Switches a multi parent element to a single parent.
	 * This sets the new parent's weight to 1.0 and disables
	 * weights for all other potential parents.
	 * @param InChild The multi parented element
	 * @param InParent The parent to look up the weight for
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
     * @param InDependencyProvider An additional object providing dependencies to respect
	 * @param OutFailureReason An optional pointer to retrieve the reason for failure
	 * @return Returns true if changing the weight was successful
	 */
	CONTROLRIG_API bool SwitchToParent(FRigBaseElement* InChild,
		FRigBaseElement* InParent,
		bool bInitial = false,
		bool bAffectChildren = true,
		const IRigDependenciesProvider& InDependencyProvider = FEmptyRigDependenciesProvider(),
		FString* OutFailureReason = nullptr);

	/**
	 * Switches a multi parent element to a single parent.
	 * This sets the new parent's weight to 1.0 and disables
	 * weights for all other potential parents.
	 * @param InChild The multi parented element
	 * @param InParentIndex The index of the parent inside of the multi parent element
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	CONTROLRIG_API bool SwitchToParent(FRigBaseElement* InChild, int32 InParentIndex, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Switches a multi parent element to its first parent
	 * @param InChild The key of the multi parented element
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API bool SwitchToDefaultParent(FRigElementKey InChild, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Switches a multi parent element to its first parent
	 * This sets the new parent's weight to 1.0 and disables
	 * weights for all other potential parents.
	 * @param InChild The multi parented element
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	CONTROLRIG_API bool SwitchToDefaultParent(FRigBaseElement* InChild, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Switches a multi parent element to world space.
	 * This injects a world space reference.
	 * @param InChild The key of the multi parented element
	 * @param bInitial If true the initial weights will be used
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @return Returns true if changing the weight was successful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API bool SwitchToWorldSpace(FRigElementKey InChild, bool bInitial = false, bool bAffectChildren = true);

	/**
	* Switches a multi parent element to world space.
	* This injects a world space reference.
	* @param InChild The multi parented element
	* @param bInitial If true the initial weights will be used
	* @param bAffectChildren If set to false children will not move (maintain global).
	* @return Returns true if changing the weight was successful
	*/
	CONTROLRIG_API bool SwitchToWorldSpace(FRigBaseElement* InChild, bool bInitial = false, bool bAffectChildren = true);

	/**
	 * Adds the world space reference or returns it
	 */
	CONTROLRIG_API FRigElementKey GetOrAddWorldSpaceReference();
	
	static CONTROLRIG_API FRigElementKey GetDefaultParentKey();
	static CONTROLRIG_API FRigElementKey GetWorldSpaceReferenceKey();
	static CONTROLRIG_API const FLazyName DefaultParentKeyLabel;
	static CONTROLRIG_API const FLazyName WorldSpaceKeyLabel;

	/**
	 * Returns true if an element is parented to another element
	 * @param InChild The key of the child element to check for a parent
	 * @param InParent The key of the parent element to check for
 	 * @param bConsiderInactiveParents (Optional) when set to true, considers inactive parents, defaults to false.
	 * @return True if the given parent and given child have a parent-child relationship
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	bool IsParentedTo(FRigElementKey InChild, FRigElementKey InParent, bool bConsiderInactiveParents = false) const
	{
		constexpr FRigHierarchyDependencyChain* NoDependencyChain = nullptr;
		return IsParentedTo(GetIndex(InChild), GetIndex(InParent), FEmptyRigDependenciesProvider(), NoDependencyChain, bConsiderInactiveParents);
	}

	/**
	 * Returns true if an element is parented to another element
	 * @param InChildIndex The index of the child element to check for a parent
	 * @param InParentIndex The index of the parent element to check for
	 * @param InDependencyProvider An additional object providing dependencies to respect
	 * @param OutDependencyChain An optional output error message indicating the parent cycle
	 * @param bConsiderInactiveParents (Optional) when set to true, considers inactive parents, defaults to false.
	 * @return True if the given parent and given child have a parent-child relationship
	 */
    bool IsParentedTo(int32 InChildIndex, int32 InParentIndex, const IRigDependenciesProvider& InDependencyProvider, FRigHierarchyDependencyChain* OutDependencyChain = nullptr, bool bConsiderInactiveParents = false) const
	{
		if(Elements.IsValidIndex(InChildIndex) && Elements.IsValidIndex(InParentIndex))
		{
			return IsParentedTo(Elements[InChildIndex], Elements[InParentIndex], InDependencyProvider, OutDependencyChain, bConsiderInactiveParents);
		}
		return false;
	}

	/**
	 * Returns the animation channels of a given element key
	 * @param InKey The key of the element to retrieve the animation channels for
	 * @param bOnlyDirectChildren If set to false also animation channels with secondary parenting relationships will be retrieved
	 * @return Returns the animation channels' indices
	 */
	CONTROLRIG_API TArray<FRigElementKey> GetAnimationChannels(FRigElementKey InKey, bool bOnlyDirectChildren = true) const;

	/**
	 * Returns the animation channels of a given element index
	 * @param InIndex The index of the element to retrieve the animation channels for
	 * @param bOnlyDirectChildren If set to false also animation channels with secondary parenting relationships will be retrieved
	 * @return Returns the animation channels' indices
	 */
	CONTROLRIG_API TArray<int32> GetAnimationChannels(int32 InIndex, bool bOnlyDirectChildren = true) const;

	/**
	 * Returns the animation channels of a given element
	 * @param InElement The element to retrieve the animation channels for
	 * @param bOnlyDirectChildren If set to false also animation channels with secondary parenting relationships will be retrieved
	 * @return Returns the animation channels
	 */
	CONTROLRIG_API TArray<FRigControlElement*> GetAnimationChannels(const FRigControlElement* InElement, bool bOnlyDirectChildren = true) const;

	/**
	 * Returns all element keys of this hierarchy
	 * @param bTraverse If set to true the keys will be returned by depth first traversal
	 * @param InElementType The type filter to apply
	 * @return The keys of all elements
	 */
	CONTROLRIG_API TArray<FRigElementKey> GetAllKeys(bool bTraverse = false, ERigElementType InElementType = ERigElementType::All) const;

	/**
	 * Returns element keys of this hierarchy, filtered by a predicate.
	 * @param InPredicateFunc The predicate function to apply. Should return \c true if the element
	 *    should be added to the result array.
	 * @param bInTraverse If set to true the keys will be returned by depth first traversal
	 * @return The keys of all elements
	 */
	CONTROLRIG_API TArray<FRigElementKey> GetKeysByPredicate(TFunctionRef<bool(const FRigBaseElement&)> InPredicateFunc, bool bInTraverse = false) const;

	/**
	 * Returns all element keys of this hierarchy
	 * @param bTraverse If set to true the keys will be returned by depth first traversal
	 * @return The keys of all elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Get All Keys", ScriptName = "GetAllKeys"))
	TArray<FRigElementKey> GetAllKeys_ForBlueprint(bool bTraverse = true) const
	{
		return GetAllKeys(bTraverse, ERigElementType::All);
	}

	/**
	 * Helper function to traverse the hierarchy
	 * @param InElement The element to start the traversal at
	 * @param bTowardsChildren If set to true the traverser walks downwards (towards the children), otherwise upwards (towards the parents)
	 * @param PerElementFunction The function to call for each visited element
	 * @param bIgnoreInactiveRelations If set to true the traverser ignores zero-weight parent-child relationships 
	 */
	CONTROLRIG_API void Traverse(FRigBaseElement* InElement, bool bTowardsChildren, TFunction<void(FRigBaseElement*, bool& /* continue */)> PerElementFunction, bool bIgnoreInactiveRelations = false) const;

	/**
	 * Helper function to traverse the hierarchy from the root
	 * @param PerElementFunction The function to call for each visited element
	 * @param bTowardsChildren If set to true the traverser walks downwards (towards the children), otherwise upwards (towards the parents)
	 * @param bIgnoreInactiveRelations If set to true the traverser ignores zero-weight parent-child relationships 
	 */
	CONTROLRIG_API void Traverse(TFunction<void(FRigBaseElement*, bool& /* continue */)> PerElementFunction, bool bTowardsChildren = true, bool bIgnoreInactiveRelations = false) const;

	/**
	 * Returns the first currently resolved target for given connector key
	 */
	CONTROLRIG_API const FRigElementKey& GetResolvedTarget(const FRigElementKey& InConnectorKey) const;

	/**
	 * Returns all currently resolved targets for given connector key
	 */
	CONTROLRIG_API TArray<FRigElementKey> GetResolvedTargets(const FRigElementKey& InConnectorKey) const;

	/**
	 * Performs undo for one transform change
	 */
	CONTROLRIG_API bool Undo();

	/**
	 * Performs redo for one transform change
	 */
	CONTROLRIG_API bool Redo();

	/**
	 * Returns the event fired during undo / redo
	 */
	FRigHierarchyUndoRedoTransformEvent& OnUndoRedo() { return UndoRedoEvent; }

	/**
	 * Starts an interaction on the rig.
	 * This will cause all transform actions happening to be merged
	 */
	void StartInteraction() { bIsInteracting = true; }

	/**
	 * Starts an interaction on the rig.
	 * This will cause all transform actions happening to be merged
	 */
	void EndInteraction()
	{
		bIsInteracting = false;
		LastInteractedKey.Reset();
	}

	/**
	 * Returns the transform stack index
	 */
	int32 GetTransformStackIndex() const { return TransformStackIndex; }

	/**
	 * Sends an event from the hierarchy to the world
	 * @param InEvent The event to send
	 * @param bAsynchronous If set to true the event will go on a thread safe queue
	 */
	CONTROLRIG_API void SendEvent(const FRigEventContext& InEvent, bool bAsynchronous = true);

	/**
	* Sends an autokey event from the hierarchy to the world
	* @param InElement The element to send the autokey for
	* @param InOffsetInSeconds The time offset in seconds
	* @param bAsynchronous If set to true the event will go on a thread safe queue
	*/
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API void SendAutoKeyEvent(FRigElementKey InElement, float InOffsetInSeconds = 0.f, bool bAsynchronous = true);

	/**
	 * Returns the delegate to listen to for events coming from this hierarchy
	 * @return The delegate to listen to for events coming from this hierarchy
	 */
	FRigEventDelegate& OnEventReceived() { return EventDelegate; }
	
	/**
	 * Returns true if the hierarchy controller is currently available
	 * The controller may not be available during certain events.
	 * If the controller is not available then GetController() will return nullptr.
	 */ 
	UFUNCTION(BlueprintPure, Category = URigHierarchy)
	CONTROLRIG_API bool IsControllerAvailable() const;

	/**
	 * Returns a controller for this hierarchy.
	 * Note: If the controller is not available this will return nullptr 
	 * even if the bCreateIfNeeded flag is set to true. You can check the 
	 * controller's availability with IsControllerAvailable().
	 * @param bCreateIfNeeded Creates a controller if needed
	 * @return The Controller for this hierarchy
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API URigHierarchyController* GetController(bool bCreateIfNeeded = true);

	/**
	 * Returns a rule manager for this hierarchy
	 * Note: If the manager is not available this will return nullptr 
	 * even if the bCreateIfNeeded flag is set to true.
	 * @param bCreateIfNeeded Creates a controller if needed
	 * @return The Controller for this hierarchy
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API UModularRigRuleManager* GetRuleManager(bool bCreateIfNeeded = true);

	/**
	 * Returns the topology version of this hierarchy
	 */
	uint32 GetTopologyVersion() const { return TopologyVersion; }

	/**
	 * Returns the hash of this hierarchy used for cached element keys
	 */
	uint32 GetTopologyVersionHash() const
	{
		const uint32 Hash = HashCombine(
			(uint32)reinterpret_cast<long long>(this),
			GetTypeHash(TopologyVersion));
		if(ElementKeyRedirector)
		{
			return HashCombine(Hash, ElementKeyRedirector->GetHash());
		}
		return Hash;
	}

	/**
	 * Increments the topology version
	 */
	CONTROLRIG_API void IncrementTopologyVersion();

	/**
	 * Returns the parent weight version of this hierarchy
	 */
	uint32 GetParentWeightVersion() const { return ParentWeightVersion; }

	/**
	 * Increments the parent weight version of this hierarchy
	 */
	CONTROLRIG_API void IncrementParentWeightVersion();

	/**
	 * Returns the metadata version of this hierarchy
	 */
	uint32 GetMetadataVersion() const { return MetadataVersion; }

	/**
     * Returns the metadata tag version of this hierarchy
	 */
	uint32 GetMetadataTagVersion() const { return MetadataTagVersion; }

	/**
	 * Returns the current / initial pose of the hierarchy
	 * @param bInitial If set to true the initial pose will be returned
	 * @return The pose of the hierarchy
	 * @param bIncludeTransientControls If true the transient controls will be included in the pose
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	FRigPose GetPose(
		bool bInitial = false,
		bool bIncludeTransientControls = true
	) const
	{
		return GetPose(bInitial, ERigElementType::All, FRigElementKeyCollection(), bIncludeTransientControls);
	}

	/**
	 * Returns the current / initial pose of the hierarchy
	 * @param bInitial If set to true the initial pose will be returned
	 * @param InElementType The types of elements to get
	 * @param InItems An optional list of items to get
	 * @param bIncludeTransientControls If true the transient controls will be included in the pose
	 * @return The pose of the hierarchy
	 */
	CONTROLRIG_API FRigPose GetPose(
		bool bInitial,
		ERigElementType InElementType,
		const FRigElementKeyCollection& InItems ,
		bool bIncludeTransientControls = true
	) const;

	/**
	 * Returns the current / initial pose of the hierarchy
	 * @param bInitial If set to true the initial pose will be returned
	 * @param InElementType The types of elements to get
	 * @param InItems An optional list of items to get
	 * @param bIncludeTransientControls If true the transient controls will be included in the pose
	 * @return The pose of the hierarchy
	 */
	CONTROLRIG_API FRigPose GetPose(
		bool bInitial,
		ERigElementType InElementType,
		const TArrayView<const FRigElementKey>& InItems,
		bool bIncludeTransientControls = true
	) const;

	/**
	 * Sets the current / initial pose of the hierarchy
	 * @param InPose The pose to set on the hierarchy
	 * @param InTransformType The transform type to set
	 */
	void SetPose(
		const FRigPose& InPose,
		ERigTransformType::Type InTransformType = ERigTransformType::CurrentLocal
	)
	{
		SetPose(InPose, InTransformType, ERigElementType::All, FRigElementKeyCollection(), 1.f);
	}

	/**
	 * Sets the current / initial pose of the hierarchy
	 * @param InPose The pose to set on the hierarchy
	 * @param InTransformType The transform type to set
	 * @param InElementType The types of elements to set
	 * @param InItems An optional list of items to set
	 * @param InWeight A weight to define how much the pose needs to be mixed in
	 */
	CONTROLRIG_API void SetPose(
		const FRigPose& InPose,
		ERigTransformType::Type InTransformType,
		ERigElementType InElementType,
		const FRigElementKeyCollection& InItems,
		float InWeight
	);

	/**
	 * Sets the current / initial pose of the hierarchy
	 * @param InPose The pose to set on the hierarchy
	 * @param InTransformType The transform type to set
	 * @param InElementType The types of elements to set
	 * @param InItems An optional list of items to set
	 * @param InWeight A weight to define how much the pose needs to be mixed in
	 */
	CONTROLRIG_API void SetPose(
		const FRigPose& InPose,
		ERigTransformType::Type InTransformType,
		ERigElementType InElementType,
		const TArrayView<const FRigElementKey>& InItems, 
		float InWeight
	);

	/**
	 * Sets the current / initial pose of the hierarchy
	 * @param InPose The pose to set on the hierarchy
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy, meta = (DisplayName = "Set Pose", ScriptName = "SetPose"))
	void SetPose_ForBlueprint(FRigPose InPose)
	{
		return SetPose(InPose);
	}

	/**
	 * Sets the pose adapter used for storage of pose data
	 * @param InPoseAdapter The pose adapter to set on the hierarchy
	 */
	CONTROLRIG_API void LinkPoseAdapter(TSharedPtr<FRigHierarchyPoseAdapter> InPoseAdapter);

	/**
	 * Clears the pose adapter used for storage of pose data
	 */
	void UnlinkPoseAdapter()
	{
		return LinkPoseAdapter(nullptr);
	}

	/**
	 * Creates a rig control value from a bool value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FRigControlValue MakeControlValueFromBool(bool InValue)
	{
		return FRigControlValue::Make<bool>(InValue);
	}

	/**
	 * Creates a rig control value from a float value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromFloat(float InValue)
	{
		return FRigControlValue::Make<float>(InValue);
	}

	/**
	 * Returns the contained float value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted float value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static float GetFloatFromControlValue(FRigControlValue InValue)
	{
		return InValue.Get<float>();
	}

	/**
	 * Creates a rig control value from a int32 value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromInt(int32 InValue)
	{
		return FRigControlValue::Make<int32>(InValue);
	}

	/**
	 * Returns the contained int32 value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted int32 value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static int32 GetIntFromControlValue(FRigControlValue InValue)
	{
		return InValue.Get<int32>();
	}

	/**
	 * Creates a rig control value from a FVector2D value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromVector2D(FVector2D InValue)
	{
		return FRigControlValue::Make<FVector3f>(FVector3f(InValue.X, InValue.Y, 0.f));
	}

	/**
	 * Returns the contained FVector2D value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted FVector2D value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FVector2D GetVector2DFromControlValue(FRigControlValue InValue)
	{
		const FVector3f Vector = InValue.Get<FVector3f>();
		return FVector2D(Vector.X, Vector.Y);
	}

	/**
	 * Creates a rig control value from a FVector value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromVector(FVector InValue)
	{
		return FRigControlValue::Make<FVector>(InValue);
	}

	/**
	 * Returns the contained FVector value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted FVector value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FVector GetVectorFromControlValue(FRigControlValue InValue)
	{
		return (FVector)InValue.Get<FVector3f>();
	}

	/**
	 * Creates a rig control value from a FRotator value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromRotator(FRotator InValue)
	{
		return FRigControlValue::Make<FVector>(InValue.Euler());
	}

	/**
	 * Returns the contained FRotator value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted FRotator value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FRotator GetRotatorFromControlValue(FRigControlValue InValue)
	{
		return FRotator::MakeFromEuler((FVector)InValue.Get<FVector3f>());
	}

	/**
	 * Creates a rig control value from a FTransform value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromTransform(FTransform InValue)
	{
		return FRigControlValue::Make<FRigControlValue::FTransform_Float>(InValue);
	}

	/**
	 * Returns the contained FTransform value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted FTransform value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FTransform GetTransformFromControlValue(FRigControlValue InValue)
	{
		return InValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
	}

	/**
	 * Creates a rig control value from a FEulerTransform value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromEulerTransform(FEulerTransform InValue)
	{
		return FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(InValue);
	}

	/**
	 * Returns the contained FEulerTransform value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted FEulerTransform value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FEulerTransform GetEulerTransformFromControlValue(FRigControlValue InValue)
	{
		return InValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
	}

	/**
	 * Creates a rig control value from a FTransformNoScale value
	 * @param InValue The value to create the rig control value from
	 * @return The converted control rig val ue
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
    static FRigControlValue MakeControlValueFromTransformNoScale(FTransformNoScale InValue)
	{
		return FRigControlValue::Make<FRigControlValue::FTransformNoScale_Float>(InValue);
	}

	/**
	 * Returns the contained FTransformNoScale value from a a Rig Control Value
	 * @param InValue The Rig Control value to convert from
	 * @return The converted FTransformNoScale value
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	static FTransformNoScale GetTransformNoScaleFromControlValue(FRigControlValue InValue)
	{
		return InValue.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
	}

private:

	FRigHierarchyModifiedEvent ModifiedEvent;

	UPROPERTY(BlueprintReadOnly, Category = RigHierarchy, meta = (ScriptName = "ModifiedEvent", AllowPrivateAccess = "true"))
	FRigHierarchyModifiedDynamicEvent ModifiedEventDynamic;
	
	FRigHierarchyMetadataChangedDelegate MetadataChangedDelegate;
	FRigHierarchyMetadataTagChangedDelegate MetadataTagChangedDelegate;
	FRigHierarchyDismissDependencyDelegate DismissDependencyDelegate;
	FRigEventDelegate EventDelegate;

	TSharedPtr<FRigHierarchyPoseAdapter> PoseAdapter;

public:

	CONTROLRIG_API void Notify(ERigHierarchyNotification InNotifType, const FRigNotificationSubject& InSubject);

	/**
	 * Returns a transform based on a given transform type
	 * @param InTransformElement The element to retrieve the transform for
	 * @param InTransformType The type of transform to retrieve
	 * @return The local current or initial transform's value.
	 */
	CONTROLRIG_API FTransform GetTransform(FRigTransformElement* InTransformElement, const ERigTransformType::Type InTransformType) const;

	/**
	 * Returns a transform for a given element's parent based on the transform type
	 * If the element does not have a parent FTransform::Identity will be returned.
	 * @param InElement The element to retrieve the transform for
	 * @param InTransformType The type of transform to retrieve
	 * @return The element's parent's transform
	 */
	CONTROLRIG_API FTransform GetParentTransform(FRigBaseElement* InElement, const ERigTransformType::Type InTransformType) const;

	/**
	 * Sets a transform for a given element based on the transform type
	 * @param InTransformElement The element to set the transform for
	 * @param InTransform The type of transform to set
	 * @param InTransformType The type of transform to set
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bForce Set the transform even if it is the same as the previously set one
	 */
	CONTROLRIG_API void SetTransform(FRigTransformElement* InTransformElement, const FTransform& InTransform, const ERigTransformType::Type InTransformType, bool bAffectChildren, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false);

	/**
	 * Returns the global offset transform for a given control element.
	 * @param InControlElement The control element to retrieve the offset transform for
	 * @param InTransformType The type of transform to set
	 * @return The global offset transform
	 */
	CONTROLRIG_API FTransform GetControlOffsetTransform(FRigControlElement* InControlElement, const ERigTransformType::Type InTransformType) const;

	/**
	 * Sets the offset transform for a given control element
	 * @param InControlElement The element to set the transform for
	 * @param InTransform The offset transform to set
	 * @param InTransformType The type of transform to set. Note: for offset transform, setting the initial value also updates the current value
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bForce Set the transform even if it is the same as the previously set one
	 */
	CONTROLRIG_API void SetControlOffsetTransform(FRigControlElement* InControlElement, const FTransform& InTransform, const ERigTransformType::Type InTransformType, bool bAffectChildren, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false);

	/**
	 * Returns the global shape transform for a given control element.
	 * @param InControlElement The control element to retrieve the shape transform for
	 * @param InTransformType The type of transform to set
	 * @return The global shape transform
	 */
	CONTROLRIG_API FTransform GetControlShapeTransform(FRigControlElement* InControlElement, const ERigTransformType::Type InTransformType) const;

	/**
	 * Sets the shape transform for a given control element
	 * @param InControlElement The element to set the transform for
	 * @param InTransform The shape transform to set
	 * @param InTransformType The type of transform to set. Note: for shape transform, setting the initial value also updates the current value
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bForce Set the transform even if it is the same as the previously set one
	 */
	CONTROLRIG_API void SetControlShapeTransform(FRigControlElement* InControlElement, const FTransform& InTransform, const ERigTransformType::Type InTransformType, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false);

	/**
	 * Sets the control settings for a given control element
	 * @param InControlElement The element to set the settings for
	 * @param InSettings The new control settings value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	CONTROLRIG_API void SetControlSettings(FRigControlElement* InControlElement, FRigControlSettings InSettings, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false);

	/**
	 * Returns a control's current value
	 * @param InControlElement The element to retrieve the current value for
	 * @param InValueType The type of value to return
	 * @param bUsePreferredAngles When true will use euler preferred angles to compute the value
	 * @return Returns the current value of the control
	 */
	CONTROLRIG_API FRigControlValue GetControlValue(FRigControlElement* InControlElement, ERigControlValueType InValueType, bool bUsePreferredAngles = true) const;

	/**
	 * Sets a control's preferred euler angle.
	 * @param InControlElement The element to set the current value for
	 * @param InValue The value to ge the angle from to set on the control
	 * @param InValueType The type of value to set
	 * @param bFixEulerFlips If true the angle will use the shortest path
	 */
	CONTROLRIG_API void SetPreferredEulerAnglesFromValue(FRigControlElement* InControlElement,
		const FRigControlValue& InValue,
		const ERigControlValueType& InValueType,
		bool bFixEulerFlips) const;

	template<typename T>
	T GetControlValue(FRigControlElement* InControlElement, ERigControlValueType InValueType) const
	{
		return GetControlValue(InControlElement, InValueType).Get<T>();
	}

	/**
	 * Sets a control's current value
	 * @param InControlElement The element to set the current value for
	 * @param InValueType The type of value to set
	 * @param InValue The value to set on the control
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bForce Set the transform even if it is the same as the previously set one
	 */
	CONTROLRIG_API void SetControlValue(FRigControlElement* InControlElement, const FRigControlValue& InValue, ERigControlValueType InValueType, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false, bool bFixEulerFlips = false);

	template<typename T>
	void SetControlValue(FRigControlElement* InControlElement, const T& InValue, ERigControlValueType InValueType, bool bSetupUndo = false, bool bForce = false) const
	{
		SetControlValue(InControlElement, FRigControlValue::Make<T>(InValue), InValueType, bSetupUndo, bForce);
	}

	/**
	 * Sets a control's current visibility
	 * @param InControlElement The element to set the visibility for
	 * @param bVisibility The new visibility for the control
	 */
	CONTROLRIG_API void SetControlVisibility(FRigControlElement* InControlElement, bool bVisibility);

	/**
	 * Sets the connector settings for a given connector element
	 * @param InConnectorElement The element to set the settings for
	 * @param InSettings The new connector settings value to set
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 */
	CONTROLRIG_API void SetConnectorSettings(FRigConnectorElement* InConnectorElement, FRigConnectorSettings InSettings, bool bSetupUndo = false, bool bForce = false, bool bPrintPythonCommands = false);

	/**
	 * Returns a curve's value. If the curve value is not set, returns 
	 * @param InCurveElement The element to retrieve the value for
	 * @return Returns the value of the curve
	 */
	CONTROLRIG_API float GetCurveValue(FRigCurveElement* InCurveElement) const;

	/**
	 * Returns whether a curve's value is set. If the curve value is not set, returns false. 
	 * @param InCurveElement The element to retrieve the value for
	 * @return Returns true if the value is set, false otherwise.
	 */
	CONTROLRIG_API bool IsCurveValueSet(FRigCurveElement* InCurveElement) const;

	/**
	 * Sets a curve's value
	 * @param InCurveElement The element to set the value for
	 * @param InValue The value to set on the control
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bForce Set the transform even if it is the same as the previously set one
	 */
	CONTROLRIG_API void SetCurveValue(FRigCurveElement* InCurveElement, float InValue, bool bSetupUndo = false, bool bForce = false);

	/**
	 * Unsets a curve's value. Basically the curve's value becomes meaningless.
	 * @param InCurveElement The element to set the value for
	 * @param bSetupUndo If true the transform stack will be setup for undo / redo
	 * @param bForce Unset the curve even if it was already unset.
	 */
	CONTROLRIG_API void UnsetCurveValue(FRigCurveElement* InCurveElement, bool bSetupUndo = false, bool bForce = false);

	/**
	 * Returns the previous name of an element prior to a rename operation
	 * @param InKey The key of the element to request the old name for
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API FName GetPreviousName(const FRigElementKey& InKey) const;

	/**
	 * Returns the previous name of an element or a component prior to a rename operation
	 * @param InKey The key to request the old name for
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API FName GetPreviousHierarchyName(const FRigHierarchyKey& InKey) const;

	/**
	 * Returns the previous name map used by this hierarchy
	 * @return The name map from old to new key
	 */
	const TMap<FRigHierarchyKey, FRigHierarchyKey>& GetPreviousNameMap() const { return PreviousHierarchyNameMap; }

	/**
	 * Returns the previous parent of an element prior to a reparent operation
	 * @param InKey The key of the element to request the old parent  for
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API FRigElementKey GetPreviousParent(const FRigElementKey& InKey) const;

	/**
	 * Returns the previous parent of an element or a component prior to a reparent operation
	 * @param InKey The key to request the old parent  for
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchy)
	CONTROLRIG_API FRigHierarchyKey GetPreviousHierarchyParent(const FRigHierarchyKey& InKey) const;

	/**
	 * Returns true if an element is parented to another element
	 * @param InChild The child element to check for a parent
	 * @param InParent The parent element to check for
	 * @param InDependencyProvider An additional object providing dependencies to respect
	 * @param OutDependencyChain An optional output structural chain providing insight into the chain causing the dependency cycle
	 * @param bConsiderInactiveParents (Optional) when set to true, considers inactive parents, defaults to false.
	 * @return True if the given parent and given child have a parent-child relationship
	 */
	CONTROLRIG_API bool IsParentedTo(FRigBaseElement* InChild,
		FRigBaseElement* InParent,
		const IRigDependenciesProvider& InDependencyProvider = FEmptyRigDependenciesProvider(),
		FRigHierarchyDependencyChain* OutDependencyChain = nullptr,
		bool bConsiderInactiveParents = false) const;

private:
	/**
	 * Returns true if an element is affected to another element
	 * @param InDependent The dependent element to check for a dependency
	 * @param InDependency The dependency element to check for
	 * @param InDependencyProvider An additional object providing dependencies to respect
	 * @param OutDependencyChain An optional output structural chain providing insight into the chain causing the dependency cycle
	 * @param bConsiderInactiveParents (Optional) when set to true, considers inactive parents, defaults to false.
	 * @return True if the given dependent is affected by the given dependency
	 */
	CONTROLRIG_API bool IsDependentOn(FRigBaseElement* InDependent,
		FRigBaseElement* InDependency,
		const IRigDependenciesProvider& InDependencyProvider,
		THierarchyCache<TMap<TTuple<int32, int32>, bool>>* InElementDependencyCache,
		FRigHierarchyDependencyChain& OutDependencyChain,
		bool bConsiderInactiveParents = false) const;

public:
	/**
	 * Returns the index of an element given its key within its default parent (or root)
	 * @param InElement The element to retrieve the index for
	 * @return The index of the element or INDEX_NONE
	 */
	CONTROLRIG_API int32 GetLocalIndex(const FRigBaseElement* InElement) const;

	/**
	 * Returns a reference to the suspend notifications flag
	 */
	bool& GetSuspendNotificationsFlag() { return bSuspendNotifications; }

	/*
	 * Returns true if a hierarchy will record any change.
	 * This is used for debugging purposes.
	 */
	CONTROLRIG_API bool IsTracingChanges() const;

	/**
	 * Returns true if the control is animatable
	 */
	CONTROLRIG_API bool IsAnimatable(const FRigElementKey& InKey) const;

	/**
	 * Returns true if the control is animatable
	 */
	CONTROLRIG_API bool IsAnimatable(const FRigControlElement* InControlElement) const;

	/**
	 * Returns true if the control should be grouped in editor
	 */
	CONTROLRIG_API bool ShouldBeGrouped(const FRigElementKey& InKey) const;

	/**
	 * Returns true if the control should be grouped in editor
	 */
	CONTROLRIG_API bool ShouldBeGrouped(const FRigControlElement* InControlElement) const;

#if WITH_EDITOR

	/**
	 * Clears the undo / redo stack of this hierarchy
	 */
	CONTROLRIG_API void ResetTransformStack();

	/**
	 * Stores the current pose for tracing
	 */
	CONTROLRIG_API void StorePoseForTrace(const FString& InPrefix);

	/**
	 * Updates the format for trace floating point numbers
	 */
	static CONTROLRIG_API void CheckTraceFormatIfRequired();
	
	/**
	 * Dumps the content of the transform stack to a string
	 */
	CONTROLRIG_API void DumpTransformStackToFile(FString* OutFilePath = nullptr);

	/**
	 * Tells this hierarchy to trace a series of frames
	 */
	CONTROLRIG_API void TraceFrames(int32 InNumFramesToTrace);
	
#endif

private:

	/**
	 * Returns true if a given element is selected
	 * @param InElement The element to check
	 * @return true if a given element is selected
	 */
    CONTROLRIG_API bool IsSelected(const FRigBaseElement* InElement) const;

	/**
	 * Returns true if a given component is selected
	 * @param InComponent The component to check
	 * @return true if a given component is selected
	 */
	CONTROLRIG_API bool IsComponentSelected(const FRigBaseComponent* InComponent) const;

	/**
	 * Updates the transient cached children table if the topology version is out of date with the one
	 * stored with the cached table.
	 * @return Returns true if a change was performed
	 */
	CONTROLRIG_API void EnsureCachedChildrenAreCurrent() const;

	CONTROLRIG_API void UpdateCachedChildren();
	
	/**
	 * Corrects a parent element key for space switching
	 */
	CONTROLRIG_API FRigElementKey PreprocessParentElementKeyForSpaceSwitching(const FRigElementKey& InChildKey, const FRigElementKey& InParentKey) const;

	/*
	 * Helper function to create an element for a given type
	 */
	CONTROLRIG_API FRigBaseElement* MakeElement(ERigElementType InElementType, int32 InCount = 1, int32* OutStructureSize = nullptr); 

	/*
	* Helper function to destroy an element
	*/
	CONTROLRIG_API void DestroyElement(FRigBaseElement*& InElement, bool bDestroyComponents = true, bool bDestroyElementStorage = true, bool bDestroyMetadata = true);

	/*
	 * Templated helper function to create an element
	 */
	template<typename ElementType = FRigBaseElement>
	ElementType* NewElement(int32 Num = 1, bool bAllocateStorage = false, bool bUpdateAllElementsStorage = false)
	{
		ElementType* NewElements = nullptr;
		if (Num > 0)
		{
			NewElements = static_cast<ElementType*>(FMemory::Malloc(sizeof(ElementType) * Num));
			for (int32 Index = 0; Index < Num; Index++)
			{
				new(&NewElements[Index]) ElementType(this);
			}
			NewElements[0].OwnedInstances = Num;
			if (bAllocateStorage)
			{
				for (int32 Index = 0; Index < Num; Index++)
				{
					AllocateDefaultElementStorage(&NewElements[Index], bUpdateAllElementsStorage);
				}
			}
		}
		return NewElements;
	}

	CONTROLRIG_API FRigBaseComponent* MakeComponent(const UScriptStruct* InComponentStruct, const FName& InName, FRigBaseElement* InElement); 

	/*
	 * Helper function to destroy a component
	 */
	CONTROLRIG_API void DestroyComponent(FRigBaseComponent*& InComponent);

	/*
	 * Helper function to destroy all components of an element
	 */
	CONTROLRIG_API void DestroyComponents(FRigBaseElement* InElement);

	/*
	 * Helper function to shrink component memory
	 */
	CONTROLRIG_API void ShrinkComponentStorage();

	/*
	 * Templated helper function to create an element
	 */
	template<typename ComponentType = FRigBaseComponent>
	ComponentType* NewComponent(const FName& InName, FRigBaseElement* InElement)
	{
		return MakeComponent(ComponentType::StaticStruct(), InName, InElement);
	}

	/**
	 * Marks all affected elements of a given element as dirty
	 * @param InTransformElement The element that has changed
	 * @param bInitial If true the initial transform will be dirtied
	 * @param bAffectChildren If set to false children will not move (maintain global).
	 */
	CONTROLRIG_API void PropagateDirtyFlags(FRigTransformElement* InTransformElement, bool bInitial, bool bAffectChildren, bool bComputeOpposed = true, bool bMarkDirty = true) const;

public:

	/**
	* Performs validation of the cache within the hierarchy on any mutation.
	*/
	void EnsureCacheValidity() const
	{
#if WITH_EDITOR
		if(bEnableCacheValidityCheck)
		{
			URigHierarchy* MutableThis = (URigHierarchy*)this; 
			MutableThis->EnsureCacheValidityImpl();
		}
#endif
	}

	/*
	 * Cleans up caches after load
	 */
	CONTROLRIG_API void CleanupInvalidCaches();

#if WITH_EDITOR
	// helper function to convert a dependency chain into a message string.
	CONTROLRIG_API FString GetMessageFromDependencyChain(const FRigHierarchyDependencyChain& InDependencyChain) const;
#endif

private:
	
	/**
	 * The topology version of the hierarchy changes when elements are
	 * added, removed, re-parented or renamed.
	 */
	UPROPERTY(transient)
	uint32 TopologyVersion;

	/**
	 * The parent weight version of this hierarchy changes whenever parents are
	 * added / removed or when the weights of a parent relationship changes.
	 */
	UPROPERTY(transient)
	uint32 ParentWeightVersion;
	
	/**
	 * The metadata version of the hierarchy changes when metadata is being
	 * created or removed (not when the metadata values changes)
	 */
	UPROPERTY(transient)
	uint32 MetadataVersion;

	/**
	 * The metadata version of the hierarchy changes when metadata is being
	 * created or removed (not when the metadata values changes)
	 */
	UPROPERTY(transient)
	uint16 MetadataTagVersion;

	/**
	 * If set to false the dirty flag propagation will be disabled
	 */
	UPROPERTY(transient)
	bool bEnableDirtyPropagation;

	// Storage for the elements
	mutable TArray<FRigBaseElement*> Elements;
	mutable FTransactionallySafeCriticalSection ElementsLock;
	mutable FTransactionallySafeCriticalSection IsDependentOnLock;

	// Storage for the elements
	mutable TArray<TArray<FRigBaseElement*>> ElementsPerType;

	TArray<int32> ChangedCurveIndices;
	bool bRecordCurveChanges;
	
	//
	struct FMetadataStorage
	{
		TMap<FName, FRigBaseMetadata*> MetadataMap;
		FName LastAccessName;
		FRigBaseMetadata* LastAccessMetadata = nullptr;

		void Reset();
		void Serialize(FArchive& Ar);

		friend FArchive& operator<<(FArchive& Ar, FMetadataStorage& Storage)
		{
			Storage.Serialize(Ar);
			return Ar;
		}
	};
	friend class UControlRigTestData;
	friend class UControlRigReplay;
	friend struct FControlRigTestDataFrame;

	// Managed lookup from Element Key to Index
	TMap<FRigElementKey, int32> ElementIndexLookup;

	// Managed lookup from Component Key to Index
	TMap<FRigComponentKey, int32> ComponentIndexLookup;

	// Per element pose storage. Storage is defined here rather than on the elements
	// to reduce memory consumption. Only elements created by MakeElement point to
	// the element storage. Copied elements via the copy constructor or copy operator
	// do not have URigHierarchy as an owner and therefore do not carry poses with them.
	FRigReusableElementStorage<FTransform> ElementTransforms;

	// Per element dirty state storage. Storage is defined here rather than on the elements
	// to reduce memory consumption. Only elements created by MakeElement point to
	// the element storage. Copied elements via the copy constructor or copy operator
	// do not have URigHierarchy as an owner and therefore do not carry metadata with them.
	FRigReusableElementStorage<bool> ElementDirtyStates;

	// Per element curve storage. Storage is defined here rather than on the elements
	// to reduce memory consumption. Only elements created by MakeElement point to
	// the element storage. Copied elements via the copy constructor or copy operator
	// do not have URigHierarchy as an owner and therefore do not carry curves with them.
	FRigReusableElementStorage<float> ElementCurves;

	// A list of ranges which can be used to copy all poses from initial to current, for example 
	TArray<TTuple<int32, int32>> ElementTransformRanges;

	// Allocates the default element storage for an element
	CONTROLRIG_API void AllocateDefaultElementStorage(FRigBaseElement* InElement, bool bUpdateAllElements);

	// Deallocates the default element storage for an element
	CONTROLRIG_API void DeallocateElementStorage(FRigBaseElement* InElement);

	// Updates all storage pointers of the elements for poses and dirty states
	CONTROLRIG_API void UpdateElementStorage();

	// Orders the element storage by storing first initial, then current,
	// within each first local, then global, and within each of those lists
	// we'll place bones, nulls, controls etc in that order
	CONTROLRIG_API bool SortElementStorage();

	// Compacts the element storage
	CONTROLRIG_API bool ShrinkElementStorage();

	// Helper function to iterate all transform element storage
	CONTROLRIG_API void ForEachTransformElementStorage(TFunction<void(FRigTransformElement*,ERigTransformType::Type,ERigTransformStorageType::Type,FRigComputedTransform*,FRigTransformDirtyState*)> InCallback);

	// Returns the computed transform and dirty state for a given element
	CONTROLRIG_API TTuple<FRigComputedTransform*,FRigTransformDirtyState*> GetElementTransformStorage(
		const FRigElementKeyAndIndex& InKey,
		ERigTransformType::Type InTransformType,
		ERigTransformStorageType::Type InStorageType = ERigTransformStorageType::Pose);

	// Returns the range of the element transform / dirty state storage for a given
	// transform type. this is only valid if the element storage has been sorted. 
	CONTROLRIG_API TOptional<TTuple<int32,int32>> GetElementStorageRange(ERigTransformType::Type InTransformType) const;

	// Element metadata storage. Storage is defined here rather than on the elements
	// to reduce memory consumption. Only elements created by MakeElement point to
	// the element storage. Copied elements via the copy constructor or copy operator
	// do not have URigHierarchy as an owner and therefore do not carry metadata with them.
	FRigReusableElementStorage<FMetadataStorage> ElementMetadata;

	// Element component storage. Storage is defined here rather than on the elements
	// to reduce memory consumption. Only elements created by MakeComponent point to
	// the component storage.
	TArray<FInstancedStruct> ElementComponents;

	// Batch heads (OwnedInstances > 1) removed from Elements via RemoveElement.
	// These own contiguous memory blocks that can't be freed at removal time because
	// sibling elements in the same batch may still be live. Freed during Reset_Impl.
	TArray<FRigBaseElement*> OrphanedBatchHeads;

	// A quick-lookup cache for elements' children. Each element that has a ChildCacheIndex
	// not equal to INDEX_NONE is an index into the offset and count cache below, which in
	// turn contains an offset into the ChildElementCache, which stores consecutive runs of
	// children for that element. This makes it quick to get a TArrayView on the list of
	// children.
	struct FChildElementOffsetAndCount
	{
		int32 Offset;
		int32 Count;
	};
	
	TArray<FChildElementOffsetAndCount> ChildElementOffsetAndCountCache;
	TArray<FRigBaseElement*> ChildElementCache;

	// The topology version at which the child element cache was constructed. If it differs
	// from the stored TopologyVersion, then the cache is rebuilt. 
	uint32 ChildElementCacheTopologyVersion = std::numeric_limits<uint32>::max();

	///////////////////////////////////////////////
	/// Undo redo related
	///////////////////////////////////////////////

	/**
	 * The index identifying where we stand with the stack
	 */
	UPROPERTY()
	int32 TransformStackIndex;

	/**
	 * A flag to indicate if the next serialize should contain only transform changes
	 */
	bool bTransactingForTransformChange;
	
	/**
	 * The stack of actions to undo.
	 * Note: This is also used when performing traces on the hierarchy.
	 */
	TArray<FRigTransformStackEntry> TransformUndoStack;

	/**
	 * The stack of actions to undo
	 */
	TArray<FRigTransformStackEntry> TransformRedoStack;

#if WITH_EDITORONLY_DATA
	/** The keys that were selected before undo / redo. */
	TArray<FRigHierarchyKey> SelectedKeysBeforeUndo;
#endif

	/**
	 * Sets the transform stack index - which in turns performs a series of undos / redos
	 * @param InTransformStackIndex The new index for the transform stack
	 */
	CONTROLRIG_API bool SetTransformStackIndex(int32 InTransformStackIndex);

	/**
	 * Stores a transform on the stack
	 */
	CONTROLRIG_API void PushTransformToStack(
			const FRigElementKey& InKey,
            ERigTransformStackEntryType InEntryType,
            ERigTransformType::Type InTransformType,
            const FTransform& InOldTransform,
            const FTransform& InNewTransform,
            bool bAffectChildren,
            bool bModify);

	/**
	 * Stores a curve value on the stack
	 */
	CONTROLRIG_API void PushCurveToStack(
            const FRigElementKey& InKey,
            float InOldCurveValue,
            float InNewCurveValue,
            bool bInOldIsCurveValueSet,
            bool bInNewIsCurveValueSet,
            bool bModify);

	/**
	 * Restores a transform on the stack
	 */
	CONTROLRIG_API bool ApplyTransformFromStack(const FRigTransformStackEntry& InEntry, bool bUndo);

	/**
	 * Computes all parts of the pose
	 */
	CONTROLRIG_API void ComputeAllTransforms();

#if WITH_EDITOR
	/** Compares the selection changes made by between the last PreEditUndo and PostEditUndo calls and calls Notify to inform listening systems. */
	void NotifyPostUndoSelectionChanges();
#endif

	/**
	 * The max length for an element / component name within this hierarchy.
	 * Defaults to GetDefaultMaxNameLength().
	 */
	int32 MaxNameLength;
	
	/**
	 * Manages merging transform actions into one during an interaction
	 */
	bool bIsInteracting;

	/**
	* Stores the last key being interacted on
	*/
	FRigElementKey LastInteractedKey;

	/** 
	 * If set to true all notifs coming from this hierarchy will be suspended
	 */
	bool bSuspendNotifications;

	/** 
	 * If set to true all metadata changes notifs coming from this hierarchy will be suspended
	 */
	bool bSuspendMetadataNotifications = false;

	/**
	 * If set to true all name space key related warnings will be suspended
	 */
	bool bSuspendNameSpaceKeyWarnings;

	/**
	 * The event fired during undo / redo
	 */
	FRigHierarchyUndoRedoTransformEvent UndoRedoEvent;

	TWeakObjectPtr<URigHierarchy> HierarchyForSelectionPtr;
	TWeakObjectPtr<URigHierarchy> DefaultHierarchyPtr;
	TArray<FRigHierarchyKey> OrderedSelection;

	UPROPERTY(Transient)
	TObjectPtr<URigHierarchyController> HierarchyController;
	bool bIsControllerAvailable;

	UPROPERTY(Transient)
	mutable TObjectPtr<UModularRigRuleManager> RuleManager;

	TMap<FRigHierarchyKey, FRigHierarchyKey> PreviousHierarchyParentMap;

	/*We save this so Sequencer can remap this after load*/
	TMap<FRigHierarchyKey, FRigHierarchyKey> PreviousHierarchyNameMap;

	int32 ResetPoseHash;
	TArray<bool> ResetPoseIsFilteredOut;
	TArray<int32> ElementsToRetainLocalTransform;

	mutable THierarchyCache<TMap<TTuple<int32, int32>, bool>> ElementDependencyCache;

	bool bIsCopyingHierarchy;

#if WITH_EDITOR

	// this is mainly used for propagating changes between hierarchies in the direction of blueprint -> CDO -> other instances
	struct FRigHierarchyListener
	{
		FRigHierarchyListener()
			: Hierarchy(nullptr)
			, bShouldReactToInitialChanges(true)
			, bShouldReactToCurrentChanges(true) 
		{}

		bool ShouldReactToChange(ERigTransformType::Type InTransformType) const
		{
			if(Hierarchy.IsValid())
			{
				if(ERigTransformType::IsInitial(InTransformType))
				{
					return bShouldReactToInitialChanges;
				}

				if(ERigTransformType::IsCurrent(InTransformType))
				{
					return bShouldReactToCurrentChanges;
				}
			}
			return false;
		}
		
		TWeakObjectPtr<URigHierarchy> Hierarchy;
		bool bShouldReactToInitialChanges;
		bool bShouldReactToCurrentChanges;
	};
	
	TArray<FRigHierarchyListener> ListeningHierarchies;
	friend class FRigHierarchyListenerGuard;

	// a bool to guard against circular dependencies among listening hierarchies
	bool bPropagatingChange;

	// a bool to disable any propagation checks and force propagation
	bool bForcePropagation;
	
#endif

#if WITH_EDITOR

protected:
	
	int32 TraceFramesLeft;
	int32 TraceFramesCaptured;
	TMap<FName, FRigPose> TracePoses;

#endif

	static int32 RigElementTypeToFlatIndex(ERigElementType InElementType)
	{
		switch(InElementType)
		{
			case ERigElementType::Bone:
			{
				return 0;
			}
			case ERigElementType::Null:
			{
				return 1;
			}
			case ERigElementType::Control:
			{
				return 2;
			}
			case ERigElementType::Curve:
			{
				return 3;
			}
			case ERigElementType::Reference:
			{
				return 4;
			}
			case ERigElementType::Connector:
			{
				return 5;
			}
			case ERigElementType::Socket:
			{
				return 6;
			}
			case ERigElementType::All:
			default:
			{
				break;
			}
		}

		return INDEX_NONE;
	}

	static ERigElementType FlatIndexToRigElementType(int32 InIndex)
	{
		switch(InIndex)
		{
			case 0:
			{
				return ERigElementType::Bone;
			}
			case 1:
			{
				return ERigElementType::Null;
			}
			case 2:
			{
				return ERigElementType::Control;
			}
			case 3:
			{
				return ERigElementType::Curve;
			}
			case 4:
			{
				return ERigElementType::Reference;
			}
			case 5:
			{
				return ERigElementType::Connector;
			}
			case 6:
			{
				return ERigElementType::Socket;
			}
			default:
			{
				break;
			}
		}

		return ERigElementType::None;
	}
	
	template<typename ElementType>
	static constexpr int32 ElementToFlatIndex()
	{
		static_assert(std::is_convertible_v<ElementType, FRigBaseElement>, "ElementType can only be a FRigBaseElement type.");
		
		if constexpr (std::is_same_v<ElementType, FRigBoneElement>)
		{
			return 0;
		}
		else if constexpr (std::is_same_v<ElementType, FRigNullElement>)
		{
			return 1;
		}
		else if constexpr (std::is_same_v<ElementType, FRigControlElement>)
		{
			return 2;
		}
		else if constexpr (std::is_same_v<ElementType, FRigCurveElement>)
		{
			return 3;
		}
		else if constexpr (std::is_same_v<ElementType, FRigReferenceElement>)
		{
			return 4;
		}
		else if constexpr (std::is_same_v<ElementType, FRigConnectorElement>)
		{
			return 5;
		}
		else if constexpr (std::is_same_v<ElementType, FRigSocketElement>)
		{
			return 6;
		}
		else
		{
			return INDEX_NONE;
		}
	}

public:

	const FRigElementKeyCollection* FindCachedCollection(uint32 InHash) const
	{
		return KeyCollectionCache.Find(InHash);
	}
	
	FRigElementKeyCollection& FindOrAddCachedCollection(uint32 InHash) const
	{
		return KeyCollectionCache.FindOrAdd(InHash);
	};
	
	void AddCachedCollection(uint32 InHash, const FRigElementKeyCollection& InCollection) const
	{
		KeyCollectionCache.Add(InHash, InCollection);
	}

private:
	
	mutable TMap<uint32, FRigElementKeyCollection> KeyCollectionCache;

	CONTROLRIG_API FTransform GetWorldTransformForReference(const FRigVMExecuteContext* InContext, const FRigElementKey& InKey, bool bInitial);
	
	static float GetWeightForLerp(const float WeightA, const float WeightB)
	{
		float Weight = 0.f;
		const float ClampedWeightA = FMath::Max(WeightA, 0.f);
		const float ClampedWeightB = FMath::Max(WeightB, 0.f);
		const float OverallWeight = ClampedWeightA + ClampedWeightB;
		if(OverallWeight > SMALL_NUMBER)
		{
			Weight = ClampedWeightB / OverallWeight;
		}
		return Weight;
	}

	struct FConstraintIndex
	{
		int32 Location;
		int32 Rotation;
		int32 Scale;

		FConstraintIndex()
			: Location(INDEX_NONE)
			, Rotation(INDEX_NONE)
			, Scale(INDEX_NONE)
		{}

		FConstraintIndex(int32 InIndex)
			: Location(InIndex)
			, Rotation(InIndex)
			, Scale(InIndex)
		{}
	};

	CONTROLRIG_API FTransform ComputeLocalControlValue(
		FRigControlElement* ControlElement,
		const FTransform& InGlobalTransform,
		ERigTransformType::Type InTransformType) const;
	
	CONTROLRIG_API FTransform SolveParentConstraints(
		const FRigElementParentConstraintArray& InConstraints,
		const ERigTransformType::Type InTransformType,
		const FTransform& InLocalOffsetTransform,
		bool bApplyLocalOffsetTransform,
		const FTransform& InLocalPoseTransform,
		bool bApplyLocalPoseTransform) const;

	CONTROLRIG_API FTransform InverseSolveParentConstraints(
		const FTransform& InGlobalTransform,
		const FRigElementParentConstraintArray& InConstraints,
		const ERigTransformType::Type InTransformType,
		const FTransform& InLocalOffsetTransform) const;

	CONTROLRIG_API FTransform LazilyComputeParentConstraint(
		const FRigElementParentConstraintArray& Constraints,
		int32 InIndex,
		const ERigTransformType::Type InTransformType,
		const FTransform& InLocalOffsetTransform,
		bool bApplyLocalOffsetTransform,
		const FTransform& InLocalPoseTransform,
		bool bApplyLocalPoseTransform) const;

	static CONTROLRIG_API void ComputeParentConstraintIndices(
		const FRigElementParentConstraintArray& InConstraints,
		ERigTransformType::Type InTransformType,
		FConstraintIndex& OutFirstConstraint,
		FConstraintIndex& OutSecondConstraint,
		FConstraintIndex& OutNumConstraintsAffecting,
		FRigElementWeight& OutTotalWeight
	);

	static CONTROLRIG_API void IntegrateParentConstraintVector(
		FVector& OutVector,
		const FTransform& InTransform,
		float InWeight,
		bool bIsLocation);

	static CONTROLRIG_API void IntegrateParentConstraintQuat(
		int32& OutNumMixedRotations,
		FQuat& OutFirstRotation,
		FQuat& OutMixedRotation,
		const FTransform& InTransform,
		float InWeight);

#if WITH_EDITOR
	static CONTROLRIG_API TArray<FString> ControlSettingsToPythonCommands(const FRigControlSettings& Settings, const FString& NameSettings);
	static CONTROLRIG_API TArray<FString> ConnectorSettingsToPythonCommands(const FRigConnectorSettings& Settings, const FString& NameSettings);
#endif

	template<typename T>
	const T& GetMetadata(const FRigElementKey& InItem, ERigMetadataType InType, const FRigName& InMetadataName, const T& DefaultValue) const
	{
		return GetMetadata<T>(Find(InItem), InType, InMetadataName, DefaultValue);
	}

	template<typename T>
	const T& GetMetadata(const FRigBaseElement* InElement, ERigMetadataType InType, const FRigName& InMetadataName, const T& DefaultValue) const
	{
		if(InElement)
		{
			if(const FRigBaseMetadata* Metadata = FindMetadataForElement(InElement, InMetadataName, InType))
			{
				return *static_cast<const T*>(Metadata->GetValueData());
			}
		}
		return DefaultValue;
	}

	template<typename T>
	const TArray<T>& GetArrayMetadata(const FRigElementKey& InItem, ERigMetadataType InType, const FRigName& InMetadataName) const
	{
		return GetArrayMetadata<T>(Find(InItem), InType, InMetadataName);
	}

	template<typename T>
	const TArray<T>& GetArrayMetadata(const FRigBaseElement* InElement, ERigMetadataType InType, const FRigName& InMetadataName) const
	{
		static const TArray<T> EmptyArray;
		return GetMetadata<TArray<T>>(InElement, InType, InMetadataName, EmptyArray);
	}

	template<typename T>
	bool SetMetadata(const FRigElementKey& InItem, ERigMetadataType InType, const FRigName& InMetadataName, const T& InValue)
	{
		return SetMetadata<T>(Find(InItem), InType, InMetadataName, InValue);
	}

	template<typename T>
	bool SetMetadata(FRigBaseElement* InElement, ERigMetadataType InType, const FRigName& InMetadataName, const T& InValue)
	{
		if(InElement)
		{
			constexpr bool bNotify = true;
			if (FRigBaseMetadata* Metadata = GetMetadataForElement(InElement, InMetadataName, InType, bNotify))
			{
				return Metadata->SetValueData(&InValue, sizeof(T));
			}
		}
		return false;
	}

	template<typename T>
	bool SetArrayMetadata(const FRigElementKey& InItem, ERigMetadataType InType, const FRigName& InMetadataName, const TArray<T>& InValue)
	{
		return SetMetadata<TArray<T>>(Find(InItem), InType, InMetadataName, InValue);
	}

	template<typename T>
	bool SetArrayMetadata(FRigBaseElement* InElement, ERigMetadataType InType, const FRigName& InMetadataName, const TArray<T>& InValue)
	{
		return SetMetadata<TArray<T>>(InElement, InType, InMetadataName, InValue);
	}

public:
	
	CONTROLRIG_API void PropagateMetadata(const FRigElementKey& InKey, const FName& InName, bool bNotify = true);
	CONTROLRIG_API void PropagateMetadata(const FRigBaseElement* InElement, const FName& InName, bool bNotify = true);


	CONTROLRIG_API TMap<FRigElementKey, URigHierarchy::FMetadataStorage> CopyMetadata() const;
	CONTROLRIG_API bool SetMetadata(const TMap<FRigElementKey, URigHierarchy::FMetadataStorage>& InMetadata);
	
private:
	
	CONTROLRIG_API void OnMetadataChanged(const FRigElementKey& InKey, const FName& InName);
	CONTROLRIG_API void OnMetadataTagChanged(const FRigElementKey& InKey, const FName& InTag, bool bAdded);

public:
	/** Returns a metadata ptr to the given element's metadata. If the meta data, with the same name, doesn't exist already a new entry
	    is created for that element. If the name matches but the type differs, the existing metadata is destroyed and a new one with the
	    matching type is created instead.
	    */
	CONTROLRIG_API FRigBaseMetadata* GetMetadataForElement(FRigBaseElement* InElement, const FName& InName, ERigMetadataType InType, bool bInNotify);
	
	/** Attempts to find element's metadata of the given name and type. If either the element doesnt exist, the name doesn't exist or the
	    type doesn't match, then \c nullptr is returned.
	    */ 
	CONTROLRIG_API FRigBaseMetadata* FindMetadataForElement(const FRigBaseElement* InElement, const FName& InName, ERigMetadataType InType);
	CONTROLRIG_API const FRigBaseMetadata* FindMetadataForElement(const FRigBaseElement* InElement, const FName& InName, ERigMetadataType InType) const;

	/** Attempts to find if the element has any metadata.
		*/ 
	CONTROLRIG_API bool HasMetadata(const FRigBaseElement* InElement) const;
	
	/** Removes the named meta data for the given element, regardless of type. If the element doesn't exist, or it doesn't have any
	    metadata of the given name, this function does nothing and returns \c false.
		*/
	CONTROLRIG_API bool RemoveMetadataForElement(FRigBaseElement* InElement, const FName& InName);
	CONTROLRIG_API bool RemoveAllMetadataForElement(FRigBaseElement* InElement);
	CONTROLRIG_API bool RemoveAllMetadata();
	
	CONTROLRIG_API void CopyAllMetadataFromElement(FRigBaseElement* InTargetElement, const FRigBaseElement* InSourceElement);
	
protected:
	bool bEnableCacheValidityCheck;

	static CONTROLRIG_API bool bEnableValidityCheckbyDefault;

	UPROPERTY(transient)
	TObjectPtr<URigHierarchy> HierarchyForCacheValidation;

	mutable TMap<FRigElementKey, FRigElementKey> DefaultParentPerElement;
	mutable uint32 DefaultParentCacheTopologyVersion;

	bool bUsePreferredEulerAngles;
	mutable bool bAllowNameSpaceWhenSanitizingName;

public:
	bool UsesPreferredEulerAngles() const { return bUsePreferredEulerAngles; }

private:
	
	CONTROLRIG_API void EnsureCacheValidityImpl();

	static CONTROLRIG_API FName GetUniqueName(const FName& InName, TFunction<bool(const FName&)> IsNameAvailablePredicate);

	// NOTE: it is not safe to read or write the execute context without locking it first as the
	// FRigHierarchyExecuteContextBracket can change the context from the main thread (from sequencer for example)
	// at the same time that a control rig is being evaluated on an animation thread.
	mutable FTransactionallySafeCriticalSection ExecuteContextLock;
	const FRigVMExtendedExecuteContext* ExecuteContext;

#if WITH_EDITOR
	mutable bool bRecordInstructionsAtRuntime;

private:

#endif

	mutable TArray<int32> PoseVersionPerElement;
	inline int32 GetPoseVersion(int32 InIndex) const
	{
		if (InIndex > INDEX_NONE)
		{
			if(!PoseVersionPerElement.IsValidIndex(InIndex))
			{
				PoseVersionPerElement.SetNumZeroed(InIndex + 1);
			}
			return PoseVersionPerElement[InIndex];
		}
		return INDEX_NONE;
	}
	
	inline void IncrementPoseVersion(int32 InIndex) const
	{
		// don't do anything if the pose version array is empty
		// or the element has not been requested yet.
		if(PoseVersionPerElement.IsValidIndex(InIndex))
		{
			PoseVersionPerElement[InIndex]++;
		}
	}

	FRigElementKeyRedirector* ElementKeyRedirector;

	CONTROLRIG_API void UpdateVisibilityOnProxyControls();

	static CONTROLRIG_API const TArray<FString>& GetTransformTypeStrings();

	struct FQueuedNotification
	{
		ERigHierarchyNotification Type;
		FRigElementKey Key;
		FName ComponentName;
		
		bool operator == (const FQueuedNotification& InOther) const
		{
			return Type == InOther.Type && Key == InOther.Key && ComponentName == InOther.ComponentName;
		}
	};
	TQueue<FQueuedNotification, EQueueMode::SingleThreaded> QueuedNotifications;

	CONTROLRIG_API void QueueNotification(ERigHierarchyNotification InNotification, const FRigNotificationSubject& InSubject);
	CONTROLRIG_API void SendQueuedNotifications();
	CONTROLRIG_API void Reset_Impl(bool bResetElements);
#if WITH_EDITOR
	CONTROLRIG_API void ForEachListeningHierarchy(TFunctionRef<void(const FRigHierarchyListener&)> PerListeningHierarchyFunction);
#endif

public:

	CONTROLRIG_API FRigElementKey PatchElementKeyInLookup(const FRigElementKey& InKey, const TMap<FRigHierarchyModulePath, FName>* InModulePathToName = nullptr) const;
	CONTROLRIG_API void PatchElementMetadata(const TMap<FRigHierarchyModulePath, FName>& InModulePathToName);
	CONTROLRIG_API void PatchModularRigComponentKeys(const TMap<FRigHierarchyModulePath, FName>& InModulePathToName);
	CONTROLRIG_API void PatchZeroWeightAnimationChannels();

	template<typename RangeType>
	static void ConvertElementsToKeys(RangeType InElements, TArray<FRigElementKey>& OutKeys)
	{
		OutKeys.Reserve(OutKeys.Num() + InElements.Num());
		for (const typename RangeType::ElementType Element: InElements)
		{
			OutKeys.Add(Element->Key);
		}
	}

	template<typename RangeType>
	static void ConvertElementsToIndices(RangeType InElements, TArray<int32>& OutIndices)
	{
		OutIndices.Reserve(OutIndices.Num() + InElements.Num());
		for (const typename RangeType::ElementType Element: InElements)
		{
			OutIndices.Add(Element->Index);
		}
	}

	template<typename ElementType = FRigBaseElement, typename RangeType = TArray<FRigBaseElement*>>
	static void ConvertElements(RangeType InElements, TArray<ElementType*>& OutElements, bool bFilterNull = true)
	{
		OutElements.Reserve(OutElements.Num() + InElements.Num());
		for (const typename RangeType::ElementType Element: InElements)
		{
			ElementType* CastElement = Cast<ElementType>(Element);
			if(CastElement || bFilterNull)
			{
				OutElements.Add(CastElement);
			}
		}
	}

	template<typename RangeType>
	static TArray<FRigElementKey> ConvertElementsToKeys(RangeType InElements)
	{
		TArray<FRigElementKey> ElementKeys;
		ConvertElementsToKeys(InElements, ElementKeys);
		return ElementKeys;
	}

	template<typename RangeType>
	static TArray<int32> ConvertElementsToIndices(RangeType InElements)
	{
		TArray<int32> ElementIndices;
		ConvertElementsToIndices(InElements, ElementIndices);
		return ElementIndices;
	}

	template<typename ElementType = FRigBaseElement, typename RangeType = TArray<FRigBaseElement*>>
	static TArray<ElementType*> ConvertElements(RangeType InElements, bool bFilterNull = true)
	{
		TArray<ElementType*> OutElements;
		ConvertElements<ElementType, RangeType>(InElements, OutElements, bFilterNull);
		return OutElements;
	}

private:
	
	// the currently destroyed element - used to avoid notification storms
	const FRigBaseElement* ElementBeingDestroyed;

#if WITH_EDITOR
	mutable TArray<FRigElementKey> ReceivedNameSpaceBasedKeys;
#endif

	mutable THierarchyCache< TSet< FRigElementKey > > NonUniqueShortNamesCache;
	
	friend class URigHierarchyController;
	friend class UControlRig;
	friend class UFKControlRig;
	friend class UModularRig;
	friend class FControlRigBaseEditor;
	friend struct FRigBaseElement;
	friend struct FRigHierarchyValidityBracket;
	friend struct FRigHierarchyGlobalValidityBracket;
	friend struct FControlRigVisualGraphUtils;
	friend struct FRigHierarchyEnableControllerBracket;
	friend struct FRigHierarchyExecuteContextBracket;
	friend struct FRigHierarchyRedirectorGuard;
	friend struct FRigDispatch_GetMetadata;
	friend struct FRigDispatch_SetMetadata;
	friend struct FRigDispatch_GetModuleMetadata;
	friend struct FRigDispatch_SetModuleMetadata;
	friend class FControlRigHierarchySortElementStorage;
	friend class FControlRigHierarchyShrinkElementStorage;
	friend class FControlRigHierarchyRelinkElementStorage;
	friend class FRigHierarchyPoseAdapter;
	friend class UControlRigGraph;
	friend struct FRigHierarchyNameSpaceWarningBracket;
	friend class UControlRigRuntimeAsset;
	friend class UControlRigEditorAsset;
};

struct FRigHierarchyInteractionBracket
{
public:
	
	FRigHierarchyInteractionBracket(URigHierarchy* InHierarchy)
		: Hierarchy(InHierarchy)
	{
		check(Hierarchy);
		Hierarchy->Notify(ERigHierarchyNotification::InteractionBracketOpened, {});
	}

	~FRigHierarchyInteractionBracket()
	{
		Hierarchy->Notify(ERigHierarchyNotification::InteractionBracketClosed, {});
	}

private:

	URigHierarchy* Hierarchy;
};

struct FRigHierarchyEnableControllerBracket
{
private:
	FRigHierarchyEnableControllerBracket(URigHierarchy* InHierarchy, bool bEnable)
		: GuardIsControllerAvailable(InHierarchy->bIsControllerAvailable, bEnable)
	{
	}

	friend class URigHierarchy;
	friend class UControlRig;
	friend class UControlRig;

	// certain units are allowed to use this
	friend struct FRigUnit_AddParent;
	friend struct FRigUnit_AddParents;
	friend struct FRigUnit_SetDefaultParent;
	friend struct FRigUnit_SetChannelHosts;
	friend struct FRigUnit_AddAvailableSpaces;

private:
	TGuardValue<bool> GuardIsControllerAvailable;
};

struct FRigHierarchyExecuteContextBracket
{
private:

	FRigHierarchyExecuteContextBracket(URigHierarchy* InHierarchy, const FRigVMExtendedExecuteContext* InContext)
		: Hierarchy(InHierarchy)
		, PreviousContext(InHierarchy->ExecuteContext)
	{
		Hierarchy->ExecuteContextLock.Lock();
		Hierarchy->ExecuteContext = InContext;
	}

	~FRigHierarchyExecuteContextBracket()
	{
		Hierarchy->ExecuteContext = PreviousContext;
		Hierarchy->SendQueuedNotifications();
		Hierarchy->ExecuteContextLock.Unlock();
	}

	URigHierarchy* Hierarchy;
	const FRigVMExtendedExecuteContext* PreviousContext;

	friend class UControlRig;
	friend class UModularRig;
	friend class IControlRigEditorAssetInterface;
};

struct FRigHierarchyValidityBracket
{
	public:
	FRigHierarchyValidityBracket(URigHierarchy* InHierarchy)
		: bPreviousValue(false)
		, HierarchyPtr() 
	{
		if(InHierarchy)
		{
			bPreviousValue = InHierarchy->bEnableCacheValidityCheck;
			InHierarchy->bEnableCacheValidityCheck = false;
			HierarchyPtr = InHierarchy;
		}
	}

	~FRigHierarchyValidityBracket()
	{
		if(HierarchyPtr.IsValid())
		{
			URigHierarchy* Hierarchy = HierarchyPtr.Get();
			Hierarchy->bEnableCacheValidityCheck = bPreviousValue;
			Hierarchy->EnsureCacheValidity();
		}
	}

	private:

	bool bPreviousValue;
	TWeakObjectPtr<URigHierarchy> HierarchyPtr;
};

struct FRigHierarchyGlobalValidityBracket
{
public:
	FRigHierarchyGlobalValidityBracket(bool bEnable = true)
		: bPreviousValue(URigHierarchy::bEnableValidityCheckbyDefault)
	{
		URigHierarchy::bEnableValidityCheckbyDefault = true;
	}

	~FRigHierarchyGlobalValidityBracket()
	{
		URigHierarchy::bEnableValidityCheckbyDefault = bPreviousValue;
	}

private:

	bool bPreviousValue;
};

struct FRigHierarchyRedirectorGuard
{
public:
	FRigHierarchyRedirectorGuard(URigHierarchy* InHierarchy, FRigElementKeyRedirector& InRedirector)
		: Guard(InHierarchy->ElementKeyRedirector, &InRedirector)
	{
	}

	CONTROLRIG_API FRigHierarchyRedirectorGuard(UControlRig* InControlRig);

private:
	TGuardValue<FRigElementKeyRedirector*> Guard;
};

template<>
inline FVector2D URigHierarchy::GetControlValue(FRigControlElement* InControlElement, ERigControlValueType InValueType) const
{
	const FVector3f Value = GetControlValue(InControlElement, InValueType).Get<FVector3f>();
	return FVector2D(Value.X, Value.Y);
}

template<>
inline void URigHierarchy::SetControlValue(int32 InElementIndex, const FVector2D& InValue, ERigControlValueType InValueType, bool bSetupUndo)
{
	return SetControlValue(InElementIndex, FRigControlValue::Make<FVector3f>(FVector3f(InValue.X, InValue.Y, 0.f)), InValueType, bSetupUndo);
}

#if WITH_EDITOR

class FRigHierarchyListenerGuard
{
public:
	FRigHierarchyListenerGuard(
		URigHierarchy* InHierarchy, 
		bool bInEnableInitialChanges, 
		bool bInEnableCurrentChanges,
		URigHierarchy* InListeningHierarchy = nullptr)
			: Hierarchy(InHierarchy)
			, bEnableInitialChanges(bInEnableInitialChanges)
			, bEnableCurrentChanges(bInEnableCurrentChanges)
			, ListeningHierarchy(InListeningHierarchy)
	{
		check(Hierarchy);

		if(ListeningHierarchy == nullptr)
		{
			InitialFlags.AddUninitialized(Hierarchy->ListeningHierarchies.Num());
			CurrentFlags.AddUninitialized(Hierarchy->ListeningHierarchies.Num());

			for(int32 Index = 0; Index < Hierarchy->ListeningHierarchies.Num(); Index++)
			{
				URigHierarchy::FRigHierarchyListener& Listener = Hierarchy->ListeningHierarchies[Index];
				InitialFlags[Index] = Listener.bShouldReactToInitialChanges;
				CurrentFlags[Index] = Listener.bShouldReactToCurrentChanges;

				Listener.bShouldReactToInitialChanges = bInEnableInitialChanges; 
				Listener.bShouldReactToCurrentChanges = bInEnableCurrentChanges; 
			}
		}
		else
		{
			for(int32 Index = 0; Index < Hierarchy->ListeningHierarchies.Num(); Index++)
			{
				URigHierarchy::FRigHierarchyListener& Listener = Hierarchy->ListeningHierarchies[Index];

				if(Listener.Hierarchy.Get() == ListeningHierarchy)
				{
					InitialFlags.Add(Listener.bShouldReactToInitialChanges);
					CurrentFlags.Add(Listener.bShouldReactToCurrentChanges);

					Listener.bShouldReactToInitialChanges = bInEnableInitialChanges; 
					Listener.bShouldReactToCurrentChanges = bInEnableCurrentChanges;
					break;
				}
			}
		}
	}

	~FRigHierarchyListenerGuard()
	{
		if(ListeningHierarchy == nullptr)
		{
			check(Hierarchy->ListeningHierarchies.Num() == InitialFlags.Num());
			check(Hierarchy->ListeningHierarchies.Num() == CurrentFlags.Num());
			
			for(int32 Index = 0; Index < Hierarchy->ListeningHierarchies.Num(); Index++)
			{
				URigHierarchy::FRigHierarchyListener& Listener = Hierarchy->ListeningHierarchies[Index];
				Listener.bShouldReactToInitialChanges = InitialFlags[Index];
				Listener.bShouldReactToCurrentChanges = CurrentFlags[Index];
			}
		}
		else
		{
			for(int32 Index = 0; Index < Hierarchy->ListeningHierarchies.Num(); Index++)
			{
				URigHierarchy::FRigHierarchyListener& Listener = Hierarchy->ListeningHierarchies[Index];

				if(Listener.Hierarchy.Get() == ListeningHierarchy)
				{
					check(InitialFlags.Num() == 1);
					check(CurrentFlags.Num() == 1);

					Listener.bShouldReactToInitialChanges = InitialFlags[0];
					Listener.bShouldReactToCurrentChanges = CurrentFlags[0];
					break;
				}
			}
		}
	}

private:

	URigHierarchy* Hierarchy; 
	bool bEnableInitialChanges; 
	bool bEnableCurrentChanges;
	URigHierarchy* ListeningHierarchy;
	TArray<bool> InitialFlags;
	TArray<bool> CurrentFlags;
};

#endif

UINTERFACE(MinimalAPI)
class URigHierarchyProvider : public UInterface
{
	GENERATED_BODY()
};

class IRigHierarchyProvider
{
	GENERATED_BODY()

public:

	virtual URigHierarchy* GetHierarchy() const = 0;
};

class FRigHierarchyMemoryWriter : public FMemoryWriter
{
public:
	FRigHierarchyMemoryWriter() = delete;
	CONTROLRIG_API FRigHierarchyMemoryWriter(TArray<uint8>& InOutBuffer, TArray<FName>& InOutNames, bool bIsPersistent = false);
	using FMemoryWriter::operator<<; // For visibility of the overloads we don't override
	CONTROLRIG_API virtual FArchive& operator<<(FName& Value) override;
	CONTROLRIG_API virtual FArchive& operator<<(FText& Value) override;

protected:
	TArray<FName>& Names;
	TMap<FName,int32> NameToIndex;
};

class FRigHierarchyMemoryReader : public FMemoryReader
{
public:
	FRigHierarchyMemoryReader() = delete;
	CONTROLRIG_API FRigHierarchyMemoryReader(TArray<uint8>& InOutBuffer, TArray<FName>& InOutNames, bool bIsPersistent = false);
	using FMemoryReader::operator<<; // For visibility of the overloads we don't override
	CONTROLRIG_API virtual FArchive& operator<<(FName& Value) override;
	CONTROLRIG_API virtual FArchive& operator<<(FText& Value) override;

protected:
	TArray<FName>& Names;
};

struct FRigHierarchyNameSpaceWarningBracket : TGuardValue<bool>
{
	FRigHierarchyNameSpaceWarningBracket(URigHierarchy* InHierarchy)
		: TGuardValue<bool>(InHierarchy->bSuspendNameSpaceKeyWarnings, true)
	{
	}
};
