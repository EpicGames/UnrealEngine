// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyElements.h"

#include "RigHierarchyCache.generated.h"

#define UE_API CONTROLRIG_API

class URigHierarchy;
struct FModularRigConnections;

USTRUCT(BlueprintType)
struct FCachedRigElement
{
	GENERATED_BODY()

public:

	FCachedRigElement()
		: Key()
		, Index(UINT16_MAX)
		, ContainerVersion(INDEX_NONE)
		, Element(nullptr)
	{}

	FCachedRigElement(const FRigElementKey& InKey, const URigHierarchy* InHierarchy, bool bForceStoreKey = false)
		: Key()
		, Index(UINT16_MAX)
		, ContainerVersion(INDEX_NONE)
		, Element(nullptr)
	{
		UpdateCache(InKey, InHierarchy);
		if(bForceStoreKey)
		{
			Key = InKey;
		}
	}

	bool IsValid() const
	{
		return GetIndex() != INDEX_NONE && Key.IsValid();
	}

	void Invalidate()
	{
		Index = UINT16_MAX;
		ContainerVersion = INDEX_NONE;
		Element = nullptr;
	}

	void Reset()
	{
		Key = FRigElementKey();
		Index = UINT16_MAX;
		ContainerVersion = INDEX_NONE;
		Element = nullptr;
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	operator int32() const
	{
		return GetIndex();
	}

	explicit operator FRigElementKey() const
	{
		return Key;
	}

	int32 GetIndex() const
	{
		if(Index == UINT16_MAX)
		{
			return INDEX_NONE;
		}
		return (int32)Index;
	}

	const FRigElementKey& GetKey() const
	{
		return Key;
	}

	UE_API const FRigElementKey& GetResolvedKey() const;

	const FRigBaseElement* GetElement() const
	{
		return Element;
	}

	const FRigBaseElement* GetElement(const URigHierarchy* InHierarchy)
	{
		if(UpdateCache(InHierarchy))
		{
			return Element;
		}
		return nullptr;
	}

	UE_API bool UpdateCache(const URigHierarchy* InHierarchy);

	UE_API bool UpdateCache(const FRigElementKey& InKey, const URigHierarchy* InHierarchy);

	friend uint32 GetTypeHash(const FCachedRigElement& Cache)
	{
		return GetTypeHash(Cache.Key) * 13 + (uint32)Cache.Index;
	}

	UE_API bool IsIdentical(const FRigElementKey& InKey, const URigHierarchy* InHierarchy);

	bool operator ==(const FCachedRigElement& Other) const
	{
		return Index == Other.Index && Key == Other.Key;
	}

	bool operator !=(const FCachedRigElement& Other) const
	{
		return Index != Other.Index || Key != Other.Key;
	}

	bool operator ==(const FRigElementKey& Other) const
	{
		return Key == Other;
	}

	bool operator !=(const FRigElementKey& Other) const
	{
		return Key != Other;
	}

	bool operator ==(const int32& Other) const
	{
		return GetIndex() == Other;
	}

	bool operator !=(const int32& Other) const
	{
		return GetIndex() != Other;
	}

	bool operator <(const FCachedRigElement& Other) const
	{
		if (Key < Other.Key)
		{
			return true;
		}
		return Index < Other.Index;
	}

	bool operator >(const FCachedRigElement& Other) const
	{
		if (Key > Other.Key)
		{
			return true;
		}
		return Index > Other.Index;
	}

private:

	UE_API void Set(const FRigBaseElement* InElement, int32 InTopologyHashVersion = INDEX_NONE);

	UPROPERTY()
	FRigElementKey Key;

	UPROPERTY()
	uint16 Index;

	UPROPERTY()
	int32 ContainerVersion;

	const FRigBaseElement* Element;
	friend struct FCachedRigComponent;
	friend class URigHierarchy;
};

USTRUCT(BlueprintType)
struct FCachedRigComponent
{
	GENERATED_BODY()

public:

	FCachedRigComponent()
		: CachedElement()
		, Name(NAME_None)
		, Index(UINT16_MAX)
		, Component(nullptr)
	{}

	FCachedRigComponent(const FRigElementKey& InKey, const FName& InName, const URigHierarchy* InHierarchy, bool bForceStoreName = false)
		: CachedElement(InKey, InHierarchy, bForceStoreName)
		, Name(NAME_None)
		, Index(UINT16_MAX)
		, Component(nullptr)
	{
		UpdateCache(InKey, InName, InHierarchy);
		if(bForceStoreName)
		{
			Name = InName;
		}
	}

	FCachedRigComponent(const FRigComponentKey& InKey, const URigHierarchy* InHierarchy, bool bForceStoreName = false)
	: CachedElement(InKey.ElementKey, InHierarchy, bForceStoreName)
	, Name(NAME_None)
	, Index(UINT16_MAX)
	, Component(nullptr)
	{
		UpdateCache(InKey, InHierarchy);
		if(bForceStoreName)
		{
			Name = InKey.Name;
		}
	}

	bool IsValid() const
	{
		return GetIndex() != INDEX_NONE && CachedElement.IsValid();
	}

	void Invalidate()
	{
		CachedElement.Invalidate();
		Index = UINT16_MAX;
		Component = nullptr;
	}

	void Reset()
	{
		CachedElement.Reset();
		Index = UINT16_MAX;
		Name = NAME_None;
		Component = nullptr;
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	operator int32() const
	{
		return GetIndex();
	}

	int32 GetIndex() const
	{
		if(Index == UINT16_MAX)
		{
			return INDEX_NONE;
		}
		return (int32)Index;
	}

	const FRigElementKey& GetElementKey() const
	{
		return CachedElement.GetKey();
	}

	UE_API const FRigElementKey& GetResolvedElementKey() const;

	FRigComponentKey GetComponentKey() const
	{
		return FRigComponentKey(GetElementKey(), GetName());
	}

	FRigComponentKey GetResolvedComponentKey() const
	{
		return FRigComponentKey(GetResolvedElementKey(), GetName());
	}

	const FName& GetName() const
	{
		return Name;
	}

	const FRigBaseElement* GetElement() const
	{
		return CachedElement.GetElement();
	}

	const FRigBaseElement* GetElement(const URigHierarchy* InHierarchy)
	{
		return CachedElement.GetElement(InHierarchy);
	}

	const FRigBaseComponent* GetComponent() const
	{
		return Component;
	}

	const FRigBaseComponent* GetComponent(const URigHierarchy* InHierarchy)
	{
		if(UpdateCache(InHierarchy))
		{
			return Component;
		}
		return nullptr;
	}

	UE_API bool UpdateCache(const URigHierarchy* InHierarchy);

	UE_API bool UpdateCache(const FRigElementKey& InKey, const FName& InName, const URigHierarchy* InHierarchy);

	UE_API bool UpdateCache(const FRigComponentKey& InKey, const URigHierarchy* InHierarchy);

	friend uint32 GetTypeHash(const FCachedRigComponent& CachedComponent)
	{
		return GetTypeHash(CachedComponent.CachedElement) * 17 + (uint32)CachedComponent.Index;
	}

	UE_API bool IsIdentical(const FRigElementKey& InKey, const FName& InName, const URigHierarchy* InHierarchy);

	bool operator ==(const FCachedRigComponent& Other) const
	{
		return Index == Other.Index && CachedElement == Other.CachedElement;
	}

	bool operator !=(const FCachedRigComponent& Other) const
	{
		return Index != Other.Index || CachedElement != Other.CachedElement;
	}

	bool operator ==(const int32& Other) const
	{
		return GetIndex() == Other;
	}

	bool operator !=(const int32& Other) const
	{
		return GetIndex() != Other;
	}

	bool operator <(const FCachedRigComponent& Other) const
	{
		if (CachedElement < Other.CachedElement)
		{
			return true;
		}
		return Index < Other.Index;
	}

	bool operator >(const FCachedRigComponent& Other) const
	{
		if (CachedElement > Other.CachedElement)
		{
			return true;
		}
		return Index > Other.Index;
	}

private:

	UPROPERTY()
	FCachedRigElement CachedElement;

	UPROPERTY()
	FName Name;

	UPROPERTY()
	uint16 Index;

	const FRigBaseComponent* Component;
};

class FRigElementKeyRedirector
{
public:

	FRigElementKeyRedirector()
		: InternalKeyToExternalKey()
		, Hash(UINT32_MAX)
	{}

	typedef TArray<FRigElementKey, TInlineAllocator<1>> FKeyArray;
	typedef TMap<FRigElementKey, FKeyArray> FKeyMap;
	typedef TPair<FRigElementKey, FKeyArray> FKeyPair;
	typedef TArray<FCachedRigElement, TInlineAllocator<1>> FCachedKeyArray;
	typedef TMap<FRigElementKey, FCachedKeyArray> FCachedKeyMap;
	typedef TPair<FRigElementKey, FCachedKeyArray> FCachedKeyPair;

	UE_API FRigElementKeyRedirector(const FKeyMap& InMap, const URigHierarchy* InHierarchy);
	UE_API FRigElementKeyRedirector(const TMap<FRigElementKey, FRigElementKeyCollection>& InMap, const URigHierarchy* InHierarchy);
	UE_API FRigElementKeyRedirector(const FRigElementKeyRedirector& InOther, const URigHierarchy* InHierarchy);
	UE_API FRigElementKeyRedirector(const FModularRigConnections& InOther, const URigHierarchy* InHierarchy);

	bool Contains(const FRigElementKey& InKey) const { return InternalKeyToExternalKey.Contains(InKey); }
	const FCachedKeyArray* Find(const FRigElementKey& InKey) const { return InternalKeyToExternalKey.Find(InKey); }
	const FKeyArray* FindExternalKey(const FRigElementKey& InKey) const { return ExternalKeys.Find(InKey); }
	FCachedKeyArray* Find(const FRigElementKey& InKey) { return InternalKeyToExternalKey.Find(InKey); }
	UE_API const FRigElementKey* FindReverse(const FRigElementKey& InKey) const;
	uint32 GetHash() const { return Hash; }

	static UE_API FKeyArray Convert(const FCachedKeyArray& InCachedKeys);
	static UE_API FKeyArray Convert(const TArray<FRigElementKey>& InKeys);
	static UE_API FCachedKeyArray Convert(const FKeyArray& InKeys, const URigHierarchy* InHierarchy, bool bForceStoreKey = false, bool bOnlyValidItems = false);

private:

	UE_API void Add(const FRigElementKey& InSource, const FKeyArray& InTargets, const URigHierarchy* InHierarchy);

	template<typename T = FKeyArray>
	static bool IsValid(const T& InKeys)
	{
		if(InKeys.IsEmpty())
		{
			return false;
		}
		for(const FRigElementKey& Key : InKeys)
		{
			if(!Key.IsValid())
			{
				return false;
			}
		}
		return true;
	}

	FCachedKeyMap InternalKeyToExternalKey;
	FKeyMap ExternalKeys;
	uint32 Hash;

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend class FRigModuleInstanceDetails;
	friend class UModularRig;
	friend class FControlRigSchematicModel;
};

#undef UE_API
