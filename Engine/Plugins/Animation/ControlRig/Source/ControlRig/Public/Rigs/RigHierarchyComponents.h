// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "Textures/SlateIcon.h"
#include "Styling/SlateColor.h"
#include "Serialization/CustomVersion.h"
#include "RigHierarchyComponents.generated.h"

#define UE_API CONTROLRIG_API

class URigHierarchy;
class URigHierarchyController;

#define DECLARE_RIG_COMPONENT_METHODS(ComponentType) \
virtual UScriptStruct* GetScriptStruct() const override { return ComponentType::StaticStruct(); } \
template<typename T> \
friend const T* Cast(const ComponentType* InComponent) \
{ \
   return Cast<T>((const FRigBaseComponent*) InComponent); \
} \
template<typename T> \
friend T* Cast(ComponentType* InComponent) \
{ \
   return Cast<T>((FRigBaseComponent*) InComponent); \
} \
template<typename T> \
friend const T* CastChecked(const ComponentType* InComponent) \
{ \
	return CastChecked<T>((const FRigBaseComponent*) InComponent); \
} \
template<typename T> \
friend T* CastChecked(ComponentType* InComponent) \
{ \
	return CastChecked<T>((FRigBaseComponent*) InComponent); \
} \
static bool IsClassOf(const FRigBaseComponent* InComponent) \
{ \
	return InComponent->GetScriptStruct()->IsChildOf(StaticStruct()); \
}

struct FRigComponentState
{
public:

	UE_API bool IsValid() const;
	UE_API const UScriptStruct* GetComponentStruct() const;

	UE_API bool operator == (const FRigComponentState& InOther) const;

private:

	const UScriptStruct* ComponentStruct = nullptr;
	TArray<uint8> Data;
	FCustomVersionContainer Versions;

	friend struct FRigBaseComponent;
};

USTRUCT(BlueprintType)
struct FRigBaseComponent
{
	GENERATED_BODY()

public:

	FRigBaseComponent() = default;
	UE_API virtual ~FRigBaseComponent();

	FRigBaseComponent(const FRigBaseComponent& InOther)
	{
		*this = InOther;
	}
	
	FRigBaseComponent& operator=(const FRigBaseComponent& InOther)
	{
		// We purposefully do not copy any non-UPROPERTY entries, including Element
		Key = InOther.Key;
		CachedNameString.Reset();
		IndexInElement = InOther.IndexInElement;
		IndexInHierarchy = InOther.IndexInHierarchy;
		CreatedAtInstructionIndex = InOther.CreatedAtInstructionIndex;
		SpawnIndex = InOther.SpawnIndex;
		bSelected = InOther.bSelected;
		return *this;
	}

	// returns the default name to use when instantiating the component.
	UE_API virtual FName GetDefaultComponentName() const;

	// returns true if this component can be renamed
	virtual bool CanBeRenamed() const { return true; }

	// returns true if this component can only be added once
	virtual bool IsSingleton() const { return false; }

	// returns true if this component can be added to a given key.
	// if you want to determine if a component can be added as a top level component
	// you need to pass the URigHierarchy::GetTopLevelComponentElementKey() as InElementKey
	virtual bool CanBeAddedTo(const FRigElementKey& InElementKey, const URigHierarchy* InHierarchy, FString* OutFailureReason = nullptr) const { return true; }

	// allows the component to react to it being spawned
	virtual void OnAddedToHierarchy(URigHierarchy* InHierarchy, URigHierarchyController* InController) {}

	// return the icon to use for this component in the UI
	UE_API virtual const FSlateIcon& GetIconForUI() const;

	// returns the color to use for this component in the UI
	UE_API virtual FSlateColor GetColorForUI() const;

	// react to an element or component being renamed / reparented in the hierarchy
	virtual void OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey) {}
	
protected:

	static bool IsClassOf(const FRigBaseComponent* InComponent)
	{
		return true;
	}

	FRigComponentKey Key;

	// ReSharper disable once CppUE4ProbableMemoryIssuesWithUObject
	FRigBaseElement* Element = nullptr;
	int32 IndexInHierarchy = INDEX_NONE;
	int32 IndexInElement = INDEX_NONE;
	int32 CreatedAtInstructionIndex = INDEX_NONE;
	int32 SpawnIndex = INDEX_NONE;
	bool bSelected = false;

	mutable FString CachedNameString;

public:

	virtual UScriptStruct* GetScriptStruct() const { return FRigBaseComponent::StaticStruct(); }

	static UE_API TArray<UScriptStruct*> GetAllComponentScriptStructs(bool bSorted = false);

	const FRigComponentKey& GetKey() const { return Key; }
	const FName& GetFName() const { return Key.Name; }
	const FString& GetName() const
	{
		if(CachedNameString.IsEmpty() && !Key.Name.IsNone())
		{
			CachedNameString = Key.Name.ToString();
		}
		return CachedNameString;
	}
	bool IsTopLevel() const
	{
		return Key.IsTopLevel();
	}

	virtual const FName& GetDisplayName() const { return GetFName(); }
	int32 GetIndexInElement() const { return IndexInElement; }
	int32 GetIndexInHierarchy() const { return IndexInHierarchy; }
	bool IsSelected() const { return bSelected; }
	int32 GetCreatedAtInstructionIndex() const { return CreatedAtInstructionIndex; }
	int32 GetSpawnIndex() const { return SpawnIndex; }
	bool IsProcedural() const { return CreatedAtInstructionIndex != INDEX_NONE; }
	const FRigElementKey& GetElementKey() const { return Key.ElementKey; }
	const FRigBaseElement* GetElement() const { return Element; }
	FRigBaseElement* GetElement() { return Element; }

	UE_API bool Serialize(FArchive& Ar);
	UE_API virtual void Save(FArchive& Ar);
	UE_API virtual void Load(FArchive& Ar);

	UE_API FRigComponentState GetState() const;
	UE_API bool SetState(const FRigComponentState& InState);

	UE_API FString GetContentAsText() const;

	bool IsA(const UScriptStruct* InScriptStruct) const
	{
		return GetScriptStruct()->IsChildOf(InScriptStruct);
	}

	template<typename T>
	bool IsA() const { return T::IsClassOf(this); }

	template<typename T>
    friend const T* Cast(const FRigBaseComponent* InComponent)
	{
		if(InComponent)
		{
			if(InComponent->IsA<T>())
			{
				return static_cast<const T*>(InComponent);
			}
		}
		return nullptr;
	}

	template<typename T>
    friend T* Cast(FRigBaseComponent* InComponent)
	{
		if(InComponent)
		{
			if(InComponent->IsA<T>())
			{
				return static_cast<T*>(InComponent);
			}
		}
		return nullptr;
	}

	template<typename T>
    friend const T* CastChecked(const FRigBaseComponent* InComponent)
	{
		const T* Component = Cast<T>(InComponent);
		check(Component);
		return Component;
	}

	template<typename T>
    friend T* CastChecked(FRigBaseComponent* InComponent)
	{
		T* Component = Cast<T>(InComponent);
		check(Component);
		return Component;
	}

protected:

	friend class FControlRigBaseEditor;
	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FControlRigReplayTracks;
	friend class UControlRigWrapperObject;
};

template<>
struct TStructOpsTypeTraits<FRigBaseComponent> : public TStructOpsTypeTraitsBase2<FRigBaseComponent>
{
	enum 
	{
		WithSerializer = true,
	};
};

#undef UE_API
