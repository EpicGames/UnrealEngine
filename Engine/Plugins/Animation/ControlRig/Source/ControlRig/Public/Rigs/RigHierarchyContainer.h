// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyPose.h"
#include "RigBoneHierarchy.h"
#include "RigSpaceHierarchy.h"
#include "RigControlHierarchy.h"
#include "RigCurveContainer.h"
#include "RigHierarchyCache.h"
#include "RigInfluenceMap.h"
#include "RigHierarchyContainer.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
struct FRigHierarchyContainer;

USTRUCT()
struct FRigHierarchyContainer
{
public:

	GENERATED_BODY()

	UE_API FRigHierarchyContainer();
	UE_API FRigHierarchyContainer(const FRigHierarchyContainer& InOther);
	UE_API FRigHierarchyContainer& operator= (const FRigHierarchyContainer& InOther);

	UPROPERTY()
	FRigBoneHierarchy BoneHierarchy;

	UPROPERTY()
	FRigSpaceHierarchy SpaceHierarchy;

	UPROPERTY()
	FRigControlHierarchy ControlHierarchy;

	UPROPERTY()
	FRigCurveContainer CurveContainer;

private:

	UE_API TArray<FRigElementKey> ImportFromText(const FRigHierarchyCopyPasteContent& InData);

	friend class SRigHierarchy;
	friend class URigHierarchyController;
};

// this struct is still here for backwards compatibility - but not used anywhere
USTRUCT()
struct FRigHierarchyRef
{
	GENERATED_BODY()
};

#undef UE_API
