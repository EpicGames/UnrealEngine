// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchy.h"
#include "ReferenceSkeleton.h"
#include "RigHierarchyContainer.h"
#include "Animation/Skeleton.h"
#include "RigHierarchyContainer.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "RigHierarchyController.generated.h"

#define UE_API CONTROLRIG_API

UCLASS(MinimalAPI, BlueprintType)
class URigHierarchyController : public UObject
{
	GENERATED_BODY()

public:

	URigHierarchyController()
	: bReportWarningsAndErrors(true)
	, bSuspendAllNotifications(false)
	, bSuspendSelectionNotifications(false)
	, bSuspendPythonPrinting(false)
	, bSuspendParentCycleCheck(false)
	, CurrentInstructionIndex(INDEX_NONE)
	{}

	UE_API virtual ~URigHierarchyController();

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;

	// Returns the hierarchy currently linked to this controller
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API URigHierarchy* GetHierarchy() const;

	// Sets the hierarchy currently linked to this controller
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API void SetHierarchy(URigHierarchy* InHierarchy);

	/**
	 * Selects or deselects an element in the hierarchy
	 * @param InKey The key of the element to select
	 * @param bSelect If set to false the element will be deselected
	 * @param bClearSelection If this is true the selection will be cleared
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SelectElement(FRigElementKey InKey, bool bSelect = true, bool bClearSelection = false, bool bSetupUndo = false);

	/**
	 * Deselects or deselects an element in the hierarchy
	 * @param InKey The key of the element to deselect
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    bool DeselectElement(FRigElementKey InKey)
	{
		return SelectElement(InKey, false);
	}

	/**
	 * Selects or deselects a component in the hierarchy
	 * @param InKey The key of the component to select
	 * @param bSelect If set to false the component will be deselected
	 * @param bClearSelection If this is true the selection will be cleared
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SelectComponent(FRigComponentKey InKey, bool bSelect = true, bool bClearSelection = false, bool bSetupUndo = false);

	/**
	 * Deselects or deselects a component in the hierarchy
	 * @param InKey The key of the component to deselect
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	bool DeselectComponent(FRigComponentKey InKey)
	{
		return SelectComponent(InKey, false);
	}

	/**
	 * Selects or deselects a component or an element in the hierarchy
	 * @param InKey The key of the component or an element to select
	 * @param bSelect If set to false the component or an element will be deselected
	 * @param bClearSelection If this is true the selection will be cleared
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SelectHierarchyKey(FRigHierarchyKey InKey, bool bSelect = true, bool bClearSelection = false, bool bSetupUndo = false);

	/**
	 * Deselects or deselects a component or an element in the hierarchy
	 * @param InKey The key of the component or element to deselect
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	bool DeselectHierarchyKey(FRigHierarchyKey InKey, bool bSetupUndo = false)
	{
		return SelectHierarchyKey(InKey, false, false, bSetupUndo);
	}

	/**
	 * Sets the selection based on a list of element keys
	 * @param InKeys The array of keys of the elements to select
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetSelection(const TArray<FRigElementKey>& InKeys, bool bPrintPythonCommand = false, bool bSetupUndo = false);

	/**
	 * Sets the selection based on a list of component keys
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @param InKeys The array of keys of the component to select
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetComponentSelection(const TArray<FRigComponentKey>& InKeys, bool bPrintPythonCommand = false);

	/**
	 * Sets the selection based on a list of component keys
	 * @param InKeys The array of keys of the component to select
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetHierarchySelection(const TArray<FRigHierarchyKey>& InKeys, bool bPrintPythonCommand = false, bool bSetupUndo = false);

	/**
	 * Clears the selection
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if the selection was applied
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    bool ClearSelection(bool bSetupUndo = false)
	{
		return SetSelection(TArray<FRigElementKey>(), false, bSetupUndo);
	}
	
	/**
	 * Adds a bone to the hierarchy
	 * @param InName The suggested name of the new bone - will eventually be corrected by the namespace
	 * @param InParent The (optional) parent of the new bone. If you don't need a parent, pass FRigElementKey()
	 * @param InTransform The transform for the new bone - either in local or global space, based on bTransformInGlobal
	 * @param bTransformInGlobal Set this to true if the Transform passed is expressed in global space, false for local space.
	 * @param InBoneType The type of bone to add. This can be used to differentiate between imported bones and user defined bones.
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created bone.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigElementKey AddBone(FName InName, FRigElementKey InParent, FTransform InTransform, bool bTransformInGlobal = true, ERigBoneType InBoneType = ERigBoneType::User, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a null to the hierarchy
	 * @param InName The suggested name of the new null - will eventually be corrected by the namespace
	 * @param InParent The (optional) parent of the new null. If you don't need a parent, pass FRigElementKey()
	 * @param InTransform The transform for the new null - either in local or global null, based on bTransformInGlobal
	 * @param bTransformInGlobal Set this to true if the Transform passed is expressed in global null, false for local null.
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created null.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API FRigElementKey AddNull(FName InName, FRigElementKey InParent, FTransform InTransform, bool bTransformInGlobal = true, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a control to the hierarchy
	 * @param InName The suggested name of the new control - will eventually be corrected by the namespace
	 * @param InParent The (optional) parent of the new control. If you don't need a parent, pass FRigElementKey()
	 * @param InSettings All of the control's settings
	 * @param InValue The value to use for the control
	 * @param InOffsetTransform The transform to use for the offset
	 * @param InShapeTransform The transform to use for the shape
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created control.
	 */
    UE_API FRigElementKey AddControl(
    	FName InName,
    	FRigElementKey InParent,
    	FRigControlSettings InSettings,
    	FRigControlValue InValue,
    	FTransform InOffsetTransform = FTransform::Identity,
        FTransform InShapeTransform = FTransform::Identity,
        bool bSetupUndo = true,
        bool bPrintPythonCommand = false
        );

	/**
	 * Adds a control to the hierarchy
	 * @param InName The suggested name of the new control - will eventually be corrected by the namespace
	 * @param InParent The (optional) parent of the new control. If you don't need a parent, pass FRigElementKey()
	 * @param InSettings All of the control's settings
	 * @param InValue The value to use for the control
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The key for the newly created control.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController, meta = (DisplayName = "Add Control", ScriptName = "AddControl"))
    FRigElementKey AddControl_ForBlueprint(
        FName InName,
        FRigElementKey InParent,
        FRigControlSettings InSettings,
        FRigControlValue InValue,
        bool bSetupUndo = true,
        bool bPrintPythonCommand = false
    )
	{
		return AddControl(InName, InParent, InSettings, InValue, FTransform::Identity, FTransform::Identity, bSetupUndo, bPrintPythonCommand);
	}

		/**
	 * Adds a control to the hierarchy
	 * @param InName The suggested name of the new animation channel - will eventually be corrected by the namespace
	 * @param InParentControl The parent of the new animation channel.
	 * @param InSettings All of the animation channel's settings
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created animation channel.
	 */
    UE_API FRigElementKey AddAnimationChannel(
    	FName InName,
    	FRigElementKey InParentControl,
    	FRigControlSettings InSettings,
        bool bSetupUndo = true,
        bool bPrintPythonCommand = false
    );

	/**
	 * Adds a control to the hierarchy
	 * @param InName The suggested name of the new animation channel - will eventually be corrected by the namespace
	 * @param InParentControl The parent of the new animation channel.
	 * @param InSettings All of the animation channel's settings
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The key for the newly created animation channel.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController, meta = (DisplayName = "Add Control", ScriptName = "AddAnimationChannel"))
    FRigElementKey AddAnimationChannel_ForBlueprint(
        FName InName,
        FRigElementKey InParentControl,
        FRigControlSettings InSettings,
        bool bSetupUndo = true,
        bool bPrintPythonCommand = false
    )
	{
		return AddAnimationChannel(InName, InParentControl, InSettings, bSetupUndo, bPrintPythonCommand);
	}

	/**
	 * Adds a curve to the hierarchy
	 * @param InName The suggested name of the new curve - will eventually be corrected by the namespace
	 * @param InValue The value to use for the curve
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created curve.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API FRigElementKey AddCurve(
        FName InName,
        float InValue = 0.f,
        bool bSetupUndo = true,
		bool bPrintPythonCommand = false
        );

	/**
	* Adds an reference to the hierarchy
	* @param InName The suggested name of the new reference - will eventually be corrected by the namespace
	* @param InParent The (optional) parent of the new reference. If you don't need a parent, pass FRigElementKey()
	* @param InDelegate The delegate to use to pull the local transform
	* @param bSetupUndo If set to true the stack will record the change for undo / redo
	* @return The key for the newly created reference.
	*/
    UE_API FRigElementKey AddReference(
        FName InName,
        FRigElementKey InParent,
        FRigReferenceGetWorldTransformDelegate InDelegate,
        bool bSetupUndo = false);

	/**
	 * Adds a connector to the hierarchy
	 * @param InName The suggested name of the new connector - will eventually be corrected by the namespace
	 * @param InSettings All of the connector's settings
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created bone.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigElementKey AddConnector(FName InName, FRigConnectorSettings InSettings = FRigConnectorSettings(), bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a socket to the hierarchy
	 * @param InName The suggested name of the new socket - will eventually be corrected by the namespace
	 * @param InParent The (optional) parent of the new null. If you don't need a parent, pass FRigElementKey()
	 * @param InTransform The transform for the new socket - either in local or global space, based on bTransformInGlobal
	 * @param bTransformInGlobal Set this to true if the Transform passed is expressed in global space, false for local space.
	 * @param InColor The color of the socket
	 * @param InDescription The description of the socket
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The key for the newly created bone.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigElementKey AddSocket(FName InName, FRigElementKey InParent, FTransform InTransform, bool bTransformInGlobal = true, const FLinearColor& InColor = FLinearColor::White, const FString& InDescription = TEXT(""), bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a socket to the first determined root bone the hierarchy
	 * @return The key for the newly created bone (or an invalid key).
	 */
	UE_API FRigElementKey AddDefaultRootSocket();
	/**
	 * Returns the control settings of a given control
	 * @param InKey The key of the control to receive the settings for
	 * @return The settings of the given control
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigControlSettings GetControlSettings(FRigElementKey InKey) const;

	/**
	 * Sets a control's settings given a control key
	 * @param InKey The key of the control to set the settings for
	 * @param InSettings The settings to set
	 * @return Returns true if the settings have been set correctly
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API bool SetControlSettings(FRigElementKey InKey, FRigControlSettings InSettings, bool bSetupUndo = false) const;

	/**
	 * Adds a component to the hierarchy
	 * @param InComponentStruct The script struct of the component to add
	 * @param InName The suggested name of the new component.
	 * @param InElement The element the component will be added to.
	 * @param InContent The (optional) serialized text default for the component
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The name of the newly created component
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigComponentKey AddComponent(UScriptStruct* InComponentStruct, FName InName, FRigElementKey InElement, FString InContent = TEXT(""), bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
     * Removes a component from the hierarchy
     * @param InComponent The component to remove
     * @param bSetupUndo If set to true the stack will record the change for undo / redo
     * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
     * @return True if the component was removed successfully
     */
    UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API bool RemoveComponent(FRigComponentKey InComponent, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Renames an existing component in the hierarchy
	 * @param InComponent The key of the component to rename
	 * @param InName The new name to set for the component
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @param bClearSelection True if the selection should be cleared after a rename
	 * @return Returns the new key used for the component
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigComponentKey RenameComponent(FRigComponentKey InComponent, FName InName, bool bSetupUndo = false, bool bPrintPythonCommand = false, bool bClearSelection = true);

	/**
	 * Reparents an existing component in the hierarchy
	 * @param InComponentKey The component key to reparent
	 * @param InParentElementKey The new element key to reparent to
	 * @param bClearSelection True if the selection should be cleared after a reparenting
	 * @return Returns the new component key if successful or an invalid key if unsuccessful
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FRigComponentKey ReparentComponent(FRigComponentKey InComponentKey, FRigElementKey InParentElementKey, bool bSetupUndo = false, bool bPrintPythonCommand = false, bool bClearSelection = true);

	/**
	 * Updates the content of a component in the hierarchy
	 * @param InComponent The component to change the content for
	 * @param InContent The serialized text default for the component
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return True if the component was updated correctly
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetComponentContent(FRigComponentKey InComponent, const FString& InContent, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Updates the state of a component in the hierarchy
	 * @param InComponent The component to change the content for
	 * @param InState The serialized state of the component
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return True if the component was updated correctly
	 */
	UE_API bool SetComponentState(FRigComponentKey InComponent, const FRigComponentState& InState, bool bSetupUndo = false);

	/**
	 * Imports an existing skeleton to the hierarchy
	 * @param InSkeleton The skeleton to import
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UE_API TArray<FRigElementKey> ImportBones(
		const FReferenceSkeleton& InSkeleton,
		const FName& InNameSpace,
		bool bReplaceExistingBones = true,
		bool bRemoveObsoleteBones = true,
		bool bSelectBones = false,
		bool bSetupUndo = false);

	/**
	 * Imports an existing skeleton to the hierarchy, restricting the bone list to the ones that exist in the provided Skeletal Mesh
	 * @param InSkeletalMesh The skeletal mesh asset to import
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UE_API TArray<FRigElementKey> ImportBones(
		USkeletalMesh* InSkeletalMesh,
		const FName& InNameSpace,
		bool bReplaceExistingBones = true,
		bool bRemoveObsoleteBones = true,
		bool bSelectBones = false,
		bool bSetupUndo = false);

	/**
	 * Imports the provided bone list to the hierarchy
	 * @param BoneInfos The array of MeshBoneInfos to use as source. Bones with name set to NAME_Mone are bones that have been excluded.
	 * @param BoneTransforms The transforms to initialize the hierarchy elements
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */	
	 UE_API TArray<FRigElementKey> ImportBones(
		const TArray<FMeshBoneInfo>& BoneInfos,
		const TArray<FTransform>& BoneTransforms,
		const FName& InNameSpace,
		bool bReplaceExistingBones = true,
		bool bRemoveObsoleteBones = true,
		bool bSelectBones = false,
		bool bSetupUndo = false);

	/**
	 * Imports an existing skeleton to the hierarchy
	 * @param InSkeleton The skeleton to import
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FRigElementKey> ImportBones(
		USkeleton* InSkeleton,
		FName InNameSpace = NAME_None,
		bool bReplaceExistingBones = true,
		bool bRemoveObsoleteBones = true,
		bool bSelectBones = false,
		bool bSetupUndo = false,
		bool bPrintPythonCommand = false);

	/**
	 * Imports an existing skeleton to the hierarchy, restricting the bone list to the ones that exist in the provided Skeletal Mesh
	 * @param InSkeletalMesh The skeletal mesh asset to import
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	 UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	 UE_API TArray<FRigElementKey> ImportBonesFromSkeletalMesh(
		USkeletalMesh* InSkeletalMesh,
		const FName& InNameSpace,
		bool bReplaceExistingBones = true,
		bool bRemoveObsoleteBones = true,
		bool bSelectBones = false,
		bool bSetupUndo = false,
		bool bPrintPythonCommand = false);

	/**
	 * Imports the sockets from existing skeleton to the hierarchy as nulls
	 * @param InSkeletalMesh The skeletal mesh asset to import
	 * @param InNameSpace The namespace to prefix the socket names with
	 * @param bReplaceExistingSockets If true existing sockets will be removed
	 * @param bRemoveObsoleteSockets If true sockets non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectSockets If true the sockets will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FRigElementKey> ImportSocketsFromSkeletalMesh(
	   USkeletalMesh* InSkeletalMesh,
	   const FName& InNameSpace,
	   bool bReplaceExistingSockets = true,
	   bool bRemoveObsoleteSockets = true,
	   bool bSelectSockets = false,
	   bool bSetupUndo = false,
	   bool bPrintPythonCommand = false);

#if WITH_EDITOR
	/**
	 * Imports an existing skeleton to the hierarchy
	 * @param InAssetPath The path to the uasset to import from
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API TArray<FRigElementKey> ImportBonesFromAsset(
        FString InAssetPath,
        FName InNameSpace = NAME_None,
        bool bReplaceExistingBones = true,
        bool bRemoveObsoleteBones = true,
        bool bSelectBones = false,
        bool bSetupUndo = false);
#endif

	/**
	 * Imports all curves from an anim curve metadata object to the hierarchy
	 * @param InAnimCurvesMetadata The anim curve metadata object to import the curves from
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UE_API TArray<FRigElementKey> ImportCurves(
		UAnimCurveMetaData* InAnimCurvesMetadata, 
		FName InNameSpace = NAME_None,
		bool bSetupUndo = false);

	/**
	 * Imports all curves from a skeleton to the hierarchy
	 * @param InSkeleton The skeleton to import the curves from
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bSelectCurves If true the curves will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FRigElementKey> ImportCurves(
		USkeleton* InSkeleton, 
		FName InNameSpace = NAME_None,  
		bool bSelectCurves = false,
		bool bSetupUndo = false,
		bool bPrintPythonCommand = false);

	/**
	 * Imports all curves from a skeletalmesh to the hierarchy
	 * @param InSkeletalMesh The skeletalmesh to import the curves from
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bSelectCurves If true the curves will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FRigElementKey> ImportCurvesFromSkeletalMesh(
		USkeletalMesh* InSkeletalMesh, 
		FName InNameSpace = NAME_None,  
		bool bSelectCurves = false,
		bool bSetupUndo = false,
		bool bPrintPythonCommand = false);

#if WITH_EDITOR
	/**
	 * Imports all curves from a skeleton to the hierarchy
	 * @param InAssetPath The path to the uasset to import from
	 * @param InNameSpace The namespace to prefix the bone names with
	 * @param bSelectCurves If true the curves will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API TArray<FRigElementKey> ImportCurvesFromAsset(
        FString InAssetPath,
        FName InNameSpace = NAME_None, 
        bool bSelectCurves = false,
        bool bSetupUndo = false);
#endif

	/**
	 * Imports all bones from a preview skeletal mesh. Used for rig modules and their preview skeleton
	 * @param bReplaceExistingBones If true existing bones will be removed
	 * @param bRemoveObsoleteBones If true bones non-existent in the skeleton will be removed from the hierarchy
	 * @param bSelectBones If true the bones will be selected upon import
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return The keys of the imported elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FRigElementKey> ImportPreviewSkeletalMesh(USkeletalMesh* InSkeletalMesh,
		bool bReplaceExistingBones = true,
		bool bRemoveObsoleteBones = true,
		bool bSelectBones = false,
		bool bSetupUndo = false);

	/**
	 * Exports the selected items to text
	 * @return The text representation of the selected items
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FString ExportSelectionToText() const;

	/**
	 * Exports a list of items to text
	 * @param InKeys The keys to export to text
	 * @return The text representation of the requested elements
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FString ExportToText(TArray<FRigElementKey> InKeys) const;

	/**
	 * Imports the content of a text buffer to the hierarchy
	 * @param InContent The string buffer representing the content to import
	 * @param bReplaceExistingElements If set to true existing items will be replaced / updated with the content in the buffer
	 * @param bSelectNewElements If set to true the new elements will be selected
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommands If set to true a python command equivalent to this call will be printed out
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FRigElementKey> ImportFromText(
		FString InContent,
		bool bReplaceExistingElements = false,
		bool bSelectNewElements = true,
		bool bSetupUndo = false,
		bool bPrintPythonCommands = false);

	UE_API TArray<FRigElementKey> ImportFromText(
		FString InContent,
		ERigElementType InAllowedTypes,
		bool bReplaceExistingElements = false,
		bool bSelectNewElements = true,
		bool bSetupUndo = false,
		bool bPrintPythonCommands = false);

	/**
	* Imports the content of a RigHierachyContainer (the hierarchy v1 pre 5.0)
	* This is used for backwards compatbility only during load and does not support undo.
	* @param InContainer The input hierarchy container
	*/
    UE_API TArray<FRigElementKey> ImportFromHierarchyContainer(const FRigHierarchyContainer& InContainer, bool bIsCopyAndPaste);

	/**
	 * Removes an existing element from the hierarchy
	 * @param InElement The key of the element to remove
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool RemoveElement(FRigElementKey InElement, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Renames an existing element in the hierarchy
	 * @param InElement The key of the element to rename
	 * @param InName The new name to set for the element
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @param bClearSelection True if the selection should be cleared after a rename
	 * @return Returns the new element key used for the element
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API FRigElementKey RenameElement(FRigElementKey InElement, FName InName, bool bSetupUndo = false, bool bPrintPythonCommand = false, bool bClearSelection = true);

	/**
	 * Changes the element's index within its default parent (or the top level)
	 * @param InElement The key of the element to rename
	 * @param InIndex The new index of the element to take within its default parent (or the top level)
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if the element has been reordered accordingly
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool ReorderElement(FRigElementKey InElement, int32 InIndex, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
 	 * Sets the display name on a control
 	 * @param InControl The key of the control to change the display name for
 	 * @param InDisplayName The new display name to set for the control
 	 * @param bRenameElement True if the control should also be renamed
 	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
 	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns the new display name used for the control
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API FName SetDisplayName(FRigElementKey InControl, FName InDisplayName, bool bRenameElement = false, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a new parent to an element. For elements that allow only one parent the parent will be replaced (Same as ::SetParent).
	 * @param InChild The key of the element to add the parent for
	 * @param InParent The key of the new parent to add
	 * @param InWeight The initial weight to give to the parent
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @param InDisplayLabel The optional display label to use for the parent constraint / space.
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool AddParent(FRigElementKey InChild, FRigElementKey InParent, float InWeight = 0.f, bool bMaintainGlobalTransform = true, FName InDisplayLabel = NAME_None, bool bSetupUndo = false);

	/**
	* Adds a new parent to an element. For elements that allow only one parent the parent will be replaced (Same as ::SetParent).
	* @param InChild The element to add the parent for
	* @param InParent The new parent to add
	* @param InWeight The initial weight to give to the parent
	* @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	* @param bRemoveAllParents If set to true all parents of the child will be removed first.
	* @param InDisplayLabel The optional display label to use for the parent constraint / space.
	* @return Returns true if successful.
	*/
	UE_API bool AddParent(FRigBaseElement* InChild, FRigBaseElement* InParent, float InWeight = 0.f, bool bMaintainGlobalTransform = true, bool bRemoveAllParents = false, const FName& InDisplayLabel = NAME_None);

	/**
	 * Removes an existing parent from an element in the hierarchy. For elements that allow only one parent the element will be unparented (same as ::RemoveAllParents)
	 * @param InChild The key of the element to remove the parent for
	 * @param InParent The key of the parent to remove
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool RemoveParent(FRigElementKey InChild, FRigElementKey InParent, bool bMaintainGlobalTransform = true, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
 	 * Removes all parents from an element in the hierarchy.
 	 * @param InChild The key of the element to remove all parents for
 	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool RemoveAllParents(FRigElementKey InChild, bool bMaintainGlobalTransform = true, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Sets a new parent to an element. For elements that allow more than one parent the parent list will be replaced.
	 * @param InChild The key of the element to set the parent for
	 * @param InParent The key of the new parent to set
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetParent(FRigElementKey InChild, FRigElementKey InParent, bool bMaintainGlobalTransform = true, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a new available space to the given control
	 * @param InControl The control to add the available space for
	 * @param InSpace The space to add to the available spaces list
	 * @param InDisplayLabel The optional display label to use for this space
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool AddAvailableSpace(FRigElementKey InControl, FRigElementKey InSpace, FName InDisplayLabel = NAME_None, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a new available space to the given control
	 * @param InControlElement The control element to add the available space for
	 * @param InSpaceElement The space element to add to the available spaces list
	 * @param InDisplayLabel The optional display label to use for the space
	 * @return Returns true if successful.
	 */
	UE_API bool AddAvailableSpace(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement, const FName& InDisplayLabel = NAME_None);

	/**
	 * Removes an available space from the given control
	 * @param InControl The control to remove the available space from
	 * @param InSpace The space to remove from the available spaces list
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool RemoveAvailableSpace(FRigElementKey InControl, FRigElementKey InSpace, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Reorders an available space for the given control
	 * @param InControl The control to reorder the host for
	 * @param InSpace The space to set the new index for
	 * @param InIndex The new index of the available space
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetAvailableSpaceIndex(FRigElementKey InControl, FRigElementKey InSpace, int32 InIndex, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Updates the label on an available space
	 * @param InControl The control to reorder the host for
	 * @param InSpace The space to set the new index for
	 * @param InDisplayLabel The new label of the available space
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool SetAvailableSpaceLabel(FRigElementKey InControl, FRigElementKey InSpace, FName InDisplayLabel, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Adds a new channel host to the animation channel
	 * @note This is just an overload of AddAvailableSpace for readability
	 * @param InChannel The animation channel to add the channel host for
	 * @param InHost The host to add to the channel to
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool AddChannelHost(FRigElementKey InChannel, FRigElementKey InHost, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Removes an channel host from the animation channel
	 * @note This is just an overload of RemoveAvailableSpace for readability
	 * @param InChannel The animation channel to remove the channel host from
	 * @param InHost The host to remove from the channel from
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommand If set to true a python command equivalent to this call will be printed out
	 * @return Returns true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API bool RemoveChannelHost(FRigElementKey InChannel, FRigElementKey InHost, bool bSetupUndo = false, bool bPrintPythonCommand = false);

	/**
	 * Duplicate the given elements
	 * @param InKeys The keys of the elements to duplicate
	 * @param bSelectNewElements If set to true the new elements will be selected
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommands If set to true a python command equivalent to this call will be printed out
	 * @return The keys of the 4d items
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API TArray<FRigElementKey> DuplicateElements(TArray<FRigElementKey> InKeys, bool bSelectNewElements = true, bool bSetupUndo = false, bool bPrintPythonCommands = false);

	/**
	 * Mirrors the given elements
	 * @param InKeys The keys of the elements to mirror
	 * @param InSettings The settings to use for the mirror operation
	 * @param bSelectNewElements If set to true the new elements will be selected
	 * @param bSetupUndo If set to true the stack will record the change for undo / redo
	 * @param bPrintPythonCommands If set to true a python command equivalent to this call will be printed out
	 * @return The keys of the mirrored items
	 */
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
    UE_API TArray<FRigElementKey> MirrorElements(TArray<FRigElementKey> InKeys, FRigVMMirrorSettings InSettings, bool bSelectNewElements = true, bool bSetupUndo = false, bool bPrintPythonCommands = false);

	/**
	 * Returns the modified event, which can be used to 
	 * subscribe to topological changes happening within the hierarchy.
	 * @return The event used for subscription.
	 */
	FRigHierarchyModifiedEvent& OnModified() { return ModifiedEvent; }

	/**
	 * Reports a warning to the console. This does nothing if bReportWarningsAndErrors is false.
	 * @param InMessage The warning message to report.
	 */
	UE_API void ReportWarning(const FString& InMessage) const;

	/**
	 * Reports an error to the console. This does nothing if bReportWarningsAndErrors is false.
	 * @param InMessage The error message to report.
	 */
	UE_API void ReportError(const FString& InMessage) const;

	/**
	 * Reports an error to the console and logs a notification to the UI. This does nothing if bReportWarningsAndErrors is false.
	 * @param InMessage The error message to report / notify.
	 */
	UE_API void ReportAndNotifyError(const FString& InMessage) const;

	template <typename... Types>
	void ReportWarningf(::UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportWarning(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportErrorf(::UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportError(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportAndNotifyErrorf(::UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportAndNotifyError(FString::Printf(Fmt, Args...));
	}

	UPROPERTY(transient)
	bool bReportWarningsAndErrors;

	/**
	 * Returns a reference to the suspend notifications flag
	 */
	bool& GetSuspendNotificationsFlag() { return bSuspendAllNotifications; }

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = URigHierarchyController)
	UE_API TArray<FString> GeneratePythonCommands();

	UE_API TArray<FString> GetAddElementPythonCommands(FRigBaseElement* Element) const;

	UE_API TArray<FString> GetAddBonePythonCommands(FRigBoneElement* Bone) const;

	UE_API TArray<FString> GetAddNullPythonCommands(FRigNullElement* Null) const;

	UE_API TArray<FString> GetAddControlPythonCommands(FRigControlElement* Control) const;

	UE_API TArray<FString> GetAddCurvePythonCommands(FRigCurveElement* Curve) const;

	UE_API TArray<FString> GetAddConnectorPythonCommands(FRigConnectorElement* Connector) const;

	UE_API TArray<FString> GetAddSocketPythonCommands(FRigSocketElement* Socket) const;

	UE_API TArray<FString> GetSetControlValuePythonCommands(const FRigControlElement* Control, const FRigControlValue& Value, const ERigControlValueType& Type) const;
	
	UE_API TArray<FString> GetSetControlOffsetTransformPythonCommands(const FRigControlElement* Control, const FTransform& Offset, bool bInitial = false, bool bAffectChildren = true) const;
	
	UE_API TArray<FString> GetSetControlShapeTransformPythonCommands(const FRigControlElement* Control, const FTransform& Transform, bool bInitial = false) const;

	UE_API TArray<FString> GetAddComponentPythonCommands(const FRigBaseComponent* Component) const;
#endif
	
private:

	FRigHierarchyModifiedEvent ModifiedEvent;
	UE_API void Notify(ERigHierarchyNotification InNotifType, const FRigNotificationSubject& InSubject);
	UE_API void HandleHierarchyModified(ERigHierarchyNotification InNotifType, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject) const;

	/**
	 * Returns true if this controller is valid / linked to a valid hierarchy.
	 * @return Returns true if this controller is valid / linked to a valid hierarchy.
	 */
	UE_API bool IsValid() const;

	/**
	 * Determine a safe new name for an element. If a name is passed which contains a namespace
	 * (for example "MyNameSpace:Control") we'll remove the namespace and just use the short name
	 * prefixed with the current namespace (for example: "MyOtherNameSpace:Control"). Name clashes
	 * will be resolved for the full name. So two modules with two different namespaces can have
	 * elements with the same short name (for example "MyModuleA:Control" and "MyModuleB:Control").
	 * @param InDesiredName The name provided by the user
	 * @param InElementType The kind of element we are about to create
	 * @param bAllowNameSpace If true the name won't be changed for namespaces
	 * @return The safe name of the element to create.
	 */
	UE_API FName GetSafeNewName(const FName& InDesiredName, ERigElementType InElementType, bool bAllowNameSpace = true) const;
	
	/**
	 * Adds a new element to the hierarchy
	 * @param InElementToAdd The new element to add to the hierarchy 
	 * @param InFirstParent The (optional) parent of the new bone. If you don't need a parent, pass nullptr
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @param InDesiredName The original desired name
	 * @return The index of the newly added element
	 */
	UE_API int32 AddElement(FRigBaseElement* InElementToAdd, FRigBaseElement* InFirstParent, bool bMaintainGlobalTransform, const FName& InDesiredName = NAME_None);

	/**
	 * Removes an existing element from the hierarchy.
	 * @param InElement The element to remove
	 * @return Returns true if successful.
	 */
	UE_API bool RemoveElement(FRigBaseElement* InElement);

	/**
	 * Renames an existing element in the hierarchy
	 * @param InElement The element to rename
	 * @param InName The new name to set for the element
	 * @param bClearSelection True if the selection should be cleared after a rename
	 * @return Returns true if successful.
	 */
    UE_API bool RenameElement(FRigBaseElement* InElement, const FName &InName, bool bClearSelection = true, bool bSetupUndoRedo = false);

	/**
	 * Renames an existing component in the hierarchy
	 * @param InComponent The component to rename
	 * @param InName The new name to set for the component
	 * @param bClearSelection True if the selection should be cleared after a rename
	 * @return Returns true if successful.
	 */
	UE_API bool RenameComponent(FRigBaseComponent* InComponent, const FName &InName, bool bClearSelection = true, bool bSetupUndoRedo = false);

	/**
	 * Reparents an existing component in the hierarchy
	 * @param InComponent The component to reparent
	 * @param InParentElement The new element to reparent to
	 * @param bClearSelection True if the selection should be cleared after a reparenting
	 * @return Returns true if successful.
	 */
	UE_API bool ReparentComponent(FRigBaseComponent* InComponent, FRigBaseElement* InParentElement, bool bClearSelection = true, bool bSetupUndoRedo = false);

	/**
	 * Changes the element's index within its default parent (or the top level)
	 * @param InElement The key of the element to rename
	 * @param InIndex The new index of the element to take within its default parent (or the top level)
	 * @return Returns true if the element has been reordered accordingly
	 */
	UE_API bool ReorderElement(FRigBaseElement* InElement, int32 InIndex);

	/**
 	 * Sets the display name on a control
 	 * @param InControlElement The element to change the display name for
 	 * @param InDisplayName The new display name to set for the control
 	 * @param bRenameElement True if the control should also be renamed
 	 * @return Returns true if successful.
 	 */
	UE_API FName SetDisplayName(FRigControlElement* InControlElement, const FName &InDisplayName, bool bRenameElement = false);

	/**
	 * Removes an existing parent from an element in the hierarchy. For elements that allow only one parent the element will be unparented (same as ::RemoveAllParents)
	 * @param InChild The element to remove the parent for
	 * @param InParent The parent to remove
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @return Returns true if successful.
	 */
	UE_API bool RemoveParent(FRigBaseElement* InChild, FRigBaseElement* InParent, bool bMaintainGlobalTransform = true);

	/**
	 * Removes all parents from an element in the hierarchy.
	 * @param InChild The element to remove all parents for
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @return Returns true if successful.
	 */
	UE_API bool RemoveAllParents(FRigBaseElement* InChild, bool bMaintainGlobalTransform = true);

	/**
	 * Sets a new parent to an element. For elements that allow more than one parent the parent list will be replaced.
	 * @param InChild The element to set the parent for
	 * @param InParent The new parent to set
	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @return Returns true if successful.
 	 */
	UE_API bool SetParent(FRigBaseElement* InChild, FRigBaseElement* InParent, bool bMaintainGlobalTransform = true);

	/**
	 * Removes an available space from the given control
	 * @param InControlElement The control element to remove the available space from
	 * @param InSpaceElement The space element to remove from the available spaces list
	 * @return Returns true if successful.
	 */
	UE_API bool RemoveAvailableSpace(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement);

	/**
	 * Reorders an available space for the given control
	 * @param InControlElement The control element to remove the available space from
	 * @param InSpaceElement The space element to remove from the available spaces list
	 * @param InIndex The new index of the available space
	 * @return Returns true if successful.
	 */
	UE_API bool SetAvailableSpaceIndex(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement, int32 InIndex);

	/**
	 * Updates the label on an available space
	 * @param InControlElement The control to reorder the host for
	 * @param InSpaceElement The space to set the new index for
	 * @param InDisplayLabel The new label of the available space
	 * @return Returns true if successful.
	 */
	UE_API bool SetAvailableSpaceLabel(FRigControlElement* InControlElement, const FRigTransformElement* InSpaceElement, const FName& InDisplayLabel);

	/**
	 * Adds a new element to the dirty list of the given parent.
	 * This function is recursive and will affect all parents in the tree.
	 * @param InParent The parent element to change the dirty list for
	 * @param InElementToAdd The child element to add to the dirty list
	 * @param InHierarchyDistance The distance of number of elements in the hierarchy
	 */
	UE_API void AddElementToDirty(FRigBaseElement* InParent, FRigBaseElement* InElementToAdd, int32 InHierarchyDistance = 1) const;

	/**
	 * Remove an existing element to the dirty list of the given parent.
	 * This function is recursive and will affect all parents in the tree.
	 * @param InParent The parent element to change the dirty list for
	 * @param InElementToRemove The child element to remove from the dirty list
	 */
	UE_API void RemoveElementToDirty(FRigBaseElement* InParent, FRigBaseElement* InElementToRemove) const;

#if WITH_EDITOR
	static UE_API USkeletalMesh* GetSkeletalMeshFromAssetPath(const FString& InAssetPath);
	static UE_API USkeleton* GetSkeletonFromAssetPath(const FString& InAssetPath);
#endif

	UE_API void UpdateComponentsOnHierarchyKeyChange(const TArray<TPair<FRigHierarchyKey, FRigHierarchyKey>>& InKeyMap, bool bSetupUndoRedo);

	/**
	 * Adds a newly created transform element to the hierarchy
	 * @param InNewElement The new transform element previously created.
	 * @param InParent The (optional) parent of the new transform element (this can be null).
	 * @param InInitialTransform The initial transform for the new element - either in local or global space, based on bTransformInGlobal.
	 * @param bTransformInGlobal Set this to true if the Transform passed is expressed in global space, false for local space.
 	 * @param bMaintainGlobalTransform If set to true the child will stay in the same place spatially, otherwise it will maintain it's local transform (and potential move).
	 * @param InName The original desired name.
	 */
	UE_API void AddTransformElementInternal(FRigTransformElement* InNewElement, FRigBaseElement* InParent, const FTransform& InInitialTransform, const bool bTransformInGlobal, const bool bMaintainGlobalTransform, const FName InName);
	
	/** 
	 * If set to true all notifications coming from this hierarchy will be suspended
	 */
	bool bSuspendAllNotifications;

	/** 
	 * If set to true selection related notifications coming from this hierarchy will be suspended
	 */
	bool bSuspendSelectionNotifications;

	/** 
	* If set to true all python printing can be disabled.  
	*/
	bool bSuspendPythonPrinting;

	/**
	 * If set to true parent cycle checking will be only deployed for elements
	 * that already have children. This is an optimization used in a few cases only.
	 */
	bool bSuspendParentCycleCheck;

	/**
	 * If set the controller will mark new items as procedural and created at the current instruction
	 */
	int32 CurrentInstructionIndex;

	/**
	 * This function can be used to override the controller's logging mechanism
	 */
	TFunction<void(EMessageSeverity::Type,const FString&)> LogFunction = nullptr;

	template<typename T>
	T* MakeElement(bool bAllocateStorage = false, bool bUpdateAllElementsStorage = false)
	{
		T* Element = GetHierarchy()->NewElement<T>(1, bAllocateStorage, bUpdateAllElementsStorage);
		Element->CreatedAtInstructionIndex = CurrentInstructionIndex;
		return Element;
	}
	
	friend class UControlRig;
	friend class UControlRig;
	friend class URigHierarchy;
	friend class FRigHierarchyControllerInstructionBracket;
	friend class UControlRigBlueprint;
};

class FRigHierarchyControllerInstructionBracket : TGuardValue<int32>
{
public:
	
	FRigHierarchyControllerInstructionBracket(URigHierarchyController* InController, int32 InInstructionIndex)
		: TGuardValue<int32>(InController->CurrentInstructionIndex, InInstructionIndex)
	{
	}
};

#undef UE_API
