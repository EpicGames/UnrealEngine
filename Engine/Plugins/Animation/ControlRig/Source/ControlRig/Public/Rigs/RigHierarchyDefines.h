// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TransformNoScale.h"
#include "EulerTransform.h"
#include "ControlRigObjectVersion.h"
#include "RigName.h"
#include "RigHierarchyDefines.generated.h"

#define UE_API CONTROLRIG_API

class URigHierarchy;
struct FRigBaseElement;
struct FRigBaseComponent;

// Debug define which performs a full check on the cache validity for all elements of the hierarchy.
// This can be useful for debugging cache validity bugs.
#define URIGHIERARCHY_ENSURE_CACHE_VALIDITY 0

/* 
 * This is rig element types that we support
 * This can be used as a mask so supported as a bitfield
 */
UENUM(BlueprintType, meta = (RigVMTypeAllowed))
enum class ERigElementType : uint8
{
	None = 0,
	Bone = 0x001,
	Null = 0x002,
	Space = Null UMETA(Hidden),
	Control = 0x004,
	Curve = 0x008,
	Physics = 0x010 UMETA(Hidden),
	Reference = 0x020,
	Connector = 0x040,
	Socket = 0x080,

	// All is used in iterations, it has to be after the last valid type
	All = Bone | Null | Control | Curve | Reference | Connector | Socket,

	First = Bone UMETA(Hidden), 
	Last = Socket UMETA(Hidden), 
	ToResetAfterConstructionEvent = Bone | Control | Curve | Socket UMETA(Hidden),
	TransformTypes = Bone | Null | Control | Reference | Socket UMETA(Hidden),
};

UENUM(BlueprintType)
enum class ERigBoneType : uint8
{
	Imported,
	User
};

/* 
 * The type of meta data stored on an element
 */
UENUM(BlueprintType, meta = (RigVMTypeAllowed))
enum class ERigMetadataType : uint8
{
	Bool,
	BoolArray,
	Float,
	FloatArray,
	Int32,
	Int32Array,
	Name,
	NameArray,
	Vector,
	VectorArray,
	Rotator,
	RotatorArray,
	Quat,
	QuatArray,
	Transform,
	TransformArray,
	LinearColor,
	LinearColorArray,
	RigElementKey,
	RigElementKeyArray,

	/** MAX - invalid */
	Invalid UMETA(Hidden),
};

UENUM()
enum class ERigHierarchyNotification : uint8
{
	ElementAdded,
	ElementRemoved,
	ElementRenamed,
	ElementSelected,
	ElementDeselected,
	ParentChanged,
	HierarchyReset,
	ControlSettingChanged,
	ControlVisibilityChanged,
	ControlDrivenListChanged,
	ControlShapeTransformChanged,
	ParentWeightsChanged,
	InteractionBracketOpened,
	InteractionBracketClosed,
	ElementReordered,
	ConnectorSettingChanged,
	SocketColorChanged,
	SocketDescriptionChanged,
	SocketDesiredParentChanged,
	HierarchyCopied,
	ComponentAdded,
	ComponentRemoved,
	ComponentContentChanged,
	ComponentSelected,
	ComponentDeselected,
	ComponentRenamed,
	ComponentReparented,
	ShortNameChanged,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

/*
 * Used for notifications - the subject can be a variety of things
 */
struct FRigNotificationSubject
{
	FRigNotificationSubject() = default;

	FRigNotificationSubject(const FRigBaseElement* InElement)
	: Element(InElement)
	, Component(nullptr)
	{}

	FRigNotificationSubject(const FRigBaseComponent* InComponent)
	: Element(nullptr)
	, Component(InComponent)
	{}
	
	const FRigBaseElement* Element = nullptr;
	const FRigBaseComponent* Component = nullptr;
};

UENUM(meta = (RigVMTypeAllowed))
enum class ERigEvent : uint8
{
	/** Invalid event */
	None,

	/** Request to Auto-Key the Control in Sequencer */
	RequestAutoKey,

	/** Request to open an Undo bracket in the client */
	OpenUndoBracket,

	/** Request to close an Undo bracket in the client */
	CloseUndoBracket,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

struct FRigHierarchySerializationSettings
{
	UE_API FRigHierarchySerializationSettings(const FArchive& InArchive);
	
	enum ESerializationPhase
	{
		StaticData,
		InterElementData
	};

	UE_API void Save(FArchive& InArchive);
	UE_API void Load(FArchive& InArchive);

	FControlRigObjectVersion::Type ControlRigVersion = FControlRigObjectVersion::LatestVersion;
	bool bIsSerializingToPackage = false;
	bool bUseCompressedArchive = false;
	bool bStoreCompactTransforms = true;
	bool bSerializeLocalTransform = true;
	bool bSerializeGlobalTransform = true;
	bool bSerializeInitialTransform = true;
	bool bSerializeCurrentTransform = true;
	ESerializationPhase SerializationPhase = ESerializationPhase::StaticData;
};

/** When setting control values what to do with regards to setting key.*/
UENUM()
enum class EControlRigSetKey : uint8
{
	DoNotCare = 0x0,    //Don't care if a key is set or not, may get set, say if auto key is on somewhere.
	Always,				//Always set a key here
	Never				//Never set a key here.
};

UENUM(BlueprintType, meta = (RigVMTypeAllowed))
enum class ERigControlType : uint8
{
	Bool,
    Float,
    Integer,
    Vector2D,
    Position,
    Scale,
    Rotator,
    Transform UMETA(Hidden),
    TransformNoScale UMETA(Hidden),
    EulerTransform,
	ScaleFloat,
};

UENUM(BlueprintType)
enum class ERigControlAnimationType : uint8
{
	// A visible, animatable control.
	AnimationControl,
	// An animation channel without a 3d shape
	AnimationChannel,
	// A control to drive other controls,
	// not animatable in sequencer.
	ProxyControl,
	// Visual feedback only - the control is
	// neither animatable nor selectable.
	VisualCue
};

UENUM(BlueprintType)
enum class ERigControlValueType : uint8
{
	Initial,
    Current,
    Minimum,
    Maximum
};

UENUM(BlueprintType, meta = (RigVMTypeAllowed))
enum class ERigControlVisibility : uint8
{
	// Visibility controlled by the graph
	UserDefined, 
	// Visibility Controlled by the selection of driven controls
	BasedOnSelection 
};

UENUM(BlueprintType, meta = (RigVMTypeAllowed))
enum class ERigControlAxis : uint8
{
	X,
    Y,
    Z
};

USTRUCT(BlueprintType)
struct FRigControlLimitEnabled
{
	GENERATED_BODY()
	
	FRigControlLimitEnabled()
		: bMinimum(false)
		, bMaximum(false)
	{}

	FRigControlLimitEnabled(bool InValue)
		: bMinimum(false)
		, bMaximum(false)
	{
		Set(InValue);
	}

	FRigControlLimitEnabled(bool InMinimum, bool InMaximum)
		: bMinimum(false)
		, bMaximum(false)
	{
		Set(InMinimum, InMaximum);
	}

	FRigControlLimitEnabled& Set(bool InValue)
	{
		return Set(InValue, InValue);
	}

	FRigControlLimitEnabled& Set(bool InMinimum, bool InMaximum)
	{
		bMinimum = InMinimum;
		bMaximum = InMaximum;
		return *this;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigControlLimitEnabled& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	bool operator ==(const FRigControlLimitEnabled& Other) const
	{
		return bMinimum == Other.bMinimum && bMaximum == Other.bMaximum;
	}

	bool operator !=(const FRigControlLimitEnabled& Other) const
	{
		return bMinimum != Other.bMinimum || bMaximum != Other.bMaximum;
	}

	bool IsOn() const { return bMinimum || bMaximum; }
	bool IsOff() const { return !bMinimum && !bMaximum; }
	UE_API bool GetForValueType(ERigControlValueType InValueType) const;
	UE_API void SetForValueType(ERigControlValueType InValueType, bool InValue);

	template<typename T>
	T Apply(const T& InValue, const T& InMinimum, const T& InMaximum) const
	{
		if(IsOff())
		{
			return InValue;
		}

		if(bMinimum && bMaximum)
		{
			if(InMinimum < InMaximum)
			{
				return FMath::Clamp<T>(InValue, InMinimum, InMaximum);
			}
			else
			{
				return FMath::Clamp<T>(InValue, InMaximum, InMinimum);
			}
		}

		if(bMinimum)
		{
			return FMath::Max<T>(InValue, InMinimum);
		}

		return FMath::Min<T>(InValue, InMaximum);
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limit)
	bool bMinimum;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limit)
	bool bMaximum;
};

USTRUCT(BlueprintType)
struct FRigControlValueStorage
{
public:

	GENERATED_BODY()

	FRigControlValueStorage()
	{
		FMemory::Memzero(this, sizeof(FRigControlValueStorage));
	}

	UPROPERTY()
	float Float00;

	UPROPERTY()
	float Float01;

	UPROPERTY()
	float Float02;

	UPROPERTY()
	float Float03;

	UPROPERTY()
	float Float10;

	UPROPERTY()
	float Float11;

	UPROPERTY()
	float Float12;

	UPROPERTY()
	float Float13;

	UPROPERTY()
	float Float20;

	UPROPERTY()
	float Float21;

	UPROPERTY()
	float Float22;

	UPROPERTY()
	float Float23;

	UPROPERTY()
	float Float30;

	UPROPERTY()
	float Float31;

	UPROPERTY()
	float Float32;

	UPROPERTY()
	float Float33;

	UPROPERTY()
	float Float00_2;

	UPROPERTY()
	float Float01_2;

	UPROPERTY()
	float Float02_2;

	UPROPERTY()
	float Float03_2;

	UPROPERTY()
	float Float10_2;

	UPROPERTY()
	float Float11_2;

	UPROPERTY()
	float Float12_2;

	UPROPERTY()
	float Float13_2;

	UPROPERTY()
	float Float20_2;

	UPROPERTY()
	float Float21_2;

	UPROPERTY()
	float Float22_2;

	UPROPERTY()
	float Float23_2;
	
	UPROPERTY()
	float Float30_2;

	UPROPERTY()
	float Float31_2;

	UPROPERTY()
	float Float32_2;

	UPROPERTY()
	float Float33_2;
	
	UPROPERTY()
	bool bValid;
};

USTRUCT(BlueprintType)
struct FRigControlValue
{
	GENERATED_BODY()

public:

	FRigControlValue()
		: FloatStorage()
#if WITH_EDITORONLY_DATA
		, Storage_DEPRECATED(FTransform::Identity)
#endif
	{
	}

	bool IsValid() const
	{
		return FloatStorage.bValid;
	}

	template<class T>
	T Get() const
	{
		return GetRef<T>();
	}

	template<class T>
	T& GetRef()
	{
		FloatStorage.bValid = true;
		return *(T*)&FloatStorage;
	}

	template<class T>
	const T& GetRef() const
	{
		return *(T*)&FloatStorage;
	}

	template<class T>
	void Set(T InValue)
	{
		GetRef<T>() = InValue;
	}

	template<class T>
	FString ToString() const
	{
		FString Result;
		TBaseStructure<T>::Get()->ExportText(Result, &GetRef<T>(), nullptr, nullptr, PPF_None, nullptr);
		return Result;
	}

	template<class T>
	T SetFromString(const FString& InString)
	{
		T Value;
		TBaseStructure<T>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<T>::Get()->GetName());
		Set<T>(Value);
		return Value;
	}

	friend FArchive& operator <<(FArchive& Ar, FRigControlValue& Value);

	FString ToPythonString(ERigControlType InControlType) const
	{
		FString ValueStr;
		
		switch (InControlType)
		{
			case ERigControlType::Bool: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_bool(%s)"), Get<bool>() ? TEXT("True") : TEXT("False")); break;							
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
				ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_float(%.6f)"), Get<float>()); break;
			case ERigControlType::Integer: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_int(%d)"), Get<int>()); break;
			case ERigControlType::Position:
			case ERigControlType::Scale:
				ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_vector(unreal.Vector(%.6f, %.6f, %.6f))"),
					Get<FVector3f>().X, Get<FVector3f>().Y, Get<FVector3f>().Z); break;
			case ERigControlType::Rotator: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_rotator(unreal.Rotator(pitch=%.6f, roll=%.6f, yaw=%.6f))"),
				Get<FVector3f>().X, Get<FVector3f>().Z, Get<FVector3f>().Y); break;
			case ERigControlType::Transform: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_euler_transform(unreal.EulerTransform(location=[%.6f,%.6f,%.6f],rotation=[%.6f,%.6f,%.6f],scale=[%.6f,%.6f,%.6f]))"),
				Get<FTransform_Float>().TranslationX,
				Get<FTransform_Float>().TranslationY,
				Get<FTransform_Float>().TranslationZ,
				Get<FTransform_Float>().GetRotation().Rotator().Pitch,
				Get<FTransform_Float>().GetRotation().Rotator().Yaw,
				Get<FTransform_Float>().GetRotation().Rotator().Roll,
				Get<FTransform_Float>().ScaleX,
				Get<FTransform_Float>().ScaleY,
				Get<FTransform_Float>().ScaleZ); break;
			case ERigControlType::EulerTransform: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_euler_transform(unreal.EulerTransform(location=[%.6f,%.6f,%.6f],rotation=[%.6f,%.6f,%.6f],scale=[%.6f,%.6f,%.6f]))"),
				Get<FEulerTransform_Float>().TranslationX,
				Get<FEulerTransform_Float>().TranslationY,
				Get<FEulerTransform_Float>().TranslationZ,
				Get<FEulerTransform_Float>().RotationPitch,
				Get<FEulerTransform_Float>().RotationYaw,
				Get<FEulerTransform_Float>().RotationRoll,
				Get<FEulerTransform_Float>().ScaleX,
				Get<FEulerTransform_Float>().ScaleY,
				Get<FEulerTransform_Float>().ScaleZ); break;
			case ERigControlType::Vector2D: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_vector2d(unreal.Vector2D(%.6f, %.6f))"),
				Get<FVector3f>().X, Get<FVector3f>().Y); break;
			case ERigControlType::TransformNoScale: ValueStr = FString::Printf(TEXT("unreal.RigHierarchy.make_control_value_from_euler_transform(unreal.EulerTransform(location=[%.6f,%.6f,%.6f],rotation=[%.6f,%.6f,%.6f],scale=[%.6f,%.6f,%.6f]))"),
				FEulerTransform(Get<FTransformNoScale_Float>().ToTransform().ToFTransform()).Location.X,
				FEulerTransform(Get<FTransformNoScale_Float>().ToTransform().ToFTransform()).Location.Y,
				FEulerTransform(Get<FTransformNoScale_Float>().ToTransform().ToFTransform()).Location.Z,
				FEulerTransform(Get<FTransformNoScale_Float>().ToTransform().ToFTransform()).Rotation.Euler().X,
				FEulerTransform(Get<FTransformNoScale_Float>().ToTransform().ToFTransform()).Rotation.Euler().Y,
				FEulerTransform(Get<FTransformNoScale_Float>().ToTransform().ToFTransform()).Rotation.Euler().Z,
				1.f, 1.f, 1.f); break;
			default: ensure(false);
		}

		return ValueStr;
	}

	template<class T>
	static FRigControlValue Make(T InValue)
	{
		FRigControlValue Value;
		Value.Set<T>(InValue);
		return Value;
	}

	FTransform GetAsTransform(ERigControlType InControlType, ERigControlAxis InPrimaryAxis) const
	{
		FTransform Transform = FTransform::Identity;
		switch (InControlType)
		{
			case ERigControlType::Bool:
			{
				Transform.SetLocation(FVector(Get<bool>() ? 1.f : 0.f, 0.f, 0.f));
				break;
			}
			case ERigControlType::Float:
			{
				const float ValueToGet = Get<float>();
				switch (InPrimaryAxis)
				{
					case ERigControlAxis::X:
					{
						Transform.SetLocation(FVector(ValueToGet, 0.f, 0.f));
						break;
					}
					case ERigControlAxis::Y:
					{
						Transform.SetLocation(FVector(0.f, ValueToGet, 0.f));
						break;
					}
					case ERigControlAxis::Z:
					{
						Transform.SetLocation(FVector(0.f, 0.f, ValueToGet));
						break;
					}
				}
				break;
			}
			case ERigControlType::ScaleFloat:
			{
				const float ValueToGet = Get<float>();
				const FVector ScaleVector(ValueToGet, ValueToGet, ValueToGet);
				Transform.SetScale3D(ScaleVector);
				break;
			}
			case ERigControlType::Integer:
			{
				const int32 ValueToGet = Get<int32>();
				switch (InPrimaryAxis)
				{
					case ERigControlAxis::X:
					{
						Transform.SetLocation(FVector((float)ValueToGet, 0.f, 0.f));
						break;
					}
					case ERigControlAxis::Y:
					{
						Transform.SetLocation(FVector(0.f, (float)ValueToGet, 0.f));
						break;
					}
					case ERigControlAxis::Z:
					{
						Transform.SetLocation(FVector(0.f, 0.f, (float)ValueToGet));
						break;
					}
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				const FVector3f ValueToGet = Get<FVector3f>();
				switch (InPrimaryAxis)
				{
					case ERigControlAxis::X:
					{
						Transform.SetLocation(FVector(0.f, ValueToGet.X, ValueToGet.Y));
						break;
					}
					case ERigControlAxis::Y:
					{
						Transform.SetLocation(FVector(ValueToGet.X, 0.f, ValueToGet.Y));
						break;
					}
					case ERigControlAxis::Z:
					{
						Transform.SetLocation(FVector(ValueToGet.X, ValueToGet.Y, 0.f));
						break;
					}
				}
				break;
			}
			case ERigControlType::Position:
			{
				Transform.SetLocation((FVector)Get<FVector3f>());
				break;
			}
			case ERigControlType::Scale:
			{
				Transform.SetScale3D((FVector)Get<FVector3f>());
				break;
			}
			case ERigControlType::Rotator:
			{
				const FVector3f RotatorAxes = Get<FVector3f>();
				Transform.SetRotation(FQuat(FRotator::MakeFromEuler((FVector)RotatorAxes)));
				break;
			}
			case ERigControlType::Transform:
			{
				Transform = Get<FTransform_Float>().ToTransform();
				Transform.NormalizeRotation();
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				const FTransformNoScale TransformNoScale = Get<FTransformNoScale_Float>().ToTransform();
				Transform = TransformNoScale;
				Transform.NormalizeRotation();
				break;
			}
			case ERigControlType::EulerTransform:
			{
				const FEulerTransform EulerTransform = Get<FEulerTransform_Float>().ToTransform();
				Transform = FTransform(EulerTransform.ToFTransform());
				Transform.NormalizeRotation();
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}
		return Transform;
	}
	
	void SetFromTransform(const FTransform& InTransform, ERigControlType InControlType, ERigControlAxis InPrimaryAxis)
	{
		switch (InControlType)
		{
			case ERigControlType::Bool:
			{
				Set<bool>(InTransform.GetLocation().X > SMALL_NUMBER);
				break;
			}
			case ERigControlType::Float:
			{
				switch (InPrimaryAxis)
				{
					case ERigControlAxis::X:
					{
						Set<float>(InTransform.GetLocation().X);
						break;
					}
					case ERigControlAxis::Y:
					{
						Set<float>(InTransform.GetLocation().Y);
						break;
					}
					case ERigControlAxis::Z:
					{
						Set<float>(InTransform.GetLocation().Z);
						break;
					}
				}
				break;
			}
			case ERigControlType::ScaleFloat:
			{
				Set<float>(InTransform.GetScale3D().X);
				break;
			}
			case ERigControlType::Integer:
			{
				switch (InPrimaryAxis)
				{
					case ERigControlAxis::X:
					{
						Set<int32>((int32)InTransform.GetLocation().X);
						break;
					}
					case ERigControlAxis::Y:
					{
						Set<int32>((int32)InTransform.GetLocation().Y);
						break;
					}
					case ERigControlAxis::Z:
					{
						Set<int32>((int32)InTransform.GetLocation().Z);
						break;
					}
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				const FVector Location = InTransform.GetLocation();
				switch (InPrimaryAxis)
				{
					case ERigControlAxis::X:
					{
						Set<FVector3f>(FVector3f(Location.Y, Location.Z, 0.f));
						break;
					}
					case ERigControlAxis::Y:
					{
						Set<FVector3f>(FVector3f(Location.X, Location.Z, 0.f));
						break;
					}
					case ERigControlAxis::Z:
					{
						Set<FVector3f>(FVector3f(Location.X, Location.Y, 0.f));
						break;
					}
				}
				break;
			}
			case ERigControlType::Position:
			{
				Set<FVector3f>((FVector3f)InTransform.GetLocation());
				break;
			}
			case ERigControlType::Scale:
			{
				Set<FVector3f>((FVector3f)InTransform.GetScale3D());
				break;
			}
			case ERigControlType::Rotator:
			{
				//allow for values ><180/-180 by getting diff and adding that back in.
				FRotator CurrentRotator = FRotator::MakeFromEuler((FVector)Get<FVector3f>());
				FRotator CurrentRotWind, CurrentRotRem;
				CurrentRotator.GetWindingAndRemainder(CurrentRotWind, CurrentRotRem);

				//Get Diff
				const FRotator NewRotator = FRotator(InTransform.GetRotation());
				FRotator DeltaRot = NewRotator - CurrentRotRem;
				DeltaRot.Normalize();

				//Add Diff
				CurrentRotator = CurrentRotator + DeltaRot;
				Set<FVector3f>((FVector3f)CurrentRotator.Euler());
				break;
			}
			case ERigControlType::Transform:
			{
				Set<FTransform_Float>(InTransform);
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				const FTransformNoScale NoScale = InTransform;
				Set<FTransformNoScale_Float>(NoScale);
				break;
			}
			case ERigControlType::EulerTransform:
			{
				//Find Diff of the rotation from current and just add that instead of setting so we can go over/under -180
				FEulerTransform NewTransform(InTransform);

				const FEulerTransform CurrentEulerTransform = Get<FEulerTransform_Float>().ToTransform();
				FRotator CurrentWinding;
				FRotator CurrentRotRemainder;
				CurrentEulerTransform.Rotation.GetWindingAndRemainder(CurrentWinding, CurrentRotRemainder);
				const FRotator NewRotator = InTransform.GetRotation().Rotator();
				FRotator DeltaRot = NewRotator - CurrentRotRemainder;
				DeltaRot.Normalize();
				const FRotator NewRotation(CurrentEulerTransform.Rotation + DeltaRot);
				NewTransform.Rotation = NewRotation;
				Set<FEulerTransform_Float>(NewTransform);
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}
	}

	void ApplyLimits(
		const TArray<FRigControlLimitEnabled>& LimitEnabled,
		ERigControlType InControlType,
		const FRigControlValue& InMinimumValue,
		const FRigControlValue& InMaximumValue)
	{
		if(LimitEnabled.IsEmpty())
		{
			return;
		}

		switch(InControlType)
		{
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				if (LimitEnabled[0].IsOn())
				{
					float& ValueRef = GetRef<float>();
					ValueRef = LimitEnabled[0].Apply<float>(ValueRef, InMinimumValue.Get<float>(), InMaximumValue.Get<float>());
				}
				break;
			}
			case ERigControlType::Integer:
			{
				if (LimitEnabled[0].IsOn())
				{
					int32& ValueRef = GetRef<int32>();
					ValueRef = LimitEnabled[0].Apply<int32>(ValueRef, InMinimumValue.Get<int32>(), InMaximumValue.Get<int32>());
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				if(LimitEnabled.Num() < 2)
				{
					return;
				}
				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn())
				{
					FVector3f& ValueRef = GetRef<FVector3f>();
					const FVector3f& Min = InMinimumValue.GetRef<FVector3f>();
					const FVector3f& Max = InMaximumValue.GetRef<FVector3f>();
					ValueRef.X = LimitEnabled[0].Apply<float>(ValueRef.X, Min.X, Max.X);
					ValueRef.Y = LimitEnabled[1].Apply<float>(ValueRef.Y, Min.Y, Max.Y);
				}
				break;
			}
			case ERigControlType::Position:
			{
				if(LimitEnabled.Num() < 3)
				{
					return;
				}
				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn() || LimitEnabled[2].IsOn())
				{
					FVector3f& ValueRef = GetRef<FVector3f>();
					const FVector3f& Min = InMinimumValue.GetRef<FVector3f>();
					const FVector3f& Max = InMaximumValue.GetRef<FVector3f>();
					ValueRef.X = LimitEnabled[0].Apply<float>(ValueRef.X, Min.X, Max.X);
					ValueRef.Y = LimitEnabled[1].Apply<float>(ValueRef.Y, Min.Y, Max.Y);
					ValueRef.Z = LimitEnabled[2].Apply<float>(ValueRef.Z, Min.Z, Max.Z);
				}
				break;
			}
			case ERigControlType::Scale:
			{
				if(LimitEnabled.Num() < 3)
				{
					return;
				}
				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn() || LimitEnabled[2].IsOn())
				{
					FVector3f& ValueRef = GetRef<FVector3f>();
					const FVector3f& Min = InMinimumValue.GetRef<FVector3f>();
					const FVector3f& Max = InMaximumValue.GetRef<FVector3f>();
					ValueRef.X = LimitEnabled[0].Apply<float>(ValueRef.X, Min.X, Max.X);
					ValueRef.Y = LimitEnabled[1].Apply<float>(ValueRef.Y, Min.Y, Max.Y);
					ValueRef.Z = LimitEnabled[2].Apply<float>(ValueRef.Z, Min.Z, Max.Z);
				}
				break;
			}
			case ERigControlType::Rotator:
			{
				if(LimitEnabled.Num() < 3)
				{
					return;
				}
				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn() || LimitEnabled[2].IsOn())
				{
					FVector3f& ValueRef = GetRef<FVector3f>();
					const FVector3f& Min = InMinimumValue.GetRef<FVector3f>();
					const FVector3f& Max = InMaximumValue.GetRef<FVector3f>();
					ValueRef.X = LimitEnabled[0].Apply<float>(ValueRef.X, Min.X, Max.X);
					ValueRef.Y = LimitEnabled[1].Apply<float>(ValueRef.Y, Min.Y, Max.Y);
					ValueRef.Z = LimitEnabled[2].Apply<float>(ValueRef.Z, Min.Z, Max.Z);
				}
				break;
			}
			case ERigControlType::Transform:
			{
				if(LimitEnabled.Num() < 9)
				{
					return;
				}

				FTransform_Float& ValueRef = GetRef<FTransform_Float>();
				const FTransform Min = InMinimumValue.GetRef<FTransform_Float>().ToTransform();
				const FTransform Max = InMaximumValue.GetRef<FTransform_Float>().ToTransform();

				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn() || LimitEnabled[2].IsOn())
				{
					ValueRef.TranslationX = LimitEnabled[0].Apply<float>(ValueRef.TranslationX, Min.GetLocation().X, Max.GetLocation().X);
					ValueRef.TranslationY = LimitEnabled[1].Apply<float>(ValueRef.TranslationY, Min.GetLocation().Y, Max.GetLocation().Y);
					ValueRef.TranslationZ = LimitEnabled[2].Apply<float>(ValueRef.TranslationZ, Min.GetLocation().Z, Max.GetLocation().Z);
				}
				if (LimitEnabled[3].IsOn() || LimitEnabled[4].IsOn() || LimitEnabled[5].IsOn())
				{
					const FRotator Rotator = FQuat(ValueRef.RotationX, ValueRef.RotationY, ValueRef.RotationZ, ValueRef.RotationW).Rotator();
					const FRotator MinRotator = Min.GetRotation().Rotator();
					const FRotator MaxRotator = Max.GetRotation().Rotator();

					FRotator LimitedRotator = Rotator;
					LimitedRotator.Pitch = LimitEnabled[3].Apply<float>(LimitedRotator.Pitch, MinRotator.Pitch, MaxRotator.Pitch);
					LimitedRotator.Yaw = LimitEnabled[4].Apply<float>(LimitedRotator.Yaw, MinRotator.Yaw, MaxRotator.Yaw);
					LimitedRotator.Roll = LimitEnabled[5].Apply<float>(LimitedRotator.Roll, MinRotator.Roll, MaxRotator.Roll);

					FQuat LimitedQuat(LimitedRotator);
					
					ValueRef.RotationX = LimitedQuat.X;
					ValueRef.RotationY = LimitedQuat.Y;
					ValueRef.RotationZ = LimitedQuat.Z;
					ValueRef.RotationW = LimitedQuat.W;
				}
				if (LimitEnabled[6].IsOn() || LimitEnabled[7].IsOn() || LimitEnabled[8].IsOn())
				{
					ValueRef.ScaleX = LimitEnabled[6].Apply<float>(ValueRef.ScaleX, Min.GetScale3D().X, Max.GetScale3D().X);
					ValueRef.ScaleY = LimitEnabled[7].Apply<float>(ValueRef.ScaleY, Min.GetScale3D().Y, Max.GetScale3D().Y);
					ValueRef.ScaleZ = LimitEnabled[8].Apply<float>(ValueRef.ScaleZ, Min.GetScale3D().Z, Max.GetScale3D().Z);
				}
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				if(LimitEnabled.Num() < 6)
				{
					return;
				}

				FTransformNoScale_Float& ValueRef = GetRef<FTransformNoScale_Float>();
				const FTransformNoScale Min = InMinimumValue.GetRef<FTransformNoScale_Float>().ToTransform();
				const FTransformNoScale Max = InMaximumValue.GetRef<FTransformNoScale_Float>().ToTransform();

				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn() || LimitEnabled[2].IsOn())
				{
					ValueRef.TranslationX = LimitEnabled[0].Apply<float>(ValueRef.TranslationX, Min.Location.X, Max.Location.X);
					ValueRef.TranslationY = LimitEnabled[1].Apply<float>(ValueRef.TranslationY, Min.Location.Y, Max.Location.Y);
					ValueRef.TranslationZ = LimitEnabled[2].Apply<float>(ValueRef.TranslationZ, Min.Location.Z, Max.Location.Z);
				}
				if (LimitEnabled[3].IsOn() || LimitEnabled[4].IsOn() || LimitEnabled[5].IsOn())
				{
					const FRotator Rotator = FQuat(ValueRef.RotationX, ValueRef.RotationY, ValueRef.RotationZ, ValueRef.RotationW).Rotator();
					const FRotator MinRotator = Min.Rotation.Rotator();
					const FRotator MaxRotator = Max.Rotation.Rotator();

					FRotator LimitedRotator = Rotator;
					LimitedRotator.Pitch = LimitEnabled[3].Apply<float>(LimitedRotator.Pitch, MinRotator.Pitch, MaxRotator.Pitch);
					LimitedRotator.Yaw = LimitEnabled[4].Apply<float>(LimitedRotator.Yaw, MinRotator.Yaw, MaxRotator.Yaw);
					LimitedRotator.Roll = LimitEnabled[5].Apply<float>(LimitedRotator.Roll, MinRotator.Roll, MaxRotator.Roll);

					FQuat LimitedQuat(LimitedRotator);

					ValueRef.RotationX = LimitedQuat.X;
					ValueRef.RotationY = LimitedQuat.Y;
					ValueRef.RotationZ = LimitedQuat.Z;
					ValueRef.RotationW = LimitedQuat.W;
				}
				break;
			}

			case ERigControlType::EulerTransform:
			{
				if(LimitEnabled.Num() < 9)
				{
					return;
				}

				FEulerTransform_Float& ValueRef = GetRef<FEulerTransform_Float>();
				const FEulerTransform_Float& Min = InMinimumValue.GetRef<FEulerTransform_Float>();
				const FEulerTransform_Float& Max = InMaximumValue.GetRef<FEulerTransform_Float>();

				if (LimitEnabled[0].IsOn() || LimitEnabled[1].IsOn() || LimitEnabled[2].IsOn())
				{
					ValueRef.TranslationX = LimitEnabled[0].Apply<float>(ValueRef.TranslationX, Min.TranslationX, Max.TranslationX);
					ValueRef.TranslationY = LimitEnabled[1].Apply<float>(ValueRef.TranslationY, Min.TranslationY, Max.TranslationY);
					ValueRef.TranslationZ = LimitEnabled[2].Apply<float>(ValueRef.TranslationZ, Min.TranslationZ, Max.TranslationZ);
				}
				if (LimitEnabled[3].IsOn() || LimitEnabled[4].IsOn() || LimitEnabled[5].IsOn())
				{
					ValueRef.RotationPitch = LimitEnabled[3].Apply<float>(ValueRef.RotationPitch, Min.RotationPitch, Max.RotationPitch);
					ValueRef.RotationYaw = LimitEnabled[4].Apply<float>(ValueRef.RotationYaw, Min.RotationYaw, Max.RotationYaw);
					ValueRef.RotationRoll = LimitEnabled[5].Apply<float>(ValueRef.RotationRoll, Min.RotationRoll, Max.RotationRoll);
				}
				if (LimitEnabled[6].IsOn() || LimitEnabled[7].IsOn() || LimitEnabled[8].IsOn())
				{
					ValueRef.ScaleX = LimitEnabled[6].Apply<float>(ValueRef.ScaleX, Min.ScaleX, Max.ScaleX);
					ValueRef.ScaleY = LimitEnabled[7].Apply<float>(ValueRef.ScaleY, Min.ScaleY, Max.ScaleY);
					ValueRef.ScaleZ = LimitEnabled[8].Apply<float>(ValueRef.ScaleZ, Min.ScaleZ, Max.ScaleZ);
				}
				break;
			}
			case ERigControlType::Bool:
			default:
			{
				break;
			}
		}
	}

	struct FTransform_Float
	{
		float RotationX;
		float RotationY;
		float RotationZ;
		float RotationW;
		float TranslationX;
		float TranslationY;
		float TranslationZ;
#if ENABLE_VECTORIZED_TRANSFORM
		float TranslationW;
#endif
		float ScaleX;
		float ScaleY;
		float ScaleZ;
#if ENABLE_VECTORIZED_TRANSFORM
		float ScaleW;
#endif

		FTransform_Float()
		{
			*this = FTransform_Float(FTransform::Identity);
		}

		FTransform_Float(const FTransform& InTransform)
		{
			RotationX = InTransform.GetRotation().X;
			RotationY = InTransform.GetRotation().Y;
			RotationZ = InTransform.GetRotation().Z;
			RotationW = InTransform.GetRotation().W;
			TranslationX = InTransform.GetTranslation().X;
			TranslationY = InTransform.GetTranslation().Y;
			TranslationZ = InTransform.GetTranslation().Z;
			ScaleX = InTransform.GetScale3D().X;
			ScaleY = InTransform.GetScale3D().Y;
			ScaleZ = InTransform.GetScale3D().Z;
#if ENABLE_VECTORIZED_TRANSFORM
			TranslationW = ScaleW = 0.f;
#endif
		}

		FTransform ToTransform() const
		{
			FTransform Transform;
			Transform.SetRotation(FQuat(RotationX, RotationY, RotationZ, RotationW));
			Transform.SetTranslation(FVector(TranslationX, TranslationY, TranslationZ));
			Transform.SetScale3D(FVector(ScaleX, ScaleY, ScaleZ));
			return Transform;
		}

		FVector3f GetTranslation() const { return FVector3f(TranslationX, TranslationY, TranslationZ); }
		FQuat GetRotation() const { return FQuat(RotationX, RotationY, RotationZ, RotationW); }
		FVector3f GetScale3D() const { return FVector3f(ScaleX, ScaleY, ScaleZ); }
	};

	struct FTransformNoScale_Float
	{
		float RotationX;
		float RotationY;
		float RotationZ;
		float RotationW;
		float TranslationX;
		float TranslationY;
		float TranslationZ;
#if ENABLE_VECTORIZED_TRANSFORM
		float TranslationW;
#endif

		FTransformNoScale_Float()
		{
			*this = FTransformNoScale_Float(FTransformNoScale::Identity);
		}

		FTransformNoScale_Float(const FTransformNoScale& InTransform)
		{
			RotationX = InTransform.Rotation.X;
			RotationY = InTransform.Rotation.Y;
			RotationZ = InTransform.Rotation.Z;
			RotationW = InTransform.Rotation.W;
			TranslationX = InTransform.Location.X;
			TranslationY = InTransform.Location.Y;
			TranslationZ = InTransform.Location.Z;
#if ENABLE_VECTORIZED_TRANSFORM
			TranslationW = 0.f;
#endif
		}

		FTransformNoScale ToTransform() const
		{
			FTransformNoScale Transform;
			Transform.Rotation = FQuat(RotationX, RotationY, RotationZ, RotationW);
			Transform.Location = FVector(TranslationX, TranslationY, TranslationZ);
			return Transform;
		}

		FVector3f GetTranslation() const { return FVector3f(TranslationX, TranslationY, TranslationZ); }
		FQuat GetRotation() const { return FQuat(RotationX, RotationY, RotationZ, RotationW); }
	};

	struct FEulerTransform_Float
	{
		float RotationPitch;
		float RotationYaw;
		float RotationRoll;
		float TranslationX;
		float TranslationY;
		float TranslationZ;
		float ScaleX;
		float ScaleY;
		float ScaleZ;

		FEulerTransform_Float()
		{
			*this = FEulerTransform_Float(FEulerTransform::Identity);
		}

		FEulerTransform_Float(const FEulerTransform& InTransform)
		{
			RotationPitch = InTransform.Rotation.Pitch;
			RotationYaw = InTransform.Rotation.Yaw;
			RotationRoll = InTransform.Rotation.Roll;
			TranslationX = InTransform.Location.X;
			TranslationY = InTransform.Location.Y;
			TranslationZ = InTransform.Location.Z;
			ScaleX = InTransform.Scale.X;
			ScaleY = InTransform.Scale.Y;
			ScaleZ = InTransform.Scale.Z;
		}

		FEulerTransform ToTransform() const
		{
			FEulerTransform Transform;
			Transform.Rotation = FRotator(RotationPitch, RotationYaw, RotationRoll);
			Transform.Location = FVector(TranslationX, TranslationY, TranslationZ);
			Transform.Scale = FVector(ScaleX, ScaleY, ScaleZ);
			return Transform;
		}
		
		FVector3f GetTranslation() const { return FVector3f(TranslationX, TranslationY, TranslationZ); }
		FRotator GetRotator() const { return FRotator(RotationPitch, RotationYaw, RotationRoll); }
		FVector3f GetScale3D() const { return FVector3f(ScaleX, ScaleY, ScaleZ); }
	};

private:
	
	UPROPERTY()
	FRigControlValueStorage FloatStorage;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FTransform Storage_DEPRECATED;
#endif

	friend struct FRigControlHierarchy;
	friend class UControlRigBlueprint;
	friend class URigHierarchyController;
	friend class UControlRig;
	friend struct FRigControlSettings;
};

template<>
inline FRigControlValue FRigControlValue::Make(FVector2D InValue)
{
	return Make<FVector3f>(FVector3f(InValue.X, InValue.Y, 0.f));
}

template<>
inline FRigControlValue FRigControlValue::Make(FVector InValue)
{
	return Make<FVector3f>((FVector3f)InValue);
}

template<>
inline FRigControlValue FRigControlValue::Make(FRotator InValue)
{
	return Make<FVector3f>((FVector3f)InValue.Euler());
}

template<>
inline FRigControlValue FRigControlValue::Make(FTransform InValue)
{
	return Make<FTransform_Float>(InValue);
}

template<>
inline FRigControlValue FRigControlValue::Make(FTransformNoScale InValue)
{
	return Make<FTransformNoScale_Float>(InValue);
}

template<>
inline FRigControlValue FRigControlValue::Make(FEulerTransform InValue)
{
	return Make<FEulerTransform_Float>(InValue);
}

template<>
inline FString FRigControlValue::ToString<bool>() const
{
	const bool Value = Get<bool>();
	static const TCHAR* True = TEXT("True");
	static const TCHAR* False = TEXT("False");
	return Value ? True : False;
}

template<>
inline FString FRigControlValue::ToString<int32>() const
{
	const int32 Value = Get<int32>();
	return FString::FromInt(Value);
}

template<>
inline FString FRigControlValue::ToString<float>() const
{
	const float Value = Get<float>();
	return FString::SanitizeFloat(Value);
}

template<>
inline FString FRigControlValue::ToString<FVector>() const
{
	FVector3f Value = GetRef<FVector3f>();
	FString Result;
	TBaseStructure<FVector>::Get()->ExportText(Result, &Value, nullptr, nullptr, PPF_None, nullptr);
	return Result;
}

template<>
inline FString FRigControlValue::ToString<FVector2D>() const
{
	const FVector3f Value = GetRef<FVector3f>();
	FVector2D Value2D(Value.X, Value.Y);
	FString Result;
	TBaseStructure<FVector2D>::Get()->ExportText(Result, &Value2D, nullptr, nullptr, PPF_None, nullptr);
	return Result;
}

template<>
inline FString FRigControlValue::ToString<FRotator>() const
{
	FRotator Value = FRotator::MakeFromEuler((FVector)GetRef<FVector3f>());
	FString Result;
	TBaseStructure<FRotator>::Get()->ExportText(Result, &Value, nullptr, nullptr, PPF_None, nullptr);
	return Result;
}

template<>
inline FString FRigControlValue::ToString<FTransform>() const
{
	FTransform Value = GetRef<FTransform_Float>().ToTransform();
	FString Result;
	TBaseStructure<FTransform>::Get()->ExportText(Result, &Value, nullptr, nullptr, PPF_None, nullptr);
	return Result;
}

template<>
inline FString FRigControlValue::ToString<FTransformNoScale>() const
{
	FTransformNoScale Value = GetRef<FTransformNoScale_Float>().ToTransform();
	FString Result;
	TBaseStructure<FTransformNoScale>::Get()->ExportText(Result, &Value, nullptr, nullptr, PPF_None, nullptr);
	return Result;
}

template<>
inline FString FRigControlValue::ToString<FEulerTransform>() const
{
	FEulerTransform Value = GetRef<FEulerTransform_Float>().ToTransform();
	FString Result;
	TBaseStructure<FEulerTransform>::Get()->ExportText(Result, &Value, nullptr, nullptr, PPF_None, nullptr);
	return Result;
}

template<>
inline FVector FRigControlValue::SetFromString<FVector>(const FString& InString)
{
	FVector Value;
	TBaseStructure<FVector>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<FVector>::Get()->GetName());
	Set<FVector3f>((FVector3f)Value);
	return Value;
}

template<>
inline FQuat FRigControlValue::SetFromString<FQuat>(const FString& InString)
{
	FQuat Value;
	TBaseStructure<FQuat>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<FQuat>::Get()->GetName());
	Set<FVector3f>((FVector3f)Value.Rotator().Euler());
	return Value;
}

template<>
inline FRotator FRigControlValue::SetFromString<FRotator>(const FString& InString)
{
	FRotator Value;
	TBaseStructure<FRotator>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<FRotator>::Get()->GetName());
	Set<FVector3f>((FVector3f)Value.Euler());
	return Value;
}

template<>
inline FTransform FRigControlValue::SetFromString<FTransform>(const FString& InString)
{
	FTransform Value;
	TBaseStructure<FTransform>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<FTransform>::Get()->GetName());
	Set<FTransform_Float>(Value);
	return Value;
}

template<>
inline FTransformNoScale FRigControlValue::SetFromString<FTransformNoScale>(const FString& InString)
{
	FTransformNoScale Value;
	TBaseStructure<FTransformNoScale>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<FTransformNoScale>::Get()->GetName());
	Set<FTransformNoScale_Float>(Value);
	return Value;
}

template<>
inline FEulerTransform FRigControlValue::SetFromString<FEulerTransform>(const FString& InString)
{
	FEulerTransform Value;
	TBaseStructure<FEulerTransform>::Get()->ImportText(*InString, &Value, nullptr, PPF_None, nullptr, TBaseStructure<FEulerTransform>::Get()->GetName());
	Set<FEulerTransform_Float>(Value);
	return Value;
}

#define UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(T) \
template<> \
const T FRigControlValue::Get() const = delete;  \
template<>  \
const T& FRigControlValue::GetRef() const = delete; \
template<> \
T& FRigControlValue::GetRef() = delete; \
template<> \
void FRigControlValue::Set(T InValue) = delete;

// We disable all of these to force Control Rig
// to store values as floats. We've added our own
// float variations purely for storage purposes,
// from which we'll cast back and forth to FTransform etc.
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FVector2D)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FVector)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FVector4)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FRotator)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FQuat)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FTransform)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FTransformNoScale)
UE_CONTROLRIG_VALUE_DELETE_TEMPLATE(FEulerTransform)

enum class EControlRigContextChannelToKey : uint32
{
	None = 0x000,

	TranslationX = 0x001,
	TranslationY = 0x002,
	TranslationZ = 0x004,
	Translation = TranslationX | TranslationY | TranslationZ,

	RotationX = 0x008,
	RotationY = 0x010,
	RotationZ = 0x020,
	Rotation = RotationX | RotationY | RotationZ,

	ScaleX = 0x040,
	ScaleY = 0x080,
	ScaleZ = 0x100,
	Scale = ScaleX | ScaleY | ScaleZ,

	AllTransform = Translation | Rotation | Scale,

};
ENUM_CLASS_FLAGS(EControlRigContextChannelToKey)

USTRUCT(BlueprintType)
struct FRigControlModifiedContext
{
	GENERATED_BODY()

	FRigControlModifiedContext()
	: SetKey(EControlRigSetKey::DoNotCare)
	, KeyMask((uint32)EControlRigContextChannelToKey::AllTransform)
	, LocalTime(FLT_MAX)
	, EventName(NAME_None)
	{}

	FRigControlModifiedContext(EControlRigSetKey InSetKey)
		: SetKey(InSetKey)
		, KeyMask((uint32)EControlRigContextChannelToKey::AllTransform)
		, LocalTime(FLT_MAX)
		, EventName(NAME_None)
	{}

	FRigControlModifiedContext(EControlRigSetKey InSetKey, float InLocalTime, const FName& InEventName = NAME_None, EControlRigContextChannelToKey InKeyMask = EControlRigContextChannelToKey::AllTransform)
		: SetKey(InSetKey)
		, KeyMask((uint32)InKeyMask)
		, LocalTime(InLocalTime)
		, EventName(InEventName)
	{}
	
	EControlRigSetKey SetKey;
	uint32 KeyMask;
	float LocalTime;
	FName EventName;
	bool bConstraintUpdate = false;
};

USTRUCT(BlueprintType)
struct FRigHierarchyModulePath
{
	GENERATED_BODY();

public:
	
	FRigHierarchyModulePath()
		: ModulePath()
	{}

	FRigHierarchyModulePath(const FString& InModulePath)
		: ModulePath(InModulePath)
	{}

	explicit FRigHierarchyModulePath(const FName& InModulePathFName)
	: ModulePath(InModulePathFName.IsNone() ? FString() : InModulePathFName.ToString())
	{}

	FRigHierarchyModulePath(const TCHAR* InModulePath)
		: ModulePath(InModulePath)
	{}

	template<typename StringTypeA = FString, typename StringTypeB = FString>
	FRigHierarchyModulePath(const StringTypeA& InModuleName, const StringTypeB& InElementName)
		: ModulePath()
	{
		*this = Join(InModuleName, InElementName);
	}

	bool IsEmpty() const
	{
		return ModulePath.IsEmpty();
	}
	
	UE_API bool IsValid() const;
	UE_API bool UsesNameSpaceFormat() const;
	UE_API bool UsesModuleNameFormat() const;

	operator const FString&() const
	{
		return ModulePath;
	}

	const FString& GetPath() const
	{
		return ModulePath;
	}

	UE_API FName GetPathFName() const;

	UE_API FStringView GetModuleName() const;
	UE_API const FString& GetModuleNameString() const;
	UE_API const FName& GetModuleFName() const;
	UE_API FStringView GetModulePrefix() const;
	UE_API FString GetModulePrefixString() const;
	UE_API FStringView GetElementName() const;
	UE_API const FString& GetElementNameString() const;
	UE_API const FName& GetElementFName() const;

	template<typename StringType = FString>
	bool HasModuleName(const StringType& InModuleNameString) const
	{
		return GetModuleName().Equals(InModuleNameString, ESearchCase::IgnoreCase);
	}

	template<typename StringType = FString>
	bool HasElementName(const StringType& InElementNameString) const
	{
		return GetElementName().Equals(InElementNameString, ESearchCase::IgnoreCase);
	}

	template<typename StringType = FString>
	bool SetModuleName(const StringType& InModuleNameString)
	{
		if(!HasModuleName(InModuleNameString))
		{
			*this = Join(InModuleNameString, GetElementNameString());
			return true;
		}
		return false;
	}

	template<typename StringType = FString>
	bool SetElementName(const StringType& InElementNameString)
	{
		if(!HasElementName(InElementNameString))
		{
			*this = Join(GetModuleNameString(), InElementNameString);
			return true;
		}
		return false;
	}

	template<typename StringType = FString>
	bool ReplaceModuleNameInline(const StringType& InOldModuleName, const StringType& InNewModuleName)
	{
		if(HasModuleName(InOldModuleName))
		{
			*this = Join(InNewModuleName, GetElementNameString());
			return true;
		}
		return false;
	}

	template<typename StringType = FString>
	FRigHierarchyModulePath ReplaceModuleName(const StringType& InOldModuleName, const StringType& InNewModuleName) const
	{
		FRigHierarchyModulePath Result = *this;
		Result.ReplaceModuleNameInline<StringType>(InOldModuleName, InNewModuleName);
		return Result;
	}

	template<typename StringType = FString>
	FRigHierarchyModulePath ReplaceModuleName(const StringType& InNewModuleName) const
	{
		FRigHierarchyModulePath Result = *this;
		Result.SetModuleName(InNewModuleName);
		return Result;
	}

	template<typename StringType = FString>
	bool ReplaceElementNameInline(const StringType& InOldElementName, const StringType& InNewElementName)
	{
		if(HasElementName(InOldElementName))
		{
			*this = Join(GetModuleNameString(), InNewElementName);
			return true;
		}
		return false;
	}

	template<typename StringType = FString>
	FRigHierarchyModulePath ReplaceElementName(const StringType& InOldElementName, const StringType& InNewElementName) const
	{
		FRigHierarchyModulePath Result = *this;
		Result.ReplaceElementNameInline(InOldElementName, InNewElementName);
		return Result;
	}

	template<typename StringType = FString>
	FRigHierarchyModulePath ReplaceElementName(const StringType& InNewElementName) const
	{
		FRigHierarchyModulePath Result = *this;
		Result.SetElementName(InNewElementName);
		return Result;
	}

	friend uint32 GetTypeHash(const FRigHierarchyModulePath& InModulePath)
	{
		return GetTypeHash(InModulePath.ModulePath.ToLower());
	}

	bool operator==(const FRigHierarchyModulePath& InOther) const
	{
		return ModulePath.ToLower() == InOther.ModulePath.ToLower();
	}

	bool operator==(const FString& InOther) const
	{
		return ModulePath.ToLower() == InOther.ToLower();
	}

	UE_API bool Split(FStringView* OutModuleName, FStringView* OutElementName) const;
	UE_API bool Split(FString* OutModuleName, FString* OutElementName) const;

	UE_API FRigHierarchyModulePath ConvertToModuleNameFormat(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr) const;
	UE_API bool ConvertToModuleNameFormatInline(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr);

	static inline const TCHAR* NamespaceSeparator_Deprecated = TEXT(":");
	static inline constexpr TCHAR NamespaceSeparatorChar_Deprecated = TEXT(':');
	static inline const TCHAR* ModuleNameSuffix = TEXT("/");
	static inline constexpr TCHAR ModuleNameSuffixChar = TEXT('/');

private:

	static UE_API FRigHierarchyModulePath Join(const FString& InModuleName, const FString& InElementName);
	static UE_API FRigHierarchyModulePath Join(const FName& InModuleFName, const FName& InElementFName);

	UPROPERTY()
	FString ModulePath;

	mutable TOptional<FString> CachedModuleNameString;
	mutable TOptional<FString> CachedElementNameString;
	mutable TOptional<FName> CachedModuleFName;
	mutable TOptional<FName> CachedElementFName;
};

template<>
inline bool FRigHierarchyModulePath::HasModuleName<FName>(const FName& InModuleFName) const
{
	if(InModuleFName.IsNone())
	{
		return false;
	}
	return GetModuleFName() == InModuleFName;
}

template<>
inline bool FRigHierarchyModulePath::HasElementName<FName>(const FName& InElementFName) const
{
	if(InElementFName.IsNone())
	{
		return false;
	}
	return GetElementFName() == InElementFName;
}

template<>
inline bool FRigHierarchyModulePath::SetModuleName<FName>(const FName& InModuleFName)
{
	if(!HasModuleName(InModuleFName))
	{
		*this = Join(InModuleFName, GetElementFName());
		return true;
	}
	return false;
}

template<>
inline bool FRigHierarchyModulePath::SetElementName<FName>(const FName& InElementFName)
{
	if(!HasElementName(InElementFName))
	{
		*this = Join(GetModuleFName(), InElementFName);
		return true;
	}
	return false;
}

template<>
inline bool FRigHierarchyModulePath::ReplaceModuleNameInline<FName>(const FName& InOldModuleFName, const FName& InNewModuleFName)
{
	if(HasModuleName(InOldModuleFName))
	{
		*this = Join(InNewModuleFName, GetElementFName());
		return true;
	}
	return false;
}

template<>
inline bool FRigHierarchyModulePath::ReplaceElementNameInline<FName>(const FName& InOldElementFName, const FName& InNewElementFName)
{
	if(HasElementName(InOldElementFName))
	{
		*this = Join(GetModuleFName(), InNewElementFName);
		return true;
	}
	return false;
}

/*
 * Because it's bitfield, we support some basic functionality
 */
namespace FRigElementTypeHelper
{
	static uint32 Add(uint32 InMasks, ERigElementType InType)
	{
		return InMasks & (uint32)InType;
	}

	static uint32 Remove(uint32 InMasks, ERigElementType InType)
	{
		return InMasks & ~((uint32)InType);
	}

	static uint32 ToMask(ERigElementType InType)
	{
		return (uint32)InType;
	}

	static bool DoesHave(uint32 InMasks, ERigElementType InType)
	{
		return (InMasks & (uint32)InType) != 0;
	}
}

USTRUCT(BlueprintType)
struct FRigElementKey
{
public:
	
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Hierarchy")
	ERigElementType Type;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Hierarchy", meta = (CustomWidget = "ElementName"))
	FName Name;

	FRigElementKey(const ERigElementType InType = ERigElementType::None)
		: Type(InType)
		, Name(NAME_None)
	{}

	FRigElementKey(const FName& InName, const ERigElementType InType)
		: Type(InType)
		, Name(InName)
	{}

	explicit FRigElementKey(const FString& InKeyString)
	{
		FString TypeStr, NameStr;
		verify(InKeyString.Split(TEXT("("), &TypeStr, &NameStr));
		NameStr.RemoveFromEnd(TEXT(")"));
		Name = *NameStr;

		const UEnum* ElementTypeEnum = StaticEnum<ERigElementType>();
		Type = (ERigElementType)ElementTypeEnum->GetValueByName(*TypeStr);
		check(Type > ERigElementType::None && Type < ERigElementType::All);
	}

	UE_API void Serialize(FArchive& Ar);
	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigElementKey& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	bool IsValid() const
	{
		return Name.IsValid() && Name != NAME_None && Type != ERigElementType::None;
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	void Reset()
	{
		Type = ERigElementType::Curve;
		Name = NAME_None;
	}

	bool IsTypeOf(ERigElementType InElementType) const
	{
		return ((uint8)InElementType & (uint8)Type) == (uint8)Type;
	}

	friend uint32 GetTypeHash(const FRigElementKey& Key)
	{
		return GetTypeHash(Key.Name) * 10 + (uint32)Key.Type;
	}

	friend uint32 GetTypeHash(const TArrayView<const FRigElementKey>& Keys)
	{
		uint32 Hash = (uint32)(Keys.Num() * 17 + 3);
		for (const FRigElementKey& Key : Keys)
		{
			Hash += GetTypeHash(Key);
		}
		return Hash;
	}

	friend uint32 GetTypeHash(const TArray<FRigElementKey>& Keys)
	{
		return GetTypeHash(TArrayView<const FRigElementKey>(Keys.GetData(), Keys.Num()));
	}

	bool operator ==(const FRigElementKey& Other) const
	{
		return Name == Other.Name && Type == Other.Type;
	}

	bool operator !=(const FRigElementKey& Other) const
	{
		return Name != Other.Name || Type != Other.Type;
	}

	bool operator <(const FRigElementKey& Other) const
	{
		if (Type < Other.Type)
		{
			return true;
		}
		return Name.LexicalLess(Other.Name);
	}

	bool operator >(const FRigElementKey& Other) const
	{
		if (Type > Other.Type)
		{
			return true;
		}
		return Other.Name.LexicalLess(Name);
	}

	FString ToString() const
	{
		switch (Type)
		{
			case ERigElementType::Bone:
			{
				return FString::Printf(TEXT("Bone(%s)"), *Name.ToString());
			}
			case ERigElementType::Null:
			{
				return FString::Printf(TEXT("Null(%s)"), *Name.ToString());
			}
			case ERigElementType::Control:
			{
				return FString::Printf(TEXT("Control(%s)"), *Name.ToString());
			}
			case ERigElementType::Curve:
			{
				return FString::Printf(TEXT("Curve(%s)"), *Name.ToString());
			}
			case ERigElementType::Reference:
			{
				return FString::Printf(TEXT("Reference(%s)"), *Name.ToString());
			}
			case ERigElementType::Connector:
			{
				return FString::Printf(TEXT("Connector(%s)"), *Name.ToString());
			}
			case ERigElementType::Socket:
			{
				return FString::Printf(TEXT("Socket(%s)"), *Name.ToString());
			}
		}
		
		return FString();
	}

	UE_API FString ToPythonString() const;

	UE_API FRigElementKey ConvertToModuleNameFormat(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr) const;
	UE_API bool ConvertToModuleNameFormatInline(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr);

};

USTRUCT(BlueprintType)
struct FRigComponentKey
{
public:
	
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Hierarchy", meta = (DisplayName="Item"))
	FRigElementKey ElementKey;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Hierarchy", meta = (CustomWidget = "ComponentName"))
	FName Name;

	FRigComponentKey()
		: ElementKey()
		, Name(NAME_None)
	{
	}

	FRigComponentKey(const FRigElementKey& InElementKey, const FName& InName)
		: ElementKey(InElementKey)
		, Name(InName)
	{}

	FRigComponentKey(const FString& InKeyString)
	{
		FString KeyString = InKeyString;
		KeyString.RemoveFromStart(TEXT("Component("));
		KeyString.RemoveFromEnd(TEXT(")"));
		FString NameStr, ElementKeyStr;
		verify(KeyString.Split(TEXT(","), &NameStr, &ElementKeyStr));
		Name = *NameStr;
		ElementKey = FRigElementKey(ElementKeyStr);
	}
	
	UE_API void Serialize(FArchive& Ar);
	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigComponentKey& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	UE_API bool IsValid() const;

	explicit operator bool() const
	{
		return IsValid();
	}

	void Reset()
	{
		ElementKey.Reset();
		Name = NAME_None;
	}

	friend uint32 GetTypeHash(const FRigComponentKey& Key)
	{
		return GetTypeHash(Key.Name) * 15 + GetTypeHash(Key.ElementKey);
	}

	friend uint32 GetTypeHash(const TArrayView<const FRigComponentKey>& Keys)
	{
		uint32 Hash = (uint32)(Keys.Num() * 17 + 3);
		for (const FRigComponentKey& Key : Keys)
		{
			Hash += GetTypeHash(Key);
		}
		return Hash;
	}

	friend uint32 GetTypeHash(const TArray<FRigComponentKey>& Keys)
	{
		return GetTypeHash(TArrayView<const FRigComponentKey>(Keys.GetData(), Keys.Num()));
	}

	bool operator ==(const FRigComponentKey& Other) const
	{
		return Name == Other.Name && ElementKey == Other.ElementKey;
	}

	bool operator !=(const FRigComponentKey& Other) const
	{
		return Name != Other.Name || ElementKey != Other.ElementKey;
	}

	bool operator <(const FRigComponentKey& Other) const
	{
		if (ElementKey < Other.ElementKey)
		{
			return true;
		}
		return Name.LexicalLess(Other.Name);
	}

	bool operator >(const FRigComponentKey& Other) const
	{
		if (ElementKey > Other.ElementKey)
		{
			return true;
		}
		return Other.Name.LexicalLess(Name);
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("Component(%s,%s)"), *Name.ToString(), *ElementKey.ToString());
	}

	UE_API FString ToPythonString() const;

	UE_API bool IsTopLevel() const;

	UE_API FRigComponentKey ConvertToModuleNameFormat(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr) const;
	UE_API bool ConvertToModuleNameFormatInline(const TMap<FRigHierarchyModulePath, FName>* InModulePathToModuleName = nullptr);
};

USTRUCT(BlueprintType)
struct FRigHierarchyKey
{
	GENERATED_BODY()

public:
	FRigHierarchyKey()
	{
	}

	FRigHierarchyKey(const FRigElementKey& InElementKey, bool bForce = false)
	{
		if(InElementKey.IsValid() || bForce)
		{
			Element = InElementKey;
		}
	}
	
	FRigHierarchyKey(const FRigComponentKey& InComponentKey, bool bForce = false)
	{
		if(InComponentKey.IsValid() || bForce)
		{
			Component = InComponentKey;
		}
	}

	bool IsElement() const { return Element.IsSet(); }
	bool IsComponent() const { return Component.IsSet(); }
	bool IsValid() const
	{
		return IsElement() || IsComponent();
	}

	UE_API const FName& GetFName() const;

	FString GetName() const
	{
		return GetFName().ToString();
	}

	void SetName(const FName& InName)
	{
		if(IsElement())
		{
			Element.GetValue().Name = InName;
		}
		if(IsComponent())
		{
			Component.GetValue().Name = InName;
		}
	}

	const FRigElementKey& GetElement() const
	{
		if(IsElement())
		{
			return Element.GetValue();
		}
		if(IsComponent())
		{
			return Component.GetValue().ElementKey;
		}
		static const FRigElementKey InvalidKey;
		return InvalidKey;
	}

	const FRigComponentKey& GetComponent() const
	{
		if(IsComponent())
		{
			return Component.GetValue();
		}
		static const FRigComponentKey InvalidComponent;
		return InvalidComponent;
	}

	bool operator ==(const FRigHierarchyKey& InOther) const
	{
		if(IsElement() != InOther.IsElement() ||
			IsComponent() != InOther.IsComponent())
		{
			return false;
		}
		if(IsElement())
		{
			return Element == InOther.Element;
		}
		if(IsComponent())
		{
			return Component == InOther.Component;
		}
		return true;
	}

	bool operator ==(const FRigElementKey& InOther) const
	{
		if(IsElement())
		{
			return Element == InOther;
		}
		return false;
	}

	bool operator ==(const FRigComponentKey& InOther) const
	{
		if(IsComponent())
		{
			return Component == InOther;
		}
		return false;
	}
	
	bool operator <(const FRigHierarchyKey& Other) const
	{
		if(IsElement() && Other.IsComponent())
		{
			return false;
		}
		if(IsComponent() && Other.IsElement())
		{
			return true;
		}
		if(IsComponent() && Other.IsComponent())
		{
			return GetComponent() < Other.GetComponent();
		}
		if(IsElement() && Other.IsElement())
		{
			return GetElement() < Other.GetElement();
		}
		return false;
	}

	bool operator >(const FRigHierarchyKey& Other) const
	{
		return !(*this < Other);
	}
	
	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigHierarchyKey& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	friend uint32 GetTypeHash(const FRigHierarchyKey& InKey)
	{
		if(InKey.IsElement())
		{
			return GetTypeHash(InKey.GetElement());
		}
		if(InKey.IsComponent())
		{
			return GetTypeHash(InKey.GetComponent());
		}
		return 0;
	}

private:
	
	TOptional<FRigElementKey> Element;
	TOptional<FRigComponentKey> Component;
};

struct FRigElementKeyAndIndex
{
public:

	inline static const FRigElementKey InvalidKey = FRigElementKey(NAME_None, ERigElementType::Bone);
	inline static constexpr int32 InvalidIndex = INDEX_NONE;

	FRigElementKeyAndIndex()
		: Key(InvalidKey)
		, Index(InvalidIndex)
	{
	}

	UE_API explicit FRigElementKeyAndIndex(const FRigBaseElement* InElement);

	FRigElementKeyAndIndex(const FRigElementKey& InKey, const int32& InIndex)
		: Key(InKey)
		, Index(InIndex)
	{
	}

	bool IsValid() const
	{
		return Key.IsValid() && Index != INDEX_NONE;
	}

	operator int32() const
	{
		return Index;
	}

	friend uint32 GetTypeHash(const FRigElementKeyAndIndex& InKeyAndIndex)
	{
		return GetTypeHash(InKeyAndIndex.Index);
	}
	
	const FRigElementKey& Key;
	const int32& Index;
};

USTRUCT(BlueprintType)
struct FRigElementKeyCollection
{
	GENERATED_BODY()

	FRigElementKeyCollection()
	{
	}

	FRigElementKeyCollection(const TArray<FRigElementKey>& InKeys)
		: Keys(InKeys)
	{
	}

	// Resets the data structure and maintains all storage.
	void Reset()
	{
		Keys.Reset();
	}

	// Resets the data structure and removes all storage.
	void Empty()
	{
		Keys.Empty();
	}

	// Returns true if a given instruction index is valid.
	bool IsValidIndex(int32 InIndex) const
	{
		return Keys.IsValidIndex(InIndex);
	}

	// Returns the number of elements in this collection.
	int32 Num() const { return Keys.Num(); }

	// Returns true if this collection contains no elements.
	bool IsEmpty() const
	{
		return Num() == 0;
	}

	// Returns the first element of this collection
	const FRigElementKey& First() const
	{
		return Keys[0];
	}

	// Returns the first element of this collection
	FRigElementKey& First()
	{
		return Keys[0];
	}

	// Returns the last element of this collection
	const FRigElementKey& Last() const
	{
		return Keys.Last();
	}

	// Returns the last element of this collection
	FRigElementKey& Last()
	{
		return Keys.Last();
	}

	int32 Add(const FRigElementKey& InKey)
	{
		return Keys.Add(InKey);
	}

	int32 AddUnique(const FRigElementKey& InKey)
	{
		return Keys.AddUnique(InKey);
	}

	bool Contains(const FRigElementKey& InKey) const
	{
		return Keys.Contains(InKey);
	}

	// const accessor for an element given its index
	const FRigElementKey& operator[](int32 InIndex) const
	{
		return Keys[InIndex];
	}

	const TArray<FRigElementKey>& GetKeys() const
	{
		return Keys;
	}
	   
	TArray<FRigElementKey>::RangedForIteratorType      begin() { return Keys.begin(); }
	TArray<FRigElementKey>::RangedForConstIteratorType begin() const { return Keys.begin(); }
	TArray<FRigElementKey>::RangedForIteratorType      end() { return Keys.end(); }
	TArray<FRigElementKey>::RangedForConstIteratorType end() const { return Keys.end(); }

	friend uint32 GetTypeHash(const FRigElementKeyCollection& Collection)
	{
		return GetTypeHash(Collection.GetKeys());
	}

	// creates a collection containing all of the children of a given 
	static UE_API FRigElementKeyCollection MakeFromChildren(
		URigHierarchy* InHierarchy, 
		const FRigElementKey& InParentKey,
		bool bRecursive = true,
		bool bIncludeParent = false,
		uint8 InElementTypes = (uint8)ERigElementType::All);

	// creates a collection containing all of the elements with a given name
	static UE_API FRigElementKeyCollection MakeFromName(
		URigHierarchy* InHierarchy,
		const FName& InPartialName,
		uint8 InElementTypes = (uint8)ERigElementType::All);

	// creates a collection containing an item chain
	static UE_API FRigElementKeyCollection MakeFromChain(
		URigHierarchy* InHierarchy,
		const FRigElementKey& InFirstItem,
		const FRigElementKey& InLastItem,
		bool bReverse = false);

	// creates a collection containing all keys of a hierarchy
	static UE_API FRigElementKeyCollection MakeFromCompleteHierarchy(
		URigHierarchy* InHierarchy,
		uint8 InElementTypes = (uint8)ERigElementType::All);

	// returns the union between two collections
	static UE_API FRigElementKeyCollection MakeUnion(const FRigElementKeyCollection& A, const FRigElementKeyCollection& B, bool bAllowDuplicates = false);

	// returns the intersection between two collections
	static UE_API FRigElementKeyCollection MakeIntersection(const FRigElementKeyCollection& A, const FRigElementKeyCollection& B);

	// returns the difference between two collections
	static UE_API FRigElementKeyCollection MakeDifference(const FRigElementKeyCollection& A, const FRigElementKeyCollection& B);

	// returns the collection in the reverse order
	static UE_API FRigElementKeyCollection MakeReversed(const FRigElementKeyCollection& InCollection);

	// filters a collection by element type
	UE_API FRigElementKeyCollection FilterByType(uint8 InElementTypes) const;

	// filters a collection by name
	UE_API FRigElementKeyCollection FilterByName(const FName& InPartialName) const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Collection)
	TArray<FRigElementKey> Keys;
};

USTRUCT(BlueprintType)
struct FRigElement
{
	GENERATED_BODY()

	FRigElement()
		: Name(NAME_None)
		, Index(INDEX_NONE)
	{}
	virtual ~FRigElement() = default;
	
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = FRigElement)
	FName Name;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = FRigElement)
	int32 Index;

	virtual ERigElementType GetElementType() const
	{
		return ERigElementType::None; 
	}

	FRigElementKey GetElementKey() const
	{
		return FRigElementKey(Name, GetElementType());
	}
};

USTRUCT(BlueprintType)
struct FRigEventContext
{
	GENERATED_BODY()

	FRigEventContext()
		: Event(ERigEvent::None)
		, SourceEventName(NAME_None)
		, Key()
		, LocalTime(0.f)
		, Payload(nullptr)
	{}
	
	ERigEvent Event;
	FName SourceEventName;
	FRigElementKey Key;
	float LocalTime;
	void* Payload;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FRigEventDelegate, URigHierarchy*, const FRigEventContext&);

UENUM()
enum class ERigElementResolveState : uint8
{
	Unknown,
	InvalidTarget,
	PossibleTarget,
	DefaultTarget,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

USTRUCT(BlueprintType)
struct FRigElementResolveResult
{
	GENERATED_BODY()

public:
	
	FRigElementResolveResult()
	: State(ERigElementResolveState::Unknown)
	{
	}

	FRigElementResolveResult(const FRigElementKey& InKey, ERigElementResolveState InState = ERigElementResolveState::PossibleTarget, const FText& InMessage = FText())
		: Key(InKey)
		, State(InState)
		, Message(InMessage)
	{
	}

	UE_API bool IsValid() const;
	const FRigElementKey& GetKey() const
	{
		return Key;
	}
	const ERigElementResolveState& GetState() const
	{
		return State;
	}
	const FText& GetMessage() const
	{
		return Message;
	}
	UE_API void SetInvalidTarget(const FText& InMessage);
	UE_API void SetPossibleTarget(const FText& InMessage = FText());
	UE_API void SetDefaultTarget(const FText& InMessage = FText());

private:
	
	UPROPERTY()
	FRigElementKey Key;

	UPROPERTY()
	ERigElementResolveState State;

	UPROPERTY()
	FText Message;

	friend class UModularRigRuleManager;
};

UENUM()
enum class EModularRigResolveState : uint8
{
	Success,
	Error,

	/** MAX - invalid */
	Max UMETA(Hidden),
};


USTRUCT(BlueprintType)
struct FModularRigResolveResult
{
	GENERATED_BODY()

public:
	
	FModularRigResolveResult()
		: State(EModularRigResolveState::Success)
	{
	}

	UE_API bool IsValid() const;

	const FRigElementKey& GetConnectorKey() const
	{
		return Connector;
	}

	EModularRigResolveState GetState() const
	{
		return State;
	}

	const FText& GetMessage() const
	{
		return Message;
	}

	const TArray<FRigElementResolveResult>& GetMatches() const
	{
		return Matches;
	}

	const TArray<FRigElementResolveResult>& GetExcluded() const
	{
		return Excluded;
	}

	UE_API bool ContainsMatch(const FRigElementKey& InKey, FString* OutErrorMessage = nullptr) const;
	UE_API const FRigElementResolveResult* FindMatch(const FRigElementKey& InKey) const;
	UE_API const FRigElementResolveResult* GetDefaultMatch() const;

private:

	UPROPERTY()
	FRigElementKey Connector;

	UPROPERTY()
	TArray<FRigElementResolveResult> Matches;

	UPROPERTY()
	TArray<FRigElementResolveResult> Excluded;

	UPROPERTY()
	EModularRigResolveState State;

	UPROPERTY()
	FText Message;

	friend class UModularRigRuleManager;
	friend class UModularRig;
	friend struct FRigUnit_GetCandidates;
	friend struct FRigUnit_DiscardMatches;
	friend struct FRigUnit_SetDefaultMatch;
	friend struct FRigUnit_SetDefaultMatchArray;
};

/* 
 * Defines how to retrieve the UI name for an element
 */
UENUM(BlueprintType)
enum class EElementNameDisplayMode : uint8
{
	// Relies on the setting in the referenced asset. The setting for each asset can be changed
	// in the class defaults - HierarchySettings of the Control Rig.
	// With this setting you can have different name display mode for each control rig in the sequencer.
	AssetDefault,
	
	// Shows full paths only for elements that need it
	Auto,

	// Always shows short names
	// (potentially resulting in clashing labels)
	ForceShort,

	// Always shows full paths
	ForceLong,
};

#undef UE_API
