// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigHierarchyComponents.h"
#include "RigHierarchyMetadata.h"
#include "RigConnectionRules.h"
#include "RigReusableElementStorage.h"
#include "RigHierarchyElements.generated.h"

#define UE_API CONTROLRIG_API

struct FRigVMExecuteContext;
struct FRigBaseElement;
struct FRigControlElement;
class URigHierarchy;
class FRigElementKeyRedirector;

DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FRigReferenceGetWorldTransformDelegate, const FRigVMExecuteContext*, const FRigElementKey& /* Key */, bool /* bInitial */);
DECLARE_DELEGATE_TwoParams(FRigElementMetadataChangedDelegate, const FRigElementKey& /* Key */, const FName& /* Name */);
DECLARE_DELEGATE_ThreeParams(FRigElementMetadataTagChangedDelegate, const FRigElementKey& /* Key */, const FName& /* Tag */, bool /* AddedOrRemoved */);
DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FRigReferenceGetWorldTransformDelegate, const FRigVMExecuteContext*, const FRigElementKey& /* Key */, bool /* bInitial */);

#define DECLARE_RIG_ELEMENT_METHODS(ElementType) \
virtual UScriptStruct* GetScriptStruct() const override { return ElementType::StaticStruct(); } \
template<typename T> \
friend const T* Cast(const ElementType* InElement) \
{ \
	return Cast<T>((const FRigBaseElement*) InElement); \
} \
template<typename T> \
friend T* Cast(ElementType* InElement) \
{ \
	return Cast<T>((FRigBaseElement*) InElement); \
} \
template<typename T> \
friend const T* CastChecked(const ElementType* InElement) \
{ \
	return CastChecked<T>((const FRigBaseElement*) InElement); \
} \
template<typename T> \
friend T* CastChecked(ElementType* InElement) \
{ \
	return CastChecked<T>((FRigBaseElement*) InElement); \
} \
static bool IsClassOf(const FRigBaseElement* InElement) \
{ \
	return InElement->GetScriptStruct()->IsChildOf(StaticStruct()); \
} \
virtual int32 GetElementTypeIndex() const override { return (int32)ElementType::ElementTypeIndex; }

UENUM()
namespace ERigTransformType
{
	enum Type : int
	{
		InitialLocal,
		CurrentLocal,
		InitialGlobal,
		CurrentGlobal,
		NumTransformTypes
	};
}

namespace ERigTransformType
{
	inline ERigTransformType::Type SwapCurrentAndInitial(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
			{
				return InitialLocal;
			}
			case CurrentGlobal:
			{
				return InitialGlobal;
			}
			case InitialLocal:
			{
				return CurrentLocal;
			}
			default:
			{
				break;
			}
		}
		return CurrentGlobal;
	}

	inline Type SwapLocalAndGlobal(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
			{
				return CurrentGlobal;
			}
			case CurrentGlobal:
			{
				return CurrentLocal;
			}
			case InitialLocal:
			{
				return InitialGlobal;
			}
			default:
			{
				break;
			}
		}
		return InitialLocal;
	}

	inline Type MakeLocal(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
			case CurrentGlobal:
			{
				return CurrentLocal;
			}
			default:
			{
				break;
			}
		}
		return InitialLocal;
	}

	inline Type MakeGlobal(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
			case CurrentGlobal:
			{
				return CurrentGlobal;
			}
			default:
			{
				break;
			}
		}
		return InitialGlobal;
	}

	inline Type MakeInitial(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
			case InitialLocal:
			{
				return InitialLocal;
			}
			default:
			{
				break;
			}
		}
		return InitialGlobal;
	}

	inline Type MakeCurrent(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
        	case InitialLocal:
			{
				return CurrentLocal;
			}
			default:
			{
				break;
			}
		}
		return CurrentGlobal;
	}

	inline bool IsLocal(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
        	case InitialLocal:
			{
				return true;
			}
			default:
			{
				break;
			}
		}
		return false;
	}

	inline bool IsGlobal(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentGlobal:
        	case InitialGlobal:
			{
				return true;;
			}
			default:
			{
				break;
			}
		}
		return false;
	}

	inline bool IsInitial(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case InitialLocal:
        	case InitialGlobal:
			{
				return true;
			}
			default:
			{
				break;
			}
		}
		return false;
	}

	inline bool IsCurrent(const Type InTransformType)
	{
		switch(InTransformType)
		{
			case CurrentLocal:
        	case CurrentGlobal:
			{
				return true;
			}
			default:
			{
				break;
			}
		}
		return false;
	}
}

UENUM()
namespace ERigTransformStorageType
{
	enum Type : int
	{
		Pose,
		Offset,
		Shape,
		NumStorageTypes
	};
}

class FRigCompactTransform
{
public:
	UE_API FRigCompactTransform(FTransform& InTransform);

	enum ERepresentation : uint8
	{
		Float_Zero_Identity_One = 0,
		Float_Zero_Identity_Uniform = 1,
		Float_Zero_Identity_NonUniform = 2,
		Float_Zero_Quat_One = 3,
		Float_Zero_Quat_Uniform = 4,
		Float_Zero_Quat_NonUniform = 5,
		Float_Position_Identity_One = 6,
		Float_Position_Identity_Uniform = 7,
		Float_Position_Identity_NonUniform = 8,
		Float_Position_Quat_One = 9,
		Float_Position_Quat_Uniform = 10,
		Float_Position_Quat_NonUniform = 11,
		Double_Complete = 12,
		Last = Double_Complete,
		Max = Last + 1 
	};

	UE_API void Serialize(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, ERepresentation* OutRepresentation = nullptr);
	UE_API void Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, ERepresentation* OutRepresentation = nullptr);
	UE_API void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, ERepresentation* OutRepresentation = nullptr);

private:

	FTransform& Transform;
};

USTRUCT(BlueprintType)
struct FRigTransformDirtyState
{
public:

	GENERATED_BODY()

	FRigTransformDirtyState()
	: StorageIndex(INDEX_NONE)
	, Storage(nullptr)
	{
	}

	UE_API const bool& Get() const;
	UE_API bool& Get();
	UE_API bool Set(bool InDirty);
	UE_API FRigTransformDirtyState& operator =(const FRigTransformDirtyState& InOther);

	int32 GetStorageIndex() const
	{
		return StorageIndex;
	}

private:

	UE_API void LinkStorage(const TArrayView<bool>& InStorage);
	UE_API void UnlinkStorage(FRigReusableElementStorage<bool>& InStorage);

	int32 StorageIndex;
	bool* Storage;

	friend struct FRigLocalAndGlobalDirtyState;
	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend class FRigHierarchyPoseAdapter;
	friend struct FRigReusableElementStorage<bool>;
	friend class FControlRigHierarchyRelinkElementStorage;
};

USTRUCT(BlueprintType)
struct FRigLocalAndGlobalDirtyState
{
	GENERATED_BODY()

public:
	
	FRigLocalAndGlobalDirtyState()
	{
	}

	UE_API FRigLocalAndGlobalDirtyState& operator =(const FRigLocalAndGlobalDirtyState& InOther);

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DirtyState")
	FRigTransformDirtyState Global;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DirtyState")
	FRigTransformDirtyState Local;

private:

	UE_API void LinkStorage(const TArrayView<bool>& InStorage);
	UE_API void UnlinkStorage(FRigReusableElementStorage<bool>& InStorage);

	friend struct FRigCurrentAndInitialDirtyState;
};

USTRUCT(BlueprintType)
struct FRigCurrentAndInitialDirtyState
{
	GENERATED_BODY()

public:
	
	FRigCurrentAndInitialDirtyState()
	{
	}

	bool& GetDirtyFlag(const ERigTransformType::Type InTransformType)
	{
		switch (InTransformType)
		{
			case ERigTransformType::CurrentLocal:
				return Current.Local.Get();

			case ERigTransformType::CurrentGlobal:
				return Current.Global.Get();

			case ERigTransformType::InitialLocal:
				return Initial.Local.Get();
				
			default:
				return Initial.Global.Get();
		}
	}

	const bool& GetDirtyFlag(const ERigTransformType::Type InTransformType) const
	{
		switch (InTransformType)
		{
			case ERigTransformType::CurrentLocal:
				return Current.Local.Get();

			case ERigTransformType::CurrentGlobal:
				return Current.Global.Get();

			case ERigTransformType::InitialLocal:
				return Initial.Local.Get();

			default:
				return Initial.Global.Get();
		}
	}

	bool IsDirty(const ERigTransformType::Type InTransformType) const
	{
		return GetDirtyFlag(InTransformType);
	}

	void MarkDirty(const ERigTransformType::Type InTransformType)
	{
		ensure(!(GetDirtyFlag(ERigTransformType::SwapLocalAndGlobal(InTransformType))));
		GetDirtyFlag(InTransformType) = true;
	}

	void MarkClean(const ERigTransformType::Type InTransformType)
	{
		GetDirtyFlag(InTransformType) = false;
	}

	UE_API FRigCurrentAndInitialDirtyState& operator =(const FRigCurrentAndInitialDirtyState& InOther);

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DirtyState")
	FRigLocalAndGlobalDirtyState Current;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DirtyState")
	FRigLocalAndGlobalDirtyState Initial;

private:

	UE_API void LinkStorage(const TArrayView<bool>& InStorage);
	UE_API void UnlinkStorage(FRigReusableElementStorage<bool>& InStorage);

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigTransformElement;
	friend struct FRigControlElement;
};

USTRUCT(BlueprintType)
struct FRigComputedTransform
{
	GENERATED_BODY()

public:
	
	FRigComputedTransform()
	: StorageIndex(INDEX_NONE)
	, Storage(nullptr)
	{}

	UE_API void Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, const FRigTransformDirtyState& InDirtyState);
	UE_API void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, FRigTransformDirtyState& InDirtyState);
	
	UE_API const FTransform& Get() const;

	void Set(const FTransform& InTransform)
	{
#if WITH_EDITOR
		ensure(InTransform.GetRotation().IsNormalized());
#endif
		// ensure(!FMath::IsNearlyZero(InTransform.GetScale3D().X));
		// ensure(!FMath::IsNearlyZero(InTransform.GetScale3D().Y));
		// ensure(!FMath::IsNearlyZero(InTransform.GetScale3D().Z));
		if(Storage)
		{
			*Storage = InTransform;
		}
	}

	static bool Equals(const FTransform& A, const FTransform& B, const float InTolerance = 0.0001f)
	{
		return (A.GetTranslation() - B.GetTranslation()).IsNearlyZero(InTolerance) &&
			A.GetRotation().Equals(B.GetRotation(), InTolerance) &&
			(A.GetScale3D() - B.GetScale3D()).IsNearlyZero(InTolerance);
	}

	bool operator == (const FRigComputedTransform& Other) const
	{
		return Equals(Get(), Other.Get());
	}

	UE_API FRigComputedTransform& operator =(const FRigComputedTransform& InOther);

	int32 GetStorageIndex() const
	{
		return StorageIndex;
	}

private:

	UE_API void LinkStorage(const TArrayView<FTransform>& InStorage);
	UE_API void UnlinkStorage(FRigReusableElementStorage<FTransform>& InStorage);
	
	int32 StorageIndex;
	FTransform* Storage;
	
	friend struct FRigLocalAndGlobalTransform;
	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend class FRigHierarchyPoseAdapter;
	friend struct FRigReusableElementStorage<FTransform>;
	friend class FControlRigHierarchyRelinkElementStorage;
};

USTRUCT(BlueprintType)
struct FRigLocalAndGlobalTransform
{
	GENERATED_BODY()

public:
	
	FRigLocalAndGlobalTransform()
    : Local()
    , Global()
	{}

	UE_API void Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, const FRigLocalAndGlobalDirtyState& InDirtyState);
	UE_API void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, FRigLocalAndGlobalDirtyState& OutDirtyState);

	UE_API FRigLocalAndGlobalTransform& operator =(const FRigLocalAndGlobalTransform& InOther);

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	FRigComputedTransform Local;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	FRigComputedTransform Global;

private:
	
	UE_API void LinkStorage(const TArrayView<FTransform>& InStorage);
	UE_API void UnlinkStorage(FRigReusableElementStorage<FTransform>& InStorage);

	friend struct FRigCurrentAndInitialTransform;
};

USTRUCT(BlueprintType)
struct FRigCurrentAndInitialTransform
{
	GENERATED_BODY()

public:
	
	FRigCurrentAndInitialTransform()
    : Current()
    , Initial()
	{}

	const FRigComputedTransform& operator[](const ERigTransformType::Type InTransformType) const
	{
		switch(InTransformType)
		{
			case ERigTransformType::CurrentLocal:
			{
				return Current.Local;
			}
			case ERigTransformType::CurrentGlobal:
			{
				return Current.Global;
			}
			case ERigTransformType::InitialLocal:
			{
				return Initial.Local;
			}
			default:
			{
				break;
			}
		}
		return Initial.Global;
	}

	FRigComputedTransform& operator[](const ERigTransformType::Type InTransformType)
	{
		switch(InTransformType)
		{
			case ERigTransformType::CurrentLocal:
			{
				return Current.Local;
			}
			case ERigTransformType::CurrentGlobal:
			{
				return Current.Global;
			}
			case ERigTransformType::InitialLocal:
			{
				return Initial.Local;
			}
			default:
			{
				break;
			}
		}
		return Initial.Global;
	}

	const FTransform& Get(const ERigTransformType::Type InTransformType) const
	{
		return operator[](InTransformType).Get();
	}

	void Set(const ERigTransformType::Type InTransformType, const FTransform& InTransform)
	{
		operator[](InTransformType).Set(InTransform);
	}

	UE_API void Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, const FRigCurrentAndInitialDirtyState& InDirtyState);
	UE_API void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings, FRigCurrentAndInitialDirtyState& OutDirtyState);

	bool operator == (const FRigCurrentAndInitialTransform& Other) const
	{
		return Current.Local  == Other.Current.Local
			&& Current.Global == Other.Current.Global
			&& Initial.Local  == Other.Initial.Local
			&& Initial.Global == Other.Initial.Global;
	}

	UE_API FRigCurrentAndInitialTransform& operator =(const FRigCurrentAndInitialTransform& InOther);

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	FRigLocalAndGlobalTransform Current;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	FRigLocalAndGlobalTransform Initial;
	
private:
	
	UE_API void LinkStorage(const TArrayView<FTransform>& InStorage);
	UE_API void UnlinkStorage(FRigReusableElementStorage<FTransform>& InStorage);

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigTransformElement;
	friend struct FRigControlElement;
};

USTRUCT(BlueprintType)
struct FRigPreferredEulerAngles
{
	GENERATED_BODY()

	static constexpr EEulerRotationOrder DefaultRotationOrder = EEulerRotationOrder::YZX;

	FRigPreferredEulerAngles()
	: RotationOrder(DefaultRotationOrder) // default for rotator
	, Current(FVector::ZeroVector)
	, Initial(FVector::ZeroVector)
	{}

	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);

	bool operator == (const FRigPreferredEulerAngles& Other) const
	{
		return RotationOrder == Other.RotationOrder &&
			Current == Other.Current &&
			Initial == Other.Initial;
	}

	UE_API void Reset();
	FVector& Get(bool bInitial = false) { return bInitial ? Initial : Current; }
	const FVector& Get(bool bInitial = false) const { return bInitial ? Initial : Current; }
	UE_API FRotator GetRotator(bool bInitial = false) const;
	UE_API FRotator SetRotator(const FRotator& InValue, bool bInitial = false, bool bFixEulerFlips = false);
	UE_API FVector GetAngles(bool bInitial = false, EEulerRotationOrder InRotationOrder = DefaultRotationOrder) const;
	UE_API void SetAngles(const FVector& InValue, bool bInitial = false, EEulerRotationOrder InRotationOrder = DefaultRotationOrder, bool bFixEulerFlips = false);
	UE_API void SetRotationOrder(EEulerRotationOrder InRotationOrder);


	UE_API FRotator GetRotatorFromQuat(const FQuat& InQuat) const;
	UE_API FQuat GetQuatFromRotator(const FRotator& InRotator) const;
	
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	EEulerRotationOrder RotationOrder;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	FVector Current;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Pose")
	FVector Initial;
};


struct FRigBaseElement;
//typedef TArray<FRigBaseElement*> FRigBaseElementChildrenArray;
typedef TArray<FRigBaseElement*, TInlineAllocator<3>> FRigBaseElementChildrenArray;
//typedef TArray<FRigBaseElement*> FRigBaseElementParentArray;
typedef TArray<FRigBaseElement*, TInlineAllocator<1>> FRigBaseElementParentArray;

struct FRigElementHandle
{
public:

	FRigElementHandle()
		: Hierarchy(nullptr)
		, Key()
	{}

	UE_API FRigElementHandle(URigHierarchy* InHierarchy, const FRigElementKey& InKey);
	UE_API FRigElementHandle(URigHierarchy* InHierarchy, const FRigBaseElement* InElement);

	bool IsValid() const { return Get() != nullptr; }
	operator bool() const { return IsValid(); }
	
	const URigHierarchy* GetHierarchy() const { return Hierarchy.Get(); }
	URigHierarchy* GetHierarchy() { return Hierarchy.Get(); }
	const FRigElementKey& GetKey() const { return Key; }

	UE_API const FRigBaseElement* Get() const;
	UE_API FRigBaseElement* Get();

	template<typename T>
	T* Get()
	{
		return Cast<T>(Get());
	}

	template<typename T>
	const T* Get() const
	{
		return Cast<T>(Get());
	}

	template<typename T>
	T* GetChecked()
	{
		return CastChecked<T>(Get());
	}

	template<typename T>
	const T* GetChecked() const
	{
		return CastChecked<T>(Get());
	}

private:

	TWeakObjectPtr<URigHierarchy> Hierarchy;
	FRigElementKey Key;
};

struct FRigComponentHandle
{
public:

	FRigComponentHandle()
		: Key()
	{
	}

	UE_API FRigComponentHandle(URigHierarchy* InHierarchy, const FRigComponentKey& InKey);
	UE_API FRigComponentHandle(URigHierarchy* InHierarchy, const FRigBaseComponent* InComponent);

	bool IsValid() const { return Get() != nullptr; }
	operator bool() const { return IsValid(); }
	
	const URigHierarchy* GetHierarchy() const { return Hierarchy.Get(); }
	URigHierarchy* GetHierarchy() { return Hierarchy.Get(); }
	const FRigElementKey& GetElementKey() const { return Key.ElementKey; }
	const FRigComponentKey& GetComponentKey() const { return Key; }
	const FName& GetComponentName() const { return Key.Name; }

	UE_API const FRigBaseComponent* Get() const;
	UE_API FRigBaseComponent* Get();

	template<typename T>
	T* Get()
	{
		return Cast<T>(Get());
	}

	template<typename T>
	const T* Get() const
	{
		return Cast<T>(Get());
	}

	template<typename T>
	T* GetChecked()
	{
		return CastChecked<T>(Get());
	}

	template<typename T>
	const T* GetChecked() const
	{
		return CastChecked<T>(Get());
	}

private:

	TWeakObjectPtr<URigHierarchy> Hierarchy;
	FRigComponentKey Key;
};

USTRUCT(BlueprintType)
struct FRigBaseElement
{
	GENERATED_BODY()

public:

	enum EElementIndex
	{
		BaseElement,
		TransformElement,
		SingleParentElement,
		MultiParentElement,
		BoneElement,
		NullElement,
		ControlElement,
		CurveElement,
		ReferenceElement,
		ConnectorElement,
		SocketElement,

		Max
	};

	static UE_API const EElementIndex ElementTypeIndex;

	FRigBaseElement() = default;
	UE_API virtual ~FRigBaseElement();

	FRigBaseElement(const FRigBaseElement& InOther)
	{
		*this = InOther;
	}
	
	FRigBaseElement& operator=(const FRigBaseElement& InOther)
	{
		// We purposefully do not copy any non-UPROPERTY entries, including Owner. This is so that when the copied
		// element is deleted, the metadata is not deleted with it. These copies are purely intended for interfacing
		// with BP and details view wrappers. 
		// These copies are solely intended for UControlRig::OnControlSelected_BP
		Key = InOther.Key;
		CachedNameString.Reset();
		Index = InOther.Index;
		SubIndex = InOther.SubIndex;
		CreatedAtInstructionIndex = InOther.CreatedAtInstructionIndex;
		bSelected = InOther.bSelected;
		return *this;
	}
	
	virtual int32 GetElementTypeIndex() const { return ElementTypeIndex; }
	static int32 GetElementTypeCount() { return EElementIndex::Max; }

protected:
	// Only derived types should be able to construct this one.
	explicit FRigBaseElement(URigHierarchy* InOwner, ERigElementType InElementType)
		: Owner(InOwner)
		, Key(InElementType)
	{
	}

	static bool IsClassOf(const FRigBaseElement* InElement)
	{
		return true;
	}

	// ReSharper disable once CppUE4ProbableMemoryIssuesWithUObject
	URigHierarchy* Owner = nullptr;
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = RigElement, meta = (AllowPrivateAccess = "true"))
	FRigElementKey Key;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = RigElement,
		meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "Index cannot be used directly. Storage indices are invalidated on hierarchy reconstruction. Use the Get Index node (URigHierarchy::GetIndex) with the element's Key instead."))
	int32 Index = INDEX_NONE;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = RigElement,
		meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "SubIndex cannot be used directly. Storage indices are invalidated on hierarchy reconstruction. Use the Get Local Index node (URigHierarchy::GetLocalIndex) with the element's Key instead."))
	int32 SubIndex = INDEX_NONE;

	UPROPERTY(BlueprintReadOnly, Transient, Category = RigElement, meta = (AllowPrivateAccess = "true"))
	int32 CreatedAtInstructionIndex = INDEX_NONE;

	UPROPERTY(BlueprintReadOnly, Transient, Category = RigElement, meta = (AllowPrivateAccess = "true"))
	int32 SpawnIndex = INDEX_NONE;

	UPROPERTY(BlueprintReadOnly, Transient, Category = RigElement, meta = (AllowPrivateAccess = "true", DeprecatedProperty, DeprecationMessage = "bSelected cannot be accessed directly. Use URigHierarchy::IsSelected instead."))
	bool bSelected = false;

	// used for constructing / destructing the memory. typically == 1
	// Set by URigHierarchy::NewElement.
	int32 OwnedInstances = 0;

	// Index into the child cache offset and count table in URigHierarchy. Set by URigHierarchy::UpdateCachedChildren
	int32 ChildCacheIndex = INDEX_NONE;

	// Index into the metadata storage for this element.
	int32 MetadataStorageIndex = INDEX_NONE;

	// The indices of the components on this element
	TArray<int32> ComponentIndices;
	
	mutable FString CachedNameString;

public:

	virtual UScriptStruct* GetScriptStruct() const { return FRigBaseElement::StaticStruct(); }
	UE_API void Serialize(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings);
	UE_API virtual void Save(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings);
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings);

	const FName& GetFName() const { return Key.Name; }
	const FString& GetName() const
	{
		if(CachedNameString.IsEmpty() && !Key.Name.IsNone())
		{
			CachedNameString = Key.Name.ToString();
		}
		return CachedNameString;
	}
	virtual const FName& GetDisplayName() const { return GetFName(); }
	ERigElementType GetType() const { return Key.Type; }
	const FRigElementKey& GetKey() const { return Key; }
	FRigElementKeyAndIndex GetKeyAndIndex() const { return {Key, Index}; };
	int32 GetIndex() const { return Index; }
	int32 GetSubIndex() const { return SubIndex; }
	bool IsSelected() const { return bSelected; }
	int32 GetCreatedAtInstructionIndex() const { return CreatedAtInstructionIndex; }
	int32 GetSpawnIndex() const { return SpawnIndex; }
	bool IsProcedural() const { return CreatedAtInstructionIndex != INDEX_NONE; }
	const URigHierarchy* GetOwner() const { return Owner; }
	URigHierarchy* GetOwner() { return Owner; }

	// Metadata
	UE_API FRigBaseMetadata* GetMetadata(const FName& InName, ERigMetadataType InType = ERigMetadataType::Invalid);
	UE_API const FRigBaseMetadata* GetMetadata(const FName& InName, ERigMetadataType InType) const;
	UE_API bool SetMetadata(const FName& InName, ERigMetadataType InType, const void* InData, int32 InSize);
	UE_API FRigBaseMetadata* SetupValidMetadata(const FName& InName, ERigMetadataType InType);
	UE_API bool RemoveMetadata(const FName& InName);
	UE_API bool RemoveAllMetadata();
	
	UE_API void NotifyMetadataTagChanged(const FName& InTag, bool bAdded);

	// Components
	UE_API int32 NumComponents() const;
	UE_API const FRigBaseComponent* GetComponent(int32 InIndex) const;
	UE_API FRigBaseComponent* GetComponent(int32 InIndex);
	UE_API const FRigBaseComponent* FindComponent(const FName& InName) const;
	UE_API FRigBaseComponent* FindComponent(const FName& InName);
	UE_API const FRigBaseComponent* GetFirstComponent(const UScriptStruct* InComponentStruct) const;
	UE_API FRigBaseComponent* GetFirstComponent(const UScriptStruct* InComponentStruct);
	UE_API TArray<FRigComponentKey> GetComponentKeys() const;

	template<typename T>
	const T* GetFirstComponent() const
	{
		return Cast<T>(GetFirstComponent(T::StaticStruct()));
	}

	template<typename T>
	T* GetFirstComponent()
	{
		return Cast<T>(GetFirstComponent(T::StaticStruct()));
	}

	virtual int32 GetNumTransforms() const { return 0; }
	virtual int32 GetNumCurves() const { return 0; }

	bool IsA(const UScriptStruct* InScriptStruct) const
	{
		return GetScriptStruct()->IsChildOf(InScriptStruct);
	}

	template<typename T>
	bool IsA() const { return T::IsClassOf(this); }

	bool IsTypeOf(ERigElementType InElementType) const
	{
		return Key.IsTypeOf(InElementType);
	}

	template<typename T>
    friend const T* Cast(const FRigBaseElement* InElement)
	{
		if(InElement)
		{
			if(InElement->IsA<T>())
			{
				return static_cast<const T*>(InElement);
			}
		}
		return nullptr;
	}

	template<typename T>
    friend T* Cast(FRigBaseElement* InElement)
	{
		if(InElement)
		{
			if(InElement->IsA<T>())
			{
				return static_cast<T*>(InElement);
			}
		}
		return nullptr;
	}

	template<typename T>
    friend const T* CastChecked(const FRigBaseElement* InElement)
	{
		const T* Element = Cast<T>(InElement);
		check(Element);
		return Element;
	}

	template<typename T>
    friend T* CastChecked(FRigBaseElement* InElement)
	{
		T* Element = Cast<T>(InElement);
		check(Element);
		return Element;
	}

	virtual void CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights) {}

protected:
	// Used to initialize this base element during URigHierarchy::CopyHierarchy. Once all elements are
	// initialized, the sub-class data copying is done using CopyFrom.
	UE_API void InitializeFrom(const FRigBaseElement* InOther);

	// helper function to be called as part of URigHierarchy::CopyHierarchy
	UE_API virtual void CopyFrom(const FRigBaseElement* InOther);

	virtual void LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates, const TArrayView<float>& InCurves) {}
	virtual void UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms, FRigReusableElementStorage<bool>& InDirtyStates, FRigReusableElementStorage<float>& InCurves) {}

	friend class FControlRigBaseEditor;
	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigElementKeyAndIndex;
	friend class UControlRigWrapperObject;
};

USTRUCT(BlueprintType)
struct FRigTransformElement : public FRigBaseElement
{
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigTransformElement)

	static UE_API const EElementIndex ElementTypeIndex;

	FRigTransformElement() = default;
	FRigTransformElement(const FRigTransformElement& InOther)
	{
		*this = InOther;
	}
	FRigTransformElement& operator=(const FRigTransformElement& InOther)
	{
		Super::operator=(InOther);
		GetTransform() = InOther.GetTransform();
		GetDirtyState() = InOther.GetDirtyState();
		return *this;
	}
	virtual ~FRigTransformElement() override = default;

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API virtual void CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights) override;

	// Current and Initial, both in Local and Global
	virtual int32 GetNumTransforms() const override { return 4; }

	UE_API const FRigCurrentAndInitialTransform& GetTransform() const;
	UE_API FRigCurrentAndInitialTransform& GetTransform();
	UE_API const FRigCurrentAndInitialDirtyState& GetDirtyState() const;
	UE_API FRigCurrentAndInitialDirtyState& GetDirtyState();

	UE_API virtual bool IsElementDirty(ERigTransformType::Type InTransformType) const;
	UE_API virtual void MarkElementDirty(ERigTransformType::Type InTransformType);

protected:
	
	FRigTransformElement(URigHierarchy* InOwner, const ERigElementType InType) :
		FRigBaseElement(InOwner, InType)
	{}

	// Pose storage for this element.
	FRigCurrentAndInitialTransform PoseStorage;

	// Dirty state storage for this element.
	FRigCurrentAndInitialDirtyState PoseDirtyState;
	
	struct FElementToDirty
	{
		FElementToDirty()
			: Element(nullptr)
			, HierarchyDistance(INDEX_NONE)
		{}

		FElementToDirty(FRigTransformElement* InElement, int32 InHierarchyDistance = INDEX_NONE)
			: Element(InElement)
			, HierarchyDistance(InHierarchyDistance)
		{}

		bool operator ==(const FElementToDirty& Other) const
		{
			return Element == Other.Element;
		}

		bool operator !=(const FElementToDirty& Other) const
		{
			return Element != Other.Element;
		}
		
		FRigTransformElement* Element;
		int32 HierarchyDistance;
	};

	//typedef TArray<FElementToDirty> FElementsToDirtyArray;
	typedef TArray<FElementToDirty, TInlineAllocator<3>> FElementsToDirtyArray;  
	FElementsToDirtyArray ElementsToDirty;

	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	UE_API virtual void LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates, const TArrayView<float>& InCurves) override;
	UE_API virtual void UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms, FRigReusableElementStorage<bool>& InDirtyStates, FRigReusableElementStorage<float>& InCurves) override;

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigBaseElement;
	friend class FControlRigBaseEditor;
};

USTRUCT(BlueprintType)
struct FRigSingleParentElement : public FRigTransformElement
{
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigSingleParentElement)

	static UE_API const EElementIndex ElementTypeIndex;

	FRigSingleParentElement() = default;
	virtual ~FRigSingleParentElement() override = default;

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	FRigTransformElement* ParentElement = nullptr;

protected:
	explicit FRigSingleParentElement(URigHierarchy* InOwner, ERigElementType InType)
		: FRigTransformElement(InOwner, InType)
	{}

	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	friend class URigHierarchy;
	friend struct FRigBaseElement;
};

USTRUCT(BlueprintType)
struct FRigElementWeight
{
public:
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Weight)
	float Location;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Weight)
	float Rotation;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Weight)
	float Scale;

	FRigElementWeight()
		: Location(1.f)
		, Rotation(1.f)
		, Scale(1.f)
	{}

	FRigElementWeight(float InWeight)
		: Location(InWeight)
		, Rotation(InWeight)
		, Scale(InWeight)
	{}

	FRigElementWeight(float InLocation, float InRotation, float InScale)
		: Location(InLocation)
		, Rotation(InRotation)
		, Scale(InScale)
	{}

	friend FArchive& operator <<(FArchive& Ar, FRigElementWeight& Weight)
	{
		Ar << Weight.Location;
		Ar << Weight.Rotation;
		Ar << Weight.Scale;
		return Ar;
	}

	bool AffectsLocation() const
	{
		return Location > SMALL_NUMBER;
	}

	bool AffectsRotation() const
	{
		return Rotation > SMALL_NUMBER;
	}

	bool AffectsScale() const
	{
		return Scale > SMALL_NUMBER;
	}

	bool IsAlmostZero() const
	{
		return !AffectsLocation() && !AffectsRotation() && !AffectsScale();
	}

	friend FRigElementWeight operator *(FRigElementWeight InWeight, float InScale)
	{
		return FRigElementWeight(InWeight.Location * InScale, InWeight.Rotation * InScale, InWeight.Scale * InScale);
	}

	friend FRigElementWeight operator *(float InScale, FRigElementWeight InWeight)
	{
		return FRigElementWeight(InWeight.Location * InScale, InWeight.Rotation * InScale, InWeight.Scale * InScale);
	}
};

USTRUCT()
struct FRigElementParentConstraint
{
	GENERATED_BODY()

	FRigTransformElement* ParentElement;
	FRigElementWeight Weight;
	FRigElementWeight InitialWeight;
	FName DisplayLabel;
	mutable FTransform Cache;
	mutable bool bCacheIsDirty;
		
	FRigElementParentConstraint()
		: ParentElement(nullptr)
		, DisplayLabel(NAME_None)
	{
		Cache = FTransform::Identity;
		bCacheIsDirty = true;
	}

	const FRigElementWeight& GetWeight(bool bInitial = false) const
	{
		return bInitial ? InitialWeight : Weight;
	}

	void CopyPose(const FRigElementParentConstraint& InOther, bool bCurrent, bool bInitial)
	{
		if(bCurrent)
		{
			Weight = InOther.Weight;
		}
		if(bInitial)
		{
			InitialWeight = InOther.InitialWeight;
		}
		bCacheIsDirty = true;
	}
};

#if URIGHIERARCHY_ENSURE_CACHE_VALIDITY
typedef TArray<FRigElementParentConstraint, TInlineAllocator<8>> FRigElementParentConstraintArray;
#else
typedef TArray<FRigElementParentConstraint, TInlineAllocator<1>> FRigElementParentConstraintArray;
#endif

USTRUCT(BlueprintType)
struct FRigMultiParentElement : public FRigTransformElement
{
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigMultiParentElement)

	static UE_API const EElementIndex ElementTypeIndex;

	FRigMultiParentElement() = default;	
	virtual ~FRigMultiParentElement() override = default;

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API virtual void CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights) override;
	
	UE_API virtual bool IsElementDirty(ERigTransformType::Type InTransformType) const override;
	UE_API virtual void MarkElementDirty(ERigTransformType::Type InTransformType)  override;

	FRigElementParentConstraintArray ParentConstraints;
	TMap<FRigElementKey, int32> IndexLookup;

protected:
	explicit FRigMultiParentElement(URigHierarchy* InOwner, const ERigElementType InType)
		: FRigTransformElement(InOwner, InType)
	{}

	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;
	
private:

	friend class URigHierarchy;
	friend struct FRigBaseElement;
};


USTRUCT(BlueprintType)
struct FRigBoneElement : public FRigSingleParentElement
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigBoneElement)

	static UE_API const EElementIndex ElementTypeIndex;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = RigElement)
	ERigBoneType BoneType = ERigBoneType::User;

	FRigBoneElement()
		: FRigBoneElement(nullptr)
	{}
	FRigBoneElement(const FRigBoneElement& InOther)
	{
		*this = InOther;
	}
	FRigBoneElement& operator=(const FRigBoneElement& InOther)
	{
		BoneType = InOther.BoneType;
		return *this;
	}
	
	virtual ~FRigBoneElement() override = default;

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

private:
	explicit FRigBoneElement(URigHierarchy* InOwner)
		: FRigSingleParentElement(InOwner, ERigElementType::Bone)
	{}

	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	friend class URigHierarchy;
	friend struct FRigBaseElement;
};

USTRUCT(BlueprintType)
struct FRigNullElement final : public FRigMultiParentElement
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigNullElement)

	static UE_API const EElementIndex ElementTypeIndex;

	FRigNullElement() 
		: FRigNullElement(nullptr)
	{}

	virtual ~FRigNullElement() override = default;

private:
	explicit FRigNullElement(URigHierarchy* InOwner)
		: FRigMultiParentElement(InOwner, ERigElementType::Null)
	{}

	friend class URigHierarchy;
	friend struct FRigBaseElement;
};

USTRUCT(BlueprintType)
struct FRigElementKeyWithLabel
{
	GENERATED_BODY()

	FRigElementKeyWithLabel()
		: Key()
		, Label(NAME_None)
	{
	}
	
	explicit FRigElementKeyWithLabel(const FRigElementKey& InKey, const FName& InLabel = NAME_None)
		: Key(InKey)
		, Label(InLabel)
	{
	}

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Customization, meta=(DisplayName="Item"))
	FRigElementKey Key;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Customization)
	FName Label;

	const FName& GetLabel() const
	{
		return Label.IsNone() ? Key.Name : Label;
	}

	friend uint32 GetTypeHash(const FRigElementKeyWithLabel& InKeyWithLabel)
	{
		return GetTypeHash(InKeyWithLabel.Key);
	}

	bool operator ==(const FRigElementKeyWithLabel& InOther) const
	{
		return InOther.Key == Key && InOther.Label == Label;
	}

	bool operator ==(const FRigElementKey& InOther) const
	{
		return InOther == Key;
	}

	friend FArchive& operator<<(FArchive& Ar, FRigElementKeyWithLabel& KeyWithLabel)
	{
		Ar << KeyWithLabel.Key;
		Ar << KeyWithLabel.Label;
		return Ar;
	}
};

USTRUCT(BlueprintType)
struct FRigControlElementCustomization
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Customization)
	TArray<FRigElementKeyWithLabel> AvailableSpaces;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Customization)
	TArray<FRigElementKey> RemovedSpaces;
};

UENUM(BlueprintType)
enum class ERigControlTransformChannel : uint8
{
	TranslationX,
	TranslationY,
	TranslationZ,
	Pitch,
	Yaw,
	Roll,
	ScaleX,
	ScaleY,
	ScaleZ
};

USTRUCT(BlueprintType)
struct FRigControlSettings
{
	GENERATED_BODY()

	UE_API FRigControlSettings();

	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);

	friend uint32 GetTypeHash(const FRigControlSettings& Settings);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	ERigControlAnimationType AnimationType;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control, meta=(DisplayName="Value Type"))
	ERigControlType ControlType;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	FName DisplayName;

	/** the primary axis to use for float controls */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	ERigControlAxis PrimaryAxis;

	/** If Created from a Curve  Container*/
	UPROPERTY(transient)
	bool bIsCurve;

	/** True if the control has limits. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	TArray<FRigControlLimitEnabled> LimitEnabled;

	/**
	 * True if the limits should be drawn in debug.
	 * For this to be enabled you need to have at least one min and max limit turned on.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	bool bDrawLimits;

	/** The minimum limit of the control's value */
	UPROPERTY(BlueprintReadWrite, Category = Limits)
	FRigControlValue MinimumValue;

	/** The maximum limit of the control's value */
	UPROPERTY(BlueprintReadWrite, Category = Limits)
	FRigControlValue MaximumValue;

	/** Set to true if the shape is currently visible in 3d */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Shape)
	bool bShapeVisible;

	/** Defines how the shape visibility should be changed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Shape)
	ERigControlVisibility ShapeVisibility;

	/* This is optional UI setting - this doesn't mean this is always used, but it is optional for manipulation layer to use this*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Shape)
	FName ShapeName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Shape)
	FLinearColor ShapeColor;

	/** If the control is transient and only visible in the control rig editor */
	UPROPERTY(BlueprintReadWrite, Category = Control)
	bool bIsTransientControl;

	/** If the control is integer it can use this enum to choose values */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	TObjectPtr<UEnum> ControlEnum;

	/**
	 * The User interface customization used for a control
	 * This will be used as the default content for the space picker and other widgets
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation, meta = (DisplayName = "Customization"))
	FRigControlElementCustomization Customization;

	/**
	 * The list of driven controls for this proxy control.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation)
	TArray<FRigElementKey> DrivenControls;

	/**
	 * The list of previously driven controls - prior to a procedural change
	 */
	TArray<FRigElementKey> PreviouslyDrivenControls;

	/**
	 * If set to true the animation channel will be grouped with the parent control in sequencer
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation)
	bool bGroupWithParentControl;

	/**
	 * Allow to space switch only to the available spaces
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation)
	bool bRestrictSpaceSwitching;

	/**
	 * Filtered Visible Transform channels. If this is empty everything is visible
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation)
	TArray<ERigControlTransformChannel> FilteredChannels;

	/**
	 * The euler rotation order this control prefers for animation, if we aren't using default UE rotator
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation)
	EEulerRotationOrder PreferredRotationOrder;

	/**
	* Whether to use a specified rotation order or just use the default FRotator order and conversion functions
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Animation)
	bool bUsePreferredRotationOrder;

	/**
	* The euler rotation order this control prefers for animation if it is active. If not set then we use the default UE rotator.
	*/
	TOptional<EEulerRotationOrder> GetRotationOrder() const
	{
		TOptional<EEulerRotationOrder> RotationOrder;
		if (bUsePreferredRotationOrder)
		{
			RotationOrder = PreferredRotationOrder;
		}
		return RotationOrder;
	}

	/**
	*  Set the rotation order if the rotation is set otherwise use default rotator
	*/
	void SetRotationOrder(const TOptional<EEulerRotationOrder>& EulerRotation)
	{
		if (EulerRotation.IsSet())
		{
			bUsePreferredRotationOrder = true;
			PreferredRotationOrder = EulerRotation.GetValue();
		}
		else
		{
			bUsePreferredRotationOrder = false;
		}
	}
#if WITH_EDITORONLY_DATA
	/**
	 * Deprecated properties.
	 */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use animation_type instead."))
	bool bAnimatable_DEPRECATED = true;
	
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use animation_type or shape_visible instead."))
	bool bShapeEnabled_DEPRECATED = true;
#endif

	/**
	 * Transient storage for overrides when changing the shape transform
	 */
	UPROPERTY(BlueprintReadOnly, Transient, Category = Animation)
	FTransform ShapeTransform;
	
	/** Applies the limits expressed by these settings to a value */
	void ApplyLimits(FRigControlValue& InOutValue) const
	{
		InOutValue.ApplyLimits(LimitEnabled, ControlType, MinimumValue, MaximumValue);
	}

	/** Applies the limits expressed by these settings to a transform */
	void ApplyLimits(FTransform& InOutValue) const
	{
		FRigControlValue Value;
		Value.SetFromTransform(InOutValue, ControlType, PrimaryAxis);
		ApplyLimits(Value);
		InOutValue = Value.GetAsTransform(ControlType, PrimaryAxis);
	}

	FRigControlValue GetIdentityValue() const
	{
		FRigControlValue Value;
		Value.SetFromTransform(FTransform::Identity, ControlType, PrimaryAxis);
		return Value;
	}

	UE_API bool operator == (const FRigControlSettings& InOther) const;

	bool operator != (const FRigControlSettings& InOther) const
	{
		return !(*this == InOther);
	}

	UE_API void SetupLimitArrayForType(bool bLimitTranslation = false, bool bLimitRotation = false, bool bLimitScale = false);

	bool IsAnimatable() const
	{
		return (AnimationType == ERigControlAnimationType::AnimationControl) ||
			(AnimationType == ERigControlAnimationType::AnimationChannel);
	}

	bool ShouldBeGrouped() const
	{
		return IsAnimatable() && bGroupWithParentControl;
	}

	bool SupportsShape() const
	{
		return (AnimationType != ERigControlAnimationType::AnimationChannel) &&
			(ControlType != ERigControlType::Bool);
	}

	bool IsVisible() const
	{
		return SupportsShape() && bShapeVisible;
	}
	
	bool SetVisible(bool bVisible, bool bForce = false)
	{
		if(!bForce)
		{
			if(AnimationType == ERigControlAnimationType::ProxyControl)
			{
				if(ShapeVisibility == ERigControlVisibility::BasedOnSelection)
				{
					return false;
				}
			}
		}
		
		if(SupportsShape())
		{
			if(bShapeVisible == bVisible)
			{
				return false;
			}
			bShapeVisible = bVisible;
		}
		return SupportsShape();
	}

	bool IsSelectable(bool bRespectVisibility = true) const
	{
		return (AnimationType == ERigControlAnimationType::AnimationControl ||
			AnimationType == ERigControlAnimationType::ProxyControl) &&
			(IsVisible() || !bRespectVisibility);
	}

	void SetAnimationTypeFromDeprecatedData(bool bAnimatable, bool bShapeEnabled)
	{
		if(bAnimatable)
		{
			if(bShapeEnabled && (ControlType != ERigControlType::Bool))
			{
				AnimationType = ERigControlAnimationType::AnimationControl;
			}
			else
			{
				AnimationType = ERigControlAnimationType::AnimationChannel;
			}
		}
		else
		{
			AnimationType = ERigControlAnimationType::ProxyControl;
		}
	}
};

USTRUCT(BlueprintType)
struct FRigControlElement final : public FRigMultiParentElement
{
	public:
	
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigControlElement)

	static UE_API const EElementIndex ElementTypeIndex;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control,
		meta = (DeprecatedProperty, DeprecationMessage = "Settings cannot be accessed directly. Use the Get Control Settings node (URigHierarchy::GetControlSettings) instead."))
	FRigControlSettings Settings;

	// Current and Initial, both in Local and Global, for Pose, Offset and Shape
	virtual int32 GetNumTransforms() const override { return 12; }
	
	UE_API const FRigCurrentAndInitialTransform& GetOffsetTransform() const;
	UE_API FRigCurrentAndInitialTransform& GetOffsetTransform();
	UE_API const FRigCurrentAndInitialDirtyState& GetOffsetDirtyState() const;
	UE_API FRigCurrentAndInitialDirtyState& GetOffsetDirtyState();
	UE_API const FRigCurrentAndInitialTransform& GetShapeTransform() const;
	UE_API FRigCurrentAndInitialTransform& GetShapeTransform();
	UE_API const FRigCurrentAndInitialDirtyState& GetShapeDirtyState() const;
	UE_API FRigCurrentAndInitialDirtyState& GetShapeDirtyState();

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = RigElement)
	FRigPreferredEulerAngles PreferredEulerAngles;

	FRigControlElement()
		: FRigControlElement(nullptr)
	{ }
	
	FRigControlElement(const FRigControlElement& InOther)
	{
		*this = InOther;
	}
	
	FRigControlElement& operator=(const FRigControlElement& InOther)
	{
		Super::operator=(InOther);
		Settings = InOther.Settings;
		GetOffsetTransform() = InOther.GetOffsetTransform();
		GetShapeTransform() = InOther.GetShapeTransform();
		PreferredEulerAngles = InOther.PreferredEulerAngles;
		return *this;
	}

	virtual ~FRigControlElement() override = default;
	
	virtual const FName& GetDisplayName() const override
	{
		if(!Settings.DisplayName.IsNone())
		{
			return Settings.DisplayName;
		}
		return FRigMultiParentElement::GetDisplayName();
	}

	bool IsAnimationChannel() const { return Settings.AnimationType == ERigControlAnimationType::AnimationChannel; }

	bool CanDriveControls() const { return Settings.AnimationType == ERigControlAnimationType::ProxyControl || Settings.AnimationType == ERigControlAnimationType::AnimationControl; }

	bool CanTreatAsAdditive() const
	{
		if (Settings.ControlType == ERigControlType::Bool)
		{
			return false;
		}
		if (Settings.ControlType == ERigControlType::Integer && Settings.ControlEnum != nullptr)
		{
			return false;
		}
		if (Settings.AnimationType == ERigControlAnimationType::ProxyControl)
		{
			return false;
		}
		return true;
	}

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API virtual void CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights) override;

	UE_API virtual bool IsElementDirty(ERigTransformType::Type InTransformType) const override;
	UE_API virtual void MarkElementDirty(ERigTransformType::Type InTransformType) override;

protected:

	UE_API virtual void LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates, const TArrayView<float>& InCurves) override;
	UE_API virtual void UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms, FRigReusableElementStorage<bool>& InDirtyStates, FRigReusableElementStorage<float>& InCurves) override;
	
private:
	explicit FRigControlElement(URigHierarchy* InOwner)
		: FRigMultiParentElement(InOwner, ERigElementType::Control)
	{ }

	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	// Offset storage for this element.
	FRigCurrentAndInitialTransform OffsetStorage;
	FRigCurrentAndInitialDirtyState OffsetDirtyState;

	// Shape storage for this element.
	FRigCurrentAndInitialTransform ShapeStorage;
	FRigCurrentAndInitialDirtyState ShapeDirtyState;

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigBaseElement;
	friend class FControlRigBaseEditor;
};

USTRUCT(BlueprintType)
struct FRigCurveElement final : public FRigBaseElement
{
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigCurveElement)

	static UE_API const EElementIndex ElementTypeIndex;

	FRigCurveElement()
		: FRigCurveElement(nullptr)
	{}
	
	virtual ~FRigCurveElement() override = default;

	virtual int32 GetNumCurves() const override { return 1; }

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API virtual void CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights) override;

	UE_API const float& Get() const;
	UE_API void Set(const float& InValue, bool InValueIsSet = true);

	bool IsValueSet() const { return bIsValueSet; }

	int32 GetStorageIndex() const
	{
		return StorageIndex;
	}

protected:

	UE_API virtual void LinkStorage(const TArrayView<FTransform>& InTransforms, const TArrayView<bool>& InDirtyStates, const TArrayView<float>& InCurves) override;
	UE_API virtual void UnlinkStorage(FRigReusableElementStorage<FTransform>& InTransforms, FRigReusableElementStorage<bool>& InDirtyStates, FRigReusableElementStorage<float>& InCurves) override;
	
private:
	FRigCurveElement(URigHierarchy* InOwner)
		: FRigBaseElement(InOwner, ERigElementType::Curve)
	{}
	
	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	// Set to true if the value was actually set. Used to carry back and forth blend curve
	// value validity state.
	bool bIsValueSet = true;

	int32 StorageIndex = INDEX_NONE;
	float* Storage = nullptr;

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigBaseElement;
	friend class FRigHierarchyPoseAdapter;
	friend struct FRigReusableElementStorage<float>;
	friend class FControlRigHierarchyRelinkElementStorage;
};

USTRUCT(BlueprintType)
struct FRigReferenceElement final : public FRigSingleParentElement
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigReferenceElement)

	static UE_API const EElementIndex ElementTypeIndex;

    FRigReferenceElement()
        : FRigReferenceElement(nullptr)
	{ }
	
	virtual ~FRigReferenceElement() override = default;

	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API FTransform GetReferenceWorldTransform(const FRigVMExecuteContext* InContext, bool bInitial) const;

	UE_API virtual void CopyPose(FRigBaseElement* InOther, bool bCurrent, bool bInitial, bool bWeights) override;

private:
	explicit FRigReferenceElement(URigHierarchy* InOwner)
		: FRigSingleParentElement(InOwner, ERigElementType::Reference)
	{ }

	FRigReferenceGetWorldTransformDelegate GetWorldTransformDelegate;

	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend struct FRigBaseElement;
};

UENUM(BlueprintType)
enum class EConnectorType : uint8
{
	Primary, // Single primary connector, non-optional and always visible. When dropped on another element, this connector will resolve to that element.
	Secondary, // Could be multiple, can auto-solve (visible if not solved), can be optional
};


USTRUCT(BlueprintType)
struct FRigConnectorSettings
{
	GENERATED_BODY()

	UE_API FRigConnectorSettings();
	static UE_API FRigConnectorSettings DefaultSettings();

	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);

	friend uint32 GetTypeHash(const FRigConnectorSettings& Settings);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FString Description;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	EConnectorType Type;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	bool bOptional;

	// by enabling this the connector will be able to connect to more than one target
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	bool bIsArray;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	bool bPostConstruction;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	TArray<FRigConnectionRuleStash> Rules;

	UE_API bool operator == (const FRigConnectorSettings& InOther) const;

	bool operator != (const FRigConnectorSettings& InOther) const
	{
		return !(*this == InOther);
	}

	template<typename T>
	int32 AddRule(const T& InRule)
	{
		return Rules.Emplace(&InRule);
	}

	UE_API uint32 GetRulesHash() const;
};

USTRUCT(BlueprintType)
struct FRigConnectorState
{
	GENERATED_BODY()

	FRigConnectorState()
		: Name(NAME_None)
	{}
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FName Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FRigElementKey ResolvedTarget;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FRigConnectorSettings Settings;
};

USTRUCT(BlueprintType)
struct FRigConnectorElement final : public FRigBaseElement
{
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigConnectorElement)

	static UE_API const EElementIndex ElementTypeIndex;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	FRigConnectorSettings Settings;
	
	FRigConnectorElement()
		: FRigConnectorElement(nullptr)
	{}
	FRigConnectorElement(const FRigConnectorElement& InOther)
	{
		*this = InOther;
	}
	FRigConnectorElement& operator=(const FRigConnectorElement& InOther)
	{
		Super::operator=(InOther);
		Settings = InOther.Settings;
		return *this;
	}

	virtual ~FRigConnectorElement() override = default;
	
	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API FRigConnectorState GetConnectorState(const URigHierarchy* InHierarchy) const;

	bool IsPrimary() const { return Settings.Type == EConnectorType::Primary; }
	bool IsSecondary() const { return Settings.Type == EConnectorType::Secondary; }
	bool IsOptional() const { return IsSecondary() && Settings.bOptional; }
	bool IsArrayConnector() const { return IsSecondary() && Settings.bIsArray; }
	bool IsPostConstructionConnector() const { return IsSecondary() && Settings.bPostConstruction; }

private:
	explicit FRigConnectorElement(URigHierarchy* InOwner)
		: FRigBaseElement(InOwner, ERigElementType::Connector)
	{ }
	
	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	friend class URigHierarchy;
	friend struct FRigBaseElement;
};

USTRUCT(BlueprintType)
struct FRigSocketState
{
	GENERATED_BODY()
	
	UE_API FRigSocketState();
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FName Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FRigElementKey Parent;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FTransform InitialLocalTransform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FLinearColor Color;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FString Description;
};

USTRUCT(BlueprintType)
struct FRigSocketElement final : public FRigSingleParentElement
{
	GENERATED_BODY()
	DECLARE_RIG_ELEMENT_METHODS(FRigSocketElement)

	static UE_API const EElementIndex ElementTypeIndex;
	static UE_API const FName ColorMetaName;
	static UE_API const FName DescriptionMetaName;
	static UE_API const FName DesiredParentMetaName;
	static UE_API const FLinearColor SocketDefaultColor;

	FRigSocketElement()
		: FRigSocketElement(nullptr)
	{}
			
	virtual ~FRigSocketElement() override = default;
	
	UE_API virtual void Save(FArchive& A, const FRigHierarchySerializationSettings& InSettings) override;
	UE_API virtual void Load(FArchive& Ar, const FRigHierarchySerializationSettings& InSettings) override;

	UE_API FRigSocketState GetSocketState(const URigHierarchy* InHierarchy) const;

	UE_API FLinearColor GetColor(const URigHierarchy* InHierarchy) const;
	UE_API void SetColor(const FLinearColor& InColor, URigHierarchy* InHierarchy, bool bNotify = true);

	UE_API FString GetDescription(const URigHierarchy* InHierarchy) const;
	UE_API void SetDescription(const FString& InDescription, URigHierarchy* InHierarchy, bool bNotify = true);

private:
	explicit FRigSocketElement(URigHierarchy* InOwner)
		: FRigSingleParentElement(InOwner, ERigElementType::Socket)
	{ }
	
	UE_API virtual void CopyFrom(const FRigBaseElement* InOther) override;

	friend class URigHierarchy;
	friend struct FRigBaseElement;
};


USTRUCT()
struct FRigHierarchyCopyPasteContentPerElement
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FRigElementKey Key;

	UPROPERTY()
	FString Content;

	UPROPERTY()
	TArray<FRigElementKeyWithLabel> Parents;

	UPROPERTY()
	TArray<FRigElementWeight> ParentWeights;

	UPROPERTY()
	TArray<FTransform> Poses;

	UPROPERTY()
	TArray<bool> DirtyStates;
};

USTRUCT()
struct FRigHierarchyCopyPasteContent
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<FRigHierarchyCopyPasteContentPerElement> Elements;

	// Maintain properties below for backwards compatibility pre-5.0
	UPROPERTY()
	TArray<ERigElementType> Types;

	UPROPERTY()
	TArray<FString> Contents;

	UPROPERTY()
	TArray<FTransform> LocalTransforms;

	UPROPERTY()
	TArray<FTransform> GlobalTransforms;
};

#undef UE_API
