// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyMetadata.generated.h"

#define UE_API CONTROLRIG_API

struct FRigBaseElement;
class URigHierarchyController;

#define DECLARE_RIG_METADATA_METHODS(MetadataType) \
template<typename T> \
friend const T* Cast(const MetadataType* InMetadata) \
{ \
   return Cast<T>((const FRigBaseMetadata*) InMetadata); \
} \
template<typename T> \
friend T* Cast(MetadataType* InMetadata) \
{ \
   return Cast<T>((FRigBaseMetadata*) InMetadata); \
} \
template<typename T> \
friend const T* CastChecked(const MetadataType* InMetadata) \
{ \
	return CastChecked<T>((const FRigBaseMetadata*) InMetadata); \
} \
template<typename T> \
friend T* CastChecked(MetadataType* InMetadata) \
{ \
	return CastChecked<T>((FRigBaseMetadata*) InMetadata); \
}

USTRUCT()
struct FRigBaseMetadata
{
	GENERATED_BODY()

protected:

	UPROPERTY()
	FName Name;

	UPROPERTY()
	ERigMetadataType Type = ERigMetadataType::Invalid;

	mutable const FProperty* ValueProperty = nullptr;

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return true;
	}

public:
	virtual ~FRigBaseMetadata() = default;

	UE_API UScriptStruct* GetMetadataStruct() const;
	UE_API virtual void Serialize(FArchive& Ar);

	bool IsValid() const { return GetType() != ERigMetadataType::Invalid; }
	const FName& GetName() const { return Name; }
	ERigMetadataType GetType() const { return Type; }
	bool IsArray() const
	{
		return GetValueProperty()->IsA<FArrayProperty>();
	}
	const void* GetValueData() const
	{
		return GetValueProperty()->ContainerPtrToValuePtr<void>(this);
	}
	int32 GetValueSize() const
	{
		return GetValueProperty()->GetElementSize();
	}
	bool SetValueData(const void* InData, int32 InSize)
	{
		if(InData)
		{
			if(InSize == GetValueSize())
			{
				return SetValueDataImpl(InData);
			}
		}
		return false;
	}

	template<typename T>
	bool IsA() const { return T::IsClassOf(this); }

	template<typename T>
    friend const T* Cast(const FRigBaseMetadata* InMetadata)
	{
		if(InMetadata)
		{
			if(InMetadata->IsA<T>())
			{
				return static_cast<const T*>(InMetadata);
			}
		}
		return nullptr;
	}

	template<typename T>
    friend T* Cast(FRigBaseMetadata* InMetadata)
	{
		if(InMetadata)
		{
			if(InMetadata->IsA<T>())
			{
				return static_cast<T*>(InMetadata);
			}
		}
		return nullptr;
	}

	template<typename T>
    friend const T* CastChecked(const FRigBaseMetadata* InMetadata)
	{
		const T* CastElement = Cast<T>(InMetadata);
		check(CastElement);
		return CastElement;
	}

	template<typename T>
    friend T* CastChecked(FRigBaseMetadata* InMetadata)
	{
		T* CastElement = Cast<T>(InMetadata);
		check(CastElement);
		return CastElement;
	}

protected:

	static UE_API UScriptStruct* GetMetadataStruct(const ERigMetadataType& InType);
	static UE_API FRigBaseMetadata* MakeMetadata(const FName& InName, ERigMetadataType InType);
	static UE_API void DestroyMetadata(FRigBaseMetadata** Metadata);

	const FProperty* GetValueProperty() const
	{
		if(ValueProperty == nullptr)
		{
			static constexpr TCHAR ValueName[] = TEXT("Value");
			ValueProperty = GetMetadataStruct()->FindPropertyByName(ValueName);
		}
		return ValueProperty;
	}

	void* GetValueData()
	{
		return GetValueProperty()->ContainerPtrToValuePtr<void>(this);
	}

	virtual bool SetValueDataImpl(const void* InData)
	{
		const FProperty* Property = GetValueProperty();
		if(!Property->Identical(GetValueData(), InData))
		{
			Property->CopyCompleteValue(GetValueData(), InData);
			return true;
		}
		return false;
	}

	friend struct FRigBaseElement;
	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend class UControlRigTestData;
	friend struct FControlRigTestDataFrame;
};

USTRUCT()
struct FRigBoolMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigBoolMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const bool& GetValue() const { return Value; }
	bool& GetValue() { return Value; }
	bool SetValue(const bool& InValue) { return SetValueData(&InValue, sizeof(bool)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::Bool;
	}

protected:

	UPROPERTY()
	bool Value = false;
};

USTRUCT()
struct FRigBoolArrayMetadata : public FRigBaseMetadata
{
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigBoolArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<bool>& GetValue() const { return Value; }
	TArray<bool>& GetValue() { return Value; }
	bool SetValue(const TArray<bool>& InValue) { return SetValueData(&InValue, sizeof(TArray<bool>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::BoolArray;
	}

protected:

	UPROPERTY()
	TArray<bool> Value;
};

USTRUCT()
struct FRigFloatMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigFloatMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const float& GetValue() const { return Value; } 
	float& GetValue() { return Value; }
	bool SetValue(const float& InValue) { return SetValueData(&InValue, sizeof(float)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Float;
    }

protected:

	UPROPERTY()
	float Value = 0.f;
};

USTRUCT()
struct FRigFloatArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigFloatArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<float>& GetValue() const { return Value; }
	TArray<float>& GetValue() { return Value; }
	bool SetValue(const TArray<float>& InValue) { return SetValueData(&InValue, sizeof(TArray<float>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::FloatArray;
	}

protected:

	UPROPERTY()
	TArray<float> Value;
};

USTRUCT()
struct FRigInt32Metadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigInt32Metadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const int32& GetValue() const { return Value; } 
	int32& GetValue() { return Value; }
	bool SetValue(const int32& InValue) { return SetValueData(&InValue, sizeof(int32)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Int32;
    }

protected:

	UPROPERTY()
	int32 Value = 0;
};

USTRUCT()
struct FRigInt32ArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigInt32ArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<int32>& GetValue() const { return Value; }
	TArray<int32>& GetValue() { return Value; }
	bool SetValue(const TArray<int32>& InValue) { return SetValueData(&InValue, sizeof(TArray<int32>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::Int32Array;
	}

protected:

	UPROPERTY()
	TArray<int32> Value;
};

USTRUCT()
struct FRigNameMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigNameMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FName& GetValue() const { return Value; } 
	FName& GetValue() { return Value; }
	bool SetValue(const FName& InValue) { return SetValueData(&InValue, sizeof(FName)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Name;
    }

protected:

	UPROPERTY()
	FName Value;
};

USTRUCT()
struct FRigNameArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigNameArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FName>& GetValue() const { return Value; }
	TArray<FName>& GetValue() { return Value; }
	bool SetValue(const TArray<FName>& InValue) { return SetValueData(&InValue, sizeof(TArray<FName>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::NameArray;
	}

protected:

	UPROPERTY()
	TArray<FName> Value;
};

USTRUCT()
struct FRigVectorMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigVectorMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FVector& GetValue() const { return Value; } 
	FVector& GetValue() { return Value; }
	bool SetValue(const FVector& InValue) { return SetValueData(&InValue, sizeof(FVector)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Vector;
    }

protected:

	UPROPERTY()
	FVector Value = FVector::ZeroVector;
};

USTRUCT()
struct FRigVectorArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigVectorArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FVector>& GetValue() const { return Value; }
	TArray<FVector>& GetValue() { return Value; }
	bool SetValue(const TArray<FVector>& InValue) { return SetValueData(&InValue, sizeof(TArray<FVector>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::VectorArray;
	}

protected:

	UPROPERTY()
	TArray<FVector> Value;
};

USTRUCT()
struct FRigRotatorMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigRotatorMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FRotator& GetValue() const { return Value; } 
	FRotator& GetValue() { return Value; }
	bool SetValue(const FRotator& InValue) { return SetValueData(&InValue, sizeof(FRotator)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Rotator;
    }

protected:

	UPROPERTY()
	FRotator Value = FRotator::ZeroRotator;
};

USTRUCT()
struct FRigRotatorArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigRotatorArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FRotator>& GetValue() const { return Value; }
	TArray<FRotator>& GetValue() { return Value; }
	bool SetValue(const TArray<FRotator>& InValue) { return SetValueData(&InValue, sizeof(TArray<FRotator>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::RotatorArray;
	}

protected:

	UPROPERTY()
	TArray<FRotator> Value;
};

USTRUCT()
struct FRigQuatMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigQuatMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FQuat& GetValue() const { return Value; } 
	FQuat& GetValue() { return Value; }
	bool SetValue(const FQuat& InValue) { return SetValueData(&InValue, sizeof(FQuat)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Quat;
    }

protected:

	UPROPERTY()
	FQuat Value = FQuat::Identity;
};

USTRUCT()
struct FRigQuatArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigQuatArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FQuat>& GetValue() const { return Value; }
	TArray<FQuat>& GetValue() { return Value; }
	bool SetValue(const TArray<FQuat>& InValue) { return SetValueData(&InValue, sizeof(TArray<FQuat>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::QuatArray;
	}

protected:

	UPROPERTY()
	TArray<FQuat> Value;
};

USTRUCT()
struct FRigTransformMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigTransformMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FTransform& GetValue() const { return Value; } 
	FTransform& GetValue() { return Value; }
	bool SetValue(const FTransform& InValue) { return SetValueData(&InValue, sizeof(FTransform)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::Transform;
    }

protected:

	UPROPERTY()
	FTransform Value = FTransform::Identity;
};

USTRUCT()
struct FRigTransformArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigTransformArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FTransform>& GetValue() const { return Value; }
	TArray<FTransform>& GetValue() { return Value; }
	bool SetValue(const TArray<FTransform>& InValue) { return SetValueData(&InValue, sizeof(TArray<FTransform>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::TransformArray;
	}

protected:

	UPROPERTY()
	TArray<FTransform> Value;
};

USTRUCT()
struct FRigLinearColorMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigLinearColorMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FLinearColor& GetValue() const { return Value; } 
	FLinearColor& GetValue() { return Value; }
	bool SetValue(const FLinearColor& InValue) { return SetValueData(&InValue, sizeof(FLinearColor)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::LinearColor;
	}

protected:

	UPROPERTY()
	FLinearColor Value = FLinearColor::White;
};

USTRUCT()
struct FRigLinearColorArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigLinearColorArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FLinearColor>& GetValue() const { return Value; }
	TArray<FLinearColor>& GetValue() { return Value; }
	bool SetValue(const TArray<FLinearColor>& InValue) { return SetValueData(&InValue, sizeof(TArray<FLinearColor>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::LinearColorArray;
	}

protected:
	UPROPERTY()
	TArray<FLinearColor> Value;
};

USTRUCT()
struct FRigElementKeyMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigElementKeyMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const FRigElementKey& GetValue() const { return Value; } 
	FRigElementKey& GetValue() { return Value; }
	bool SetValue(const FRigElementKey& InValue) { return SetValueData(&InValue, sizeof(FRigElementKey)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
    {
    	return InMetadata->GetType() == ERigMetadataType::RigElementKey;
    }

protected:

	UPROPERTY()
	FRigElementKey Value;
};

USTRUCT()
struct FRigElementKeyArrayMetadata : public FRigBaseMetadata
{
public:
	
	GENERATED_BODY()
	DECLARE_RIG_METADATA_METHODS(FRigElementKeyArrayMetadata)

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);
		Ar << Value;
	}
	const TArray<FRigElementKey>& GetValue() const { return Value; }
	TArray<FRigElementKey>& GetValue() { return Value; }
	bool SetValue(const TArray<FRigElementKey>& InValue) { return SetValueData(&InValue, sizeof(TArray<FRigElementKey>)); }

	static bool IsClassOf(const FRigBaseMetadata* InMetadata)
	{
		return InMetadata->GetType() == ERigMetadataType::RigElementKeyArray;
	}

protected:

	UPROPERTY()
	TArray<FRigElementKey> Value;
};

#undef UE_API
