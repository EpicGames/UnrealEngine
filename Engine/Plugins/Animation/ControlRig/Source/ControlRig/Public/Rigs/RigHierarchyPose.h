// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyCache.h"
#include "RigHierarchyPose.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(BlueprintType)
struct FRigPoseElement
{
public:

	GENERATED_BODY()

	UE_API FRigPoseElement();

	UPROPERTY()
	FCachedRigElement Index;

	UPROPERTY()
	FTransform GlobalTransform;

	UPROPERTY()
	FTransform LocalTransform;

	UPROPERTY()
	FVector PreferredEulerAngle;

	UPROPERTY()
	FRigElementKey ActiveParent;

	UPROPERTY()
	float CurveValue;
};

USTRUCT(BlueprintType)
struct FRigPose
{
public:

	GENERATED_BODY()

	FRigPose()
	: HierarchyTopologyVersion(INDEX_NONE)
	, PoseHash(INDEX_NONE)
	, CachedPoseHash(INDEX_NONE)
	{
	}

	void Reset() { Elements.Reset(); }

	int32 Num() const { return Elements.Num(); }
	bool IsValidIndex(int32 InIndex) const { return Elements.IsValidIndex(InIndex); }

	int32 GetIndex(const FRigElementKey& InKey) const
	{
		if(CachedPoseHash != PoseHash)
		{
			KeyToIndex.Reset();
			CachedPoseHash = PoseHash;
		}

		if(const int32* IndexPtr = KeyToIndex.Find(InKey))
		{
			return *IndexPtr;
		}
		
		for(int32 Index=0;Index<Elements.Num();Index++)
		{
			const FRigElementKey& Key = Elements[Index].Index.GetKey();

			KeyToIndex.Add(Key, Index);
			if(Key == InKey)
			{
				return Index;
			}
		}
		return INDEX_NONE;
	}

	bool Contains(const FRigElementKey& InKey) const
	{
		return GetIndex(InKey) != INDEX_NONE;
	}

	const FRigPoseElement& operator[](int32 InIndex) const { return Elements[InIndex]; }
	FRigPoseElement& operator[](int32 InIndex) { return Elements[InIndex]; }
	TArray<FRigPoseElement>::RangedForIteratorType      begin()       { return Elements.begin(); }
	TArray<FRigPoseElement>::RangedForConstIteratorType begin() const { return Elements.begin(); }
	TArray<FRigPoseElement>::RangedForIteratorType      end()         { return Elements.end();   }
	TArray<FRigPoseElement>::RangedForConstIteratorType end() const   { return Elements.end();   }

	UPROPERTY()
	TArray<FRigPoseElement> Elements;

	UPROPERTY()
	int32 HierarchyTopologyVersion;

	UPROPERTY()
	int32 PoseHash;

private:
	
	mutable int32 CachedPoseHash;
	mutable TMap<FRigElementKey, int32> KeyToIndex;
};

#undef UE_API
