// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyElements.h"

#define UE_API CONTROLRIG_API

class URigHierarchy;

class FRigHierarchyPoseAdapter : public TSharedFromThis<FRigHierarchyPoseAdapter>
{
public:

	virtual ~FRigHierarchyPoseAdapter() = default;

protected:

	UE_API URigHierarchy* GetHierarchy() const;

	UE_API virtual void PostLinked(URigHierarchy* InHierarchy);
	UE_API virtual void PreUnlinked(URigHierarchy* InHierarchy);
	UE_API virtual void PostUnlinked(URigHierarchy* InHierarchy);
	bool IsLinked() const { return bLinked; }
	UE_API bool IsLinkedTo(const URigHierarchy* InHierarchy) const;
	UE_API virtual bool IsUpdateToDate(const URigHierarchy* InHierarchy) const;

	UE_API TTuple<FRigComputedTransform*, FRigTransformDirtyState*> GetElementTransformStorage(const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType, ERigTransformStorageType::Type InStorageType) const;

	UE_API bool RelinkTransformStorage(const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType, ERigTransformStorageType::Type InStorageType, FTransform* InTransformStorage, bool* InDirtyFlagStorage);
	UE_API bool RestoreTransformStorage(const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType, ERigTransformStorageType::Type InStorageType, bool bUpdateElementStorage);
	UE_API bool RelinkTransformStorage(const TArrayView<TTuple<FRigElementKeyAndIndex,ERigTransformType::Type,ERigTransformStorageType::Type,FTransform*,bool*>>& InData);
	UE_API bool RestoreTransformStorage(const TArrayView<TTuple<FRigElementKeyAndIndex,ERigTransformType::Type,ERigTransformStorageType::Type>>& InData, bool bUpdateElementStorage);

	UE_API bool RelinkCurveStorage(const FRigElementKeyAndIndex& InKeyAndIndex, float* InCurveStorage);
	UE_API bool RestoreCurveStorage(const FRigElementKeyAndIndex& InKeyAndIndex, bool bUpdateElementStorage);
	UE_API bool RelinkCurveStorage(const TArrayView<TTuple<FRigElementKeyAndIndex,float*>>& InData);
	UE_API bool RestoreCurveStorage(const TArrayView<FRigElementKeyAndIndex>& InData, bool bUpdateElementStorage);

	UE_API bool SortHierarchyStorage();
	UE_API bool ShrinkHierarchyStorage();
	UE_API bool UpdateHierarchyStorage();

	TWeakObjectPtr<URigHierarchy> WeakHierarchy;
	bool bLinked = false;
	uint32 LastTopologyVersion = UINT32_MAX;

	friend class URigHierarchy;
	friend class URigHierarchyController;
	friend class FControlRigHierarchyPoseAdapter;
};

#undef UE_API
