// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigInfluenceMap.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(BlueprintType)
struct FRigInfluenceEntry
{
	GENERATED_BODY()

	FRigInfluenceEntry()
		: Source()
		, AffectedList()
	{
	}

	const FRigElementKey& GetSource() const { return Source; }

	int32 Num() const { return AffectedList.Num(); }
	const FRigElementKey& operator[](int32 InIndex) const { return AffectedList[InIndex]; }
	FRigElementKey& operator[](int32 InIndex) { return AffectedList[InIndex]; }

	TArray<FRigElementKey>::RangedForIteratorType      begin() { return AffectedList.begin(); }
	TArray<FRigElementKey>::RangedForConstIteratorType begin() const { return AffectedList.begin(); }
	TArray<FRigElementKey>::RangedForIteratorType      end() { return AffectedList.end(); }
	TArray<FRigElementKey>::RangedForConstIteratorType end() const { return AffectedList.end(); }

	int32 AddUnique(const FRigElementKey& InKey)
	{
		return AffectedList.AddUnique(InKey);
	}

	void Remove(const FRigElementKey& InKey)
	{
		AffectedList.Remove(InKey);
	}

	bool Contains(const FRigElementKey& InKey) const
	{
		return AffectedList.Contains(InKey);
	}

	bool Merge(const FRigInfluenceEntry& Other)
	{
		if(Other.Source != Source)
		{
			return false;
		}

		for(const FRigElementKey& OtherAffected : Other)
		{
			AddUnique(OtherAffected);
		}

		return true;
	}

	void OnKeyRemoved(const FRigElementKey& InKey);

	void OnKeyRenamed(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey);

protected:

	UPROPERTY()
	FRigElementKey Source;

	UPROPERTY()
	TArray<FRigElementKey> AffectedList;

	friend struct FRigInfluenceMap;
};

USTRUCT()
struct FRigInfluenceEntryModifier
{
	GENERATED_BODY()

	FRigInfluenceEntryModifier()
		: AffectedList()
	{
	}

	UPROPERTY(EditAnywhere, Category = "Inversion")
	TArray<FRigElementKey> AffectedList;

	int32 Num() const { return AffectedList.Num(); }
	const FRigElementKey& operator[](int32 InIndex) const { return AffectedList[InIndex]; }
	FRigElementKey& operator[](int32 InIndex) { return AffectedList[InIndex]; }

	TArray<FRigElementKey>::RangedForIteratorType      begin() { return AffectedList.begin(); }
	TArray<FRigElementKey>::RangedForConstIteratorType begin() const { return AffectedList.begin(); }
	TArray<FRigElementKey>::RangedForIteratorType      end() { return AffectedList.end(); }
	TArray<FRigElementKey>::RangedForConstIteratorType end() const { return AffectedList.end(); }
};


USTRUCT(BlueprintType)
struct FRigInfluenceMap
{
	GENERATED_BODY()

public:

	FRigInfluenceMap()
	:EventName(NAME_None)
	{}

	const FName& GetEventName() const { return EventName; }

	int32 Num() const { return Entries.Num(); }
	const FRigInfluenceEntry& operator[](int32 InIndex) const { return Entries[InIndex]; }
	FRigInfluenceEntry& operator[](int32 InIndex) { return Entries[InIndex]; }
	const FRigInfluenceEntry& operator[](const FRigElementKey& InKey) const { return Entries[GetIndex(InKey)]; }
	FRigInfluenceEntry& operator[](const FRigElementKey& InKey) { return FindOrAdd(InKey); }

	TArray<FRigInfluenceEntry>::RangedForIteratorType      begin()       { return Entries.begin(); }
	TArray<FRigInfluenceEntry>::RangedForConstIteratorType begin() const { return Entries.begin(); }
	TArray<FRigInfluenceEntry>::RangedForIteratorType      end()         { return Entries.end();   }
	TArray<FRigInfluenceEntry>::RangedForConstIteratorType end() const   { return Entries.end();   }

	UE_API FRigInfluenceEntry& FindOrAdd(const FRigElementKey& InKey);

	UE_API const FRigInfluenceEntry* Find(const FRigElementKey& InKey) const;

	UE_API void Remove(const FRigElementKey& InKey);

	bool Contains(const FRigElementKey& InKey) const
	{
		return GetIndex(InKey) != INDEX_NONE;
	}

	int32 GetIndex(const FRigElementKey& InKey) const
	{
		const int32* Index = KeyToIndex.Find(InKey);
		if (Index)
		{
			return *Index;
		}
		return INDEX_NONE;
	}

	UE_API FRigInfluenceMap Inverse() const;

	UE_API bool Merge(const FRigInfluenceMap& Other);

	UE_API FRigInfluenceEntryModifier GetEntryModifier(const FRigElementKey& InKey) const;

	UE_API void SetEntryModifier(const FRigElementKey& InKey, const FRigInfluenceEntryModifier& InModifier);

	UE_API void OnKeyRemoved(const FRigElementKey& InKey);

	UE_API void OnKeyRenamed(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey);

protected:

	UE_API bool Merge(const FRigInfluenceMap& Other, bool bIgnoreEventName);

	UPROPERTY()
	FName EventName;

	UPROPERTY()
	TArray<FRigInfluenceEntry> Entries;

	UPROPERTY()
	TMap<FRigElementKey, int32> KeyToIndex;

	friend struct FRigInfluenceMapPerEvent;
	friend class UControlRig;
};

USTRUCT(BlueprintType)
struct FRigInfluenceMapPerEvent
{
	GENERATED_BODY()

public:

	int32 Num() const { return Maps.Num(); }
	const FRigInfluenceMap& operator[](int32 InIndex) const { return Maps[InIndex]; }
	FRigInfluenceMap& operator[](int32 InIndex) { return Maps[InIndex]; }
	const FRigInfluenceMap& operator[](const FName& InEventName) const { return Maps[GetIndex(InEventName)]; }
	FRigInfluenceMap& operator[](const FName& InEventName) { return FindOrAdd(InEventName); }

	TArray<FRigInfluenceMap>::RangedForIteratorType      begin()       { return Maps.begin(); }
	TArray<FRigInfluenceMap>::RangedForConstIteratorType begin() const { return Maps.begin(); }
	TArray<FRigInfluenceMap>::RangedForIteratorType      end()         { return Maps.end();   }
	TArray<FRigInfluenceMap>::RangedForConstIteratorType end() const   { return Maps.end();   }

	UE_API FRigInfluenceMap& FindOrAdd(const FName& InEventName);

	UE_API const FRigInfluenceMap* Find(const FName& InEventName) const;

	UE_API void Remove(const FName& InEventName);

	bool Contains(const FName& InEventName) const
	{
		return GetIndex(InEventName) != INDEX_NONE;
	}

	int32 GetIndex(const FName& InEventName) const
	{
		const int32* Index = EventToIndex.Find(InEventName);
		if (Index)
		{
			return *Index;
		}
		return INDEX_NONE;
	}

	UE_API FRigInfluenceMapPerEvent Inverse() const;

	UE_API bool Merge(const FRigInfluenceMapPerEvent& Other);

	UE_API FRigInfluenceEntryModifier GetEntryModifier(const FName& InEventName, const FRigElementKey& InKey) const;

	UE_API void SetEntryModifier(const FName& InEventName, const FRigElementKey& InKey, const FRigInfluenceEntryModifier& InModifier);

	UE_API void OnKeyRemoved(const FRigElementKey& InKey);

	UE_API void OnKeyRenamed(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey);

protected:

	UPROPERTY()
	TArray<FRigInfluenceMap> Maps;

	UPROPERTY()
	TMap<FName, int32> EventToIndex;
};

#undef UE_API
