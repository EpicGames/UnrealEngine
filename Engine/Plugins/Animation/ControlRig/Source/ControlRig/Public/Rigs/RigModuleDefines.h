// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyElements.h"
#include "UObject/SoftObjectPath.h"
#include "ControlRigAssetReference.h"
#include "RigModuleDefines.generated.h"

#define UE_API CONTROLRIG_API

struct FRigModuleReference;

USTRUCT(BlueprintType)
struct FModularRigSettings
{
	GENERATED_BODY()

	// Whether or not to autoresolve secondary connectors once the primary connector is resolved
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ModularRig)
	bool bAutoResolve = true;
	
	bool operator==(const FModularRigSettings& InOther) const
	{
		return bAutoResolve == InOther.bAutoResolve;
	}
};

USTRUCT(BlueprintType)
struct FRigModuleIdentifier
{
	GENERATED_BODY()
	
	FRigModuleIdentifier()
		: Name()
		, Type(TEXT("Module"))
	{}

	// The name of the module used to find it in the module library
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FString Name;

	// The kind of module this is (for example "Arm")
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FString Type;

	bool IsValid() const { return !Name.IsEmpty(); }
	
	bool operator==(const FRigModuleIdentifier& InOther) const
	{
		return Name == InOther.Name && Type == InOther.Type;
	}
};

USTRUCT(BlueprintType)
struct FRigModuleConnector
{
	GENERATED_BODY()
	
	FRigModuleConnector()
	{}

	UE_API bool operator==(const FRigModuleConnector& Other) const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Connector)
	FRigConnectorSettings Settings;
	
	bool IsPrimary() const { return Settings.Type == EConnectorType::Primary; }
	bool IsSecondary() const { return Settings.Type == EConnectorType::Secondary; }
	bool IsOptional() const { return IsSecondary() && Settings.bOptional; }
};

USTRUCT(BlueprintType)
struct FRigModuleSettings
{
	GENERATED_BODY()
	
	FRigModuleSettings()
	{}

	bool IsValidModule(bool bRequireExposedConnectors = true) const
	{
		return
			Identifier.IsValid() &&
			(!bRequireExposedConnectors || !ExposedConnectors.IsEmpty());
	}

	const FRigModuleConnector* FindPrimaryConnector() const
	{
		return ExposedConnectors.FindByPredicate([](const FRigModuleConnector& Connector)
		{
			return Connector.IsPrimary();
		});
	}
	
	bool operator==(const FRigModuleSettings& InOther) const
	{
		return Identifier == InOther.Identifier
			&& Icon.ToString() == InOther.Icon.ToString()
			&& Category == InOther.Category
			&& Keywords == InOther.Keywords
			&& Description == InOther.Description
			&& ExposedConnectors == InOther.ExposedConnectors;
	}

	// The identifier used to retrieve the module in the module library
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Module)
	FRigModuleIdentifier Identifier;

	// The icon used for the module in the module library
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Module,  meta = (AllowedClasses = "/Script/Engine.Texture2D"))
	FSoftObjectPath Icon;

	// The category of the module
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Module)
	FString Category;

	// The keywords of the module
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Module)
	FString Keywords;

	// The description of the module
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Module, meta = (MultiLine = true))
	FString Description;

	UPROPERTY(BlueprintReadOnly, Category = Module)
	TArray<FRigModuleConnector> ExposedConnectors;
};


USTRUCT(BlueprintType)
struct FRigModuleDescription
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Module)
	FSoftObjectPath Path;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = Module)
	FRigModuleSettings Settings;
};

UENUM(BlueprintType)
enum class EControlRigType : uint8
{
	IndependentRig = 0,
	RigModule = 1,
	ModularRig = 2,
	MAX // Invalid
};

USTRUCT(BlueprintType)
struct FModuleReferenceData
{
	GENERATED_BODY()

public:
	
	FModuleReferenceData(){}

	UE_API FModuleReferenceData(const FRigModuleReference* InModule);
	
	bool operator==(const FModuleReferenceData& InOther) const
	{
		return ModulePath == InOther.ModulePath && ReferencedModuleAsset == InOther.ReferencedModuleAsset;
	}
	
	UE_API void PostSerialize(const FArchive& Ar);

	UPROPERTY()
	FString ModulePath;

	UPROPERTY(meta=(DeprecatedProperty))
	FSoftClassPath ReferencedModule_DEPRECATED;
	
	UPROPERTY()
	FControlRigAssetSoftReference ReferencedModuleAsset;
};

template<>
struct TStructOpsTypeTraits<FModuleReferenceData> : public TStructOpsTypeTraitsBase2<FModuleReferenceData>
{
	enum
	{
		WithPostSerialize = true,
	};
};

#undef UE_API
