// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CONTROLRIG_API

struct FRigName
{
public:
	FRigName()
	{}

	FRigName(const FRigName& InOther) = default;
	FRigName& operator =(const FRigName& InOther) = default;

	FRigName(const FName& InName)
		:FRigName()
	{
		SetFName(InName);
	}

	explicit FRigName(const FString& InNameString)
		:FRigName()
	{
		SetName(InNameString);
	}

	FRigName& operator =(const FName& InName)
	{
		SetFName(InName);
		return *this;
	}

	FRigName& operator =(const FString& InNameString)
	{
		SetName(InNameString);
		return *this;
	}

	operator FName() const
	{
		return GetFName();
	}
	
	bool IsValid() const
	{
		return Name.IsSet() || NameString.IsSet();
	}

	bool IsNone() const
	{
		if(!IsValid())
		{
			return true;
		}
		if(Name.IsSet())
		{
			return Name.GetValue().IsNone();
		}
		return GetName().IsEmpty();
	}

	bool operator ==(const FRigName& InOther) const
	{
		return Equals(InOther, ESearchCase::CaseSensitive);
	}

	bool Equals(const FRigName& InOther, const ESearchCase::Type& InCase) const
	{
		if(Name.IsSet() && InOther.Name.IsSet())
		{
			return GetFName().IsEqual(InOther.GetFName(), InCase == ESearchCase::CaseSensitive ? ENameCase::CaseSensitive : ENameCase::IgnoreCase);
		}
		
		if(NameString.IsSet() && InOther.NameString.IsSet())
		{
			return GetName().Equals(InOther.GetName(), InCase);
		}

		if(Name.IsSet() && InOther.NameString.IsSet())
		{
			return GetName().Equals(InOther.GetName(), InCase);
		}

		if(NameString.IsSet() && InOther.Name.IsSet())
		{
			return GetName().Equals(InOther.GetName(), InCase);
		}

		return IsNone() == InOther.IsNone();
	}

	bool operator !=(const FRigName& InOther) const
	{
		return !(*this == InOther);
	}

	int32 Len() const
	{
		if(NameString.IsSet())
		{
			return NameString->Len();
		}
		if(Name.IsSet())
		{
			return (int32)Name->GetStringLength();
		}
		return 0;
	}

	const FName& GetFName() const
	{
		if(Name.IsSet())
		{
			return Name.GetValue();
		}
		if(NameString.IsSet())
		{
			if(!NameString->IsEmpty())
			{
				Name = *NameString.GetValue();
				return Name.GetValue();
			}
		}
		return EmptyName;
	}

	const FString& GetName() const
	{
		if(NameString.IsSet())
		{
			return NameString.GetValue();
		}
		if(Name.IsSet())
		{
			NameString = FString();
			FString& ValueString = NameString.GetValue();
			Name.GetValue().ToString(ValueString);
			return ValueString;
		}
		return EmptyString;
	}

	const FString& ToString() const
	{
		return GetName();
	}

	void SetFName(const FName& InName)
	{
		if(Name.IsSet())
		{
			if(Name.GetValue().IsEqual(InName, ENameCase::CaseSensitive))
			{
				return;
			}
		}
		Name = InName;
		NameString.Reset();
	}

	void SetName(const FString& InNameString)
	{
		if(NameString.IsSet())
		{
			if(NameString.GetValue().Equals(InNameString, ESearchCase::CaseSensitive))
			{
				return;
			}
		}
		Name.Reset();
		NameString = InNameString;
	}

private:

	mutable TOptional<FName> Name;
	mutable TOptional<FString> NameString;

	static UE_API const FString EmptyString;
	static UE_API const FName EmptyName;
};

#undef UE_API
