// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyComponents.h"
#include "RigPhysics.generated.h"

#define UE_API CONTROLRIG_API

// Control Rig core no longer supports simulations. Control Rig Physics (for example) now stores it
// in the solver component.
USTRUCT(meta = (Deprecated = "5.8"))
struct FRigPhysicsSimulationBase
{
	GENERATED_BODY()

	FRigPhysicsSimulationBase(const UScriptStruct* InType = nullptr) : Type(InType) {}

	template<typename T> friend
		T* Cast(FRigPhysicsSimulationBase* BasePtr)
	{
		if (BasePtr && BasePtr->Type == T::StaticStruct())
		{
			return static_cast<T*>(BasePtr);
		}
		return nullptr;
	}

	virtual ~FRigPhysicsSimulationBase() = default;

	const UScriptStruct* Type;
	FRigComponentKey PhysicsSolverComponentKey;
};

// Note - FRigPhysicsSolverID is no long used - solvers are referenced using their component key
USTRUCT(BlueprintType, meta = (Deprecated = "5.6"))
struct FRigPhysicsSolverID
{
public:
	
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta=(ShowOnlyInnerProperties))
	FGuid Guid;

	FRigPhysicsSolverID() : Guid() { }
	explicit FRigPhysicsSolverID(const FGuid& InGuid) : Guid(InGuid) { }

	bool IsValid() const
	{
		return Guid.IsValid();
	}

	FString ToString() const
	{
		return Guid.ToString();
	}

	bool operator == (const FRigPhysicsSolverID& InOther) const
	{
		return Guid == InOther.Guid;
	}

	bool operator != (const FRigPhysicsSolverID& InOther) const
	{
		return !(*this == InOther);
	}

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsSolverID& ID)
	{
		Ar << ID.Guid;
		return Ar;
	}

	friend uint32 GetTypeHash(const FRigPhysicsSolverID& InID)
	{
		return GetTypeHash(InID.Guid);
	}
};

// Note - FRigPhysicsSolverDescription is no long used - solvers are referenced using their component key
USTRUCT()
struct FRigPhysicsSolverDescription
{
public:
	
	GENERATED_BODY()

	UPROPERTY()
	FRigPhysicsSolverID ID;

	UPROPERTY()
	FName Name;

	FRigPhysicsSolverDescription()
	: ID()
	, Name(NAME_None)
	{}
	
	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigPhysicsSolverDescription& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

#undef UE_API
