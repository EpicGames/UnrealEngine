// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

template<typename T>
struct FRigReusableElementStorage
{
private:
	TArray<T> Storage;
	TArray<int32> FreeList;
	bool bFreeListIsSorted = true;

public:
	bool IsValidIndex(const int32& InIndex) const
	{
		return Storage.IsValidIndex(InIndex);
	}

	int32 Num() const
	{
		return Storage.Num();
	}

	void Reset(TFunction<void(int32, T*)> OnDestroyCallback = nullptr)
	{
		if(OnDestroyCallback)
		{
			for(int32 Index = 0; Index < Storage.Num(); Index++)
			{
				OnDestroyCallback(Index, &Storage[Index]);
			}
		}
		Storage.Reset();
		FreeList.Reset();
		bFreeListIsSorted = true;
	}

	void Empty(TFunction<void(int32, T*)> OnDestroyCallback = nullptr)
	{
		if(OnDestroyCallback)
		{
			for(int32 Index = 0; Index < Storage.Num(); Index++)
			{
				OnDestroyCallback(Index, &Storage[Index]);
			}
		}
		Storage.Empty();
		FreeList.Empty();
		bFreeListIsSorted = true;
	}

	void Reserve(int32 InCount)
	{
		check(FreeList.IsEmpty());
		Storage.Reserve(InCount);
	}

	T& operator[](int InIndex)
	{
		return Storage[InIndex];
	}

	const T& operator[](int InIndex) const
	{
		return Storage[InIndex];
	}

	typename TArray<T>::RangedForIteratorType begin() { return Storage.begin(); }
	typename TArray<T>::RangedForIteratorType end() { return Storage.end(); }
	typename TArray<T>::RangedForConstIteratorType begin() const { return Storage.begin(); }
	typename TArray<T>::RangedForConstIteratorType end() const { return Storage.end(); }

	const T* GetData() const
	{
		return Storage.GetData();
	}

	T* GetData()
	{
		return Storage.GetData();
	}

	TArrayView<const T> GetStorage() const
	{
		return TArrayView<const T>(Storage.GetData(), Storage.Num());
	}

	TArrayView<T> GetStorage()
	{
		return TArrayView<T>(Storage.GetData(), Storage.Num());
	}

	TArrayView<const int32> GetFreeList() const
	{
		return TArrayView<const int32>(FreeList.GetData(), FreeList.Num());
	}

	TArrayView<int32> GetFreeList()
	{
		return TArrayView<int32>(FreeList.GetData(), FreeList.Num());
	}

	int32 Add(const T& InDefault)
	{
		TArray<int32, TInlineAllocator<4>> Indices = Allocate(1, InDefault);
		check(Indices.Num() == 1);
		return Indices[0];
	}

	int32 Add(const T&& InDefault)
	{
		TArray<int32, TInlineAllocator<4>> Indices = Allocate(1, InDefault);
		check(Indices.Num() == 1);
		return Indices[0];
	}

	int32 AddZeroed(int32 InNum)
	{
		check(FreeList.IsEmpty());
		return Storage.AddZeroed(InNum);
	}

	int32 AddUninitialized(int32 InNum)
	{
		check(FreeList.IsEmpty());
		return Storage.AddUninitialized(InNum);
	}

	TArray<int32, TInlineAllocator<4>> Allocate(int32 InCount, const T& InDefault, bool bContiguous = false)
	{
		TArray<int32, TInlineAllocator<4>> Indices;

		if(bContiguous)
		{
			if(InCount > 0)
			{
				const int32 FirstIndex = AllocateContiguous(InCount, InDefault);
				const int32 UpperBound = FirstIndex + InCount;
				Indices.Reserve(InCount);
				for(int32 Index = FirstIndex; Index < UpperBound; Index++)
				{
					Indices.Add(Index);
				}
			}
			return Indices;
		}

		const int32 NumToAllocate = InCount - FMath::Min(InCount, FreeList.Num());
		if(NumToAllocate > 0)
		{
			Storage.Reserve(Storage.Num() + NumToAllocate);
		}

		Indices.Reserve(InCount);
		for(int32 Index = 0; Index < InCount; Index++)
		{
			if(FreeList.IsEmpty())
			{
				Indices.Push(Storage.Add(InDefault));
			}
			else
			{
				Indices.Push(FreeList.Pop(EAllowShrinking::No));
				Storage[Indices.Last()] = InDefault;
			}
		}

		return Indices;
	}

	int32 AllocateContiguous(int32 InCount, const T& InDefault)
	{
		int32 FirstIndex = INDEX_NONE;
		if(InCount == 0)
		{
			return FirstIndex;
		}

		// if the freelist has enough room for us to allocate
		// let's look for a candidate
		if(FreeList.Num() >= InCount)
		{
			if(!bFreeListIsSorted)
			{
				FreeList.Sort();
				bFreeListIsSorted = true;
			}
			
			FirstIndex = FreeList[0];
			int32 Remainder = InCount - 1;

			for(int32 Index = 1; Index < FreeList.Num(); Index++)
			{
				if(Remainder == 0)
				{
					break;
				}
				
				// make sure they are consecutive
				if(FreeList[Index - 1] == FreeList[Index] - 1)
				{
					Remainder--;
				}
				else
				{
					FirstIndex = FreeList[Index];
					Remainder = InCount - 1;
				}
			}

			// if we didn't find enough room for all elements....
			if(Remainder > 0)
			{
				FirstIndex = INDEX_NONE;
			}
		}

		if(FirstIndex == INDEX_NONE)
		{
			FirstIndex = Storage.AddUninitialized(InCount);
		}
		else
		{
			const int32 UpperBound = FirstIndex + InCount;
			FreeList.RemoveAll([FirstIndex, UpperBound](int32 Index) -> bool
			{
				return (Index >= FirstIndex) && (Index < UpperBound);
			});
		}

		const int32 UpperBound = FirstIndex + InCount;
		for(int32 Index = FirstIndex; Index < UpperBound; Index++)
		{
			Storage[Index] = InDefault;
		}

		return FirstIndex;
	}

	void Deallocate(const int32& InIndex)
	{
		int32 MutableIndex = InIndex;
		Deallocate(MutableIndex, nullptr);
	}

	void Deallocate(int32& InIndex, T** InStorage)
	{
		if(InIndex == INDEX_NONE)
		{
			return;
		}
#if WITH_EDITOR
		check(Storage.IsValidIndex(InIndex));
		check(!FreeList.Contains(InIndex));
#endif
		if(bFreeListIsSorted && !FreeList.IsEmpty())
		{
			if(FreeList.Last() > InIndex)
			{
				bFreeListIsSorted = false;
			}
		}
		FreeList.Add(InIndex);
		InIndex = INDEX_NONE;
		if(InStorage)
		{
			*InStorage = nullptr;
		}
	}

	void Deallocate(const TConstArrayView<int32>& InIndices)
	{
		if(InIndices.IsEmpty())
		{
			return;
		}
		FreeList.Reserve(FreeList.Num() + InIndices.Num());
		for(int32 Index : InIndices)
		{
			if(Index != INDEX_NONE && !FreeList.Contains(Index))
			{
				Deallocate(Index);
			}
		}
	}

	void Deallocate(int32 InStartIndex, int32 InCount)
	{
		const int32 UpperBound = InStartIndex + InCount;
		const int32 EndIndex = UpperBound - 1;
		
		if(InCount == 0 || !Storage.IsValidIndex(InStartIndex) || !Storage.IsValidIndex(EndIndex))
		{
			return;
		}
		
		FreeList.Reserve(FreeList.Num() + InCount);
		for(int32 Index = InStartIndex; Index < UpperBound; Index++)
		{
			if(Index != INDEX_NONE && !FreeList.Contains(Index))
			{
				Deallocate(Index);
			}
		}
	}

	template<typename OwnerType>
	void Deallocate(OwnerType* InOwner)
	{
		check(InOwner);
		Deallocate(InOwner->StorageIndex, &InOwner->Storage);
	}

	bool Contains(int32 InIndex, const T* InStorage)
	{
		if(!IsValidIndex(InIndex))
		{
			return false;
		}
		return GetData() + InIndex == InStorage;
	}

	template<typename OwnerType>
	bool Contains(const OwnerType* InOwner)
	{
		check(InOwner);
		return Contains(InOwner->StorageIndex, InOwner->Storage);
	}

	TMap<int32, int32> Shrink(TFunction<void(int32, T&)> OnDestroyCallback = nullptr)
	{
		TMap<int32, int32> OldToNew;
		
		if(!FreeList.IsEmpty())
		{
			
			TArray<bool> ToRemove;

			if(FreeList.Num() != Storage.Num() || OnDestroyCallback != nullptr)
			{
				ToRemove.AddZeroed(Storage.Num());
				for(int32 FreeIndex : FreeList)
				{
					ToRemove[FreeIndex] = true;
					if(OnDestroyCallback)
					{
						OnDestroyCallback(FreeIndex, Storage[FreeIndex]);
					}
				}
			}

			if(FreeList.Num() != Storage.Num())
			{
				const int32 NewNum = Storage.Num() - FreeList.Num();
				OldToNew.Reserve(NewNum);
				
				TArray<T> NewStorage;
				NewStorage.Reserve(FMath::Max(NewNum, 0));
				for(int32 OldIndex = 0; OldIndex < Storage.Num(); OldIndex++)
				{
					if(!ToRemove[OldIndex])
					{
						const int32 NewIndex = NewStorage.Add(Storage[OldIndex]);
						if(OldIndex != NewIndex)
						{
							OldToNew.Add(OldIndex, NewIndex);
						}
					}
				}
				Storage = MoveTemp(NewStorage);
			}
			else
			{
				Storage.Reset();
			}

			FreeList.Reset();
		}

		FreeList.Shrink();
		Storage.Shrink();

		return OldToNew;
	}

	friend class URigHierarchy;
};
