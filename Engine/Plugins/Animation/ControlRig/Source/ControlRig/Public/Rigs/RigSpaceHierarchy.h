// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigHierarchyDefines.h"
#include "RigHierarchyPose.h"
#include "RigSpaceHierarchy.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
struct FRigHierarchyContainer;

UENUM(BlueprintType)
enum class ERigSpaceType : uint8
{
	/** Not attached to anything */
	Global,

	/** Attached to a bone */
	Bone,

	/** Attached to a control */
	Control,

	/** Attached to a space*/
	Space
};

USTRUCT()
struct FRigSpace : public FRigElement
{
	GENERATED_BODY()

	FRigSpace()
		: FRigElement()
		, SpaceType(ERigSpaceType::Global)
		, ParentName(NAME_None)
		, ParentIndex(INDEX_NONE)
		, InitialTransform(FTransform::Identity)
		, LocalTransform(FTransform::Identity)
	{
	}
	virtual ~FRigSpace() = default;

	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category = FRigElement)
	ERigSpaceType SpaceType;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = FRigElement)
	FName ParentName;

	UPROPERTY(BlueprintReadOnly, transient, Category = FRigElement)
	int32 ParentIndex;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = FRigElement)
	FTransform InitialTransform;

	UPROPERTY(BlueprintReadOnly, transient, EditAnywhere, Category = FRigElement)
	FTransform LocalTransform;

	virtual ERigElementType GetElementType() const override
	{
		return ERigElementType::Null;
	}

	virtual FRigElementKey GetParentElementKey() const
	{
		switch (SpaceType)
		{
			case ERigSpaceType::Bone:
			{
				return FRigElementKey(ParentName, ERigElementType::Bone);
			}
			case ERigSpaceType::Control:
			{
				return FRigElementKey(ParentName, ERigElementType::Control);
			}
			case ERigSpaceType::Space:
			{
				return FRigElementKey(ParentName, ERigElementType::Null);
			}
			default:
			{
				break;
			}
		}
		return FRigElementKey();
	}
};

USTRUCT()
struct FRigSpaceHierarchy
{
	GENERATED_BODY()

	UE_API FRigSpaceHierarchy();

	TArray<FRigSpace>::RangedForIteratorType      begin()       { return Spaces.begin(); }
	TArray<FRigSpace>::RangedForConstIteratorType begin() const { return Spaces.begin(); }
	TArray<FRigSpace>::RangedForIteratorType      end()         { return Spaces.end();   }
	TArray<FRigSpace>::RangedForConstIteratorType end() const   { return Spaces.end();   }

	UE_API FRigSpace& Add(const FName& InNewName, ERigSpaceType InSpaceType, const FName& InParentName, const FTransform& InTransform);

	// Pretty weird that this type is copy/move assignable (needed for USTRUCTs) but not copy/move constructible
	FRigSpaceHierarchy(FRigSpaceHierarchy&& InOther) = delete;
	FRigSpaceHierarchy(const FRigSpaceHierarchy& InOther) = delete;
	FRigSpaceHierarchy& operator=(FRigSpaceHierarchy&& InOther) = default;
	FRigSpaceHierarchy& operator=(const FRigSpaceHierarchy& InOther) = default;

private:
	UPROPERTY(EditAnywhere, Category = FRigSpaceHierarchy)
	TArray<FRigSpace> Spaces;

	friend struct FRigHierarchyContainer;
	friend struct FCachedRigElement;
	friend class UControlRigHierarchyModifier;
	friend class UControlRig;
};

#undef UE_API
