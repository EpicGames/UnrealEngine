// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This Instance only contains one AnimationAsset, and produce poses
 * Used by Preview in AnimGraph, Playing single animation in Kismet2 and etc
 */

#pragma once
#include "Animation/AnimInstance.h"
#include "SequencerAnimationSupport.h"
#include "AnimNode_ControlRigBase.h"
#include "ControlRigLayerInstance.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;

UCLASS(MinimalAPI, transient, NotBlueprintable)
class UControlRigLayerInstance : public UAnimInstance, public ISequencerAnimationSupport
{
	GENERATED_UCLASS_BODY()

public:
	/** ControlRig related support */
	UE_API void AddControlRigTrack(int32 ControlRigID, UControlRig* InControlRig);
	UE_API void UpdateControlRigTrack(int32 ControlRigID, float Weight, const FControlRigIOSettings& InputSettings, bool bExecute);
	UE_API void RemoveControlRigTrack(int32 ControlRigID);
	UE_API bool HasControlRigTrack(int32 ControlRigID);
	UE_API void ResetControlRigTracks();

	/** Sequencer AnimInstance Interface */
	UE_API void AddAnimation(int32 SequenceId, UAnimSequenceBase* InAnimSequence);
	UE_API virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InPosition, float Weight, bool bFireNotifies) override;
	UE_API virtual void UpdateAnimTrack(UAnimSequenceBase* InAnimSequence, int32 SequenceId, float InFromPosition, float InToPosition, float Weight, bool bFireNotifies) override;
	UE_API void RemoveAnimation(int32 SequenceId);

	/** Construct all nodes in this instance */
	UE_API virtual void ConstructNodes() override;
	/** Reset all nodes in this instance */
	UE_API virtual void ResetNodes() override;
	/** Reset the pose in this instance*/
	UE_API virtual void ResetPose() override;
	/** Saved the named pose to restore after */
	UE_API virtual void SavePose() override;

	/** Return the first available control rig */
	UE_API UControlRig* GetFirstAvailableControlRig() const;

	UFUNCTION(BlueprintPure, Category = "Animation|ControlRigLayer")
	UE_API virtual UAnimInstance* GetSourceAnimInstance() override;
	UE_API virtual void SetSourceAnimInstance(UAnimInstance* SourceAnimInstance) override;
	virtual bool DoesSupportDifferentSourceAnimInstance() const override { return true; }

#if WITH_EDITOR
	UE_API virtual void HandleObjectsReinstanced(const TMap<UObject*, UObject*>& OldToNewInstanceMap) override;
#endif

protected:
	// UAnimInstance interface
	UE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
public:
	UE_API virtual void NativeUpdateAnimation(float DeltaSeconds) override;

public:
	static UE_API const FName SequencerPoseName;
};

#undef UE_API
