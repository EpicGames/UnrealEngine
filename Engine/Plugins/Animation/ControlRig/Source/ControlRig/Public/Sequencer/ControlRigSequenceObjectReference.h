// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/LazyObjectPtr.h"
#include "ControlRig.h"
#include "ControlRigSequenceObjectReference.generated.h"


/**
 * An external reference to an level sequence object, resolvable through an arbitrary context.
 */
USTRUCT()
struct FControlRigSequenceObjectReference
{
	GENERATED_BODY()

	/**
	 * Default construction to a null reference
	 */
	FControlRigSequenceObjectReference() {}

	/**
	 * Generates a new reference to an object within a given context.
	 *
	 * @param InControlRig The ControlRig to create a reference for
	 */
	CONTROLRIG_API static FControlRigSequenceObjectReference Create(UControlRig* InControlRig);

	/**
	 * Check whether this object reference is valid or not
	 */
	bool IsValid() const
	{
		return ControlRigAssetReference.IsValid();
	}

	/**
	 * Equality comparator
	 */
	friend bool operator==(const FControlRigSequenceObjectReference& A, const FControlRigSequenceObjectReference& B)
	{
		return A.ControlRigAssetReference == B.ControlRigAssetReference;
	}
	
	void PostSerialize(const FArchive& Ar)
	{
		if (ControlRigClass_DEPRECATED)
		{
			ControlRigAssetReference.Set(ControlRigClass_DEPRECATED);
			ControlRigClass_DEPRECATED = nullptr;
		}
	}

private:

	/** The type of this animation ControlRig */
	UPROPERTY(meta=(DeprecatedProperty))
	TSubclassOf<UControlRig> ControlRigClass_DEPRECATED;
	
	UPROPERTY()
	FControlRigAssetStrongReference ControlRigAssetReference;
};

template<>
struct TStructOpsTypeTraits<FControlRigSequenceObjectReference> : public TStructOpsTypeTraitsBase2<FControlRigSequenceObjectReference>
{
	enum
	{
		WithPostSerialize = true,
	};
};

USTRUCT()
struct FControlRigSequenceObjectReferences
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FControlRigSequenceObjectReference> Array;
};

USTRUCT()
struct FControlRigSequenceObjectReferenceMap
{
	GENERATED_BODY()

	/**
	 * Check whether this map has a binding for the specified object id
	 * @return true if this map contains a binding for the id, false otherwise
	 */
	bool HasBinding(const FGuid& ObjectId) const;

	/**
	 * Remove a binding for the specified ID
	 *
	 * @param ObjectId	The ID to remove
	 */
	void RemoveBinding(const FGuid& ObjectId);

	/**
	 * Create a binding for the specified ID
	 *
	 * @param ObjectId				The ID to associate the component with
	 * @param ObjectReference	The component reference to bind
	 */
	void CreateBinding(const FGuid& ObjectId, const FControlRigSequenceObjectReference& ObjectReference);

private:
	
	UPROPERTY()
	TArray<FGuid> BindingIds;

	UPROPERTY()
	TArray<FControlRigSequenceObjectReferences> References;
};
