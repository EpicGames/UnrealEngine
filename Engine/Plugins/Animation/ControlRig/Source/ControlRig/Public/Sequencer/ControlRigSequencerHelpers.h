// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CONTROLRIG_API

struct FChannelMapInfo;
class UControlRig;
class UMovieSceneControlRigParameterSection;
struct FMovieSceneFloatChannel;
struct FMovieSceneBoolChannel;
struct FMovieSceneIntegerChannel;
struct FMovieSceneByteChannel;
class UMovieSceneSection;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneSequence;

struct FControlRigSequencerHelpers
{
	static UE_API TPair<const FChannelMapInfo*, int32> GetInfoAndNumFloatChannels(
		const UControlRig* InControlRig,
		const FName& InControlName,
		const UMovieSceneControlRigParameterSection* InSection);

	static UE_API TArrayView<FMovieSceneFloatChannel*> GetFloatChannels(const UControlRig* InControlRig,
		const FName& InControlName, const UMovieSceneSection* InSection);

	static UE_API TArrayView<FMovieSceneBoolChannel*> GetBoolChannels(const UControlRig* InControlRig,
		const FName& InControlName, const UMovieSceneSection* InSection);

	static UE_API TArrayView<FMovieSceneByteChannel*> GetByteChannels(const UControlRig* InControlRig,
		const FName& InControlName, const UMovieSceneSection* InSection);

	static UE_API TArrayView<FMovieSceneIntegerChannel*> GetIntegerChannels(const UControlRig* InControlRig,
		const FName& InControlName, const UMovieSceneSection* InSection);

	static UE_API UMovieSceneControlRigParameterTrack*  FindControlRigTrack(UMovieSceneSequence* InSequencer, const UControlRig* InControlRig);

};

#undef UE_API
