// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/QualifiedFrameTime.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "UObject/WeakObjectPtr.h"

struct FMovieSceneControlRigSpaceChannel;
struct FMovieSceneControlRigSpaceBaseKey;

class UControlRig;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;


namespace UE::MovieScene
{

struct FAccumulatedControlEntryIndex;

/** Entity type used for encoding Control Rig entities during import */
enum class EControlRigEntityType : uint8
{
	Base,
	Space,
	BoolParameter,
	EnumParameter,
	IntegerParameter,
	ScalarParameter,
	VectorParameter,
	TransformParameter,
};

/** Encodes a Control Rig entity ID with type in upper 8 bits, index in lower 24 bits */
inline uint32 EncodeControlRigEntityID(int32 InIndex, EControlRigEntityType InType)
{
	check(InIndex >= 0 && InIndex < int32(0x00FFFFFF));
	return static_cast<uint32>(InIndex) | (uint32(InType) << 24);
}

/** Decodes a Control Rig entity ID into its index and type components */
inline void DecodeControlRigEntityID(uint32 InEntityID, int32& OutIndex, EControlRigEntityType& OutType)
{
	OutIndex = static_cast<int32>(InEntityID & 0x00FFFFFF);
	OutType = static_cast<EControlRigEntityType>(InEntityID >> 24);
}

/** Component data present on all base and parameter Control Rig entities */
struct FControlRigSourceData
{
	UMovieSceneControlRigParameterTrack* Track = nullptr;

	// The section this entity was imported from. Used to detect stale entities
	// when procedural rigs modify the section during binding.
	TWeakObjectPtr<UMovieSceneControlRigParameterSection> Section;

	// Hash of the section's LastControlsUsedToReconstruct at import time.
	// If this doesn't match the section's current hash, the entity is stale.
	uint32 ImportedControlsHash = 0;

	// The encoded entity ID from import (type in upper 8 bits, index in lower 24 bits).
	// Used to look up the correct channel when repairing stale pointers.
	uint32 ImportedEntityID = 0;
};

/**
 * Component that exists for base-eval control rig entities
 */
struct FBaseControlRigEvalData
{
	UMovieSceneControlRigParameterSection* Section = nullptr;

	TWeakObjectPtr<UControlRig> WeakControlRig;
	FFrameTime WarpedTime;  //time after timewarp
	uint8 bIsActive : 1 = true;
	uint8 bHasWeight : 1 = false;
	uint8 bWasDoNotKey : 1 = false;
	uint8 bAnimMixerPoseProducer : 1 = false;
};

/**
 * Singleton Control Rig component types
 */
struct FControlRigComponentTypes
{
public:
	CONTROLRIG_API static FControlRigComponentTypes* Get();
	CONTROLRIG_API static void Destroy();

	TComponentTypeID<FControlRigSourceData> ControlRigSource;

	TComponentTypeID<FBaseControlRigEvalData> BaseControlRigEvalData;

	TComponentTypeID<FAccumulatedControlEntryIndex> AccumulatedControlEntryIndex;

	TComponentTypeID<const FMovieSceneControlRigSpaceChannel*> SpaceChannel;
	TComponentTypeID<FMovieSceneControlRigSpaceBaseKey> SpaceResult;

	struct
	{
		FComponentTypeID BaseControlRig;
		FComponentTypeID ControlRigParameter;
		FComponentTypeID Space;

		FComponentTypeID IgnoredBaseControlRig;
	} Tags;

private:
	CONTROLRIG_API FControlRigComponentTypes();
};


} // namespace UE::MovieScene

