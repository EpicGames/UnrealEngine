// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneSequenceInstanceData.h"
#include "Evaluation/MovieSceneEvaluationOperand.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneControlRigInstanceData.generated.h"

USTRUCT()
struct FMovieSceneControlRigInstanceData : public FMovieSceneSequenceInstanceData
{
	GENERATED_BODY()

	/** Blend this track in additively (using the reference pose as a base) */
	UPROPERTY()
	bool bAdditive = false;

	/** Only apply bones that are in the filter */
	UPROPERTY()
	bool bApplyBoneFilter = false;

	/** Per-bone filter to apply to our animation */
	UPROPERTY()
	FInputBlendPose BoneFilter;

	/** The weight curve for this animation controller section */
	UPROPERTY()
	FMovieSceneFloatChannel Weight;

	/** The operand the control rig instance should operate on */
	UPROPERTY()
	FMovieSceneEvaluationOperand Operand;

private:

	/**
	 * Implemented in derived types to retrieve the type of this struct
	 */
	virtual UScriptStruct& GetScriptStructImpl() const { return *StaticStruct(); }
};