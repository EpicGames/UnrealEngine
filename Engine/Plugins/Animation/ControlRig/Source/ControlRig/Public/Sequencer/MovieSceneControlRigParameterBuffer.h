// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ControlRig.h"
#include "MovieSceneControlRigComponentTypes.h"
#include "Rigs/RigHierarchyDefines.h"

#define UE_API CONTROLRIG_API

struct FRigControlElement;
struct FMovieSceneControlRigSpaceBaseKey;

class UMovieSceneControlRigParameterTrack;

namespace UE::MovieScene
{

struct FPreAnimatedControlRigParameterStorage;

enum class EControlRigControlType : uint8
{
	Space,
	Parameter_Bool,
	Parameter_Enum,
	Parameter_Integer,
	Parameter_Scalar,
	Parameter_Vector,
	Parameter_Transform,

	Num,
};

namespace Private
{
	/** Template specializations that produce the correct enum value for a given c++ control type */
	template<typename T>
	static constexpr EControlRigControlType ControlRigTypeFromNativeType()
	{
		static_assert(!std::is_same_v<T, T>, "This type is not supported");
		return EControlRigControlType::Num;
	}

	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<FMovieSceneControlRigSpaceBaseKey>(){ return EControlRigControlType::Space; }
	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<bool>()                             { return EControlRigControlType::Parameter_Bool; }
	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<uint8>()                            { return EControlRigControlType::Parameter_Enum; }
	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<int32>()                            { return EControlRigControlType::Parameter_Integer; }
	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<float>()                            { return EControlRigControlType::Parameter_Scalar; }
	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<FVector3f>()                        { return EControlRigControlType::Parameter_Vector; }
	template<> constexpr EControlRigControlType ControlRigTypeFromNativeType<FEulerTransform>()                  { return EControlRigControlType::Parameter_Transform; }
}


/**
 * Enumeration that determines the stability of entries allocated within a FControlRigParameterBuffer.
 * Depending on the use case of the buffer, it can be preferable to have entries stable or unstable.
 * Typically:
 *   - Stable is used when the buffer is used as a container that is only populated and assigned to rig.
 *   - Unstable is used when parameter values need to be retrieved from the buffer by name.
 */
enum class EControlRigParameterBufferIndexStability
{
	/** Once allocated, the name:value index must not be changed. Typically used for fast, index-based assignment. */
	Stable,
	/** Entries are freely relocatable: allows for more efficient binary searching of parameters typically required in editor or introspection code. */
	Unstable,
};

/**
 * Map-like structure containing key/value pairs of FName:T using a binary-search.
 * Parameter names and values are allocated contiguously rather than as pairs.
 */
struct FControlRigParameterValueHeader
{
	/**
	 * Construct this header for a specific control type
	 * */
	UE_API FControlRigParameterValueHeader(EControlRigControlType InType, EControlRigParameterBufferIndexStability IndexStability);

	/**
	 * Move construction/assignment
	 */
	UE_API FControlRigParameterValueHeader(FControlRigParameterValueHeader&& RHS);
	UE_API FControlRigParameterValueHeader& operator=(FControlRigParameterValueHeader&& RHS);

	/**
	 * Copy construction/assignment
	 */
	UE_API FControlRigParameterValueHeader(const FControlRigParameterValueHeader& RHS);
	UE_API FControlRigParameterValueHeader& operator=(const FControlRigParameterValueHeader& RHS);

	/**
	 * Destructor
	 */
	UE_API ~FControlRigParameterValueHeader();

	UE_API void Reset();
	UE_API EControlRigControlType GetType() const;
	UE_API int32 Num() const;
	UE_API void Apply(UControlRig* Rig) const;
	UE_API void Apply(UControlRig* Rig, FName Name, int32 Index, const void* ParameterBuffer) const;
	UE_API void ApplyAndRemove(UControlRig* Rig, FName InName);

	UE_API TArrayView<const FName> GetNames() const;

	UE_API uint8* GetParameterBuffer() const;

	UE_API void* GetParameter(int32 Index) const;

	template<typename T>
	const T* GetParameterBuffer() const
	{
		check(Private::ControlRigTypeFromNativeType<T>() == Type);
		return reinterpret_cast<const T*>(GetParameterBuffer());
	}

	template<typename T>
	T* GetParameterBuffer()
	{
		check(Private::ControlRigTypeFromNativeType<T>() == Type);
		return reinterpret_cast<T*>(GetParameterBuffer());
	}

	template<typename T>
	TArrayView<const T> GetParameters() const
	{
		return TArrayView<const T>(GetParameterBuffer<T>(), NumElements);
	}

	template<typename T>
	TArrayView<T> GetParameters()
	{
		return TArrayView<T>(GetParameterBuffer<T>(), NumElements);
	}

	UE_API int32 Add_GetIndex(FName InName);

	UE_API void* Add_GetPtr(FName InName);

	UE_API void Add(FName InName, const void* Value);

	UE_API void Remove(FName InName);

	UE_API bool Contains(FName InName) const;

	UE_API int32 Find(FName InName) const;

	UE_API void OptimizeForLookup();
	UE_API void Resort();

private:
	UE_API TArrayView<FName> GetMutableNames();
	UE_API void RemoveAt(int32 Index);
	UE_API void InsertDefaulted(FName Name, int32 Index);
	static UE_API uint8* GetParameterBuffer(uint8* BasePtr, uint16 Capacity, uint8 InAlignment);
	UE_API size_t GetParameterSize() const;
	UE_API void Reserve(size_t NewCapacity);

private:

	uint8* Data = nullptr;

	uint16 Capacity;
	uint16 NumElements;

	uint8 Alignment : 7;
	uint8 bStableIndices : 1;

	EControlRigControlType Type;
};

/**
 * Reflects a single value within a FControlRigParameterValues structure, allowing for type-safe casting and checking.
 * A FControlRigValueView is only valid as long its owning header remains unmodified. Any insertion, removal or sorting
 * will invalidate all views.
 */
struct FControlRigValueView
{
	FControlRigValueView()
		: Value(nullptr)
		, Type(EControlRigControlType::Num)
	{}
	FControlRigValueView(void* InValue, EControlRigControlType InType)
		: Value(InValue)
		, Type(InType)
	{}

	/**
	 * Test to see if this value is valid
	 */
	explicit operator bool() const
	{
		return Value && Type != EControlRigControlType::Num;
	}

	/**
	 * Cast this view to a specific value, returning nullptr if the cast is invalid.
	 */
	template<typename T>
	T* Cast() const
	{
		if (Type == Private::ControlRigTypeFromNativeType<T>())
		{
			return static_cast<T*>(Value);
		}
		return nullptr;
	}

	/**
	 * Get the type of this value. Must be checked for valididity first.
	 */
	EControlRigControlType GetType() const
	{
		check(*this);
		return Type;
	}

private:

	void* Value;
	EControlRigControlType Type;
};


/**
 * Parameter buffer structure containing any number of control values arranged by type and name
 */
struct FControlRigParameterValues
{
	UE_API FControlRigParameterValues(EControlRigParameterBufferIndexStability IndexStability);

	UE_API FControlRigParameterValueHeader& GetHeader(EControlRigControlType InType);
	UE_API const FControlRigParameterValueHeader& GetHeader(EControlRigControlType InType) const;


	template<typename T>
	T* Add(FName InName)
	{
		FControlRigParameterValueHeader& Header = GetHeader(Private::ControlRigTypeFromNativeType<T>());
		return static_cast<T*>(Header.Add_GetPtr(InName));
	}

	template<typename T>
	void Add(FName InName, T&& InValue)
	{
		using DecayedType = std::decay_t<T>;
		*Add<DecayedType>(InName) = Forward<T>(InValue);
	}
	template<typename T>
	const T* Find(FName InName) const
	{
		const FControlRigParameterValueHeader& Header = GetHeader(Private::ControlRigTypeFromNativeType<T>());
		const int32 Index = Header.Find(InName);
		if (Index != INDEX_NONE)
		{
			return &Header.GetParameters<T>()[Index];
		}
		return nullptr;
	}

	template<typename T>
	bool Find(FName InName, T& OutValue) const
	{
		const FControlRigParameterValueHeader& Header = GetHeader(Private::ControlRigTypeFromNativeType<T>());
		const int32 Index = Header.Find(InName);
		if (Index != INDEX_NONE)
		{
			OutValue = Header.GetParameters<T>()[Index];
			return true;
		}
		return false;
	}

	UE_API FControlRigValueView Find(FName Name) const;
	UE_API FControlRigValueView Find(FName Name, EControlRigControlType ControlRigType) const;
	UE_API FControlRigValueView FindParameter(FName Name) const;

	UE_API ~FControlRigParameterValues();

	UE_API void CopyFrom(const FControlRigParameterValues& Other, FName ControlName);

	UE_API void AddCurrentValue(UControlRig* Rig, FRigControlElement* ControlElement);

	UE_API void ApplyTo(UControlRig* Rig) const;

	UE_API void ApplyAndRemove(UControlRig* Rig, FName Name);

	UE_API void PopulateFrom(UControlRig* Rig);

	UE_API bool InitializeParameters(UControlRig* Rig, FPreAnimatedControlRigParameterStorage& Storage);

	UE_API void OptimizeForLookup();

	UE_API void Reset();

private:

	FControlRigParameterValueHeader Headers[(uint8)EControlRigControlType::Num];
};


struct FAccumulatedControlEntryIndex
{
	FAccumulatedControlEntryIndex() : EntryIndex(uint16(-1)), AccumulatorIndex(uint16(-1)) {};
	FAccumulatedControlEntryIndex(uint16 InEntryIndex, uint16 InAccumulatorIndex, EControlRigControlType InControlType)
		: EntryIndex(InEntryIndex)
		, AccumulatorIndex(InAccumulatorIndex) 
		, ControlType(InControlType)
	{};

	bool IsValid() const
	{
		return (EntryIndex != (uint16(-1)) && AccumulatorIndex != (uint16(-1)));
	}

	uint16 EntryIndex;
	uint16 AccumulatorIndex;
	EControlRigControlType ControlType;
};

struct FControlRigParameterBuffer
{
	UE_API FControlRigParameterBuffer(UControlRig* InRig, EControlRigParameterBufferIndexStability IndexStability);

	TWeakObjectPtr<UControlRig> WeakControlRig;
	FControlRigParameterValues Values;

	UE_API void AddCurrentValue(UControlRig* Rig, FRigControlElement* ControlElement);

	UE_API void Apply() const;
	UE_API void Populate();
};


struct FAccumulatedControlRigValues
{
	void Apply() const;

	UControlRig* FindControlRig(FAccumulatedControlEntryIndex Entry) const;
	UControlRig* FindControlRig(UMovieSceneControlRigParameterTrack* Track) const;
	const FControlRigParameterBuffer* FindParameterBuffer(UMovieSceneControlRigParameterTrack* Track) const;

	bool DoesEntryExistForTrack(UMovieSceneControlRigParameterTrack* Track);
	int32 InitializeRig(UMovieSceneControlRigParameterTrack* Track, UControlRig* InRig);
	void MarkAsActive(int32 Index);

	FAccumulatedControlEntryIndex AllocateEntryIndex(UMovieSceneControlRigParameterTrack* Track, FName Name, EControlRigControlType Type);

	void InitializeParameters(FPreAnimatedControlRigParameterStorage& Storage);

	void Store(FAccumulatedControlEntryIndex Entry, FMovieSceneControlRigSpaceBaseKey InValue);
	void Store(FAccumulatedControlEntryIndex Entry, bool InValue);
	void Store(FAccumulatedControlEntryIndex Entry, uint8 InValue);
	void Store(FAccumulatedControlEntryIndex Entry, int32 InValue);
	void Store(FAccumulatedControlEntryIndex Entry, float InValue);
	void Store(FAccumulatedControlEntryIndex Entry, FVector3f InValue);
	void Store(FAccumulatedControlEntryIndex Entry, FEulerTransform InValue);

	void Compact();

	void PrimeForInstantiation();

private:

	struct FEntry : FControlRigParameterBuffer
	{
		FEntry(UMovieSceneControlRigParameterTrack* InTrack, UControlRig* InRig);

		TObjectPtr<UMovieSceneControlRigParameterTrack> Track;
		bool bIsActive = false;
	};

	void* GetData(FAccumulatedControlEntryIndex Entry);

	TMap<FObjectKey, int32> ParameterValuesByTrack;
	TArray<FEntry> ValuesArray;
	int32 NextValidIndex = 0;
};


} // namespace UE::MovieScene

#undef UE_API
