// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConstraintsManager.h"
#include "Sections/MovieSceneParameterSection.h"
#include "Sections/MovieSceneConstrainedSection.h"
#include "UObject/ObjectMacros.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Sections/MovieSceneSubSection.h"
#include "ControlRig.h"
#include "MovieSceneSequencePlayer.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "MovieSceneObjectBindingID.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Sequencer/MovieSceneControlRigSpaceChannel.h"
#include "ConstraintChannel.h"
#include "KeyParams.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneControlRigParameterSection.generated.h"

#define UE_API CONTROLRIG_API

class UAnimSequence;
class USkeletalMeshComponent;
struct FKeyDataOptimizationParams;

namespace UE::MovieScene
{

	enum class EControlRigControlType : uint8;

	struct FControlRigChannelMetaData
	{
		CONTROLRIG_API FControlRigChannelMetaData();
		CONTROLRIG_API FControlRigChannelMetaData(EControlRigControlType InType, FName InControlName, int32 InIndexWithinControl, uint32 InEntitySystemID);

		CONTROLRIG_API explicit operator bool() const;

		EControlRigControlType GetType() const
		{
			check((bool)*this);
			return Type;
		}

		FName GetControlName() const
		{
			return ControlName;
		}

		int32 GetChannelIndex() const
		{
			return IndexWithinControl;
		}

		uint32 GetEntitySystemID() const
		{
			return EntitySystemID;
		}

	private:
		EControlRigControlType Type;
		FName ControlName;
		int32 IndexWithinControl;
		uint32 EntitySystemID;
	};

} // namespace UE::MovieScene

struct FControlRigBindingHelper
{
	static UE_API bool BindToSequencerInstance(UControlRig* ControlRig);
	static UE_API void UnBindFromSequencerInstance(UControlRig* ControlRig);
};

struct FEnumParameterNameAndValue //uses uint8
{
	FEnumParameterNameAndValue(FName InParameterName, uint8 InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	FName ParameterName;

	uint8 Value;
};

struct FIntegerParameterNameAndValue
{
	FIntegerParameterNameAndValue(FName InParameterName, int32 InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	FName ParameterName;

	int32 Value;
};

USTRUCT()
struct FEnumParameterNameAndCurve : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FEnumParameterNameAndCurve()
	{}

	FEnumParameterNameAndCurve(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	UPROPERTY()
	FMovieSceneByteChannel ParameterCurve;
};

USTRUCT()
struct FIntegerParameterNameAndCurve : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FIntegerParameterNameAndCurve()
	{}

	FIntegerParameterNameAndCurve(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}
	
	UPROPERTY()
	FMovieSceneIntegerChannel ParameterCurve;
};

USTRUCT()
struct FSpaceControlNameAndChannel
{
	GENERATED_USTRUCT_BODY()

	FSpaceControlNameAndChannel(){}
	FSpaceControlNameAndChannel(FName InControlName) : ControlName(InControlName) {};

	UPROPERTY()
	FName ControlName;

	UPROPERTY()
	FMovieSceneControlRigSpaceChannel SpaceCurve;
};


/**
*  Data that's queried during an interrogation
*/
struct FFloatInterrogationData
{
	float Val;
	FName ParameterName;
};

struct FVector2DInterrogationData
{
	FVector2D Val;
	FName ParameterName;
};

struct FVectorInterrogationData
{
	FVector Val;
	FName ParameterName;
};

struct FEulerTransformInterrogationData
{
	FEulerTransform Val;
	FName ParameterName;
};

USTRUCT()
struct FChannelMapInfo
{
	GENERATED_USTRUCT_BODY()

	FChannelMapInfo() = default;

	FChannelMapInfo(int32 InControlIndex, int32 InTotalChannelIndex, int32 InChannelIndex, int32 InParentControlIndex = INDEX_NONE, FName InChannelTypeName = NAME_None,
		int32 InMaskIndex = INDEX_NONE, int32 InCategoryIndex = INDEX_NONE) :
		ControlIndex(InControlIndex),TotalChannelIndex(InTotalChannelIndex), ChannelIndex(InChannelIndex), ParentControlIndex(InParentControlIndex), ChannelTypeName(InChannelTypeName),MaskIndex(InMaskIndex),CategoryIndex(InCategoryIndex) {};
	UPROPERTY()
	int32 ControlIndex = 0; 
	UPROPERTY()
	int32 TotalChannelIndex = 0;
	UPROPERTY()
	int32 ChannelIndex = 0; //channel index for it's type.. (e.g  float, int, bool).
	UPROPERTY()
	int32 ParentControlIndex = 0;
	UPROPERTY()
	FName ChannelTypeName; 
	UPROPERTY()
	bool bDoesHaveSpace = false;
	UPROPERTY()
	int32 SpaceChannelIndex = -1; //if it has space what's the space channel index
	UPROPERTY()
	int32 MaskIndex = -1; //index for the mask
	UPROPERTY()
	int32 CategoryIndex = -1; //index for the Sequencer category node

	int32 GeneratedKeyIndex = -1; //temp index set by the ControlRigParameterTrack, not saved

	UPROPERTY()
	TArray<uint32> ConstraintsIndex; //constraints data
};

struct FMovieSceneControlRigSpaceChannel;

/**
 * Movie scene section that controls animation controller animation
 */
UCLASS(MinimalAPI)
class UMovieSceneControlRigParameterSection : public UMovieSceneParameterSection
															, public IMovieSceneConstrainedSection

{
	GENERATED_BODY()

public:

	/** Bindable events for when we add space or constraint channels. */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FSpaceChannelAddedEvent, UMovieSceneControlRigParameterSection*, const FName&, FMovieSceneControlRigSpaceChannel*);

	UE_API void AddEnumParameterKey(FName InParameterName, FFrameNumber InTime, uint8 InValue);
	UE_API void AddIntegerParameterKey(FName InParameterName, FFrameNumber InTime, int32 InValue);

	UE_API bool RemoveEnumParameter(FName InParameterName);
	UE_API bool RemoveIntegerParameter(FName InParameterName);

	UE_API TArray<FEnumParameterNameAndCurve>& GetEnumParameterNamesAndCurves();
	UE_API const TArray<FEnumParameterNameAndCurve>& GetEnumParameterNamesAndCurves() const;

	UE_API TArray<FIntegerParameterNameAndCurve>& GetIntegerParameterNamesAndCurves();
	UE_API const TArray<FIntegerParameterNameAndCurve>& GetIntegerParameterNamesAndCurves() const;

	UE_API void FixRotationWinding(const FName& ControlName, FFrameNumber StartFrame, FFrameNumber EndFrame);
	UE_API void OptimizeSection(const FName& ControlName, const FKeyDataOptimizationParams& InParams);
	UE_API void AutoSetTangents(const FName& ControlName);

	UE_API TArray<FSpaceControlNameAndChannel>& GetSpaceChannels();
	UE_API const TArray< FSpaceControlNameAndChannel>& GetSpaceChannels() const;
	UE_API FName FindControlNameFromSpaceChannel(const FMovieSceneControlRigSpaceChannel* SpaceChannel) const;
	
	FSpaceChannelAddedEvent& SpaceChannelAdded() { return OnSpaceChannelAdded; }

	UE_API const FName& FindControlNameFromConstraintChannel(const FMovieSceneConstraintChannel* InConstraintChannel) const;

	UE_API void ForEachParameter(TFunction<void(FBaseParameterNameAndValue*)> InCallback);
	UE_API void ForEachParameter(TOptional<ERigControlType> InControlType, TFunction<void(FBaseParameterNameAndValue*)> InCallback);

	template<typename T>
	void ForEachParameter(TArray<T>& InParameterArray, TFunction<void(FBaseParameterNameAndValue*)> InCallback)
	{
		if(!InCallback)
		{
			return;
		}
		for(T& Parameter : InParameterArray)
		{
			InCallback(&Parameter);
		}
	}

	UE_API void ChangeControlRotationOrder(const FName& InControlName, const TOptional<EEulerRotationOrder>& CurrentOrder, 
		const  TOptional<EEulerRotationOrder>& NewOrder, EMovieSceneKeyInterpolation Interpolation);

	UE_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	UE_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	//mz todo virtual void InterrogateEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;


	UE_API bool RenameParameterName(const FName& OldParameterName, const FName& NewParameterName, TOptional<ERigControlType> ControlType);
	/** Copy existing internal vector parameter of Position/Rotator/Scale type to a transform parameter type*/
	UE_API bool  CopyVectorParameterCurvesToTransform(const FName& InControlName, ERigControlType InType);

#if WITH_EDITOR
	UE_API void OnControlRigEditorSettingChanged(UObject* InSettingsChanged, struct FPropertyChangedEvent& InPropertyChangedEvent);
#endif

private:

#if WITH_EDITOR
	FDelegateHandle OnControlRigEditorSettingChangedHandle;
	void BindEditorDelegates();
	void UnbindEditorDelegates();
#endif

	FSpaceChannelAddedEvent OnSpaceChannelAdded;
	/** Control Rig that controls us*/
	UPROPERTY()
	TObjectPtr<UControlRig> ControlRig;

public:

	/** The class of control rig to instantiate */
	UPROPERTY(meta = (DeprecatedProperty))
	TSubclassOf<UControlRig> ControlRigClass_DEPRECATED;
	
	/** The type of control rig to instantiate (defined by its class or ControlRigRuntimeAsset) */
	UPROPERTY(EditAnywhere, Category = "Animation", meta = (DisallowCreateNew))
	FControlRigAssetStrongReference ControlRigAssetReference;

	/** Deprecrated, use ControlNameMask*/
	UPROPERTY()
	TArray<bool> ControlsMask;

	/** Names of Controls that are masked out on this section*/
	UPROPERTY()
	TSet<FName> ControlNameMask;

	/** Mask for Transform Mask*/
	UPROPERTY()
	FMovieSceneTransformMask TransformMask;

	/** The weight curve for this animation controller section */
	UPROPERTY()
	FMovieSceneFloatChannel Weight;

	/** Map from the control name to where it starts as a channel*/
	UPROPERTY()
	TMap<FName, FChannelMapInfo> ControlChannelMap;


protected:
	/** Enum Curves*/
	UPROPERTY()
	TArray<FEnumParameterNameAndCurve> EnumParameterNamesAndCurves;

	/*Integer Curves*/
	UPROPERTY()
	TArray<FIntegerParameterNameAndCurve> IntegerParameterNamesAndCurves;

	/** Space Channels*/
	UPROPERTY()
	TArray<FSpaceControlNameAndChannel>  SpaceChannels;

	/** Space Channels*/
	UPROPERTY()
	TArray<FConstraintAndActiveChannel> ConstraintsChannels;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Customization)
	TArray<TSoftObjectPtr<UControlRigOverrideAsset>> OverrideAssets;
	bool bSuspendOverrideAssetSync = false;
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback")
	FMovieSceneTimeWarpVariant PlayRate;

public:

	UE_API UMovieSceneControlRigParameterSection();

	//UMovieSceneSection virtuals
	UE_API virtual void SetBlendType(EMovieSceneBlendType InBlendType) override;
	UE_API virtual UObject* GetImplicitObjectOwner() override;
	UE_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	UE_API virtual FMovieSceneTimeWarpVariant* GetTimeWarp() override;
	UE_API bool HasTimeWarp() const;


	// IMovieSceneConstrainedSection overrides
	/*
	* Whether it has that channel
	*/
	UE_API virtual bool HasConstraintChannel(const FGuid& InConstraintName) const override;

	/*
	* Get constraint with that name
	*/
	UE_API virtual FConstraintAndActiveChannel* GetConstraintChannel(const FGuid& InConstraintID) override;

	/*
	*  Add Constraint channel
	*/
	UE_API virtual void AddConstraintChannel(UTickableConstraint* InConstraint) override;

	/*
	*  Remove Constraint channel
	*/
	UE_API virtual void RemoveConstraintChannel(const UTickableConstraint* InConstraint) override;

	/*
	*  Get a reference to the channels
	*/
	UE_API virtual TArray<FConstraintAndActiveChannel>& GetConstraintsChannels()  override;

	/*
	*  Get a const reference to the channels
	*/
	UE_API virtual const TArray<FConstraintAndActiveChannel>& GetConstraintsChannels() const override;

	/*
	*  Replace the constraint with the specified name with the new one
	*/
	UE_API virtual void ReplaceConstraint(const FName InConstraintName, UTickableConstraint* InConstraint)  override;

	/*
	*  What to do if the constraint object has been changed, for example by an undo or redo.
	*/
	UE_API virtual void OnConstraintsChanged() override;

#if WITH_EDITOR
	//Function to save control rig key when recording.
	UE_API void RecordControlRigKey(FFrameNumber FrameNumber, bool bSetDefault, EMovieSceneKeyInterpolation InInterpMode, bool bOntoSelectedControls = false);

	struct FLoadAnimSequenceData
	{
		//key reduce
		bool bKeyReduce = false;
		// key reduction tolerance
		float Tolerance = 0.5;
		//Whether to rest to default control states
		bool bResetControls = false;
		//onto selected controls only
		bool bOntoSelectedControls = false;
		// retarget or not 
		bool bShouldRetarget = true;
		//Frame to Insert at
		FFrameNumber StartFrame;
		//If set only load the animation from the specified range
		TOptional<TRange<FFrameNumber>> AnimFrameRange;
	};
	
	//Function to load an Anim Sequence into this section. It will automatically resize to the section size.
	//Will return false if fails or is canceled
	UE_API virtual bool LoadAnimSequenceIntoThisSection(UAnimSequence* Sequence, const FFrameNumber& SequenceStart, UMovieScene* MovieScene, UObject* BoundObject, const FLoadAnimSequenceData& LoadData, EMovieSceneKeyInterpolation Interpolation);
#endif
	
	UE_API void FillControlNameMask(bool bValue);

	UE_API void SetControlNameMask(const FName& Name, bool bValue, bool bReconstructChannel);

	UE_API bool GetControlNameMask(const FName& Name) const;


	/**
	* This function returns the active category index of the control, based upon what controls are active/masked or not
	* If itself is masked it returns INDEX_NONE
	*/
	UE_API int32 GetActiveCategoryIndex(FName ControlName) const;
	/**
	* Access the transform mask that defines which channels this track should animate
	*/
	FMovieSceneTransformMask GetTransformMask() const
	{
		return TransformMask;
	}

	/**
	 * Set the transform mask that defines which channels this track should animate
	 */
	void SetTransformMask(FMovieSceneTransformMask NewMask)
	{
		TransformMask = NewMask;
		ReconstructChannelProxy();
	}

	UE_API void DeleteTimeWarp();

public:

	/** Recreate with this Control Rig*/
	UE_API void RecreateWithThisControlRig(UControlRig* InControlRig, bool bSetDefault);

	/* Set the control rig for this section */
	UE_API void SetControlRig(UControlRig* InControlRig);

	/* Get the control rig for this section, by default in non-game world */
	UE_API UControlRig* GetControlRig() const;

	/* Get the control rig for this section - given a gameworld and an optional bound object (pass nullptr to auto-determine) */
	UE_API UControlRig* GetControlRig(UWorld* InGameWorld, UObject* InBoundObject) const;

	/** Whether or not to key currently, maybe evaluating so don't*/
	void  SetDoNotKey(bool bIn) const { bDoNotKey = bIn; }
	/** Get Whether to key or not*/
	bool GetDoNotKey() const { return bDoNotKey; }

	/**  Whether or not this section has scalar*/
	UE_API bool HasScalarParameter(FName InParameterName) const;

	/**  Whether or not this section has bool*/
	UE_API bool HasBoolParameter(FName InParameterName) const;

	/**  Whether or not this section has enum*/
	UE_API bool HasEnumParameter(FName InParameterName) const;

	/**  Whether or not this section has int*/
	UE_API bool HasIntegerParameter(FName InParameterName) const;

	/**  Whether or not this section has scalar*/
	UE_API bool HasVector2DParameter(FName InParameterName) const;

	/**  Whether or not this section has scalar*/
	UE_API bool HasVectorParameter(FName InParameterName) const;

	/**  Whether or not this section has scalar*/
	UE_API bool HasColorParameter(FName InParameterName) const;

	/**  Whether or not this section has scalar*/
	UE_API bool HasTransformParameter(FName InParameterName) const;

	/**  Whether or not this section has space*/
	UE_API bool HasSpaceChannel(FName InParameterName) const;

	/** Get The Space Channel for the Control*/
	UE_API FSpaceControlNameAndChannel* GetSpaceChannel(FName InParameterName);

	/** Adds specified scalar parameter. */
	UE_API void AddScalarParameter(FName InParameterName,  TOptional<float> DefaultValue, bool bReconstructChannel);

	/** Adds specified bool parameter. */
	UE_API void AddBoolParameter(FName InParameterName, TOptional<bool> DefaultValue, bool bReconstructChannel);

	/** Adds specified enum parameter. */
	UE_API void AddEnumParameter(FName InParameterName, UEnum* Enum,TOptional<uint8> DefaultValue, bool bReconstructChannel);

	/** Adds specified int parameter. */
	UE_API void AddIntegerParameter(FName InParameterName, TOptional<int32> DefaultValue, bool bReconstructChannel);

	/** Adds a a key for a specific vector parameter. */
	UE_API void AddVectorParameter(FName InParameterName, TOptional<FVector> DefaultValue, bool bReconstructChannel);

	/** Adds a a key for a specific vector2D parameter. */
	UE_API void AddVector2DParameter(FName InParameterName, TOptional<FVector2D> DefaultValue, bool bReconstructChannel);

	/** Adds a a key for a specific color parameter. */
	UE_API void AddColorParameter(FName InParameterName, TOptional<FLinearColor> DefaultValue, bool bReconstructChannel);

	/** Adds a a key for a specific transform parameter*/
	UE_API void AddTransformParameter(FName InParameterName, TOptional<FEulerTransform> DefaultValue, bool bReconstructChannel);

	/** Add Space Parameter for a specified Control, no Default since that is Parent space*/
	UE_API void AddSpaceChannel(FName InControlName, bool bReconstructChannel);

	/** Clear Everything Out*/
	UE_API void ClearAllParameters();

	/** Evaluates specified scalar parameter. Will not get set if not found */
	UE_API TOptional<float> EvaluateScalarParameter(const  FFrameTime& InTime, FName InParameterName);

	/** Evaluates specified bool parameter. Will not get set if not found */
	UE_API TOptional<bool> EvaluateBoolParameter(const  FFrameTime& InTime, FName InParameterName);

	/** Evaluates specified enum parameter. Will not get set if not found */
	UE_API TOptional<uint8> EvaluateEnumParameter(const  FFrameTime& InTime, FName InParameterName);

	/** Evaluates specified int parameter. Will not get set if not found */
	UE_API TOptional<int32> EvaluateIntegerParameter(const  FFrameTime& InTime, FName InParameterName);

	/** Evaluates a a key for a specific vector parameter. Will not get set if not found */
	UE_API TOptional<FVector> EvaluateVectorParameter(const FFrameTime& InTime, FName InParameterName);

	/** Evaluates a a key for a specific vector2D parameter. Will not get set if not found */
	UE_API TOptional<FVector2D> EvaluateVector2DParameter(const  FFrameTime& InTime, FName InParameterName);

	/** Evaluates a a key for a specific color parameter. Will not get set if not found */
	UE_API TOptional<FLinearColor> EvaluateColorParameter(const  FFrameTime& InTime, FName InParameterName);

	/** Evaluates a a key for a specific transform parameter. Will not get set if not found */
	UE_API TOptional<FEulerTransform> EvaluateTransformParameter(const  FFrameTime& InTime, FName InParameterName);
	

	/** Evaluates a a key for a specific space parameter. Will not get set if not found */
	UE_API TOptional<FMovieSceneControlRigSpaceBaseKey> EvaluateSpaceChannel(const  FFrameTime& InTime, FName InParameterName);

	/** Key Zero Values on all or just selected controls in these section at the specified time */
	UE_API void KeyZeroValue(FFrameNumber InFrame, EMovieSceneKeyInterpolation DefaultInterpolation, bool bSelected);

	/** Key the Weights to the specified value */
	UE_API void KeyWeightValue(FFrameNumber InFrame, EMovieSceneKeyInterpolation DefaultInterpolation, float InVal);
;
	/** Remove All Keys, but maybe not space keys if bIncludeSpaceKeys is false */
	UE_API void RemoveAllKeys(bool bIncludeSpaceKeys);

	/** Whether or not create a space channel for a particular control */
	UE_API bool CanCreateSpaceChannel(FName InControlName) const;

	/** Perform any time warping or anything else to remap current time*/
	UE_API FFrameTime RemapTime(const FFrameTime& InFrameTime) const;

public:
	/**
	* Access the interrogation key for control rig data 
	*/
	 static UE_API FMovieSceneInterrogationKey GetFloatInterrogationKey();
	 static UE_API FMovieSceneInterrogationKey GetVector2DInterrogationKey();
	 static UE_API FMovieSceneInterrogationKey GetVector4InterrogationKey();
	 static UE_API FMovieSceneInterrogationKey GetVectorInterrogationKey();
	 static UE_API FMovieSceneInterrogationKey GetTransformInterrogationKey();

	/**
	 * Retrieve meta-data pertaining to a given channel ptr including the control it animates and its index within the control.
	 * 
	 * @param Channel A pointer to a channel
	 * @return The channel's meta-data. Must be checked for validity before use
	 */
	UE_API UE::MovieScene::FControlRigChannelMetaData GetChannelMetaData(const FMovieSceneChannel* Channel) const;

	UE_API virtual void ReconstructChannelProxy() override;
	UE_API virtual float GetTotalWeightValue(FFrameTime InTime) const override;

protected:

	UE_API void ConvertMaskArrayToNameSet();
	UE_API void MaskOutIfThereAreMaskedControls(const FName& InControlName);

	//~ UMovieSceneSection interface
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual void GetReferencedBindings(TArray<FGuid>& OutBindings) override;
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual bool IsBlendingHandledExternally() const
	{
		return BlendType.Get() == EMovieSceneBlendType::Absolute;
	}

#if WITH_EDITOR
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif

	void HandleOverrideAssetsChanged(UControlRig* InControlRig);
	void HandleOverrideAssetChanged(const UControlRigOverrideAsset* InOverrideAsset);
	UE_API void UpdateOverrideAssetDelegates();

	FDelegateHandle OnOverrideAssetsChangedHandle;

	void PostLoadConstraintChannels();

	// When true we do not set a key on the section, since it will be set because we changed the value
	// We need this because control rig notifications are set on every change even when just changing sequencer time
	// which forces a sequencer eval, not like the editor where changes are only set on UI changes(changing time doesn't send change delegate)
	mutable bool bDoNotKey;

public:
	// Special list of Names that we should only Modify. Needed to handle Interaction (FK/IK) since Control Rig expecting only changed value to be set
	//not all Controls
	mutable TSet<FName> ControlsToSet;


public:
	// Returns true if NewControls differs from the last controls used to reconstruct (hash comparison).
	UE_API bool IsDifferentThanLastControlsUsedToReconstruct(const TArray<FRigControlElement*>& NewControls) const;

	// Returns the cached hash of the last controls used to reconstruct. Used by ECS to detect
	// when the section has been reconstructed since entities were imported (stale channel pointers).
	uint32 GetLastControlsHash() const { return LastControlsHash; }

private:
	UE_API void StoreLastControlsUsedToReconstruct(const TArray<FRigControlElement*>& NewControls);

	// Computes a hash from an array of control elements (name + type).
	static uint32 ComputeControlsHash(const TArray<FRigControlElement*>& Controls);

	// Cached hash of the last controls used to reconstruct, updated by StoreLastControlsUsedToReconstruct.
	uint32 LastControlsHash = 0;

private:

	//there was a regression that caused certain controls present in the Hierarchy->GetPreviousName map to be applied backwards so it would incorrectly try to
	//replace controls with new names with their old name s instead of vice versa.
	//hope we can get rid of this function since it's only a few files with bad data hopefully
	//this will go through the parameter names and if there are duplicates it will remove them, the first one is always the one to keep
	template<typename T>
	bool  HACK_CheckForDupParameters(TArray<T>& InParameterArray)
	{
		TSet<FName> ExistingNames;
		TArray<int32> IndicesToRemove;
		for (T& Parameter : InParameterArray)
		{
			if (ExistingNames.Contains(Parameter.ParameterName))
			{
				return true;
			}
			else
			{
				ExistingNames.Add(Parameter.ParameterName);
			}
		}
		return false;
	}
	
	
	template<typename T>
	bool  HACK_CleanForEachParameter(TArray<T>& InParameterArray)
	{
		TSet<FName> ExistingNames;
		int32 Index = 0;
		TArray<int32> IndicesToRemove; 
		for (T& Parameter : InParameterArray)
		{
			if (ExistingNames.Contains(Parameter.ParameterName))
			{
				IndicesToRemove.Insert(Index,0);  //put first since we want to remove last
			}
			else
			{
				ExistingNames.Add(Parameter.ParameterName);
			}
			++Index;
		}
		for (int32 RemoveIndex : IndicesToRemove)
		{
			InParameterArray.RemoveAt(RemoveIndex);
		}
		return IndicesToRemove.Num() > 0;
	}
	UE_API void HACK_FixMultipleParamsWithSameName();

	friend class SControlRigEditModeTools;
};

#undef UE_API
