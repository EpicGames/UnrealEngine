// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Evaluation/MovieSceneParameterTemplate.h"
#include "MovieSceneControlRigParameterSection.h"

#include "MovieSceneControlRigParameterTemplate.generated.h"

class UMovieSceneControlRigParameterTrack;
struct FEvaluatedControlRigParameterSectionValues;
USTRUCT()
struct FMovieSceneControlRigParameterTemplate : public FMovieSceneParameterSectionTemplate
{
	GENERATED_BODY()

	FMovieSceneControlRigParameterTemplate() {}
	FMovieSceneControlRigParameterTemplate(const UMovieSceneControlRigParameterSection& Section, const UMovieSceneControlRigParameterTrack& Track);
	static FMovieSceneAnimTypeID GetAnimTypeID();

#if WITH_EDITOR
	virtual ~FMovieSceneControlRigParameterTemplate() override;

	TArray<FDelegateHandle> PreCompileHandles;
#endif
	
private:
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
	virtual void Interrogate(const FMovieSceneContext& Context, FMovieSceneInterrogationData& Container, UObject* BindingOverride) const override;

private:
	void EvaluateCurvesWithMasks(const FMovieSceneContext& Context, struct FEvaluatedControlRigParameterSectionChannelMasks& InMasks, FEvaluatedControlRigParameterSectionValues& Values) const;

protected:
	/** The bool parameter names and their associated curves. */
	UPROPERTY()
	TArray<FEnumParameterNameAndCurve> Enums;
	/** The enum parameter names and their associated curves. */
	UPROPERTY()
	TArray<FIntegerParameterNameAndCurve> Integers;
	/** Controls and their space keys*/
	UPROPERTY()
	TArray<FSpaceControlNameAndChannel> Spaces;
	
	UPROPERTY()
	TArray<FConstraintAndActiveChannel> Constraints;
};

