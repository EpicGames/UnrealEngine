// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sections/MovieSceneParameterSection.h"
#include "MovieSceneNameableTrack.h"
#include "ControlRig.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "INodeAndChannelMappings.h"
#include "MovieSceneControlRigParameterSection.h"
#include "EulerTransform.h"
#include "Tracks/IMovieSceneSectionsToKey.h"
#include "MovieSceneControlRigParameterTrack.generated.h"

#define UE_API CONTROLRIG_API

struct FEndLoadPackageContext;
struct FRigControlElement;
struct FMovieSceneEvaluationTrack;

USTRUCT()
struct FControlRotationOrder
{
	GENERATED_BODY()

	UE_API FControlRotationOrder();

	/* Rotation Order*/
	UPROPERTY()
	EEulerRotationOrder RotationOrder;

	/** Override the default control setting*/
	UPROPERTY()
	bool bOverrideSetting;
};

/**
 * Handles animation of skeletal mesh actors using animation ControlRigs
 */

UCLASS(MinimalAPI)
class UMovieSceneControlRigParameterTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
	, public INodeAndChannelMappings
	, public IMovieSceneSectionsToKey

{
	GENERATED_UCLASS_BODY()

public:

	CONTROLRIG_API static bool ShouldUseLegacyTemplate();
	CONTROLRIG_API static bool ShouldCacheLegacyTemplate();

	// UMovieSceneTrack interface
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual FName GetTrackName() const override { return TrackName; }
	virtual void OnRemovedFromMovieSceneImpl() override;

	// UObject
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	virtual void PostEditImport() override;
#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

	//INodeAndMappingsInterface
	virtual TArray<FRigControlFBXNodeAndChannels>* GetNodeAndChannelMappings(UMovieSceneSection* InSection)  override;
	virtual void GetSelectedNodes(TArray<FName>& OutSelectedNodes) override;
#if WITH_EDITOR
	virtual bool GetFbxCurveDataFromChannelMetadata(const FMovieSceneChannelMetaData& MetaData, FControlRigFbxCurveData& OutCurveData) override;
#endif // WITH_EDITOR

	CONTROLRIG_API void BindControlRigDelegates();
private:
	void UnbindControlRigDelegates();
	void BindGameWorldControlRigDelegates(UControlRig* GameWorldControlRig);
	void UnbindGameWorldControlRigDelegates(UControlRig* GameWorldControlRig);
	
	//UControlRig Delegates
	void HandleOnPreConstruct_AnyThread(UControlRig* Subject, const FName& InEventName);
	void HandleOnPostConstructed_GameThread();
	void HandleOnPostConstructed(UControlRig* Subject, const FName& InEventName);


#if WITH_EDITOR
	void BindPIEDelegates();
	void UnbindPIEDelegates();
	void HandlePackageDone(const FEndLoadPackageContext& Context);
	// control Rigs are ready only after its package is fully end-loaded
	void HandleControlRigPackageDone(URigVMHost* InControlRig);
#endif

public:
	/** Add a section at that start time*/
	CONTROLRIG_API UMovieSceneSection* CreateControlRigSection(FFrameNumber StartTime, UControlRig* InControlRig, bool bInOwnsControlRig);

	CONTROLRIG_API void ReplaceControlRig(UControlRig* NewControlRig, bool RecreateChannels);

	CONTROLRIG_API void RenameParameterName(const FName& OldParameterName, const FName& NewParameterName);

public:
	UControlRig* GetControlRig() const { return ControlRig; }

	/**
	* Find all sections at the current time.
	*
	*@param Time  The Time relative to the owning movie scene where the section should be
	*@Return All sections at that time
	*/
	CONTROLRIG_API TArray<UMovieSceneSection*, TInlineAllocator<4>> FindAllSections(FFrameNumber Time);

	/**
	 * Finds a section at the current time.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @return The found section.
	 */
	CONTROLRIG_API class UMovieSceneSection* FindSection(FFrameNumber Time);

	/**
	 * Finds a section at the current time or extends an existing one
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @param OutWeight The weight of the section if found
	 * @return The found section.
	 */
	CONTROLRIG_API class UMovieSceneSection* FindOrExtendSection(FFrameNumber Time, float& OutWeight);

	/**
	 * Finds a section at the current time.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @param bSectionAdded Whether a section was added or not
	 * @return The found section, or the new section.
	 */
	CONTROLRIG_API class UMovieSceneSection* FindOrAddSection(FFrameNumber Time, bool& bSectionAdded);

	/**
	 * Set the section we want to key and receive globally changed values.
	 *
	 * @param Section The section that changes.
	 */

	virtual void SetSectionToKey(UMovieSceneSection* Section) override;

	/**
	 * Finds a section we want to key and receive globally changed values.
	 * @return The Section that changes.
	 */
	virtual UMovieSceneSection* GetSectionToKey() const override;

	/**
	* Get multiple sections to key
	* @return multiple sections to key
	*/
	virtual TArray<TWeakObjectPtr<UMovieSceneSection>> GetSectionsToKey() const override;

	/**
	*  Control rig supports per control sections to key needed for anim layer workflows
	*/
	CONTROLRIG_API UMovieSceneSection* GetSectionToKey(const FName& InControlName) const;
	CONTROLRIG_API void SetSectionToKey(UMovieSceneSection* InSection, const FName& inControlName);

	void SetTrackName(FName InName) { TrackName = InName; }

	UMovieSceneControlRigParameterSection::FSpaceChannelAddedEvent& SpaceChannelAdded() { return OnSpaceChannelAdded; }
	IMovieSceneConstrainedSection::FConstraintChannelAddedEvent& ConstraintChannelAdded() { return OnConstraintChannelAdded; }

	CONTROLRIG_API void UpdateControlRigSettingsOverrides(const FInstancedPropertyBag& InNewOverrides);
	CONTROLRIG_API void ApplyControlRigSettingsOverrides(UControlRig* InControlRig);
	CONTROLRIG_API void RecreateControlRig();
	

	CONTROLRIG_API FInstancedPropertyBag GetCurrentRigSettings();
	CONTROLRIG_API void SyncRigSettingsToCurrentRig(FInstancedPropertyBag& InSettings);
	CONTROLRIG_API FInstancedPropertyBag BuildNewRigSettings(const TMap<FName, FString>& InSettingsToOverride);
	
public:
	CONTROLRIG_API TArray<FName> GetControlsWithDifferentRotationOrders() const;
	CONTROLRIG_API void ResetControlsToSettingsRotationOrder(const TArray<FName>& Names,
		EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::SmartAuto);
	// if order is not set then it uses the default FRotator conversions
	CONTROLRIG_API void ChangeControlRotationOrder(const FName& InControlName, const TOptional<EEulerRotationOrder>& NewOrder,
		EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::SmartAuto);

	CONTROLRIG_API int32 GetPriorityOrder() const;
	CONTROLRIG_API void SetPriorityOrder(int32 InPriorityIndex);
private:
	UPROPERTY()
	TMap<FName, TWeakObjectPtr<UMovieSceneSection>> SectionToKeyPerControl;
public:
	bool bSetSectionToKeyPerControl = true;// set up the map when calling the UMovieSceneTrack::SetSectionToKey()
private:
	//get the rotation orderm will not be set if it's default FRotator order. If bCurrent is true, it uses what's set
	//if false it uses the default setting from the current control rig.
	TOptional<EEulerRotationOrder> GetControlRotationOrder(const FRigControlElement* ControlElement, bool bCurrent) const;
private:

	//we register this with sections.
	void HandleOnSpaceAdded(UMovieSceneControlRigParameterSection* Section, const FName& InControlName, FMovieSceneControlRigSpaceChannel* Channel);
	
	//then send this event out to the track editor.
	UMovieSceneControlRigParameterSection::FSpaceChannelAddedEvent OnSpaceChannelAdded;

	void HandleOnConstraintAdded(
		IMovieSceneConstrainedSection* InSection,
		FMovieSceneConstraintChannel* InChannel) const;
	IMovieSceneConstrainedSection::FConstraintChannelAddedEvent OnConstraintChannelAdded;

	void RemoveAllConstraints() const;
	
	void ReconstructControlRig();

	//look for name changes in particularly with modular rigs so we can update the rotation order map
	void CheckForNameChanges();

#if WITH_EDITOR
	void OnPreBeginPIE(bool bIsSimulating);
	void OnEndPIE(bool bIsSimulating);
	void OnShutdownPIE(bool bIsSimulating);
	void OnCleanupGameControlRigs();
	FDelegateHandle OnPreBeginPIEHandle;
	FDelegateHandle EndPIEHandle;
	FDelegateHandle ShutdownPIEHandle;
#endif

private:


	/** Control Rig we control*/
	UPROPERTY()
	TObjectPtr<UControlRig> ControlRig;

	/** Section we should Key */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> SectionToKey;

	/** The sections owned by this track .*/
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** Unique Name*/
	UPROPERTY()
	FName TrackName;

	/** Uses Rotation Order*/
	UPROPERTY()
	TMap<FName,FControlRotationOrder> ControlsRotationOrder;

	UPROPERTY()
	int32 PriorityOrder;

	// Stores control rig public variable overrides, will switch to the same override system that rig module
	// uses in the future
	UPROPERTY()
	FInstancedPropertyBag ControlRigSettingsOverrides;

public:
	static CONTROLRIG_API FColor AbsoluteRigTrackColor;
	static CONTROLRIG_API FColor LayeredRigTrackColor;

public:
	CONTROLRIG_API UControlRig* GetGameWorldControlRig(UWorld* InWorld, UObject* InBoundObject);
	bool IsAGameInstance(const UControlRig* InControlRig, const bool bCheckValidWorld = false) const;
	
	CONTROLRIG_API const TUniquePtr<FMovieSceneEvaluationTrack>& GetLegacyEvaluationTrack() const;
    
private:
	/** copy of the controlled control rig that we use in the game world so editor control rig doesn't conflict*/
	UPROPERTY(transient)
	TMap<TWeakObjectPtr<UWorld>,TObjectPtr<UControlRig>> GameWorldControlRigs;

	// Cached FMovieSceneEvaluationTrack used to speed up legacy evaluation
	int32 LegacyEvaluationTrackCount = 0;
	TUniquePtr<FMovieSceneEvaluationTrack> LegacyEvaluationTrack; 

	friend struct FControlRigParameterTrackSectionToKeyRestore;
	friend struct FScopedLegacyEvaluationTrack;
};

/**
 * FScopedLegacyEvaluationTrack allows FMovieSceneEvaluationTrack to be cached when bulk-adding keyframes which significantly improves performance.
 * Note that this code can be called multiple times for the same track but is not thread-safe as is.
 */

struct FScopedLegacyEvaluationTrack
{
	CONTROLRIG_API FScopedLegacyEvaluationTrack(UMovieSceneControlRigParameterTrack* InTrack);
	CONTROLRIG_API FScopedLegacyEvaluationTrack(const UMovieSceneSection* InSection);
	CONTROLRIG_API ~FScopedLegacyEvaluationTrack();
private:
	UE_NONCOPYABLE(FScopedLegacyEvaluationTrack);
	TWeakObjectPtr<UMovieSceneControlRigParameterTrack> WeakTrack;
	bool bCacheTemplate = false;
};

//override the section to key, section to key per control, temporarily
struct FControlRigParameterTrackSectionToKeyRestore
{
	FControlRigParameterTrackSectionToKeyRestore(UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneSection* NewSectionToKey, TMap<FName, TWeakObjectPtr<UMovieSceneSection>>& NewSectionToKeyPerControl)
		: Track(InTrack)
	{
		if (Track)
		{
			OldSectionToKey = Track->SectionToKey;
			OldSectionToKeyPerControl = Track->SectionToKeyPerControl;
			Track->SectionToKey = NewSectionToKey;
			Track->SectionToKeyPerControl = NewSectionToKeyPerControl;
		}
	}

	~FControlRigParameterTrackSectionToKeyRestore()
	{
		if (Track)
		{
			Track->SectionToKey = OldSectionToKey;
			Track->SectionToKeyPerControl = OldSectionToKeyPerControl;
		}
	}

	FControlRigParameterTrackSectionToKeyRestore() = delete;
	
	UMovieSceneControlRigParameterTrack* Track;
	UMovieSceneSection* OldSectionToKey;
	TMap<FName, TWeakObjectPtr<UMovieSceneSection>> OldSectionToKeyPerControl;
};

#undef UE_API
