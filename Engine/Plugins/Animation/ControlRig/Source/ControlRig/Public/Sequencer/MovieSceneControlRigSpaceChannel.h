// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Curves/KeyHandle.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSection.h"
#include "Curves/IntegralCurve.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "MovieSceneClipboard.h"
#include "Rigs/RigHierarchyDefines.h"
#include "MovieSceneControlRigSpaceChannel.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
struct FMovieSceneControlRigSpaceChannel;

DECLARE_MULTICAST_DELEGATE_TwoParams(FMovieSceneControlRigSpaceChannelSpaceNoLongerUsedEvent, FMovieSceneControlRigSpaceChannel*, const TArray<FRigElementKey>&);

UENUM(Blueprintable)
enum class EMovieSceneControlRigSpaceType : uint8
{
	Parent = 0,
	World,
	ControlRig
};

USTRUCT(BlueprintType)
struct FMovieSceneControlRigSpaceBaseKey
{
	GENERATED_BODY()
	
	friend bool operator==(const FMovieSceneControlRigSpaceBaseKey& A, const FMovieSceneControlRigSpaceBaseKey& B)
	{
		return A.SpaceType == B.SpaceType && (A.SpaceType != EMovieSceneControlRigSpaceType::ControlRig || A.ControlRigElement == B.ControlRigElement);
	}

	friend bool operator!=(const FMovieSceneControlRigSpaceBaseKey& A, const FMovieSceneControlRigSpaceBaseKey& B)
	{
		return A.SpaceType != B.SpaceType || (A.SpaceType == EMovieSceneControlRigSpaceType::ControlRig && A.ControlRigElement != B.ControlRigElement);
	}

	UE_API FName GetName() const;

	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Key")
	EMovieSceneControlRigSpaceType SpaceType = EMovieSceneControlRigSpaceType::Parent;
	UPROPERTY(BlueprintReadWrite,EditAnywhere, Category = "Key")
	FRigElementKey ControlRigElement;

};

struct FSpaceRange
{
	TRange<FFrameNumber> Range;
	FMovieSceneControlRigSpaceBaseKey Key;
};

/** A curve of spaces */
USTRUCT()
struct FMovieSceneControlRigSpaceChannel : public FMovieSceneChannel
{
	using CurveValueType = FMovieSceneControlRigSpaceBaseKey;

	GENERATED_BODY()

	FMovieSceneControlRigSpaceChannel()
	{}

	/**
	* Access a mutable interface for this channel's data
	*
	* @return An object that is able to manipulate this channel's data
	*/
	TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey>(&KeyTimes, &KeyValues, this, &KeyHandles);
	}

	/**
	* Access a constant interface for this channel's data
	*
	* @return An object that is able to interrogate this channel's data
	*/
	TMovieSceneChannelData<const FMovieSceneControlRigSpaceBaseKey> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneControlRigSpaceBaseKey>(&KeyTimes, &KeyValues);
	}

	/**
	* Evaluate this channel
	*
	* @param InTime     The time to evaluate at
	* @param OutValue   A value to receive the result
	* @return true if the channel was evaluated successfully, false otherwise
	*/
	UE_API bool Evaluate(FFrameTime InTime, FMovieSceneControlRigSpaceBaseKey& OutValue) const;

public:

	// ~ FMovieSceneChannel Interface
	UE_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	UE_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	UE_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	UE_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	UE_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	UE_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	UE_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	UE_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	UE_API virtual int32 GetNumKeys() const override;
	UE_API virtual void Reset() override;
	UE_API virtual void Offset(FFrameNumber DeltaPosition) override;
	UE_API virtual FKeyHandle GetHandle(int32 Index) override;
	UE_API virtual int32 GetIndex(FKeyHandle Handle) override;

	UE_API void GetUniqueSpaceList(TArray<FRigElementKey>* OutList);
	FMovieSceneControlRigSpaceChannelSpaceNoLongerUsedEvent& OnSpaceNoLongerUsed() { return SpaceNoLongerUsedEvent; }

	UE_API TArray <FSpaceRange> FindSpaceIntervals();

	/**
	 * Set this channel's default value that should be used when no keys are present
	 * @param InDefaultValue The desired default value
	 */
	void SetDefault(const FMovieSceneControlRigSpaceBaseKey& InDefaultValue);

	/**
	 * Get this channel's default value that will be used when no keys are present
	 * @return (Optional) The channel's default value
	 */
	TOptional<FMovieSceneControlRigSpaceBaseKey> GetDefault() const;

	/**
	 * Clear the default value on this channel
	 */
	virtual void ClearDefault() override;

	
private:

	UE_API void BroadcastSpaceNoLongerUsed(const TArray<FRigElementKey>& BeforeKeys, const TArray<FRigElementKey>& AfterKeys);

	/** Sorted array of key times */
	UPROPERTY(meta = (KeyTimes))
	TArray<FFrameNumber> KeyTimes;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta = (KeyValues))
	TArray<FMovieSceneControlRigSpaceBaseKey> KeyValues;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
	
	FMovieSceneControlRigSpaceChannelSpaceNoLongerUsedEvent SpaceNoLongerUsedEvent;

	UPROPERTY()
	FMovieSceneControlRigSpaceBaseKey DefaultValue;

	UPROPERTY()
	bool bHasDefaultValue = false;
	
	friend struct FControlRigSpaceChannelHelpers;
};


template<>
struct TMovieSceneChannelTraits<FMovieSceneControlRigSpaceChannel> : TMovieSceneChannelTraitsBase<FMovieSceneControlRigSpaceChannel>
{
	enum { SupportsDefaults = false };
};

inline bool EvaluateChannel(const FMovieSceneControlRigSpaceChannel* InChannel, FFrameTime InTime, FMovieSceneControlRigSpaceBaseKey& OutValue)
{
	return InChannel->Evaluate(InTime, OutValue);
}

#if WITH_EDITOR
namespace MovieSceneClipboard
{
	template<> inline FName GetKeyTypeName<FMovieSceneControlRigSpaceBaseKey>()
	{
		return "FMovieSceneControlRigSpaceBaseKey";
	}
}
#endif

//mz todoo TSharedPtr<FStructOnScope> GetKeyStruct(TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel> Channel, FKeyHandle InHandle);

#undef UE_API
