// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Sequencer/MovieSceneControlRigComponentTypes.h"
#include "Sequencer/MovieSceneControlRigParameterBuffer.h"
#include "SkeletalMeshRestoreState.h"
#include "UObject/ObjectKey.h"

#include "MovieSceneControlRigSystem.generated.h"

class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;
class UMovieSceneBlenderSystem;
class UMovieScenePiecewiseDoubleBlenderSystem;

namespace UE::MovieScene
{
	struct FBaseControlRigEvalData;
	struct FControlRigParameterValue;
	struct FEntityGroupID;
	struct FEvaluateControlRigChannels;
	struct FMovieSceneEntityID;
	struct FPreAnimatedControlRigParameterStorage;
	struct FPreAnimatedControlRigStorage;
	struct FRootInstanceHandle;

	struct FAnimatedControlRigParameterInfo
	{
		/** Weak linker ptr - only assigned if the output entity is ever allocated */
		TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
		/** Weak blender system ptr - only assigned if the blend channel is ever allocated */
		TWeakObjectPtr<UMovieSceneBlenderSystem> WeakBlenderSystem;
		int32 NumContributors = 0;
		FMovieSceneEntityID OutputEntityID;
		FMovieSceneBlendChannelID BlendChannelID;

		~FAnimatedControlRigParameterInfo();
	};

	CONTROLRIG_API void CollectGarbageForOutput(FAnimatedControlRigParameterInfo* Output);


	struct FPreAnimatedControlRigState
	{
		FSkeletalMeshRestoreState SkeletalMeshRestoreState;
		EAnimationMode::Type AnimationMode;
		bool bAnimMixerPoseProducer = false;
		void SetSkelMesh(USkeletalMeshComponent* InComponent)
		{
			SkeletalMeshRestoreState.SaveState(InComponent);
			AnimationMode = InComponent->GetAnimationMode();
		}
	};

	/** Pre-animation traits for control rig bindings */
	struct FPreAnimatedControlRigTraits : FBoundObjectPreAnimatedStateTraits
	{
		using KeyType = FObjectKey;
		using StorageType = FPreAnimatedControlRigState;

		static FPreAnimatedControlRigState CachePreAnimatedValue(FObjectKey InObject);
		static void RestorePreAnimatedValue(const FObjectKey& Object, FPreAnimatedControlRigState& State, const FRestoreStateParams& Params);
	};

	/** Pre-animation traits for control rig parameters */
	struct FPreAnimatedControlRigParameterTraits : FBoundObjectPreAnimatedStateTraits
	{
		using FBoundObjectPreAnimatedStateTraits::ResolveComponent;

		struct FPreAnimatedBufferPairs
		{
			FControlRigParameterValues Transient = FControlRigParameterValues(EControlRigParameterBufferIndexStability::Unstable);
			FControlRigParameterValues Persistent = FControlRigParameterValues(EControlRigParameterBufferIndexStability::Unstable);
		};

		struct FPreAnimatedParameterKey
		{
			TSharedPtr<FPreAnimatedBufferPairs> Buffer;
			FName ParameterName;
			bool bPersistent;

			friend uint32 GetTypeHash(const FPreAnimatedParameterKey& In)
			{
				return GetTypeHash(In.Buffer) ^ GetTypeHash(In.ParameterName) + (uint32)In.bPersistent;
			}

			friend bool operator==(const FPreAnimatedParameterKey& A, const FPreAnimatedParameterKey& B)
			{
				return A.Buffer == B.Buffer && A.ParameterName == B.ParameterName && A.bPersistent == B.bPersistent;
			}

			FPreAnimatedParameterKey() = default;
			FPreAnimatedParameterKey(TSharedPtr<FPreAnimatedBufferPairs> InBuffer, FName InName)
				: Buffer(InBuffer)
				, ParameterName(InName)
				, bPersistent(false)
			{}

			FPreAnimatedParameterKey(FPreAnimatedParameterKey&&) = default;
			FPreAnimatedParameterKey& operator=(FPreAnimatedParameterKey&&) = default;

			// Keys can only be copied when being elevated to persistent
			FPreAnimatedParameterKey(const FPreAnimatedParameterKey& RHS)
				: Buffer(RHS.Buffer)
				, bPersistent(true)
			{
				// Copy the parameter value from the transient to the persistent
				if (!RHS.bPersistent)
				{
					Buffer->Persistent.CopyFrom(Buffer->Transient, ParameterName);
				}
			}
			FPreAnimatedParameterKey& operator=(const FPreAnimatedParameterKey& RHS)
			{
				Buffer = RHS.Buffer;
				bPersistent = true;
				if (!RHS.bPersistent)
				{
					Buffer->Persistent.CopyFrom(Buffer->Transient, ParameterName);
				}
				return *this;
			}
		};
		using KeyType = TTuple<FObjectKey, FName>;
		using StorageType = FPreAnimatedParameterKey;

		TMap<FObjectKey, TWeakPtr<FPreAnimatedBufferPairs>> PreAnimatedBuffers;

		TSharedPtr<FPreAnimatedBufferPairs> GetBuffers(UControlRig* Rig);

		FPreAnimatedParameterKey CachePreAnimatedValue(UControlRig* Rig, FName ParameterName);
		void RestorePreAnimatedValue(const TTuple<FObjectKey, FName>& Rig, FPreAnimatedParameterKey& ParameterBuffer, const FRestoreStateParams& Params);
	};

	/** Preanimated storage that manages unbinding the rig from the target component */
	struct FPreAnimatedControlRigStorage
		: public TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedControlRigTraits>
	{
		static CONTROLRIG_API TAutoRegisterPreAnimatedStorageID<FPreAnimatedControlRigStorage> StorageID;
	};

	/** Preanimated storage that manages resetting parameters from the rig */
	struct FPreAnimatedControlRigParameterStorage
		: public TPreAnimatedStateStorage<FPreAnimatedControlRigParameterTraits>
	{
		static CONTROLRIG_API TAutoRegisterPreAnimatedStorageID<FPreAnimatedControlRigParameterStorage> StorageID;
	};

} // namespace UE::MovieScene


/**
 * System that tracks and evaluates control rig control parameters and spaces.
 * 
 * This system is relatively complex, and implements the following behavior:
 * 
 * Instantiation:
 *   - Initializes rigs by ensuring the anim instance set up and bound to the correct rig
 *   - Tracks contributing control rig parameters by rig
 *   - Caches pre-animated state
 *   - Assigns accumulation indices for each parameter
 *   - Caches initial values if necessary

 * Evaluation:
 *   - (GameThread) Evaluates any base rigs by calling SetDoNotKey(true) on the section, and updating the control rig track in the anim instance with its weight (if active)
 *   - (GameThread) Evaluates constraints
 *   - (Async)      Evaluates space channels
 *   - (Async)      Accumulates all control rig parameters and spaces into a single parameter buffer for each control rig
 *   - (GameThread) Applies the control rig parameter buffer to the rig
 *   - (Async)      Resets the 'do not key' states on the sections
 */
UCLASS(MinimalAPI)
class UMovieSceneControlRigParameterEvaluatorSystem : public UMovieSceneEntitySystem
{

public:

	GENERATED_BODY()
	UMovieSceneControlRigParameterEvaluatorSystem(const FObjectInitializer& ObjInit);

	CONTROLRIG_API UControlRig* GetRigFromTrack(UMovieSceneControlRigParameterTrack* Track) const;

	CONTROLRIG_API const UE::MovieScene::FControlRigParameterBuffer* FindParameters(UMovieSceneControlRigParameterTrack* Track) const;

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	void OnInstantiation();
	void OnEvaluation(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

private:

	void InitializeBaseRigComponent(
		UObject* BoundObject,
		UE::MovieScene::FMovieSceneEntityID EntityID,
		UE::MovieScene::FRootInstanceHandle RootInstanceHandle,
		bool bWantsRestoreState,
		bool bHasWeight,
		bool bAnimMixerPoseProducer,
		UE::MovieScene::FControlRigSourceData ControlRigSource,
		UE::MovieScene::FBaseControlRigEvalData& OutBaseData,
		TMap<UControlRig*, UE::MovieScene::FBaseControlRigEvalData*>& OutBaseComponentTracker
	);

	/**
	 * Checks for parameter entities whose sections have been reconstructed since import
	 * (due to procedural rigs adding controls during binding). Repairs stale entities by
	 * updating their channel pointers to the newly-allocated arrays. If a channel no longer
	 * exists, marks the entity with NeedsUnlink so it is removed.
	 */
	void RepairOrInvalidateStaleParameterEntities();

private:

	/**
	 * Tracker that operates on group IDs to initialize blend outputs for each control rig parameter type
	 */
	UE::MovieScene::TOverlappingEntityTracker<UE::MovieScene::FAnimatedControlRigParameterInfo, UE::MovieScene::FEntityGroupID> ControlRigParameterTracker;

	/**
	 * Persistent accumulation buffer for all known rig values.
	 * FControlRigComponentTypes::AccumulatedControlEntryIndex components directly reference entries within this structure.
	 * Care is taken to avoid async read/write through task dependencies in OnEvaluation.
	 * */
	UE::MovieScene::FAccumulatedControlRigValues AccumulatedValues;

public:

	/** Pre-animated state storage for all control rig parameters */
	TSharedPtr<UE::MovieScene::FPreAnimatedControlRigParameterStorage> ControlRigParameterStorage;

	/** Pre-animated state storage for control rigs themselves (stores which rigs are currently being animated along with their anim instance and binding) */
	TSharedPtr<UE::MovieScene::FPreAnimatedControlRigStorage> ControlRigStorage;

	/** Registered key for the grouping policy that groups control rig parameters together by name and type */
	UE::MovieScene::FEntityGroupingPolicyKey ParameterGroupingKey;

	/** Cached blender system used for blending parameters */
	UPROPERTY()
	TObjectPtr<UMovieScenePiecewiseDoubleBlenderSystem> DoubleBlenderSystem;

#if WITH_EDITOR
	virtual  ~UMovieSceneControlRigParameterEvaluatorSystem() override;
	
	TArray<FDelegateHandle> PreCompileHandles;
#endif
	
};

