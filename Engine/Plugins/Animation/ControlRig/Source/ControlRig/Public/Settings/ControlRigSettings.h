// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ControlRigSettings.h: Declares the ControlRigSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RigVMSettings.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ControlRigGizmoLibrary.h"

#if WITH_EDITOR
#include "RigVMModel/RigVMGraph.h"
#endif

#include "ControlRigSettings.generated.h"

#define UE_API CONTROLRIG_API

class UStaticMesh;
class UControlRig;

USTRUCT()
struct FControlRigSettingsPerPinBool
{
	GENERATED_BODY();

	UPROPERTY(EditAnywhere, Category = Settings)
	TMap<FString, bool> Values;
};

UENUM()
enum class EMultiRigTreeDisplayMode
{
	All, // show all rigs
	SelectedRigs, // show only rigs with selected controls
	SelectedModules // show only modules with selected controls (for modular rigs)
};

UENUM()
enum class EModularRigHierarchyEditorConnectorVisibilityFlags : uint8
{
	ShowPrimaryConnectors = 1,
	ShowSecondaryConnectors = 1 << 1,
	ShowOptionalConnectors = 1 << 2,
	ShowUnresolvedConnectors = 1 << 3,

	Default = ShowPrimaryConnectors | ShowUnresolvedConnectors
};
ENUM_CLASS_FLAGS(EModularRigHierarchyEditorConnectorVisibilityFlags);

/**
 * Default ControlRig settings.
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig, meta=(DisplayName="Control Rig"))
class UControlRigSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, config, Category = Shapes)
	TSoftObjectPtr<UControlRigShapeLibrary> DefaultShapeLibrary;

	UPROPERTY(EditAnywhere, config, Category = ModularRigging, meta=(AllowedClasses="/Script/ControlRigDeveloper.ControlRigBlueprint, /Script/ControlRig.ControlRigBlueprintGeneratedClass"))
	FSoftObjectPath DefaultRootModule;
#endif

	static UControlRigSettings* Get() { return GetMutableDefault<UControlRigSettings>(); }
};

/**
 * Customize Control Rig Editor.
 */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings, meta=(DisplayName="Control Rig Editor"))
class UControlRigEditorSettings : public URigVMEditorSettings
{
	GENERATED_UCLASS_BODY()

public:

#if WITH_EDITORONLY_DATA

	// When this is checked all controls will return to their initial
	// value as the user hits the Compile button.
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bResetControlsOnCompile;

	// When this is checked all controls will return to their initial
	// value as the user interacts with a pin value
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bResetControlsOnPinValueInteraction;
	
	// When this is checked all elements will be reset to their initial value
	// if the user changes the event queue (for example between forward / backward solve)
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bResetPoseWhenTogglingEventQueue;

	// When this is checked any hierarchy interaction within the Control Rig
	// Editor will be stored on the undo stack
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bEnableUndoForPoseInteraction;

	/**
	 * When checked controls will be reset during a manual compilation
	 * (when pressing the Compile button)
	 */
	UPROPERTY(EditAnywhere, config, Category = Compilation)
	bool bResetControlTransformsOnCompile;

	/**
	 * A map which remembers the expansion setting for each rig unit pin.
	 */
	UPROPERTY(EditAnywhere, config, Category = NodeGraph)
	TMap<FString, FControlRigSettingsPerPinBool> RigUnitPinExpansion;

	/**
	 * Whether or not to use a flash light effect around the mouse pointer
	 * in the dependency viewer to brighten up faded out nodes
	 */
	UPROPERTY(EditAnywhere, config, Category = NodeGraph)
	bool bEnableFlashlightInDependencyViewer;

	/**
	 * The border color of the viewport when entering "Construction Event" mode
	 */
	UPROPERTY(EditAnywhere, config, Category = Viewport)
	FLinearColor ConstructionEventBorderColor;
	
	/**
	 * The border color of the viewport when entering "Backwards Solve" mode
	 */
	UPROPERTY(EditAnywhere, config, Category = Viewport)
	FLinearColor BackwardsSolveBorderColor;
	
	/**
	 * The border color of the viewport when entering "Backwards And Forwards" mode
	 */
	UPROPERTY(EditAnywhere, config, Category = Viewport)
	FLinearColor BackwardsAndForwardsBorderColor;

	/**
	 * Show or hide the schematic view icons in the modular rig viewport
	 */
	UPROPERTY(EditAnywhere, config, Category = Viewport)
	bool bShowSchematicViewInModularRig;

	/**
	 * When set to true, only shows empty sockets in the schematic viewport
	 */
	UPROPERTY(EditAnywhere, config, Category = Viewport)
	bool bSchematicViewportShowEmptySocketsOnly = true;

	/**
	 * Option to toggle displaying the stacked hierarchy items.
	 * Note that changing this option potentially requires to re-open the editors in question. 
	 */
	UPROPERTY(EditAnywhere, config, Category = Hierarchy)
	bool bShowStackedHierarchy;

	/**
 	 * The maximum number of stacked items in the view 
 	 * Note that changing this option potentially requires to re-open the editors in question. 
 	 */
	UPROPERTY(EditAnywhere, config, Category = Hierarchy, meta = (EditCondition = "bShowStackedHierarchy"))
	int32 MaxStackSize;

	/**
	 * If turned on we'll offer box / marquee selection in the control rig editor viewport.
	 */
	UPROPERTY(EditAnywhere, config, Category = Hierarchy)
	bool bLeftMouseDragDoesMarquee;

	/**
	 * If turned on the controls in the Anim Outliner will be arranged by modules in Modular Rigs.
	 */
	UPROPERTY(EditAnywhere, config, Category = Outliner)
	bool bArrangeByModules;

	/**
	 * If turned on the modules in the Anim Outliner will be arranged in a flat list.
	 */
	UPROPERTY(EditAnywhere, config, Category = Outliner)
	bool bFlattenModules;

	/**
	 *  Outliner reflection mode for multi rigs 
	 */
	UPROPERTY(EditAnywhere, config, Category = Outliner)
	EMultiRigTreeDisplayMode OutlinerMultiRigDisplayMode;

	/**
	 * Defines how the element names will be displayed in treeviews, anim outliner etc
	 */
	UPROPERTY(EditAnywhere, config, Category = Outliner)
	EElementNameDisplayMode ElementNameDisplayMode;

	/**
	 * If turned on the Anim Outliner will focus on the selection.
	 */
	UPROPERTY(EditAnywhere, config, Category = Outliner)
	bool bFocusOnSelection;

	/** Whether to show all nulls in the hierarchy */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplayNulls = false;

	/** Should we show sockets in the viewport */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplaySockets = false;

	/** When true, controls are hidden in editor */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings", meta = (EditCondition = "!bHideControlShapes"))
	bool bHideControlShapes = false;

	/** Determines if controls should be rendered on top of other controls */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings", meta = (EditCondition = "!bHideControlShapes"))
	bool bShowControlsAsOverlay = false;

	/** Should we show axes for the selected elements */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplayAxesOnSelection = true;

	/** The scale for axes to draw on the selection */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	float AxisScale = 10.f;

	/** The mirror settings to use when mirroring rig elements */
	UPROPERTY(config)
	FRigVMMirrorSettings MirrorSettings;

	/** Hidden columns in the modular rig hierarchy tree view */
	UPROPERTY(config)
	TArray<FName> ModularRigHierarchyHiddenColums = { "ModuleClass", "ModuleTags" };

	/** When true the Event Queue is reset to Forward Solve after compiling */
	UPROPERTY(config)
	bool bCompileResetsEventQueue = true;
	
private:
	/**
	 * Visibility flags for different types of connectors in the rig hierarchy.
	 * Private since UProperties don't serialize enum flags correctly, hence flags are stored as uint8. Use related methods to access.
	 */
	UPROPERTY(config)
	uint8 ModularRigHierarchyConnectorVisibilityFlags = static_cast<uint8>(EModularRigHierarchyEditorConnectorVisibilityFlags::Default);

#endif // WITH_EDITORONLY_DATA

public:
#if WITH_EDITOR
	/** Gets the modular rig hierarchy connector visibility flags */
	UE_API EModularRigHierarchyEditorConnectorVisibilityFlags GetModularRigHierarchyConnectorVisibilityFlags() const;

	/** Returns true if these settings have any modular rig hierarchy connector visibility flags */
	UE_API bool HasAnyModularRigHierarchyConnectorVisibilityFlags(const EModularRigHierarchyEditorConnectorVisibilityFlags Flags) const;

	/** Sets the modular rig hierarchy connector visibility flags */
	UE_API void SetModularRigHierarchyConnectorVisibilityFlags(const EModularRigHierarchyEditorConnectorVisibilityFlags Flags);

	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
			SaveConfig();
		}
		
		OnSettingChanged().Broadcast(this, PropertyChangedEvent);
	}
#endif // WITH_EDITOR

	static UControlRigEditorSettings * Get() { return GetMutableDefault<UControlRigEditorSettings>(); }
};

#undef UE_API
