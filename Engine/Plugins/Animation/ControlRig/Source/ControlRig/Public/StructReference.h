// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructReference.generated.h"

/** 
 * Base class used to reference a struct in the graph. Don't use this directly, only derived classes.
 * Use the IMPLEMENT_STRUCT_REFERENCE macro to create new struct reference types easily.
 */
USTRUCT()
struct FStructReference
{
	GENERATED_BODY()

	FStructReference()
		: StructPointer(nullptr)
	{}

protected:
	/** Get the struct that this references */
	template<typename StructType>
	const StructType* GetInternal() const
	{
		return static_cast<const StructType*>(StructPointer);
	}

	/** Set the struct that this references */
	template<typename StructType>
	void SetInternal(const StructType* InStructPointer)
	{
		StructPointer = InStructPointer;
	}

private:
	/** Pointer to the struct */
	const void* StructPointer;
};

/** Use this macro to implement new struct reference types */
#define IMPLEMENT_STRUCT_REFERENCE(StructType)								\
	const StructType* Get() const											\
	{																		\
		return GetInternal<StructType>();									\
	}																		\
																			\
	void Set(const StructType* InStructPointer)								\
	{																		\
		SetInternal<StructType>(InStructPointer);							\
	}
