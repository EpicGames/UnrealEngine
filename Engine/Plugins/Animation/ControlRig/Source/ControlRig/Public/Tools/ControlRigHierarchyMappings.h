// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class URigHierarchy;
class UNodeMappingContainer;
class FControlRigPoseAdapter;
class USkeletalMeshComponent;
struct FBoneContainer;
struct FBoneReference;
struct FPoseContext;
struct FControlRigIOSettings;
struct FCompactPose;

template <class T>
class TAutoConsoleVariable;

extern TAutoConsoleVariable<int32> CVarControlRigEnableAnimNodePerformanceOptimizations;

struct FControlRigHierarchyMappings
{
	FControlRigHierarchyMappings() = default;

	UE_API void InitializeInstance();

	UE_API void LinkToHierarchy(URigHierarchy* InHierarchy);

	bool CanExecute() const
	{
		return !bEnablePoseAdapter || PoseAdapter.IsValid();
	}

	bool IsPoseAdapterEnabled() const
	{
		return bEnablePoseAdapter;
	}

	void ResetRefPoseSetterHash()
	{
		RefPoseSetterHash.Reset();
	}

	UE_API void UpdateControlRigRefPoseIfNeeded(UControlRig* ControlRig
		, const UObject* InstanceObject
		, const USkeletalMeshComponent* SkeletalMeshComponent
		, const FBoneContainer& RequiredBones
		, bool bInSetRefPoseFromSkeleton
		, bool bIncludePoseInHash);

	UE_API void UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
		, URigHierarchy* InHierarchy
		, const FBoneContainer& InRequiredBones
		, const TArray<FBoneReference>& InInputBonesToTransfer
		, const TArray<FBoneReference>& InOutputBonesToTransfer
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInTransferPoseInGlobalSpace
		, bool bResetInputPoseToInitial);

	UE_API void UpdateInput(UControlRig* ControlRig
		, FPoseContext& InOutput
		, const FControlRigIOSettings& InInputSettings
		, const FControlRigIOSettings& InOutputSettings
		, TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInExecute
		, bool bInTransferInputPose
		, bool bInResetInputPoseToInitial
		, bool bInTransferPoseInGlobalSpace
		, bool bInTransferInputCurves);

	UE_API void UpdateOutput(UControlRig* ControlRig
		, FPoseContext& InOutput
		, const FControlRigIOSettings& InOutputSettings
		, TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInExecute
		, bool bInTransferPoseInGlobalSpace);

	const TArray<TPair<uint16, uint16>>& GetControlRigBoneInputMappingByIndex() const
	{
		return ControlRigBoneInputMappingByIndex;
	}

	TArray<TPair<uint16, uint16>>& GetControlRigBoneOutputMappingByIndex()
	{
		return ControlRigBoneOutputMappingByIndex;
	}

	const TMap<FName, uint16>& GetControlRigBoneInputMappingByName() const
	{
		return ControlRigBoneInputMappingByName;
	}

	TMap<FName, uint16>& GetControlRigBoneOutputMappingByName()
	{
		return ControlRigBoneOutputMappingByName;
	}

	UE_API bool CheckPoseAdapter() const;
	UE_API bool IsUpdateToDate(const URigHierarchy* InHierarchy) const;
	UE_API void PerformUpdateToDate(UControlRig* ControlRig
		, URigHierarchy* InHierarchy
		, const FBoneContainer& InRequiredBones
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInTransferPoseInGlobalSpace
		, bool bInResetInputPoseToInitial);

private:
	UE_API void SetEnablePoseAdapter(bool bInEnablePoseAdapter);

	// Update IO whithout PoseAdapter
	UE_API void UpdateInputOutputMappingIfRequiredImpl(UControlRig* InControlRig
		, URigHierarchy* InHierarchy
		, const FBoneContainer& InRequiredBones
		, const TArray<FBoneReference>& InInputBonesToTransfer
		, const TArray<FBoneReference>& InOutputBonesToTransfer
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer);

	/** Complete mapping from skeleton to control rig bone index */
	TArray<TPair<uint16, uint16>> ControlRigBoneInputMappingByIndex;
	TArray<TPair<uint16, uint16>> ControlRigBoneOutputMappingByIndex;

	/** Complete mapping from skeleton to curve name */
	TArray<TPair<uint16, FName>> ControlRigCurveMappingByIndex;

	/** Rig Hierarchy bone name to required array index mapping */
	TMap<FName, uint16> ControlRigBoneInputMappingByName;
	TMap<FName, uint16> ControlRigBoneOutputMappingByName;

	/** Rig Curve name to Curve mapping */
	TMap<FName, FName> ControlRigCurveMappingByName;

	TArray<bool> HierarchyCurveCopied;

	TSharedPtr<FControlRigPoseAdapter> PoseAdapter;

	// A hash to encode the pointer to anim instance
	TOptional<int32> RefPoseSetterHash;

	bool bEnablePoseAdapter = false;
};

#undef UE_API
