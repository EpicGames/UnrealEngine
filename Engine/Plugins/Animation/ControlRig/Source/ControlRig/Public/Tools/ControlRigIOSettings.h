// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigIOSettings.generated.h"

/** Struct defining the settings to override when driving a control rig */
USTRUCT()
struct FControlRigIOSettings
{
	GENERATED_BODY()

	FControlRigIOSettings()
		: bUpdatePose(true)
		, bUpdateCurves(true)
	{}

	static FControlRigIOSettings MakeEnabled()
	{
		return FControlRigIOSettings();
	}

	static FControlRigIOSettings MakeDisabled()
	{
		FControlRigIOSettings Settings;
		Settings.bUpdatePose = Settings.bUpdateCurves = false;
		return Settings;
	}

	UPROPERTY()
	bool bUpdatePose;

	UPROPERTY()
	bool bUpdateCurves;
};
