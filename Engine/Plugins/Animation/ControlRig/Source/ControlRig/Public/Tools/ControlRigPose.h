// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
/**
*  Data To Store and Apply the Control Rig Pose
*/

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "TransformNoScale.h"
#include "Engine/SkeletalMesh.h"
#include "ControlRigToolAsset.h"
#include "Tools/ControlRigPoseMirrorTable.h"
#include "Rigs/RigControlHierarchy.h"
#include "ControlRigPose.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
/**
* The Data Stored For Each Control in A Pose.
*/
USTRUCT(BlueprintType)
struct FRigControlCopy
{
	GENERATED_BODY()

		FRigControlCopy()
		: Name(NAME_None)
		, ControlType(ERigControlType::Transform)
		, ParentKey()
		, Value()
		, OffsetTransform(FTransform::Identity)
		, ParentTransform(FTransform::Identity)
		, LocalTransform(FTransform::Identity)
		, GlobalTransform(FTransform::Identity)

	{
	}

	FRigControlCopy(FRigControlElement* InControlElement, URigHierarchy* InHierarchy, bool bGetValues = true)
	{
		Name = InControlElement->GetFName();
		ControlType = InControlElement->Settings.ControlType;
		if (bGetValues)
		{
			Value = InHierarchy->GetControlValue(InControlElement, ERigControlValueType::Current);
			ParentKey = InHierarchy->GetFirstParent(InControlElement->GetKey());
			OffsetTransform = InHierarchy->GetControlOffsetTransform(InControlElement, ERigTransformType::CurrentLocal);

			ParentTransform = InHierarchy->GetParentTransform(InControlElement, ERigTransformType::CurrentGlobal);
			LocalTransform = InHierarchy->GetTransform(InControlElement, ERigTransformType::CurrentLocal);
			GlobalTransform = InHierarchy->GetTransform(InControlElement, ERigTransformType::CurrentGlobal);
		}
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Names")
	FName Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Type")
	ERigControlType ControlType;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Names")
	FRigElementKey ParentKey;

	UPROPERTY()
	FRigControlValue Value;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transforms")
	FTransform OffsetTransform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transforms")
	FTransform ParentTransform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transforms")
	FTransform LocalTransform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Transforms")
	FTransform GlobalTransform;

};

/**
* The Data Stored For Each Pose and associated Functions to Store and Paste It
*/
USTRUCT(BlueprintType)
struct FControlRigControlPose
{
	GENERATED_USTRUCT_BODY()

	FControlRigControlPose() {};
	FControlRigControlPose(UControlRig* InControlRig, bool bUseAll)
	{
		SavePose(InControlRig, bUseAll);
	}
	~FControlRigControlPose() = default;

	UE_API void SavePose(UControlRig* ControlRig, bool bUseAll, bool bSaveValues = true);
	UE_API void PastePose(UControlRig* ControlRig, bool bDoKey, bool bDoMirror);
	UE_API void SetControlMirrorTransform(bool bDoLocalSpace, UControlRig* ControlRig, const FName& Name, bool bIsMatched,
		const FTransform& GlobalTrnnsform, const FTransform& LocalTransform, bool bNotify, const  FRigControlModifiedContext& Context, bool bSetupUndo);
	UE_API void PastePoseInternal(UControlRig* ControlRig, bool bDoKey, bool bDoMirror, const TArray<FRigControlCopy>& ControlsToPaste);
	UE_API void BlendWithInitialPoses(FControlRigControlPose& InitialPose, UControlRig* ControlRig, bool bDoKey, bool bDoMirror, float BlendValue, bool bDoAdditive = false);

	UE_API bool ContainsName(const FName& Name) const;
	UE_API void ReplaceControlName(const FName& Name, const FName& NewName);
	UE_API TArray<FName> GetControlNames() const;

	UE_API void SetUpControlMap();
	TArray<FRigControlCopy> GetPoses() const {return CopyOfControls;};

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Controls")
	TArray<FRigControlCopy> CopyOfControls;

	//Cache of the Map, Used to make pasting faster.
	TMap<FName, int32>  CopyOfControlsNameToIndex;
};


/**
* An individual Pose made of Control Rig Controls
*/
UCLASS(MinimalAPI, BlueprintType)
class UControlRigPoseAsset : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	//UOBJECT
	UE_API virtual void PostLoad() override;

	UFUNCTION(BlueprintCallable, Category = "Pose")
	UE_API void SavePose(UControlRig* InControlRig, bool bUseAll, bool bSaveValues = true);

	UFUNCTION(BlueprintCallable, Category = "Pose")
	UE_API void PastePose(UControlRig* InControlRig, bool bDoKey = false, bool bDoMirror = false, bool bDoAdditive = false);
	
	UFUNCTION(BlueprintCallable, Category = "Pose")
	UE_API void SelectControls(UControlRig* InControlRig, bool bDoMirror = false, bool bClearSelection = true);

	UE_API TArray<FRigControlCopy> GetCurrentPose(UControlRig* InControlRig);

	UFUNCTION(BlueprintCallable, Category = "Pose")
	UE_API void GetCurrentPose(UControlRig* InControlRig, FControlRigControlPose& OutPose);

	UFUNCTION(BlueprintPure, Category = "Pose")
	UE_API TArray<FName> GetControlNames() const;

	UFUNCTION(BlueprintCallable, Category = "Pose")
	UE_API void ReplaceControlName(const FName& CurrentName, const FName& NewName);

	UE_DEPRECATED(5.7, "Function is no longer needed.")
	UFUNCTION(BlueprintCallable, Category = "Pose")
	void SetUpMirrorMatchTable(UControlRig* InControlRig){};


	UFUNCTION(BlueprintPure, Category = "Pose")
	UE_API bool DoesMirrorMatch(UControlRig* ControlRig, const FName& ControlName);

	UE_API void BlendWithInitialPoses(FControlRigControlPose& InitialPose, UControlRig* ControlRig, bool bDoKey, bool bdoMirror, float BlendValue, bool bDoAdditive = false);

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Pose")
	FControlRigControlPose Pose;

private:
	FControlRigPoseMirrorTable MirrorMatchTable;
};

#undef UE_API
