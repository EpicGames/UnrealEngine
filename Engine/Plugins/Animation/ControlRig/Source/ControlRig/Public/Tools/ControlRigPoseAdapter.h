// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rigs/RigHierarchyPoseAdapter.h"
#include "Rigs/RigHierarchyDefines.h"
#include "BonePose.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class URigHierarchy;
class UNodeMappingContainer;
struct FBoneContainer;
struct FBlendedCurve;
struct FRigElementKeyAndIndex;

class FControlRigPoseAdapter : public FRigHierarchyPoseAdapter
{
public:
	FControlRigPoseAdapter() = default;

	// --- FRigHierarchyPoseAdapter overrides start ---
	UE_API virtual void PostLinked(URigHierarchy* InHierarchy) override;
	UE_API virtual void PreUnlinked(URigHierarchy* InHierarchy) override;
	UE_API virtual bool IsUpdateToDate(const URigHierarchy* InHierarchy) const override;
	// --- FRigHierarchyPoseAdapter overrides end ---

	const TArray<int32>& GetBonesToResetToInitial() const
	{
		return BonesToResetToInitial;
	}

	const TArray<FTransform>& GetLocalPose() const
	{
		return LocalPose;
	}

	const TArray<FTransform>& GetGlobalPose() const
	{
		return GlobalPose;
	}

	void CopyBonesFrom(const FCompactPose& InPose)
	{
		InPose.CopyBonesTo(LocalPose);
	}

	const TMap<FName, int32>& GetHierarchyCurveLookup() const
	{
		return HierarchyCurveLookup;
	}

	bool GetTransferInLocalSpace() const
	{
		return bTransferInLocalSpace;
	}

	UE_API void SetHierarchyCurvesLookup(const TArray<FRigBaseElement*>& InHierarchyCurves);

	const TArray<int32>& GetPoseCurveToHierarchyCurve() const
	{
		return PoseCurveToHierarchyCurve;
	}

	UE_API void SetPoseCurveToHierarchyCurve(const TArray<FRigBaseElement*>& InHierarchyCurves, const FBlendedCurve& InCurve);

	UE_API void UpdateInputOutputMappingIfRequired(UControlRig* InControlRig
		, URigHierarchy* InHierarchy
		, const FBoneContainer& InRequiredBones
		, const TWeakObjectPtr<UNodeMappingContainer>& InNodeMappingContainer
		, bool bInTransferPoseInGlobalSpace
		, bool bInResetInputPoseToInitial);

	UE_API void ConvertToLocalPose();
	UE_API void ConvertToGlobalPose();
	UE_API const FTransform& GetLocalTransform(int32 InIndex);
	UE_API const FTransform& GetGlobalTransform(int32 InIndex);

	UE_API void UpdateDirtyStates(const TOptional<bool> InLocalIsPrimary = TOptional<bool>());
	UE_API void ComputeDependentTransforms();
	UE_API void MarkDependentsDirty();

	UE_API void UnlinkTransformStorage();

protected:
	struct FDependentTransform
	{
		FDependentTransform()
			: KeyAndIndex()
			, TransformType(ERigTransformType::InitialLocal)
			, StorageType(ERigTransformStorageType::Pose)
			, DirtyState(nullptr)
		{}

		FDependentTransform(const FRigElementKeyAndIndex& InKeyAndIndex, ERigTransformType::Type InTransformType, ERigTransformStorageType::Type InStorageType, FRigLocalAndGlobalDirtyState* InDirtyState)
			: KeyAndIndex(InKeyAndIndex)
			, TransformType(InTransformType)
			, StorageType(InStorageType)
			, DirtyState(InDirtyState)
		{}

		FRigElementKeyAndIndex KeyAndIndex;
		ERigTransformType::Type TransformType;
		ERigTransformStorageType::Type StorageType;
		FRigLocalAndGlobalDirtyState* DirtyState;
	};

	TArray<FCompactPoseBoneIndex> ParentPoseIndices;
	TArray<bool> RequiresHierarchyForSpaceConversion;
	TArray<FTransform> LocalPose;
	TArray<FTransform> GlobalPose;
	TArray<bool> LocalPoseIsDirty;
	TArray<bool> GlobalPoseIsDirty;
	TArray<int32> PoseCurveToHierarchyCurve;
	
	TMap<FName, int32> HierarchyCurveLookup;
	TArray<int32> BonesToResetToInitial;

	TMap<uint16, uint16> ElementIndexToPoseIndex;
	TArray<int32> PoseIndexToElementIndex;

	TArray<FDependentTransform> Dependents;

	bool bTransferInLocalSpace = true;
	bool bRequiresResetPoseToInitial = true;
	bool bEnabled = true;

	friend struct FAnimNode_ControlRigBase;
};

#undef UE_API
