// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
/**
* Per Project Settings for the Mirroring, in particular what axis to use to mirror and the matching strings.
*/
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ControlRigPoseMirrorSettings.generated.h"

#define UE_API CONTROLRIG_API


UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class 
UControlRigPoseMirrorSettings : public UObject
{

public:
	GENERATED_BODY()

	UE_API UControlRigPoseMirrorSettings();

	UE_DEPRECATED(5.7, "This property is no longer used, instead we use the MirrorFindExpressions defined in UAnimationSettings")
	UPROPERTY()
	FString RightSide;

	UE_DEPRECATED(5.7, "This property is no longer used, instead we use the MirrorFindExpressions defined in UAnimationSettings")
	UPROPERTY()
	FString LeftSide;

	// the axis to mirror translations against
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Mirror Settings", meta = (ToolTip = "Axis to Mirror Translations"))
	TEnumAsByte<EAxis::Type> MirrorAxis;

	// the axis to flip for rotations
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Mirror Settings", meta = (ToolTip = "Axis to Flip for Rotations"))
	TEnumAsByte<EAxis::Type> AxisToFlip;

	// Tolerance to find the matching mirror control
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Mirror Settings", meta = (DisplayName = "Tolerance", ToolTip = "Tolerance to find the matching mirror control"))
	double MirrorMatchTolerance;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};


#undef UE_API
