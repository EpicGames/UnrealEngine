// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

/**
* Class to hold information on how a Pose May be Mirrored
*
*/

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Tools/MirrorCalculator.h"

#define UE_API CONTROLRIG_API

class UControlRig;
struct FRigControl;
struct FRigControlCopy;
struct FControlRigControlPose;
struct FMirrorFindReplaceExpression;

struct FControlRigPoseMirrorContext
{
	bool bDoLocal = true;
	bool bIsMatched = false;
	//values to modify the translation or rotation when bIsMatched == true;
	FVector TranslationFactor = FVector(1.0, 1.0, 1.0);
	FVector RotationFactor = FVector(1.0, 1.0, 1.0);

};

struct FControlRigPoseMirrorTable
{
public:
	FControlRigPoseMirrorTable() {};
	~FControlRigPoseMirrorTable() = default;

	/*Get the matched control with the given name*/
	UE_API FRigControlCopy* GetControl(const UControlRig* ControlRig, FControlRigControlPose& Pose, FName ControlrigName, bool bDoMiror);

	/*Whether or not the Control with this name is matched*/
	UE_API bool IsMatched(const UControlRig* ControlRig,const FName& ControlName);

	UE_DEPRECATED(5.8, "Use the version that uses FControlRigPoseMirrorContext")
	UE_API void GetMirrorTransform(const FRigControlCopy& ControlCopy, bool bDoLocal, bool bIsMatched,FTransform& OutGlobalTransform, FTransform& OutLocalTransform) const;

	/*Return the Mirrored Global(In Control Rig Space) and Mirrored Local */
	UE_API void GetMirrorTransform(const FRigControlCopy& ControlCopy, const FControlRigPoseMirrorContext& InContext, FTransform& OutGlobalTransform, FTransform& OutLocalTransform) const;

	UE_API void Reset();

	UE_API static FString GetMirrorString(const FString& InNameString, const TArray<FMirrorFindReplaceExpression>& MirrorFindReplaceExpressions);
private:
	/*Set up the Mirror Table*/
	void SetUpMirrorTable(const UControlRig* ControlRig);

	struct FMatchedControls
	{
		TMap<FName, FName> Match;
	};
	//cache of matches, note raw pointer is fine for UControlRig, just used for lookup not accessed.
	static TMap<const UControlRig*,FMatchedControls>  MatchedControls;

};

#undef UE_API
