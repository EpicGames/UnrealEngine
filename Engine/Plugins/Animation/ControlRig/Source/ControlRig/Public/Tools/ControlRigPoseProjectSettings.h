// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
/**
* Per Project user settings for Control Rig Poses(and maybe animations etc).
*/
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "ControlRigPoseProjectSettings.generated.h"

#define UE_API CONTROLRIG_API


UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UControlRigPoseProjectSettings : public UObject
{

public:
	GENERATED_BODY()

	UE_API UControlRigPoseProjectSettings();

	/** The pose asset path  */
	UE_API TArray<FString> GetAssetPaths() const;

	/** The root of the directory in which to save poses */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Control Rig Poses", meta = (ContentDir))
	TArray<FDirectoryPath> RootSaveDirs;

	/** Not used but may put bad if we support other types.
	bool bFilterPoses;

	bool bFilterAnimations;

	bool bFilterSelectionSets;
	*/
};

#undef UE_API
