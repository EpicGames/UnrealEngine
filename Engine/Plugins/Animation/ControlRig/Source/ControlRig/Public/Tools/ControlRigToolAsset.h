// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/PackageName.h"
#include "UObject/Object.h"
#include "ControlRig.h"
#include "Tools/ControlRigPoseProjectSettings.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ControlRigObjectBinding.h"
#include "Components/SkeletalMeshComponent.h"

/**
Create One of the Control Rig Assets, Currently Only support Pose Assets but may add Animation/Selection Sets.
*/
class FControlRigToolAsset
{
public:
	template< typename Type> static  UObject* SaveAsset(UControlRig* InControlRig, const FString& CurrentPath, const FString& InString, bool bUseAllControls)
	{
		if (InControlRig)
		{
			FString InPackageName = CurrentPath;
			InPackageName += FString("/") + InString;
			if (!FPackageName::IsValidLongPackageName(InPackageName))
			{
				FText ErrorText = FText::Format(NSLOCTEXT("ControlRigToolAsset", "InvalidPathError", "{0} is not a valid asset path."), FText::FromString(InPackageName));
				FNotificationInfo NotifyInfo(ErrorText);
				NotifyInfo.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(NotifyInfo)->SetCompletionState(SNotificationItem::CS_Fail);
				return nullptr;
			}

			int32 UniqueIndex = 1;
			int32 BasePackageLength = InPackageName.Len();

			// Generate a unique control asset name for this take if there are already assets of the same name
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			TArray<FAssetData> OutAssetData;
			AssetRegistry.GetAssetsByPackageName(*InPackageName, OutAssetData);
			while (OutAssetData.Num() > 0)
			{
				int32 TrimCount = InPackageName.Len() - BasePackageLength;
				if (TrimCount > 0)
				{
					InPackageName.RemoveAt(BasePackageLength, TrimCount, EAllowShrinking::No);
				}

				InPackageName += FString::Printf(TEXT("_%04d"), UniqueIndex++);
				OutAssetData.Empty();
				AssetRegistry.GetAssetsByPackageName(*InPackageName, OutAssetData);
			}

			// Create the asset to record into
			const FString NewAssetName = FPackageName::GetLongPackageAssetName(InPackageName);
			UPackage* NewPackage = CreatePackage(*InPackageName);


			// Create a new Pose From Scracth
			Type* Asset = NewObject<Type>(NewPackage, *NewAssetName, RF_Public | RF_Standalone | RF_Transactional);
			USkeletalMesh* SkelMesh = nullptr;
			if (USkeletalMeshComponent* RigMeshComp = Cast<USkeletalMeshComponent>(InControlRig->GetObjectBinding()->GetBoundObject()))
			{
				SkelMesh = RigMeshComp->GetSkeletalMeshAsset();
			}

			Asset->SavePose(InControlRig, bUseAllControls);
			Asset->MarkPackageDirty();
			FAssetRegistryModule::AssetCreated(Asset);
			return Asset;

		}
		return nullptr;
	}
};



