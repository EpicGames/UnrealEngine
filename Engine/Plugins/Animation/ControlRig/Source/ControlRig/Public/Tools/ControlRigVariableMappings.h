// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigAssetReference.h"
#include "Misc/Optional.h"
#include "Animation/NamedValueArray.h"
#include "Rigs/RigHierarchyDefines.h"
#include "RigVMCore/RigVMExternalVariable.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class USkeletalMeshComponent;
class FProperty;
class URigHierarchy;
struct FBlendedCurve;
struct FCompactPose;
struct FBoneContainer;
struct FRigControlElement;

struct FControlRigVariableMappings
{
	struct FControlRigCurveMapping
	{
		FControlRigCurveMapping() = default;

		FControlRigCurveMapping(FName InSourceName, FName InTargetName)
			: Name(InTargetName)
			, SourceName(InSourceName)
		{}

		FName Name = NAME_None;
		FName SourceName = NAME_None;
	};

	using FCurveMappings = UE::Anim::TNamedValueArray<FDefaultAllocator, FControlRigCurveMapping>;

	struct FCustomPropertyData
	{
		enum class ECustomPropertyType : uint8
		{
			Variable = 0,
			Control,
			// --- ---
			Num,
			Invalid = Num
		};

		FCustomPropertyData() = default;
		FCustomPropertyData(const FName& InTargetName, uint8* InTargetMemory, const FProperty* InProperty, const uint8* InSourceMemory, ECustomPropertyType InType)
			: TargetName(InTargetName)
			, TargetMemory(InTargetMemory)
			, Property(InProperty)
			, SourceMemory(InSourceMemory)
			, Type(InType)
		{}
		FCustomPropertyData(const FName& InTargetName, uint8* InTargetMemory, const FProperty* InProperty, const uint8* InSourceMemory, ECustomPropertyType InType, ERigControlType InControlType)
			: TargetName(InTargetName)
			, TargetMemory(InTargetMemory)
			, Property(InProperty)
			, SourceMemory(InSourceMemory)
			, Type(InType)
			, ControlType(InControlType)
		{}

		const FName TargetName = NAME_None;
		uint8* TargetMemory = nullptr;
		const FProperty* Property = nullptr;
		const uint8* SourceMemory = nullptr;
		ECustomPropertyType Type = ECustomPropertyType::Invalid;
		ERigControlType ControlType = ERigControlType::EulerTransform; // There is no invalid type for initialization
	};

	struct FCustomPropertyMappings
	{
		FCustomPropertyMappings() = default;
		explicit FCustomPropertyMappings(int32 ReserveSize)
		{
			Mappings.Reserve(ReserveSize);
		}

		void Reset(int32 NewSize = 0) { Mappings.Reset(NewSize); }

		void AddVariable(const FName& TargetName, uint8* TargetMemory, const FProperty* Property, const uint8* SourceMemory)
		{
			Mappings.Add(FCustomPropertyData(TargetName, TargetMemory, Property, SourceMemory, FCustomPropertyData::ECustomPropertyType::Variable));
		}
		void AddControl(ERigControlType InControlType, const FName& TargetName, uint8* TargetMemory, const FProperty* Property, const uint8* SourceMemory)
		{
			Mappings.Add(FCustomPropertyData(TargetName, TargetMemory, Property, SourceMemory, FCustomPropertyData::ECustomPropertyType::Control, InControlType));
		}

		const TArray<FCustomPropertyData>& GetMappings() const { return Mappings; }

	private:
		TArray<FCustomPropertyData> Mappings;
	};

	UE_API void InitializeProperties(const UClass* InSourceClass
		, UObject* TargetInstance
		, FControlRigAssetStrongReference InTargetClass
		, const TArray<FName>& SourcePropertyNames
		, TArray<FName>& DestPropertyNames);
	UE_API void InitializeProperties(const UClass* InSourceClass
		, UObject* TargetInstance
		, UClass* InTargetClass
		, const TArray<FName>& SourcePropertyNames
		, TArray<FName>& DestPropertyNames);

	UE_API bool RequiresInitAfterConstruction() const;

	UE_API void InitializeCustomProperties(UControlRig* TargetControlRig, const FCustomPropertyMappings& CustomPropertyMapping);
	UE_API void ResetCustomProperties(int32 NewSize = 0);

	UE_API void PropagateInputProperties(const UObject* InSourceInstance, UControlRig* InTargetInstance, TArray<FName>& InDestPropertyNames);
	UE_API void PropagateCustomInputProperties(UControlRig* InTargetControlRig);

	UE_API void UpdateCurveInputs(UControlRig* InControlRig, const TMap<FName, FName>& InputMapping, const FBlendedCurve& CurveData) const;

	UE_API void UpdateCurveOutputs(UControlRig* InControlRig, const TMap<FName, FName>& OutputMapping, FBlendedCurve& InCurveData) const;

	void ResetCurvesInputToControlCache()
	{
		CurveInputToControlIndex.Reset();
	}

	UE_API void CacheCurveMappings(const TMap<FName, FName>& InInputMappings, const TMap<FName, FName>& InOutputMappings, URigHierarchy* InHierarchy);

	const FCurveMappings& GetInputCurveMappings()
	{
		return InputCurveMappings;
	}

	const FCurveMappings& GetOutputCurveMappings()
	{
		return OutputCurveMappings;
	}

private:
	using PropertyUpdateFunction = TFunction<void(URigHierarchy*, const UObject*)>;
	using CustomPropertyUpdateFunction = TFunction<void()>;

	/*
	 * Cached functions used to copy property values to avoid looking for them for every PropagateInputProperties calls.
	 * They are stored in InitializeProperties and basically represent 'set value' functions on controls and variables.
	 */
	UE_API void AddControlFunction(FRigControlElement* InControlElement, FProperty* InSourceProperty, URigHierarchy* InTargetHierarchy);
	UE_API void AddUpdateFunction(PropertyUpdateFunction&& InFunction);
	UE_API void AddCustomUpdateFunction(CustomPropertyUpdateFunction&& InFunction);
	UE_API void AddVariableFunction(const FRigVMExternalVariable& InVariable, FProperty* InSourceProperty);

	/**
	 * Cached functions for non property based source data
	 */
	UE_API void AddCustomControlFunction(FRigControlElement* InControlElement, ERigControlType InControlType, const uint8* InSourcePropertyMemory, URigHierarchy* InTargetHierarchy);
	UE_API void AddCustomVariableFunction(const FRigVMExternalVariable& InVariable, const FProperty* InSourceProperty, const uint8* InSourcePropertyMemory);
	
	/* This function is used to copy the property values going thru the hierarchy without any caching mechanism. */
	UE_API void PropagateInputPropertiesNoCache(const UObject* InSourceInstance, const UControlRig* InTargetControlRig, URigHierarchy* InTargetHierarchy, TArray<FName>& InDestPropertyNames);

	UE_API void CacheCurveMappings(const TMap<FName, FName>& InMapping, FControlRigVariableMappings::FCurveMappings& InCurveMappings, URigHierarchy* InHierarchy);

	TArray<FProperty*> SourceProperties;
	TArray<FProperty*> DestProperties;

	TArray<PropertyUpdateFunction> UpdateFunctions;
	TArray<CustomPropertyUpdateFunction> CustomUpdateFunctions;
	TArray<FRigVMExternalVariable> Variables;

	TArray<const uint8*> SourcePropertiesMemory;
	TArray<FName> DestPropertyNames;

	// Bulk curves for I/O
	FCurveMappings InputCurveMappings;
	FCurveMappings OutputCurveMappings;

	TMap<FName, int32> CurveInputToControlIndex;
};

#undef UE_API
