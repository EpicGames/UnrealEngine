// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Math/Vector.h"
#include "Math/Axis.h"
#include "Misc/Guid.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

namespace UE
{
namespace AIE
{

/**
* Item To Mirror, will get set by the Callee
**/
struct FMirrorItem
{
	FGuid Guid;
	FVector Location;

	FMirrorItem() = delete;
	FMirrorItem(const FVector& InLocation) : Location(InLocation) { Guid = FGuid::NewGuid(); }
	FMirrorItem(const FMirrorItem&) = default;
};

//indices from an array of FMirrorItems
struct FMirrorItemResults
{
	//mirrored items will containt both A, B and B, A pairs.
	TMap<int32, int32> MirroredItems;
	//not mirrored and along the plane of mirroring
	TSet<int32> ExactlyOnAxis;
	//not mirrored and not along the plane of mirrorin.
	TSet<int32> NotOnAxis;
};

/**
	Calculates a set of mirrored item's based upon it's location, and the mirror plane defined by mirror axis and the mirror axis location.
**/
struct FMirrorCalculator
{
	/*
		Calculate the mirror results given a set of FMirrorItems, the axis, and plane location.
		Will return true if at least one item is mirrored
	*/
	bool FindMirroredItems(const TArray<FMirrorItem>& Items, FMirrorItemResults& OutMirrorItemResults,
		TEnumAsByte<EAxis::Type> MirrorAxis = EAxis::X, double AxisLocation = 0.0, double Tolerance = 1e-3f);

private:
	/*
	* Compare two items already know wone is is less than the AxisLocation, the other is greater than
	*/
	inline bool	IsMirror(const FMirrorItem& ItemLess, const FMirrorItem& ItemGreater, double AxisLocation, double Tolerance,
		int32 MirrorIndex, int32 OtherIndex[2])
	{
		return (FMath::IsNearlyEqual(AxisLocation - ItemLess.Location[MirrorIndex], ItemGreater.Location[MirrorIndex] - AxisLocation, Tolerance) &&
			FMath::IsNearlyEqual(ItemLess.Location[OtherIndex[0]], ItemGreater.Location[OtherIndex[0]], Tolerance) &&
			FMath::IsNearlyEqual(ItemLess.Location[OtherIndex[1]], ItemGreater.Location[OtherIndex[1]], Tolerance));
	}
};

} //AIE
} //UE


