// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SampleTrack.h"
#include "SampleTrackContainer.h"

template<typename T>
class FComposedSampleTrack : public FSampleTrack<T>
{
public:

	FComposedSampleTrack()
	{
		FSampleTrackBase::Mode = FSampleTrackBase::EMode_Singleton;
		FSampleTrack<T>::ValuesStorage.AddDefaulted(1);
		FSampleTrack<T>::UpdateValueArrayView();
	}
	
	FComposedSampleTrack(const FName& InName)
	: FSampleTrack<T>(InName)
	{
		FSampleTrackBase::Mode = FSampleTrackBase::EMode_Singleton;
		FSampleTrack<T>::ValuesStorage.AddDefaulted(1);
		FSampleTrack<T>::UpdateValueArrayView();
	}
	
	virtual bool IsComposed() const override { return true; } 

	virtual TArray<FSampleTrackBase::ETrackType> GetChildTrackTypes() const override
	{
		return {};
	}

	virtual FString GetChildTrackNameSuffix(int32 InChildTrackIndex) const override
	{
		return FSampleTrack<T>::GetChildTrackNameSuffix(InChildTrackIndex);
	}

	virtual ~FComposedSampleTrack() override = default;

	// returns the value of this track at a given time index
	virtual T GetValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const override
	{
		// this needs to be implemented in template specializations
		return FSampleTrack<T>::GetValueAtTimeIndex(InTimeIndex, InOutIndex);
	}
	
	// adds a new sample to this track
	virtual void AddSample(const T& InValue, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		// this needs to be implemented in template specializations
	}

	virtual void Reserve(int32 InSampleCount, int32 InValueCount) override
	{
		FSampleTrack<T>::Reserve(InSampleCount, InValueCount);
		for(const TSharedPtr<FSampleTrackBase>& ChildTrack : ChildTracks)
		{
			ChildTrack->Reserve(InSampleCount, InValueCount);
		}
	}

	virtual void Shrink() override
	{
		FSampleTrack<T>::Shrink();
		for(const TSharedPtr<FSampleTrackBase>& ChildTrack : ChildTracks)
		{
			ChildTrack->Shrink();
		}
	}

	virtual void Empty() override
	{
		FSampleTrack<T>::Empty();
		for(const TSharedPtr<FSampleTrackBase>& ChildTrack : ChildTracks)
		{
			ChildTrack->Empty();
		}
	}
	
	// returns true if this track is identical to another one
	virtual bool IsIdentical(const FSampleTrackBase* InOther, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		if(!FSampleTrack<T>::IsIdentical(InOther))
		{
			return false;
		}

		const FComposedSampleTrack<T>* OtherComposedTrack = static_cast<const FComposedSampleTrack<T>*>(InOther);
		
		if(ChildTracks.Num() != OtherComposedTrack->ChildTracks.Num())
		{
			return false;
		}

		for(int32 ChildTrackIndex = 0; ChildTrackIndex < ChildTracks.Num(); ChildTrackIndex++)
		{
			if(!ChildTracks[ChildTrackIndex]->IsIdentical(OtherComposedTrack->ChildTracks[ChildTrackIndex].Get(), InTolerance))
			{
				return false;
			}
		}
		
		return true;
	}

	virtual void Serialize(FArchive& InArchive) override
	{
		FSampleTrack<T>::Serialize(InArchive);
		InArchive << ChildTrackIndices;

		if(InArchive.IsLoading())
		{
			SetChildTracks(ChildTrackIndices);
		}
	}

protected:

	virtual int32 RemoveObsoleteTimes(int32 InNumTimesToRemove, const TArray<bool>& InRemoveTimeIndexArray, const TArray<int32>& InOldTimeIndexToNewTimeIndex) override
	{
		return 0;
	}
	
	virtual const TArray<int32>& GetChildTracks() const override
	{
		return ChildTrackIndices;
	}

	virtual void SetChildTracks(const TArray<int32>& InChildTrackIndices) override
	{
		ChildTrackIndices = InChildTrackIndices;

		check(FSampleTrackBase::Container);
		ChildTracks.Reset();
		for(const int32& ChildTrackIndex : ChildTrackIndices)
		{
			ChildTracks.Add(FSampleTrackBase::Container->GetTrack(ChildTrackIndex));
		}
	}

	virtual void UpdateChildTracks() override
	{
		SetChildTracks(ChildTrackIndices);
	}

public:
	
	/**
	 * Analyses the values stored and introduces the atlas index array,
	 * an indirection to save memory. After this Values is going to store unique elements,
	 * and the Atlas array is used to pick the right unique value for each sample index.
	 */
	virtual bool AddAtlas(bool bForce = false, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		return false;
	}

	/**
	 * Removes an existing Atlas indirection by flattening out all unique values into a larger, non-unique Value array
	 */
	virtual bool RemoveAtlas() override
	{
		return false;
	}

	/**
	 * Analyses the memory footprint of a sample track (just complete values vs time indices + values)
	 * and converts the track back to a complete representation for memory efficiency
	 */
	virtual bool ConvertToComplete(bool bForce = false) override
	{
		return false;
	}

	/**
	 * Adds all of the values back onto the track using a sampled representation
	 */
	virtual bool ConvertToSampled(bool bForce = false, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		return false;
	}

	/**
	 * Localizes values if this track is referenced
	 */
	virtual bool LocalizeValues() override
	{
		return false;
	}

protected:
	
	TArray<int32> ChildTrackIndices;
	TArray<TSharedPtr<FSampleTrackBase>> ChildTracks;
	
	friend class FSampleTrackContainer;
	friend class FControlRigBasicsSampleTrackTest;
};

template<>
CONTROLRIG_API TArray<FSampleTrackBase::ETrackType> FComposedSampleTrack<FTransform3f>::GetChildTrackTypes() const;

template<>
CONTROLRIG_API FString FComposedSampleTrack<FTransform3f>::GetChildTrackNameSuffix(int32 InChildTrackIndex) const;

template<>
CONTROLRIG_API FTransform3f FComposedSampleTrack<FTransform3f>::GetValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const;
	
template<>
CONTROLRIG_API void FComposedSampleTrack<FTransform3f>::AddSample(const FTransform3f& InValue, float InTolerance);
