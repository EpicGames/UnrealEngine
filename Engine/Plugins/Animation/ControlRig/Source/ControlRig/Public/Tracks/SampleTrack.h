// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SampleTrackBase.h"
#include "Math/UnrealMath.h"
#include "Math/UnrealMathUtility.h"

namespace UE::ControlRig::Internal
{

template<typename T>
class FSampleTrackShared : public FSampleTrackBase
{
public:

	FSampleTrackShared()
	{
	}
	
	FSampleTrackShared(const FName& InName)
	: FSampleTrackBase(InName)
	{
	}

	virtual ~FSampleTrackShared() override = default;

	// the number of stored values - which can be less than NumSamples if there is an atlas present
	virtual int32 NumStoredValues() const override
	{
		return Values.Num();
	}

	// returns the value of this track at a given time index
	virtual T GetValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const
	{
		verify(GetSampleIndexForTimeIndex(InTimeIndex, InOutIndex));
		return GetValueAtSampleIndex(InOutIndex.GetSample(TrackIndex));
	}
	
	// returns the value of this track at a given time index (overload for instanced structs only)
	template <
		typename StructType,
		typename TEnableIf<TIsInstancedStructCompliant<T, StructType>::Value, StructType>::Type* = nullptr
	>
	StructType GetValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const
	{
		check(StructType::StaticStruct() == ScriptStruct);
		const FInstancedStruct& InstancedStruct = GetValueAtTimeIndex(InTimeIndex, InOutIndex);
		StructType Result;
		check(InstancedStruct.GetScriptStruct() == ScriptStruct);
		ScriptStruct->CopyScriptStruct(&Result, InstancedStruct.GetMemory(), 1);
		return Result;
	}

	// returns the value of this track at a given time index (overload for instanced struct arrays only)
	template <
		typename StructArrayType,
		typename TEnableIf<TIsInstancedStructArrayCompliant<T, StructArrayType>::Value, StructArrayType>::Type* = nullptr
	>
	StructArrayType GetValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const
	{
		check(StructArrayType::ElementType::StaticStruct() == ScriptStruct);
		const TArray<FInstancedStruct>& InstancedStructArray = GetValueAtTimeIndex(InTimeIndex, InOutIndex);
		StructArrayType Result;
		Result.SetNum(InstancedStructArray.Num());
		for(int32 ElementIndex = 0; ElementIndex < InstancedStructArray.Num(); ElementIndex++)
		{
			check(InstancedStructArray[ElementIndex].GetScriptStruct() == ScriptStruct);
			ScriptStruct->CopyScriptStruct(&Result[ElementIndex], InstancedStructArray[ElementIndex].GetMemory(), 1);
		}
		return Result;
	}

	// returns the value of this track at a given time index (overload for instanced struct arrays only)
	template <
		typename StructType,
		typename TEnableIf<TIsInstancedStructArrayCompliant<T, TArray<StructType>>::Value, StructType>::Type* = nullptr
	>
	TArray<StructType> GetArrayValueAtTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const
	{
		return GetValueAtTimeIndex<TArray<StructType>>(InTimeIndex, InOutIndex);
	}

	// returns the value of this track at a given sample index
	virtual const T& GetValueAtSampleIndex(int32 InSampleIndex) const
	{
		if(UsesAtlas(InSampleIndex))
		{
			return Values[Atlas[InSampleIndex]];
		}
		return Values[InSampleIndex];
	}

	// returns the value of this track at a given sample index (overload for instanced structs only)
	template <
		typename StructType,
		typename TEnableIf<TIsInstancedStructCompliant<T, StructType>::Value, StructType>::Type* = nullptr
	>
	StructType GetValueAtSampleIndex(int32 InSampleIndex) const
	{
		check(StructType::StaticStruct() == ScriptStruct);
		const FInstancedStruct& InstancedStruct = GetValueAtSampleIndex(InSampleIndex);
		StructType Result;
		check(InstancedStruct.GetScriptStruct() == ScriptStruct);
		ScriptStruct->CopyScriptStruct(&Result, InstancedStruct.GetMemory(), 1);
		return Result;
	}

	// returns the value of this track at a given sample index (overload for instanced struct arrays only)
	template <
		typename StructType,
		typename TEnableIf<TIsInstancedStructArrayCompliant<T, TArray<StructType>>::Value, StructType>::Type* = nullptr
	>
	TArray<StructType> GetValueAtSampleIndex(int32 InSampleIndex) const
	{
		return GetValueAtSampleIndex<TArray<StructType>>(InSampleIndex);
	}

	// returns the value of this track at a given sample index (overload for instanced struct arrays only)
	template <
		typename StructArrayType,
		typename TEnableIf<TIsInstancedStructArrayCompliant<T, StructArrayType>::Value, StructArrayType>::Type* = nullptr
	>
	StructArrayType GetValueAtSampleIndex(int32 InSampleIndex) const
	{
		check(StructArrayType::ElementType::StaticStruct() == ScriptStruct);
		const TArray<FInstancedStruct>& InstancedStructArray = GetValueAtSampleIndex(InSampleIndex);
		StructArrayType Result;
		Result.SetNum(InstancedStructArray.Num());
		for(int32 ElementIndex = 0; ElementIndex < InstancedStructArray.Num(); ElementIndex++)
		{
			check(InstancedStructArray[ElementIndex].GetScriptStruct() == ScriptStruct);
			ScriptStruct->CopyScriptStruct(&Result[ElementIndex], InstancedStructArray[ElementIndex].GetMemory(), 1);
		}
		return Result;
	}

	// adds a new sample to this track
	virtual void AddSample(const T& InValue, float InTolerance = KINDA_SMALL_NUMBER)
	{
		check(!IsReferenced());
		
		if(ValuesStorage.IsEmpty())
		{
			AddTimeIndex(false);
			ValuesStorage.Add(InValue);
			UpdateValueArrayView();
		}
		else if(Equals(ValuesStorage.Last(), InValue, InTolerance))
		{
			AddTimeIndex(true);
		}
		else
		{
			AddTimeIndex(false);
			ValuesStorage.Add(InValue);
			UpdateValueArrayView();
		}
	}

	// adds a new sample to this track (overload for instanced struct only) 
	template <
		typename StructType,
		typename TEnableIf<TIsInstancedStructCompliant<T, StructType>::Value, T>::Type* = nullptr
	>
	void AddSample(const StructType& InStructValue, float InTolerance = KINDA_SMALL_NUMBER)
	{
		check(StructType::StaticStruct() == ScriptStruct);
		FInstancedStruct InstancedStruct;
		InstancedStruct.InitializeAs(ScriptStruct, (uint8*)&InStructValue);
		AddSample(InstancedStruct, InTolerance);
	}

	// adds a new sample to this track (overload for instanced struct array only) 
	template <
		typename StructArrayType,
		typename TEnableIf<TIsInstancedStructArrayCompliant<T, StructArrayType>::Value, T>::Type* = nullptr
	>
	void AddSample(const StructArrayType& InStructArrayValue, float InTolerance = KINDA_SMALL_NUMBER)
	{
		check(StructArrayType::ElementType::StaticStruct() == ScriptStruct);
		TArray<FInstancedStruct> InstancedStructArray;
		InstancedStructArray.SetNum(InStructArrayValue.Num());
		for(int32 ElementIndex = 0; ElementIndex < InStructArrayValue.Num(); ElementIndex++)
		{
			InstancedStructArray[ElementIndex].InitializeAs(ScriptStruct, (uint8*)&InStructArrayValue[ElementIndex]);
		}
		AddSample(InstancedStructArray, InTolerance);
	}

	// returns the type of this track
	virtual ETrackType GetTrackType() const override
	{
		return ETrackType_Unknown;
	}

	virtual void Reset() override
	{
		FSampleTrackBase::Reset();
		ValuesStorage.Reset();
		UpdateValueArrayView();
	}

	virtual void Reserve(int32 InSampleCount, int32 InValueCount) override
	{
		FSampleTrackBase::Reserve(InSampleCount, InValueCount);
		ValuesStorage.Reserve(InValueCount);
		UpdateValueArrayView();
	}

	virtual void Shrink() override
	{
		FSampleTrackBase::Shrink();
		ValuesStorage.Shrink();
		UpdateValueArrayView();
	}

	virtual void Empty() override
	{
		FSampleTrackBase::Empty();
		ValuesStorage.Empty();
		UpdateValueArrayView();
	}
	
	// returns true if this track is identical to another one
	virtual bool IsIdentical(const FSampleTrackBase* InOther, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		check(InOther);

		if(!FSampleTrackBase::IsIdentical(InOther, InTolerance))
		{
			return false;
		}

		const FSampleTrackShared<T>* Other = static_cast<const FSampleTrackShared<T>*>(InOther);
		if(NumSamples() != Other->NumSamples())
		{
			return false;
		}

		for(int32 SampleIndex = 0; SampleIndex < NumSamples(); SampleIndex++)
		{
			if(!Equals(GetValueAtSampleIndex(SampleIndex), Other->GetValueAtSampleIndex(SampleIndex), InTolerance))
			{
				return false;
			}
		}

		return true;
	}

	const FSampleTrackShared* GetCastReferencedTrack() const
	{
		return static_cast<const FSampleTrackShared*>(GetReferencedTrack());
	}
	
	FSampleTrackShared* GetCastReferencedTrack()
	{
		return static_cast<FSampleTrackShared*>(GetReferencedTrack());
	}

protected:

	/**
	 * Removes samples from the track which don't present any value anywhere.
	 * This may only shift time indices and not actually remove any value.
	 * Data passed in here is slightly redundant, but passed for performance reasons.
	 * @param InNumTimesToRemove The number of samples to remove
	 * @param InRemoveTimeIndexArray A bool array with a true value for each time index to remove (.Count(true) == InNumTimesToRemove
	 * @param InOldTimeIndexToNewTimeIndex An array mapping the time indices from old to new
	 */
	virtual int32 RemoveObsoleteTimes(int32 InNumTimesToRemove, const TArray<bool>& InRemoveTimeIndexArray, const TArray<int32>& InOldTimeIndexToNewTimeIndex) override
	{
		UpdateArrayViews();
		
		// remember the previous indices
		TArray<int32> OldTimeIndices;
		if(GetMode() == EMode_Sampled)
		{
			OldTimeIndices = TimeIndices;
		}
		
		const int32 NumRemoved = FSampleTrackBase::RemoveObsoleteTimes(InNumTimesToRemove, InRemoveTimeIndexArray, InOldTimeIndexToNewTimeIndex);
		if(NumRemoved > 0)
		{
			TArray<T> NewValues;
			if(GetMode() == EMode_Complete)
			{
				check(NumTimesInContainer == NumSamples());

				NewValues.Reserve(NumSamples() - NumRemoved);
				for(int32 SampleIndex = 0; SampleIndex < NumSamples(); SampleIndex++)
				{
					if(!InRemoveTimeIndexArray[SampleIndex])
					{
						NewValues.Add(Values[SampleIndex]);
					}
				}
			}
			else
			{
				TArray<bool> PerSampleRemoveIndex;
				PerSampleRemoveIndex.Reserve(OldTimeIndices.Num());
				for(const int32 OldTimeIndex : OldTimeIndices)
				{
					PerSampleRemoveIndex.Add(InRemoveTimeIndexArray[OldTimeIndex]);
				}

				NewValues.Reserve(OldTimeIndices.Num() - NumRemoved);
				for(int32 Index = 0; Index < OldTimeIndices.Num(); Index++)
				{
					if(!PerSampleRemoveIndex[Index])
					{
						NewValues.Add(Values[Index]);
					}
				}
			}
			ValuesStorage = MoveTemp(NewValues);
			AtlasStorage.Reset();
			UpdateArrayViews();
		}
		return NumRemoved;
	}

public:
	virtual int32 GetSizePerValue() const = 0;
	
	static bool IsArrayValue()
	{
		return TIsArray<T>::Value;
	}

	/**
	 * Analyses the values stored and introduces the atlas index array,
	 * an indirection to save memory. After this Values is going to store unique elements,
	 * and the Atlas array is used to pick the right unique value for each sample index.
	 */
	virtual bool AddAtlas(bool bForce = false, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		UpdateArrayViews();
		
		if(UsesAtlas() || !IsValid() || GetMode() == EMode_Singleton)
		{
			return false;
		}

		TArray<uint32> HashPerValue;
		TMap<uint32, TTuple<int32, int32>> HashToValueIndex;
		HashPerValue.Reserve(NumStoredValues());
		HashToValueIndex.Reserve(NumStoredValues());

		int32 NumNewValues = 0;
		TArray<int32> OldValueIndexToNewValueIndex;
		OldValueIndexToNewValueIndex.Reserve(NumStoredValues());

		for(int32 SampleIndex = 0; SampleIndex < NumStoredValues(); SampleIndex++)
		{
			const uint32 Hash = GetValueHash(GetValueAtSampleIndex(SampleIndex));
			HashPerValue.Add(Hash);

			if(const TTuple<int32, int32>* IndexPair = HashToValueIndex.Find(Hash))
			{
				if(Equals(GetValueAtSampleIndex(SampleIndex), GetValueAtSampleIndex(IndexPair->Get<0>()), InTolerance))
				{
					OldValueIndexToNewValueIndex.Add(IndexPair->Get<1>());
				}
				else
				{
					OldValueIndexToNewValueIndex.Add(NumNewValues++);
				}
			}
			else
			{
				const int32 NewValueIndex =  NumNewValues++;
				HashToValueIndex.Add(Hash, TTuple<int32, int32>(SampleIndex, NewValueIndex));
				OldValueIndexToNewValueIndex.Add(NewValueIndex);
			}
		}

		// if each value is unique we don't need an atlas
		if(NumNewValues == NumStoredValues())
		{
			return false;
		}

		if(!bForce)
		{
			// if using the atlas will increase the memory footprint - stick with the current representation
			const int32 SizePerValue = GetSizePerValue();
			const int32 CurrentSize = SizePerValue * NumStoredValues();
			const int32 SizeWithAtlas = SizePerValue * HashToValueIndex.Num() + Atlas.GetTypeSize() * HashPerValue.Num();
			if(CurrentSize <= SizeWithAtlas)
			{
				return false;
			}
		}

		TArray<T> NewValues;
		NewValues.Reserve(NumNewValues);
		TArray<int32> NewAtlas;
		NewAtlas.Reserve(NumStoredValues());

		for(int32 Index = 0; Index < OldValueIndexToNewValueIndex.Num(); Index++)
		{
			const int32& NewValueIndex = OldValueIndexToNewValueIndex[Index];
			if(!NewValues.IsValidIndex(NewValueIndex))
			{
				verify(NewValueIndex == NewValues.Add(GetValueAtSampleIndex(Index)));
			}
			NewAtlas.Add(NewValueIndex);
		}

		ValuesStorage = MoveTemp(NewValues);
		AtlasStorage = MoveTemp(NewAtlas);
		UpdateArrayViews();
		return true;
	}

	/**
	 * Removes an existing Atlas indirection by flattening out all unique values into a larger, non-unique Value array
	 */
	virtual bool RemoveAtlas() override
	{
		UpdateArrayViews();

		if(!UsesAtlas())
		{
			return false;
		}

		TArray<T> NewValues;
		NewValues.Reserve(NumSamples());
		for(int32 SampleIndex = 0; SampleIndex < NumSamples(); SampleIndex++)
		{
			NewValues.Add(GetValueAtSampleIndex(SampleIndex));
		}

		ValuesStorage = MoveTemp(NewValues);
		AtlasStorage.Reset();
		UpdateArrayViews();
		return true;
	}

	/**
	 * Analyses the memory footprint of a sample track (just complete values vs time indices + values)
	 * and converts the track back to a complete representation for memory efficiency
	 */
	virtual bool ConvertToComplete(bool bForce = false) override
	{
		UpdateArrayViews();

		if(IsArrayValue() || GetMode() != EMode_Sampled || IsReferenced())
		{
			return false;
		}

		if(!bForce)
		{
			const int32 SizePerValue = GetSizePerValue();
			const int32 CurrentSize = TimeIndices.GetTypeSize() * TimeIndices.Num() + SizePerValue * NumStoredValues();
			const int32 CompleteSize = SizePerValue * NumTimesInContainer;
			if(CurrentSize <= CompleteSize)
			{
				return false;
			}
		}

		if(UsesAtlas())
		{
			TArray<int32> NewAtlas;
			NewAtlas.Reserve(NumTimesInContainer);
			FSampleTrackIndex SampleTrackIndex = FSampleTrackIndex::MakeSingleton();
			for(int32 TimeIndex = 0; TimeIndex < NumTimesInContainer; TimeIndex++)
			{
				NewAtlas.Add(Atlas[GetSampleIndexForTimeIndex(TimeIndex, SampleTrackIndex)]);
			}
			AtlasStorage = MoveTemp(NewAtlas);
		}
		else
		{
			TArray<T> NewValues;
			NewValues.Reserve(NumTimesInContainer);
			FSampleTrackIndex SampleIndex = FSampleTrackIndex::MakeSingleton();
			for(int32 TimeIndex = 0; TimeIndex < NumTimesInContainer; TimeIndex++)
			{
				NewValues.Add(GetValueAtTimeIndex(TimeIndex, SampleIndex));
			}
			ValuesStorage = MoveTemp(NewValues);
		}
		
		Mode = EMode_Complete;
		TimeIndicesStorage.Reset();
		UpdateArrayViews();
		return true;
	}

	/**
	 * Adds all of the values back onto the track using a sampled representation
	 */
	virtual bool ConvertToSampled(bool bForce = false, float InTolerance = KINDA_SMALL_NUMBER) override
	{
		if(GetMode() != EMode_Complete || IsReferenced())
		{
			return false;
		}

		UpdateArrayViews();

		if(!bForce)
		{
			int32 NumSampledValues = 1;
			int32 LastAddedIndex = 0;
			for(int32 SampleIndex = 1; SampleIndex < NumSamples(); SampleIndex++)
			{
				if(!Equals(GetValueAtSampleIndex(LastAddedIndex), GetValueAtSampleIndex(SampleIndex), InTolerance))
				{
					LastAddedIndex = SampleIndex;
					NumSampledValues++;
				}
			}
			
			const int32 SizePerValue = GetSizePerValue();
			const int32 CompleteSize = SizePerValue * NumTimesInContainer;
			const int32 SampledSizeSize = TimeIndices.GetTypeSize() * NumSampledValues + SizePerValue * NumSampledValues;
			if(CompleteSize <= SampledSizeSize)
			{
				return false;
			}
		}

		RemoveAtlas();
		
		TArray<T> CopyOfValues;
		Swap(CopyOfValues, ValuesStorage);
		Reset();
		Mode = EMode_Invalid;
		UpdateArrayViews();

		// add the values back
		for(const T& Value : CopyOfValues)
		{
			AddSample(Value, InTolerance);
		}

		Shrink();

		return true;
	}

	/**
	 * Localizes values if this track is referenced
	 */
	virtual bool LocalizeValues() override
	{
		if(FSampleTrackBase::LocalizeValues())
		{
			ValuesStorage = Values;
			UpdateValueArrayView();

			// remove and add the atlas back to optimize the size of the values
			if(RemoveAtlas())
			{
				AddAtlas();
			}
			Shrink();
			return true;
		}
		return false;
	}

protected:
	
	virtual int32 AppendValuesFromTrack(const FSampleTrackBase* InOtherTrack) override
	{
		const int32 FirstValueIndex = ValuesStorage.Num(); 
		ValuesStorage.Append(static_cast<const FSampleTrackShared*>(InOtherTrack)->ValuesStorage);
		UpdateValueArrayView();
		return FirstValueIndex;
	}

	virtual void UpdateArrayViews() override
	{
		FSampleTrackBase::UpdateArrayViews();
		UpdateValueArrayView();
	}

	void UpdateValueArrayView()
	{
		if(IsReferenced())
		{
			FSampleTrackShared* ReferencedTrack = GetCastReferencedTrack();
			if(ReferencedTrack == nullptr)
			{
				Values = TArrayView<T>();
				return;
			}

			if(ReferencedValuesRange.Get<1>() <= 0)
			{
				Values = TArrayView<T>();
			}
			else
			{
				check(ReferencedTrack->ValuesStorage.IsValidIndex(ReferencedValuesRange.Get<0>()));
				check(ReferencedTrack->ValuesStorage.IsValidIndex(ReferencedValuesRange.Get<0>() + ReferencedValuesRange.Get<1>() - 1));
			
				Values = TArrayView<T>(
					ReferencedTrack->ValuesStorage.GetData() + ReferencedValuesRange.Get<0>(),
					ReferencedValuesRange.Get<1>());
			}
		}
		else
		{
			Values = TArrayView<T>(ValuesStorage.GetData(), ValuesStorage.Num());
		}
	}

	TArray<T> ValuesStorage;
	TArrayView<T> Values;
	
	friend class FSampleTrackContainer;
	friend class FControlRigBasicsSampleTrackTest;
};

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<bool>::GetTrackType() const
{
	return ETrackType_Bool;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<int32>::GetTrackType() const
{
	return ETrackType_Int32;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<uint32>::GetTrackType() const
{
	return ETrackType_Uint32;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<float>::GetTrackType() const
{
	return ETrackType_Float;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FName>::GetTrackType() const
{
	return ETrackType_Name;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FString>::GetTrackType() const
{
	return ETrackType_String;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FVector3f>::GetTrackType() const
{
	return ETrackType_Vector3f;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FQuat4f>::GetTrackType() const
{
	return ETrackType_Quatf;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FTransform3f>::GetTrackType() const
{
	return ETrackType_Transformf;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FLinearColor>::GetTrackType() const
{
	return ETrackType_LinearColor;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FRigElementKey>::GetTrackType() const
{
	return ETrackType_ElementKey;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FRigComponentKey>::GetTrackType() const
{
	return ETrackType_ComponentKey;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<FInstancedStruct>::GetTrackType() const
{
	return ETrackType_Struct;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<bool>>::GetTrackType() const
{
	return ETrackType_BoolArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<int32>>::GetTrackType() const
{
	return ETrackType_Int32Array;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<uint32>>::GetTrackType() const
{
	return ETrackType_Uint32Array;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<float>>::GetTrackType() const
{
	return ETrackType_FloatArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FName>>::GetTrackType() const
{
	return ETrackType_NameArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FString>>::GetTrackType() const
{
	return ETrackType_StringArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FVector3f>>::GetTrackType() const
{
	return ETrackType_Vector3fArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FQuat4f>>::GetTrackType() const
{
	return ETrackType_QuatfArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FTransform3f>>::GetTrackType() const
{
	return ETrackType_TransformfArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FLinearColor>>::GetTrackType() const
{
	return ETrackType_LinearColorArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FRigElementKey>>::GetTrackType() const
{
	return ETrackType_ElementKeyArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FRigComponentKey>>::GetTrackType() const
{
	return ETrackType_ComponentKeyArray;
}

template<>
inline FSampleTrackBase::ETrackType FSampleTrackShared<TArray<FInstancedStruct>>::GetTrackType() const
{
	return ETrackType_StructArray;
}
}


template<typename T>
class FSampleTrack : public UE::ControlRig::Internal::FSampleTrackShared<T>
{
	using Base = UE::ControlRig::Internal::FSampleTrackShared<T>;
public:
	FSampleTrack() = default;
	FSampleTrack(const FName& InName) : Base(InName) {}
	virtual ~FSampleTrack() override = default;
	
	virtual void Serialize(FArchive& InArchive) override
	{
		FSampleTrackBase::Serialize(InArchive);
		InArchive << Base::ValuesStorage;
		Base::UpdateValueArrayView();
	}

	virtual int32 GetSizePerValue() const override
	{
		return sizeof(T);
	}

	// helper to store a sample based on a given property
	virtual void AddSampleFromProperty(const FProperty* InProperty, const uint8* InMemory) override
	{
		T Value;
		InProperty->CopyCompleteValue(&Value, InMemory);
		Base::AddSample(Value);
	}

	// helper to retrieve a sample based on a given property
	virtual void GetSampleForProperty(int32 InTimeIndex, FSampleTrackIndex& InOutSampleTrackIndex, const FProperty* InProperty, uint8* OutMemory) const override
	{
		const T& Value = Base::GetValueAtTimeIndex(InTimeIndex, InOutSampleTrackIndex);
		InProperty->CopyCompleteValue(OutMemory, &Value);
	}
	
	friend class FControlRigBasicsSampleTrackTest;
};

template<>
class FSampleTrack<FInstancedStruct> : public UE::ControlRig::Internal::FSampleTrackShared<FInstancedStruct>
{
	using Base = FSampleTrackShared<FInstancedStruct>;
public:
	FSampleTrack() = default;
	FSampleTrack(const FName& InName) : Base(InName) {}
	virtual ~FSampleTrack() override = default;
	
	// Specialized functions for this type.
	CONTROLRIG_API virtual void Serialize(FArchive& InArchive) override;
	CONTROLRIG_API virtual int32 GetSizePerValue() const override;
	CONTROLRIG_API virtual void AddSampleFromProperty(const FProperty* InProperty, const uint8* InMemory) override;
	CONTROLRIG_API virtual void GetSampleForProperty(int32 InTimeIndex, FSampleTrackIndex& InOutSampleTrackIndex, const FProperty* InProperty, uint8* OutMemory) const override;
	
	friend class FControlRigBasicsSampleTrackTest;
};

template<>
class FSampleTrack<TArray<FInstancedStruct>> : public UE::ControlRig::Internal::FSampleTrackShared<TArray<FInstancedStruct>>
{
	using Base = FSampleTrackShared<TArray<FInstancedStruct>>;
public:
	FSampleTrack() = default;
	FSampleTrack(const FName& InName) : Base(InName) {}
	virtual ~FSampleTrack() override = default;

	// Specialized functions for this type.
	CONTROLRIG_API virtual void Serialize(FArchive& InArchive) override;
	virtual int32 GetSizePerValue() const override
	{
		return sizeof(TArray<FInstancedStruct>);
	}
	CONTROLRIG_API virtual void AddSampleFromProperty(const FProperty* InProperty, const uint8* InMemory) override;
	CONTROLRIG_API virtual void GetSampleForProperty(int32 InTimeIndex, FSampleTrackIndex& InOutSampleTrackIndex, const FProperty* InProperty, uint8* OutMemory) const override;

	friend class FControlRigBasicsSampleTrackTest;
};
