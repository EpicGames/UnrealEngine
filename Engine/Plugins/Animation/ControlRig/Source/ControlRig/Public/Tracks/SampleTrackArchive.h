// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "SampleTrackArchive.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FSampleTrackMemoryData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<uint8> Buffer;

	UPROPERTY()
	TArray<FName> Names;

	UPROPERTY()
	TArray<FString> ObjectPaths;
	
	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FSampleTrackMemoryData& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

class FSampleTrackMemoryWriter : public FMemoryWriter
{
public:
	FSampleTrackMemoryWriter() = delete;
	UE_API FSampleTrackMemoryWriter(FSampleTrackMemoryData& InOutData, bool bIsPersistent = false);
	
	using FMemoryWriter::operator<<; // For visibility of the overloads we don't override

	UE_API virtual FArchive& operator<<(FName& Value) override;
	UE_API virtual FArchive& operator<<(FText& Value) override;
	UE_API virtual FArchive& operator<<(class UObject*& Value) override;
	UE_API virtual FArchive& operator<<(struct FObjectPtr& Value) override;

protected:
	FSampleTrackMemoryData& Data;
	TMap<FName,int32> NameToIndex;
	TMap<UObject*, int32> ObjectToIndex;
};

class FSampleTrackMemoryReader : public FMemoryReader
{
public:
	FSampleTrackMemoryReader() = delete;
	UE_API FSampleTrackMemoryReader(FSampleTrackMemoryData& InOutData, bool bIsPersistent = false);
	
	using FMemoryReader::operator<<; // For visibility of the overloads we don't override
	
	UE_API virtual FArchive& operator<<(FName& Value) override;
	UE_API virtual FArchive& operator<<(FText& Value) override;
	UE_API virtual FArchive& operator<<(class UObject*& Value) override;
	UE_API virtual FArchive& operator<<(struct FObjectPtr& Value) override;

protected:
	
	FSampleTrackMemoryData& Data;
	TArray<UObject*> Objects;
};

#undef UE_API
