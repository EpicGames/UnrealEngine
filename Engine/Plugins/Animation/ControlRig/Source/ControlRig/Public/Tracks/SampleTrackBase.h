// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "StructUtils/InstancedStruct.h"
#include "Rigs/RigHierarchyDefines.h"

#define UE_API CONTROLRIG_API

class FSampleTrackContainer;

struct FSampleTrackIndex
{
public:
	UE_API FSampleTrackIndex();
	UE_API explicit FSampleTrackIndex(int32 NumTracks);
	UE_API explicit FSampleTrackIndex(const FSampleTrackContainer& InContainer);
	UE_API int32& GetSample(int32 InTrackIndex);
	static UE_API FSampleTrackIndex MakeSingleton();
	UE_API void Update(const FSampleTrackContainer& InContainer);

private:
	void Allocate(int32 NumTracks);

	TArray<int32, TInlineAllocator<1>> Samples;
	bool bIsSingleton;
};

class FSampleTrackBase : public TSharedFromThis<FSampleTrackBase>
{
public:

	enum ETrackType : uint8
	{
		ETrackType_Unknown,
		ETrackType_Bool,
		ETrackType_Int32,
		ETrackType_Uint32,
		ETrackType_Float,
		ETrackType_Name,
		ETrackType_String,
		ETrackType_Vector3f,
		ETrackType_Quatf,
		ETrackType_Transformf,
		ETrackType_LinearColor,
		ETrackType_ElementKey,
		ETrackType_ComponentKey,
		ETrackType_Struct,
		ETrackType_BoolArray,
		ETrackType_Int32Array,
		ETrackType_Uint32Array,
		ETrackType_FloatArray,
		ETrackType_NameArray,
		ETrackType_StringArray,
		ETrackType_Vector3fArray,
		ETrackType_QuatfArray,
		ETrackType_TransformfArray,
		ETrackType_LinearColorArray,
		ETrackType_ElementKeyArray,
		ETrackType_ComponentKeyArray,
		ETrackType_StructArray
	};
	
	enum EMode : uint8
	{
		EMode_Invalid, // no data whatsoever
		EMode_Singleton, // a single, constant value
		EMode_Sampled, // values at certain time indices, each addressed time index is stored in TimeIndices
		EMode_Complete, // a value for each time index, TimeIndices is empty
		EMode_Raw, // unstructured data - used for referencing into from other tracks
	};
	
	UE_API FSampleTrackBase();
	UE_API FSampleTrackBase(const FName& InName);
	virtual ~FSampleTrackBase() = default;

	bool IsValid() const { return Mode != EMode_Invalid; }

	// the first name of this track
	UE_API const FName& GetName() const;

	// all names this track is using / this track represents
	const TArray<FName>& GetAllNames() const { return Names; }

	// the index of the track within the container
	int32 GetTrackIndex() const { return TrackIndex; }

	// the script struct the track is using if it is storing values of FInstancedStruct
	const UScriptStruct* GetScriptStruct() const { return ScriptStruct; }

	// the type of track
	virtual ETrackType GetTrackType() const { return ETrackType_Unknown; }

	// the mode in which the track is storing its data
	EMode GetMode() const { return Mode; }

	// the number of global time indices within the container
	UE_API int32 NumTimes() const;

	// the number of samples stored within this track
	UE_API int32 NumSamples() const;

	// the number of stored values - which can be less than NumSamples if there is an atlas present
	virtual int32 NumStoredValues() const = 0;

	// returns the sample index within the track given a time index
	UE_API bool GetSampleIndexForTimeIndex(int32 InTimeIndex, FSampleTrackIndex& InOutIndex) const;

	// returns true if the track stores a value specifically at a given time index
	UE_API bool StoresValueForTimeIndex(int32 InTimeIndex) const;

	// returns true if this track is identical to another one
	UE_API virtual bool IsIdentical(const FSampleTrackBase* InOther, float InTolerance = KINDA_SMALL_NUMBER);

	UE_API virtual void Reserve(int32 InSampleCount, int32 InValueCount);
	UE_API virtual void Reset();
	UE_API virtual void Shrink();
	UE_API virtual void Empty();
	UE_API virtual void Serialize(FArchive& InArchive);

	template<typename T>
	static bool Equals(const T& InA, const T& InB, float InTolerance)
	{
		return InA == InB;
	}

	template<typename T>
	static bool ArrayEquals(const TArray<T>& A, const TArray<T>& B, float InTolerance)
	{
		if(A.Num() != B.Num())
		{
			return false;
		}
		for(int32 Index = 0; Index < A.Num(); Index++)
		{
			if(!Equals<T>(A[Index], B[Index], InTolerance))
			{
				return false;
			}
		}
		return true;
	}

	// returns true if this track is referencing another track
	bool IsReferenced() const { return ReferencedTrackIndex != INDEX_NONE; }

	// returns the index the track this track is referencing
	int32 GetReferencedTrackIndex() const { return ReferencedTrackIndex; }

	// returns true if this track is composed out of other tracks
	virtual bool IsComposed() const { return false; }

	// returns the required track types for the child tracks
	virtual TArray<FSampleTrackBase::ETrackType> GetChildTrackTypes() const { return {}; }

	// returns the name for a given child track
	virtual FString GetChildTrackNameSuffix(int32 InChildTrackIndex) const { return TEXT("Child"); }

	// helper to store a sample based on a given property
	virtual void AddSampleFromProperty(const FProperty* InProperty, const uint8* InMemory) = 0;

	// helper to retrieve a sample based on a given property
	virtual void GetSampleForProperty(int32 InTimeIndex, FSampleTrackIndex& InOutSampleTrackIndex, const FProperty* InProperty, uint8* OutMemory) const = 0;

protected:

	UE_API const FSampleTrackBase* GetReferencedTrack() const;
	UE_API FSampleTrackBase* GetReferencedTrack();
	UE_API virtual void UpdateArrayViews();
	UE_API void UpdateTimeAndAtlasArrayViews();
	UE_API bool UsesAtlas(int32 InAtlasIndex = INDEX_NONE) const;
	UE_API void AddTimeIndex(bool bOnlyIncreaseUpperBound);
	virtual int32 AppendValuesFromTrack(const FSampleTrackBase* InOtherTrack) = 0;
	UE_API virtual const TArray<int32>& GetChildTracks() const;
	virtual void SetChildTracks(const TArray<int32>& InChildTrackIndices) {}
	virtual void UpdateChildTracks() {}

	/**
	 * Removes samples from the track which don't present any value anywhere.
	 * This may only shift time indices and not actually remove any value.
	 * Data passed in here is slightly redundant, but passed for performance reasons.
	 * @param InNumTimesToRemove The number of samples to remove
	 * @param InRemoveTimeIndexArray A bool array with a true value for each time index to remove (.Count(true) == InNumTimesToRemove
	 * @param InOldTimeIndexToNewTimeIndex An array mapping the time indices from old to new
	 */
	UE_API virtual int32 RemoveObsoleteTimes(int32 InNumTimesToRemove, const TArray<bool>& InRemoveTimeIndexArray, const TArray<int32>& InOldTimeIndexToNewTimeIndex);

public:
	
	/**
	 * Analyses the values stored and introduces the atlas index array,
	 * an indirection to save memory. After this Values is going to store unique elements,
	 * and the Atlas array is used to pick the right unique value for each sample index.
	 */
	virtual bool AddAtlas(bool bForce = false, float InTolerance = KINDA_SMALL_NUMBER) = 0;

	/**
	 * Removes an existing Atlas indirection by flattening out all unique values into a larger, non-unique Value array
	 */
	virtual bool RemoveAtlas() = 0;

	/**
	 * Analyses the memory footprint of a sample track (just complete values vs time indices + values)
	 * and converts the track back to a complete representation for memory efficiency
	 */
	virtual bool ConvertToComplete(bool bForce = false) = 0;

	/**
	 * Adds all of the values back onto the track using a sampled representation
	 */
	virtual bool ConvertToSampled(bool bForce = false, float InTolerance = KINDA_SMALL_NUMBER) = 0;

	/**
	 * Localizes values if this track is referenced
	 */
	UE_API virtual bool LocalizeValues();
	
protected:

	struct CIsStruct
	{
		template <typename T>
		auto Requires(const UScriptStruct*& Val) -> decltype(
			Val = T::StaticStruct()
		);
	};
	
	struct CIsStructArray
	{
		template <typename T>
		auto Requires(const UScriptStruct*& Val) -> decltype(
			Val = T::ElementType::StaticStruct()
		);
	};

	template <typename T>
	struct TIsInstancedStruct
	{
		enum { Value = false };
	};

	template <typename T>
	struct TIsInstancedStructArray
	{
		enum { Value = false };
	};

	template <typename ValueType, typename StructType>
	struct TIsInstancedStructCompliant
	{
		enum
		{
			Value =
				TAnd<
					TIsInstancedStruct<ValueType>,
					TAnd<
						TModels<CIsStruct, StructType>,
						TNot<TIsInstancedStruct<StructType>>
					>
				>::Value
		};
	};

	template <typename ValueType, typename StructArrayType>
	struct TIsInstancedStructArrayCompliant
	{
		enum
		{
			Value =
				TAnd<
					TIsInstancedStructArray<ValueType>,
					TAnd<
						TModels<CIsStructArray, StructArrayType>,
						TNot<TIsInstancedStructArray<StructArrayType>>
					>
				>::Value
		};
	};

	template<typename T>
	static uint32 GetSingleValueHash(const T& InValue)
	{
		return GetTypeHash(InValue);
	}

	template<typename T>
	static uint32 GetValueHash(const T& InValue)
	{
		return GetSingleValueHash<T>(InValue);
	}

	template<typename T>
	static uint32 GetValueHash(const TArray<T>& InValue)
	{
		uint32 Hash = GetTypeHash(InValue.Num());
		for(const T& Element : InValue)
		{
			Hash = HashCombine(GetSingleValueHash<T>(Element));
		}
		return Hash;
	}

	TArray<FName> Names;
	int32 TrackIndex;
	int32 ReferencedTrackIndex;
	TTuple<int32, int32> ReferencedTimeIndicesRange;
	TTuple<int32, int32> ReferencedAtlasRange;
	TTuple<int32, int32> ReferencedValuesRange;
	const UScriptStruct* ScriptStruct;
	EMode Mode;
	TArray<int32> TimeIndicesStorage;
	TArrayView<int32> TimeIndices;
	TArray<int32> AtlasStorage;
	TArrayView<int32> Atlas;
	int32 NumTimesInContainer;
	FSampleTrackContainer* Container;

	friend class FSampleTrackContainer;
};

template <> struct FSampleTrackBase::TIsInstancedStruct<FInstancedStruct> { enum { Value = true }; };
template <> struct FSampleTrackBase::TIsInstancedStructArray<TArray<FInstancedStruct>> { enum { Value = true }; };

template<>
inline bool FSampleTrackBase::Equals<float>(const float& InA, const float& InB, float InTolerance)
{
	return FMath::IsNearlyEqual(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<FVector3f>(const FVector3f& InA, const FVector3f& InB, float InTolerance)
{
	return InA.Equals(InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<FQuat4f>(const FQuat4f& InA, const FQuat4f& InB, float InTolerance)
{
	return InA.Equals(InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<FTransform3f>(const FTransform3f& InA, const FTransform3f& InB, float InTolerance)
{
	return InA.Equals(InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<FLinearColor>(const FLinearColor& InA, const FLinearColor& InB, float InTolerance)
{
	return InA.Equals(InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<FInstancedStruct>(const FInstancedStruct& InA, const FInstancedStruct& InB, float InTolerance)
{
	return InA.Identical(&InB, PPF_None);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<bool>>(const TArray<bool>& InA, const TArray<bool>& InB, float InTolerance)
{
	return ArrayEquals<bool>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<int32>>(const TArray<int32>& InA, const TArray<int32>& InB, float InTolerance)
{
	return ArrayEquals<int32>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<uint32>>(const TArray<uint32>& InA, const TArray<uint32>& InB, float InTolerance)
{
	return ArrayEquals<uint32>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<float>>(const TArray<float>& InA, const TArray<float>& InB, float InTolerance)
{
	return ArrayEquals<float>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FName>>(const TArray<FName>& InA, const TArray<FName>& InB, float InTolerance)
{
	return ArrayEquals<FName>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FString>>(const TArray<FString>& InA, const TArray<FString>& InB, float InTolerance)
{
	return ArrayEquals<FString>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FVector3f>>(const TArray<FVector3f>& InA, const TArray<FVector3f>& InB, float InTolerance)
{
	return ArrayEquals<FVector3f>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FQuat4f>>(const TArray<FQuat4f>& InA, const TArray<FQuat4f>& InB, float InTolerance)
{
	return ArrayEquals<FQuat4f>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FTransform3f>>(const TArray<FTransform3f>& InA, const TArray<FTransform3f>& InB, float InTolerance)
{
	return ArrayEquals<FTransform3f>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FLinearColor>>(const TArray<FLinearColor>& InA, const TArray<FLinearColor>& InB, float InTolerance)
{
	return ArrayEquals<FLinearColor>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FRigElementKey>>(const TArray<FRigElementKey>& InA, const TArray<FRigElementKey>& InB, float InTolerance)
{
	return ArrayEquals<FRigElementKey>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FRigComponentKey>>(const TArray<FRigComponentKey>& InA, const TArray<FRigComponentKey>& InB, float InTolerance)
{
	return ArrayEquals<FRigComponentKey>(InA, InB, InTolerance);
}

template<>
inline bool FSampleTrackBase::Equals<TArray<FInstancedStruct>>(const TArray<FInstancedStruct>& InA, const TArray<FInstancedStruct>& InB, float InTolerance)
{
	return ArrayEquals<FInstancedStruct>(InA, InB, InTolerance);
}

template<>
inline uint32 FSampleTrackBase::GetSingleValueHash<FInstancedStruct>(const FInstancedStruct& InValue)
{
	if(InValue.IsValid())
	{
		uint32 Hash = GetTypeHash(InValue.GetScriptStruct());
		const int32 Size = InValue.GetScriptStruct()->GetStructureSize();
		for(int32 Index = 0; Index < Size; Index++)
		{
			Hash = HashCombine(Hash, GetTypeHash(InValue.GetMemory()[Index]));
		}
		return Hash;
	}
	return 0;
}

#undef UE_API
