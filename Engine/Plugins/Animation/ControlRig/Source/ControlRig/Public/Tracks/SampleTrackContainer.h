// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SampleTrack.h"
#include "Rigs/RigHierarchyDefines.h"
#include "SampleTrackContainer.generated.h"

#define UE_API CONTROLRIG_API

class FSampleTrackContainer;

USTRUCT()
struct FSampleTrackHost
{
public:
	GENERATED_BODY();

	UE_API FSampleTrackHost();
	UE_API FSampleTrackHost(const FSampleTrackHost& InOther);
	virtual ~FSampleTrackHost() = default;

	UE_API FSampleTrackHost& operator =(const FSampleTrackHost& InOther);

	UE_API virtual void Reset();
	UE_API virtual void Compact();

	/** By always returning false here, we can disable delta serialization */
	virtual bool Identical(const FSampleTrackHost* Other, uint32 PortFlags) const { return false; }

	UE_API virtual bool Serialize(FArchive& InArchive);

	UE_API const FSampleTrackContainer* GetContainer() const;
	UE_API FSampleTrackContainer* GetContainer();

	UE_API int32 AddTimeSample(float InAbsoluteTime, float InDeltaTime);
	UE_API int32 AddTimeSampleFromDeltaTime(float InDeltaTime);
	UE_API int32 GetNumTimes() const;
	UE_API FVector2f GetTimeRange() const;
	UE_API int32 GetTimeIndex(float InAbsoluteTime, FSampleTrackIndex& InOutTrackIndex) const;
	UE_API float GetAbsoluteTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const;
	UE_API float GetDeltaTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const;
	UE_API float GetAbsoluteTime(int32 InTimeIndex) const;
	UE_API int32 GetTimeIndex(float InAbsoluteTime) const;
	UE_API float GetDeltaTime(int32 InTimeIndex) const;
	UE_API float GetLastAbsoluteTime() const;
	UE_API float GetLastDeltaTime() const;

private:

	TUniquePtr<FSampleTrackContainer> Container;
};

template<>
struct TStructOpsTypeTraits<FSampleTrackHost> : public TStructOpsTypeTraitsBase2<FSampleTrackHost>
{
	enum 
	{
		WithSerializer = true, // struct has a Serialize function for serializing its state to an FArchive.
		WithCopy = true, // struct can be copied via its copy assignment operator.
		WithIdentical = true, // struct can be compared via an Identical(const T* Other, uint32 PortFlags) function.
	};
};

class FSampleTrackContainer : public TSharedFromThis<FSampleTrackContainer>
{
public:
	UE_API FSampleTrackContainer();
	UE_API ~FSampleTrackContainer();

	UE_API void Reset();
	UE_API void Shrink();
	UE_API void Compact(float InTolerance = KINDA_SMALL_NUMBER);
	UE_API void Reserve(int32 InNum);
	UE_API bool Serialize(FArchive& InArchive);
	UE_API void SetForceToUseCompression(bool InForce = true);

	UE_API TSharedPtr<FSampleTrackBase> AddTrack(const FName& InName, FSampleTrackBase::ETrackType InTrackType, const UScriptStruct* InScriptStruct = nullptr);
	UE_API TSharedPtr<FSampleTrackBase> FindOrAddTrack(const FName& InName, FSampleTrackBase::ETrackType InTrackType, const UScriptStruct* InScriptStruct = nullptr);

	template<typename T>
	TSharedPtr<FSampleTrack<T>> AddTrack(const FName& InName, FSampleTrackBase::ETrackType InTrackType, const UScriptStruct* InScriptStruct = nullptr)
	{
		check(FSampleTrack<T>().GetTrackType() == InTrackType);
		return StaticCastSharedPtr<FSampleTrack<T>>(AddTrack(InName, InTrackType, InScriptStruct));
	}
	
	template<typename T>
	TSharedPtr<FSampleTrack<T>> FindOrAddTrack(const FName& InName, FSampleTrackBase::ETrackType InTrackType, const UScriptStruct* InScriptStruct = nullptr)
	{
		check(FSampleTrack<T>().GetTrackType() == InTrackType);
		return StaticCastSharedPtr<FSampleTrack<T>>(FindOrAddTrack(InName, InTrackType, InScriptStruct));
	}
	
	UE_API TSharedPtr<FSampleTrack<bool>> AddBoolTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<int32>> AddInt32Track(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<uint32>> AddUint32Track(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<float>> AddFloatTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FName>> AddNameTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FString>> AddStringTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FVector3f>> AddVectorTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FQuat4f>> AddQuatTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FTransform3f>> AddTransformTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FLinearColor>> AddLinearColorTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FRigElementKey>> AddRigElementKeyTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FRigComponentKey>> AddRigComponentKeyTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<FInstancedStruct>> AddStructTrack(const FName& InName, const UScriptStruct* InScriptStruct);
	UE_API TSharedPtr<FSampleTrack<TArray<bool>>> AddBoolArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<int32>>> AddInt32ArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<uint32>>> AddUint32ArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<float>>> AddFloatArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FName>>> AddNameArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FString>>> AddStringArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FVector3f>>> AddVectorArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FQuat4f>>> AddQuatArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FTransform3f>>> AddTransformArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FLinearColor>>> AddLinearColorArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FRigElementKey>>> AddRigElementKeyArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FRigComponentKey>>> AddRigComponentKeyArrayTrack(const FName& InName);
	UE_API TSharedPtr<FSampleTrack<TArray<FInstancedStruct>>> AddStructArrayTrack(const FName& InName, const UScriptStruct* InScriptStruct);

	int32 NumTracks() const
	{
		return Tracks.Num();
	}

	UE_API int32 GetNumTimes() const;
	UE_API FVector2f GetTimeRange() const;
	UE_API int32 GetTimeIndex(float InAbsoluteTime, FSampleTrackIndex& InOutTrackIndex) const;
	UE_API int32 GetTimeIndex(float InAbsoluteTime) const;
	UE_API float GetAbsoluteTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const;
	UE_API float GetDeltaTime(int32 InTimeIndex, FSampleTrackIndex& InOutTrackIndex) const;
	UE_API float GetAbsoluteTime(int32 InTimeIndex) const;
	UE_API float GetDeltaTime(int32 InTimeIndex) const;
	UE_API float GetLastAbsoluteTime() const;
	UE_API float GetLastDeltaTime() const;

	UE_API int32 GetTrackIndex(const FName& InName) const;
	UE_API TSharedPtr<const FSampleTrackBase> GetTrack(int32 InIndex) const;
	UE_API TSharedPtr<FSampleTrackBase> GetTrack(int32 InIndex);
	UE_API TSharedPtr<const FSampleTrackBase> FindTrack(const FName& InName) const;
	UE_API TSharedPtr<FSampleTrackBase> FindTrack(const FName& InName);
	
	template<typename T>
	TSharedPtr<const FSampleTrack<T>> GetTrack(int32 InIndex) const
	{
		const TSharedPtr<const FSampleTrackBase> Track = GetTrack(InIndex);
#if WITH_EDITOR
		static const FSampleTrack<T> TypeTestTrack;
		check(TypeTestTrack.GetTrackType() == Track->GetTrackType());
#endif
		return StaticCastSharedPtr<const FSampleTrack<T>, const FSampleTrackBase>(Track);
	}

	template<typename T>
	TSharedPtr<FSampleTrack<T>> GetTrack(int32 InIndex)
	{
		const TSharedPtr<FSampleTrackBase> Track = GetTrack(InIndex);
#if WITH_EDITOR
		static const FSampleTrack<T> TypeTestTrack;
		check(TypeTestTrack.GetTrackType() == Track->GetTrackType());
#endif
		return StaticCastSharedPtr<FSampleTrack<T>, FSampleTrackBase>(Track);
	}

	template<typename T>
	TSharedPtr<const FSampleTrack<TArray<T>>> GetArrayTrack(int32 InIndex) const
	{
		return GetTrack<TArray<T>>(InIndex);
	}

	template<typename T>
	TSharedPtr<FSampleTrack<TArray<T>>> GetArrayTrack(int32 InIndex)
	{
		return GetTrack<TArray<T>>(InIndex);
	}

	TSharedPtr<const FSampleTrack<FInstancedStruct>> GetStructTrack(int32 InIndex) const
	{
		return GetTrack<FInstancedStruct>(InIndex);
	}

	TSharedPtr<FSampleTrack<FInstancedStruct>> GetStructTrack(int32 InIndex)
	{
		return GetTrack<FInstancedStruct>(InIndex);
	}

	TSharedPtr<const FSampleTrack<TArray<FInstancedStruct>>> GetStructArrayTrack(int32 InIndex) const
	{
		return GetArrayTrack<FInstancedStruct>(InIndex);
	}

	TSharedPtr<FSampleTrack<TArray<FInstancedStruct>>> GetStructArrayTrack(int32 InIndex)
	{
		return GetArrayTrack<FInstancedStruct>(InIndex);
	}

	template<typename T>
	TSharedPtr<const FSampleTrack<T>> FindTrack(const FName& InName) const
	{
		const int32 Index = GetTrackIndex(InName);
		if(Tracks.IsValidIndex(Index))
		{
			return GetTrack<T>(Index);
		}
		return nullptr;
	}

	template<typename T>
	TSharedPtr<FSampleTrack<T>> FindTrack(const FName& InName)
	{
		const int32 Index = GetTrackIndex(InName);
		if(Tracks.IsValidIndex(Index))
		{
			return GetTrack<T>(Index);
		}
		return nullptr;
	}

	template<typename T>
	TSharedPtr<const FSampleTrack<TArray<T>>> FindArrayTrack(const FName& InName) const
	{
		return FindTrack<TArray<T>>(InName);
	}

	template<typename T>
	TSharedPtr<FSampleTrack<TArray<T>>> FindArrayTrack(const FName& InName)
	{
		return FindTrack<TArray<T>>(InName);
	}

	TSharedPtr<const FSampleTrack<FInstancedStruct>> FindStructTrack(const FName& InName) const
	{
		return FindTrack<FInstancedStruct>(InName);
	}

	TSharedPtr<FSampleTrack<FInstancedStruct>> FindStructTrack(const FName& InName)
	{
		return FindTrack<FInstancedStruct>(InName);
	}

	TSharedPtr<const FSampleTrack<TArray<FInstancedStruct>>> FindStructArrayTrack(const FName& InName) const
	{
		return FindArrayTrack<FInstancedStruct>(InName);
	}

	TSharedPtr<FSampleTrack<TArray<FInstancedStruct>>> FindStructArrayTrack(const FName& InName)
	{
		return FindArrayTrack<FInstancedStruct>(InName);
	}
	
	UE_API int32 AddTimeSample(float InAbsoluteTime, float InDeltaTime);
	UE_API int32 AddTimeSampleFromDeltaTime(float InDeltaTime);

	// returns true if all tracks are complete / singleton tracks
	// without references and without atlases
	UE_API bool IsEditable() const;

	// makes this container editable, essentially the opposite of making it compact
	UE_API bool MakeEditable();

private:

	UE_API void AddTrack(TSharedPtr<FSampleTrackBase> InTrack, bool bCreateChildTracks = true);
	static UE_API TSharedPtr<FSampleTrackBase> MakeTrack(FSampleTrackBase::ETrackType InTrackType);

public:
	
	// removes all tracks with no data in them or nullptr tracks
	UE_API void RemoveInvalidTracks(bool bUpdateNameToIndexMap = true);

	// combines tracks that have the exact same data in them
	UE_API void RemoveRedundantTracks(bool bUpdateNameToIndexMap = true, float InTolerance = KINDA_SMALL_NUMBER);

	// combines tracks of the same type into a single, longer track and
	// spawns tracks referencing the larger one's sections
	UE_API void MergeTypedTracks(bool bUpdateNameToIndexMap = true, float InTolerance = KINDA_SMALL_NUMBER);

	// introduces a memory optimization to all tracks where necessary, by
	// which the value storage is moved to a unique store and an atlas index
	// array is stored (per sample index) to look up the unique value per sample.
	UE_API void EnableTrackAtlas(float InTolerance = KINDA_SMALL_NUMBER);

	/**
	 * Analyses the memory footprint of a sample track (just complete values vs time indices + values)
	 * and converts the track back to a complete representation for memory efficiency
	 */
	UE_API void ConvertTracksToComplete();

	/**
	 * After a track has been made editable - we can convert it back to sampled to save memory.
	 */
	UE_API void ConvertTracksToSampled(float InTolerance = KINDA_SMALL_NUMBER);

private:
	
	/**
	 * Updates referenced and child track indices given a list providing the new index for each track
	 */
	UE_API void UpdateTrackIndices(const TArray<int32>& InNewTrackIndices);

	// Updates the name lookup after larger changes within the container
	UE_API void UpdateNameToIndexMap();

	TMap<FName, int32> NameToIndex;
	TArray<TSharedPtr<FSampleTrackBase>> Tracks;
	bool bForceToUseCompression;
	mutable FSampleTrackIndex TimeSampleTrackIndex;

	static UE_API const FLazyName AbsoluteTimeName;
	static UE_API const FLazyName DeltaTimeName;
};

#undef UE_API
