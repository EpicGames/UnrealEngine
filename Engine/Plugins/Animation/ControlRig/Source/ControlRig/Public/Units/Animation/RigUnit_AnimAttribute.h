// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/SkeletalMesh.h"
#include "Units/RigUnit.h"
#include "Components/SkeletalMeshComponent.h"
#include "Units/RigDispatchFactory.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "ControlRigComponent.h"
#include "RigUnit_AnimAttribute.generated.h"

#define UE_API CONTROLRIG_API

namespace RigUnit_AnimAttribute
{
	template<typename T>
	class TAnimAttributeType;

	template<>
	class TAnimAttributeType<int>
	{
	public:
		using Type = FIntegerAnimationAttribute;
	};

	template<>
	class TAnimAttributeType<float>
	{
	public:
		using Type = FFloatAnimationAttribute;
	};

	template<>
	class TAnimAttributeType<FString>
	{
	public:
		using Type = FStringAnimationAttribute;
	};

	template<>
	class TAnimAttributeType<FTransform>
	{
	public:
		using Type = FTransformAnimationAttribute;
	};

	template<>
	class TAnimAttributeType<FVector>
	{
	public:
		using Type = FVectorAnimationAttribute;
	};

	template<>
	class TAnimAttributeType<FQuat>
	{
	public:
		using Type = FQuaternionAnimationAttribute;
	};


	template<typename T>
	T* GetAnimAttributeValue(
		bool bAddIfNotFound,
		const FControlRigExecuteContext& Context,
		const FName& Name,
		const FName& BoneName,
		FName& CachedBoneName,
		int32& CachedBoneIndex)
	{
		if (Name.IsNone())
		{
			return nullptr;
		}

		if (!Context.UnitContext.AnimAttributeContainer)
		{
			return nullptr;
		}
	
		const USkeletalMeshComponent* OwningComponent = Cast<USkeletalMeshComponent>(Context.GetOwningComponent());
		if (!OwningComponent)
		{
			if (const UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(Context.GetOwningComponent()))
			{
				if (ControlRigComponent->MappedElements.Num() > 0)
				{
					OwningComponent = Cast<USkeletalMeshComponent>(ControlRigComponent->MappedElements[0].SceneComponent);
				}				
			}
		}

		if (!OwningComponent ||
			!OwningComponent->GetSkeletalMeshAsset())
		{
			return nullptr;
		}

		if (BoneName == NAME_None)
		{
			// default to use root bone
			CachedBoneIndex = 0;
		}
		else
		{
			// Invalidate cache if input changed
			if (CachedBoneName != BoneName)
			{
				CachedBoneIndex = OwningComponent->GetSkeletalMeshAsset()->GetRefSkeleton().FindBoneIndex(BoneName);
			}
		}
	
		CachedBoneName = BoneName;

		if (CachedBoneIndex != INDEX_NONE)
		{
			const UE::Anim::FAttributeId Id = {Name, FCompactPoseBoneIndex(CachedBoneIndex)} ;
			typename TAnimAttributeType<T>::Type* Attribute = bAddIfNotFound ?
				Context.UnitContext.AnimAttributeContainer->FindOrAdd<typename TAnimAttributeType<T>::Type>(Id) :
				Context.UnitContext.AnimAttributeContainer->Find<typename TAnimAttributeType<T>::Type>(Id);
			if (Attribute)
			{
				return &Attribute->Value;
			}
		}

		return nullptr;
	}
}




/**
 * Animation Attributes allow dynamically added data to flow from
 * one Anim Node to other Anim Nodes downstream in the Anim Graph
 * and accessible from deformer graph
 */
USTRUCT(meta=(Abstract, Category="Animation Attribute", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigDispatch_AnimAttributeBase : public FRigDispatchFactory
{
	GENERATED_BODY()

	UE_API virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	UE_API virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const override;;
#endif
	
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSet() const { return false; }

#if WITH_EDITOR
	UE_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	UE_API bool IsTypeSupported(const TRigVMTypeIndex& InTypeIndex, const FRigVMRegistryHandle& InRegistry) const;
	static UE_API const TArray<FRigVMTemplateArgument::ETypeCategory>& GetValueTypeCategory();
	
	// input
	mutable int32 NameArgIndex = INDEX_NONE;
	mutable int32 BoneNameArgIndex = INDEX_NONE;
	mutable int32 DefaultArgIndex = INDEX_NONE;


	// output
	mutable int32 ValueArgIndex = INDEX_NONE;
	mutable int32 FoundArgIndex = INDEX_NONE;
	mutable int32 SuccessArgIndex = INDEX_NONE;

	// hidden
	mutable int32 CachedBoneNameArgIndex = INDEX_NONE;
	mutable int32 CachedBoneIndexArgIndex = INDEX_NONE;

	mutable TArray<TRigVMTypeIndex> SpecialTypes;

	static inline const FLazyName NameArgName = FLazyName(TEXT("Name"));
	static inline const FLazyName BoneNameArgName = FLazyName(TEXT("BoneName"));
	static inline const FLazyName CachedBoneNameArgName = FLazyName(TEXT("CachedBoneName"));
	static inline const FLazyName CachedBoneIndexArgName = FLazyName(TEXT("CachedBoneIndex"));
	static inline const FLazyName DefaultArgName = FLazyName(TEXT("Default"));
	static inline const FLazyName ValueArgName = FLazyName(TEXT("Value"));
	static inline const FLazyName FoundArgName = FLazyName(TEXT("Found"));
	static inline const FLazyName SuccessArgName = FLazyName(TEXT("Success"));
};


/*
 * Get the value of an animation attribute from the skeletal mesh
 */
USTRUCT(meta=(DisplayName="Get Animation Attribute"))
struct FRigDispatch_GetAnimAttribute: public FRigDispatch_AnimAttributeBase
{
	GENERATED_BODY()

	FRigDispatch_GetAnimAttribute()
	{
		FactoryScriptStruct = StaticStruct();
	}

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;

	
protected:
	UE_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	template<typename ValueType>
	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[BoneNameArgIndex].IsType<FName>(), BoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneNameArgIndex].IsType<FName>(true), CachedBoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneIndexArgIndex].IsType<int32>(true), CachedBoneIndexArgName) &&
			CheckArgumentType(Handles[DefaultArgIndex].IsType<ValueType>(), DefaultArgName) &&
			CheckArgumentType(Handles[ValueArgIndex].IsType<ValueType>(), ValueArgName) &&
			CheckArgumentType(Handles[FoundArgIndex].IsType<bool>(), FoundArgName);
	}

	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[BoneNameArgIndex].IsType<FName>(), BoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneNameArgIndex].IsType<FName>(true), CachedBoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneIndexArgIndex].IsType<int32>(true), CachedBoneIndexArgName) &&
			CheckArgumentType(Handles[FoundArgIndex].IsType<bool>(), FoundArgName);
	}
#endif

	// dispatch function for built-in types
	template<typename ValueType>	
	static void GetAnimAttributeDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		const FRigDispatch_GetAnimAttribute* Factory = static_cast<const FRigDispatch_GetAnimAttribute*>(InContext.Factory);
		
#if WITH_EDITOR
		if(!Factory->CheckArgumentTypes<ValueType>(Handles))
		{
			return;
		}
#endif

		// unpack the memory
		const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
		const FName& BoneName = *(const FName*)Handles[Factory->BoneNameArgIndex].GetInputData();
		const ValueType& Default = *(const ValueType*)Handles[Factory->DefaultArgIndex].GetInputData();
		
		ValueType& Value = *(ValueType*)Handles[Factory->ValueArgIndex].GetOutputData();
		bool& Found = *(bool*)Handles[Factory->FoundArgIndex].GetOutputData();
		
		FName& CachedBoneName = *(FName*)Handles[Factory->CachedBoneNameArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
		int32& CachedBoneIndex = *(int32*)Handles[Factory->CachedBoneIndexArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());

		// extract the animation attribute
		const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();
		const ValueType* ValuePtr = RigUnit_AnimAttribute::GetAnimAttributeValue<ValueType>(false, Context, Name, BoneName, CachedBoneName, CachedBoneIndex);
		Found = ValuePtr ? true : false;
		Value = ValuePtr ? *ValuePtr : Default;
	}

	// dispatch function for user/dev defined types
	static UE_API void GetAnimAttributeDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
	
};

/*
 * Modify an animation attribute if one is found, otherwise add a new animation attribute
 */
USTRUCT(meta=(DisplayName="Set Animation Attribute"))
struct FRigDispatch_SetAnimAttribute: public FRigDispatch_AnimAttributeBase
{
	GENERATED_BODY()

	FRigDispatch_SetAnimAttribute()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual bool IsSet() const override { return true; }
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	
protected:
	UE_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	template<typename ValueType>
	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[BoneNameArgIndex].IsType<FName>(), BoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneNameArgIndex].IsType<FName>(true), CachedBoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneIndexArgIndex].IsType<int32>(true), CachedBoneIndexArgName) &&
			CheckArgumentType(Handles[ValueArgIndex].IsType<ValueType>(), ValueArgName) &&
			CheckArgumentType(Handles[SuccessArgIndex].IsType<bool>(), FoundArgName);
	}

	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[BoneNameArgIndex].IsType<FName>(), BoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneNameArgIndex].IsType<FName>(true), CachedBoneNameArgName) &&
			CheckArgumentType(Handles[CachedBoneIndexArgIndex].IsType<int32>(true), CachedBoneIndexArgName) &&
			CheckArgumentType(Handles[SuccessArgIndex].IsType<bool>(), FoundArgName);
	}
#endif

	// dispatch function for built-in types
	template<typename ValueType>	
	static void SetAnimAttributeDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		const FRigDispatch_SetAnimAttribute* Factory = static_cast<const FRigDispatch_SetAnimAttribute*>(InContext.Factory);
		
#if WITH_EDITOR
		if(!Factory->CheckArgumentTypes<ValueType>(Handles))
		{
			return;
		}
#endif

		// unpack the memory
		const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
		const FName& BoneName = *(const FName*)Handles[Factory->BoneNameArgIndex].GetInputData();
		const ValueType& Value = *(ValueType*)Handles[Factory->ValueArgIndex].GetInputData();
		
		bool& Success = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();
		Success = false;
		
		FName& CachedBoneName = *(FName*)Handles[Factory->CachedBoneNameArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
		int32& CachedBoneIndex = *(int32*)Handles[Factory->CachedBoneIndexArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());

		// extract the animation attribute
		const FControlRigExecuteContext& Context = InContext.GetPublicDataSafe<FControlRigExecuteContext>();
		ValueType* ValuePtr = RigUnit_AnimAttribute::GetAnimAttributeValue<ValueType>(true, Context, Name, BoneName, CachedBoneName, CachedBoneIndex);
		if (ValuePtr)
		{
			Success = true;
			*ValuePtr = Value;
		}
	}

	// dispatch function for user/dev defined types
	static UE_API void SetAnimAttributeDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

#undef UE_API
