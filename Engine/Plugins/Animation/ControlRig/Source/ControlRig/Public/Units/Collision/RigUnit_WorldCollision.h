// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Units/RigUnit.h"
#include "RigUnit_WorldCollision.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_SphereTrace_WorkData
{
	GENERATED_BODY()

	FRigUnit_SphereTrace_WorkData()
	: Hash(INDEX_NONE)
	, bHit(false)
	, HitLocation(EForceInit::ForceInitToZero)
	, HitNormal(FVector::UpVector)
	{
	}

	void Reset()
	{
		Hash = INDEX_NONE;
	}

	UPROPERTY()
	uint32 Hash;

	UPROPERTY()
	bool bHit;

	UPROPERTY()
	FVector HitLocation;
	
	UPROPERTY()
	FVector HitNormal;
};

/**
 * Sweeps a sphere against the world and return the first blocking hit using a specific channel
 */
USTRUCT(meta=(DisplayName="Sphere Trace", Category="Collision", DocumentationPolicy = "Strict", Keywords="Sweep,Raytrace,Collision,Collide,Trace", Varying, NodeColor = "0.2 0.4 0.7", Deprecated = "5.0"))
struct FRigUnit_SphereTraceWorld : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_SphereTraceWorld()
	: Start(EForceInit::ForceInitToZero)
	, End(EForceInit::ForceInitToZero)
	, Channel(ECC_Visibility)
	, Radius(5.f)
	, bHit(false)
	, HitLocation(EForceInit::ForceInitToZero)
	, HitNormal(FVector::UpVector)
	{

	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** The 'channel' that this trace is in, used to determine which components to hit */
	UPROPERTY(meta = (Input))
	TEnumAsByte<ECollisionChannel> Channel;

	/** Radius of the sphere to use for sweeping / tracing */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "100.0"))
	float Radius;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;
	
	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;

	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Sweeps a sphere against the world and return the first blocking hit using a specific channel. Target objects can have different object types, but they need to have the same trace channel set to "block" in their collision response settings. 
 * You can create custom trace channels in Project Setting - Collision. 
 */
USTRUCT(meta = (DisplayName = "Sphere Trace By Trace Channel", Category = "Collision", DocumentationPolicy = "Strict", Keywords = "Sweep,Raytrace,Collision,Collide,Trace", Varying, NodeColor = "0.2 0.4 0.7"))
struct FRigUnit_SphereTraceByTraceChannel : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_SphereTraceByTraceChannel()
		: Start(EForceInit::ForceInitToZero)
		, End(EForceInit::ForceInitToZero)
		, TraceChannel(TraceTypeQuery1)
		, Radius(5.f)
		, bHit(false)
		, HitLocation(EForceInit::ForceInitToZero)
		, HitNormal(FVector::UpVector)
	{

	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** The 'channel' that this trace is in, used to determine which components to hit */
	UPROPERTY(meta = (Input))
	TEnumAsByte<ETraceTypeQuery> TraceChannel;

	/** Radius of the sphere to use for sweeping / tracing */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "100.0"))
	float Radius;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;

	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;
	
	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;
};

/**
 * Performs a line trace against the world and return the first blocking hit using a specific channel. Target objects can have different object types, but they need to have the same trace channel set to "block" in their collision response settings. 
 * You can create custom trace channels in Project Setting - Collision. 
 */
USTRUCT(meta = (DisplayName = "Line Trace By Trace Channel", Category = "Collision", DocumentationPolicy = "Strict", Keywords = "Line,Raytrace,Collision,Collide,Trace", Varying, NodeColor = "0.2 0.4 0.7"))
struct FRigUnit_LineTraceByTraceChannel : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_LineTraceByTraceChannel()
		: Start(EForceInit::ForceInitToZero)
		, End(EForceInit::ForceInitToZero)
		, TraceChannel(TraceTypeQuery1)
		, bHit(false)
		, HitLocation(EForceInit::ForceInitToZero)
		, HitNormal(FVector::UpVector)
	{

	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** The 'channel' that this trace is in, used to determine which components to hit */
	UPROPERTY(meta = (Input))
	TEnumAsByte<ETraceTypeQuery> TraceChannel;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;

	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;
	
	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;
};

/**
 * Sweeps a sphere against the world and return the first blocking hit. The trace is filtered by object types only, the collision response settings are ignored.
 * You can create custom object types in Project Setting - Collision
 */
USTRUCT(meta = (DisplayName = "Sphere Trace By Object Types", Category = "Collision", DocumentationPolicy = "Strict", Keywords = "Sweep,Raytrace,Collision,Collide,Trace", Varying, NodeColor = "0.2 0.4 0.7"))
struct FRigUnit_SphereTraceByObjectTypes : public FRigUnit
{
	GENERATED_BODY()

		FRigUnit_SphereTraceByObjectTypes()
		: Start(EForceInit::ForceInitToZero)
		, End(EForceInit::ForceInitToZero)
		, ObjectTypes()
		, Radius(5.f)
		, bHit(false)
		, HitLocation(EForceInit::ForceInitToZero)
		, HitNormal(FVector::UpVector)
	{

	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** The types of objects that this trace can hit */
	UPROPERTY(meta = (Input))
	TArray<TEnumAsByte<EObjectTypeQuery>> ObjectTypes;

	/** Radius of the sphere to use for sweeping / tracing */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "100.0"))
	float Radius;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;

	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;
	
	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;
};

/**
 * Performs a line trace against the world and return the first blocking hit. The trace is filtered by object types only, the collision response settings are ignored.
 * You can create custom object types in Project Setting - Collision
 */
USTRUCT(meta = (DisplayName = "Line Trace By Object Types", Category = "Collision", DocumentationPolicy = "Strict", Keywords = "Line,Raytrace,Collision,Collide,Trace", Varying, NodeColor = "0.2 0.4 0.7"))
struct FRigUnit_LineTraceByObjectTypes : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_LineTraceByObjectTypes()
	: Start(EForceInit::ForceInitToZero)
	, End(EForceInit::ForceInitToZero)
	, ObjectTypes()
	, bHit(false)
	, HitLocation(EForceInit::ForceInitToZero)
	, HitNormal(FVector::UpVector)
	{

	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** The types of objects that this trace can hit */
	UPROPERTY(meta = (Input))
	TArray<TEnumAsByte<EObjectTypeQuery>> ObjectTypes;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;

	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;
	
	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;
};

/**
 * Performs a sphere trace against the world and return the first blocking hit. The trace is filtered by specifying a collision profile.
 */
USTRUCT(meta = (DisplayName = "Sphere Trace By Profile", Category = "Collision", DocumentationPolicy = "Strict", Keywords = "Sweep,Raytrace,Collision,Collide,Trace,Profile", Varying, NodeColor = "0.2 0.4 0.7"))
struct FRigUnit_SphereTraceByProfile : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_SphereTraceByProfile()
	: Start(EForceInit::ForceInitToZero)
	, End(EForceInit::ForceInitToZero)
	, Radius(0.0f)
	, bHit(false)
	, HitLocation(EForceInit::ForceInitToZero)
	, HitNormal(FVector::UpVector)
	{

	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** Radius of the sphere to use for sweeping / tracing */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "100.0"))
	float Radius;

	/** The profile to use */
	UPROPERTY(meta=(Input, GetOptions="Engine.KismetSystemLibrary.GetCollisionProfileNames"))
	FName ProfileName;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;

	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;
	
	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;
};

/**
 * Performs a line trace against the world and return the first blocking hit. The trace is filtered by specifying a collision profile.
 */
USTRUCT(meta = (DisplayName = "Line Trace By Profile", Category = "Collision", DocumentationPolicy = "Strict", Keywords = "Line,Raytrace,Collision,Collide,Trace,Profile", Varying, NodeColor = "0.2 0.4 0.7"))
struct FRigUnit_LineTraceByProfile : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_LineTraceByProfile()
	: Start(EForceInit::ForceInitToZero)
	, End(EForceInit::ForceInitToZero)
	, bHit(false)
	, HitLocation(EForceInit::ForceInitToZero)
	, HitNormal(FVector::UpVector)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Start of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector Start;

	/** End of the trace in rig / global space */
	UPROPERTY(meta = (Input))
	FVector End;

	/** The profile to use */
	UPROPERTY(meta=(Input, GetOptions="Engine.KismetSystemLibrary.GetCollisionProfileNames"))
	FName ProfileName;

	/** Returns true if there was a hit */
	UPROPERTY(meta = (Output))
	bool bHit;

	/** Hit location in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitLocation;

	/** Hit normal in rig / global Space */
	UPROPERTY(meta = (Output))
	FVector HitNormal;
	
	/** Cache / workstate */
	UPROPERTY()
	FRigUnit_SphereTrace_WorkData WorkData;
};


#undef UE_API
