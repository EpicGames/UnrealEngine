// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Constraint.h"
#include "EulerTransform.h"
#include "RigUnit_Control.generated.h"

#define UE_API CONTROLRIG_API

/** A control unit used to drive a transform from an external source */
USTRUCT(meta=(DisplayName="Control", Category="Controls", ShowVariableNameInTitle, Deprecated = "4.24.0"))
struct FRigUnit_Control : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_Control()
		: Transform(FEulerTransform::Identity)
		, Base(FTransform::Identity)
		, Result(FTransform::Identity)
		, bIsInitialized(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Combine Transform and Base to make the resultant transform */
	UE_API FTransform GetResultantTransform() const;
	static UE_API FTransform StaticGetResultantTransform(const FEulerTransform& InTransform, const FTransform& InBase, const FTransformFilter& InFilter);

	/** Combine Transform and Base to make the resultant transform (as a matrix) */
	UE_API FMatrix GetResultantMatrix() const;
	static UE_API FMatrix StaticGetResultantMatrix(const FEulerTransform& InTransform, const FTransform& InBase, const FTransformFilter& InFilter);

	/** Set the transform using a resultant transform (already incorporating Base) */
	UE_API void SetResultantTransform(const FTransform& InResultantTransform);
	static UE_API void StaticSetResultantTransform(const FTransform& InResultantTransform, const FTransform& InBase, FEulerTransform& OutTransform);

	/** Set the transform using a resultant matrix (already incorporating Base) */
	UE_API void SetResultantMatrix(const FMatrix& InResultantMatrix);
	static UE_API void StaticSetResultantMatrix(const FMatrix& InResultantMatrix, const FTransform& InBase, FEulerTransform& OutTransform);

	/** Get the local transform (i.e. without base) with filter applied */
	UE_API FEulerTransform GetFilteredTransform() const;
	static UE_API FEulerTransform StaticGetFilteredTransform(const FEulerTransform& InTransform, const FTransformFilter& InFilter);

	/** The transform of this control */
	UPROPERTY(EditAnywhere, Category="Control", Interp, meta = (AnimationInput))
	FEulerTransform Transform;

	/** The base that transform is relative to */
	UPROPERTY(meta=(Input))
	FTransform Base;

	/** The initial transform that The Transform is initialized to. */
	UPROPERTY(meta = (Input))
	FTransform InitTransform;

	/** The resultant transform of this unit (Base * Filter(Transform)) */
	UPROPERTY(meta=(Output))
	FTransform Result;

	/** The filter determines what axes can be manipulated by the in-viewport widgets */
	UPROPERTY(meta=(Input))
 	FTransformFilter Filter;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	UPROPERTY()
 	bool bIsInitialized;
};

#undef UE_API
