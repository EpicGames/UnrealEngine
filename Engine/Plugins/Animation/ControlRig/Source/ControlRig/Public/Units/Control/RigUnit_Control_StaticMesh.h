// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_Control.h"
#include "RigUnit_Control_StaticMesh.generated.h"

#define UE_API CONTROLRIG_API

class UStaticMesh;
class UMaterialInterface;

/** A control unit used to drive a transform from an external source */
USTRUCT(meta=(DisplayName="Static Mesh Control", Category="Controls", ShowVariableNameInTitle, Deprecated="5.0"))
struct FRigUnit_Control_StaticMesh : public FRigUnit_Control
{
	GENERATED_BODY()

	UE_API FRigUnit_Control_StaticMesh();

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** The the transform the mesh will be rendered with (applied on top of the control's transform in the viewport) */
	UPROPERTY(meta=(Input))
	FTransform MeshTransform;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
