// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchy.h"
#include "RigVMCore/RigVMUserWorkflow.h"
#include "ControlRigNodeWorkflow.generated.h"

#define UE_API CONTROLRIG_API

UCLASS(MinimalAPI, BlueprintType)
class UControlRigWorkflowOptions : public URigVMUserWorkflowOptions
{
	GENERATED_BODY()

public:
	
	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = Options)
	TObjectPtr<const URigHierarchy> Hierarchy;

	UPROPERTY(BlueprintReadWrite, EditDefaultsOnly, Category = Options)
	TArray<FRigElementKey> Selection;

	UFUNCTION(BlueprintCallable, Category = "Options")
	UE_API bool EnsureAtLeastOneRigElementSelected() const;
};

UCLASS(MinimalAPI, BlueprintType)
class UControlRigTransformWorkflowOptions : public UControlRigWorkflowOptions
{
	GENERATED_BODY()

public:

	// The type of transform to retrieve from the hierarchy
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Options)
	TEnumAsByte<ERigTransformType::Type> TransformType = ERigTransformType::CurrentGlobal;

	UFUNCTION()
	UE_API TArray<FRigVMUserWorkflow> ProvideWorkflows(const UObject* InSubject);

protected:

#if WITH_EDITOR
	static UE_API bool PerformTransformWorkflow(const URigVMUserWorkflowOptions* InOptions, UObject* InController);
#endif
};

#undef UE_API
