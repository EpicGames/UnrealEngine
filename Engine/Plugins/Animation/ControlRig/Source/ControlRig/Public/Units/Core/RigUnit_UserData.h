// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigDispatchFactory.h"
#include "Units/RigUnit.h"
#include "RigUnit_UserData.generated.h"

#define UE_API CONTROLRIG_API

/*
 * Retrieves a value from a namespaced user data
 */
USTRUCT(meta=(DisplayName = "Get User Data", Keywords = "AssetUserData,Metadata", ExecuteContext="FControlRigExecuteContext", Varying))
struct FRigDispatch_GetUserData : public FRigDispatchFactory
{
	GENERATED_BODY()

public:
	FRigDispatch_GetUserData()
	{
		FactoryScriptStruct = StaticStruct();
	}

#if WITH_EDITOR
	virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;

protected:

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);

	static inline constexpr TCHAR ArgNameSpaceName[] = TEXT("NameSpace");
	static inline constexpr TCHAR ArgPathName[] = TEXT("Path");
	static inline constexpr TCHAR ArgDefaultName[] = TEXT("Default");
	static inline constexpr TCHAR ArgResultName[] = TEXT("Result");
	static inline constexpr TCHAR ArgFoundName[] = TEXT("Found");
};

/**
 * Allows to set / add a shape library on the running control rig from user data
 */
USTRUCT(meta=(DisplayName="Set Shape Library from User Data", Keywords="Shape,Gizmo,Library"))
struct FRigUnit_SetupShapeLibraryFromUserData : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetupShapeLibraryFromUserData()
	{
		LogShapeLibraries = false;
		bFound = false;
	}

	/*
	 * The name space of the user data to look the shape library up within
	 */
	UPROPERTY(meta = (Input, CustomWidget=UserDataNameSpace))
	FString NameSpace;

	/*
	 * The path within the user data for the shape library
	 */
	UPROPERTY(meta = (Input, CustomWidget=UserDataPath, AllowUObjects))
	FString Path;

	/*
	 * Optionally provide the namespace of the shape library to use.
	 * This is only useful if you have multiple shape libraries and you
	 * want to override a specific one.
	 */
	UPROPERTY(meta = (Input))
	FString LibraryName;

	/*
	 * If this is checked we'll output the resulting shape libraries to the log for debugging.
	 */
	UPROPERTY(meta = (Input))
	bool LogShapeLibraries;

	/*
	 * True if the shape library was found and successfully set.
	 */
	UPROPERTY(meta = (Output))
	bool bFound;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Checks whether or not a shape is available in the rig's shape libraries
 */
USTRUCT(meta=(DisplayName="Shape Exists", Keywords="Shape,Gizmo,Library,Exists,Contains"))
struct FRigUnit_ShapeExists : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ShapeExists()
	{
		ShapeName = NAME_None;
		Result = false;
	}

	/*
	 * The name of the shape to search for
	 */
	UPROPERTY(meta = (Input))
	FName ShapeName;

	/*
	 * True if the shape name exists in any of the shape libraries 
	 */
	UPROPERTY(meta = (Output))
	bool Result;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

#undef UE_API
