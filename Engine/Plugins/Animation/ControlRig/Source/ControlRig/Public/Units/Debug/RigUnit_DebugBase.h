// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigVMCore/RigVMDebugDrawSettings.h"
#include "RigUnit_DebugBase.generated.h"

/*
 * Base class for all pure debug nodes
 */
USTRUCT(meta=(Abstract, Category="Debug", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict"))
struct FRigUnit_DebugBase : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input, DetailsOnly, DisplayName = "Draw Settings"))
	FRigVMDebugDrawSettings DebugDrawSettings;
};

/*
 * Base class for all mutable debug nodes
 */
USTRUCT(meta=(Abstract, Category="Debug", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict", Pure))
struct FRigUnit_DebugBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input, DetailsOnly, DisplayName = "Draw Settings"))
	FRigVMDebugDrawSettings DebugDrawSettings;
};
