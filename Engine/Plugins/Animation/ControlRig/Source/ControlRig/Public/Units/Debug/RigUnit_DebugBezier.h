// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Debug/RigVMFunction_DebugBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_DebugBezier.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Draw Bezier", Deprecated = "4.25"))
struct FRigUnit_DebugBezier : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugBezier()
	{
		Bezier = FRigVMFourPointBezier();
		Color = FLinearColor::Red;
		MinimumU = 0.f;
		MaximumU = 1.f;
		Thickness = 0.f;
		Detail = 16.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigVMFourPointBezier Bezier;

	UPROPERTY(meta = (Input))
	float MinimumU;

	UPROPERTY(meta = (Input))
	float MaximumU;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	int32 Detail;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Draw Bezier", Deprecated = "4.25"))
struct FRigUnit_DebugBezierItemSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugBezierItemSpace()
	{
		Bezier = FRigVMFourPointBezier();
		Color = FLinearColor::Red;
		MinimumU = 0.f;
		MaximumU = 1.f;
		Thickness = 0.f;
		Detail = 16.f;
		WorldOffset = FTransform::Identity;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigVMFourPointBezier Bezier;

	UPROPERTY(meta = (Input))
	float MinimumU;

	UPROPERTY(meta = (Input))
	float MaximumU;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	int32 Detail;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	static UE_API void DrawBezier(const FRigVMExecuteContext& InContext, const FTransform& WorldOffset, const FRigVMFourPointBezier& InBezier, float MinimumU, float MaximumU, const FLinearColor& Color, float Thickness, int32 Detail);
};

#undef UE_API
