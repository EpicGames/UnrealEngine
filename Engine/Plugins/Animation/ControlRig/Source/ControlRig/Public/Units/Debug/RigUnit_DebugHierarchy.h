// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Debug/RigVMFunction_DebugBase.h"
#include "Rigs/RigHierarchy.h"
#include "Units/RigUnitContext.h"
#include "RigUnit_DebugHierarchy.generated.h"

#define UE_API CONTROLRIG_API

UENUM()
namespace EControlRigDrawHierarchyMode
{
	enum Type : int
	{
		/** Draw as axes */
		Axes,

		/** MAX - invalid */
		Max UMETA(Hidden),
	};
}

/**
 * Draws vectors on each bone in the viewport across the entire hierarchy
 */
USTRUCT(meta=(DisplayName="Draw Hierarchy", ExecuteContext="FControlRigExecuteContext"))
struct FRigUnit_DebugHierarchy : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigUnit_DebugHierarchy()
	{
		Scale = 10.f;
		Color = FLinearColor::White;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;

	// the items to draw the pose for.
	// if this is empty we'll draw the whole hierarchy
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	// The scale to apply to the drawn transform
	UPROPERTY(meta = (Input))
	float Scale;

	// The color to use when drawing
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use when drawing
	UPROPERTY(meta = (Input))
	float Thickness;

	// The world offset to pre-multiply the drawn transform with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;

	static UE_API void DrawHierarchy(const FRigVMExecuteContext& InContext, const FTransform& WorldOffset, URigHierarchy* Hierarchy, EControlRigDrawHierarchyMode::Type Mode, float Scale, const FLinearColor& Color, float Thickness, const FRigPose* InPose, const TArrayView<const FRigElementKey>* InItems, const FRigVMDebugDrawSettings& DebugDrawSettings);
};

/**
* Draws vectors on each bone in the viewport across the entire pose
*/
USTRUCT(meta=(DisplayName="Draw Pose Cache", ExecuteContext="FControlRigExecuteContext"))
struct FRigUnit_DebugPose : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigUnit_DebugPose()
	{
		Scale = 10.f;
		Color = FLinearColor::White;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;

	// The input pose to draw
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// the items to draw the pose cache for.
	// if this is empty we'll draw the whole pose cache
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	// The scale to apply to the drawn transform
	UPROPERTY(meta = (Input))
	float Scale;

	// The color to use when drawing
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use when drawing
	UPROPERTY(meta = (Input))
	float Thickness;

	// The world offset to pre-multiply the drawn transform with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};

#undef UE_API
