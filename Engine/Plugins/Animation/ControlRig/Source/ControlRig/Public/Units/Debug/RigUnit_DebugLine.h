// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_DebugBase.h"
#include "RigUnit_DebugLine.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Draw Line", Deprecated = "4.25"))
struct FRigUnit_DebugLine : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugLine()
	{
		A = B = FVector::ZeroVector;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FVector A;

	UPROPERTY(meta = (Input))
	FVector B;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Draws a line in the viewport given a start and end vector
 */
USTRUCT(meta=(DisplayName="Draw Line", Deprecated = "5.2"))
struct FRigUnit_DebugLineItemSpace : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugLineItemSpace()
	{
		A = B = FVector::ZeroVector;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FVector A;

	UPROPERTY(meta = (Input))
	FVector B;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
