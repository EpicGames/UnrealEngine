// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_DebugBase.h"
#include "RigUnit_DebugLineStrip.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Draw Line Strip", Deprecated = "4.25"))
struct FRigUnit_DebugLineStrip : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugLineStrip()
	{
		Color = FLinearColor::Red;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	TArray<FVector> Points;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;
	
	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Draws a line strip in the viewport given any number of points
 */
USTRUCT(meta=(DisplayName="Draw Line Strip", Deprecated = "5.2"))
struct FRigUnit_DebugLineStripItemSpace : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugLineStripItemSpace()
	{
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	TArray<FVector> Points;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;
	
	UPROPERTY(meta = (Input))
	bool bEnabled;
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
