// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_DebugBase.h"
#include "RigUnit_DebugPrimitives.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Draw Rectangle", Keywords="Draw Square", Deprecated = "4.25"))
struct FRigUnit_DebugRectangle : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugRectangle()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Draws a rectangle in the viewport given a transform
 */
USTRUCT(meta=(DisplayName="Draw Rectangle", Keywords="Draw Square", Deprecated = "5.2"))
struct FRigUnit_DebugRectangleItemSpace : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugRectangleItemSpace()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};


USTRUCT(meta=(DisplayName="Draw Arc", Keywords="Draw Ellipse, Draw Circle", Deprecated = "4.25"))
struct FRigUnit_DebugArc : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugArc()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = MinimumDegrees = 0.f;
		Radius = 10.f;
		MaximumDegrees = 360.f;
		Detail = 16;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Radius;

	UPROPERTY(meta = (Input))
	float MinimumDegrees;

	UPROPERTY(meta = (Input))
	float MaximumDegrees;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	int32 Detail;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Draws an arc in the viewport, can take in different min and max degrees
 */
USTRUCT(meta=(DisplayName="Draw Arc", Keywords="Draw Ellipse, Draw Circle", Deprecated = "5.2"))
struct FRigUnit_DebugArcItemSpace : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugArcItemSpace()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = MinimumDegrees = 0.f;
		Radius = 10.f;
		MaximumDegrees = 360.f;
		Detail = 16;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Radius;

	UPROPERTY(meta = (Input))
	float MinimumDegrees;

	UPROPERTY(meta = (Input))
	float MaximumDegrees;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	int32 Detail;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
