// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_DebugBase.h"
#include "RigVMFunctions/Debug/RigVMFunction_DebugTransform.h"
#include "RigUnit_DebugTransform.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Draw Transform In Place", Deprecated = "4.25.0"))
struct FRigUnit_DebugTransform : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_DebugTransform()
	{
		Transform = WorldOffset = FTransform::Identity;
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Draw Transform", Deprecated = "4.25"))
struct FRigUnit_DebugTransformMutable : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugTransformMutable()
	{
		Transform = WorldOffset = FTransform::Identity;
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Given a transform, will draw a point, axis, or a box in the viewport
 */
USTRUCT(meta=(DisplayName="Draw Transform", Deprecated = "5.2"))
struct FRigUnit_DebugTransformMutableItemSpace : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugTransformMutableItemSpace()
	{
		Transform = WorldOffset = FTransform::Identity;
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT()
struct FRigUnit_DebugTransformArrayMutable_WorkData
{
	GENERATED_BODY()
		
	UPROPERTY()
	TArray<FTransform> DrawTransforms;
};

USTRUCT(meta=(DisplayName="Draw Transform Array", Deprecated = "4.25.0"))
struct FRigUnit_DebugTransformArrayMutable : public FRigUnit_DebugBaseMutable
{
 	GENERATED_BODY()

	FRigUnit_DebugTransformArrayMutable()
	{
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	TArray<FTransform> Transforms;

	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	UPROPERTY(transient)
	FRigUnit_DebugTransformArrayMutable_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Given a transform array, will draw a point, axis, or a box in the viewport
*/
USTRUCT(meta=(DisplayName="Draw Transform Array", Deprecated = "5.2"))
struct FRigUnit_DebugTransformArrayMutableItemSpace : public FRigUnit_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DebugTransformArrayMutableItemSpace()
	{
		WorldOffset = FTransform::Identity;
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::White;
		Thickness = 0.f;
		Scale = 10.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	TArray<FTransform> Transforms;

	UPROPERTY(meta = (Input))
	TArray<int32> ParentIndices;

	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	FRigElementKey Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
