// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Debug/RigVMFunction_DebugBase.h"
#include "RigUnit_ProfilingBracket.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Starts a profiling timer for debugging, used in conjunction with End Profiling Timer
 */
USTRUCT(meta=(DisplayName="Start Profiling Timer", Keywords="Measure,BeginProfiling,Profile", NodeColor="0.25, 0.25, 0.05000000074505806"))
struct FRigUnit_StartProfilingTimer : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_StartProfilingTimer()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Ends an existing profiling timer for debugging, used in conjunction with Start Profiling Timer
 */
USTRUCT(meta = (DisplayName = "End Profiling Timer", Keywords = "Measure,StopProfiling,Meter,Profile", NodeColor="0.25, 0.25, 0.05000000074505806"))
struct FRigUnit_EndProfilingTimer : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigUnit_EndProfilingTimer()
	{
		NumberOfMeasurements = 1;
		AccumulatedTime = 0.f;
		MeasurementsLeft = 0;
		Prefix = TEXT("Timer");
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The number of measurements to take for profiling
	UPROPERTY(meta = (Input, Constant))
	int32 NumberOfMeasurements;

	// The prefix to use when printing to the log
	UPROPERTY(meta = (Input, Constant))
	FString Prefix;

	// The average time - this will be zero and then set once profiling is complete 
	UPROPERTY(meta = (Output))
	float AverageTime = 0.0f;

	UPROPERTY()
	float AccumulatedTime;

	UPROPERTY()
	int32 MeasurementsLeft;

	UPROPERTY()
	bool bIsInitialized;
};

#undef UE_API
