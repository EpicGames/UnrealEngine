// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_DebugBase.h"
#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"
#include "RigUnit_VisualDebug.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName = "Visual Debug Vector", Keywords = "Draw,Point", Deprecated = "4.25", Varying))
struct FRigUnit_VisualDebugVector : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_VisualDebugVector()
	{
		Value = FVector::ZeroVector;
		bEnabled = true;
		Mode = ERigUnitVisualDebugPointMode::Point;
		Color = FLinearColor::Red;
		Thickness = 10.f;
		Scale = 1.f;
		BoneSpace = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FVector Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	ERigUnitVisualDebugPointMode Mode;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FLinearColor Color;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FName BoneSpace;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Debug draw parameters for a Point or Vector given a vector
 */
USTRUCT(meta=(DisplayName = "Visual Debug Vector", Keywords = "Draw,Point", Deprecated = "5.2", Varying))
struct FRigUnit_VisualDebugVectorItemSpace : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_VisualDebugVectorItemSpace()
	{
		Value = FVector::ZeroVector;
		bEnabled = true;
		Mode = ERigUnitVisualDebugPointMode::Point;
		Color = FLinearColor::Red;
		Thickness = 10.f;
		Scale = 1.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FVector Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	ERigUnitVisualDebugPointMode Mode;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FLinearColor Color;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FRigElementKey Space;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "Visual Debug Quat", Keywords = "Draw,Rotation", Deprecated = "4.25", Varying))
struct FRigUnit_VisualDebugQuat : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_VisualDebugQuat()
	{
		Value = FQuat::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
		BoneSpace = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FQuat Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FName BoneSpace;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Debug draw parameters for an Axis given a quaternion
 */
USTRUCT(meta = (DisplayName = "Visual Debug Quat", Keywords = "Draw,Rotation", Deprecated = "5.2", Varying))
struct FRigUnit_VisualDebugQuatItemSpace : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_VisualDebugQuatItemSpace()
	{
		Value = FQuat::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FQuat Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FRigElementKey Space;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Visual Debug Transform", Keywords = "Draw,Axes", Deprecated = "4.25", Varying))
struct FRigUnit_VisualDebugTransform : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_VisualDebugTransform()
	{
		Value = FTransform::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
		BoneSpace = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output, Constant))
	FTransform Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FName BoneSpace;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Debug draw parameters for an Axis given a transform
 */
USTRUCT(meta=(DisplayName="Visual Debug Transform", Keywords = "Draw,Axes", Deprecated = "5.2", Varying))
struct FRigUnit_VisualDebugTransformItemSpace : public FRigUnit_DebugBase
{
	GENERATED_BODY()

	FRigUnit_VisualDebugTransformItemSpace()
	{
		Value = FTransform::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output, Constant))
	FTransform Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FRigElementKey Space;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
