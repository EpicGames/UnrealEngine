// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace FRigMathLibrary
{
	template<typename Type>
	Type Multiply(const Type Argument0, const Type Argument1)
	{
		return Argument0 * Argument1;
	}

	template<typename Type>
	Type Add(const Type Argument0, const Type Argument1)
	{
		return Argument0 + Argument1;
	}

	template<typename Type>
	Type Subtract(const Type Argument0, const Type Argument1)
	{
		return Argument0 - Argument1;
	}

	template<typename Type>
	Type Divide(const Type Argument0, const Type Argument1)
	{
		return Argument0 / Argument1;
	}
};
