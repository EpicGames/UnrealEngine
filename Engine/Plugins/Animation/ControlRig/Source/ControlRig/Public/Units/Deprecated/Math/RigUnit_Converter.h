// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Units/RigUnit.h"
#include "EulerTransform.h"
#include "RigUnit_Converter.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta = (DisplayName = "ConvertToEulerTransform", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertTransform : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FTransform Input;

	UPROPERTY(meta=(Output))
	FEulerTransform Result;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertToTransform", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertEulerTransform : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FEulerTransform Input;

	UPROPERTY(meta=(Output))
	FTransform Result;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertToQuaternion", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertRotation : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FRotator Input = FRotator(0.f);

	UPROPERTY(meta = (Output))
	FQuat	Result = FQuat::Identity;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertToQuaternionDeprecated", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertVectorRotation : public FRigUnit_ConvertRotation
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "ConvertToRotation", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertQuaternion: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FQuat	Input = FQuat::Identity;

	UPROPERTY(meta = (Output))
	FRotator	Result = FRotator(0.f);

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertVectorToRotation", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertVectorToRotation: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Input = FVector(0.f);

	UPROPERTY(meta = (Output))
	FRotator	Result = FRotator(0.f);

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertVectorToQuaternion", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertVectorToQuaternion: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Input = FVector(0.f);

	UPROPERTY(meta = (Output))
	FQuat	Result = FQuat::Identity;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertRotationToVector", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertRotationToVector: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FRotator Input = FRotator(0.f);

	UPROPERTY(meta = (Output))
	FVector Result = FVector(0.f);

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ConvertQuaternionToVector", Category = "Math|Convert", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ConvertQuaternionToVector: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat	Input = FQuat::Identity;

	UPROPERTY(meta = (Output))
	FVector Result = FVector(0.f);

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "ToSwingAndTwist", Category = "Math|Transform", NodeColor = "0.1 0.1 0.7", Deprecated="4.23.0"))
struct FRigUnit_ToSwingAndTwist : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FQuat	Input;

	UPROPERTY(meta = (Input))
	FVector TwistAxis;

	UPROPERTY(meta = (Output))
	FQuat	Swing;

	UPROPERTY(meta = (Output))
	FQuat	Twist;

	FRigUnit_ToSwingAndTwist()
		: Input(ForceInitToZero)
		, TwistAxis(FVector(1.f, 0.f, 0.f))
		, Swing(ForceInitToZero)
		, Twist(ForceInitToZero)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
