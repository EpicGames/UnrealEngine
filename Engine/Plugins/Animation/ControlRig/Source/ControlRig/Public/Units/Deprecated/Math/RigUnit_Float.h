// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "MathLibrary.h"
#include "RigUnit_Float.generated.h"

#define UE_API CONTROLRIG_API

/** Two args and a result of float type */
USTRUCT(meta=(Abstract, NodeColor = "0.1 0.7 0.1", Deprecated="5.8.0"))
struct FRigUnit_BinaryFloatOp : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	float Argument0 = 0.f;

	UPROPERTY(meta=(Input))
	float Argument1 = 0.f;

	UPROPERTY(meta=(Output))
	float Result = 0.f;
};

USTRUCT(meta=(DisplayName="Multiply", Category="Math|Float", Keywords="*", Deprecated="4.23.0"))
struct FRigUnit_Multiply_FloatFloat : public FRigUnit_BinaryFloatOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Add", Category="Math|Float", Keywords = "+,Sum", Deprecated="4.23.0"))
struct FRigUnit_Add_FloatFloat : public FRigUnit_BinaryFloatOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Subtract", Category="Math|Float", Keywords = "-", Deprecated="4.23.0"))
struct FRigUnit_Subtract_FloatFloat : public FRigUnit_BinaryFloatOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Divide", Category="Math|Float", Keywords = "/", Deprecated="4.23.0"))
struct FRigUnit_Divide_FloatFloat : public FRigUnit_BinaryFloatOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/** Two args and a result of float type */
USTRUCT(meta = (DisplayName = "Clamp", Category = "Math|Float", NodeColor = "0.1 0.7 0.1", Deprecated="4.23.0"))
struct FRigUnit_Clamp_Float: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	float Value = 0.f;

	UPROPERTY(meta = (Input))
	float Min = 0.f;

	UPROPERTY(meta = (Input))
	float Max = 0.f;

	UPROPERTY(meta = (Output))
	float Result = 0.f;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/** Two args and a result of float type */
USTRUCT(meta = (DisplayName = "MapRange", Category = "Math|Float", Deprecated="4.23.0"))
struct FRigUnit_MapRange_Float: public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	float Value = 0.f;

	UPROPERTY(meta = (Input))
	float MinIn = 0.f;

	UPROPERTY(meta = (Input))
	float MaxIn = 0.f;

	UPROPERTY(meta = (Input))
	float MinOut = 0.f;

	UPROPERTY(meta = (Input))
	float MaxOut = 0.f;

	UPROPERTY(meta = (Output))
	float Result = 0.f;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
