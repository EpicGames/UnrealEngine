// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "MathLibrary.h"
#include "RigUnit_Quaternion.generated.h"

#define UE_API CONTROLRIG_API

/** Two args and a result of Quaternion type */
USTRUCT(meta=(Abstract, NodeColor = "0.1 0.7 0.1", Deprecated="5.8.0"))
struct FRigUnit_BinaryQuaternionOp : public FRigUnit
{
	GENERATED_BODY()

		FRigUnit_BinaryQuaternionOp()
		: Argument0(FQuat::Identity)
		, Argument1(FQuat::Identity)
		, Result(FQuat::Identity)
	{}

	UPROPERTY(meta=(Input))
	FQuat Argument0;

	UPROPERTY(meta=(Input))
	FQuat Argument1;

	UPROPERTY(meta=(Output))
	FQuat Result;
};

USTRUCT(meta=(DisplayName="Multiply(Quaternion)", Category="Math|Quaternion", Deprecated="4.23.0"))
struct FRigUnit_MultiplyQuaternion : public FRigUnit_BinaryQuaternionOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/** Two args and a result of Quaternion type */
USTRUCT(meta=(Abstract, NodeColor = "0.1 0.7 0.1", Deprecated="5.8.0"))
struct FRigUnit_UnaryQuaternionOp : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FQuat Argument = FQuat::Identity;

	UPROPERTY(meta=(Output))
	FQuat Result = FQuat::Identity;
};

USTRUCT(meta = (DisplayName = "Inverse(Quaternion)", Category = "Math|Quaternion", Deprecated="4.23.0"))
struct FRigUnit_InverseQuaterion: public FRigUnit_UnaryQuaternionOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "To Axis And Angle(Quaternion)", Category = "Math|Quaternion", NodeColor = "0.1 0.7 0.1", Deprecated="4.23.0"))
struct FRigUnit_QuaternionToAxisAndAngle : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat Argument = FQuat::Identity;

	UPROPERTY(meta = (Output))
	FVector Axis = FVector(0.f);

	UPROPERTY(meta = (Output))
	float Angle = 0.f;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "From Axis And Angle(Quaternion)", Category = "Math|Quaternion", NodeColor = "0.1 0.7 0.1", Deprecated="4.23.0"))
struct FRigUnit_QuaternionFromAxisAndAngle : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_QuaternionFromAxisAndAngle()
		: Axis(1.f, 0.f, 0.f)
		, Angle(0.f)
		, Result(ForceInitToZero)
	{}

	UPROPERTY(meta = (Input))
	FVector Axis;

	UPROPERTY(meta = (Input))
	float Angle;

	UPROPERTY(meta = (Output))
	FQuat Result;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "Get Angle Around Axis", Category = "Math|Quaternion", NodeColor = "0.1 0.7 0.1", Deprecated="4.23.0"))
struct FRigUnit_QuaternionToAngle : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_QuaternionToAngle()
		: Axis(1.f, 0.f, 0.f)
		, Argument(ForceInitToZero)
		, Angle(0.f)
	{}

	UPROPERTY(meta = (Input))
	FVector Axis;

	UPROPERTY(meta = (Input))
	FQuat Argument;

	UPROPERTY(meta = (Output))
	float Angle;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
