// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "MathLibrary.h"
#include "RigUnit_Transform.generated.h"

#define UE_API CONTROLRIG_API

/** Two args and a result of Transform type */
USTRUCT(meta=(Abstract, NodeColor = "0.1 0.7 0.1", Deprecated="5.8.0"))
struct FRigUnit_BinaryTransformOp : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FTransform Argument0;

	UPROPERTY(meta=(Input))
	FTransform Argument1;

	UPROPERTY(meta=(Output))
	FTransform Result;
};

USTRUCT(meta=(DisplayName="Multiply(Transform)", Category="Math|Transform", Deprecated="4.23.0"))
struct FRigUnit_MultiplyTransform : public FRigUnit_BinaryTransformOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "GetRelativeTransform", Category = "Math|Transform", Deprecated="4.23.0"))
struct FRigUnit_GetRelativeTransform : public FRigUnit_BinaryTransformOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
