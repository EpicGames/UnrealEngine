// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "MathLibrary.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigVMFunctions/Math/RigVMFunction_MathVector.h"
#include "RigUnit_Vector.generated.h"

#define UE_API CONTROLRIG_API

/** Two args and a result of Vector type */
USTRUCT(meta=(Abstract, NodeColor = "0.1 0.7 0.1", Deprecated="5.8.0"))
struct FRigUnit_BinaryVectorOp : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FVector Argument0 = FVector(0.f);

	UPROPERTY(meta=(Input))
	FVector Argument1 = FVector(0.f);

	UPROPERTY(meta=(Output))
	FVector Result = FVector(0.f);
};

USTRUCT(meta=(DisplayName="Multiply(Vector)", Category="Math|Vector", Deprecated="4.23.0"))
struct FRigUnit_Multiply_VectorVector : public FRigUnit_BinaryVectorOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Add(Vector)", Category="Math|Vector", Deprecated="4.23.0"))
struct FRigUnit_Add_VectorVector : public FRigUnit_BinaryVectorOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Subtract(Vector)", Category="Math|Vector", Deprecated="4.23.0"))
struct FRigUnit_Subtract_VectorVector : public FRigUnit_BinaryVectorOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Divide(Vector)", Category="Math|Vector", Deprecated="4.23.0"))
struct FRigUnit_Divide_VectorVector : public FRigUnit_BinaryVectorOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta = (DisplayName = "Distance", Category = "Math|Vector", Deprecated="4.23.0", NodeColor = "0.1 0.7 0.1"))
struct FRigUnit_Distance_VectorVector : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta=(Input))
	FVector Argument0 = FVector(0.f);

	UPROPERTY(meta=(Input))
	FVector Argument1 = FVector(0.f);

	UPROPERTY(meta=(Output))
	float Result = 0.f;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
