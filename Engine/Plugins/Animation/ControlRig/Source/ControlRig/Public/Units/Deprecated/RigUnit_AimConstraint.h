// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Constraint.h"
#include "ControlRigDefines.h"
#include "RigUnit_AimConstraint.generated.h"

#define UE_API CONTROLRIG_API

 /*
 ENUM: Aim Mode (Default: Aim At Target Transform )  # How to perform an aim
 Aim At Target Transforms
 Orient To Target Transforms
 */

UENUM()
enum class EAimMode : uint8
{
	/** Aim at Target Transform*/
	AimAtTarget,

	/** Orient to Target Transform */
	OrientToTarget,

	/** MAX - invalid */
	MAX,
};

USTRUCT(BlueprintType)
struct FAimTarget
{
	GENERATED_BODY()

	// # Target Weight
	UPROPERTY(EditAnywhere, Category = FAimTarget)
	float Weight = 0.f;

	// # Aim at/Align to this Transform
	UPROPERTY(EditAnywhere, Category = FAimTarget)
	FTransform Transform;

	//# Orient To Target Transforms mode only : Vector in the space of Target Transform to which the Aim Vector will be aligned
	UPROPERTY(EditAnywhere, Category = FAimTarget)
	FVector AlignVector = FVector(0.f);
};

USTRUCT()
struct FRigUnit_AimConstraint_WorkData
{
	GENERATED_BODY()

	// note that Targets.Num () != ConstraintData.Num()
	UPROPERTY()
	TArray<FConstraintData>	ConstraintData;
};

USTRUCT(meta=(DisplayName="Aim Constraint", Category="Transforms", Deprecated = "4.23.0"))
struct FRigUnit_AimConstraint : public FRigUnitMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	FName Joint;

	//# How to perform an aim
	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	EAimMode AimMode = EAimMode::AimAtTarget;

	//# How to perform an upvector stabilization
	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	EAimMode UpMode = EAimMode::AimAtTarget;

	// # Vector in the space of Named joint which will be aligned to the aim target
	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	FVector AimVector = FVector(0.f);

	//# Vector in the space of Named joint which will be aligned to the up target for stabilization
	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	FVector UpVector = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	TArray<FAimTarget> AimTargets;

	UPROPERTY(EditAnywhere, Category = FRigUnit_AimConstraint, meta = (Input))
	TArray<FAimTarget> UpTargets;

	// note that Targets.Num () != ConstraintData.Num()
	UPROPERTY()
	FRigUnit_AimConstraint_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
