// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Constraint.h"
#include "ControlRigDefines.h"
#include "RigUnit_ApplyFK.generated.h"

#define UE_API CONTROLRIG_API

UENUM()
enum class EApplyTransformMode : uint8
{
	/** Override existing motion */
	Override,

	/** Additive to existing motion*/
	Additive,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

USTRUCT(meta=(DisplayName="Apply FK", Category="Transforms", Deprecated = "4.23.0"))
struct FRigUnit_ApplyFK : public FRigUnitMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = "ApplyFK", meta = (Input))
	FName Joint;

	UPROPERTY(meta=(Input))
	FTransform Transform;

	/** The filter determines what axes can be manipulated by the in-viewport widgets */
	UPROPERTY(EditAnywhere, Category = "ApplyFK", meta = (Input))
	FTransformFilter Filter;

	UPROPERTY(EditAnywhere, Category = "ApplyFK", meta = (Input))
	EApplyTransformMode ApplyTransformMode = EApplyTransformMode::Override;

	UPROPERTY(EditAnywhere, Category = "ApplyFK", meta = (Input))
	ETransformSpaceMode ApplyTransformSpace = ETransformSpaceMode::LocalSpace;

	// Transform op option. Use if ETransformSpace is BaseTransform
	UPROPERTY(EditAnywhere, Category = "ApplyFK", meta = (Input))
	FTransform BaseTransform;

	// Transform op option. Use if ETransformSpace is BaseJoint
	UPROPERTY(EditAnywhere, Category = "ApplyFK", meta = (Input))
	FName BaseJoint;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
