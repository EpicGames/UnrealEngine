// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_BlendTransform.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(BlueprintType)
struct FBlendTarget
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "FConstraintTarget")
	FTransform	Transform;

	UPROPERTY(EditAnywhere, Category = "FConstraintTarget")
	float Weight;

	FBlendTarget()
		: Weight (1.f)
	{}
};

USTRUCT(meta = (DisplayName = "Blend(Transform)", Category = "Blend", Deprecated = "4.26.0"))
struct FRigUnit_BlendTransform : public FRigUnit
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FTransform Source;

	UPROPERTY(meta = (Input))
	TArray<FBlendTarget> Targets;

	UPROPERTY(meta = (Output))
	FTransform Result;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
