// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Constraint.h"
#include "ControlRigDefines.h"
#include "RigUnit_GetJointTransform.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Get Joint Transform", Category="Transforms", Deprecated = "4.23.0"))
struct FRigUnit_GetJointTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_GetJointTransform()
		: Type(ETransformGetterType::Current)
		, TransformSpace(ETransformSpaceMode::GlobalSpace)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FName Joint;

	UPROPERTY(meta = (Input))
	ETransformGetterType Type;

	UPROPERTY(meta = (Input))
	ETransformSpaceMode TransformSpace;

	// Transform op option. Use if ETransformSpace is BaseTransform
	UPROPERTY(meta = (Input))
	FTransform BaseTransform;

	// Transform op option. Use if ETransformSpace is BaseJoint
	UPROPERTY(meta = (Input))
	FName BaseJoint;

	// possibly space, relative transform so on can be input
	UPROPERTY(meta=(Output))
	FTransform Output;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
