// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Rigs/RigHierarchyContainer.h"
#include "ControlRigDefines.h"
#include "RigUnit_TwoBoneIKFK.generated.h"

#define UE_API CONTROLRIG_API


USTRUCT(meta=(DisplayName="TwoBoneIK/FK", Category="Transforms", Deprecated = "4.23.0"))
struct FRigUnit_TwoBoneIKFK : public FRigUnitMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_TwoBoneIKFK()
		: PoleTarget(ForceInitToZero)
		, Spin(0.f)
		, IKBlend(1.f)
		, PreviousFKIKBlend(1.f)
		, StartJointIndex(INDEX_NONE)
		, MidJointIndex(INDEX_NONE)
		, EndJointIndex(INDEX_NONE)
		, UpperLimbLength(0.f)
		, LowerLimbLength(0.f)
		, bIsInitialized(false)
	{}

	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (Input))
	FName StartJoint;

	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (Input))
	FName EndJoint;

	// # Transform to use as the pole target(specifies the plane of solution)
	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (Input))
	FVector PoleTarget;

	// Float: Spin(Default : 0.0) # Amount of twist to apply to the solution plane(Additive after application of Pole Target motion)
	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (Input))
	float Spin;

	//# Transform to use as the end effector of the IK system
	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (Input, AllowSourceAccess))
	FTransform EndEffector;

	//Float : IKBlend(Default : 0.0)             # Blend between 0.0 (FK) and 1.0 (IK)solutions
	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (Input, ClampMin = "0", AllowSourceAccess))
	float IKBlend; 

private:

	// Transform : Start Joint FK Transform         # Transform for the Start Joint when in FK mode
	// Transform: Mid Joint FK Transform           # Transform for the Mid Joint when in FK mode
	// Transform : End Joint FK Transform          # Transform for the End Joint when in FK mode
	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (AllowSourceAccess))
	FTransform StartJointFKTransform;

	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (AllowSourceAccess))
	FTransform MidJointFKTransform;

	UPROPERTY(EditAnywhere, Category = "TwoBoneIKFK", meta = (AllowSourceAccess))
	FTransform EndJointFKTransform;

	UPROPERTY(transient)
	float PreviousFKIKBlend;

	UPROPERTY(transient)
	FTransform StartJointIKTransform;
	
	UPROPERTY(transient)
	FTransform MidJointIKTransform;

	UPROPERTY(transient)
	FTransform EndJointIKTransform;

	UPROPERTY(transient)
	int32 StartJointIndex;

	UPROPERTY(transient)
	int32 MidJointIndex;

	UPROPERTY(transient)
	int32 EndJointIndex;

	UPROPERTY(transient)
	float UpperLimbLength;
	
	UPROPERTY(transient)
	float LowerLimbLength;

	UPROPERTY()
	bool bIsInitialized;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
