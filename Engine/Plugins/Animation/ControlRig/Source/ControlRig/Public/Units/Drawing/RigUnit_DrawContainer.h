// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_DrawContainer.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Get Imported Draw Container curve transform and color
 */
USTRUCT(meta=(DisplayName="Get Draw Instruction", Category="Drawing", NodeColor = "0.83077 0.846873 0.049707", Keywords = "Curve,Shape", Deprecated = "5.0"))
struct FRigUnit_DrawContainerGetInstruction : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_DrawContainerGetInstruction()
	{
		InstructionName = NAME_None;
		Transform = FTransform::Identity;
		Color = FLinearColor::Red;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Constant, CustomWidget = "DrawingName"))
	FName InstructionName;

	UPROPERTY(meta = (Output))
	FLinearColor Color;

	UPROPERTY(meta = (Output))
	FTransform Transform;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Set Imported Draw Container curve color
 */
USTRUCT(meta = (DisplayName = "Set Draw Color", Category = "Drawing", NodeColor = "0.83077 0.846873 0.049707", Keywords = "Curve,Shape", Deprecated = "5.0"))
struct FRigUnit_DrawContainerSetColor: public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_DrawContainerSetColor()
	{
		InstructionName = NAME_None;
		Color = FLinearColor::Red;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Constant, CustomWidget = "DrawingName"))
	FName InstructionName;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Set Imported Draw Container curve thickness
 */
USTRUCT(meta = (DisplayName = "Set Draw Thickness", Category = "Drawing", NodeColor = "0.83077 0.846873 0.049707", Keywords = "Curve,Shape", Deprecated = "5.0"))
struct FRigUnit_DrawContainerSetThickness : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_DrawContainerSetThickness()
	{
		InstructionName = NAME_None;
		Thickness = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Constant, CustomWidget = "DrawingName"))
	FName InstructionName;

	UPROPERTY(meta = (Input))
	float Thickness;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Set Imported Draw Container curve transform
 */
USTRUCT(meta = (DisplayName = "Set Draw Transform", Category = "Drawing", NodeColor = "0.83077 0.846873 0.049707", Keywords = "Curve,Shape", Deprecated = "5.0"))
struct FRigUnit_DrawContainerSetTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_DrawContainerSetTransform()
	{
		InstructionName = NAME_None;
		Transform = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Constant, CustomWidget = "DrawingName"))
	FName InstructionName;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
