// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_BeginExecution.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Event for driving the skeleton hierarchy with variables and rig elements
 */
USTRUCT(meta=(DisplayName="Forwards Solve", Category="Events", NodeColor="1, 0, 0", Keywords="Begin,Update,Tick,Forward,Event"))
struct FRigUnit_BeginExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "BeginExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("Forwards Solve"));
};

/**
 * Event always executed before the forward solve
 */
USTRUCT(meta=(DisplayName="Pre Forwards Solve", Category="Events", NodeColor="1, 0, 0", Keywords="Begin,Update,Tick,PreForward,Event"))
struct FRigUnit_PreBeginExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "BeginExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("Pre Forwards Solve"));
};

/**
 * Event always executed after the forward solve
 */
USTRUCT(meta=(DisplayName="Post Forwards Solve", Category="Events", NodeColor="1, 0, 0", Keywords="Begin,Update,Tick,PostForward,Event"))
struct FRigUnit_PostBeginExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "BeginExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("Post Forwards Solve"));
};

#undef UE_API
