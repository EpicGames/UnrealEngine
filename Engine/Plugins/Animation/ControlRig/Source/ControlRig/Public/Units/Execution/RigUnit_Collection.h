// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h" 
#include "RigUnit_Collection.generated.h"

#define UE_API CONTROLRIG_API

/*
 * Base class for all pure collection nodes
 */
USTRUCT(meta = (Abstract, NodeColor = "0.4627450108528137 1.0 0.3294120132923126", Category = "Items", DocumentationPolicy = "Strict"))
struct FRigUnit_CollectionBase : public FRigUnit
{
	GENERATED_BODY()
};

/*
 * Base class for all mutable collection nodes
 */
USTRUCT(meta = (Abstract, NodeColor = "0.4627450108528137 1.0 0.3294120132923126", Category = "Items", DocumentationPolicy = "Strict"))
struct FRigUnit_CollectionBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

/**
 * Creates a collection based on a first and last item within a chain.
 * Chains can refer to bone chains or chains within a control hierarchy.
 */
USTRUCT(meta=(DisplayName="Item Chain", Keywords="Bone,Joint,Collection", Varying, Deprecated = "5.0"))
struct FRigUnit_CollectionChain : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionChain()
	{
		FirstItem = LastItem = FRigElementKey(NAME_None, ERigElementType::Bone);
		Reverse = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey FirstItem;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey LastItem;

	UPROPERTY(meta = (Input))
	bool Reverse;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Creates an item array based on a first and last item within a chain.
* Chains can refer to bone chains or chains within a control hierarchy.
*/
USTRUCT(meta=(DisplayName="Item Chain", Keywords="Bone,Joint,Collection", Varying, Deprecated = "5.4"))
struct FRigUnit_CollectionChainArray : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionChainArray()
	{
		FirstItem = LastItem = FRigElementKey(NAME_None, ERigElementType::Bone);
		Reverse = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey FirstItem;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey LastItem;

	UPROPERTY(meta = (Input))
	bool Reverse;

	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Creates a collection based on a name search.
 * The name search is case sensitive.
 */
USTRUCT(meta = (DisplayName = "Item Name Search", Keywords = "Bone,Joint,Collection,Filter", Varying, Deprecated = "5.0"))
struct FRigUnit_CollectionNameSearch : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionNameSearch()
	{
		PartialName = NAME_None;
		TypeToSearch = ERigElementType::All;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FName PartialName;

	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Creates an item array based on a name search.
* The name search is case sensitive.
*/
USTRUCT(meta = (DisplayName = "Item Name Search", Keywords = "Bone,Joint,Collection,Filter", Varying))
struct FRigUnit_CollectionNameSearchArray : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionNameSearchArray()
	{
		PartialName = NAME_None;
		TypeToSearch = ERigElementType::All;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The partial name to look for
	UPROPERTY(meta = (Input))
	FName PartialName;

	// The type of elements to look for
	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	// The resulting array of elements matching the filters
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
 * Creates a collection based on the direct or recursive children
 * of a provided parent item. Returns an empty collection for an invalid parent item.
 */
USTRUCT(meta = (DisplayName = "Get Children", Keywords = "Bone,Joint,Collection,Filter,Parent", Varying, Deprecated = "5.0"))
struct FRigUnit_CollectionChildren : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionChildren()
	{
		Parent = FRigElementKey(NAME_None, ERigElementType::Bone);
		bIncludeParent = false;
		bRecursive = false;
		TypeToSearch = ERigElementType::All;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	UPROPERTY(meta = (Input))
	bool bIncludeParent;

	UPROPERTY(meta = (Input))
	bool bRecursive;

	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Creates an item array based on the direct or recursive children
* of a provided parent item. Returns an empty array for an invalid parent item.
*/
USTRUCT(meta = (DisplayName = "Get Children", Category = "Hierarchy", Keywords = "Bone,Joint,Collection,Filter,Parent", Varying))
struct FRigUnit_CollectionChildrenArray : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionChildrenArray()
	{
		Parent = FRigElementKey(NAME_None, ERigElementType::Bone);
		bIncludeParent = false;
		bRecursive = false;
		bDefaultChildren = true;
		TypeToSearch = ERigElementType::All;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The parent to search the children for
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	// If True the parent will be included in the list of results
	UPROPERTY(meta = (Input))
	bool bIncludeParent;

	// If True children of children will be included
	UPROPERTY(meta = (Input))
	bool bRecursive;

	/** When true, it will return all children, regardless of whether the parent is active or not.
	 * When false, will return only the children which are influenced by this parent */
	UPROPERTY(meta = (Input))
	bool bDefaultChildren;

	// The type of children to look for
	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	// The resulting array of children
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
* Creates an item array for all elements given the filter.
*/
USTRUCT(meta = (DisplayName = "Get All", Category = "Hierarchy", Keywords = "Bone,Joint,Collection,Filter,Parent", Varying))
struct FRigUnit_CollectionGetAll : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionGetAll()
	{
		TypeToSearch = ERigElementType::All;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The type of elements to look for
	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	// The resulting array of elements
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
 * Replaces all names within the collection
 */
USTRUCT(meta = (DisplayName = "Replace Items", Keywords = "Replace,Find,Collection", Varying, Deprecated = "5.0"))
struct FRigUnit_CollectionReplaceItems : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionReplaceItems()
	{
		Old = New = NAME_None;
		RemoveInvalidItems = false;
		bAllowDuplicates = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	UPROPERTY(meta = (Input))
	FName Old;

	UPROPERTY(meta = (Input))
	FName New;

	UPROPERTY(meta = (Input))
	bool RemoveInvalidItems;

	UPROPERTY(meta = (Input))
	bool bAllowDuplicates;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Replaces all names within the item array
*/
USTRUCT(meta = (DisplayName = "Replace Items", Keywords = "Replace,Find,Collection", Varying))
struct FRigUnit_CollectionReplaceItemsArray : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionReplaceItemsArray()
	{
		Old = New = NAME_None;
		RemoveInvalidItems = false;
		bAllowDuplicates = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input array of items to replace
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	// The old name to look for
	UPROPERTY(meta = (Input))
	FName Old;

	// The new name to replace the old name with
	UPROPERTY(meta = (Input))
	FName New;

	// If True items that don't exist (after the replace operation) will be removed from the list of results
	UPROPERTY(meta = (Input))
	bool RemoveInvalidItems;

	// If True the list may contain duplicates
	UPROPERTY(meta = (Input))
	bool bAllowDuplicates;

	// The resulting array of items
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Result;
};

/**
 * Returns a collection provided a specific array of items.
 */
USTRUCT(meta = (DisplayName = "Collection from Items", Category = "Items|Collections", Keywords = "Collection,Array", Varying))
struct FRigUnit_CollectionItems : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionItems()
	{
		Items.Add(FRigElementKey(NAME_None, ERigElementType::Bone));
		bAllowDuplicates = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input array of items
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigElementKey> Items;

	// If True the collection may contain duplicates 
	UPROPERTY(meta = (Input))
	bool bAllowDuplicates;

	// The resulting collection
	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;
};

/**
* Returns an array of items provided a collection
*/
USTRUCT(meta = (DisplayName = "Get Items from Collection", Category = "Items|Collections", Keywords = "Collection,Array", Varying))
struct FRigUnit_CollectionGetItems : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionGetItems()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input collection to retrieve the items from
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	// The resulting items in the collection 
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
 * Returns an array of relative parent indices for each item. Several options here
 * a) If an item has multiple parents the major parent (based on the weights) will be returned.
 * b) If an item has a parent that's not part of the collection INDEX_NONE will be returned.
 * c) If an item has a parent that's not part of the collection, but a grand parent is we'll use that index instead.
 */
USTRUCT(meta = (DisplayName = "Get Parent Indices", Keywords = "Collection,Array", Varying, Deprecated = "5.0"))
struct FRigUnit_CollectionGetParentIndices : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionGetParentIndices()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The collection to retrieve the parent indices from
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	// The resulting parent indices
	UPROPERTY(meta = (Output))
	TArray<int32> ParentIndices;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Returns an array of relative parent indices for each item. Several options here
* a) If an item has multiple parents the major parent (based on the weights) will be returned.
* b) If an item has a parent that's not part of the collection INDEX_NONE will be returned.
* c) If an item has a parent that's not part of the collection, but a grand parent is we'll use that index instead.
*/
USTRUCT(meta = (DisplayName = "Get Parent Indices", Keywords = "Collection,Array", Varying))
struct FRigUnit_CollectionGetParentIndicesItemArray : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionGetParentIndicesItemArray()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input array of items to retrieve the parent indices for
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	// The resulting array of parent indices
	UPROPERTY(meta = (Output))
	TArray<int32> ParentIndices;
};

/**
 * Returns the union of two provided collections
 * (the combination of all items from both A and B).
 */
USTRUCT(meta = (DisplayName = "Union", Keywords = "Combine,Add,Merge,Collection,Hierarchy", Deprecated = "5.0"))
struct FRigUnit_CollectionUnion : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionUnion()
	{
		bAllowDuplicates = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection A;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection B;

	UPROPERTY(meta = (Input))
	bool bAllowDuplicates;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the intersection of two provided collections
 * (the items present in both A and B).
 */
USTRUCT(meta = (DisplayName = "Intersection", Keywords = "Combine,Merge,Collection,Hierarchy", Deprecated = "5.0"))
struct FRigUnit_CollectionIntersection : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionIntersection()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection A;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection B;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the difference between two collections
 * (the items present in A but not in B).
 */
USTRUCT(meta = (DisplayName = "Difference", Keywords = "Collection,Exclude,Subtract", Deprecated = "5.0"))
struct FRigUnit_CollectionDifference : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionDifference()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection A;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection B;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Collection;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the collection in reverse order
 */
USTRUCT(meta = (DisplayName = "Reverse", Keywords = "Direction,Order,Reverse", Deprecated = "5.0"))
struct FRigUnit_CollectionReverse : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionReverse()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Reversed;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the number of elements in a collection
 */
USTRUCT(meta = (DisplayName = "Count", Keywords = "Collection,Array,Count,Num,Length,Size", Deprecated = "5.0"))
struct FRigUnit_CollectionCount : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionCount()
	{
		Collection = FRigElementKeyCollection();
		Count = 0;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	UPROPERTY(meta = (Output))
	int32 Count;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns a single item within a collection by index
 */
USTRUCT(meta = (DisplayName = "Item At Index", Keywords = "Item,GetIndex,AtIndex,At,ForIndex,[]", Deprecated = "5.0"))
struct FRigUnit_CollectionItemAtIndex : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionItemAtIndex()
	{
		Collection = FRigElementKeyCollection();
		Index = 0;
		Item = FRigElementKey();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	UPROPERTY(meta = (Input))
	int32 Index;

	UPROPERTY(meta = (Output))
	FRigElementKey Item;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Given a collection of items, execute iteratively across all items in a given collection
 */
USTRUCT(meta=(DisplayName="For Each Item", Keywords="Collection,Loop,Iterate", Icon="EditorStyle|GraphEditor.Macro.ForEach_16x", Deprecated = "5.0"))
struct FRigUnit_CollectionLoop : public FRigUnit_CollectionBaseMutable
{
	GENERATED_BODY()

	FRigUnit_CollectionLoop()
	{
		BlockToRun = NAME_None;
		Count = 0;
		Index = 0;
		Ratio = 0.f;
	}

	// FRigVMStruct overrides
#if WITH_EDITORONLY_DATA
	virtual const TArray<FName>& GetControlFlowBlocks_Impl() const override
	{
		static const TArray<FName> Blocks = {ExecutePinName, ForLoopCompletedPinName};
		return Blocks;
	}
#endif
	virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const { return InBlockName == ExecutePinName; }
	virtual int32 GetNumSlices() const override { return Count; }

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Singleton))
	FName BlockToRun;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	UPROPERTY(meta = (Output))
	FRigElementKey Item;

	UPROPERTY(meta = (Singleton, Output))
	int32 Index;

	UPROPERTY(meta = (Singleton, Output))
	int32 Count;

	/**
	 * Ranging from 0.0 (first item) and 1.0 (last item)
	 * This is useful to drive a consecutive node with a 
	 * curve or an ease to distribute a value.
	 */
	UPROPERTY(meta = (Singleton, Output))
	float Ratio;

	UPROPERTY(meta = (Output))
	FControlRigExecuteContext Completed;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Adds an element to an existing collection
*/
USTRUCT(meta = (DisplayName = "Add Item", Keywords = "Item,Add,Push,Insert", Deprecated = "5.0"))
struct FRigUnit_CollectionAddItem : public FRigUnit_CollectionBase
{
	GENERATED_BODY()

	FRigUnit_CollectionAddItem()
	{
		Collection = Result = FRigElementKeyCollection();
		Item = FRigElementKey();
	}

	RIGVM_METHOD()
    UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Collection;

	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Result;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
