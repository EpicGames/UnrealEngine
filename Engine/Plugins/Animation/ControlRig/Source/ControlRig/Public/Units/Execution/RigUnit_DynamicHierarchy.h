// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h" 
#include "RigUnit_DynamicHierarchy.generated.h"

#define UE_API CONTROLRIG_API

/*
 * The base class for all pure dynamic hierarchy nodes
 */
USTRUCT(meta = (Abstract, NodeColor="0.262745, 0.8, 0, 0.229412", Category = "DynamicHierarchy", DocumentationPolicy = "Strict"))
struct FRigUnit_DynamicHierarchyBase : public FRigUnit
{
	GENERATED_BODY()

	static UE_API bool IsValidToRunInContext(
		const FControlRigExecuteContext& InExecuteContext,
		bool bAllowOnlyConstructionEvent,
		FString* OutErrorMessage = nullptr);
};

/*
 * The base class for all mutable dynamic hierarchy nodes
 */
USTRUCT(meta = (Abstract, NodeColor="0.262745, 0.8, 0, 0.229412", Category = "DynamicHierarchy", DocumentationPolicy = "Strict"))
struct FRigUnit_DynamicHierarchyBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

/**
* Adds a new parent to an element. The weight for the new parent will be 0.0.
* You can use the SetParentWeights node to change the parent weights later.
*
* If you just want to add a space to a control you can use the 'Add Spaces' node.
*/
USTRUCT(meta=(DisplayName="Create Parent Relationship", Keywords="Children,Parent,Constraint,Space,AddParent", Varying))
struct FRigUnit_AddParent : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddParent()
	{
		Child = Parent = FRigElementKey(NAME_None, ERigElementType::Control);
		DisplayLabel = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The child to be parented under the new parent
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/*
	 * The new parent to be added to the child
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	/*
	 * The optional display label for the parent constraint / space
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FName DisplayLabel;
};

/**
 * Changes the default parent for an item - this removes all other current parents.
 */
USTRUCT(meta=(DisplayName="Set Default Parent", Keywords="Children,Parent,Constraint,Space,SetParent,AddParent", Varying))
struct FRigUnit_SetDefaultParent : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetDefaultParent()
	{
		Child = Parent = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The child to be parented under the new default parent
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/*
	 * The default parent to be used for the child
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;
};

/**
* Adds available spaces for a given control. This is used for the space switching widget.
*/
USTRUCT(meta=(DisplayName="Add Spaces", Keywords="Children,AddParent,Constraint,Space", Varying))
struct FRigUnit_AddAvailableSpaces : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddAvailableSpaces()
	{
		Control = FRigElementKey(NAME_None, ERigElementType::Control);
		Spaces = { FRigElementKeyWithLabel(FRigElementKey(NAME_None, ERigElementType::Control), NAME_None)};
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The available spaces to add for a given control
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Control;

	/*
	 * The spaces to add
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigElementKeyWithLabel> Spaces;
};

/**
 * Allows an animation channel to be hosted by multiple controls
 */
USTRUCT(meta=(DisplayName="Set Channel Hosts", Keywords="Children,Parent,Constraint,Space,SetParent,AddParent,Channel", Varying))
struct FRigUnit_SetChannelHosts : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetChannelHosts()
	{
		Channel = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The channel to receive more hosts
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Channel;

	/*
	 * The hosts to add to the channel
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigElementKey> Hosts;
};

UENUM(meta = (RigVMTypeAllowed))
enum class ERigSwitchParentMode : uint8
{
	/** Switches the element to be parented to the world */
	World,

	/** Switches back to the original / default parent */
	DefaultParent,

	/** Switches the child to the provided parent item */
	ParentItem
};

/**
* Switches an element to a new parent.
*/
USTRUCT(meta=(DisplayName="Switch Parent", Keywords="Children,Parent,Constraint,Space,Switch", Varying))
struct FRigUnit_SwitchParent : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SwitchParent()
	{
		Mode = ERigSwitchParentMode::ParentItem;
		Child = Parent = FRigElementKey(NAME_None, ERigElementType::Control);
		bMaintainGlobal = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/* Depending on this the child will switch to the world,
	 * back to its default or to the item provided by the Parent pin
	 */
	UPROPERTY(meta = (Input))
	ERigSwitchParentMode Mode;

	/* The child to switch to a new parent */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/* The optional parent to switch to. This is only used if the mode is set to 'Parent Item' */
	UPROPERTY(meta = (Input, ExpandByDefault, EditCondition="Mode==ParentItem"))
	FRigElementKey Parent;

	/* If set to true the item will maintain its global transform,
	 * otherwise it will maintain local
	 */
	UPROPERTY(meta = (Input))
	bool bMaintainGlobal;
};

/**
* Returns the item's parents' weights
*/
USTRUCT(meta=(DisplayName="Get Parent Weights", Keywords="Chain,Parents,Hierarchy", Varying, Deprecated = "5.0"))
struct FRigUnit_HierarchyGetParentWeights : public FRigUnit_DynamicHierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetParentWeights()
	{
		Child = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The child to retrieve the weights for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/*
	 * The weight of each parent
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementWeight> Weights;

	/*
	 * The key for each parent
	 */
	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Parents;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Returns the item's parents' weights
*/
USTRUCT(meta=(DisplayName="Get Parent Weights", Keywords="Chain,Parents,Hierarchy", Varying))
struct FRigUnit_HierarchyGetParentWeightsArray : public FRigUnit_DynamicHierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetParentWeightsArray()
	{
		Child = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The child to retrieve the weights for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/*
	 * The weight of each parent
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementWeight> Weights;

	/*
	 * The key for each parent
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Parents;
};

/**
* Sets the item's parents' weights
*/
USTRUCT(meta=(DisplayName="Set Parent Weights", Keywords="Chain,Parents,Hierarchy", Varying))
struct FRigUnit_HierarchySetParentWeights : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetParentWeights()
	{
		Child = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The child to set the parents' weights for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/*
	 * The weights to set for the child's parents.
	 * The number of weights needs to match the current number of parents.
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementWeight> Weights;
};

/**
 * Removes all elements from the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Reset Hierarchy", Keywords="Delete,Erase,Clear,Empty,RemoveAll", Varying))
struct FRigUnit_HierarchyReset : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyReset()
	{
	}

	RIGVM_METHOD()
	virtual void Execute() override;
};

/**
 * Imports all bones (and curves) from the currently assigned skeleton.
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Import Skeleton", Keywords="Skeleton,Mesh,AddBone,AddCurve,Spawn", Varying))
struct FRigUnit_HierarchyImportFromSkeleton : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyImportFromSkeleton()
	{
		NameSpace = NAME_None;
		bIncludeCurves = bIncludeMeshSockets = false;
		bIncludeVirtualBones = true;
	}

	RIGVM_METHOD()
	virtual void Execute() override;

	// The namespace to use when importing bones and curves
	UPROPERTY(meta = (Input))
	FName NameSpace;

	// If True the curves will be imported
	UPROPERTY(meta = (Input))
	bool bIncludeCurves;

	// If true the mesh sockets will be imported
	UPROPERTY(meta = (Input))
	bool bIncludeMeshSockets;

	// If true virtual bones will be imported
	UPROPERTY(meta = (Input))
	bool bIncludeVirtualBones;

	// The resulting imported items
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
 * Removes an element from the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Remove Item", Keywords="Delete,Erase,Joint,Skeleton", Varying))
struct FRigUnit_HierarchyRemoveElement : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyRemoveElement()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		bSuccess = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The item to remove
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/*
	 * True if the element has been removed successfully 
	 */
	UPROPERTY(meta = (Output, ExpandByDefault))
	bool bSuccess;
};

/*
 * The base class for all spawning nodes for the dynamic hierarchy set of nodes
 */
USTRUCT(meta=(Abstract))
struct FRigUnit_HierarchyAddElement : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddElement()
	{
		Parent = Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		Name = NAME_None;
	}

	virtual ERigElementType GetElementTypeToSpawn() const { return ERigElementType::None; }

	/*
	 * The parent of the new element to add
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	/*
	 * The name of the new element to add
	 */
	UPROPERTY(meta = (Input))
	FName Name;

	/*
	 * The resulting item
	 */
	UPROPERTY(meta = (Output))
	FRigElementKey Item;
};

/**
 * Adds a new bone to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Bone", Keywords="Construction,Create,New,Joint,Skeleton", Varying))
struct FRigUnit_HierarchyAddBone : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddBone()
	{
		Name = TEXT("NewBone");
		Transform = FTransform::Identity;
		Space = ERigVMTransformSpace::LocalSpace;
	}

	virtual ERigElementType GetElementTypeToSpawn() const override { return ERigElementType::Bone; }

	/*
	 * The initial transform of the new element
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * Defines if the transform should be interpreted in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Adds a new null to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Null", Keywords="Construction,Create,New,Locator,Group", Varying))
struct FRigUnit_HierarchyAddNull : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddNull()
	{
		Name = TEXT("NewNull");
		Transform = FTransform::Identity;
		Space = ERigVMTransformSpace::LocalSpace;
	}

	virtual ERigElementType GetElementTypeToSpawn() const override { return ERigElementType::Null; }

	/*
	 * The initial transform of the new element
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * Defines if the transform should be interpreted in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControl_Settings()
		: DisplayName(NAME_None), bSelectable(true)
	{}
	virtual ~FRigUnit_HierarchyAddControl_Settings()= default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FName DisplayName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bSelectable;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControl_ShapeSettings
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControl_ShapeSettings()
		: bVisible(true)
		, Name(TEXT("Default"))
		, Color(FLinearColor::Red)
		, Transform(FTransform::Identity)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bVisible;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta=(CustomWidget = "ShapeName"))
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FLinearColor Color;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FTransform Transform;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControl_ProxySettings
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControl_ProxySettings()
		: bIsProxy(false)
		, ShapeVisibility(ERigControlVisibility::BasedOnSelection)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bIsProxy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<FRigElementKey> DrivenControls;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigControlVisibility ShapeVisibility;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlFloat_LimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddControlFloat_LimitSettings()
		: Limit(true, true)
		, MinValue(0.f)
		, MaxValue(100.f)
		, bDrawLimits(true)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Limit;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	float MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	float MaxValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDrawLimits;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlFloat_Settings : public FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()
	
	FRigUnit_HierarchyAddControlFloat_Settings()
		: FRigUnit_HierarchyAddControl_Settings()
		, PrimaryAxis(ERigControlAxis::X)
		, bIsScale(false)
	{}
	virtual ~FRigUnit_HierarchyAddControlFloat_Settings() override = default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings) override;
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigControlAxis PrimaryAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bIsScale;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControlFloat_LimitSettings Limits;;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ProxySettings Proxy;
};

/**
 * Adds a new control to the hierarchy
 */
USTRUCT(meta=(TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl", Varying))
struct FRigUnit_HierarchyAddControlElement : public FRigUnit_HierarchyAddElement
{
public:
	
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlElement()
	{
		Name = TEXT("NewControl");
		OffsetSpace = ERigVMTransformSpace::LocalSpace;
	}

	virtual ERigElementType GetElementTypeToSpawn() const override { return ERigElementType::Control; }
	virtual ERigControlType GetControlTypeToSpawn() const { return ERigControlType::Bool; }

	/*
	 * The offset transform of the new control
	 */
	UPROPERTY(meta = (Input))
	FTransform OffsetTransform;

	/*
	 * The space the offset is in
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace OffsetSpace;

protected:
	
	static UE_API FTransform ProjectOffsetTransform(const FTransform& InOffsetTransform, ERigVMTransformSpace InOffsetSpace, const FRigElementKey& InParent, const URigHierarchy* InHierarchy);
};

/**
 * Adds a new control to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Float Control", TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl", Varying))
struct FRigUnit_HierarchyAddControlFloat : public FRigUnit_HierarchyAddControlElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlFloat()
	{
		InitialValue = 0.f;
	}

	virtual ERigControlType GetControlTypeToSpawn() const override { return Settings.bIsScale ? ERigControlType::ScaleFloat : ERigControlType::Float; }

	/*
	 * The initial value of the new control
	 */
	UPROPERTY(meta = (Input))
	float InitialValue;
	
	/*
	 * The settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControlFloat_Settings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlInteger_LimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddControlInteger_LimitSettings()
		: Limit(true, true)
		, MinValue(0)
		, MaxValue(100)
		, bDrawLimits(true)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Limit;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	int32 MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	int32 MaxValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDrawLimits;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlInteger_Settings : public FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()
	
	FRigUnit_HierarchyAddControlInteger_Settings()
		: FRigUnit_HierarchyAddControl_Settings()
		, PrimaryAxis(ERigControlAxis::X)
	{}
	virtual ~FRigUnit_HierarchyAddControlInteger_Settings() override = default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings) override;
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigControlAxis PrimaryAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TObjectPtr<UEnum> ControlEnum;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControlInteger_LimitSettings Limits;;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ProxySettings Proxy;
};

/**
 * Adds a new control to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Integer Control", TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl", Varying))
struct FRigUnit_HierarchyAddControlInteger : public FRigUnit_HierarchyAddControlElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlInteger()
	{
		InitialValue = 0;
	}

	virtual ERigControlType GetControlTypeToSpawn() const override { return ERigControlType::Integer; }

	/*
	 * The initial value of the new control
	 */
	UPROPERTY(meta = (Input))
	int32 InitialValue;
	
	/*
	 * The settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControlInteger_Settings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlVector2D_LimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddControlVector2D_LimitSettings()
		: LimitX(true, true)
		, LimitY(true, true)
		, MinValue(FVector2D::ZeroVector)
		, MaxValue(FVector2D(1.f, 1.f))
		, bDrawLimits(true)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitX;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitY;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FVector2D MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FVector2D MaxValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDrawLimits;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlVector2D_Settings : public FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()
	
	FRigUnit_HierarchyAddControlVector2D_Settings()
		: FRigUnit_HierarchyAddControl_Settings()
		, PrimaryAxis(ERigControlAxis::X)
	{
		FilteredChannels.Reset();
	}
	virtual ~FRigUnit_HierarchyAddControlVector2D_Settings() override = default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings) override;
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigControlAxis PrimaryAxis;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControlVector2D_LimitSettings Limits;;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ProxySettings Proxy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<ERigControlTransformChannel> FilteredChannels;
};

/**
 * Adds a new control to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Vector2D Control", TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl", Varying))
struct FRigUnit_HierarchyAddControlVector2D : public FRigUnit_HierarchyAddControlElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlVector2D()
	{
		InitialValue = FVector2D::ZeroVector;
	}

	virtual ERigControlType GetControlTypeToSpawn() const override { return ERigControlType::Vector2D; }

	/*
	 * The initial value of the new control
	 */
	UPROPERTY(meta = (Input))
	FVector2D InitialValue;
	
	/*
	 * The settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControlVector2D_Settings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlVector_LimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddControlVector_LimitSettings()
		: LimitX(false, false)
		, LimitY(false, false)
		, LimitZ(false, false)
		, MinValue(FVector::ZeroVector)
		, MaxValue(FVector::OneVector)
		, bDrawLimits(true)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitX;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitY;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitZ;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FVector MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FVector MaxValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDrawLimits;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlVector_Settings : public FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()
	
	FRigUnit_HierarchyAddControlVector_Settings()
		: FRigUnit_HierarchyAddControl_Settings()
		, InitialSpace(ERigVMTransformSpace::LocalSpace)
		, bIsPosition(true)
	{
		FilteredChannels.Reset();
	}
	virtual ~FRigUnit_HierarchyAddControlVector_Settings() override = default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings) override;
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigVMTransformSpace InitialSpace;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bIsPosition;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControlVector_LimitSettings Limits;;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ProxySettings Proxy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<ERigControlTransformChannel> FilteredChannels;
};

/**
 * Adds a new control to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Vector Control", TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl", Varying))
struct FRigUnit_HierarchyAddControlVector : public FRigUnit_HierarchyAddControlElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlVector()
	{
		InitialValue = FVector::ZeroVector;
	}

	virtual ERigControlType GetControlTypeToSpawn() const override
	{
		return Settings.bIsPosition ? ERigControlType::Position : ERigControlType::Scale;
	}

	/*
	 * The initial value of the new control
	 */
	UPROPERTY(meta = (Input))
	FVector InitialValue;

	/*
	 * The settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControlVector_Settings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlRotator_LimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddControlRotator_LimitSettings()
		: LimitPitch(false, false)
		, LimitYaw(false, false)
		, LimitRoll(false, false)
		, MinValue(FRotator(-180.f, -180.f, -180.f))
		, MaxValue(FRotator(180.f, 180.f, 180.f))
		, bDrawLimits(true)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitPitch;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitYaw;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitRoll;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRotator MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRotator MaxValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDrawLimits;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlRotator_Settings : public FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()
	
	FRigUnit_HierarchyAddControlRotator_Settings()
		: FRigUnit_HierarchyAddControl_Settings()
		, InitialSpace(ERigVMTransformSpace::LocalSpace)
		, bUsePreferredRotationOrder(false)
		, PreferredRotationOrder(EEulerRotationOrder::YZX)
	{
		FilteredChannels.Reset();
	}
	virtual ~FRigUnit_HierarchyAddControlRotator_Settings() override = default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings) override;
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigVMTransformSpace InitialSpace;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControlRotator_LimitSettings Limits;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ProxySettings Proxy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<ERigControlTransformChannel> FilteredChannels;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bUsePreferredRotationOrder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	EEulerRotationOrder PreferredRotationOrder;
};

/**
 * Adds a new control to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Rotator Control", TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl,Rotation", Varying))
struct FRigUnit_HierarchyAddControlRotator : public FRigUnit_HierarchyAddControlElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlRotator()
	{
		InitialValue = FRotator::ZeroRotator;
	}

	virtual ERigControlType GetControlTypeToSpawn() const override { return ERigControlType::Rotator; }

	/*
	 * The initial value of the new control
	 */
	UPROPERTY(meta = (Input))
	FRotator InitialValue;

	/*
	 * The settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControlRotator_Settings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlTransform_LimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddControlTransform_LimitSettings()
		: LimitTranslationX(false, false)
		, LimitTranslationY(false, false)
		, LimitTranslationZ(false, false)
		, LimitPitch(false, false)
		, LimitYaw(false, false)
		, LimitRoll(false, false)
		, LimitScaleX(false, false)
		, LimitScaleY(false, false)
		, LimitScaleZ(false, false)
		, MinValue(FEulerTransform(FVector(-100.f, -100.f, -100.f), FRotator(-180.f, -180.f, -180.f), FVector(0.f, 0.f, 0.f)))
		, MaxValue(FEulerTransform(FVector(100.f, 100.f, 100.f), FRotator(180.f, 180.f, 180.f), FVector(10.f, 10.f, 10.f)))
		, bDrawLimits(true)
	{}

	UE_API void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings);
	UE_API void Configure(FRigControlSettings& OutSettings) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitTranslationX;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitTranslationY;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitTranslationZ;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitPitch;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitYaw;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitRoll;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitScaleX;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitScaleY;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled LimitScaleZ;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FEulerTransform MinValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FEulerTransform MaxValue;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDrawLimits;
};


USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddControlTransform_Settings : public FRigUnit_HierarchyAddControl_Settings
{
	GENERATED_BODY()
	
	FRigUnit_HierarchyAddControlTransform_Settings()
		: FRigUnit_HierarchyAddControl_Settings()
		, InitialSpace(ERigVMTransformSpace::LocalSpace)
		, bUsePreferredRotationOrder(false)
		, PreferredRotationOrder(EEulerRotationOrder::YZX) 
	{
		FilteredChannels.Reset();
	}
	virtual ~FRigUnit_HierarchyAddControlTransform_Settings() override = default;

	UE_API virtual void ConfigureFrom(const FRigControlElement* InControlElement, const FRigControlSettings& InSettings) override;
	UE_API virtual void Configure(FRigControlSettings& OutSettings) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	ERigVMTransformSpace InitialSpace;

	// Enables overriding the preferred rotation order
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bUsePreferredRotationOrder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	EEulerRotationOrder PreferredRotationOrder;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControlTransform_LimitSettings Limits;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigUnit_HierarchyAddControl_ProxySettings Proxy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	TArray<ERigControlTransformChannel> FilteredChannels;
};

/**
 * Adds a new control to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Transform Control", TemplateName="SpawnControl", Keywords="Construction,Create,New,AddControl,NewControl,CreateControl", Varying))
struct FRigUnit_HierarchyAddControlTransform : public FRigUnit_HierarchyAddControlElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddControlTransform()
	{
		InitialValue = FTransform::Identity;
	}

	virtual ERigControlType GetControlTypeToSpawn() const override { return ERigControlType::EulerTransform; }

	/*
	 * The initial value of the new control
	 */
	UPROPERTY(meta = (Input))
	FTransform InitialValue;
	
	/*
	 * The settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControlTransform_Settings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings()
	{}
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Bool Animation Channel", TemplateName="SpawnAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelBool : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelBool()
	{
		Name = TEXT("NewChannel");
		InitialValue = false;
		MinimumValue = false;
		MaximumValue = true;
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	bool InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	bool MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	bool MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddAnimationChannelSingleLimitSettings : public FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddAnimationChannelSingleLimitSettings()
		: Enabled(true)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Enabled;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Float Animation Channel", TemplateName="SpawnAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelFloat : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelFloat()
	{
		Name = TEXT("NewChannel");
		InitialValue = 0.f;
		MinimumValue = 0.f;
		MaximumValue = 1.f;
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	float InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	float MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	float MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelSingleLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Scale Float Animation Channel", TemplateName="SpawnScaleAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelScaleFloat : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelScaleFloat()
	{
		Name = TEXT("NewChannel");
		InitialValue = 1.f;
		MinimumValue = 0.f;
		MaximumValue = 10.f;
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	float InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	float MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	float MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelSingleLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Integer Animation Channel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelInteger : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelInteger()
	{
		Name = TEXT("NewChannel");
		InitialValue = 0;
		MinimumValue = 0;
		MaximumValue = 100;
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	int32 InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	int32 MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	int32 MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelSingleLimitSettings LimitsEnabled;

	/*
	 * The enum to use to find valid values
	 */
	UPROPERTY(meta = (Input))
	TObjectPtr<UEnum> ControlEnum;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddAnimationChannel2DLimitSettings : public FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddAnimationChannel2DLimitSettings()
		: X(true)
		, Y(true)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Y;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Vector2D Animation Channel", TemplateName="SpawnAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelVector2D : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelVector2D()
	{
		Name = TEXT("NewChannel");
		InitialValue = FVector2D::ZeroVector;
		MinimumValue = FVector2D::ZeroVector;
		MaximumValue = FVector2D(1.f, 1.f);
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector2D InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector2D MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector2D MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannel2DLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddAnimationChannelVectorLimitSettings : public FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddAnimationChannelVectorLimitSettings()
		: X(true)
		, Y(true)
		, Z(true)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Z;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Vector Animation Channel", TemplateName="SpawnAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelVector : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelVector()
	{
		Name = TEXT("NewChannel");
		InitialValue = FVector::ZeroVector;
		MinimumValue = FVector::ZeroVector;
		MaximumValue = FVector(1.f, 1.f, 1.f);
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelVectorLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Scale Animation Channel", TemplateName="SpawnScaleAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelScaleVector : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelScaleVector()
	{
		Name = TEXT("NewChannel");
		InitialValue = FVector::OneVector;
		MinimumValue = FVector::ZeroVector;
		MaximumValue = FVector(10.f, 10.f, 10.f);
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	FVector MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelVectorLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};


USTRUCT(BlueprintType)
struct FRigUnit_HierarchyAddAnimationChannelRotatorLimitSettings : public FRigUnit_HierarchyAddAnimationChannelEmptyLimitSettings
{
	GENERATED_BODY();
	
	FRigUnit_HierarchyAddAnimationChannelRotatorLimitSettings()
		: Pitch(true)
		, Yaw(true)
		, Roll(true)
	{}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Pitch;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Yaw;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	FRigControlLimitEnabled Roll;
};

/**
 * Adds a new animation channel to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Rotator Animation Channel", TemplateName="SpawnAnimationChannel", Keywords="Construction,Create,New,AddAnimationChannel,NewAnimationChannel,CreateAnimationChannel,AddChannel,NewChannel,CreateChannel,SpawnChannel", Varying))
struct FRigUnit_HierarchyAddAnimationChannelRotator : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddAnimationChannelRotator()
	{
		Name = TEXT("NewChannel");
		InitialValue = FRotator::ZeroRotator;
		MinimumValue = FRotator(-180.f, -180.f, -180.f);
		MaximumValue = FRotator(180.f, 180.f, 180.f);
	}

	/*
	 * The initial value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FRotator InitialValue;

	/*
	 * The minimum value of the new animation channel
	 */
	UPROPERTY(meta = (Input))
	FRotator MinimumValue;

	/*
	 * The maximum value for the animation channel
	 */
	UPROPERTY(meta = (Input))
	FRotator MaximumValue;

	/*
	 * The enable settings for the limits
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddAnimationChannelRotatorLimitSettings LimitsEnabled;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Retrieves the shape settings of a control
 */
USTRUCT(meta=(DisplayName="Get Shape Settings", Keywords="Construction,Create,New,Control", Varying))
struct FRigUnit_HierarchyGetShapeSettings : public FRigUnit_DynamicHierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetShapeSettings()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	/*
	 * The item to change
	 */
	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	/*
	 * The shape settings for the control
	 */
	UPROPERTY(meta = (Output))
	FRigUnit_HierarchyAddControl_ShapeSettings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Changes the shape settings of a control
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Set Shape Settings", Keywords="Construction,Create,New,Control", Varying))
struct FRigUnit_HierarchySetShapeSettings : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetShapeSettings()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Control);
	}

	/*
	 * The item to change
	 */
	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	/*
	 * The shape settings for the control
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_HierarchyAddControl_ShapeSettings Settings;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Adds a new socket to the hierarchy
 * Note: This node only runs as part of the construction event.
 */
USTRUCT(meta=(DisplayName="Spawn Socket", Keywords="Construction,Create,New,Locator,Group", Varying))
struct FRigUnit_HierarchyAddSocket : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddSocket()
	{
		Name = TEXT("NewSocket");
		Transform = FTransform::Identity;
		Space = ERigVMTransformSpace::LocalSpace;
		Color = FLinearColor::White;
		Description = FString();
	}

	virtual ERigElementType GetElementTypeToSpawn() const override { return ERigElementType::Socket; }

	/*
	 * The initial transform of the new element
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * Defines if the transform should be interpreted in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/*
	 * The color of the socket
	 */
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	/*
	 * The (optional) description of the socket
	 */
	UPROPERTY(meta = (Input))
	FString Description;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

#undef UE_API
