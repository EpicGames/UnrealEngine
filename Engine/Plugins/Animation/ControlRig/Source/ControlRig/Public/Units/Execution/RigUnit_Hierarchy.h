// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h" 
#include "RigUnit_Hierarchy.generated.h"

#define UE_API CONTROLRIG_API

/*
 * Base class for all pure hierarchy nodes
 */
USTRUCT(meta = (Abstract, NodeColor="0.462745, 1,0, 0.329412", Category = "Hierarchy", DocumentationPolicy = "Strict"))
struct FRigUnit_HierarchyBase : public FRigUnit
{
	GENERATED_BODY()
};

/*
 * Base class for all mutable hierarchy nodes
 */
USTRUCT(meta = (Abstract, NodeColor="0.462745, 1,0, 0.329412", Category = "Hierarchy", DocumentationPolicy = "Strict"))
struct FRigUnit_HierarchyBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

/**
 * Returns the item's parent
 */
USTRUCT(meta=(DisplayName="Get Parent", Keywords="Child,Parent,Root,Up,Top", Varying))
struct FRigUnit_HierarchyGetParent : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetParent()
	{
		Child = Parent = FRigElementKey(NAME_None, ERigElementType::Bone);
		bDefaultParent = true;
		CachedChild = CachedParent = FCachedRigElement();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The child to look up the parent for
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/** When true, it will return the default parent, regardless of whether the parent influences the element or not  */
	UPROPERTY(meta = (Input))
	bool bDefaultParent;

	// The resulting parent of the input child
	UPROPERTY(meta = (Output))
	FRigElementKey Parent;

	// Used to cache the internally used child
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the internally used parent
	UPROPERTY()
	FCachedRigElement CachedParent;
};

/**
 * Returns the item's parents
 */
USTRUCT(meta=(DisplayName="Get Parents", Keywords="Chain,Parents,Hierarchy", Varying, Deprecated = "5.0"))
struct FRigUnit_HierarchyGetParents : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetParents()
	{
		Child = FRigElementKey(NAME_None, ERigElementType::Bone);
		CachedChild = FCachedRigElement();
		Parents = CachedParents = FRigElementKeyCollection();
		bIncludeChild = false;
		bReverse = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The child to look up all parents for
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	// If True the child will be included in the list of results
	UPROPERTY(meta = (Input))
	bool bIncludeChild;

	// If True the parents list will be reversed
	UPROPERTY(meta = (Input))
	bool bReverse;

	// The resulting array of parents
	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Parents;

	// Used to cache the internally used child
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the internally used parents
	UPROPERTY()
	FRigElementKeyCollection CachedParents;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the item's parents
 */
USTRUCT(meta=(DisplayName="Get Parents", Keywords="Chain,Parents,Hierarchy", Varying))
struct FRigUnit_HierarchyGetParentsItemArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetParentsItemArray()
	{
		Child = FRigElementKey(NAME_None, ERigElementType::Bone);
		CachedChild = FCachedRigElement();
		CachedParents = FRigElementKeyCollection();
		bIncludeChild = false;
		bReverse = false;
		bDefaultParent = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The child to look up the parents for
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	// If True the child will be included in the list of results
	UPROPERTY(meta = (Input))
	bool bIncludeChild;

	// If True the list of results will be reversed
	UPROPERTY(meta = (Input))
	bool bReverse;

	/** When true, it will return the default parent, regardless of whether the parent influences the element or not  */
	UPROPERTY(meta = (Input))
	bool bDefaultParent;

	// The resulting array of parents
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Parents;

	// Used to cache the internally used child
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the internally used parents
	UPROPERTY()
	FRigElementKeyCollection CachedParents;
};


/**
 * Returns the item's children
 */
USTRUCT(meta=(DisplayName="Get Children", Keywords="Chain,Children,Hierarchy", Deprecated = "4.25.0", Varying))
struct FRigUnit_HierarchyGetChildren : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetChildren()
	{
		Parent = FRigElementKey(NAME_None, ERigElementType::Bone);
		CachedParent = FCachedRigElement();
		Children = CachedChildren = FRigElementKeyCollection();
		bIncludeParent = false;
		bRecursive = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	UPROPERTY(meta = (Input))
	bool bIncludeParent;

	UPROPERTY(meta = (Input))
	bool bRecursive;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Children;

	// Used to cache the internally used parent
	UPROPERTY()
	FCachedRigElement CachedParent;

	// Used to cache the internally used children
	UPROPERTY()
	FRigElementKeyCollection CachedChildren;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the item's siblings
 */
USTRUCT(meta=(DisplayName="Get Siblings", Keywords="Chain,Siblings,Hierarchy", Varying, Deprecated = "5.0"))
struct FRigUnit_HierarchyGetSiblings : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetSiblings()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		CachedItem = FCachedRigElement();
		Siblings = CachedSiblings = FRigElementKeyCollection();
		bIncludeItem = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	UPROPERTY(meta = (Input))
	bool bIncludeItem;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Siblings;

	// Used to cache the internally used item
	UPROPERTY()
	FCachedRigElement CachedItem;

	// Used to cache the internally used siblings
	UPROPERTY()
	FRigElementKeyCollection CachedSiblings;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the item's siblings
 */
USTRUCT(meta=(DisplayName="Get Siblings", Keywords="Chain,Siblings,Hierarchy", Varying))
struct FRigUnit_HierarchyGetSiblingsItemArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetSiblingsItemArray()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		CachedItem = FCachedRigElement();
		CachedSiblings = FRigElementKeyCollection();
		bIncludeItem = false;
		bDefaultSiblings = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The item to find the siblings for
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	// If True the input item will be included in the list of results
	UPROPERTY(meta = (Input))
	bool bIncludeItem;

	/** When true, it will return all siblings, regardless of whether the parent is active or not.
	 * When false, will return only the siblings which are influenced by the same parent */
	UPROPERTY(meta = (Input))
	bool bDefaultSiblings;

	// The resulting array of siblings of the input item
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Siblings;

	// Used to cache the internally used item
	UPROPERTY()
	FCachedRigElement CachedItem;

	// Used to cache the internally used siblings
	UPROPERTY()
	FRigElementKeyCollection CachedSiblings;
};

/**
 * Returns a chain between two items
 */
USTRUCT(meta=(DisplayName="Get Chain", Keywords="Chain,Siblings,Hierarchy", Varying))
struct FRigUnit_HierarchyGetChainItemArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetChainItemArray()
	{
		Start = End = FRigElementKey(NAME_None, ERigElementType::Bone);
		CachedStart = CachedEnd = FCachedRigElement();
		CachedChain = FRigElementKeyCollection();
		bIncludeStart = bIncludeEnd = true;
		bReverse = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first element of the chain
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Start;

	// The last element of the chain
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey End;

	// If True the list of results will include the first element
	UPROPERTY(meta = (Input))
	bool bIncludeStart;

	// If True the list of results will include the last element
	UPROPERTY(meta = (Input))
	bool bIncludeEnd;

	// If True the list of results will be reversed
	UPROPERTY(meta = (Input))
	bool bReverse;

	// The resulting array of elements on the chain
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Chain;

	// Used to cache the internally used item
	UPROPERTY()
	FCachedRigElement CachedStart;

	// Used to cache the internally used item
	UPROPERTY()
	FCachedRigElement CachedEnd;

	// Used to cache the internally used siblings
	UPROPERTY()
	FRigElementKeyCollection CachedChain;
};

/**
 * Returns the hierarchy's pose
 */
USTRUCT(meta=(DisplayName="Get Pose Cache", Keywords="Hierarchy,Pose,State", Varying, Deprecated = "5.0"))
struct FRigUnit_HierarchyGetPose : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPose()
	{
		Initial = false;
		ElementType = ERigElementType::All;
		ItemsToGet = FRigElementKeyCollection();
		Pose = FRigPose();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	bool Initial;

	UPROPERTY(meta = (Input))
	ERigElementType ElementType;

	// An optional collection to filter against
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection ItemsToGet;

	UPROPERTY(meta = (Output))
	FRigPose Pose;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the hierarchy's pose
 */
USTRUCT(meta=(DisplayName="Get Pose Cache", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_HierarchyGetPoseItemArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPoseItemArray()
	{
		Initial = false;
		ElementType = ERigElementType::All;
		Pose = FRigPose();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// If True the pose will use the initial transforms
	UPROPERTY(meta = (Input))
	bool Initial;

	// The element type to retrieve the pose for 
	UPROPERTY(meta = (Input))
	ERigElementType ElementType;

	// An optional collection to filter against
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ItemsToGet;

	// The resulting pose
	UPROPERTY(meta = (Output))
	FRigPose Pose;
};


/**
 * Sets the hierarchy's pose
 */
USTRUCT(meta=(DisplayName="Apply Pose Cache", Keywords="Hierarchy,Pose,State", Varying, Deprecated = "5.0"))
struct FRigUnit_HierarchySetPose : public FRigUnit_HierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPose()
	{
		Pose = FRigPose();
		ElementType = ERigElementType::All;
		Space = ERigVMTransformSpace::LocalSpace;
		ItemsToSet = FRigElementKeyCollection();
		Weight = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigPose Pose;

	UPROPERTY(meta = (Input))
	ERigElementType ElementType;

	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// An optional collection to filter against
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection ItemsToSet;

	UPROPERTY(meta = (Input))
	float Weight;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Sets the hierarchy's pose
 */
USTRUCT(meta=(DisplayName="Apply Pose Cache", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_HierarchySetPoseItemArray : public FRigUnit_HierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPoseItemArray()
	{
		Pose = FRigPose();
		ElementType = ERigElementType::All;
		Space = ERigVMTransformSpace::LocalSpace;
		Weight = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The pose to apply to the hierarchy
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// The types of element to apply the pose for
	UPROPERTY(meta = (Input))
	ERigElementType ElementType;

	// The space to use to apply the pose in
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// An optional collection to filter against
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ItemsToSet;

	// The weight to use when applying the transforms
	UPROPERTY(meta = (Input))
	float Weight;
};

/**
* Returns true if the hierarchy pose is empty (has no items)
*/
USTRUCT(meta=(DisplayName="Is Pose Cache Empty", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_PoseIsEmpty : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseIsEmpty()
	{
		Pose = FRigPose();
		IsEmpty = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The pose to check
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// True if the pose is empty
	UPROPERTY(meta = (Output))
	bool IsEmpty;
};

/**
* Returns the items in the hierarchy pose
*/
USTRUCT(meta=(DisplayName="Get Pose Cache Items", Keywords="Hierarchy,Pose,State", Varying, Deprecated = "5.0"))
struct FRigUnit_PoseGetItems : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseGetItems()
	{
		Pose = FRigPose();
		ElementType = ERigElementType::All;
		Items = FRigElementKeyCollection();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FRigPose Pose;

	UPROPERTY(meta = (Input))
	ERigElementType ElementType;

	UPROPERTY(meta = (Output))
	FRigElementKeyCollection Items;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Returns the items in the hierarchy pose
*/
USTRUCT(meta=(DisplayName="Get Pose Cache Items", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_PoseGetItemsItemArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseGetItemsItemArray()
	{
		Pose = FRigPose();
		ElementType = ERigElementType::All;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The pose to retrieve the items from
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// The type of items to retrieve
	UPROPERTY(meta = (Input))
	ERigElementType ElementType;

	// The resulting items in the pose
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
* Compares two pose caches and compares their values.
*/
USTRUCT(meta=(DisplayName="Get Pose Cache Delta", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_PoseGetDelta : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseGetDelta()
	{
		PoseA = PoseB = FRigPose();
		ElementType = ERigElementType::All;
		Space = ERigVMTransformSpace::LocalSpace;
		ItemsToCompare = ItemsWithDelta = FRigElementKeyCollection();
		PositionThreshold = 0.1f;
		RotationThreshold = ScaleThreshold = CurveThreshold = 0.f;
		PosesAreEqual = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first pose to compare
	UPROPERTY(meta = (Input))
	FRigPose PoseA;

	// The second pose to compare
	UPROPERTY(meta = (Input))
	FRigPose PoseB;

	// The delta threshold for a translation / position difference. 0.0 disables position differences.
	UPROPERTY(meta = (Input))
	float PositionThreshold;

	// The delta threshold for a rotation difference (in degrees). 0.0 disables rotation differences.
	UPROPERTY(meta = (Input))
	float RotationThreshold;

	// The delta threshold for a scale difference. 0.0 disables scale differences.
	UPROPERTY(meta = (Input))
	float ScaleThreshold;

	// The delta threshold for curve value difference. 0.0 disables curve differences.
	UPROPERTY(meta = (Input))
	float CurveThreshold;

	// The type of elements to compare
	UPROPERTY(meta = (Input))
	ERigElementType ElementType;
	
	// Defines in which space transform deltas should be computed
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// An optional list of items to compare
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection ItemsToCompare;

	// True if the poses A and B are equal
	UPROPERTY(meta = (Output))
	bool PosesAreEqual;

	// An array of items with a difference in them between poses A and B
	UPROPERTY(meta = (Output))
	FRigElementKeyCollection ItemsWithDelta;

	static UE_API bool ArePoseElementsEqual(
		const FRigPoseElement& A,
		const FRigPoseElement& B,
		ERigVMTransformSpace Space,
		float PositionU,
		float RotationU,
		float ScaleU,
		float CurveU);

	static UE_API bool AreTransformsEqual(
		const FTransform& A,
		const FTransform& B,
		float PositionU,
		float RotationU,
		float ScaleU);

	static UE_API bool AreCurvesEqual(
		float A,
		float B,
		float CurveU);
};

/**
* Returns the hierarchy's pose transform
*/
USTRUCT(meta=(DisplayName="Get Pose Cache Transform", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_PoseGetTransform : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseGetTransform()
	{
		Pose = FRigPose();
		Item = FRigElementKey();
		Space = ERigVMTransformSpace::GlobalSpace;
		Valid = false;
		Transform = FTransform::Identity;
		CurveValue = 0.f;
		CachedPoseElementIndex = CachedPoseHash = INDEX_NONE;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The pose to retrieve the transform from
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// The item to retrieve the transform for
	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	/**
	* Defines if the transform should be retrieved in local or global space
	*/ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// True if the transform retrieved is valid
	UPROPERTY(meta = (Output))
	bool Valid;

	// The resulting transform
	UPROPERTY(meta = (Output))
	FTransform Transform;

	// The value of the curve (in case the pose entry was a curve)
	UPROPERTY(meta = (Output))
	float CurveValue;

	UPROPERTY()
	int32 CachedPoseElementIndex;

	UPROPERTY()
	int32 CachedPoseHash;
};

/**
* Returns an array of transforms from a given hierarchy pose
*/
USTRUCT(meta=(DisplayName="Get Pose Cache Transform Array", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_PoseGetTransformArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseGetTransformArray()
	{
		Pose = FRigPose();
		Space = ERigVMTransformSpace::GlobalSpace;
		Valid = false;
		Transforms.Reset();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The pose to retrieve the transform from
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	/**
	* Defines if the transform should be retrieved in local or global space
	*/ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// True if the transforms has been retrieved successfully
	UPROPERTY(meta = (Output))
	bool Valid;

	// The resulting array of transforms
	UPROPERTY(meta = (Output))
	TArray<FTransform> Transforms;
};

/**
* Returns the hierarchy's pose curve value
*/
USTRUCT(meta=(DisplayName="Get Pose Cache Curve", Keywords="Hierarchy,Pose,State", Varying, Category = "Pose Cache"))
struct FRigUnit_PoseGetCurve : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_PoseGetCurve()
	{
		Pose = FRigPose();
		Curve = NAME_None;
		Valid = false;
		CurveValue = 0.f;
		CachedPoseElementIndex = CachedPoseHash = INDEX_NONE;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The pose to retrieve the curve value from
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// The name of the curve to retrieve
	UPROPERTY(meta = (Input, CustomWidget = "CurveName"))
	FName Curve;

	// True if the curve value has been retrieved successfully
	UPROPERTY(meta = (Output))
	bool Valid;

	// The resulting curve value
	UPROPERTY(meta = (Output))
	float CurveValue;

	UPROPERTY()
	int32 CachedPoseElementIndex;

	UPROPERTY()
	int32 CachedPoseHash;
};

/**
* Given a pose, execute iteratively across all items in the pose
*/
USTRUCT(meta=(DisplayName="For Each Pose Cache Element", Keywords="Collection,Loop,Iterate", Icon="EditorStyle|GraphEditor.Macro.ForEach_16x", Category = "Pose Cache"))
struct FRigUnit_PoseLoop : public FRigUnit_HierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_PoseLoop()
	{
		BlockToRun = NAME_None;
		Pose = FRigPose();
		Item = FRigElementKey();
		GlobalTransform = LocalTransform = FTransform::Identity;
		CurveValue = 0.f;
		Count = 0;
		Index = 0;
		Ratio = 0.f;
	}

	// FRigVMStruct overrides
#if WITH_EDITORONLY_DATA
	virtual const TArray<FName>& GetControlFlowBlocks_Impl() const override
	{
		static const TArray<FName> Blocks = {ExecutePinName, ForLoopCompletedPinName};
		return Blocks;
	}
#endif
	virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const { return InBlockName == ExecutePinName; }
	virtual int32 GetNumSlices() const override { return Count; }

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Singleton))
	FName BlockToRun;

	// The pose to loop over
	UPROPERTY(meta = (Input))
	FRigPose Pose;

	// The item for the current pose entry
	UPROPERTY(meta = (Singleton, Output))
	FRigElementKey Item;

	// The global transform for the current pose entry
	UPROPERTY(meta = (Singleton, Output))
	FTransform GlobalTransform;

	// The local transform for the current pose entry
	UPROPERTY(meta = (Singleton, Output))
	FTransform LocalTransform;

	// The curve value for the current pose entry
	UPROPERTY(meta = (Singleton, Output))
	float CurveValue;

	// The index of the current pose entry
	UPROPERTY(meta = (Singleton, Output))
	int32 Index;

	// The overall count of entries in the pose
	UPROPERTY(meta = (Singleton, Output))
	int32 Count;

	/**
	* Ranging from 0.0 (first item) and 1.0 (last item)
	* This is useful to drive a consecutive node with a 
	* curve or an ease to distribute a value.
	*/
	UPROPERTY(meta = (Singleton, Output))
	float Ratio;

	// The branch to run when the loop has completed
	UPROPERTY(meta = (Output))
	FControlRigExecuteContext Completed;
};

USTRUCT(BlueprintType)
struct FRigUnit_HierarchyCreatePoseItemArray_Entry
{
	GENERATED_BODY()

	FRigUnit_HierarchyCreatePoseItemArray_Entry()
	: Item(NAME_None, ERigElementType::Bone)
	, LocalTransform(FTransform::Identity)
	, GlobalTransform(FTransform::Identity)
	, UseEulerAngles(false)
	, EulerAngles(FVector::ZeroVector)
	, CurveValue(0.f)
	{
	}

	UPROPERTY(BlueprintReadWrite, Category = "Entry")
	FRigElementKey Item;

	UPROPERTY(BlueprintReadWrite, Category = "Entry")
	FTransform LocalTransform;

	UPROPERTY(BlueprintReadWrite, Category = "Entry")
	FTransform GlobalTransform;

	// in case of a control this can be used to drive the preferred euler angles
	UPROPERTY(BlueprintReadWrite, Category = "Entry")
	bool UseEulerAngles;

	// in case of a control this can be used to drive the preferred euler angles
	UPROPERTY(BlueprintReadWrite, Category = "Entry")
	FVector EulerAngles;

	// in case of a curve this can be used to drive the curve value
	UPROPERTY(BlueprintReadWrite, Category = "Entry")
	float CurveValue;
};


/**
 * Creates the hierarchy's pose
 */
USTRUCT(meta=(DisplayName="Create Pose Cache", Keywords="Hierarchy,Pose,State,MakePoseCache,NewPoseCache,EmptyPoseCache", Varying, Category = "Pose Cache"))
struct FRigUnit_HierarchyCreatePoseItemArray : public FRigUnit_HierarchyBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyCreatePoseItemArray()
	{
		Pose = FRigPose();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The entries to create
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_HierarchyCreatePoseItemArray_Entry> Entries;

	// The resulting pose
	UPROPERTY(meta = (Output))
	FRigPose Pose;
};

#undef UE_API
