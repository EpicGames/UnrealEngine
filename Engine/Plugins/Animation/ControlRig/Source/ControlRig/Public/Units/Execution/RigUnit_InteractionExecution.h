// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_InteractionExecution.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Event for executing logic during an interaction
 */
USTRUCT(meta=(DisplayName="Interaction", Category="Events", NodeColor="1, 0, 0", Keywords="Manipulation,Event,During,Interacting"))
struct FRigUnit_InteractionExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "BeginExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("Interaction"));
};

#undef UE_API
