// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_InverseExecution.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Event for driving elements based off the skeleton hierarchy
 */
USTRUCT(meta=(DisplayName="Backwards Solve", Category="Events", NodeColor="1, 0, 0", Keywords="Inverse,Reverse,Backwards,Event"))
struct FRigUnit_InverseExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "InverseExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("Backwards Solve"));
};

#undef UE_API
