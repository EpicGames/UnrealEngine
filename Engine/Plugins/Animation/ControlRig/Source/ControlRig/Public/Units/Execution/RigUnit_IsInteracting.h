// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_IsInteracting.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Returns true if the Control Rig is being interacted
 */
USTRUCT(meta=(DisplayName="Get Interaction", Category="Execution", TitleColor="1 0 0", NodeColor="1 1 1", Keywords="IsInteracting,Gizmo,Manipulation,Interaction,Tracking", Varying))
struct FRigUnit_IsInteracting : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// True if there is currently an interaction happening
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Interacting", Category = "Execution", meta = (Output))
	bool bIsInteracting = false;

	// True if the current interaction is a translation
	UPROPERTY(EditAnywhere, Transient, Category = "Execution", meta = (Output))
	bool bIsTranslating = false;

	// True if the current interaction is a rotation
	UPROPERTY(EditAnywhere, Transient, Category = "Execution", meta = (Output))
	bool bIsRotating = false;

	// True if the current interaction is scaling
	UPROPERTY(EditAnywhere, Transient, Category = "Execution", meta = (Output))
	bool bIsScaling = false;

	// The items being interacted on
	UPROPERTY(EditAnywhere, Transient, Category = "Execution", meta = (Output))
	TArray<FRigElementKey> Items;
};

#undef UE_API
