// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h" 
#include "RigUnit_Item.generated.h"

#define UE_API CONTROLRIG_API

/*
 * Base class for all pure item nodes
 */
USTRUCT(meta = (Abstract, NodeColor = "0.7 0.05 0.5", Category = "Items", DocumentationPolicy = "Strict"))
struct FRigUnit_ItemBase : public FRigUnit
{
	GENERATED_BODY()
};

/*
 * Base class for all mutable item nodes
 */
USTRUCT(meta = (Abstract, NodeColor = "0.7 0.05 0.5", Category = "Items", DocumentationPolicy = "Strict"))
struct FRigUnit_ItemBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

/**
 * Returns true or false if a given item exists
 */
USTRUCT(meta=(DisplayName="Item Exists", Keywords=""))
struct FRigUnit_ItemExists : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemExists()
	{
		Item = FRigElementKey();
		Exists = false;
		CachedIndex = FCachedRigElement();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The item to look up
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	// True if the item exists
	UPROPERTY(meta = (Output))
	bool Exists;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Replaces the text within the name of the item
 */
USTRUCT(meta=(DisplayName="Item Replace", Keywords="Replace,Name"))
struct FRigUnit_ItemReplace : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemReplace()
	{
		Item = Result = FRigElementKey();
		Old = New = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The item to perform the name replace within
	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	// The old name to search for
	UPROPERTY(meta = (Input))
	FName Old;

	// The new name to replace the old name with
	UPROPERTY(meta = (Input))
	FName New;

	// The resulting item after the replacement
	UPROPERTY(meta = (Output))
	FRigElementKey Result;
};

/**
* Returns true if the two items are equal
*/
USTRUCT(meta=(DisplayName="Item Equals", Keywords="", Deprecated="5.1"))
struct FRigUnit_ItemEquals : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemEquals()
	{
		A = B = FRigElementKey();
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey A;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey B;

	UPROPERTY(meta = (Output))
	bool Result;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Returns true if the two items are not equal
*/
USTRUCT(meta=(DisplayName="Item Not Equals", Keywords="", Deprecated="5.1"))
struct FRigUnit_ItemNotEquals : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemNotEquals()
	{
		A = B = FRigElementKey();
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey A;

	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey B;

	UPROPERTY(meta = (Output))
	bool Result;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Returns true if the two items' types are equal
*/
USTRUCT(meta=(DisplayName="Item Type Equals", Keywords=""))
struct FRigUnit_ItemTypeEquals : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemTypeEquals()
	{
		A = B = FRigElementKey();
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first item to compare
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey A;

	// The second item to compare
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey B;

	// True if the types of items A and B are equal
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
* Returns true if the two items types are not equal
*/
USTRUCT(meta=(DisplayName="Item Type Not Equals", Keywords=""))
struct FRigUnit_ItemTypeNotEquals : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemTypeNotEquals()
	{
		A = B = FRigElementKey();
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first item to compare
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey A;

	// The second item to compare
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey B;

	// True if the types of items A and B are not equal 
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Casts the provided item key to its name
 */
USTRUCT(meta=(DisplayName="To Name", Keywords="", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigUnit_ItemToName : public FRigUnit_ItemBase
{
	GENERATED_BODY()

	FRigUnit_ItemToName()
	{
		Value = FRigElementKey();
		Result = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The item to cast to a name
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Value;

	// The resulting name of the item
	UPROPERTY(meta = (Output))
	FName Result;
};

#undef UE_API
