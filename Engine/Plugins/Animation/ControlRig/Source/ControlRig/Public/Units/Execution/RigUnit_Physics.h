// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigPhysics.h"
#include "RigUnit_DynamicHierarchy.h"
#include "RigUnit_Physics.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(DisplayName="Spawn Physics Solver", Keywords="Construction,Create,New,Simulation", Varying, Deprecated = "5.6"))
struct FRigUnit_HierarchyAddPhysicsSolver : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddPhysicsSolver()
	{
		Name = TEXT("Solver");
		Solver = FRigPhysicsSolverID();
	}

	UPROPERTY(meta = (Input))
	FName Name;

	UPROPERTY(meta = (Output))
	FRigPhysicsSolverID Solver;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Spawn Physics Joint", Keywords="Construction,Create,New,Joint,Skeleton", Varying, Deprecated = "5.6"))
struct FRigUnit_HierarchyAddPhysicsJoint : public FRigUnit_HierarchyAddElement
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddPhysicsJoint()
	{
		Name = TEXT("NewPhysicsJoint");
		Parent = FRigElementKey(NAME_None, ERigElementType::Bone);
		Transform = FTransform::Identity;
		Solver = FRigPhysicsSolverID();
	}

	virtual ERigElementType GetElementTypeToSpawn() const override { return ERigElementType::Physics; }


	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	UPROPERTY(meta = (Input))
	FRigPhysicsSolverID Solver;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
