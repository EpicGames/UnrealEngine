// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_PrepareForExecution.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Event to create / configure elements before any other event
 */
USTRUCT(meta=(DisplayName="Construction Event", Category="Events", NodeColor="0.6, 0, 1", Keywords="Create,Build,Spawn,Setup,Init,Fit"))
struct FRigUnit_PrepareForExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "PrepareForExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("Construction"));
};

/**
 * Event to further configure elements. Runs after the Construction Event
 */
USTRUCT(meta=(DisplayName="Post Construction", Category="Events", NodeColor="0.6, 0, 1", Keywords="Create,Build,Spawn,Setup,Init,Fit"))
struct FRigUnit_PostPrepareForExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "PostPrepareForExecution", meta = (Output))
	FRigVMExecutePin ExecutePin;

	static inline const FLazyName EventName = FLazyName(TEXT("PostConstruction"));
};

#undef UE_API
