// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h" 
#include "RigUnit_RigModules.generated.h"

#define UE_API CONTROLRIG_API

/*
 * Base class for all pure module nodes
 */
USTRUCT(meta = (Abstract, NodeColor="0.262745, 0.8, 0, 0.229412", Category = "Modules", DocumentationPolicy = "Strict"))
struct FRigUnit_RigModulesBase : public FRigUnit
{
	GENERATED_BODY()
};

/*
 * Base class for all mutable module nodes
 */
USTRUCT(meta = (Abstract, NodeColor="0.262745, 0.8, 0, 0.229412", Category = "Modules", DocumentationPolicy = "Strict"))
struct FRigUnit_RigModulesBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

/**
* Returns the resolved item of the connector.
*/
USTRUCT(meta=(DisplayName="Get Connection", Keywords="Connector,GetResolved,Target,Resolve", Varying))
struct FRigUnit_ResolveConnector : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_ResolveConnector()
	{
		Connector = Result = FRigElementKey(NAME_None, ERigElementType::Connector);
		SkipSocket = false;
		bIsConnected = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The key of the connector to resolve
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Connector;

	/*
	 * If the connector is resolved to a socket the node
	 * will return the socket's direct parent (skipping it).
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	bool SkipSocket;

	/*
	 * The resulting item the connector is resolved to
	 */
	UPROPERTY(meta = (Output))
	FRigElementKey Result;

	/*
	 * Returns true if the connector is resolved to a target.
	 */
	UPROPERTY(meta = (Output))
	bool bIsConnected;
};

/**
* Returns the resolved array of items of the connector.
*/
USTRUCT(meta=(DisplayName="Get Array Connection", Keywords="Connector,GetResolved,Target,Resolve", Varying))
struct FRigUnit_ResolveArrayConnector : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_ResolveArrayConnector()
	{
		Connector = FRigElementKey(NAME_None, ERigElementType::Connector);
		SkipSocket = false;
		bIsConnected = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The key of the connector to resolve
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Connector;

	/*
	 * If the connector is resolved to a socket the node
	 * will return the socket's direct parent (skipping it).
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	bool SkipSocket;

	/*
	 * The resulting array of items the connector is resolved to
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Result;

	/*
	 * Returns true if the connector is resolved to a target.
	 */
	UPROPERTY(meta = (Output))
	bool bIsConnected;
};

/**
 * Returns the currently used namespace of the module.
 */
USTRUCT(meta=(DisplayName="Get NameSpace", Keywords="NameSpace", Varying, Deprecated=5.6))
struct FRigUnit_GetCurrentNameSpace : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetCurrentNameSpace()
	{
		NameSpace = FString();;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	/*
	 * The current namespace of the rig module
	 */
	UPROPERTY(meta = (Output))
	FString NameSpace;
};

/**
 * Returns the short name of the given item (without the namespace)
 */
USTRUCT(meta=(DisplayName="Get Short Name", Keywords="NameSpace", Varying, Deprecated=5.6))
struct FRigUnit_GetItemShortName : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetItemShortName()
	{
		Item = FRigElementKey();
		ShortName = NAME_None;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	/*
	 * The key of the item to return the short name for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/*
	 * The short name of the item (without the namespace)
	 */
	UPROPERTY(meta = (Output))
	FName ShortName;
};

/**
 * Returns the namespace of a given item. This may be an empty string if the item doesn't have a namespace.
 */
USTRUCT(meta=(DisplayName="Get Item NameSpace", Keywords="NameSpace", Varying, Deprecated=5.6))
struct FRigUnit_GetItemNameSpace : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetItemNameSpace()
	{
		Item = FRigElementKey();
		HasNameSpace = false;
		NameSpace = FString();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	/*
	 * The key of the item to return the namespace for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/*
	 * True if the item has a valid namespace
	 */
	UPROPERTY(meta = (Output))
	bool HasNameSpace;

	/*
	 * The namespace of the item
	 */
	UPROPERTY(meta = (Output))
	FString NameSpace;
};

/**
 * Returns true if the given item has been created by this module, 
 * which means that the item's namespace is the module namespace.
 */
USTRUCT(meta=(DisplayName="Is In Current NameSpace", Keywords="NameSpace", Varying, Deprecated=5.6))
struct FRigUnit_IsItemInCurrentNameSpace : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_IsItemInCurrentNameSpace()
	{
		Item = FRigElementKey();
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	/*
	 * The key of the item to return the namespace for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/*
	 * True if the item is in this namespace / owned by this module.
	 */
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns all items (based on a filter) in the current namespace. 
 */
USTRUCT(meta=(DisplayName="Get Items In NameSpace", Keywords="NameSpace", Varying, Deprecated=5.6))
struct FRigUnit_GetItemsInNameSpace : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetItemsInNameSpace()
	{
		TypeToSearch = ERigElementType::All;
		Items.Reset();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	/*
	 * The items within the namespace
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

/**
 * Returns the name of the current module instance.
 */
USTRUCT(meta=(DisplayName="Get Module Name", Keywords="NameSpace", Varying))
struct FRigUnit_GetModuleName : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetModuleName()
	{
		Module = FString();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The name of the module
	 */
	UPROPERTY(meta = (Output))
	FString Module;
};

/**
 * Returns the namespace of a given item. This may be an empty string if the item doesn't have a namespace.
 */
USTRUCT(meta=(DisplayName="Get Item Module", Keywords="NameSpace", Varying))
struct FRigUnit_GetItemModuleName : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetItemModuleName()
	{
		Item = FRigElementKey();
		IsPartOfModule = false;
		Module = FString();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The key of the item to return the module for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/*
	 * True if the item is part of a module
	 */
	UPROPERTY(meta = (Output))
	bool IsPartOfModule;

	/*
	 * The name of the module of the given item
	 */
	UPROPERTY(meta = (Output))
	FString Module;
};

/**
 * Returns true if the given item has been created by this module. 
 */
USTRUCT(meta=(DisplayName="Is In Current Module", Keywords="NameSpace", Varying))
struct FRigUnit_IsItemInCurrentModule : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_IsItemInCurrentModule()
	{
		Item = FRigElementKey();
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * The key of the item to check the module for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/*
	 * True if the item is owned by this module.
	 */
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns all items (based on a filter) in the current module. 
 */
USTRUCT(meta=(DisplayName="Get Items In Module", Keywords="NameSpace", Varying))
struct FRigUnit_GetItemsInModule : public FRigUnit_RigModulesBase
{
	GENERATED_BODY()

	FRigUnit_GetItemsInModule()
	{
		TypeToSearch = ERigElementType::All;
		Items.Reset();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The types of elements to search for
	UPROPERTY(meta = (Input))
	ERigElementType TypeToSearch;

	/*
	 * The items within the module
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Items;
};

#undef UE_API
