// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_SequenceExecution.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Allows for a single execution pulse to trigger a series of events in order.
 */
USTRUCT(meta=(DisplayName="Sequence", Category="Execution", TitleColor="1 0 0", NodeColor="1 1 1", Icon="EditorStyle|GraphEditor.Sequence_16x", Deprecated="5.1.0"))
struct FRigUnit_SequenceExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The execution input
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "SequenceExecution", meta = (Input))
	FControlRigExecuteContext ExecuteContext;

	// The execution result A
	UPROPERTY(EditAnywhere, Transient, Category = "SequenceExecution", meta = (Output))
	FControlRigExecuteContext A;

	// The execution result B
	UPROPERTY(EditAnywhere, Transient, Category = "SequenceExecution", meta = (Output))
	FControlRigExecuteContext B;

	// The execution result C
	UPROPERTY(EditAnywhere, Transient, Category = "SequenceExecution", meta = (Output))
	FControlRigExecuteContext C;

	// The execution result D
	UPROPERTY(EditAnywhere, Transient, Category = "SequenceExecution", meta = (Output))
	FControlRigExecuteContext D;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
