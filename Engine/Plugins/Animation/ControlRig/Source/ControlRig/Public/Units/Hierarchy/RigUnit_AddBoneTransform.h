// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_AddBoneTransform.generated.h"


/**
 * Offset Transform is used to perform a change in the hierarchy by setting a single bone's transform.
 */
USTRUCT(meta=(DisplayName="Offset Transform", Category="Hierarchy", DocumentationPolicy="Strict", Keywords="Offset,AddToBoneTransform", Deprecated = "4.25"))
struct FRigUnit_AddBoneTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_AddBoneTransform()
		: Weight(1.f)
		, bPostMultiply(false)
		, bPropagateToChildren(true)
		, CachedBone()
	{}

	RIGVM_METHOD()
	virtual void Execute() override;

	/**
	 * The name of the Bone to set the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The transform value to set for the given Bone.
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * If set to true the transform will be post multiplied, otherwise pre multiplied.
	 * Post multiplying means that the transform is understood as a parent space change,
	 * while pre multiplying means that the transform is understood as a child space change.
	 */
	UPROPERTY(meta = (Input))
	bool bPostMultiply;

	/**
	 * If set to true all of the global transforms of the children 
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FCachedRigElement CachedBone;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};
