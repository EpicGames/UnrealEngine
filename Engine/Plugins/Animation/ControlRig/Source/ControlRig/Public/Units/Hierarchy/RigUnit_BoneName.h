// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_BoneName.generated.h"

#define UE_API CONTROLRIG_API

/**
 * The Item node is used to share a specific item across the graph
 */
USTRUCT(meta=(DisplayName="Item", Category="Hierarchy", NodeColor = "0.4627450108528137 1.0 0.3294120132923126", DocumentationPolicy = "Strict", Constant))
struct FRigUnit_Item : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_Item()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item
	 */
	UPROPERTY(meta = (Input, Output, ExpandByDefault))
	FRigElementKey Item;
};

/**
 * The Item Array node is used to share an array of items across the graph
 */
USTRUCT(meta=(DisplayName="Item Array", Category="Hierarchy", NodeColor = "0.4627450108528137 1.0 0.3294120132923126", DocumentationPolicy = "Strict", Constant, Deprecated = "5.7"))
struct FRigUnit_ItemArray : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ItemArray()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The items
	 */
	UPROPERTY(meta = (Input, Output, ExpandByDefault))
	TArray<FRigElementKey> Items;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * BoneName is used to represent a bone name in the graph
 */
USTRUCT(meta=(DisplayName="Bone Name", Category="Hierarchy", DocumentationPolicy = "Strict", Constant, Deprecated = "4.25"))
struct FRigUnit_BoneName : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_BoneName()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Bone
	 */
	UPROPERTY(meta = (Input, Output))
	FName Bone;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * SpaceName is used to represent a Space name in the graph
 */
USTRUCT(meta=(DisplayName="Space Name", Category="Hierarchy", DocumentationPolicy = "Strict", Deprecated = "4.25"))
struct FRigUnit_SpaceName : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_SpaceName()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Space
	 */
	UPROPERTY(meta = (Input, Output))
	FName Space;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * ControlName is used to represent a Control name in the graph
 */
USTRUCT(meta=(DisplayName="Control Name", Category="Hierarchy", DocumentationPolicy = "Strict", Deprecated = "4.25"))
struct FRigUnit_ControlName : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ControlName()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control
	 */
	UPROPERTY(meta = (Input, Output))
	FName Control;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
