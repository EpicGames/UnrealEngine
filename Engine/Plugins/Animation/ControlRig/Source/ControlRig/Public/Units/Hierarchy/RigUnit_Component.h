// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Units/RigUnit.h"
#include "Units/RigDispatchFactory.h"
#include "RigUnit_Component.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(Abstract, Category="Hierarchy", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigDispatch_ComponentBase : public FRigDispatchFactory
{
	GENERATED_BODY()

	UE_API virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const override;
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FControlRigExecuteContext::StaticStruct(); }
	UE_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;

#if WITH_EDITOR
	UE_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	UE_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#endif

protected:

	virtual bool IsMutable() const { return false; }
	static UE_API bool IsTypeSupported(const TRigVMTypeIndex& InTypeIndex, const FRigVMRegistryHandle& InRegistry);
	static UE_API const TRigVMTypeIndex& GetElementKeyType(FRigVMRegistryHandle& InRegistry);
	static UE_API const TRigVMTypeIndex& GetComponentKeyType(FRigVMRegistryHandle& InRegistry);

	mutable TArray<FRigVMExecuteArgument> ExecuteInfos;

	mutable int32 NameArgIndex = INDEX_NONE;
	mutable int32 ItemArgIndex = INDEX_NONE;
	mutable int32 KeyArgIndex = INDEX_NONE;
	mutable int32 ComponentArgIndex = INDEX_NONE;
	mutable int32 SuccessArgIndex = INDEX_NONE;

	static inline const FLazyName ExecuteArgName = FLazyName(TEXT("Execute"));
	static inline const FLazyName NameArgName = FLazyName(TEXT("Name"));
	static inline const FLazyName ItemArgName = FLazyName(TEXT("Item"));
	static inline const FLazyName KeyArgName = FLazyName(TEXT("Key"));
	static inline const FLazyName ComponentArgName = FLazyName(TEXT("Component"));
	static inline const FLazyName SuccessArgName = FLazyName(TEXT("Success"));

	friend class FControlRigBaseEditor;
};

/**
 * Adds a component under an element in the hierarchy
 */
USTRUCT(meta=(DisplayName="Spawn Component"))
struct FRigDispatch_SpawnComponent : public FRigDispatch_ComponentBase
{
	GENERATED_BODY()

	FRigDispatch_SpawnComponent()
	{
		// this is only needed for engine test
		FactoryScriptStruct = FRigDispatch_SpawnComponent::StaticStruct();
	}

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	UE_API virtual FString GetKeywords() const override;
#endif

protected:

	virtual bool IsMutable() const override { return true; }

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &Execute;
	}
	static UE_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray /* not needed */);
};

/**
 * Retrieves the content of a component by its key from the hierarchy
 */
USTRUCT(meta=(DisplayName="Get Component"))
struct FRigDispatch_GetComponentContent : public FRigDispatch_ComponentBase
{
	GENERATED_BODY()

	FRigDispatch_GetComponentContent()
	{
		// this is only needed for engine test
		FactoryScriptStruct = FRigDispatch_GetComponentContent::StaticStruct();
	}

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;

protected:

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &Execute;
	}
	static UE_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray /* not needed */);
};

/**
 * Set the content of a component
 */
USTRUCT(meta=(DisplayName="Set Component"))
struct FRigDispatch_SetComponentContent : public FRigDispatch_ComponentBase
{
	GENERATED_BODY()

	FRigDispatch_SetComponentContent()
	{
		// this is only needed for engine test
		FactoryScriptStruct = FRigDispatch_SetComponentContent::StaticStruct();
	}

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;

protected:

	virtual bool IsMutable() const override { return true; }

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &Execute;
	}
	static UE_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray /* not needed */);
};

#undef UE_API
