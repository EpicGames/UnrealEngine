// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_ControlChannel.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Get Animation Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(Abstract, Category="Controls", DocumentationPolicy = "Strict", NodeColor="0.462745, 1,0, 0.329412",Varying))
struct FRigUnit_GetAnimationChannelBase : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetAnimationChannelBase()
	{
		Control = Channel = NAME_None;
		bInitial = false;
		CachedChannelKey = FRigElementKey();
		CachedChannelHash = INDEX_NONE;
	}

	static UE_API bool UpdateCache(const URigHierarchy* InHierarchy, const FName& Control, const FName& Channel, FRigElementKey& Key, int32& Hash);

	/**
	 * The name of the Control to retrieve the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The name of the animation channel to retrieve the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "AnimationChannelName"))
	FName Channel;

	/**
	 * If set to true the initial value will be returned / set
	 */
	UPROPERTY(meta = (Input))
	bool bInitial;

	// Used to cache the internally used bone index
	UPROPERTY()
	FRigElementKey CachedChannelKey;

	// A hash combining the control, channel and topology identifiers
	UPROPERTY()
	int32 CachedChannelHash;
};

/**
 * Get Bool Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Bool Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetBoolAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetBoolAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(false)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	bool Value;
};

/**
 * Get Float Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Float Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetFloatAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetFloatAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(0.f)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output, UIMin=0, UIMax=1))
	float Value;
};

/**
 * Get Int Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Int Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetIntAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetIntAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(0)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	int32 Value;
};

/**
 * Get Vector2D Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Vector2D Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetVector2DAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetVector2DAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(FVector2D::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FVector2D Value;
};

/**
 * Get Vector Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Vector Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetVectorAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetVectorAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(FVector::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FVector Value;
};

/**
 * Get Rotator Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Rotator Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetRotatorAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetRotatorAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(FRotator::ZeroRotator)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FRotator Value;
};

/**
 * Get Transform Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Transform Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannel"))
struct FRigUnit_GetTransformAnimationChannel : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_GetTransformAnimationChannel()
		: FRigUnit_GetAnimationChannelBase()
		, Value(FTransform::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FTransform Value;
};

/**
 * Set Animation Channel is used to change a control's animation channel value
 */
USTRUCT(meta = (Abstract))
struct FRigUnit_SetAnimationChannelBase : public FRigUnit_GetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetAnimationChannelBase()
		:FRigUnit_GetAnimationChannelBase()
	{
	}

	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;
};

/**
 * Set Bool Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Bool Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetBoolAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetBoolAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(false)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	bool Value;
};

/**
 * Set Float Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Float Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetFloatAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetFloatAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(0.f)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input, UIMin=0, UIMax=1))
	float Value;
};

/**
 * Set Int Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Int Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetIntAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetIntAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(0)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	int32 Value;
};

/**
 * Set Vector2D Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Vector2D Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetVector2DAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetVector2DAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(FVector2D::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FVector2D Value;
};

/**
 * Set Vector Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Vector Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetVectorAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetVectorAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(FVector::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FVector Value;
};

/**
 * Set Rotator Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Rotator Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetRotatorAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetRotatorAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(FRotator::ZeroRotator)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FRotator Value;
};

/**
 * Set Transform Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Transform Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannel"))
struct FRigUnit_SetTransformAnimationChannel : public FRigUnit_SetAnimationChannelBase
{
	GENERATED_BODY()

	FRigUnit_SetTransformAnimationChannel()
		: FRigUnit_SetAnimationChannelBase()
		, Value(FTransform::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FTransform Value;
};

#undef UE_API
