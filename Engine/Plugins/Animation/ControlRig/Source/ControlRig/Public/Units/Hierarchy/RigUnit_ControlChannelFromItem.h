// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_ControlChannelFromItem.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Get Animation Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(Abstract, Category="Controls", DocumentationPolicy = "Strict", NodeColor="0.462745, 1,0, 0.329412",Varying))
struct FRigUnit_GetAnimationChannelFromItemBase : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetAnimationChannelFromItemBase()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Control);
		bInitial = false;
	}

	/**
	 * The item representing the channel
	 */
	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	/**
	 * If set to true the initial value will be returned
	 */
	UPROPERTY(meta = (Input))
	bool bInitial;
};

/**
 * Get Bool Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Bool Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetBoolAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetBoolAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(false)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	bool Value;
};

/**
 * Get Float Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Float Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetFloatAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetFloatAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(0.f)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output, UIMin=0, UIMax=1))
	float Value;
};

/**
 * Get Int Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Int Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetIntAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetIntAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(0)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	int32 Value;
};

/**
 * Get Vector2D Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Vector2D Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetVector2DAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetVector2DAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(FVector2D::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FVector2D Value;
};

/**
 * Get Vector Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Vector Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetVectorAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetVectorAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(FVector::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FVector Value;
};

/**
 * Get Rotator Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Rotator Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetRotatorAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetRotatorAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(FRotator::ZeroRotator)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FRotator Value;
};

/**
 * Get Transform Channel is used to retrieve a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Get Transform Channel", Keywords="Animation,Channel", TemplateName="GetAnimationChannelFromItem"))
struct FRigUnit_GetTransformAnimationChannelFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_GetTransformAnimationChannelFromItem()
		: FRigUnit_GetAnimationChannelFromItemBase()
		, Value(FTransform::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The current value of the animation channel
	UPROPERTY(meta=(Output))
	FTransform Value;
};

/**
 * Set Animation Channel is used to change a control's animation channel value
 */
USTRUCT(meta = (Abstract))
struct FRigUnit_SetAnimationChannelBaseFromItem : public FRigUnit_GetAnimationChannelFromItemBase
{
	GENERATED_BODY()

	FRigUnit_SetAnimationChannelBaseFromItem()
		:FRigUnit_GetAnimationChannelFromItemBase()
	{
	}

	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;
};

/**
 * Set Bool Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Bool Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetBoolAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetBoolAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(false)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	bool Value;
};

/**
 * Set Float Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Float Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetFloatAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetFloatAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(0.f)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input, UIMin=0, UIMax=1))
	float Value;
};

/**
 * Set Int Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Int Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetIntAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetIntAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(0)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	int32 Value;
};

/**
 * Set Vector2D Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Vector2D Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetVector2DAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetVector2DAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(FVector2D::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FVector2D Value;
};

/**
 * Set Vector Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Vector Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetVectorAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetVectorAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(FVector::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FVector Value;
};

/**
 * Set Rotator Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Rotator Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetRotatorAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetRotatorAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(FRotator::ZeroRotator)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FRotator Value;
};

/**
 * Set Transform Channel is used to set a control's animation channel value
 */
USTRUCT(meta=(DisplayName="Set Transform Channel", Keywords="Animation,Channel", TemplateName="SetAnimationChannelFromItem"))
struct FRigUnit_SetTransformAnimationChannelFromItem : public FRigUnit_SetAnimationChannelBaseFromItem
{
	GENERATED_BODY()

	FRigUnit_SetTransformAnimationChannelFromItem()
		: FRigUnit_SetAnimationChannelBaseFromItem()
		, Value(FTransform::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The new value of the animation channel
	UPROPERTY(meta=(Input))
	FTransform Value;
};

#undef UE_API
