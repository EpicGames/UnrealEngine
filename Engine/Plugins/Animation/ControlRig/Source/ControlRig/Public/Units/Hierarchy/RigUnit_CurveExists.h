// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_CurveExists.generated.h"

#define UE_API CONTROLRIG_API

/**
 * CurveExists is used to check whether a curve exists or not.
 */
USTRUCT(meta=(DisplayName="Curve Exists", Category="Curve", Keywords="CurveExists,bool", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_CurveExists : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_CurveExists()
		: Curve(NAME_None)
		, Exists(false)
		, CachedCurveIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Curve to check existence for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "CurveName"))
	FName Curve;

	/** Boolean indicating whether the named curve exists or not. 
	 *  Does not indicate whether the curve's value is valid or not.
	 */
	UPROPERTY(meta=(Output)) 
	bool Exists;

private:
	// Used to cache the internally used Curve index
	UPROPERTY()
	FCachedRigElement CachedCurveIndex;
};

#undef UE_API
