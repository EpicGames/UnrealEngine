// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_FindClosestItem.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Returns the item with the closest distance to the provided point in global space. 
 */
USTRUCT(meta = (DisplayName = "Find Closest Item", Category = "Hierarchy", Keywords = "Find,Closest,Item,Transform,Bone,Joint", NodeColor = "0.3 0.1 0.1", DocumentationPolicy="Strict"))
struct FRigUnit_FindClosestItem : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_FindClosestItem()
	{
		Point = FVector::ZeroVector;
	}

	RIGVM_METHOD()
		UE_API virtual void Execute() override;

	// The list of items to test
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	// The point in global space to test against
	UPROPERTY(meta = (Input))
	FVector Point;

	// The item closest to the provided point 
	UPROPERTY(meta = (Output, ExpandByDefault))
	FRigElementKey Item;

	UPROPERTY(transient)
	TArray<FCachedRigElement> CachedItems;

};

#undef UE_API
