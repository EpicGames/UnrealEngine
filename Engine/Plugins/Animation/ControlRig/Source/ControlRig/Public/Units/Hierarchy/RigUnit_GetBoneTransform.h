// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetBoneTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetBoneTransform is used to retrieve a single transform from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Transform", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="GetBoneTransform", Varying, Deprecated = "4.25"))
struct FRigUnit_GetBoneTransform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetBoneTransform()
		: Space(ERigVMTransformSpace::GlobalSpace)
		, CachedBone()
		, bFirstUpdate(true)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Bone to retrieve the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * Defines if the bone's transform should be retrieved
	 * in local or global space.
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// The current transform of the given bone - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FCachedRigElement CachedBone;

	// Used to force first update to return initial pose
	UPROPERTY(transient)
	bool bFirstUpdate;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
