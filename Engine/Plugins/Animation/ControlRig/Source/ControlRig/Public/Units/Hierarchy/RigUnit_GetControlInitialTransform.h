// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetControlInitialTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetControlTransform is used to retrieve a single transform from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Initial Transform", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlInitialTransform", Deprecated = "4.25"))
struct FRigUnit_GetControlInitialTransform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlInitialTransform()
		: Space(ERigVMTransformSpace::LocalSpace)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Control;

	/**
	 * Defines if the Control's transform should be retrieved
	 * in local or global space.
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// The current transform of the given bone - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
