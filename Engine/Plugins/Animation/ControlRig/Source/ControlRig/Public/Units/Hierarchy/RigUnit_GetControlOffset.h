// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetControlOffset.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetControlOffset returns the offset transform of a given control.
 */
USTRUCT(meta=(DisplayName="Get Control Offset", Category="Controls", DocumentationPolicy="Strict", Keywords = "GetControlOffset,Initial,InitialTransform,GetInitialTransform,GetInitialControlTransform", NodeColor="0, 0.364706, 1.0"))
struct FRigUnit_GetControlOffset : public FRigUnit
{

	GENERATED_BODY()

	FRigUnit_GetControlOffset()
		: Control(NAME_None)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, OffsetTransform(FTransform::Identity)
		, CachedIndex()
	{}

	RIGVM_METHOD()
		UE_API virtual void Execute() override;

	/**
	 * The name of the Control to get the offset transform for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * Defines if the transform should be retrieved in local or global space
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// The current transform of the given item - or identity in case it wasn't found.
	UPROPERTY(meta = (Output))
	FTransform OffsetTransform;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

#undef UE_API
