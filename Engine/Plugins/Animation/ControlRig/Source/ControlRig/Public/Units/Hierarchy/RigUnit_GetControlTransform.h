// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetControlTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetControlBool is used to retrieve a single Bool from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Bool", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlBool", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412",Varying))
struct FRigUnit_GetControlBool : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlBool()
		: BoolValue(false)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the bool for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	// The current bool of the given control - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	bool BoolValue;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * GetControlFloat is used to retrieve a single Float from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Float", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlFloat", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlFloat : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlFloat()
		: FloatValue(0.f)
		, Minimum(0.f)
		, Maximum(0.f)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the Float for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	// The current value of the control.
	UPROPERTY(meta=(Output, UIMin = "0.0", UIMax = "1.0"))
	float FloatValue;

	// The minimum value of the control.
	UPROPERTY(meta = (Output, UIMin = "0.0", UIMax = "1.0"))
	float Minimum;

	// The maximum value of the control.
	UPROPERTY(meta = (Output, UIMin = "0.0", UIMax = "1.0"))
	float Maximum;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};
/**
 * GetControlInteger is used to retrieve a single Integer from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Integer", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlInteger,GetControlEnum", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlInteger : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlInteger()
		: IntegerValue(0)
		, Minimum(0)
		, Maximum(0)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the Integer for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	// The current value of the control.
	UPROPERTY(meta = (Output))
	int32 IntegerValue;

	// The minimum value of the control.
	UPROPERTY(meta = (Output))
	int32 Minimum;

	// The maximum value of the control.
	UPROPERTY(meta = (Output))
	int32 Maximum;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * GetControlVector2D is used to retrieve a single Vector2D from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Vector2D", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlVector2D", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlVector2D : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlVector2D()
		: Vector(FVector2D::ZeroVector)
		, Minimum(FVector2D::ZeroVector)
		, Maximum(FVector2D::ZeroVector)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the Vector2D for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	// The current value of the control.
	UPROPERTY(meta=(Output))
	FVector2D Vector;

	// The minimum value of the control.
	UPROPERTY(meta = (Output))
	FVector2D Minimum;

	// The maximum value of the control.
	UPROPERTY(meta = (Output))
	FVector2D Maximum;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * GetControlVector is used to retrieve a single Vector from a hierarchy, can be used for Controls of type "Position" or "Scale".
 */
USTRUCT(meta=(DisplayName="Get Control Vector", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlVector, GetControlPosition, GetControlScale", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlVector : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlVector()
		: Space(ERigVMTransformSpace::GlobalSpace)
		, Vector(FVector::ZeroVector)
		, Minimum(FVector::ZeroVector)
		, Maximum(FVector::ZeroVector)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the Vector for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * Defines if the Control's Vector should be retrieved
	 * in local or global space.
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// The current value of the control.
	UPROPERTY(meta=(Output))
	FVector Vector;

	// The minimum value of the control.
	UPROPERTY(meta = (Output))
	FVector Minimum;

	// The maximum value of the control.
	UPROPERTY(meta = (Output))
	FVector Maximum;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * GetControlRotator is used to retrieve a single Rotator from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Rotator", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlRotator", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlRotator : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlRotator()
		: Space(ERigVMTransformSpace::GlobalSpace)
		, Rotator(FRotator::ZeroRotator)
		, Minimum(FRotator::ZeroRotator)
		, Maximum(FRotator::ZeroRotator)
		, CachedControlIndex(FCachedRigElement())
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the Rotator for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * Defines if the Control's Rotator should be retrieved
	 * in local or global space.
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// The current value of the control.
	UPROPERTY(meta=(Output))
	FRotator Rotator;

	// The minimum value of the control.
	UPROPERTY(meta = (Output))
	FRotator Minimum;

	// The maximum value of the control.
	UPROPERTY(meta = (Output))
	FRotator Maximum;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * This node provides access to the preferred euler angles provided by Sequencer.
 */
USTRUCT(meta=(DisplayName="Get Euler Angles From Animation", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlRotator,PreferredEulerAngles", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlPreferredEulerAngles : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlPreferredEulerAngles()
		: Angles(FVector::ZeroVector)
		, RotationOrder(FRigPreferredEulerAngles::DefaultRotationOrder)
		, CachedControlIndex(FCachedRigElement())
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the rotation for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	// The euler angles provided by sequencer
	UPROPERTY(meta=(Output))
	FVector Angles;

	// The rotation order used to represent the rotation
	UPROPERTY(meta = (Output))
	EEulerRotationOrder RotationOrder;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * GetControlTransform is used to retrieve a single transform from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Control Transform", Category="Controls", DocumentationPolicy = "Strict", Keywords="GetControlTransform", TemplateName="GetControlValue", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetControlTransform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlTransform()
		: Space(ERigVMTransformSpace::GlobalSpace)
		, Transform(FTransform::Identity)
		, Minimum(FTransform::Identity)
		, Maximum(FTransform::Identity)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to retrieve the transform for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * Defines if the Control's transform should be retrieved
	 * in local or global space.
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// The current value of the control.
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// The minimum value of the control.
	UPROPERTY(meta = (Output))
	FTransform Minimum;

	// The maximum value of the control.
	UPROPERTY(meta = (Output))
	FTransform Maximum;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

#undef UE_API
