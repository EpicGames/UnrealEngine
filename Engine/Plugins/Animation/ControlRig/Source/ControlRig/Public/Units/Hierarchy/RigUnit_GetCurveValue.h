// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetCurveValue.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetCurveValue is used to retrieve a single float from a Curve
 */
USTRUCT(meta=(DisplayName="Get Curve Value", Category="Curve", Keywords="GetCurveValue,float", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetCurveValue : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetCurveValue()
		: Curve(NAME_None)
		, Valid(true)
		, Value(0.f)
		, CachedCurveIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Curve to retrieve the transform for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "CurveName"))
	FName Curve;

	// True if the retrieved curve value is valid
	UPROPERTY(meta=(Output)) 
	bool Valid;

	// The current value of the given Curve - or identity in case it wasn't found.
	UPROPERTY(meta=(Output, UIMin = 0.f, UIMax = 1.f))
	float Value;

private:
	// Used to cache the internally used Curve index
	UPROPERTY()
	FCachedRigElement CachedCurveIndex;
};

#undef UE_API
