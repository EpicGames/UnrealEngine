// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetRelativeTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetRelativeTransform is used to retrieve a single transform from a hierarchy in the space of another transform
 */
USTRUCT(meta=(DisplayName="Get Relative Transform", Category="Transforms", DocumentationPolicy = "Strict", Keywords = "Offset,Local", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetRelativeTransformForItem : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetRelativeTransformForItem()
		: Child(NAME_None, ERigElementType::Bone)
		, bChildInitial(false)
		, Parent(NAME_None, ERigElementType::Bone)
		, bParentInitial(false)
		, RelativeTransform(FTransform::Identity)
		, CachedChild()
		, CachedParent()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The child item to retrieve the transform for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/**
	 * Defines if the child's transform should be retrieved as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bChildInitial;

	/**
	 * The parent item to use.
	 * The child transform will be retrieve in the space of the parent.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	/**
	 * Defines if the parent's transform should be retrieved as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bParentInitial;

	// The transform of the given child item relative to the provided parent
	UPROPERTY(meta=(Output))
	FTransform RelativeTransform;

	// Used to cache the child internally
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the parent internally
	UPROPERTY()
	FCachedRigElement CachedParent;
};

#undef UE_API
