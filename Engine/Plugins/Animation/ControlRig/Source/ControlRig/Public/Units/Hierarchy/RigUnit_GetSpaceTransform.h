// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetSpaceTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetSpaceTransform is used to retrieve a single transform from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Space Transform", Category="Spaces", DocumentationPolicy = "Strict", Keywords="GetSpaceTransform", Varying, Deprecated = "4.25"))
struct FRigUnit_GetSpaceTransform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetSpaceTransform()
		: SpaceType(ERigVMTransformSpace::GlobalSpace)
		, CachedSpaceIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Space to retrieve the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Space;

	/**
	 * Defines if the Space's transform should be retrieved
	 * in local or global space.
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace SpaceType;

	// The current transform of the given bone - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedSpaceIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
