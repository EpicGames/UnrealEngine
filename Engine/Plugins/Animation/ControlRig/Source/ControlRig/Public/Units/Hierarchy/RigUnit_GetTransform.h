// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_GetTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetTransform is used to retrieve a single transform from a hierarchy.
 */
USTRUCT(meta=(DisplayName="Get Transform", Category="Transforms", DocumentationPolicy = "Strict", Keywords="GetBoneTransform,GetControlTransform,GetInitialTransform,GetSpaceTransform,GetTransform", NodeColor="0.462745, 1,0, 0.329412",Varying))
struct FRigUnit_GetTransform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetTransform()
		: Item(NAME_None, ERigElementType::Bone)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Transform(FTransform::Identity)
		, CachedIndex()
	{}

	UE_API virtual FString GetUnitLabel() const override;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to retrieve the transform for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * Defines if the transform should be retrieved in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be retrieved as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The current transform of the given item - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
* GetTransformArray is used to retrieve an array of transforms from the hierarchy.
*/
USTRUCT(meta=(DisplayName="Get Transform Array", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="GetBoneTransform,GetControlTransform,GetInitialTransform,GetSpaceTransform,GetTransform", NodeColor="0.462745, 1,0, 0.329412",Varying, Deprecated = "5.0"))
struct FRigUnit_GetTransformArray : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetTransformArray()
		: Items()
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Transforms()
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	* The items to retrieve the transforms for
	*/
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/**
	* Defines if the transforms should be retrieved in local or global space
	*/ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	* Defines if the transforms should be retrieved as current (false) or initial (true).
	* Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	*/ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The current transform of the given item - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	TArray<FTransform> Transforms;

	// Used to cache the internally used index
	UPROPERTY()
	TArray<FCachedRigElement> CachedIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* GetTransformArray is used to retrieve an array of transforms from the hierarchy.
*/
USTRUCT(meta=(DisplayName="Get Transform Array", Category="Transforms", DocumentationPolicy = "Strict", Keywords="GetBoneTransform,GetControlTransform,GetInitialTransform,GetSpaceTransform,GetTransform", NodeColor="0.462745, 1,0, 0.329412",Varying))
struct FRigUnit_GetTransformItemArray : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetTransformItemArray()
		: Items()
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Transforms()
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	* The items to retrieve the transforms for
	*/
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/**
	* Defines if the transforms should be retrieved in local or global space
	*/ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	* Defines if the transforms should be retrieved as current (false) or initial (true).
	* Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	*/ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The current transform of the given item - or identity in case it wasn't found.
	UPROPERTY(meta=(Output))
	TArray<FTransform> Transforms;

	// Used to cache the internally used index
	UPROPERTY()
	TArray<FCachedRigElement> CachedIndex;
};

#undef UE_API
