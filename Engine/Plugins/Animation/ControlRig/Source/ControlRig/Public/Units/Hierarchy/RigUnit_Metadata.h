// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigDispatchFactory.h"
#include "Units/RigUnit.h"
#include "ModularRig.h"
#include "Rigs/RigDependencyRecords.h"
#include "RigUnit_Metadata.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(meta=(Abstract, Category="Hierarchy", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigDispatch_MetadataBase : public FRigDispatchFactory
{
	GENERATED_BODY()

	FRigDispatch_MetadataBase()
	{
		FactoryScriptStruct = StaticStruct();
	}

	template<typename T>
	static bool Equals(const T& A, const T& B)
	{
		return A.Equals(B);
	}

	template<typename T>
	static bool ArrayEquals(const TArray<T>& A, const TArray<T>& B)
	{
		if(A.Num() != B.Num())
		{
			return false;
		}
		for(int32 Index = 0; Index < A.Num(); Index++)
		{
			if(!Equals<T>(A[Index], B[Index]))
			{
				return false;
			}
		}
		return true;
	}

#if WITH_EDITOR
	UE_API virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const override;;
	virtual FString GetNodeTitlePrefix() const { return FString(); }
#endif
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSetMetadata() const { return false; }

#if WITH_EDITOR
	UE_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	UE_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	UE_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	UE_API const TArray<TRigVMTypeIndex>& GetValueTypes(const FRigVMRegistryHandle& InRegistry) const;

	mutable int32 ItemArgIndex = INDEX_NONE;
	mutable int32 NameArgIndex = INDEX_NONE;
	mutable int32 NameSpaceArgIndex = INDEX_NONE;
	mutable int32 CacheArgIndex = INDEX_NONE;
	mutable int32 DefaultArgIndex = INDEX_NONE;
	mutable int32 ValueArgIndex = INDEX_NONE;
	mutable int32 FoundArgIndex = INDEX_NONE;
	mutable int32 SuccessArgIndex = INDEX_NONE;
	mutable TArray<TRigVMTypeIndex> ValueTypes;

	static inline const FLazyName ItemArgName = FLazyName(TEXT("Item"));
	static inline const FLazyName NameArgName = FLazyName(TEXT("Name"));
	static inline const FLazyName NameSpaceArgName = FLazyName(TEXT("NameSpace"));
	static inline const FLazyName CacheArgName = FLazyName(TEXT("Cache"));
	static inline const FLazyName DefaultArgName = FLazyName(TEXT("Default"));
	static inline const FLazyName ValueArgName = FLazyName(TEXT("Value"));
	static inline const FLazyName FoundArgName = FLazyName(TEXT("Found"));
	static inline const FLazyName SuccessArgName = FLazyName(TEXT("Success"));
};

template<>
inline bool FRigDispatch_MetadataBase::Equals(const bool& A, const bool& B)
{
	return A == B;
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const int32& A, const int32& B)
{
	return A == B;
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const float& A, const float& B)
{
	return FMath::IsNearlyEqual(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const double& A, const double& B)
{
	return FMath::IsNearlyEqual(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const FName& A, const FName& B)
{
	return A.IsEqual(B, ENameCase::CaseSensitive);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const FRigElementKey& A, const FRigElementKey& B)
{
	return A == B;
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<bool>& A, const TArray<bool>& B)
{
	return ArrayEquals<bool>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<int32>& A, const TArray<int32>& B)
{
	return ArrayEquals<int32>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<float>& A, const TArray<float>& B)
{
	return ArrayEquals<float>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<double>& A, const TArray<double>& B)
{
	return ArrayEquals<double>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FName>& A, const TArray<FName>& B)
{
	return ArrayEquals<FName>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FRigElementKey>& A, const TArray<FRigElementKey>& B)
{
	return ArrayEquals<FRigElementKey>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FVector>& A, const TArray<FVector>& B)
{
	return ArrayEquals<FVector>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FRotator>& A, const TArray<FRotator>& B)
{
	return ArrayEquals<FRotator>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FQuat>& A, const TArray<FQuat>& B)
{
	return ArrayEquals<FQuat>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FTransform>& A, const TArray<FTransform>& B)
{
	return ArrayEquals<FTransform>(A, B);
}

template<>
inline bool FRigDispatch_MetadataBase::Equals(const TArray<FLinearColor>& A, const TArray<FLinearColor>& B)
{
	return ArrayEquals<FLinearColor>(A, B);
}

/*
 * Gets some metadata for the provided item
 */
USTRUCT(meta=(DisplayName="Get Metadata"))
struct FRigDispatch_GetMetadata : public FRigDispatch_MetadataBase
{
	GENERATED_BODY()

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;

protected:

	static UE_API FRigBaseMetadata* FindMetadata(const FRigVMExtendedExecuteContext& InContext, const FRigElementKey& InKey, const FName& InName, ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace, FCachedRigElement& Cache);
	UE_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	template<typename ValueType>
	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[ItemArgIndex].IsType<FRigElementKey>(), ItemArgName) &&
			CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[NameSpaceArgIndex].IsType<ERigMetaDataNameSpace>(), NameSpaceArgName) &&
			CheckArgumentType(Handles[CacheArgIndex].IsType<FCachedRigElement>(true), CacheArgName) &&
			CheckArgumentType(Handles[DefaultArgIndex].IsType<ValueType>(), DefaultArgName) &&
			CheckArgumentType(Handles[ValueArgIndex].IsType<ValueType>(), ValueArgName) &&
			CheckArgumentType(Handles[FoundArgIndex].IsType<bool>(), FoundArgName);
	}
#endif

	template<typename ValueType, typename MetadataType, ERigMetadataType EnumValue>
	static void GetMetadataDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		const FRigDispatch_GetMetadata* Factory = static_cast<const FRigDispatch_GetMetadata*>(InContext.Factory);

#if WITH_EDITOR
		if(!Factory->CheckArgumentTypes<ValueType>(Handles))
		{
			return;
		}
#endif

		// unpack the memory
		const FRigElementKey& Item = *reinterpret_cast<const FRigElementKey*>(Handles[Factory->ItemArgIndex].GetInputData());
		const FName& Name = *reinterpret_cast<const FName*>(Handles[Factory->NameArgIndex].GetInputData());
		const ERigMetaDataNameSpace NameSpace = *reinterpret_cast<const ERigMetaDataNameSpace*>(Handles[Factory->NameSpaceArgIndex].GetInputData());
		FCachedRigElement& Cache = *reinterpret_cast<FCachedRigElement*>(Handles[Factory->CacheArgIndex].GetPrivateData(InContext.GetSlice().GetIndex()));
		const ValueType& Default = *reinterpret_cast<const ValueType*>(Handles[Factory->DefaultArgIndex].GetInputData());
		ValueType& Value = *reinterpret_cast<ValueType*>(Handles[Factory->ValueArgIndex].GetOutputData());
		bool& Found = *reinterpret_cast<bool*>(Handles[Factory->FoundArgIndex].GetOutputData());

		// extract the metadata
		if (const MetadataType* Md = Cast<MetadataType>(FindMetadata(InContext, Item, Name, EnumValue, NameSpace, Cache)))
		{
			Value = Md->GetValue();
			Found = true;

#if WITH_EDITOR
			const FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
			if (ExecuteContext.Hierarchy->bRecordInstructionsAtRuntime)
			{
				const FName VMName = ExecuteContext.GetRigModuleInstance() ? ExecuteContext.GetRigModuleInstance()->Name : NAME_None;
				const FInstructionRecord Record(ExecuteContext.GetInstructionIndex(), InContext.GetSlice().GetIndex(), Cache.GetIndex(), VMName, Md->GetName());
				ExecuteContext.AddReadInstructionRecord(Record);
			}
#endif			
		}
		else
		{
			Value = Default;
			Found = false;
		}
	}
};

/*
 * Sets some metadata for the provided item
 */
USTRUCT(meta=(DisplayName="Set Metadata"))
struct FRigDispatch_SetMetadata : public FRigDispatch_MetadataBase
{
	GENERATED_BODY()

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;
	virtual bool IsSetMetadata() const override { return true; }

protected:

	static UE_API FRigBaseMetadata* FindOrAddMetadata(const FControlRigExecuteContext& InContext, const FRigElementKey& InKey, const FName& InName, ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace, FCachedRigElement& Cache);
	UE_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	template<typename ValueType>
	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[ItemArgIndex].IsType<FRigElementKey>(), ItemArgName) &&
			CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[NameSpaceArgIndex].IsType<ERigMetaDataNameSpace>(), NameSpaceArgName) &&
			CheckArgumentType(Handles[CacheArgIndex].IsType<FCachedRigElement>(true), CacheArgName) &&
			CheckArgumentType(Handles[ValueArgIndex].IsType<ValueType>(), ValueArgName) &&
			CheckArgumentType(Handles[SuccessArgIndex].IsType<bool>(), SuccessArgName);
	}
#endif
	
	template<typename ValueType, typename MetadataType, ERigMetadataType EnumValue>
	static void SetMetadataDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		const FRigDispatch_SetMetadata* Factory = static_cast<const FRigDispatch_SetMetadata*>(InContext.Factory);

#if WITH_EDITOR
		if(!Factory->CheckArgumentTypes<ValueType>(Handles))
		{
			return;
		}
#endif

		// unpack the memory
		FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
		const FRigElementKey& Item = *(const FRigElementKey*)Handles[Factory->ItemArgIndex].GetInputData();
		const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
		const ERigMetaDataNameSpace NameSpace = *(const ERigMetaDataNameSpace*)Handles[Factory->NameSpaceArgIndex].GetInputData();
		FCachedRigElement& Cache = *(FCachedRigElement*)Handles[Factory->CacheArgIndex].GetPrivateData(InContext.GetSlice().GetIndex());
		ValueType& Value = *(ValueType*)Handles[Factory->ValueArgIndex].GetOutputData();
		bool& Success = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();

		// extract the metadata
		if (MetadataType* Md = Cast<MetadataType>(FindOrAddMetadata(ExecuteContext, Item, Name, EnumValue, NameSpace, Cache)))
		{
#if WITH_EDITOR
			if (ExecuteContext.Hierarchy->bRecordInstructionsAtRuntime)
			{
				const FName VMName = ExecuteContext.GetRigModuleInstance() ? ExecuteContext.GetRigModuleInstance()->Name : NAME_None;
				const FInstructionRecord Record(ExecuteContext.GetInstructionIndex(), InContext.GetSlice().GetIndex(), Cache.GetIndex(), VMName, Md->GetName());
				ExecuteContext.AddWrittenInstructionRecord(Record);
			}
#endif

			if(!Equals<ValueType>(Md->GetValue(), Value))
			{
				Md->GetValue() = Value;
				ExecuteContext.Hierarchy->MetadataVersion++;
			}
			Success = true;
		}
		else
		{
			Success = false;
		}
	}
};

/**
 * Removes an existing metadata field from an item
 */
USTRUCT(meta=(DisplayName="Remove Metadata", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="DeleteMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_RemoveMetadata : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_RemoveMetadata()
		: Item(NAME_None, ERigElementType::Bone)
		, Name(NAME_None)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Removed(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to remove the metadata from 
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the metadata to remove
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataName"))
	FName Name;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// True if the metadata has been removed
	UPROPERTY(meta=(Output))
	bool Removed;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Removes an existing metadata field from an item
 */
USTRUCT(meta=(DisplayName="Remove All Metadata", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="DeleteMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_RemoveAllMetadata : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_RemoveAllMetadata()
		: Item(NAME_None, ERigElementType::Bone)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Removed(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to remove the metadata from 
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// True if any metadata has been removed
	UPROPERTY(meta=(Output))
	bool Removed;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Returns true if a given item in the hierarchy has a specific set of metadata
 */
USTRUCT(meta=(DisplayName="Has Metadata", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,SupportsMetadata,FindMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_HasMetadata : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_HasMetadata()
		: Item(NAME_None, ERigElementType::Bone)
		, Name(NAME_None)
		, Type(ERigMetadataType::Float)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Found(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to check the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the metadata to check
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataName"))
	FName Name;

	/**
	 * The type of metadata to check for
	 */ 
	UPROPERTY(meta = (Input))
	ERigMetadataType Type;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// True if the item has the metadata
	UPROPERTY(meta=(Output))
	bool Found;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Returns all items containing a specific set of metadata
 */
USTRUCT(meta=(DisplayName="Find Items with Metadata", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,SupportsMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_FindItemsWithMetadata : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_FindItemsWithMetadata()
		: Name(NAME_None)
		, Type(ERigMetadataType::Float)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Items()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the metadata to find
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataName"))
	FName Name;

	/**
	 * The type of metadata to find
	 */ 
	UPROPERTY(meta = (Input))
	ERigMetadataType Type;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// The items containing the metadata
	UPROPERTY(meta=(Output))
	TArray<FRigElementKey> Items;
};

/**
 * Returns the metadata tags on an item
 */
USTRUCT(meta=(DisplayName="Get Tags", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,Tagging,FindTag", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_GetMetadataTags : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetMetadataTags()
		: Item(NAME_None, ERigElementType::Bone)
		, Tags()
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to check the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the tag to check
	 */ 
	UPROPERTY(meta = (Output))
	TArray<FName> Tags;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Sets a single tag on an item 
 */
USTRUCT(meta=(DisplayName="Add Tag", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,Tagging,FindTag,SetTag", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_SetMetadataTag : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMetadataTag()
		: Item(NAME_None, ERigElementType::Bone)
		, Tag(NAME_None)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the tag to set
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	FName Tag;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Sets multiple tags on an item 
 */
USTRUCT(meta=(DisplayName="Add Multiple Tags", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="AddTags,MetadataExists,HasKey,Tagging,FindTag,SetTags", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_SetMetadataTagArray : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMetadataTagArray()
		: Item(NAME_None, ERigElementType::Bone)
		, Tags()
		, NameSpace(ERigMetaDataNameSpace::Self)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The tags to set for the item
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	TArray<FName> Tags;
	
	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Removes a tag from an item 
 */
USTRUCT(meta=(DisplayName="Remove Tag", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="DeleteTag", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_RemoveMetadataTag : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_RemoveMetadataTag()
		: Item(NAME_None, ERigElementType::Bone)
		, Tag(NAME_None)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Removed(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the tag to set
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	FName Tag;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;
	
	/**
	 * Returns true if the removal was successful
	 */ 
	UPROPERTY(meta = (Output))
	bool Removed;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Returns true if a given item in the hierarchy has a specific tag stored in the metadata
 */
USTRUCT(meta=(DisplayName="Has Tag", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,Tagging,FindTag", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_HasMetadataTag : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_HasMetadataTag()
		: Item(NAME_None, ERigElementType::Bone)
		, Tag(NAME_None)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Found(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to check the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the tag to check
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	FName Tag;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;
	
	// True if the item has the metadata
	UPROPERTY(meta=(Output))
	bool Found;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Returns true if a given item in the hierarchy has all of the provided tags
 */
USTRUCT(meta=(DisplayName="Has Multiple Tags", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,Tagging,FindTag", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_HasMetadataTagArray : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_HasMetadataTagArray()
		: Item(NAME_None, ERigElementType::Bone)
		, Tags()
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Found(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to check the metadata for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The name of the tag to check
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	TArray<FName> Tags;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;
	
	// True if the item has the metadata
	UPROPERTY(meta=(Output))
	bool Found;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * Returns all items with a specific tag
 */
USTRUCT(meta=(DisplayName="Find Items with Tag", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,SupportsMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_FindItemsWithMetadataTag : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_FindItemsWithMetadataTag()
		: Tag(NAME_None)
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Items()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the tag to find
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	FName Tag;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// The items containing the metadata
	UPROPERTY(meta=(Output))
	TArray<FRigElementKey> Items;
};

/**
 * Returns all items with a specific set of tags
 */
USTRUCT(meta=(DisplayName="Find Items with multiple Tags", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,SupportsMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_FindItemsWithMetadataTagArray : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_FindItemsWithMetadataTagArray()
		: Tags()
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Items()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The tags to find
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	TArray<FName> Tags;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	// The items containing the metadata
	UPROPERTY(meta=(Output))
	TArray<FRigElementKey> Items;
};

/**
 * Filters an item array by a list of tags
 */
USTRUCT(meta=(DisplayName="Filter Items by Tags", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="MetadataExists,HasKey,SupportsMetadata", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_FilterItemsByMetadataTags : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_FilterItemsByMetadataTags()
		: Items()
		, Tags()
		, NameSpace(ERigMetaDataNameSpace::Self)
		, Inclusive(true)
		, Result()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The items to filter
	UPROPERTY(meta=(Input))
	TArray<FRigElementKey> Items;

	/**
	 * The tags to find
	 */ 
	UPROPERTY(meta = (Input, CustomWidget="MetadataTagName"))
	TArray<FName> Tags;

	/**
	 * Defines in which namespace the metadata will be looked up
	 */
	UPROPERTY(meta = (Input))
	ERigMetaDataNameSpace NameSpace;

	/**
     * If set to true only items with ALL of tags will be returned,
     * if set to false items with ANY of the tags will be removed
     */ 
	UPROPERTY(meta = (Input))
	bool Inclusive;

	// The results of the filter
	UPROPERTY(meta=(Output))
	TArray<FRigElementKey> Result;

	// Used to cache the internally used indices
	UPROPERTY()
	TArray<FCachedRigElement> CachedIndices;
};

/*
 * Returns some metadata on a given module
 */
USTRUCT(meta=(DisplayName="Get Module Metadata"))
struct FRigDispatch_GetModuleMetadata : public FRigDispatch_GetMetadata
{
	GENERATED_BODY()

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	virtual FString GetNodeTitlePrefix() const override { return TEXT("Module "); }
#endif
	
protected:

	static UE_API FRigBaseMetadata* FindMetadata(const FRigVMExtendedExecuteContext& InContext, const FName& InName, ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace);
	UE_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	template<typename ValueType>
	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[NameSpaceArgIndex].IsType<ERigMetaDataNameSpace>(), NameSpaceArgName) &&
			CheckArgumentType(Handles[DefaultArgIndex].IsType<ValueType>(), DefaultArgName) &&
			CheckArgumentType(Handles[ValueArgIndex].IsType<ValueType>(), ValueArgName) &&
			CheckArgumentType(Handles[FoundArgIndex].IsType<bool>(), FoundArgName);
	}
#endif

	template<typename ValueType, typename MetadataType, ERigMetadataType EnumValue>
	static void GetModuleMetadataDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		const FRigDispatch_GetModuleMetadata* Factory = static_cast<const FRigDispatch_GetModuleMetadata*>(InContext.Factory);

#if WITH_EDITOR
		if(!Factory->CheckArgumentTypes<ValueType>(Handles))
		{
			return;
		}
#endif

		// unpack the memory
		const FName& Name = *reinterpret_cast<const FName*>(Handles[Factory->NameArgIndex].GetInputData());
		const ERigMetaDataNameSpace NameSpace = *reinterpret_cast<const ERigMetaDataNameSpace*>(Handles[Factory->NameSpaceArgIndex].GetInputData());
		const ValueType& Default = *reinterpret_cast<const ValueType*>(Handles[Factory->DefaultArgIndex].GetInputData());
		ValueType& Value = *reinterpret_cast<ValueType*>(Handles[Factory->ValueArgIndex].GetOutputData());
		bool& Found = *reinterpret_cast<bool*>(Handles[Factory->FoundArgIndex].GetOutputData());

		// extract the metadata
		if (const MetadataType* Md = Cast<MetadataType>(FindMetadata(InContext, Name, EnumValue, NameSpace)))
		{
			Value = Md->GetValue();
			Found = true;
		}
		else
		{
			Value = Default;
			Found = false;
		}
	}
};

/*
 * Sets metadata on the module
 */
USTRUCT(meta=(DisplayName="Set Module Metadata"))
struct FRigDispatch_SetModuleMetadata : public FRigDispatch_SetMetadata
{
	GENERATED_BODY()

	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	virtual FString GetNodeTitlePrefix() const override { return TEXT("Module "); }
#endif

protected:

	static UE_API FRigBaseMetadata* FindOrAddMetadata(const FControlRigExecuteContext& InContext, const FName& InName, ERigMetadataType InType, ERigMetaDataNameSpace InNameSpace);
	UE_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	template<typename ValueType>
	bool CheckArgumentTypes(FRigVMMemoryHandleArray Handles) const
	{
		return CheckArgumentType(Handles[NameArgIndex].IsType<FName>(), NameArgName) &&
			CheckArgumentType(Handles[NameSpaceArgIndex].IsType<ERigMetaDataNameSpace>(), NameSpaceArgName) &&
			CheckArgumentType(Handles[ValueArgIndex].IsType<ValueType>(), ValueArgName) &&
			CheckArgumentType(Handles[SuccessArgIndex].IsType<bool>(), SuccessArgName);
	}
#endif
	
	template<typename ValueType, typename MetadataType, ERigMetadataType EnumValue>
	static void SetModuleMetadataDispatch(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		const FRigDispatch_SetModuleMetadata* Factory = static_cast<const FRigDispatch_SetModuleMetadata*>(InContext.Factory);

#if WITH_EDITOR
		if(!Factory->CheckArgumentTypes<ValueType>(Handles))
		{
			return;
		}
#endif

		// unpack the memory
		FControlRigExecuteContext& ExecuteContext = InContext.GetPublicData<FControlRigExecuteContext>();
		const FName& Name = *(const FName*)Handles[Factory->NameArgIndex].GetInputData();
		const ERigMetaDataNameSpace NameSpace = *(const ERigMetaDataNameSpace*)Handles[Factory->NameSpaceArgIndex].GetInputData();
		ValueType& Value = *(ValueType*)Handles[Factory->ValueArgIndex].GetOutputData();
		bool& Success = *(bool*)Handles[Factory->SuccessArgIndex].GetOutputData();

		// extract the metadata
		if (MetadataType* Md = Cast<MetadataType>(FindOrAddMetadata(ExecuteContext, Name, EnumValue, NameSpace)))
		{
			if(!Equals<ValueType>(Md->GetValue(), Value))
			{
				Md->GetValue() = Value;
				ExecuteContext.Hierarchy->MetadataVersion++;
			}
			Success = true;
		}
		else
		{
			Success = false;
		}
	}
};

#undef UE_API
