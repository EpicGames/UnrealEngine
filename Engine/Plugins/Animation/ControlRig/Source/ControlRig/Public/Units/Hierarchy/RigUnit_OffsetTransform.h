// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_OffsetTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Offset Transform is used to add an offset to an existing transform in the hierarchy. The offset is post multiplied.
 */
USTRUCT(meta=(DisplayName="Offset Transform", Category="Transforms", DocumentationPolicy = "Strict", Keywords = "Offset,Relative,AddBoneTransform", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_OffsetTransformForItem : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_OffsetTransformForItem()
		: Item(NAME_None, ERigElementType::Bone)
		, OffsetTransform(FTransform::Identity)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to offset the transform for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	// The transform of the item relative to its previous transform
	UPROPERTY(meta=(Input))
	FTransform OffsetTransform;

	// Defines how much the change will be applied
	UPROPERTY(meta=(Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the item internally
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

#undef UE_API
