// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_ParentSwitchConstraint.generated.h"

#define UE_API CONTROLRIG_API


/**
 * The Parent Switch Constraint is used to have an item follow one of multiple parents,
 * and allowing to switch between the parent in question.
 */
USTRUCT(meta=(DisplayName="Parent Switch Constraint", Category="Hierarchy", DocumentationPolicy="Strict", Keywords = "SpaceSwitch", Deprecated = "5.0"))
struct FRigUnit_ParentSwitchConstraint : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_ParentSwitchConstraint()
		: Subject(NAME_None, ERigElementType::Null)
		, ParentIndex(0)
		, Weight(1.0f)
		, Switched(false)
		, CachedSubject(FCachedRigElement())
		, CachedParent(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The subject to constrain
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Subject;

	/**
	 * The parent index to use for constraining the subject
	 */
	UPROPERTY(meta = (Input))
	int32 ParentIndex;

	/**
	 * The list of parents to constrain to
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Parents;

	/**
	 * The initial global transform for the subject
	 */
	UPROPERTY(meta = (Input))
	FTransform InitialGlobalTransform;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The transform result (full without weighting)
	 */
	UPROPERTY(meta = (Output))
	FTransform Transform;

	/**
	 * Returns true if the parent has changed
	 */
	UPROPERTY(meta = (Output))
	bool Switched;

	// Used to cache the internally used subject
	UPROPERTY()
	FCachedRigElement CachedSubject;

	// Used to cache the internally used parent
	UPROPERTY()
	FCachedRigElement CachedParent;

	// The cached relative offset between subject and parent
	UPROPERTY()
	FTransform RelativeOffset;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * The Parent Switch Constraint is used to have an item follow one of multiple parents,
 * and allowing to switch between the parent in question.
 */
USTRUCT(meta=(DisplayName="Parent Switch Constraint", Category="Constraints", DocumentationPolicy="Strict", Keywords = "SpaceSwitch", Deprecated = "5.0"))
struct FRigUnit_ParentSwitchConstraintArray : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_ParentSwitchConstraintArray()
		: Subject(NAME_None, ERigElementType::Null)
		, ParentIndex(0)
		, Weight(1.0f)
		, Switched(false)
		, CachedSubject(FCachedRigElement())
		, CachedParent(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The subject to constrain
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Subject;

	/**
	 * The parent index to use for constraining the subject
	 */
	UPROPERTY(meta = (Input))
	int32 ParentIndex;

	/**
	 * The list of parents to constrain to
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Parents;

	/**
	 * The initial global transform for the subject
	 */
	UPROPERTY(meta = (Input))
	FTransform InitialGlobalTransform;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The transform result (full without weighting)
	 */
	UPROPERTY(meta = (Output))
	FTransform Transform;

	/**
	 * Returns true if the parent has changed
	 */
	UPROPERTY(meta = (Output))
	bool Switched;

	// Used to cache the internally used subject
	UPROPERTY()
	FCachedRigElement CachedSubject;

	// Used to cache the internally used parent
	UPROPERTY()
	FCachedRigElement CachedParent;

	// The cached relative offset between subject and parent
	UPROPERTY()
	FTransform RelativeOffset;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
