// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Units/RigUnit.h"
#include "Animation/PoseAsset.h"
#include "RigUnit_PoseAsset.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT(BlueprintType)
struct FRigUnit_PoseAsset_PoseInfo
{
	GENERATED_BODY()

	FRigUnit_PoseAsset_PoseInfo()
		: PoseName(NAME_None)
		, PoseIndex(INDEX_NONE)
		, UseWeightFromCurve(false)
		, Weight(1.f)
	{
	}

	/**
	 * The pose to look up and apply.
	 */
	UPROPERTY(EditAnywhere, Category = FRigUnit_PoseAsset_PoseInfo)
	FName PoseName;

	/**
	 * The pose to look up and apply.
	 */
	int32 PoseIndex;

	/**
	 * If true the weight will be determined based on a curve with the same name as the pose.
	 */
	UPROPERTY(EditAnywhere, Category = FRigUnit_PoseAsset_PoseInfo)
	bool UseWeightFromCurve;

	/**
	 * If UseWeightFromCurve is false, this is the weight when applying the pose
	 */
	UPROPERTY(EditAnywhere, Category = FRigUnit_PoseAsset_PoseInfo)
	float Weight;
};

/*
 * The base class for all pose asset related nodes
 */
USTRUCT(meta=(Abstract, Category="PoseAsset", Keywords="PoseAsset", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_PoseAssetBase : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_PoseAssetBase()
		: PoseAsset(nullptr)
	{
	}

	/**
	 * The pose asset to retrieve data from
	 */
	UPROPERTY(meta = (Input))
	TObjectPtr<UPoseAsset> PoseAsset;
};

USTRUCT()
struct FRigUnit_PoseAsset_WorkData
{
	GENERATED_BODY()

public:
	
	FRigUnit_PoseAsset_WorkData()
		: LastPoseIndex(0)
		, LastBoneHash(0)
		, LastCurveHash(0)
		, bFilterByInfluences(false)
	{
	}

	/*
	  * Cache to store the index of the retrieved pose
	  */
	UPROPERTY()
	int32 LastPoseIndex; 

	/*
	  * Cache to retrieve the bone transforms
	  */
	UPROPERTY()
	TArray<FTransform> BoneTransforms; 

	/*
	  * Cache to retrieve the curve values
	  */
	UPROPERTY()
	TArray<float> CurveValues; 

	/*
	 * Cache to use to map pose tracks to bones in the hierarchy
	 */
	UPROPERTY()
	TArray<int32> TrackIndexPerBone; 

	/*
	 * Cache to use to map curve values to curves in the hierarchy
	 */
	UPROPERTY()
	TArray<int32> TrackIndexPerCurve; 

	/*
	 * Cached filtered Bone items
     */
	UPROPERTY()
	TArray<FRigElementKey> CachedBoneItems; 

	/*
	 * Cached filtered Curve items
	 */
	UPROPERTY()
	TArray<FRigElementKey> CachedCurveItems; 

	/*
	 * Cache to store the hash to identify changes concerning the bones
	 */
	UPROPERTY()
	uint32 LastBoneHash; 

	/*
	 * Cache to store the hash to identify changes concerning the curves
	 */
	UPROPERTY()
	uint32 LastCurveHash;

	/*
	 * True if the retrieves bone keys should be filtered by influences
	 */
	UPROPERTY()
	bool bFilterByInfluences;

protected:
	
	int32 ValidatePose(const FRigVMExecuteContext& ExecuteContext, const UPoseAsset* InPoseAsset, const FName& InPoseName, bool bReportWarning);
	static const USkeleton* GetTargetSkeleton(const FRigVMExecuteContext& InExecuteContext);
	void UpdateBoneHash(const URigHierarchy* InHierarchy, const UPoseAsset* InPoseAsset, const TArrayView<const FPoseCurve>& InPoses, const TArrayView<const FRigElementKey>& InItems);
	void UpdateCurveHash(const URigHierarchy* InHierarchy, const UPoseAsset* InPoseAsset, const TArrayView<const FPoseCurve>& InPoses, const TArrayView<const FRigElementKey>& InItems);
	TArrayView<const FRigElementKey> FilterBoneItems(const UPoseAsset* InPoseAsset, const URigHierarchy* InHierarchy, const TArrayView<const FRigElementKey>& InItems, const TArray<FPoseAssetInfluenceForPose>& InInfluences);
	TArrayView<const FRigElementKey> FilterCurveItems(const UPoseAsset* InPoseAsset, const URigHierarchy* InHierarchy, const TArrayView<const FRigElementKey>& InItems);
	bool GetAnimationPose(const FControlRigExecuteContext& ExecuteContext, const UPoseAsset* InPoseAsset, const URigHierarchy* InHierarchy, const TArrayView<const FPoseCurve>& InPoses, const TArrayView<const FRigElementKey>& InItems, TArrayView<const FRigElementKey>& OutKeys, TArray<FTransform>& OutBoneTransforms, TArrayView<const FRigElementKey>& OutCurveKeys, TArray<float>& OutCurveValues);
	
	friend struct FRigUnit_PoseAssetContainsPose;
	friend struct FRigUnit_PoseAssetGetActivePoses;
	friend struct FRigUnit_PoseAssetGetPoseForItems;
	friend struct FRigUnit_PoseAssetApplyPose;
	friend struct FRigUnit_PoseAssetApplyMultiplePoses;
	friend struct FRigUnit_PoseAssetGetRigPose;
	friend struct FRigUnit_PoseAssetGetInfluences;
};

/*
 * Returns all the names of the poses contained in a pose asset
 */
USTRUCT(meta=(DisplayName="Get Pose Names"))
struct FRigUnit_PoseAssetGetPoseNames : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetPoseNames()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The names of poses stored in the pose asset
	 */
	UPROPERTY(meta = (Output))
	TArray<FName> PoseNames;
};

/*
 * Returns all active poses given the current weights of the curves and the option items mask
 */
USTRUCT(meta=(DisplayName="Get Active Poses"))
struct FRigUnit_PoseAssetGetActivePoses : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetActivePoses()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The optional list of items to apply the pose to. If this is
	 * empty the pose will be applied to all items.
	 */ 
	UPROPERTY(meta = (Input, DisplayName = "Filtered Items"))
	TArray<FRigElementKey> Items;

	/**
	 * The currently active poses
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigUnit_PoseAsset_PoseInfo> ActivePoses;

	/**
	 * Cache to store intermediate results
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData WorkData;
};

/*
 * Returns true if the given pose asset has root motion
 */
USTRUCT(meta=(DisplayName="Has Root Motion"))
struct FRigUnit_PoseAssetHasRootMotion : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetHasRootMotion()
		: Result(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * True if the given pose asset contains root motion
	 */
	UPROPERTY(meta = (Output))
	bool Result;
};

/*
 * Returns true if the given pose asset is additive
 */
USTRUCT(meta=(DisplayName="Is Additive"))
struct FRigUnit_PoseAssetIsAdditive : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetIsAdditive()
		: Result(false)
		, BasePoseName(NAME_None)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * True if the provided pose asset is additive
	 */
	UPROPERTY(meta = (Output))
	bool Result;

	/**
	 * The name of the base pose if this pose asset is additive
	 */
	UPROPERTY(meta = (Output))
	FName BasePoseName;
};

/*
 * The base class for all pose asset getter nodes
 */
USTRUCT(meta=(Abstract))
struct FRigUnit_PoseAssetGetter : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetter()
		: FRigUnit_PoseAssetBase()
		, PoseName(NAME_None)
	{
	}

	/**
	 * The name of the pose to look up
	 */
	UPROPERTY(meta = (Input))
	FName PoseName;

	/**
	 * Cache to store intermediate results
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData WorkData;
};

/*
 * Returns true if the given pose asset contains a pose base on its name
 */
USTRUCT(meta=(DisplayName="Contains Pose"))
struct FRigUnit_PoseAssetContainsPose : public FRigUnit_PoseAssetGetter
{
	GENERATED_BODY()

	FRigUnit_PoseAssetContainsPose()
		: FRigUnit_PoseAssetGetter()
		, Result(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * True if the pose asset contains the given pose
	 */
	UPROPERTY(meta = (Output))
	bool Result;
};

/*
 * Returns the names of the bone tracks stored in the pose asset
 */
USTRUCT(meta=(DisplayName="Get Pose Bone Names"))
struct FRigUnit_PoseAssetGetBoneNames : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetBoneNames()
		: FRigUnit_PoseAssetBase()
		, Success(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The names of the retrieved bones
	 */
	UPROPERTY(meta = (Output))
	TArray<FName> BoneNames;

	/**
	 * True if the bones were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;
};

/*
 * Returns the keys of the influenced items for a given pose in the pose asset
 */
USTRUCT(meta=(DisplayName="Get Pose Influences"))
struct FRigUnit_PoseAssetGetInfluences : public FRigUnit_PoseAssetGetter
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetInfluences()
		: FRigUnit_PoseAssetGetter()
		, Success(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The optional list of items to retrieve the influences for.
	 * If this is empty the influences will be retrieves for all items.
	 */ 
	UPROPERTY(meta = (Input, DisplayName = "Filtered Items"))
	TArray<FRigElementKey> Items;
	
	/**
	 * The names of the retrieved bones
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Bones;

	/**
	 * The names of the retrieved curves
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Curves;
	
	/**
	 * True if the bones / curves were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;
};

/*
 * Returns the index of a bone track based on the bone name
 */
USTRUCT(meta=(DisplayName="Get Pose Bone Index"))
struct FRigUnit_PoseAssetGetBoneIndex : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetBoneIndex()
		: FRigUnit_PoseAssetBase()
		, BoneName(NAME_None)
		, Index(INDEX_NONE)
		, Success(false)
		, LastBoneIndex(INDEX_NONE)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of bone index to retrieve
	 */
	UPROPERTY(meta = (Input))
	FName BoneName;

	/**
	 * The index of the bone
	 */
	UPROPERTY(meta = (Output))
	int32 Index;

	/**
	 * True if the bone index was determined successfully
	 */
	UPROPERTY(meta = (Output))
	bool Success;

	/**
	 * A cache to use when looking up the bone index
	 */
	UPROPERTY()
	int32 LastBoneIndex;
};

UENUM(meta = (RigVMTypeAllowed))
enum class EPoseAssetGetPoseForItemsMode : uint8
{
	/** Returns the pose without accumulating it at all. */
	Raw,

	/** Applies the pose to the current local transform */
	ApplyToCurrent,

	/** Applies the pose to the initial  pose local transform */
	ApplyToInitial
};

/*
 * Returns the transforms and curve values of a set of poses
 */
USTRUCT(meta=(DisplayName="Get Combined Pose"))
struct FRigUnit_PoseAssetGetPoseForItems : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetPoseForItems()
		: FRigUnit_PoseAssetBase()
		, Mode(EPoseAssetGetPoseForItemsMode::ApplyToInitial)
		, Success(false)
	{
		Poses.AddDefaulted(1);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The list of poses to retrieve
	 */ 
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_PoseAsset_PoseInfo> Poses;

	/**
	 * The optional list of items to retrieve the pose for. If this is
	 * empty the pose will be retrieved for all items.
	 */ 
	UPROPERTY(meta = (Input, DisplayName = "Filtered Items"))
	TArray<FRigElementKey> Items;

	/**
	 * The mode to use when retrieving the pose.
	 * NOTE: This is only used if the pose asset is additive.
	 */ 
	UPROPERTY(meta = (Input))
	EPoseAssetGetPoseForItemsMode Mode;
	
	/**
	 * Only the bones returned from the pose
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Bones;

	/**
	 * The transforms of the retrieved bones in local space
	 */
	UPROPERTY(meta = (Output))
	TArray<FTransform> LocalTransforms;

	/**
	 * Only the curves returned from the pose
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Curves;

	/**
	 * The values of the retrieved curves
	 */
	UPROPERTY(meta = (Output))
	TArray<float> CurveValues;

	/**
	 * True if the transforms were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;

	/*
	 * Cache to use to map pose tracks to bones and curves in the hierarchy
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData WorkData; 
};

/*
 * Returns a pose from a pose asset as a rig pose
 */
USTRUCT(meta=(DisplayName="Get Pose Cache"))
struct FRigUnit_PoseAssetGetRigPose : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetRigPose()
		: FRigUnit_PoseAssetBase()
		, Mode(EPoseAssetGetPoseForItemsMode::ApplyToInitial)
		, Success(false)
	{
		Poses.AddDefaulted(1);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The list of poses to retrieve
	 */ 
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_PoseAsset_PoseInfo> Poses;

	/**
	 * The optional list of items to apply the pose to. If this is
	 * empty the pose will be retrieved for all items.
	 */ 
	UPROPERTY(meta = (Input, DisplayName = "Filtered Items"))
	TArray<FRigElementKey> Items;

	/**
	 * The mode to use when retrieving the pose.
	 * NOTE: This is only used if the pose asset is additive.
	 */ 
	UPROPERTY(meta = (Input))
	EPoseAssetGetPoseForItemsMode Mode;

	/**
	 * The retrieved rig pose if successful
	 */
	UPROPERTY(meta = (DisplayName="Pose Cache", Output))
	FRigPose RigPose;

	/**
	 * True if the transforms were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;

	/*
	 * Cache to use to map pose tracks to bones and curves in the hierarchy
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData WorkData; 
};

/*
 * Returns the names of the curves stored in the pose asset
 */
USTRUCT(meta=(DisplayName="Get Pose Curve Names"))
struct FRigUnit_PoseAssetGetCurveNames : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetGetCurveNames()
		: FRigUnit_PoseAssetBase()
		, Success(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The names of the retrieved curves
	 */
	UPROPERTY(meta = (Output))
	TArray<FName> CurveNames;

	/**
	 * True if the curves were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;
};

/*
 * Applies multiple poses to the hierarchy.
 */
USTRUCT(meta=(DisplayName="Apply Multiple Poses"))
struct FRigUnit_PoseAssetApplyMultiplePoses : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetApplyMultiplePoses()
		: FRigUnit_PoseAssetBase()
		, ApplyTransforms(true)
		, ApplyCurves(true)
		, bPropagateToChildren(true)
		, Success(false)
	{
		// add one pose by default
		Poses.AddDefaulted();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * This property is used to chain multiple mutable units together
	 */
	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;

	/**
	 * The list of poses to apply
	 */ 
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_PoseAsset_PoseInfo> Poses;

	/**
	 * The optional list of items to apply the pose to. If this is
	 * empty the pose will be applied to all items.
	 */ 
	UPROPERTY(meta = (Input, DisplayName = "Filtered Items"))
	TArray<FRigElementKey> Items;

	/**
	 * If True the bone transforms will be applied
	 */ 
	UPROPERTY(meta = (Input))
	bool ApplyTransforms;

	/**
	 * If True the curve values will be applied
	 */ 
	UPROPERTY(meta = (Input))
	bool ApplyCurves;

	/**
	 * If set to true the transforms of the children
	 * of affected bones will be recalculated based on their local transforms.
	 */
	UPROPERTY(meta = (Input))
	bool bPropagateToChildren;
	

	/**
	 * True if the transforms were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;

	/*
	 * Cache to use to map pose tracks to bones and curves in the hierarchy
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData WorkData; 
};

/*
 * Applies all poses to the hierarchy. The weights per pose are determined based
 * on the curve values with matching names.
 */
USTRUCT(meta=(DisplayName="Apply Active Poses"))
struct FRigUnit_PoseAssetApplyAllPoses : public FRigUnit_PoseAssetBase
{
	GENERATED_BODY()

	FRigUnit_PoseAssetApplyAllPoses()
		: FRigUnit_PoseAssetBase()
		, ApplyTransforms(true)
		, ApplyCurves(true)
		, bPropagateToChildren(true)
		, Success(false)
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/*
	 * This property is used to chain multiple mutable units together
	 */
	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;

	/**
	 * The optional list of items to apply the poses to. If this is
	 * empty the poses will be retrieved for all items.
	 */ 
	UPROPERTY(meta = (Input, DisplayName = "Filtered Items"))
	TArray<FRigElementKey> Items;

	/**
	 * If True the bone transforms will be applied
	 */ 
	UPROPERTY(meta = (Input))
	bool ApplyTransforms;

	/**
	 * If True the curve values will be applied
	 */ 
	UPROPERTY(meta = (Input))
	bool ApplyCurves;

	/**
	 * If set to true the transforms of the children
	 * of affected bones will be recalculated based on their local transforms.
	 */
	UPROPERTY(meta = (Input))
	bool bPropagateToChildren;
	

	/**
	 * True if the transforms were retrieved successfully from the pose asset
	 */
	UPROPERTY(meta = (Output))
	bool Success;

	/**
	 * The list of poses to apply. This is computed automatically.
	 */
	UPROPERTY()
	TArray<FRigUnit_PoseAsset_PoseInfo> ActivePoses;

	/**
	 * Cache to store intermediate results
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData GetActivePosesWorkData;

	
	/*
	 * Cache to use to map pose tracks to bones and curves in the hierarchy
	 */
	UPROPERTY()
	FRigUnit_PoseAsset_WorkData WorkData; 
};

#undef UE_API
