// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_ProjectTransformToNewParent.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Gets the relative offset between the child and the old parent, then multiplies by new parent's transform.
 */
USTRUCT(meta=(DisplayName="Project to new Parent", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords = "ProjectTransformToNewParent,Relative,Reparent,Offset", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ProjectTransformToNewParent : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ProjectTransformToNewParent()
		: CachedChild(FCachedRigElement())
		, CachedOldParent(FCachedRigElement())
		, CachedNewParent(FCachedRigElement())
	{
		Child = FRigElementKey(NAME_None, ERigElementType::Bone);
		bChildInitial = true;
		OldParent = FRigElementKey(NAME_None, ERigElementType::Bone);
		bOldParentInitial = true;
		NewParent = FRigElementKey(NAME_None, ERigElementType::Bone);
		bNewParentInitial = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The element to project between parents
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/**
	 * If set to true the child will be retrieved in its initial transform
	 */
	UPROPERTY(meta = (Input))
	bool bChildInitial;

	/**
	 * The original parent of the child.
	 * Can be an actual parent in the hierarchy or any other
	 * item you want to use to compute to offset against.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey OldParent;

	/**
	 * If set to true the old parent will be retrieved in its initial transform
	 */
	UPROPERTY(meta = (Input))
	bool bOldParentInitial;

	/**
	 * The new parent of the child.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey NewParent;

	/**
	 * If set to true the new parent will be retrieved in its initial transform
	 */
	UPROPERTY(meta = (Input))
	bool bNewParentInitial;

	// The resulting transform
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// Used to cache the internally used child
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the internally used old parent
	UPROPERTY()
	FCachedRigElement CachedOldParent;

	// Used to cache the internally used new parent
	UPROPERTY()
	FCachedRigElement CachedNewParent;
};

#undef UE_API
