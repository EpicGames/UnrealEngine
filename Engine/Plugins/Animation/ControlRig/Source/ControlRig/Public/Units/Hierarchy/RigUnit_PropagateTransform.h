// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_PropagateTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Propagate Transform can be used to force a recalculation of a bone's global transform
 * from its local - as well as propagating that change onto the children.
 */
USTRUCT(meta=(DisplayName="Propagate Transform", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords = "PropagateToChildren,RecomputeGlobal,RecalculateGlobal", Varying, Deprecated = "5.0"))
struct FRigUnit_PropagateTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_PropagateTransform()
		: Item(NAME_None, ERigElementType::Bone)
		, bRecomputeGlobal(true)
		, bApplyToChildren(false)
		, bRecursive(false)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to offset the transform for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	// If set to true the item's global transform will be recomputed from
	// its parent's transform and its local.
	UPROPERTY(meta=(Input))
	bool bRecomputeGlobal;

	// If set to true the direct children of this item will be recomputed as well.
	// Combined with bRecursive all children will be affected recursively.
	UPROPERTY(meta=(Input))
	bool bApplyToChildren;

	// If set to true and with bApplyToChildren enabled
	// all children will be affected recursively.
	UPROPERTY(meta=(Input))
	bool bRecursive;

	// Used to cache the item internally
	UPROPERTY(transient)
	FCachedRigElement CachedIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
