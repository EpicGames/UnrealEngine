// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SendEvent.generated.h"

#define UE_API CONTROLRIG_API

/**
 * SendEvent is used to notify the engine / editor of a change that happened within the Control Rig.
 */
USTRUCT(meta=(DisplayName="Send Event", Category="Hierarchy", DocumentationPolicy="Strict", TemplateName = "SendEvent", Keywords="Event,Notify,Notification", NodeColor="1, 0, 0"))
struct FRigUnit_SendEvent : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SendEvent()
		: Event(ERigEvent::RequestAutoKey)
		, Item()
		, OffsetInSeconds(0.f)
		, bEnable(true)
		, bOnlyDuringInteraction(true)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The event to send to the engine
	 */
	UPROPERTY(meta = (Input))
	ERigEvent Event;

	/**
	 * The item to send the event for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The time offset to use for the send event
	 */
	UPROPERTY(meta = (Input, UIMin = "-1000", UIMax = "1000"))
	float OffsetInSeconds;

	/**
	 * The event will be sent if this is checked
	 */
	UPROPERTY(meta = (Input))
	bool bEnable;

	/**
	 * The event will be sent if this only during an interaction
	 */
	UPROPERTY(meta = (Input))
	bool bOnlyDuringInteraction;
};

#undef UE_API
