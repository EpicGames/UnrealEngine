// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetBoneInitialTransform.generated.h"

#define UE_API CONTROLRIG_API


/**
 * SetBoneInitialTransform is used to perform a change in the hierarchy by setting a single bone's transform.
 */
USTRUCT(meta=(DisplayName="Set Initial Bone Transform", Category="Setup", DocumentationPolicy="Strict", Keywords = "SetBoneInitialTransform", Deprecated="4.25"))
struct FRigUnit_SetBoneInitialTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetBoneInitialTransform()
		: Space(ERigVMTransformSpace::GlobalSpace)
		, bPropagateToChildren(true)
		, CachedBone(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Bone to set the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The transform value to set for the given Bone.
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * The transform value result (after weighting)
	 */
	UPROPERTY(meta = (Output))
	FTransform Result;

	/**
	 * Defines if the bone's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;
	
	/**
	 * If set to true all of the global transforms of the children 
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FCachedRigElement CachedBone;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
