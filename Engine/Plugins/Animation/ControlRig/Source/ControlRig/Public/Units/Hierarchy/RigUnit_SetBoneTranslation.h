// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetBoneTranslation.generated.h"

#define UE_API CONTROLRIG_API


/**
 * SetBoneTranslation is used to perform a change in the hierarchy by setting a single bone's Translation.
 */
USTRUCT(meta=(DisplayName="Set Translation", Category="Hierarchy", DocumentationPolicy="Strict", Keywords = "SetBoneTranslation,SetPosition,SetLocation,SetBonePosition,SetBoneLocation", Deprecated="4.25"))
struct FRigUnit_SetBoneTranslation : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetBoneTranslation()
		: Translation(FVector::ZeroVector)
		, Space(ERigVMTransformSpace::LocalSpace)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedBone(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Bone to set the Translation for.
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The Translation value to set for the given Bone.
	 */
	UPROPERTY(meta = (Input))
	FVector Translation;

	/**
	 * Defines if the bone's Translation should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children 
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FCachedRigElement CachedBone;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
