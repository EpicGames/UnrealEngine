// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetControlColor.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetControlColor is used to retrieve the color of control
 */
USTRUCT(meta=(DisplayName="Get Control Color", Category="Controls", DocumentationPolicy="Strict", Keywords = "GetControlColor,GetGizmoColor", TemplateName="GetControlColor", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_GetControlColor : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlColor()
		: Control(NAME_None)
		, Color(FLinearColor::Black)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to get the color for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The color of the control
	 */
	UPROPERTY(meta = (Output))
	FLinearColor Color;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlColor is used to change the control's color
 */
USTRUCT(meta=(DisplayName="Set Control Color", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlColor,SetGizmoColor", TemplateName="SetControlColor", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlColor : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlColor()
		: Control(NAME_None)
		, Color(FLinearColor::Black)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the color for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The color to set for the control
	 */
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

#undef UE_API
