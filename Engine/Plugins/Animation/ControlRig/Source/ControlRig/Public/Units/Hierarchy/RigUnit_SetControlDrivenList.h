// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetControlDrivenList.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetControlDrivenList is used to retrieve the list of affected controls of an indirect control
 */
USTRUCT(meta=(DisplayName="Get Driven Controls", Category="Controls", DocumentationPolicy="Strict", Keywords = "GetControlDrivenList,Interaction,Indirect", TemplateName="GetControlDrivenList", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_GetControlDrivenList : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlDrivenList()
		: Control(NAME_None)
		, Driven()
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to get the list for
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The list of affected controls
	 */
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> Driven;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlDrivenList is used to change the list of affected controls of an indirect control
 */
USTRUCT(meta=(DisplayName="Set Driven Controls", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlDrivenList,Interaction,Indirect", TemplateName="SetControlDrivenList", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlDrivenList : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlDrivenList()
		: Control(NAME_None)
		, Driven()
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the list for
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The list of affected controls
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Driven;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

#undef UE_API
