// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetControlOffset.generated.h"

#define UE_API CONTROLRIG_API

/**
 * SetControlOffset is used to perform a change in the hierarchy by setting a single control's transform.
 * This is typically only used during the Construction Event.
 */
USTRUCT(meta=(DisplayName="Set Control Offset", Category="Controls", TemplateName="SetControlOffset", DocumentationPolicy="Strict", Keywords = "SetControlOffset,Initial,InitialTransform,SetInitialTransform,SetInitialControlTransform", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetControlOffset : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlOffset()
		: Control(NAME_None)
		, Offset(FTransform::Identity)
		, Space(ERigVMTransformSpace::GlobalSpace)
	{
	}

	UE_API virtual FString GetUnitLabel() const override;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the offset transform for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The offset transform to set on the control
	 */
	UPROPERTY(meta = (Input, Output))
	FTransform Offset;
	
	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlTranslationOffset is used to perform a change in the hierarchy by setting a single control's translation offset.
 * This is typically only used during the Construction Event.
 */
USTRUCT(meta = (DisplayName = "Set Control Translation Offset", Category = "Controls", TemplateName="SetControlOffset", DocumentationPolicy = "Strict", Keywords = "SetControlOffset,Initial,InitialTransform,SetInitialTransform,SetInitialControlTransform,SetControlTranslationOffset,SetInitialTranslation,SetInitialLocation", NodeColor = "0, 0.364706, 1.0", Varying))
struct FRigUnit_SetControlTranslationOffset : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlTranslationOffset()
		: Control(NAME_None)
		, Offset(FVector::ZeroVector)
		, Space(ERigVMTransformSpace::GlobalSpace)
	{
	}

	UE_API virtual FString GetUnitLabel() const override;

	RIGVM_METHOD()
		UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the translation offset for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The input translation offset to set on the control
	 */
	UPROPERTY(meta = (Input, Output))
	FVector Offset;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;


	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlRotationOffset is used to perform a change in the hierarchy by setting a single control's rotation offset.
 * This is typically only used during the Construction Event.
 */
USTRUCT(meta = (DisplayName = "Set Control Rotation Offset", Category = "Controls", TemplateName="SetControlOffset", DocumentationPolicy = "Strict", Keywords = "SetControlOffset,Initial,InitialTransform,SetInitialTransform,SetInitialControlTransform,SetControlRotationOffset,SetInitialRotation,SetInitialRotation", NodeColor = "0, 0.364706, 1.0", Varying))
struct FRigUnit_SetControlRotationOffset : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlRotationOffset()
		: Control(NAME_None)
		, Offset(FQuat::Identity)
		, Space(ERigVMTransformSpace::GlobalSpace)
	{
	}

	UE_API virtual FString GetUnitLabel() const override;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the rotation offset for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The input rotation offset to set on the control
	 */
	UPROPERTY(meta = (Input, Output))
	FQuat Offset;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlScaleOffset is used to perform a change in the hierarchy by setting a single control's scale offset.
 * This is typically only used during the Construction Event.
 */
USTRUCT(meta = (DisplayName = "Set Control Scale Offset", Category = "Controls", DocumentationPolicy = "Strict", Keywords = "SetControlOffset,Initial,InitialTransform,SetInitialTransform,SetInitialControlTransform,SetControlScaleOffset,SetInitialScale", NodeColor = "0, 0.364706, 1.0", Varying))
struct FRigUnit_SetControlScaleOffset : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlScaleOffset()
		: Control(NAME_None)
		, Scale(FVector::OneVector)
		, Space(ERigVMTransformSpace::GlobalSpace)
	{
	}

	UE_API virtual FString GetUnitLabel() const override;

	RIGVM_METHOD()
		UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the scale offset for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The new scale offset to set on the control
	 */
	UPROPERTY(meta = (Input, Output))
	FVector Scale;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * GetShapeTransform is used to retrieve single control's shape transform.
 * This is typically only used during the Construction Event.
 */
USTRUCT(meta=(DisplayName="Get Shape Transform", Category="Controls", DocumentationPolicy="Strict", Keywords = "GetControlShapeTransform,Gizmo,GizmoTransform,MeshTransform", NodeColor="0, 0.364706, 1.0"))
struct FRigUnit_GetShapeTransform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetShapeTransform()
		: Control(NAME_None)
	{
		Transform = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to get the shape transform for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The shape transform to set for the control
	 */
	UPROPERTY(meta = (Output))
	FTransform Transform;

	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetShapeTransform is used to perform a change in the hierarchy by setting a single control's shape transform.
 * This is typically only used during the Construction Event.
 */
USTRUCT(meta=(DisplayName="Set Shape Transform", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlShapeTransform,Gizmo,GizmoTransform,MeshTransform", NodeColor="0, 0.364706, 1.0"))
struct FRigUnit_SetShapeTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetShapeTransform()
		: Control(NAME_None)
	{
		Transform = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the shape transform for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The shape transform to set for the control
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

#undef UE_API
