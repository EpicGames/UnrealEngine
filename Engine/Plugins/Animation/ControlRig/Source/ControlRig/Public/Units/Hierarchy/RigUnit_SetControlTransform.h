// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetControlTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * SetControlBool is used to perform a change in the hierarchy by setting a single control's bool value.
 */
USTRUCT(meta=(DisplayName="Set Control Bool", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlBool,SetGizmoBool", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlBool : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlBool()
		: BoolValue(false)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the bool for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The bool value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	bool BoolValue;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

USTRUCT()
struct FRigUnit_SetMultiControlBool_Entry
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlBool_Entry()
		: BoolValue(false)
	{}
	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;
	/**
	 * The bool value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input))
	bool BoolValue;
};

/**
 * SetMultiControlBool is used to perform a change in the hierarchy by setting multiple controls' bool value.
 */
USTRUCT(meta = (DisplayName = "Set Multiple Controls Bool", Category = "Controls", DocumentationPolicy = "Strict", Keywords = "SetMultipleControlsBool,SetControlBool,SetGizmoBool", TemplateName="SetMultiControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetMultiControlBool : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlBool()
	{
		Entries.Add(FRigUnit_SetMultiControlBool_Entry());
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The array of control-Bool pairs to be processed
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_SetMultiControlBool_Entry> Entries;

	// Used to cache the internally used control indices
	UPROPERTY()
	TArray<FCachedRigElement> CachedControlIndices;
};

/**
 * SetControlFloat is used to perform a change in the hierarchy by setting a single control's float value.
 */
USTRUCT(meta=(DisplayName="Set Control Float", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlFloat,SetGizmoFloat", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlFloat : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlFloat()
		: Weight(1.f)
		, FloatValue(0.f)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The float value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output, UIMin = "0.0", UIMax = "1.0"))
	float FloatValue;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

USTRUCT()
struct FRigUnit_SetMultiControlFloat_Entry
{ 
	GENERATED_BODY()

	FRigUnit_SetMultiControlFloat_Entry()
		: FloatValue(0.f)
	{}
	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The float value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float FloatValue; 
};

/**
 * SetMultiControlFloat is used to perform a change in the hierarchy by setting multiple controls' float value.
 */
USTRUCT(meta = (DisplayName = "Set Multiple Controls Float", Category = "Controls", DocumentationPolicy = "Strict", Keywords = "SetMultipleControlsFloat,SetControlFloat,SetGizmoFloat", TemplateName="SetMultiControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetMultiControlFloat : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlFloat()
		: Weight(1.f)
	{
		Entries.Add(FRigUnit_SetMultiControlFloat_Entry());
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The array of control-float pairs to be processed
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_SetMultiControlFloat_Entry> Entries;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// Used to cache the internally used control indices
	UPROPERTY()
	TArray<FCachedRigElement> CachedControlIndices;
};



/**
 * SetControlInteger is used to perform a change in the hierarchy by setting a single control's int32 value.
 */
USTRUCT(meta=(DisplayName="Set Control Integer", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlInteger,SetGizmoInteger", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0", Deprecated="5.8.0"))
struct FRigUnit_SetControlInteger : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlInteger()
		: Weight(1.f)
		, IntegerValue(0)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	int32 Weight;

	/**
	 * The integer value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	int32 IntegerValue;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlInteger is used to perform a change in the hierarchy by setting a single control's int32 value.
 */
USTRUCT(meta=(DisplayName="Set Control Integer", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlInteger,SetGizmoInteger", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlIntegerFloatWeight : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlIntegerFloatWeight()
		: Weight(1.f)
		, IntegerValue(0)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The integer value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	int32 IntegerValue;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

USTRUCT()
struct FRigUnit_SetMultiControlInteger_Entry
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlInteger_Entry()
		: IntegerValue(0)
	{}
	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The integer value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input))
	int32 IntegerValue;
};

/**
 * SetMultiControlInteger is used to perform a change in the hierarchy by setting multiple controls' integer value.
 */
USTRUCT(meta = (DisplayName = "Set Multiple Controls Integer", Category = "Controls", DocumentationPolicy = "Strict", Keywords = "SetMultipleControlsInteger,SetControlInteger,SetGizmoInteger", TemplateName="SetMultiControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetMultiControlInteger : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlInteger()
		: Weight(1.f)
	{
		Entries.Add(FRigUnit_SetMultiControlInteger_Entry());
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The array of control-integer pairs to be processed
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_SetMultiControlInteger_Entry> Entries;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// Used to cache the internally used control indices
	UPROPERTY()
	TArray<FCachedRigElement> CachedControlIndices;
};

/**
 * SetControlVector2D is used to perform a change in the hierarchy by setting a single control's Vector2D value.
 */
USTRUCT(meta=(DisplayName="Set Control Vector2D", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlVector2D,SetGizmoVector2D", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlVector2D : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlVector2D()
		: Weight(1.f)
		, Vector(FVector2D::ZeroVector)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The 2d vector value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	FVector2D Vector;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

USTRUCT()
struct FRigUnit_SetMultiControlVector2D_Entry
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlVector2D_Entry()
		: Vector(FVector2D::ZeroVector)
	{}

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The 2d vector value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input))
	FVector2D Vector;
};

/**
 * SetMultiControlVector2D is used to perform a change in the hierarchy by setting multiple controls' vector2D value.
 */
USTRUCT(meta = (DisplayName = "Set Multiple Controls Vector2D", Category = "Controls", DocumentationPolicy = "Strict", Keywords = "SetMultipleControlsVector2D,SetControlVector2D,SetGizmoVector2D", TemplateName="SetMultiControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetMultiControlVector2D : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlVector2D()
		: Weight(1.f)
	{
		Entries.Add(FRigUnit_SetMultiControlVector2D_Entry());
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The array of control-vector2D pairs to be processed
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_SetMultiControlVector2D_Entry> Entries;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// Used to cache the internally used control indices
	UPROPERTY()
	TArray<FCachedRigElement> CachedControlIndices;
};

/**
 * SetControlVector is used to perform a change in the hierarchy by setting a single control's Vector value.
 */
USTRUCT(meta=(DisplayName="Set Control Vector", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlVector,SetGizmoVector", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlVector : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlVector()
		: Weight(1.f)
		, Vector(FVector::OneVector)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The vector value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	FVector Vector;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlRotator is used to perform a change in the hierarchy by setting a single control's Rotator value.
 */
USTRUCT(meta=(DisplayName="Set Control Rotator", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlRotator,SetGizmoRotator", TemplateName="SetControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetControlRotator : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlRotator()
		: Weight(1.f)
		, Rotator(FRotator::ZeroRotator)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The rotator value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	FRotator Rotator;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

USTRUCT()
struct FRigUnit_SetMultiControlRotator_Entry
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlRotator_Entry()
	{
		Rotator = FRotator::ZeroRotator;
		Space = ERigVMTransformSpace::GlobalSpace;
	}

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName"))
	FName Control;

	/**
	 * The rotator value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input))
	FRotator Rotator;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;
};

/**
 * SetMultiControlRotator is used to perform a change in the hierarchy by setting multiple controls' rotator value.
 */
USTRUCT(meta = (DisplayName = "Set Multiple Controls Rotator", Category = "Controls", DocumentationPolicy = "Strict", Keywords = "SetMultipleControlsRotator,SetControlRotator,SetGizmoRotator", TemplateName="SetMultiControlValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetMultiControlRotator : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetMultiControlRotator()
		: Weight(1.f)
	{
		Entries.Add(FRigUnit_SetMultiControlRotator_Entry());
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The array of control-rotator pairs to be processed
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigUnit_SetMultiControlRotator_Entry> Entries;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// Used to cache the internally used control indices
	UPROPERTY()
	TArray<FCachedRigElement> CachedControlIndices;
};

/**
 * SetControlTransform is used to perform a change in the hierarchy by setting a single control's transform.
 */
USTRUCT(meta=(DisplayName="Set Control Transform", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlTransform,SetGizmoTransform", Deprecated = "4.25"))
struct FRigUnit_SetControlTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlTransform()
		: Weight(1.f)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, CachedControlIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName Control;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The transform value to get / set for the given Control.
	 */
	UPROPERTY(meta = (Input, Output))
	FTransform Transform;

	/**
	 * Defines if the control's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
