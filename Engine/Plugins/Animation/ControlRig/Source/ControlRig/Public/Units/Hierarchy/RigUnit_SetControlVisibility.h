// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetControlVisibility.generated.h"

#define UE_API CONTROLRIG_API

/**
 * GetControlVisibility is used to retrieve the visibility of a control
 */
USTRUCT(meta=(DisplayName="Get Control Visibility", Category="Controls", DocumentationPolicy="Strict", Keywords = "GetControlVisibility,Visibility,Hide,Show,Hidden,Visible,SetGizmoVisibility", TemplateName="GetControlVisibility", NodeColor="0, 0.364706, 1.0"))
struct FRigUnit_GetControlVisibility : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_GetControlVisibility()
		: Item(NAME_None, ERigElementType::Control)
		, bVisible(true)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to get the visibility for.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The visibility of the control
	 */
	UPROPERTY(meta = (Output))
	bool bVisible;

	// Used to cache the internally used control index
	UPROPERTY()
	FCachedRigElement CachedControlIndex;
};

/**
 * SetControlVisibility is used to change the visibility on a control at runtime
 */
USTRUCT(meta=(DisplayName="Set Control Visibility", Category="Controls", DocumentationPolicy="Strict", Keywords = "SetControlVisibility,Visibility,Hide,Show,Hidden,Visible,SetGizmoVisibility", TemplateName="SetControlVisibility", NodeColor="0, 0.364706, 1.0"))
struct FRigUnit_SetControlVisibility : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetControlVisibility()
		: Item(NAME_None, ERigElementType::Control)
		, Pattern(FString())
		, bVisible(true)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Control to set the visibility for.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * If the ControlName is set to None this can be used to look for a series of Controls
	 */
	UPROPERTY(meta = (Input, Constant))
	FString Pattern;

	/**
	 * The visibility to set for the control
	 */
	UPROPERTY(meta = (Input))
	bool bVisible;

	// Used to cache the internally used control index
	UPROPERTY()
	TArray<FCachedRigElement> CachedControlIndices;
};

#undef UE_API
