// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetCurveValue.generated.h"


/**
 * SetCurveValue is used to perform a change in the curve container by setting a single Curve value.
 */
USTRUCT(meta=(DisplayName="Set Curve Value", Category="Curve", Keywords = "SetCurveValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_SetCurveValue : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetCurveValue()
		: Value(0.f)
		, CachedCurveIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	virtual void Execute() override;

	/**
	 * The name of the Curve to set the Value for.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "CurveName"))
	FName Curve;

	/**
	 * The value to set for the given Curve.
	 */
	UPROPERTY(meta = (Input))
	float Value;

private:
	// Used to cache the internally used curve index
	UPROPERTY()
	FCachedRigElement CachedCurveIndex;
};
