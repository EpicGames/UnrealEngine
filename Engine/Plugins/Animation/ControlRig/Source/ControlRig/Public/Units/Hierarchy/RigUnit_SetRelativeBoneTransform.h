// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetRelativeBoneTransform.generated.h"

#define UE_API CONTROLRIG_API


/**
 * SetRelativeBoneTransform is used to perform a change in the hierarchy by setting a single bone's transform.
 */
USTRUCT(meta=(DisplayName="Set Relative Transform", Category="Hierarchy", DocumentationPolicy="Strict", Keywords = "SetRelativeBoneTransform", Deprecated = "4.25"))
struct FRigUnit_SetRelativeBoneTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetRelativeBoneTransform()
		: Weight(1.f)
		, bPropagateToChildren(true)
		, CachedBone(FCachedRigElement())
		, CachedSpaceIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Bone to set the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The name of the Bone to set the transform relative within.
	 */
	UPROPERTY(meta = (Input))
	FName Space;

	/**
	 * The transform value to set for the given Bone.
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children 
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FCachedRigElement CachedBone;

	// Used to cache the internally used space index
	UPROPERTY(transient)
	FCachedRigElement CachedSpaceIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
