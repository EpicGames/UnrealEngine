// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetRelativeTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * SetRelativeTransform is used to set a single transform from a hierarchy in the space of another item
 */
USTRUCT(meta=(DisplayName="Set Relative Transform", Category="Transforms", TemplateName="Set Relative Transform", DocumentationPolicy = "Strict", Keywords = "Offset,Local", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetRelativeTransformForItem : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetRelativeTransformForItem()
		: Child(NAME_None, ERigElementType::Bone)
		, Parent(NAME_None, ERigElementType::Bone)
		, bParentInitial(false)
		, Value(FTransform::Identity)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedChild()
		, CachedParent()
	{}

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The child item to set the transform for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/**
	 * The parent item to use.
	 * The child transform will be set in the space of the parent.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	/**
	 * Defines if the parent's transform should be determined as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bParentInitial;

	// The transform of the child item relative to the provided parent
	UPROPERTY(meta=(Input))
	FTransform Value;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the child internally
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the parent internally
	UPROPERTY()
	FCachedRigElement CachedParent;
};

/**
 * SetRelativeTranslation is used to set a single translation from a hierarchy in the space of another item
 */
USTRUCT(meta=(DisplayName="Set Relative Translation", Category="Transforms", TemplateName="Set Relative Transform", DocumentationPolicy = "Strict", Keywords = "Offset,Local", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetRelativeTranslationForItem : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetRelativeTranslationForItem()
		: Child(NAME_None, ERigElementType::Bone)
		, Parent(NAME_None, ERigElementType::Bone)
		, bParentInitial(false)
		, Value(FVector::ZeroVector)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedChild()
		, CachedParent()
	{}

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The child item to set the translation for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/**
	 * The parent item to use.
	 * The child translation will be set in the space of the parent.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	/**
	 * Defines if the parent's transform should be determined as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bParentInitial;

	// The translation of the child item relative to the provided parent
	UPROPERTY(meta=(Input))
	FVector Value;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the child internally
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the parent internally
	UPROPERTY()
	FCachedRigElement CachedParent;
};

/**
 * SetRelativeRotation is used to set a single rotation from a hierarchy in the space of another item
 */
USTRUCT(meta=(DisplayName="Set Relative Rotation", Category="Transforms", TemplateName="Set Relative Transform", DocumentationPolicy = "Strict", Keywords = "Offset,Local", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetRelativeRotationForItem : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetRelativeRotationForItem()
		: Child(NAME_None, ERigElementType::Bone)
		, Parent(NAME_None, ERigElementType::Bone)
		, bParentInitial(false)
		, Value(FQuat::Identity)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedChild()
		, CachedParent()
	{}

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The child item to set the rotation for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/**
	 * The parent item to use.
	 * The child rotation will be set in the space of the parent.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Parent;

	/**
	 * Defines if the parent's transform should be determined as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bParentInitial;

	// The rotation of the child item relative to the provided parent
	UPROPERTY(meta=(Input))
	FQuat Value;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the child internally
	UPROPERTY()
	FCachedRigElement CachedChild;

	// Used to cache the parent internally
	UPROPERTY()
	FCachedRigElement CachedParent;
};

#undef UE_API
