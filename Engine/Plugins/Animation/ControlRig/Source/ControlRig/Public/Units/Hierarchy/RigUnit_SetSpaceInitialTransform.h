// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetSpaceInitialTransform.generated.h"

#define UE_API CONTROLRIG_API


/**
 * SetSpaceInitialTransform is used to perform a change in the hierarchy by setting a single space's initial transform.
 */
USTRUCT(meta=(DisplayName="Set Initial Space Transform", Category="Setup", DocumentationPolicy="Strict", Keywords = "SetSpaceInitialTransform", Deprecated="4.25"))
struct FRigUnit_SetSpaceInitialTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetSpaceInitialTransform()
		: Space(ERigVMTransformSpace::GlobalSpace)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Space to set the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName SpaceName;

	/**
	 * The transform value to set for the given space.
	 */
	UPROPERTY(meta = (Input))
	FTransform Transform;

	/**
	 * The transform value result (after weighting)
	 */
	UPROPERTY(meta = (Output))
	FTransform Result;

	/**
	 * Defines if the bone's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	// user to internally cache the index of the bone
	UPROPERTY()
	FCachedRigElement CachedSpaceIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
