// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetSpaceTransform.generated.h"

#define UE_API CONTROLRIG_API


/**
 * SetSpaceTransform is used to perform a change in the hierarchy by setting a single space's transform.
 */
USTRUCT(meta=(DisplayName="Set Space", Category="Hierarchy", DocumentationPolicy="Strict", Keywords = "SetSpaceTransform", Deprecated="4.25"))
struct FRigUnit_SetSpaceTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetSpaceTransform()
		: Weight(1.f)
		, SpaceType(ERigVMTransformSpace::GlobalSpace)
		, CachedSpaceIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The name of the Space to set the transform for.
	 */
	UPROPERTY(meta = (Input))
	FName Space;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The transform value to set for the given Space.
	 */
	UPROPERTY(meta = (Input, Output))
	FTransform Transform;

	/**
	 * Defines if the bone's transform should be set
	 * in local or global space.
	 */
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace SpaceType;

	// Used to cache the internally used bone index
	UPROPERTY()
	FCachedRigElement CachedSpaceIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
