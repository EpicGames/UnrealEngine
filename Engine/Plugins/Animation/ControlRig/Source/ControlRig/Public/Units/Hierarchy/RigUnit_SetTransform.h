// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_SetTransform.generated.h"

#define UE_API CONTROLRIG_API

/**
 * SetTransform is used to set a single transform on hierarchy.
 * 
 * Note: For Controls when setting the initial transform this node
 * actually sets the Control's offset transform and resets the local
 * values to (0, 0, 0).
 */
USTRUCT(meta=(DisplayName="Set Transform", Category="Transforms", TemplateName = "Set Transform", DocumentationPolicy = "Strict", Keywords="SetBoneTransform,SetControlTransform,SetInitialTransform,SetSpaceTransform", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetTransform : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetTransform()
		: Item(NAME_None, ERigElementType::Bone)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Value(FTransform::Identity)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

	UE_API virtual FString GetUnitLabel() const override;

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the transform for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * Defines if the transform should be set in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be set as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The new transform of the given item
	UPROPERTY(meta=(Input))
	FTransform Value;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * SetTranslation is used to set a single translation on hierarchy.
 */
USTRUCT(meta=(DisplayName="Set Translation", Category="Transforms", TemplateName = "Set Transform", DocumentationPolicy = "Strict", Keywords="SetBoneTranslation,SetControlTranslation,SetInitialTranslation,SetSpaceTranslation,SetBoneLocation,SetControlLocation,SetInitialLocation,SetSpaceLocation,SetBonePosition,SetControlPosition,SetInitialPosition,SetSpacePosition,SetTranslation,SetLocation,SetPosition", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetTranslation : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetTranslation()
		: Item(NAME_None, ERigElementType::Bone)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Value(FVector::ZeroVector)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

	UE_API virtual FString GetUnitLabel() const override;

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the translation for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * Defines if the translation should be set in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be set as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The new translation of the given item
	UPROPERTY(meta=(Input))
	FVector Value;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * SetRotation is used to set a single rotation on hierarchy.
 */
USTRUCT(meta=(DisplayName="Set Rotation", Category="Transforms", TemplateName = "Set Transform", DocumentationPolicy = "Strict", Keywords="SetBoneRotation,SetControlRotation,SetInitialRotation,SetSpaceRotation,SetBoneOrientation,SetControlOrientation,SetInitialOrientation,SetSpaceOrientation,SetRotation,SetOrientation", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetRotation : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetRotation()
		: Item(NAME_None, ERigElementType::Bone)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Value(FQuat::Identity)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

	UE_API virtual FString GetUnitLabel() const override;

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the rotation for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * Defines if the rotation should be set in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be set as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The new rotation of the given item
	UPROPERTY(meta=(Input))
	FQuat Value;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * SetScale is used to set a single scale on hierarchy.
 */
USTRUCT(meta=(DisplayName="Set Scale", Category="Transforms", DocumentationPolicy = "Strict", Keywords="SetBoneScale,SetControlScale,SetInitialScale,SetSpaceScale,SetScale", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetScale : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetScale()
		: Item(NAME_None, ERigElementType::Bone)
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Scale(FVector::OneVector)
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

	virtual FString GetUnitLabel() const override;

#if WITH_EDITOR
	virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	virtual void Execute() override;

	/**
	 * The item to set the scale for
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * Defines if the scale should be set in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be set as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The new scale of the given item
	UPROPERTY(meta=(Input))
	FVector Scale;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the internally used index
	UPROPERTY()
	FCachedRigElement CachedIndex;
};

/**
 * SetTransformArray is used to set an array of transforms on the hierarchy.
 * 
 * Note: For Controls when setting the initial transform this node
 * actually sets the Control's offset transform and resets the local
 * values to (0, 0, 0).
 */
USTRUCT(meta=(DisplayName="Set Transform Array", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="SetBoneTransform,SetControlTransform,SetInitialTransform,SetSpaceTransform", NodeColor="0, 0.364706, 1.0", Varying, Deprecated = "5.0"))
struct FRigUnit_SetTransformArray : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetTransformArray()
		: Items()
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Transforms()
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the transform for
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/**
	 * Defines if the transform should be set in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be set as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The new transform of the given item
	UPROPERTY(meta=(Input))
	TArray<FTransform> Transforms;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the internally used index
	UPROPERTY()
	TArray<FCachedRigElement> CachedIndex;
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * SetTransformArray is used to set an array of transforms on the hierarchy.
 * 
 * Note: For Controls when setting the initial transform this node
 * actually sets the Control's offset transform and resets the local
 * values to (0, 0, 0).
 */
USTRUCT(meta=(DisplayName="Set Transform Array", Category="Transforms", DocumentationPolicy = "Strict", Keywords="SetBoneTransform,SetControlTransform,SetInitialTransform,SetSpaceTransform", NodeColor="0, 0.364706, 1.0", Varying))
struct FRigUnit_SetTransformItemArray : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_SetTransformItemArray()
		: Items()
		, Space(ERigVMTransformSpace::GlobalSpace)
		, bInitial(false)
		, Transforms()
		, Weight(1.f)
		, bPropagateToChildren(true)
		, CachedIndex()
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The item to set the transform for
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/**
	 * Defines if the transform should be set in local or global space
	 */ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace Space;

	/**
	 * Defines if the transform should be set as current (false) or initial (true).
	 * Initial transforms for bones and other elements in the hierarchy represent the reference pose's value.
	 */ 
	UPROPERTY(meta = (Input))
	bool bInitial;

	// The new transform of the given item
	UPROPERTY(meta=(Input))
	TArray<FTransform> Transforms;

	// Defines how much the change will be applied
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	// If set to true children of affected items in the hierarchy
	// will follow the transform change - otherwise only the parent will move.
	UPROPERTY(meta=(Input))
	bool bPropagateToChildren;

	// Used to cache the internally used index
	UPROPERTY()
	TArray<FCachedRigElement> CachedIndex;
};

#undef UE_API
