// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_UnsetCurveValue.generated.h"


/**
 * UnsetCurveValue is used to perform a change in the curve container by invalidating a single Curve value.
 */
USTRUCT(meta=(DisplayName="Unset Curve Value", Category="Curve", Keywords = "UnsetCurveValue", NodeColor = "0.0 0.36470600962638855 1.0"))
struct FRigUnit_UnsetCurveValue : public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_UnsetCurveValue()
		: CachedCurveIndex(FCachedRigElement())
	{}

	RIGVM_METHOD()
	virtual void Execute() override;

	/**
	 * The name of the Curve to unset/invalidate.
	 */
	UPROPERTY(meta = (Input, CustomWidget = "CurveName"))
	FName Curve;

private:
	// Used to cache the internally used curve index
	UPROPERTY()
	FCachedRigElement CachedCurveIndex;
};
