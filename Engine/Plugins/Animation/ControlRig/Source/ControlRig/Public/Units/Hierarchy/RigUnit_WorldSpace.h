// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_WorldSpace.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Converts a transform from rig (global) space to world space
 */
USTRUCT(meta=(DisplayName="To World", TemplateName="To World", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="Global,Local,World,Actor,ComponentSpace,FromRig", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ToWorldSpace_Transform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ToWorldSpace_Transform()
		: Value(FTransform::Identity)
		, World(FTransform::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The input transform in global / rig space
	 */
	UPROPERTY(meta = (Input))
	FTransform Value;

	/**
	 * The result transform in world space
	 */ 
	UPROPERTY(meta = (Output))
	FTransform World;
};

/**
 * Converts a transform from world space to rig (global) space
 */
USTRUCT(meta=(DisplayName="From World", TemplateName="From World", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="Global,Local,World,Actor,ComponentSpace,ToRig", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ToRigSpace_Transform : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ToRigSpace_Transform()
		: Value(FTransform::Identity)
		, Global(FTransform::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The input transform in world
	 */
	UPROPERTY(meta = (Input))
	FTransform Value;

	/**
	 * The result transform in global / rig space
	 */ 
	UPROPERTY(meta = (Output))
	FTransform Global;
};

/**
 * Converts a position / location from rig (global) space to world space
 */
USTRUCT(meta=(DisplayName="To World", TemplateName="To World", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="Global,Local,World,Actor,ComponentSpace,FromRig", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ToWorldSpace_Location : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ToWorldSpace_Location()
		: Value(FVector::ZeroVector)
		, World(FVector::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The input position / location in global / rig space
	 */
	UPROPERTY(meta = (Input))
	FVector Value;

	/**
	 * The result position / location in world space
	 */ 
	UPROPERTY(meta = (Output))
	FVector World;
};

/**
 * Converts a position / location from world space to rig (global) space
 */
USTRUCT(meta=(DisplayName="From World", TemplateName="From World", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="Global,Local,World,Actor,ComponentSpace,ToRig", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ToRigSpace_Location : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ToRigSpace_Location()
		: Value(FVector::ZeroVector)
		, Global(FVector::ZeroVector)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The input position / location in world
	 */
	UPROPERTY(meta = (Input))
	FVector Value;

	/**
	 * The result position / location in global / rig space
	 */ 
	UPROPERTY(meta = (Output))
	FVector Global;
};

/**
 * Converts a rotation from rig (global) space to world space
 */
USTRUCT(meta=(DisplayName="To World", TemplateName="To World", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="Global,Local,World,Actor,ComponentSpace,FromRig", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ToWorldSpace_Rotation : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ToWorldSpace_Rotation()
		: Value(FQuat::Identity)
		, World(FQuat::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The input rotation in global / rig space
	 */
	UPROPERTY(meta = (Input))
	FQuat Value;

	/**
	 * The result rotation in world space
	 */ 
	UPROPERTY(meta = (Output))
	FQuat World;
};

/**
 * Converts a rotation from world space to rig (global) space
 */
USTRUCT(meta=(DisplayName="From World", TemplateName="From World", Category="Hierarchy", DocumentationPolicy = "Strict", Keywords="Global,Local,World,Actor,ComponentSpace,ToRig", NodeColor="0.462745, 1,0, 0.329412", Varying))
struct FRigUnit_ToRigSpace_Rotation : public FRigUnit
{
	GENERATED_BODY()

	FRigUnit_ToRigSpace_Rotation()
		: Value(FQuat::Identity)
		, Global(FQuat::Identity)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The input rotation in world
	 */
	UPROPERTY(meta = (Input))
	FQuat Value;

	/**
	 * The result rotation in global / rig space
	 */ 
	UPROPERTY(meta = (Output))
	FQuat Global;
};

#undef UE_API
