// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_BoneHarmonics.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_BoneHarmonics_BoneTarget
{
	GENERATED_BODY()

	FRigUnit_BoneHarmonics_BoneTarget()
	{
		Bone = NAME_None;
		Ratio = 0.f;
	}

	/**
	 * The name of the bone to drive
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The ratio of where the bone sits within the harmonics system.
	 * Valid values reach from 0.0 to 1.0
	 */
	UPROPERTY(meta = (Input, Constant))
	float Ratio;
};

USTRUCT()
struct FRigUnit_Harmonics_TargetItem
{
	GENERATED_BODY()

	FRigUnit_Harmonics_TargetItem()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		Ratio = 0.f;
	}

	/**
	 * The name of the item to drive
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The ratio of where the item sits within the harmonics system.
	 * Valid values reach from 0.0 to 1.0
	 */
	UPROPERTY(meta = (Input, Constant))
	float Ratio;
};

USTRUCT()
struct FRigUnit_BoneHarmonics_WorkData
{
	GENERATED_BODY()

	FRigUnit_BoneHarmonics_WorkData()
	{
		WaveTime = FVector::ZeroVector;
	}

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	FVector WaveTime;
};

/**
 * Performs point based simulation
 */
USTRUCT(meta=(DisplayName="Harmonics", Keywords="Sin,Wave", Deprecated = 4.25))
struct FRigUnit_BoneHarmonics : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()
	
	FRigUnit_BoneHarmonics()
	{
		WaveSpeed = FVector::OneVector;
		WaveAmplitude = FVector(0.0f, 70.f, 0.f);
		WaveFrequency = FVector(1.f, 0.6f, 0.8f);
		WaveOffset = FVector(0.f, 1.f, 2.f);
		WaveNoise = FVector::ZeroVector;
		WaveEase = ERigVMAnimEasingType::Linear;
		WaveMinimum = 0.5f;
		WaveMaximum = 1.f;
		RotationOrder = EEulerRotationOrder::YZX;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** The bones to drive. */
	UPROPERTY(meta = (Input, Constant))
	TArray<FRigUnit_BoneHarmonics_BoneTarget> Bones;

	UPROPERTY(meta = (Input))
	FVector WaveSpeed;

	UPROPERTY(meta = (Input))
	FVector WaveFrequency;

	/** The amplitude in degrees per axis */
	UPROPERTY(meta = (Input))
	FVector WaveAmplitude;

	UPROPERTY(meta = (Input))
	FVector WaveOffset;

	UPROPERTY(meta = (Input))
	FVector WaveNoise;

	UPROPERTY(meta = (Input))
	ERigVMAnimEasingType WaveEase;

	UPROPERTY(meta = (Input))
	float WaveMinimum;

	UPROPERTY(meta = (Input))
	float WaveMaximum;

	UPROPERTY(meta = (Input))
	EEulerRotationOrder RotationOrder;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_BoneHarmonics_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Drives an array of items through a harmonics spectrum
 */
USTRUCT(meta=(DisplayName="Harmonics", Keywords="Sin,Wave", Category = "Simulation"))
struct FRigUnit_ItemHarmonics : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()
	
	FRigUnit_ItemHarmonics()
	{
		WaveSpeed = FVector::OneVector;
		WaveAmplitude = FVector(0.0f, 70.f, 0.f);
		WaveFrequency = FVector(1.f, 0.6f, 0.8f);
		WaveOffset = FVector(0.f, 1.f, 2.f);
		WaveNoise = FVector::ZeroVector;
		WaveEase = ERigVMAnimEasingType::Linear;
		WaveMinimum = 0.5f;
		WaveMaximum = 1.f;
		RotationOrder = EEulerRotationOrder::YZX;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** The items to drive. */
	UPROPERTY(meta = (Input, Constant))
	TArray<FRigUnit_Harmonics_TargetItem> Targets;

	// The speed of the wave to use
	UPROPERTY(meta = (Input))
	FVector WaveSpeed;

	// The frequency of the wave to use
	UPROPERTY(meta = (Input))
	FVector WaveFrequency;

	/** The amplitude in degrees per axis */
	UPROPERTY(meta = (Input))
	FVector WaveAmplitude;

	// The positional offset of the wave
	UPROPERTY(meta = (Input))
	FVector WaveOffset;

	// The amount of noise to apply to the wave
	UPROPERTY(meta = (Input))
	FVector WaveNoise;

	// The easing function to apply to the wave
	UPROPERTY(meta = (Input))
	ERigVMAnimEasingType WaveEase;

	// The minimum value for the wave
	UPROPERTY(meta = (Input))
	float WaveMinimum;

	// The maximum value for the wave
	UPROPERTY(meta = (Input))
	float WaveMaximum;

	// The rotation order to use when encoding the rotation
	UPROPERTY(meta = (Input))
	EEulerRotationOrder RotationOrder;

	UPROPERTY(transient)
	FRigUnit_BoneHarmonics_WorkData WorkData;
};

#undef UE_API
