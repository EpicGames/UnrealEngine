// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "Curves/CurveFloat.h"
#include "RigUnit_ChainHarmonics.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_ChainHarmonics_Reach
{
	GENERATED_BODY()

	FRigUnit_ChainHarmonics_Reach()
	{
		bEnabled = true;
		ReachTarget = FVector::ZeroVector;
		ReachAxis = FVector(1.f, 0.f, 0.f);
		ReachMinimum = 0.0f;
		ReachMaximum = 0.0f;
		ReachEase = ERigVMAnimEasingType::Linear;
	}

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input))
	FVector ReachTarget;

	UPROPERTY(meta = (Input, Constant))
	FVector ReachAxis;

	UPROPERTY(meta = (Input))
	float ReachMinimum;

	UPROPERTY(meta = (Input))
	float ReachMaximum;

	UPROPERTY(meta = (Input))
	ERigVMAnimEasingType ReachEase;
};

USTRUCT()
struct FRigUnit_ChainHarmonics_Wave
{
	GENERATED_BODY()

	FRigUnit_ChainHarmonics_Wave()
	{
		bEnabled = true;
		WaveAmplitude = FVector(0.0f, 1.f, 0.f);
		WaveFrequency = FVector(1.f, 0.6f, 0.8f);
		WaveOffset = FVector(0.f, 1.f, 2.f);
		WaveNoise = FVector::ZeroVector;
		WaveMinimum = 0.f;
		WaveMaximum = 1.f;
		WaveEase = ERigVMAnimEasingType::Linear;
	}

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input))
	FVector WaveFrequency;

	UPROPERTY(meta = (Input))
	FVector WaveAmplitude;

	UPROPERTY(meta = (Input))
	FVector WaveOffset;

	UPROPERTY(meta = (Input))
	FVector WaveNoise;

	UPROPERTY(meta = (Input))
	float WaveMinimum;

	UPROPERTY(meta = (Input))
	float WaveMaximum;

	UPROPERTY(meta = (Input))
	ERigVMAnimEasingType WaveEase;
};

USTRUCT()
struct FRigUnit_ChainHarmonics_Pendulum
{
	GENERATED_BODY()

	FRigUnit_ChainHarmonics_Pendulum()
	{
		bEnabled = true;
		PendulumStiffness = 2.0f;
		PendulumGravity = FVector::ZeroVector;
		PendulumBlend = 0.75f;
		PendulumDrag = 0.98f;
		PendulumMinimum = 0.0f;
		PendulumMaximum = 0.1f;
		PendulumEase = ERigVMAnimEasingType::Linear;
		UnwindAxis = FVector(0.f, 1.f, 0.f);
		UnwindMinimum = 0.2f;
		UnwindMaximum = 0.05f;
	}

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input))
	float PendulumStiffness;

	UPROPERTY(meta = (Input))
	FVector PendulumGravity;

	UPROPERTY(meta = (Input))
	float PendulumBlend;

	UPROPERTY(meta = (Input))
	float PendulumDrag;

	UPROPERTY(meta = (Input))
	float PendulumMinimum;

	UPROPERTY(meta = (Input))
	float PendulumMaximum;

	UPROPERTY(meta = (Input))
	ERigVMAnimEasingType PendulumEase;

	UPROPERTY(meta = (Input))
	FVector UnwindAxis;

	UPROPERTY(meta = (Input))
	float UnwindMinimum;

	UPROPERTY(meta = (Input))
	float UnwindMaximum;
};

USTRUCT()
struct FRigUnit_ChainHarmonics_WorkData
{
	GENERATED_BODY()

	FRigUnit_ChainHarmonics_WorkData()
	{
		Time = FVector::ZeroVector;
	}

	UPROPERTY()
	FVector Time;

	UPROPERTY()
	TArray<FCachedRigElement> Items;

	UPROPERTY()
	TArray<float> Ratio;

	UPROPERTY()
	TArray<FVector> LocalTip;

	UPROPERTY()
	TArray<FVector> PendulumTip;

	UPROPERTY()
	TArray<FVector> PendulumPosition;

	UPROPERTY()
	TArray<FVector> PendulumVelocity;

	UPROPERTY()
	TArray<FVector> HierarchyLine;

	UPROPERTY()
	TArray<FVector> VelocityLines;
};

USTRUCT(meta=(DisplayName="ChainHarmonics", Deprecated = "4.25"))
struct FRigUnit_ChainHarmonics : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ChainHarmonics()
	{
		ChainRoot = NAME_None;
		Speed = FVector::OneVector;
		
		Reach.bEnabled = false;
		Wave.bEnabled = true;
		Pendulum.bEnabled = false;

		WaveCurve = FRuntimeFloatCurve();
		WaveCurve.GetRichCurve()->AddKey(0.f, 0.f);
		WaveCurve.GetRichCurve()->AddKey(1.f, 1.f);

		bDrawDebug = true;
		DrawWorldOffset = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FName ChainRoot;

	UPROPERTY(meta = (Input))
	FVector Speed;

	UPROPERTY(meta = (Input))
	FRigUnit_ChainHarmonics_Reach Reach;

	UPROPERTY(meta = (Input))
	FRigUnit_ChainHarmonics_Wave Wave;

	UPROPERTY(meta = (Input, Constant))
	FRuntimeFloatCurve WaveCurve;

	UPROPERTY(meta = (Input))
	FRigUnit_ChainHarmonics_Pendulum Pendulum;

	UPROPERTY(meta = (Input))
	bool bDrawDebug;

	UPROPERTY(meta = (Input))
	FTransform DrawWorldOffset;

	UPROPERTY(transient)
	FRigUnit_ChainHarmonics_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Given a root will drive all items underneath in a chain based harmonics spectrum
 */
USTRUCT(meta=(DisplayName="Chain Harmonics", Category = "Simulation"))
struct FRigUnit_ChainHarmonicsPerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ChainHarmonicsPerItem()
	{
		ChainRoot = FRigElementKey(NAME_None, ERigElementType::Bone);
		Speed = FVector::OneVector;
		
		Reach.bEnabled = false;
		Wave.bEnabled = true;
		Pendulum.bEnabled = false;

		WaveCurve = FRuntimeFloatCurve();
		WaveCurve.GetRichCurve()->AddKey(0.f, 0.f);
		WaveCurve.GetRichCurve()->AddKey(1.f, 1.f);

		bDrawDebug = true;
		DrawWorldOffset = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The root of the chain to apply the harmonics to
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey ChainRoot;

	// The speed of the harmonics effects
	UPROPERTY(meta = (Input))
	FVector Speed;

	// The reach settings
	// Reach lets the chain "lean" towards the target trying to reach it.
	UPROPERTY(meta = (Input))
	FRigUnit_ChainHarmonics_Reach Reach;

	// The wave settings
	// A wave is a rocking back and forth motion to be applied to all / some axes.
	UPROPERTY(meta = (Input))
	FRigUnit_ChainHarmonics_Wave Wave;

	// The curve to use when evaluating the wave
	UPROPERTY(meta = (Input, Constant))
	FRuntimeFloatCurve WaveCurve;

	// The pendulum settings
	// The harmonic pendulum uses a simple spring interpolation to allow to follow secondary motion.
	UPROPERTY(meta = (Input))
	FRigUnit_ChainHarmonics_Pendulum Pendulum;

	// if True the debug drawing will be enabled
	UPROPERTY(meta = (Input))
	bool bDrawDebug;

	// The world offset to be used when debug drawing
	UPROPERTY(meta = (Input))
	FTransform DrawWorldOffset;

	UPROPERTY(transient)
	FRigUnit_ChainHarmonics_WorkData WorkData;
};

#undef UE_API
