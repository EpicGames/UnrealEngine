// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"

#include "Units/Highlevel/Hierarchy/RigUnit_TransformConstraint.h"

#include "RigUnit_AimBone.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_AimBone_Target
{
	GENERATED_BODY()

	FRigUnit_AimBone_Target()
	{
		Weight = 1.f;
		Axis = FVector(1.f, 0.f, 0.f);
		Target = FVector(1.f, 0.f, 0.f);
		Kind = EControlRigVectorKind::Location;
		Space = NAME_None;
	}

	/**
	 * The amount of aim rotation to apply on this target.
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "AimTarget")
	float Weight;

	/**
	 * The axis to align with the aim on this target
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "Weight > 0.0"), Category = "AimTarget")
	FVector Axis;

	/**
	 * The target to aim at - can be a direction or location based on the Kind setting
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "Weight > 0.0"), Category = "AimTarget")
	FVector Target;

	/**
	 * The kind of target this is representing - can be a direction or a location
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "Weight > 0.0"), Category = "AimTarget")
	EControlRigVectorKind Kind;

	/**
	 * The space in which the target is expressed
	 */
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Target Space", Input, EditCondition = "Weight > 0.0" ), Category = "AimTarget")
	FName Space;
};

USTRUCT(BlueprintType)
struct FRigUnit_AimItem_Target
{
	GENERATED_BODY()

	FRigUnit_AimItem_Target()
	{
		Weight = 1.f;
		Axis = FVector(1.f, 0.f, 0.f);
		Target = FVector(1.f, 0.f, 0.f);
		Kind = EControlRigVectorKind::Location;
		Space = FRigElementKey(NAME_None, ERigElementType::Bone);
	}

	/**
	 * The amount of aim rotation to apply on this target.
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "AimTarget")
	float Weight;

	/**
	 * The axis to align with the aim on this target
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "Weight > 0.0"), Category = "AimTarget")
	FVector Axis;

	/**
	 * The target to aim at - can be a direction or location based on the Kind setting
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "Weight > 0.0"), Category = "AimTarget")
	FVector Target;

	/**
	 * The kind of target this is representing - can be a direction or a location
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "Weight > 0.0"), Category = "AimTarget")
	EControlRigVectorKind Kind;

	/**
	 * The space in which the target is expressed
	 */
	UPROPERTY(EditAnywhere, meta = (DisplayName = "Target Space", Input, EditCondition = "Weight > 0.0" ), Category = "AimTarget")
	FRigElementKey Space;
};

USTRUCT(BlueprintType)
struct FRigUnit_AimBone_DebugSettings
{
	GENERATED_BODY()

	FRigUnit_AimBone_DebugSettings()
	{
		bEnabled = false;
		Scale = 10.f;
		WorldOffset = FTransform::Identity;
	}

	/**
	 * If enabled debug information will be drawn 
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "DebugSettings")
	bool bEnabled;

	/**
	 * The size of the debug drawing information
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	float Scale;

	/**
	 * The offset at which to draw the debug information in the world
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FTransform WorldOffset;
};

/**
 * Outputs an aligned transform of a primary and secondary axis of an input transform to a world target.
 * Note: This node operates in world space!
 */
USTRUCT(meta = (DisplayName = "Aim Math", Category = "Hierarchy", Keywords = "Lookat"))
struct FRigUnit_AimBoneMath : public FRigUnit_HighlevelBase
{
	GENERATED_BODY()

	FRigUnit_AimBoneMath()
	{
		InputTransform = FTransform::Identity;
		Primary = FRigUnit_AimItem_Target();
		Secondary = FRigUnit_AimItem_Target();
		Primary.Axis = FVector(1.f, 0.f, 0.f);
		Secondary.Axis = FVector(0.f, 0.f, 1.f);
		Weight = 1.f;
		DebugSettings = FRigUnit_AimBone_DebugSettings();
		PrimaryCachedSpace = FCachedRigElement();
		SecondaryCachedSpace = FCachedRigElement();
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The transform (in global space) before the aim was applied (optional)
	 */
	UPROPERTY(meta = (Input))
	FTransform InputTransform;

	/**
	 * The primary target for the aim
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigUnit_AimItem_Target Primary;

	/**
	 * The secondary target for the aim - also referred to as PoleVector / UpVector
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_AimItem_Target Secondary;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * The resulting transform
	 */
	UPROPERTY(meta = (Output))
	FTransform Result;

	/** The debug setting for the node */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_AimBone_DebugSettings DebugSettings;

	UPROPERTY()
	FCachedRigElement PrimaryCachedSpace;

	UPROPERTY()
	FCachedRigElement SecondaryCachedSpace;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Aligns the rotation of a primary and secondary axis of a bone to a global target.
 * Note: This node operates in global space!
 */
USTRUCT(meta=(DisplayName="Aim", Category="Hierarchy", Keywords="Lookat", Deprecated = "4.25"))
struct FRigUnit_AimBone : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AimBone()
	{
		Bone = NAME_None;
		Primary = FRigUnit_AimBone_Target();
		Secondary = FRigUnit_AimBone_Target();
		Primary.Axis = FVector(1.f, 0.f, 0.f);
		Secondary.Axis = FVector(0.f, 0.f, 1.f);
		Weight = 1.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_AimBone_DebugSettings();
		CachedBoneIndex = FCachedRigElement();
		PrimaryCachedSpace = FCachedRigElement();
		SecondaryCachedSpace = FCachedRigElement();
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the bone to align
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The primary target for the aim 
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigUnit_AimBone_Target Primary;

	/**
	 * The secondary target for the aim - also referred to as PoleVector / UpVector
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_AimBone_Target Secondary;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/** The debug setting for the node */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_AimBone_DebugSettings DebugSettings;

	UPROPERTY()
	FCachedRigElement CachedBoneIndex;

	UPROPERTY()
	FCachedRigElement PrimaryCachedSpace;

	UPROPERTY()
	FCachedRigElement SecondaryCachedSpace;

	UPROPERTY()
	bool bIsInitialized;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};
/**
 * Aligns the rotation of a primary and secondary axis of an item to a global target.
 * Note: This node operates in global space!
 */
USTRUCT(meta=(DisplayName="Aim", Category="Hierarchy", Keywords="Lookat"))
struct FRigUnit_AimItem: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AimItem()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		Primary = FRigUnit_AimItem_Target();
		Secondary = FRigUnit_AimItem_Target();
		Primary.Axis = FVector(1.f, 0.f, 0.f);
		Secondary.Axis = FVector(0.f, 0.f, 1.f);
		Weight = 1.f;
		DebugSettings = FRigUnit_AimBone_DebugSettings();
		CachedItem = FCachedRigElement();
		PrimaryCachedSpace = FCachedRigElement();
		SecondaryCachedSpace = FCachedRigElement();
		bIsInitialized = false;
	}

#if WITH_EDITOR
	UE_API virtual bool GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const override;
	UE_API virtual TArray<const URigVMPin*> GetPinsForDirectManipulation(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const override;
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the item to align
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	/**
	 * The primary target for the aim 
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigUnit_AimItem_Target Primary;

	/**
	 * The secondary target for the aim - also referred to as PoleVector / UpVector
	 */
	UPROPERTY(meta = (Input))
	FRigUnit_AimItem_Target Secondary;

	/**
	 * The weight of the change - how much the change should be applied
	 */
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float Weight;

	/** The debug setting for the node */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_AimBone_DebugSettings DebugSettings;

	UPROPERTY()
	FCachedRigElement CachedItem;

	UPROPERTY()
	FCachedRigElement PrimaryCachedSpace;

	UPROPERTY()
	FCachedRigElement SecondaryCachedSpace;

	UPROPERTY()
	bool bIsInitialized;
};


USTRUCT()
struct FRigUnit_AimConstraint_WorldUp
{
	GENERATED_BODY()

	FRigUnit_AimConstraint_WorldUp()
	{
		Target = FVector(0.f, 0.f, 1.f);
		Kind = EControlRigVectorKind::Direction;
		Space = FRigElementKey(NAME_None, ERigElementType::None);
	}

	/**
	 * The target to aim at - can be a direction or location based on the Kind setting
	 */
	UPROPERTY(meta = (Input))
	FVector Target;

	/**
	 * The kind of target this is representing - can be a direction or a location
	 */
	UPROPERTY(meta = (DisplayName="Target is ", Input))
	EControlRigVectorKind Kind;

	/**
	 * The space in which the target is expressed, use None to indicate world space
	 */
	UPROPERTY(meta = (DisplayName = "Target Space", Input))
	FRigElementKey Space;
};


USTRUCT()
struct FRigUnit_AimConstraint_AdvancedSettings
{
	GENERATED_BODY()

	FRigUnit_AimConstraint_AdvancedSettings()
		:RotationOrderForFilter(EEulerRotationOrder::XZY)
	{}
	
	/**
	*	Settings related to debug drawings
	*/
	UPROPERTY(meta = (Input,DetailsOnly))
	FRigUnit_AimBone_DebugSettings DebugSettings;
	
	/**
	*	Rotation is converted to euler angles using the specified order such that individual axes can be filtered.
	*/
	UPROPERTY(meta = (Input))
	EEulerRotationOrder RotationOrderForFilter;	
};

/**
 * Orients an item such that its aim axis points towards a global target.
 * Note: This node operates in global space!
 */
USTRUCT(meta=(DisplayName="Aim Constraint", Category="Constraints", Keywords="Lookat, Aim"))
struct FRigUnit_AimConstraintLocalSpaceOffset: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AimConstraintLocalSpaceOffset()
		: Child(FRigElementKey())
		, bMaintainOffset(true)
		, Filter(FFilterOptionPerAxis())
		, AimAxis(FVector(1.0f,0.0f,0.0f))
		, UpAxis(FVector(0.0f,0.0f,1.0f))
		, WorldUp(FRigUnit_AimConstraint_WorldUp())
		, AdvancedSettings(FRigUnit_AimConstraint_AdvancedSettings())
		, Weight(1.0f)
		, bIsInitialized(false)
	{
		Parents.Add(FConstraintParent());
	};
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the item to apply aim
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	/**
	 *	Maintains the offset between child and weighted average of parents based on initial transforms
	 */
	UPROPERTY(meta = (Input))
	bool bMaintainOffset;

	/**
	 * Filters the final rotation by axes based on the euler rotation order defined in the node's advanced settings
	 * If flipping is observed, try adjusting the rotation order
	 */
	UPROPERTY(meta = (Input))
	FFilterOptionPerAxis Filter;

	/**
	 * Child is rotated so that its AimAxis points to the parents
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FVector AimAxis;

	/**
	 * Child is rotated around the AimAxis so that its UpAxis points to/Aligns with the WorldUp target 
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FVector UpAxis;

	/**
	 * Defines how Child should rotate around the AimAxis. This is the aim target for the UpAxis
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigUnit_AimConstraint_WorldUp WorldUp;

	// The array of parents to constraint this aim to
	UPROPERTY(meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	// The advanced settings to use for the constraint
	UPROPERTY(meta = (Input))
	FRigUnit_AimConstraint_AdvancedSettings AdvancedSettings;

	// The weight to use when applying the resulting transform to the child
	UPROPERTY(meta = (Input))
	float Weight;
	
	UPROPERTY()
	FCachedRigElement WorldUpSpaceCache; 

	UPROPERTY()
	FCachedRigElement ChildCache;

	UPROPERTY()
	TArray<FCachedRigElement> ParentCaches;
	
	UPROPERTY()
	bool bIsInitialized;
};

#undef UE_API
