// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "CCDIK.h"
#include "RigUnit_CCDIK.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_CCDIK_RotationLimit
{
	GENERATED_BODY()

	FRigUnit_CCDIK_RotationLimit()
	{
		Bone = NAME_None;
		Limit = 30.f;
	}

	/**
	 * The name of the bone to apply the rotation limit to.
	 */
	UPROPERTY(meta = (Input))
	FName Bone;

	/**
	 * The limit of the rotation in degrees.
	 */
	UPROPERTY(meta = (Input, Constant))
	float Limit;
};

USTRUCT()
struct FRigUnit_CCDIK_RotationLimitPerItem
{
	GENERATED_BODY()

	FRigUnit_CCDIK_RotationLimitPerItem()
	{
		Item = FRigElementKey(NAME_None, ERigElementType::Bone);
		Limit = 30.f;
	}

	/**
	 * The name of the item to apply the rotation limit to.
	 */
	UPROPERTY(meta = (Input))
	FRigElementKey Item;

	/**
	 * The limit of the rotation in degrees.
	 */
	UPROPERTY(meta = (Input, Constant))
	float Limit;
};

USTRUCT()
struct FRigUnit_CCDIK_WorkData
{
	GENERATED_BODY()

	FRigUnit_CCDIK_WorkData()
	{
		CachedEffector = FCachedRigElement();
	}

	UPROPERTY()
	TArray<FCCDIKChainLink> Chain;

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	TArray<int32> RotationLimitIndex;

	UPROPERTY()
	TArray<float> RotationLimitsPerItem;

	UPROPERTY()
	FCachedRigElement CachedEffector;
};

/**
 * The CCDIK solver can solve N-Bone chains using 
 * the Cyclic Coordinate Descent Inverse Kinematics algorithm.
 * For now this node supports single effector chains only.
 */
USTRUCT(meta=(DisplayName="CCDIK", Category="Hierarchy", Keywords="N-Bone,IK", Deprecated = "4.25"))
struct FRigUnit_CCDIK : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_CCDIK()
	{
		EffectorTransform = FTransform::Identity;
		Precision = 1.f;
		Weight = 1.f;
		MaxIterations = 10;
		bStartFromTail = true;
		bPropagateToChildren = true;
		BaseRotationLimit = 30.f;
	}

	/**
	 * The first bone in the chain to solve
	 */
	UPROPERTY(meta = (Input))
	FName StartBone;

	/**
	 * The last bone in the chain to solve - the effector
	 */
	UPROPERTY(meta = (Input))
	FName EffectorBone;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FTransform EffectorTransform;

	/**
	 * The precision to use for the CCDIK solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	/**
	 * If set to true the direction of the solvers is flipped.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bStartFromTail;

	/**
	 * The general rotation limit to be applied to bones
	 */
	UPROPERTY(meta = (Input, Constant))
	float BaseRotationLimit;

	/**
	 * Defines the limits of rotation per bone.
	 */
	UPROPERTY(meta = (Input, Constant))
	TArray<FRigUnit_CCDIK_RotationLimit> RotationLimits;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY()
	FRigUnit_CCDIK_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * The CCDIK solver can solve N-Bone chains using 
 * the Cyclic Coordinate Descent Inverse Kinematics algorithm.
 * For now this node supports single effector chains only.
 */
USTRUCT(meta=(DisplayName="CCDIK", Category="Hierarchy", Keywords="N-Bone,IK", Deprecated = "5.0"))
struct FRigUnit_CCDIKPerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_CCDIKPerItem()
	{
		EffectorTransform = FTransform::Identity;
		Precision = 1.f;
		Weight = 1.f;
		MaxIterations = 10;
		bStartFromTail = true;
		bPropagateToChildren = true;
		BaseRotationLimit = 30.f;
	}

	/**
	 * The chain to use
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKeyCollection Items;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FTransform EffectorTransform;

	/**
	 * The precision to use for the CCDIK solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	/**
	 * If set to true the direction of the solvers is flipped.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bStartFromTail;

	/**
	 * The general rotation limit to be applied to bones
	 */
	UPROPERTY(meta = (Input, Constant))
	float BaseRotationLimit;

	/**
	 * Defines the limits of rotation per bone.
	 */
	UPROPERTY(meta = (Input, Constant))
	TArray<FRigUnit_CCDIK_RotationLimitPerItem> RotationLimits;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY()
	FRigUnit_CCDIK_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * The CCDIK solver can solve N-Bone chains using 
 * the Cyclic Coordinate Descent Inverse Kinematics algorithm.
 * For now this node supports single effector chains only.
 */
USTRUCT(meta=(DisplayName="CCDIK", Category="Hierarchy", Keywords="N-Bone,IK", NodeColor = "0 1 1"))
struct FRigUnit_CCDIKItemArray : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_CCDIKItemArray()
	{
		EffectorTransform = FTransform::Identity;
		Precision = 1.f;
		Weight = 1.f;
		MaxIterations = 10;
		bStartFromTail = true;
		bPropagateToChildren = true;
		BaseRotationLimit = 30.f;
	}

	/**
	 * The chain to use
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigElementKey> Items;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FTransform EffectorTransform;

	/**
	 * The precision to use for the CCDIK solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	/**
	 * If set to true the direction of the solvers is flipped.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bStartFromTail;

	/**
	 * The general rotation limit to be applied to bones
	 */
	UPROPERTY(meta = (Input, Constant))
	float BaseRotationLimit;

	/**
	 * Defines the limits of rotation per bone.
	 */
	UPROPERTY(meta = (Input, Constant))
	TArray<FRigUnit_CCDIK_RotationLimitPerItem> RotationLimits;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY()
	FRigUnit_CCDIK_WorkData WorkData;
};

#undef UE_API
