// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_ChainInfo.generated.h"

USTRUCT()
struct FRigUnit_ChainInfo_Segment 
{
    GENERATED_BODY()

	FRigUnit_ChainInfo_Segment()
		:StartItem(FCachedRigElement())
		,StartItemIndex(0)
		,EndItem(FCachedRigElement())
		,EndItemIndex(0)
		,InitialLength(0.f)
		,InitialCumLength(0.f)
		,Length(0.f)
		,CumLength(0.f)
	{}

	/*
	* Start item of the chain segment
	*/
	UPROPERTY()
	FCachedRigElement StartItem;

	/*
	* Start item index of the chain segment
	*/
	UPROPERTY()
	int StartItemIndex;

	/*
	* Emd item of the chain segment
	*/
	UPROPERTY()
	FCachedRigElement EndItem;

	/*
	* Emd item index of the chain segment
	*/
	UPROPERTY()
	int EndItemIndex;

	/*
	* Initial length of segment
	*/
	UPROPERTY()
	float InitialLength; 

	/*
	* Inclusive initial length of all previous segments and this one
	*/
	UPROPERTY()
	float InitialCumLength;

	/*
	* Length of segment
	*/
	UPROPERTY()
	float Length; 

	/*
	* Inclusive length of all previous segments and this one
	*/
	UPROPERTY()
	float CumLength;
};

USTRUCT(BlueprintType)
struct FRigUnit_ChainInfo_SegmentInfo 
{
    GENERATED_BODY()

	FRigUnit_ChainInfo_SegmentInfo()
		:SegmentIndex(0)
		,SegmentLength(0.f)	
		,SegmentParam(0.f)
		,SegmentParamLength(0.f)
		,SegmentStartItem(NAME_None, ERigElementType::Bone)
		,SegmentStartItemIndex(0)
		,SegmentEndItem(NAME_None, ERigElementType::Bone)
        ,SegmentEndItemIndex(0)
		,SegmentStretchFactor(1.f)
	{}

	/* 
	* The current segment index
	*/
	UPROPERTY()
	int SegmentIndex;
 
	/* 
	* The current segment length
	*/
	UPROPERTY()
	float SegmentLength;	
 
	/* 
	* The current segment parameter from 0 to 1 
	*/
	UPROPERTY()
	float SegmentParam;
 
	/* 
	* Local segment length of segment param
	*/
	UPROPERTY()
	float SegmentParamLength;
 
	/* 
	* The item starting the current segment 
	*/
	UPROPERTY()
	FRigElementKey SegmentStartItem;
 
	/* 
	* The item index starting the current segment 
	*/
	UPROPERTY()
	int SegmentStartItemIndex;
 
	/* 
	* The item ending the current segment 
	*/
	UPROPERTY()
	FRigElementKey SegmentEndItem;
 
	/* 
	* The item index ending the current segment 
	*/
	UPROPERTY()
	int SegmentEndItemIndex;

	/* 
	* Stretch factor of current segment
	*/
	UPROPERTY()
	float SegmentStretchFactor;
};

/*
* Retrieves various pieces of info about an interpolated transform hierarchy from an rig element item list
*/
USTRUCT(meta=(DisplayName="Chain Info", Category="Hierarchy", Keywords="Chain"))
struct FRigUnit_ChainInfo : public FRigUnit_HighlevelBase
{
	GENERATED_BODY()

	FRigUnit_ChainInfo()
		:Param(0.f)
		,bCalculateStretch(true)
		,bInitial(false)
		,bDebug(false)
		,DebugScale(1.f)
		,InterpolatedTransform(FTransform::Identity)
		,ChainLength(0.f)
		,ParamLength(0.f)
		,ChainStretchFactor(1.f)
        ,SegmentInfo(FRigUnit_ChainInfo_SegmentInfo())
	{}

	RIGVM_METHOD()
	virtual void Execute() override;

	/*
	* The items to use to interpret the chain
	*/
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/*
	* The parameter value down the chain of items from 0 to 1
	*/	
	UPROPERTY(meta = (Input, ClampMin=0.f, ClampMax=1.f))
	float Param;

	/*
	* If True calculate stretch factors of chain and current segment
	*/	
	UPROPERTY(meta = (Input))
	bool bCalculateStretch;

	/*
	* If True use initial transform values for chain
	*/	
	UPROPERTY(meta = (Input))
	bool bInitial;

	/*
	* Enable debug draw for node
	*/	
	UPROPERTY(meta = (Input))
	bool bDebug;

	/*
	* Debug draw scale
	*/	
	UPROPERTY(meta = (Input, UIMin = 0.f))
	float DebugScale;

    /* 
	* The interpolated transform at the chain's input parameter
	*/
	UPROPERTY(meta = (Output))
	FTransform InterpolatedTransform;

    /* 
	* The length of the interpolated chain
	*/
	UPROPERTY(meta = (Output))
	float ChainLength;

    /* 
	* The parametric length of the interpolated chain
	*/
	UPROPERTY(meta = (Output))
	float ParamLength;
    
	/* 
	* Stretch factor of chain
	*/
	UPROPERTY(meta = (Output))
	float ChainStretchFactor;

	/* 
	* Segment Info
	*/
	UPROPERTY(meta = (Output))
	FRigUnit_ChainInfo_SegmentInfo SegmentInfo;

	// Used to cache the internally used index
	UPROPERTY()
	TArray<FCachedRigElement> CachedElements;

};
