// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_DistributeRotation.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_DistributeRotation_Rotation
{
	GENERATED_BODY()

	FRigUnit_DistributeRotation_Rotation()
	{
		Rotation = FQuat::Identity;
		Ratio = 0.f;
	}

	/**
	 * The rotation to be applied
	 */
	UPROPERTY(meta = (Input))
	FQuat Rotation;

	/**
	 * The ratio of where this rotation sits along the chain
	 */
	UPROPERTY(meta = (Input, Constant))
	float Ratio;
};

USTRUCT()
struct FRigUnit_DistributeRotation_WorkData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	TArray<int32> ItemRotationA;

	UPROPERTY()
	TArray<int32> ItemRotationB;

	UPROPERTY()
	TArray<float> ItemRotationT;

	UPROPERTY()
	TArray<FTransform> ItemLocalTransforms;
};

/**
 * Distributes rotations provided along a chain.
 * Each rotation is expressed by a quaternion and a ratio, where the ratio is between 0.0 and 1.0
 * Note: This node adds rotation in local space of each bone!
 */
USTRUCT(meta=(DisplayName="Distribute Rotation", Category="Hierarchy", Keywords="TwistBones", Deprecated = "4.25"))
struct FRigUnit_DistributeRotation : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DistributeRotation()
	{
		StartBone = EndBone = NAME_None;
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the first bone to align
	 */
	UPROPERTY(meta = (Input))
	FName StartBone;

	/** 
	 * The name of the last bone to align
	 */
	UPROPERTY(meta = (Input))
	FName EndBone;

	/** 
	 * The list of rotations to be applied
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_DistributeRotation_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */	
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_DistributeRotation_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Distributes rotations provided across a collection of items.
 * Each rotation is expressed by a quaternion and a ratio, where the ratio is between 0.0 and 1.0
 * Note: This node adds rotation in local space of each item!
 */
USTRUCT(meta=(DisplayName="Distribute Rotation", Category="Hierarchy", Keywords="TwistBones", Deprecated = "5.0"))
struct FRigUnit_DistributeRotationForCollection : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DistributeRotationForCollection()
	{
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to use to distribute the rotation
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/** 
	 * The list of rotations to be applied
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_DistributeRotation_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */	
	UPROPERTY(meta = (Input))
	float Weight;

	UPROPERTY(transient)
	FRigUnit_DistributeRotation_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Distributes rotations provided across a array of items.
 * Each rotation is expressed by a quaternion and a ratio, where the ratio is between 0.0 and 1.0
 * Note: This node adds rotation in local space of each item!
 */
USTRUCT(meta=(DisplayName="Distribute Rotation", Category="Hierarchy", Keywords="TwistBones"))
struct FRigUnit_DistributeRotationForItemArray : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_DistributeRotationForItemArray()
	{
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to use to distribute the rotation
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/** 
	 * The list of rotations to be applied
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_DistributeRotation_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */	
	UPROPERTY(meta = (Input))
	float Weight;

	UPROPERTY(transient)
	FRigUnit_DistributeRotation_WorkData WorkData;
};

#undef UE_API
