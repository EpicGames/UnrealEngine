// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "FABRIK.h"
#include "RigUnit_FABRIK.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_FABRIK_WorkData
{
	GENERATED_BODY()

	FRigUnit_FABRIK_WorkData()
	{
		CachedEffector = FCachedRigElement();
	}

	UPROPERTY()
	TArray<FFABRIKChainLink> Chain;

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	FCachedRigElement CachedEffector;
};

/**
 * The FABRIK solver can solve N-Bone chains using 
 * the Forward and Backward Reaching Inverse Kinematics algorithm.
 * For now this node supports single effector chains only.
 */
USTRUCT(meta=(DisplayName="Basic FABRIK", Category="Hierarchy", Keywords="N-Bone,IK", Deprecated = "4.25"))
struct FRigUnit_FABRIK : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_FABRIK()
	{
		Precision = 1.f;
		Weight = 1.f;
		MaxIterations = 10;
		EffectorTransform = FTransform::Identity;
		bPropagateToChildren = true;
		bSetEffectorTransform = true;
	}

	/**
	 * The first bone in the chain to solve
	 */
	UPROPERTY(meta = (Input))
	FName StartBone;

	/**
	 * The last bone in the chain to solve - the effector
	 */
	UPROPERTY(meta = (Input))
	FName EffectorBone;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FTransform EffectorTransform;

	/**
	 * The precision to use for the fabrik solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	UPROPERTY(transient)
	FRigUnit_FABRIK_WorkData WorkData;

	/**
	* The option to set the effector transform
	*/
	UPROPERTY(meta = (Input))
	bool bSetEffectorTransform;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * The FABRIK solver can solve N-Bone chains using 
 * the Forward and Backward Reaching Inverse Kinematics algorithm.
 * For now this node supports single effector chains only.
 */
USTRUCT(meta=(DisplayName="Basic FABRIK", Category="Hierarchy", Keywords="N-Bone,IK", Deprecated = "5.0"))
struct FRigUnit_FABRIKPerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_FABRIKPerItem()
	{
		Precision = 1.f;
		Weight = 1.f;
		bPropagateToChildren = true;
		MaxIterations = 10;
		EffectorTransform = FTransform::Identity;
		bSetEffectorTransform = true;
	}

	/**
	 * The chain to use
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKeyCollection Items;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FTransform EffectorTransform;

	/**
	 * The precision to use for the fabrik solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	UPROPERTY(transient)
	FRigUnit_FABRIK_WorkData WorkData;

	/**
	* The option to set the effector transform
	*/
	UPROPERTY(meta = (Input))
	bool bSetEffectorTransform;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * The FABRIK solver can solve N-Bone chains using 
 * the Forward and Backward Reaching Inverse Kinematics algorithm.
 * For now this node supports single effector chains only.
 */
USTRUCT(meta=(DisplayName="Basic FABRIK", Category="Hierarchy", Keywords="N-Bone,IK", NodeColor="0 1 1"))
struct FRigUnit_FABRIKItemArray : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_FABRIKItemArray()
	{
		Precision = 1.f;
		Weight = 1.f;
		bPropagateToChildren = true;
		MaxIterations = 10;
		EffectorTransform = FTransform::Identity;
		bSetEffectorTransform = true;
	}

	/**
	 * The chain to use
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	TArray<FRigElementKey> Items;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FTransform EffectorTransform;

	/**
	 * The precision to use for the fabrik solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	UPROPERTY(transient)
	FRigUnit_FABRIK_WorkData WorkData;
	
	/**
	* The option to set the effector transform
	*/
	UPROPERTY(meta = (Input))
	bool bSetEffectorTransform;
};

#undef UE_API
