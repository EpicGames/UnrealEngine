// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_FitChainToCurve.generated.h"

#define UE_API CONTROLRIG_API

UENUM()
enum class EControlRigCurveAlignment : uint8
{
	Front,
	Stretched
};

USTRUCT()
struct FRigUnit_FitChainToCurve_Rotation
{
	GENERATED_BODY()

	FRigUnit_FitChainToCurve_Rotation()
	{
		Rotation = FQuat::Identity;
		Ratio = 0.f;
	}

	/**
	 * The rotation to be applied
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "Rotation")
	FQuat Rotation;

	/**
	 * The ratio of where this rotation sits along the chain
	 */
	UPROPERTY(EditAnywhere, meta = (Input, Constant), Category = "Rotation")
	float Ratio;
};

USTRUCT()
struct FRigUnit_FitChainToCurve_DebugSettings
{
	GENERATED_BODY()

	FRigUnit_FitChainToCurve_DebugSettings()
	{
		bEnabled = false;
		Scale = 1.f;
		CurveColor = FLinearColor::Yellow;
		SegmentsColor= FLinearColor::Red;
		WorldOffset = FTransform::Identity;
	}

	/**
	 * If enabled debug information will be drawn
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "DebugSettings")
	bool bEnabled;

	/**
	 * The size of the debug drawing information
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	float Scale;

	/**
	 * The color to use for debug drawing
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FLinearColor CurveColor;

	/**
	 * The color to use for debug drawing
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FLinearColor SegmentsColor;

	/**
	 * The offset at which to draw the debug information in the world
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FTransform WorldOffset;
};

USTRUCT()
struct FRigUnit_FitChainToCurve_WorkData
{
	GENERATED_BODY()

	FRigUnit_FitChainToCurve_WorkData()
	{
		ChainLength = 0.f;
	}

	UPROPERTY()
	float ChainLength;

	UPROPERTY()
	TArray<FVector> ItemPositions;

	UPROPERTY()
	TArray<float> ItemSegments;

	UPROPERTY()
	TArray<FVector> CurvePositions;

	UPROPERTY()
	TArray<float> CurveSegments;

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	TArray<int32> ItemRotationA;

	UPROPERTY()
	TArray<int32> ItemRotationB;

	UPROPERTY()
	TArray<float> ItemRotationT;

	UPROPERTY()
	TArray<FTransform> ItemLocalTransforms;
};

/**
 * Fits a given chain to a four point bezier curve.
 * Additionally provides rotational control matching the features of the Distribute Rotation node.
 */
USTRUCT(meta=(DisplayName="Fit Chain on Curve", Category="Hierarchy", Keywords="Fit,Resample,Bezier", Deprecated = "4.25"))
struct FRigUnit_FitChainToCurve : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitChainToCurve()
	{
		StartBone = EndBone = NAME_None;
		Bezier = FRigVMFourPointBezier();
		Alignment = EControlRigCurveAlignment::Stretched;
		Minimum = 0.f;
		Maximum = 1.f;
		SamplingPrecision = 12;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 0.f, 0.f);
		PoleVectorPosition = FVector::ZeroVector;
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_FitChainToCurve_DebugSettings();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the first bone to align
	 */
	UPROPERTY(meta = (Input))
	FName StartBone;

	/** 
	 * The name of the last bone to align
	 */
	UPROPERTY(meta = (Input))
	FName EndBone;

	/** 
	 * The curve to align to
	 */
	UPROPERTY(meta = (Input))
	FRigVMFourPointBezier Bezier;

	/** 
	 * Specifies how to align the chain on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigCurveAlignment Alignment;

	/** 
	 * The minimum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Minimum;

	/** 
	 * The maximum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Maximum;

	/**
	 * The number of samples to use on the curve. Clamped at 64.
	 */
	UPROPERTY(meta = (Input, Constant))
	int32 SamplingPrecision;

	/**
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/**
	 * The minor axis being aligned - towards the pole vector.
	 * You can use (0.0, 0.0, 0.0) to disable it.
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * The the position of the pole vector used for aligning the secondary axis.
	 * Only has an effect if the secondary axis is set.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVectorPosition;

	/** 
	 * The list of rotations to be applied along the curve
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_FitChainToCurve_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_FitChainToCurve_DebugSettings DebugSettings;

	UPROPERTY(transient)
	FRigUnit_FitChainToCurve_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Fits a given chain to a four point bezier curve.
 * Additionally provides rotational control matching the features of the Distribute Rotation node.
 */
USTRUCT(meta=(DisplayName="Fit Chain on Curve", Category="Hierarchy", Keywords="Fit,Resample,Bezier", Deprecated = "5.0"))
struct FRigUnit_FitChainToCurvePerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitChainToCurvePerItem()
	{
		Bezier = FRigVMFourPointBezier();
		Alignment = EControlRigCurveAlignment::Stretched;
		Minimum = 0.f;
		Maximum = 1.f;
		SamplingPrecision = 12;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 0.f, 0.f);
		PoleVectorPosition = FVector::ZeroVector;
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_FitChainToCurve_DebugSettings();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/** 
	 * The curve to align to
	 */
	UPROPERTY(meta = (Input))
	FRigVMFourPointBezier Bezier;

	/** 
	 * Specifies how to align the chain on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigCurveAlignment Alignment;

	/** 
	 * The minimum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Minimum;

	/** 
	 * The maximum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Maximum;

	/**
	 * The number of samples to use on the curve. Clamped at 64.
	 */
	UPROPERTY(meta = (Input, Constant))
	int32 SamplingPrecision;

	/**
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/**
	 * The minor axis being aligned - towards the pole vector.
	 * You can use (0.0, 0.0, 0.0) to disable it.
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * The the position of the pole vector used for aligning the secondary axis.
	 * Only has an effect if the secondary axis is set.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVectorPosition;

	/** 
	 * The list of rotations to be applied along the curve
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_FitChainToCurve_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_FitChainToCurve_DebugSettings DebugSettings;

	UPROPERTY(transient)
	FRigUnit_FitChainToCurve_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Fits a given chain to a four point bezier curve.
 * Additionally provides rotational control matching the features of the Distribute Rotation node.
 */
USTRUCT(meta=(DisplayName="Fit Chain on Curve", Category="Hierarchy", Keywords="Fit,Resample,Bezier", Deprecated = "5.0"))
struct FRigUnit_FitChainToCurveItemArray : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_FitChainToCurveItemArray()
	{
		Bezier = FRigVMFourPointBezier();
		Alignment = EControlRigCurveAlignment::Stretched;
		Minimum = 0.f;
		Maximum = 1.f;
		SamplingPrecision = 12;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 0.f, 0.f);
		PoleVectorPosition = FVector::ZeroVector;
		RotationEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_FitChainToCurve_DebugSettings();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to align
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/** 
	 * The curve to align to
	 */
	UPROPERTY(meta = (Input))
	FRigVMFourPointBezier Bezier;

	/** 
	 * Specifies how to align the chain on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigCurveAlignment Alignment;

	/** 
	 * The minimum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Minimum;

	/** 
	 * The maximum U value to use on the curve
	 */
	UPROPERTY(meta = (Input, Constant))
	float Maximum;

	/**
	 * The number of samples to use on the curve. Clamped at 64.
	 */
	UPROPERTY(meta = (Input, Constant))
	int32 SamplingPrecision;

	/**
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/**
	 * The minor axis being aligned - towards the pole vector.
	 * You can use (0.0, 0.0, 0.0) to disable it.
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * The the position of the pole vector used for aligning the secondary axis.
	 * Only has an effect if the secondary axis is set.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVectorPosition;

	/** 
	 * The list of rotations to be applied along the curve
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_FitChainToCurve_Rotation> Rotations;

	/**
	 * The easing to use between to rotations.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType RotationEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_FitChainToCurve_DebugSettings DebugSettings;

	UPROPERTY(transient)
	FRigUnit_FitChainToCurve_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
