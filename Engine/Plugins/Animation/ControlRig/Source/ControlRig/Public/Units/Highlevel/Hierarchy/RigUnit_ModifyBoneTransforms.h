// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "RigUnit_ModifyTransforms.h"
#include "RigUnit_ModifyBoneTransforms.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_ModifyBoneTransforms_PerBone
{
	GENERATED_BODY()

	FRigUnit_ModifyBoneTransforms_PerBone()
		: Bone(NAME_None)
	{
	}

	/**
	 * The name of the Bone to set the transform for.
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = FRigUnit_ModifyBoneTransforms_PerBone)
	FName Bone;

	/**
	 * The transform value to set for the given Bone.
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = FRigUnit_ModifyBoneTransforms_PerBone)
	FTransform Transform;
};

USTRUCT()
struct FRigUnit_ModifyBoneTransforms_WorkData : public FRigUnit_ModifyTransforms_WorkData
{
	GENERATED_BODY()
};

/**
 * ModifyBonetransforms is used to perform a change in the hierarchy by setting one or more bones' transforms.
 */
USTRUCT(meta=(DisplayName="Modify Transforms", Category="Hierarchy", DocumentationPolicy="Strict", Keywords = "ModifyBone", Deprecated = "4.25"))
struct FRigUnit_ModifyBoneTransforms : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ModifyBoneTransforms()
		: Weight(1.f)
		, WeightMinimum(0.f)
		, WeightMaximum(1.f)
		, Mode(EControlRigModifyBoneMode::AdditiveLocal)
	{
		BoneToModify.Add(FRigUnit_ModifyBoneTransforms_PerBone());
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The bones to modify.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault, DefaultArraySize=1))
	TArray<FRigUnit_ModifyBoneTransforms_PerBone> BoneToModify;

	/**
	 * At 1 this sets the transform, between 0 and 1 the transform is blended with previous results.
	 */
	UPROPERTY(meta = (Input, ClampMin=0.f, ClampMax=1.f, UIMin = 0.f, UIMax = 1.f))
	float Weight;

	/**
	 * The minimum of the weight - defaults to 0.0
	 */
	UPROPERTY(meta = (Input, Constant, ClampMin = 0.f, ClampMax = 1.f, UIMin = 0.f, UIMax = 1.f))
	float WeightMinimum;

	/**
	 * The maximum of the weight - defaults to 1.0
	 */
	UPROPERTY(meta = (Input, Constant, ClampMin = 0.f, ClampMax = 1.f, UIMin = 0.f, UIMax = 1.f))
	float WeightMaximum;

	/**
	 * Defines if the bone's transform should be set
	 * in local or global space, additive or override.
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigModifyBoneMode Mode;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FRigUnit_ModifyBoneTransforms_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
