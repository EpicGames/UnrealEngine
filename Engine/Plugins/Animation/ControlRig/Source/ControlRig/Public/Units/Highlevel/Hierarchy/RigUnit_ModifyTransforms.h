// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "RigUnit_ModifyTransforms.generated.h"

#define UE_API CONTROLRIG_API

UENUM()
enum class EControlRigModifyBoneMode : uint8
{
	/** Override existing local transform */
	OverrideLocal,

	/** Override existing global transform */
	OverrideGlobal,

	/** 
	 * Additive to existing local transform.
	 * Input transform is added within the bone's space.
	 */
	AdditiveLocal,

	/**
     * Additive to existing global transform.
     * Input transform is added as a global offset in the root of the hierarchy.
	 */
	AdditiveGlobal,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

USTRUCT()
struct FRigUnit_ModifyTransforms_PerItem
{
	GENERATED_BODY()

	FRigUnit_ModifyTransforms_PerItem()
		: Item(FRigElementKey(NAME_None, ERigElementType::Bone))
	{
	}

	/**
	 * The item to set the transform for.
	 */
	UPROPERTY(EditAnywhere, meta = (Input, ExpandByDefault), Category = FRigUnit_ModifyTransforms_PerItem)
	FRigElementKey Item;

	/**
	 * The transform value to set for the given Bone.
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = FRigUnit_ModifyTransforms_PerItem)
	FTransform Transform;
};

USTRUCT()
struct FRigUnit_ModifyTransforms_WorkData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;
};

/**
 * Modify Transforms is used to perform a change in the hierarchy by setting one or more bones' transforms
 */
USTRUCT(meta=(DisplayName="Modify Transforms", Category="Transforms", DocumentationPolicy="Strict", Keywords = "ModifyBone"))
struct FRigUnit_ModifyTransforms : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ModifyTransforms()
		: Weight(1.f)
		, WeightMinimum(0.f)
		, WeightMaximum(1.f)
		, Mode(EControlRigModifyBoneMode::AdditiveLocal)
	{
		ItemToModify.Add(FRigUnit_ModifyTransforms_PerItem());
		ItemToModify[0].Item = FRigElementKey(NAME_None, ERigElementType::Bone);
	}

#if WITH_EDITOR
	UE_API virtual bool GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const override;
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual TArray<const URigVMPin*> GetPinsForDirectManipulation(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const override;
	UE_API int32 GetIndexFromTarget(const FString& InTarget) const;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The items to modify.
	 */
	UPROPERTY(meta = (Input, ExpandByDefault, DefaultArraySize=1))
	TArray<FRigUnit_ModifyTransforms_PerItem> ItemToModify;

	/**
	 * At 1 this sets the transform, between 0 and 1 the transform is blended with previous results.
	 */
	UPROPERTY(meta = (Input, ClampMin=0.f, ClampMax=1.f, UIMin = 0.f, UIMax = 1.f))
	float Weight;

	/**
	 * The minimum of the weight - defaults to 0.0
	 */
	UPROPERTY(meta = (Input, Constant, ClampMin = 0.f, ClampMax = 1.f, UIMin = 0.f, UIMax = 1.f))
	float WeightMinimum;

	/**
	 * The maximum of the weight - defaults to 1.0
	 */
	UPROPERTY(meta = (Input, Constant, ClampMin = 0.f, ClampMax = 1.f, UIMin = 0.f, UIMax = 1.f))
	float WeightMaximum;

	/**
	 * Defines if the bone's transform should be set
	 * in local or global space, additive or override.
	 */
	UPROPERTY(meta = (Input, Constant))
	EControlRigModifyBoneMode Mode;

	// Used to cache the internally used bone index
	UPROPERTY(transient)
	FRigUnit_ModifyTransforms_WorkData WorkData;
};

#undef UE_API
