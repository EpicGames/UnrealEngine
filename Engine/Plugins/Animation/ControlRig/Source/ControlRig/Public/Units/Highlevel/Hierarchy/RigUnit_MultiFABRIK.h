// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "FABRIK.h"
#include "RigUnit_MultiFABRIK.generated.h"

#define UE_API CONTROLRIG_API

/* This is each element WIP transform
 * while iterating, we use this as bone transform data
 */
struct FRigUnit_MultiFABRIK_BoneWorkingData
{
	FName BoneName;
	FCachedRigElement BoneIndex; // bone index of hierarchy
	FCachedRigElement ParentIndex; // bone index of hierarchy
	int32 ParentTreeIndex; // parent index of BoneTree 
	TArray<int32> ChildrenTreeIndices;
	float BoneLength; // initial bone length 
	FVector Location; // updated every frame

	FRigUnit_MultiFABRIK_BoneWorkingData()
		: BoneName(NAME_None)
		, BoneIndex(FCachedRigElement())
		, ParentIndex(FCachedRigElement())
		, ParentTreeIndex(INDEX_NONE)
		, BoneLength(0.f)
	{}
};

/* 
 * Describes each chain. This contains list of chain and effector location data and what's desired max length 
 */
struct FRigUnit_MultiFABRIK_Chain
{
	TArray<FFABRIKChainLink> Link;
	FVector EffectorLocation; // effector location - based on where you're this location could change
	int32 EffectorArrayIndex;
	float MaxLength;

	FRigUnit_MultiFABRIK_Chain()
		: EffectorArrayIndex(INDEX_NONE)
		, MaxLength(0.f)
	{}

	void Reset()
	{
		Link.Reset();
		EffectorLocation = FVector::ZeroVector;
		EffectorArrayIndex = INDEX_NONE;
		MaxLength = 0.f;
	}
};

/** 
 * Contains groups of Chains 
 * This also contains base bone tree index of the root
 */
struct FRigUnit_MultiFABRIK_ChainGroup
{
	// current chain data
	FRigUnit_MultiFABRIK_Chain Chain;
	// root bone index for this specific chain - this index is 
	int32 RootBoneTreeIndex;

	// children of this chain
	// this means these are children AT THE END OF THE CHAIN
	TArray<FRigUnit_MultiFABRIK_ChainGroup> Children;

	FRigUnit_MultiFABRIK_ChainGroup()
		: RootBoneTreeIndex(INDEX_NONE)
	{}

	void Reset()
	{
		Chain.Reset();
		Children.Reset();
	}
};

USTRUCT()
struct FRigUnit_MultiFABRIK_WorkData
{
	GENERATED_BODY()

	FRigUnit_MultiFABRIK_WorkData()
	{
	}

	// list of effector bone indices
	TArray<FCachedRigElement> EffectorBoneIndices;

	// chain groups - contains list of sub groups (Children)
	FRigUnit_MultiFABRIK_ChainGroup	ChainGroup;

	// bone tree for WIP data structure
	TArray<FRigUnit_MultiFABRIK_BoneWorkingData> BoneTree;
};

USTRUCT()
struct FRigUnit_MultiFABRIK_EndEffector
{
	GENERATED_BODY()
	/**
	 * The last bone in the chain to solve - the effector
	 */
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	FName Bone;

	/**
	 * The transform of the effector in global space
	 */
	UPROPERTY(meta = (Input))
	FVector Location = FVector(0.f);
};
/**
 * The FABRIK solver can solve multi chains within a root using multi effectors
 * the Forward and Backward Reaching Inverse Kinematics algorithm.
 */
USTRUCT(meta=(DisplayName="Multi Effector FABRIK", Category="Hierarchy", Keywords="Multi, Effector, N-Chain,IK", NodeColor="0 1 1"))
struct FRigUnit_MultiFABRIK : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_MultiFABRIK()
	{
		Precision = 1.f;
		MaxIterations = 4;
		bPropagateToChildren = true;
		bIsInitialized = false;
	}

	virtual void Initialize() override
	{
		bIsInitialized = false;
	}

	/**
	 * The first bone in the chain to solve
	 */
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	FName RootBone;

	// The list of effectors to use for the solver
	UPROPERTY(meta = (Input))
	TArray<FRigUnit_MultiFABRIK_EndEffector> Effectors;

	/**
	 * The precision to use for the fabrik solver
	 */
	UPROPERTY(meta = (Input, Constant))
	float Precision;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input))
	bool bPropagateToChildren = true;

	/**
	 * The maximum number of iterations. Values between 4 and 16 are common.
	 */
	UPROPERTY(meta = (Input))
	int32 MaxIterations;

	UPROPERTY(transient)
	FRigUnit_MultiFABRIK_WorkData WorkData;

	UPROPERTY(transient)
	bool bIsInitialized;
};

#undef UE_API
