// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_SlideChain.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_SlideChain_WorkData
{
	GENERATED_BODY()

	FRigUnit_SlideChain_WorkData()
	{
		ChainLength = 0.f;
	}

	UPROPERTY()
	float ChainLength;

	UPROPERTY()
	TArray<float> ItemSegments;

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	TArray<FTransform> Transforms;

	UPROPERTY()
	TArray<FTransform> BlendedTransforms;
};

/**
 * Slides an existing chain along itself with control over extrapolation.
 */
USTRUCT(meta=(DisplayName="Slide Chain", Category="Hierarchy", Keywords="Fit,Refit", Deprecated = "4.25"))
struct FRigUnit_SlideChain: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SlideChain()
	{
		StartBone = EndBone = NAME_None;
		SlideAmount = 0.f;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the first bone to slide
	 */
	UPROPERTY(meta = (Input))
	FName StartBone;

	/** 
	 * The name of the last bone to slide
	 */
	UPROPERTY(meta = (Input))
	FName EndBone;

	/** 
	 * The amount of sliding. This unit is multiple of the chain length.
	 */
	UPROPERTY(meta = (Input))
	float SlideAmount;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_SlideChain_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Slides an existing chain along itself with control over extrapolation.
 */
USTRUCT(meta=(DisplayName="Slide Chain", Category="Hierarchy", Keywords="Fit,Refit", Deprecated = "5.0"))
struct FRigUnit_SlideChainPerItem: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SlideChainPerItem()
	{
		SlideAmount = 0.f;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to slide
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/** 
	 * The amount of sliding. This unit is multiple of the chain length.
	 */
	UPROPERTY(meta = (Input))
	float SlideAmount;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_SlideChain_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Slides an existing chain along itself with control over extrapolation.
 */
USTRUCT(meta=(DisplayName="Slide Chain", Category="Hierarchy", Keywords="Fit,Refit"))
struct FRigUnit_SlideChainItemArray: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SlideChainItemArray()
	{
		SlideAmount = 0.f;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to slide
	 */
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> Items;

	/** 
	 * The amount of sliding. This unit is multiple of the chain length.
	 */
	UPROPERTY(meta = (Input))
	float SlideAmount;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_SlideChain_WorkData WorkData;
};

#undef UE_API
