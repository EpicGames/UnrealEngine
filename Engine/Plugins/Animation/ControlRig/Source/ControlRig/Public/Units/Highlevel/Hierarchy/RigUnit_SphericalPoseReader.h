// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "RigUnit_SphericalPoseReader.generated.h"

#define UE_API CONTROLRIG_API


struct FEllipseQuery
{
	float ClosestX = 0.0f;
	float ClosestY = 0.0f;
	float DistSq = 0.0f;
	bool IsInside = false;
};

USTRUCT()
struct FRegionScaleFactors
{
	GENERATED_BODY()

	FRegionScaleFactors()
	: PositiveWidth(1.0f),
	NegativeWidth(1.0f),
	PositiveHeight(1.0f),
	NegativeHeight(1.0f)
	{
	}

	FRegionScaleFactors(float InPositiveWidth, float InNegativeWidth, float InPositiveHeight, float InNegativeHeight)
		: PositiveWidth(InPositiveWidth)
		, NegativeWidth(InNegativeWidth)
		, PositiveHeight(InPositiveHeight)
		, NegativeHeight(InNegativeHeight)
	{
	}

	// Scale the region in the POSITIVE width direction. Range is 0-1. Default is 1.0.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositiveWidth = 1.0f;
	// Scale the region in the NEGATIVE width direction. Range is 0-1. Default is 1.0.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float NegativeWidth = 1.0f;
	// Scale the region in the POSITIVE height direction. Range is 0-1. Default is 1.0.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositiveHeight = 1.0f;
	// Scale the region in the NEGATIVE height direction. Range is 0-1. Default is 1.0.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float NegativeHeight = 1.0f;
};

USTRUCT()
struct FSphericalRegion
{
	GENERATED_BODY()

	FSphericalRegion() : RegionAngleRadians(0.1f), ScaleFactors(FRegionScaleFactors()){}
	
	float RegionAngleRadians;
	FRegionScaleFactors ScaleFactors;

	void GetEllipseWidthAndHeight(const float PointX, const float PointY, float& OutWidth, float& OutHeight)
	{
		OutWidth = ((PointX > 0 ? ScaleFactors.PositiveWidth : ScaleFactors.NegativeWidth) * RegionAngleRadians) * INV_PI;
		OutHeight = ((PointY > 0 ? ScaleFactors.PositiveHeight : ScaleFactors.NegativeHeight) * RegionAngleRadians) * INV_PI;
	}
};

USTRUCT()
struct FSphericalPoseReaderDebugSettings
{
	GENERATED_BODY()

	FSphericalPoseReaderDebugSettings()
		: bDrawDebug(true),
		bDraw2D(false),
		DebugScale(25.0f),
		DebugSegments(20),
		DebugThickness(0.25f)
	{
	}

	UPROPERTY(meta = (Input))
	bool bDrawDebug = true;

	UPROPERTY(meta = (Input))
	bool bDraw2D = false;
	
	UPROPERTY(meta = (Input))
	bool bDrawLocalAxes = false;
	
	UPROPERTY(meta = (Input, ClampMin = "0.0", UIMin = "0.01", UIMax = "100.0"))
	float DebugScale = 25.0f;
	
	UPROPERTY(meta = (Input, ClampMin = "0", ClampMax = "50", UIMin = "0", UIMax = "10"))
	int DebugSegments = 20;

	UPROPERTY(meta = (Input, ClampMin = "0", ClampMax = "50", UIMin = "0.01", UIMax = "2.0"))
	float DebugThickness = 0.25f;
	
	void DrawDebug(
		const FTransform WorldOffset,
		FRigVMDrawInterface* DrawInterface,
		const FSphericalRegion& InnerRegion,
		const FSphericalRegion& OuterRegion,
		const FVector DriverNormal,
		const FVector Driver2D,
		const float OutputParam) const
    {
		if (DrawInterface == nullptr || !bDrawDebug)
		{
			return;
		}

		if (bDrawLocalAxes)
		{
			FTransform AxisOffset = WorldOffset;
			AxisOffset.AddToTranslation(AxisOffset.GetUnitAxis(EAxis::Z) * DebugScale);
			DrawInterface->DrawAxes(AxisOffset, FTransform::Identity, DebugScale * 0.5f, DebugThickness);	
		}

		// make colors
		FLinearColor DriverColor = FLinearColor::Red;
		FLinearColor Yellow = FLinearColor::Yellow;
		FLinearColor Green = FLinearColor::Green;
		FLinearColor DarkYellow = FLinearColor::Yellow * .4f;
		FLinearColor DarkGreen = FLinearColor::Green * .2f;
		FLinearColor Blue = FLinearColor::Blue;
		if (OutputParam < SMALL_NUMBER)
		{
			// outside range
			DriverColor = FLinearColor::Red;
			Yellow = DarkYellow;
			Green = DarkGreen;
		}else if (OutputParam >= 0.99f)
		{
			// in green range
			DriverColor = Blue;
			Yellow = DarkYellow;
		}else
		{
			// in yellow range
			DriverColor = Blue;
			Green = DarkGreen;
		}
		
        // draw inner manifold
        TArray<FVector> InnerConePoints;
        TArray<FVector> FlatInnerConePoints;
        DrawManifold(InnerConePoints, FlatInnerConePoints, InnerRegion);
		if (bDraw2D)
		{
			// flat
			DrawInterface->DrawLineStrip(WorldOffset, FlatInnerConePoints, Green, DebugThickness);
		}else
		{
			// spherical
			DrawInterface->DrawLineStrip(WorldOffset, InnerConePoints, Green, DebugThickness);
		}
		
        // draw outer manifold
        TArray<FVector> OuterConePoints;
        TArray<FVector> FlatOuterConePoints;
        DrawManifold(OuterConePoints, FlatOuterConePoints, OuterRegion);
		if (bDraw2D)
		{
			// flat
			DrawInterface->DrawLineStrip(WorldOffset, FlatOuterConePoints, Yellow, DebugThickness);
		}else
		{
			// spherical
			DrawInterface->DrawLineStrip(WorldOffset, OuterConePoints, Yellow, DebugThickness);
		}

		if (!bDraw2D)
		{
			// draw connector lines between inner/outer
			for (int I = 0; I < InnerConePoints.Num(); ++I)
			{
				for (int P = 0; P < DebugSegments; ++P)
				{
					float StartPercent = P / (float)DebugSegments;
					float EndPercent = (P + 1) / (float)DebugSegments;
					FVector StartPoint = FMath::Lerp(InnerConePoints[I], OuterConePoints[I], StartPercent).GetSafeNormal();
					FVector EndPoint = FMath::Lerp(InnerConePoints[I], OuterConePoints[I], EndPercent).GetSafeNormal();
					DrawInterface->DrawLine(WorldOffset, StartPoint * DebugScale, EndPoint * DebugScale, Yellow, DebugThickness);
				}
			}
		
			// draw connector lines from active region to origin
			for (int I = 0; I < InnerConePoints.Num(); ++I)
			{
				for (int P = 0; P < DebugSegments; ++P)
				{
					float StartPercent = P / (float)DebugSegments;
					float EndPercent = (P + 1) / (float)DebugSegments;
					FVector StartPoint = FMath::Lerp(FVector::UpVector * DebugScale, InnerConePoints[I], StartPercent).GetSafeNormal();
					FVector EndPoint = FMath::Lerp(FVector::UpVector * DebugScale, InnerConePoints[I], EndPercent).GetSafeNormal();
					DrawInterface->DrawLine(WorldOffset, StartPoint * DebugScale, EndPoint * DebugScale, Green, DebugThickness);
				}
			}
		}

		if (bDraw2D)
		{
			// draw driver vector
			DrawInterface->DrawPoint(WorldOffset, Driver2D * DebugScale, DebugThickness*5.0f, DriverColor);
		}else
		{
			// draw driver vector
			DrawInterface->DrawLine(WorldOffset, FVector::ZeroVector, DriverNormal * DebugScale, DriverColor, DebugThickness*2.0f);
			DrawInterface->DrawPoint(WorldOffset, DriverNormal * DebugScale, DebugThickness*5.0f, DriverColor);
		}   
    }
	
	void DrawManifold(TArray<FVector>& SphericalPoints, TArray<FVector>& FlatPoints, const FSphericalRegion& Region) const
	{
		const float AngleStep = (PI*2.0f) / FMath::Max(4.0f, (static_cast<float>(DebugSegments) - 1.0f));
		for (int i = 0; i < DebugSegments; ++i)
		{
			// spin around forward vector, drawing each span
			float SpinAngle = (AngleStep * i);
			// get 2d projection in normalized space 0-1
			FVector PointOnPlane = CalcSphericalCoordinates(SpinAngle, Region);
			// get 3d spherical point
			FVector PointOnSphere = SphericalCoordinatesToNormal(PointOnPlane);

			// store points
			PointOnPlane.Z = -1.0f; // move 2d debug view to base of sphere
			SphericalPoints.Add(PointOnSphere * DebugScale);
			FlatPoints.Add(PointOnPlane * DebugScale);
		}
	}

	static FVector CalcSphericalCoordinates(const float SpinAngleRadians, const FSphericalRegion& Region)
	{
		// uniform angle
		const FQuat SpinRot = FQuat(FVector::ZAxisVector, SpinAngleRadians);
		const FVector SpanAxis = SpinRot * FVector::XAxisVector;

		// scale span axis length proportional to amount of rotation towards back of sphere
		FVector Point = SpanAxis * (Region.RegionAngleRadians / PI);

		// stretch cone point non-uniformly
		const float Width = Point.X > 0 ? Region.ScaleFactors.PositiveWidth : Region.ScaleFactors.NegativeWidth;
		const float Height = Point.Y > 0 ? Region.ScaleFactors.PositiveHeight : Region.ScaleFactors.NegativeHeight;
		Point.X *= Width;
		Point.Y *= Height;
		Point.Z = 0.0f;

		return Point;
	}

	static float SignedAngleBetweenNormals(const FVector& From, const FVector& To, const FVector& Axis)
	{
		const float FromDotTo = FVector::DotProduct(From, To);
		const float Angle = FMath::Acos(FromDotTo);
		const FVector Cross = FVector::CrossProduct(From, To);
		const float Dot = FVector::DotProduct(Cross, Axis);
		return Dot >= 0 ? Angle : -Angle;
	}

	static FVector SphericalCoordinatesToNormal(FVector Point2D)
	{
		const float AngleToSquashedPoint = SignedAngleBetweenNormals(Point2D.GetSafeNormal(), FVector::XAxisVector, -FVector::ZAxisVector);
		const FQuat SquashedSpanRot = FQuat(FVector::ZAxisVector, AngleToSquashedPoint);
		const FVector SquashedSpanAxis = SquashedSpanRot * FVector::YAxisVector;
		const float NormalizedMag = Point2D.Size();
		const FQuat ConeRot = FQuat(SquashedSpanAxis, (NormalizedMag * PI));
		const FVector Point3D = ConeRot * FVector::ZAxisVector;
		return Point3D;
	}
};

/*
 * Outputs a float value between 0-1 based off of the driver item's rotation in a specified region.
 */
USTRUCT(meta=(DisplayName="Spherical Pose Reader", Category="Hierarchy", Keywords="Pose Reader, SPR"))
struct FRigUnit_SphericalPoseReader: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SphericalPoseReader()
	: OutputParam(0.0f),
	DriverItem(FRigElementKey(NAME_None, ERigElementType::Bone)),
	DriverAxis(FVector(1.0f,0.0f,0.0f)),
	RotationOffset(FVector(90.0f,0.0f,0.0f)),
	ActiveRegionSize(0.1f),
	ActiveRegionScaleFactors(FRegionScaleFactors()),
	FalloffSize(0.2f),
	FalloffRegionScaleFactors(FRegionScaleFactors()),
	FlipWidthScaling(false),
	FlipHeightScaling(false),
	OptionalParentItem(FRigElementKey(NAME_None, ERigElementType::Bone)),
	Debug(FSphericalPoseReaderDebugSettings()),
	InnerRegion(FSphericalRegion()),
	OuterRegion(FSphericalRegion()),
	DriverNormal(FVector::Zero()),
	Driver2D(FVector::Zero()),
	DriverCache(FCachedRigElement()),
	OptionalParentCache(FCachedRigElement()),
	LocalDriverTransformInit(FTransform::Identity),
	CachedRotationOffset(FVector::ZeroVector),
	bCachedInitTransforms(false)
	{
	}
	
	RIGVM_METHOD()
    UE_API virtual void Execute() override;

#if WITH_EDITOR
	UE_API virtual bool GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const override;
	UE_API virtual void ConfigureDirectManipulationControl(const URigVMUnitNode* InNode, TSharedPtr<FRigDirectManipulationInfo> InInfo, FRigControlSettings& InOutSettings, FRigControlValue& InOutValue) const override;
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif

	static UE_API void RemapAndConvertInputs(
		FSphericalRegion& InnerRegion,
		FSphericalRegion& OuterRegion,
		const float ActiveRegionSize,
		const FRegionScaleFactors& ActiveRegionScaleFactors,
		const float FalloffSize,
		const FRegionScaleFactors& FalloffRegionScaleFactors,
		const bool FlipWidth,
		const bool FlipHeight);
	static UE_API void EvaluateInterpolation(FRigUnit_SphericalPoseReader& Node, const FTransform& DriverTransform);
	
	static UE_API void DrawDebug(FRigUnit_SphericalPoseReader& PoseReader);

	static UE_API float CalcOutputParam(
		const FEllipseQuery& InnerEllipseResults,
		const FEllipseQuery& OuterEllipseResults);
	
	static UE_API void DistanceToEllipse(
		const float InX,
		const float InY,
		const float SizeX,
		const float SizeY,
		FEllipseQuery& OutEllipseQuery);
	
	static UE_API float RemapRange(
		const float T,
		const float AStart,
		const float AEnd,
		const float BStart,
		const float BEnd);

	// The normalized output parameter; ranges from 0 (when outside yellow region) to 1 (in the green region) and smoothly blends from 0-1 in the yellow region.
	UPROPERTY(meta = (Output))
	float OutputParam;
	
	// The bone that will drive the output parameter when rotated into the active regions of this pose reader.
	UPROPERTY(meta = (Input, ExpandByDefault))
    FRigElementKey DriverItem;

	// The axis of the driver transform that is compared against the falloff regions. Typically the axis that is pointing at the child; usually X by convention. Default is X-axis (1,0,0).
	UPROPERTY(meta = (Input))
	FVector DriverAxis;

	// Rotate the entire falloff region to align with the desired area of effect.
	UPROPERTY(meta = (Input))
	FVector RotationOffset;

	// The size of the active region (green) that outputs the full value (1.0). Range is 0-1. Default is 0.1.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
    float ActiveRegionSize;
	// The directional scaling parameters for the active region (green).
	UPROPERTY(meta = (Input))
	FRegionScaleFactors ActiveRegionScaleFactors;

	// The size of the falloff region (yellow) that defines the start of the output range. A value of 1 wraps the entire sphere with falloff. Range is 0-1. Default is 0.2.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
    float FalloffSize;
	// The directional scaling parameters for the falloff region (yellow).
	UPROPERTY(meta = (Input))
	FRegionScaleFactors FalloffRegionScaleFactors;

	// Flip the positive / negative directions of the width scale factors.
	UPROPERTY(meta = (Input))
	bool FlipWidthScaling;

	// Flip the positive / negative directions of the height scale factors.
	UPROPERTY(meta = (Input))
	bool FlipHeightScaling;

	// An optional parent to use as a stable frame of reference for the active regions (defaults to DriverItem's parent if unset).
	UPROPERTY(meta = (Input))
	FRigElementKey OptionalParentItem;

	// The debug settings
	UPROPERTY(meta = (Input))
	FSphericalPoseReaderDebugSettings Debug;

	// private work data
	UPROPERTY(transient)
	FSphericalRegion InnerRegion;
	UPROPERTY(transient)
	FSphericalRegion OuterRegion;
	UPROPERTY(transient)
	FVector DriverNormal;
	UPROPERTY(transient)
	FVector Driver2D;
	
	UPROPERTY()
	FCachedRigElement DriverCache;

	UPROPERTY()
	FCachedRigElement OptionalParentCache;
	
	UPROPERTY()
	FTransform LocalDriverTransformInit;
	UPROPERTY()
	FVector CachedRotationOffset;
	UPROPERTY()
	FRigElementKey CachedParentElementKey;
	UPROPERTY()
	bool bCachedInitTransforms;
};

#undef UE_API
