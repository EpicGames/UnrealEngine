// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/Simulation/CRSimPointContainer.h"
#include "RigUnit_SpringIK.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_SpringIK_DebugSettings
{
	GENERATED_BODY()

	FRigUnit_SpringIK_DebugSettings()
	{
		bEnabled = false;
		Scale = 1.f;
		Color = FLinearColor::Blue;
		WorldOffset = FTransform::Identity;
	}

	/**
	 * If enabled debug information will be drawn 
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "DebugSettings")
	bool bEnabled;

	/**
	 * The size of the debug drawing information
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	float Scale;

	/**
	 * The color to use for debug drawing
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FLinearColor Color;

	/**
	 * The offset at which to draw the debug information in the world
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FTransform WorldOffset;
};

USTRUCT()
struct FRigUnit_SpringIK_WorkData
{
	GENERATED_BODY()

	FRigUnit_SpringIK_WorkData()
	{
		CachedPoleVector = FCachedRigElement();
	}

	UPROPERTY(transient)
	TArray<FCachedRigElement> CachedBones;

	UPROPERTY()
	FCachedRigElement CachedPoleVector;

	UPROPERTY()
	TArray<FTransform> Transforms;

	UPROPERTY()
	FCRSimPointContainer Simulation;
};

/**
 * The Spring IK solver uses a verlet integrator to perform an IK solve.
 * It support custom constraints including distance, length etc.
 * Note: This node operates in world space!
 */
USTRUCT(meta=(DisplayName="Spring IK", Category="Hierarchy", Keywords="IK", NodeColor="0 1 1"))
struct FRigUnit_SpringIK : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SpringIK()
	{
		StartBone = EndBone = PoleVectorSpace = NAME_None;
		HierarchyStrength = 256.f;
		EffectorStrength = RootStrength = 64.f;
		EffectorRatio = RootRatio = 0.5f;
		Damping = 0.4f;
		PoleVector = FVector(0.f, 0.f, 1.f);
		bFlipPolePlane = false;
		PoleVectorKind = EControlRigVectorKind::Direction;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 1.f, 0.f);
		bLiveSimulation = false;
		Iterations = 10;
		bLimitLocalPosition = true;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_SpringIK_DebugSettings();
	}

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the first bone to solve
	 */
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	FName StartBone;

	/**
	 * The name of the second bone to solve
	 */
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	FName EndBone;

	/**
	 * Sets the coefficient of the springs along the hierarchy. Values between 1 and 2048 are common.
	 */
	UPROPERTY(meta = (Input, Constant))
	float HierarchyStrength;

	/**
	 * Sets the coefficient of the springs towards the effector. Values between 1 and 2048 are common.
	 */
	UPROPERTY(meta = (Input, Constant))
	float EffectorStrength;

	/**
	 * Defines the equilibrium of the effector springs.
	 * This value ranges from 0.0 (zero distance) to 1.0 (distance in initial pose)
	 */
	UPROPERTY(meta = (Input, Constant))
	float EffectorRatio;

	/**
	 * Sets the coefficient of the springs towards the root. Values between 1 and 2048 are common.
	 */
	UPROPERTY(meta = (Input, Constant))
	float RootStrength;

	/**
	 * Defines the equilibrium of the root springs.
	 * This value ranges from 0.0 (zero distance) to 1.0 (distance in initial pose)
	 */
	UPROPERTY(meta = (Input, Constant))
	float RootRatio;

	/**
	 * The higher the value to more quickly the simulation calms down. Values between 0.0001 and 0.75 are common.
	 */
	UPROPERTY(meta = (Input, Constant))
	float Damping;

	/**
	 * The polevector to use for the IK solver
	 * This can be a location or direction.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVector;

	/**
	 * If set to true the pole plane will be flipped
	 */
	UPROPERTY(meta = (Input))
	bool bFlipPolePlane;

	/**
	 * The kind of pole vector this is representing - can be a direction or a location
	 */
	UPROPERTY(meta = (Input))
	EControlRigVectorKind PoleVectorKind;

	/**
	 * The space in which the pole vector is expressed
	 */
	UPROPERTY(meta = (Input, CustomWidget = "BoneName"))
	FName PoleVectorSpace;

	/**
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/**
	 * The minor axis being aligned - towards the pole vector
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * If set to true simulation will continue for all intermediate bones over time.
	 */
	UPROPERTY(meta = (Input))
	bool bLiveSimulation;

	/**
	 * Drives how precise the solver operates. Values between 4 and 24 are common.
	 * This is only used if the simulation is not live (bLiveSimulation setting).
	 */
	UPROPERTY(meta = (Input, Constant))
	int32 Iterations;

	/**
	 * If set to true bones are placed within the original distance of
	 * the previous local transform. This can be used to avoid stretch.
	 */
	UPROPERTY(meta = (Input))
	bool bLimitLocalPosition;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/** The debug setting for the node */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_SpringIK_DebugSettings DebugSettings;

	UPROPERTY(transient)
	FRigUnit_SpringIK_WorkData WorkData;
};

#undef UE_API
