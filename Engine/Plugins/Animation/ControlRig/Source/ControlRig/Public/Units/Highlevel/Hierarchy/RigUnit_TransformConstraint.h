// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Constraint.h"
#include "ControlRigDefines.h"
#include "Math/ControlRigMathLibrary.h"

#include "RigUnit_TransformConstraint.generated.h"

#define UE_API CONTROLRIG_API


USTRUCT()
struct FConstraintTarget
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "FConstraintTarget")
	FTransform Transform;

	UPROPERTY(EditAnywhere, Category = "FConstraintTarget")
	float Weight;

	UPROPERTY(EditAnywhere, Category = "FConstraintTarget")
	bool bMaintainOffset;

	UPROPERTY(EditAnywhere, Category = "FConstraintTarget", meta = (Constant))
	FTransformFilter Filter;

	FConstraintTarget()
		: Weight (1.f)
		, bMaintainOffset(true)
	{}
};

USTRUCT()
struct FRigUnit_TransformConstraint_WorkData
{
	GENERATED_BODY()

	// note that Targets.Num () != ConstraintData.Num()
	UPROPERTY()
	TArray<FConstraintData>	ConstraintData;

	UPROPERTY()
	TMap<int32, int32> ConstraintDataToTargets;
};

USTRUCT(meta=(DisplayName="Transform Constraint", Category="Transforms", Deprecated = "4.25"))
struct FRigUnit_TransformConstraint : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TransformConstraint()
		: BaseTransformSpace(ETransformSpaceMode::GlobalSpace)
		, bUseInitialTransforms(true)
	{}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	FName Bone;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	ETransformSpaceMode BaseTransformSpace;

	// Transform op option. Use if ETransformSpace is BaseTransform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	FTransform BaseTransform;

	// Transform op option. Use if ETransformSpace is BaseJoint
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	FName BaseBone;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintTarget> Targets;

	// If checked the initial transform will be used for the constraint data
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, Constant))
	bool bUseInitialTransforms;

private:

	UPROPERTY(transient)
	FRigUnit_TransformConstraint_WorkData WorkData;

public:
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Constrains an item's transform to multiple items' transforms
 */
USTRUCT(meta=(DisplayName="Transform Constraint", Category="Transforms", Keywords = "Parent,Orient,Scale", Deprecated="5.0"))
struct FRigUnit_TransformConstraintPerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TransformConstraintPerItem()
		: BaseTransformSpace(ETransformSpaceMode::GlobalSpace)
		, bUseInitialTransforms(true)
	{
		Item = BaseItem = FRigElementKey(NAME_None, ERigElementType::Bone);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Item;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	ETransformSpaceMode BaseTransformSpace;

	// Transform op option. Use if ETransformSpace is BaseTransform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	FTransform BaseTransform;

	// Transform op option. Use if ETransformSpace is BaseJoint
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	FRigElementKey BaseItem;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintTarget> Targets;

	// If checked the initial transform will be used for the constraint data
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, Constant))
	bool bUseInitialTransforms;

private:
	static UE_API void AddConstraintData(const TArrayView<const FConstraintTarget>& Targets, ETransformConstraintType ConstraintType, const int32 TargetIndex, const FTransform& SourceTransform, const FTransform& InBaseTransform, TArray<FConstraintData>& OutConstraintData, TMap<int32, int32>& OutConstraintDataToTargets);

	UPROPERTY(transient)
	FRigUnit_TransformConstraint_WorkData WorkData;

public:

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT()
struct FConstraintParent
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "FConstraintParent", meta = (Input))
	FRigElementKey Item;

	UPROPERTY(EditAnywhere, Category = "FConstraintParent", meta = (Input))
	float Weight;

	FConstraintParent()
        : Item(FRigElementKey(NAME_None, ERigElementType::Bone))
        , Weight (1.f)
	{}
	
	FConstraintParent(const FRigElementKey InItem, const float InWeight)
		: Item(InItem)
		, Weight(InWeight)
	{}
};

/**
*	Options for interpolating rotations
*/ 
UENUM(BlueprintType)
enum class EConstraintInterpType : uint8
{ 
	/** Weighted Average of Quaternions by their X,Y,Z,W values, The Shortest Route is Respected. The Order of Parents Doesn't Matter */
	Average UMETA(DisplayName="Average(Order Independent)"),
   /** Perform Quaternion Slerp in Sequence, Different Orders of Parents can Produce Different Results */ 
    Shortest UMETA(DisplayName="Shortest(Order Dependent)"),
	
    Max UMETA(Hidden),
};

USTRUCT()
struct FRigUnit_ParentConstraint_AdvancedSettings
{
	GENERATED_BODY()

	FRigUnit_ParentConstraint_AdvancedSettings()
		: InterpolationType(EConstraintInterpType::Average)
		, RotationOrderForFilter(EEulerRotationOrder::XZY)
	{}
	
	/**
	*	Options for interpolating rotations
	*/
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	EConstraintInterpType InterpolationType;
	
	/**
	*	Rotation is converted to euler angles using the specified order such that individual axes can be filtered.
	*/
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	EEulerRotationOrder RotationOrderForFilter;
};

/**
* Constrains an item's transform to multiple items' transforms
*/
USTRUCT(meta=(DisplayName="Parent Constraint", Category="Constraints", Keywords = "Parent,Orient,Scale"))
struct FRigUnit_ParentConstraint : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ParentConstraint()
        : Child(FRigElementKey(NAME_None, ERigElementType::Bone))
        , bMaintainOffset(true)
		, Weight(1.0f)
		, ChildCache()
		, ParentCaches()
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
    UE_API virtual void Execute() override;

	// The child to constrain
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;

	// If True the offset will be computed based on the current delta between each child and parent
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	// The filter to apply on the transform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FTransformFilter Filter;

	// The array of parents to constrain the child to
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	// The advanced settings of the constraint operation
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FRigUnit_ParentConstraint_AdvancedSettings AdvancedSettings;

	// The weight to use when applying the resulting transform to the child
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	UPROPERTY()
	FCachedRigElement ChildCache;

	UPROPERTY()
	TArray<FCachedRigElement> ParentCaches;
};

USTRUCT()
struct FRigUnit_ParentConstraintMath_AdvancedSettings
{
	GENERATED_BODY()

	FRigUnit_ParentConstraintMath_AdvancedSettings()
		: InterpolationType(EConstraintInterpType::Average)
	{}
	
	/**
	*	Options for interpolating rotations
	*/
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	EConstraintInterpType InterpolationType;
};

/**
* Computes the output transform by constraining the input transform to multiple parents 
*/
USTRUCT(meta=(DisplayName="Parent Constraint Math", Category="Constraints", Keywords = "Parent,Orient,Scale"))
struct FRigUnit_ParentConstraintMath : public FRigUnit_HighlevelBase
{
	GENERATED_BODY()

	FRigUnit_ParentConstraintMath()
		: Input(FTransform::Identity)
		, Output(FTransform::Identity)
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
	virtual void Execute() override;

	/**
	*	Input is used to calculate offsets from parents' initial transform
	*/
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FTransform Input;

	// The array of parents to compute the constrained transform for
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	// The advanced settings to use for the constraint compute
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FRigUnit_ParentConstraintMath_AdvancedSettings AdvancedSettings;

	// The resulting constrained transform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Output))
	FTransform Output;

	UPROPERTY()
	TArray<FCachedRigElement> ParentCaches;
};

/**
* Constrains an item's position to multiple items' positions 
*/
USTRUCT(meta=(DisplayName="Position Constraint", Category="Constraints", Keywords = "Parent,Translation", Deprecated = "5.0"))
struct FRigUnit_PositionConstraint : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_PositionConstraint()
        : Child(FRigElementKey(NAME_None, ERigElementType::Bone))
        , bMaintainOffset(true)
		, Weight(1.0f)
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
    UE_API virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;
	
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FFilterOptionPerAxis Filter;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Constrains an item's position to multiple items' positions 
*/
USTRUCT(meta=(DisplayName="Position Constraint", Category="Constraints", Keywords = "Parent,Translation"))
struct FRigUnit_PositionConstraintLocalSpaceOffset : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_PositionConstraintLocalSpaceOffset()
		: Child(FRigElementKey(NAME_None, ERigElementType::Bone))
		, bMaintainOffset(true)
		, Weight(1.0f)
		, ChildCache()
		, ParentCaches()
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The child to constrain
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;
	
	// If True the offset will be computed based on the current delta between each child and parent
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	// The filter to apply on the transform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FFilterOptionPerAxis Filter;

	// The array of parents to constrain the child to
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	// The weight to use when applying the resulting transform to the child
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	UPROPERTY()
	FCachedRigElement ChildCache;

	UPROPERTY()
	TArray<FCachedRigElement> ParentCaches;
};


USTRUCT()
struct FRigUnit_RotationConstraint_AdvancedSettings
{
	GENERATED_BODY()

	FRigUnit_RotationConstraint_AdvancedSettings()
        : InterpolationType(EConstraintInterpType::Average)
        , RotationOrderForFilter(EEulerRotationOrder::XZY)
	{}
	
	/**
	*	Options for interpolating rotations
	*/
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	EConstraintInterpType InterpolationType;
	
	/**
	*	Rotation is converted to euler angles using the specified order such that individual axes can be filtered.
	*/
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	EEulerRotationOrder RotationOrderForFilter;
};

/**
* Constrains an item's rotation to multiple items' rotations 
*/
USTRUCT(meta=(DisplayName="Rotation Constraint", Category="Constraints", Keywords = "Parent,Orientation,Orient,Rotate", Deprecated = "5.0"))
struct FRigUnit_RotationConstraint : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_RotationConstraint()
        : Child(FRigElementKey(NAME_None, ERigElementType::Bone))
        , bMaintainOffset(true)
		, Weight(1.0f)
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
    virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;
	
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FFilterOptionPerAxis Filter;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;
	
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FRigUnit_RotationConstraint_AdvancedSettings AdvancedSettings;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Constrains an item's rotation to multiple items' rotations 
*/
USTRUCT(meta=(DisplayName="Rotation Constraint", Category="Constraints", Keywords = "Parent,Orientation,Orient,Rotate"))
struct FRigUnit_RotationConstraintLocalSpaceOffset : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_RotationConstraintLocalSpaceOffset()
		: Child(FRigElementKey(NAME_None, ERigElementType::Bone))
		, bMaintainOffset(true)
		, Weight(1.0f)
		, ChildCache()
		, ParentCaches()
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
	virtual void Execute() override;

	// The child to constrain
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;
	
	// If True the offset will be computed based on the current delta between each child and parent
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	// The filter to apply on the transform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FFilterOptionPerAxis Filter;

	// The array of parents to constrain the child to
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;
	
	// The advanced settings of the constraint operation
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FRigUnit_RotationConstraint_AdvancedSettings AdvancedSettings;

	// The weight to use when applying the resulting transform to the child
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	UPROPERTY()
	FCachedRigElement ChildCache;

	UPROPERTY()
	TArray<FCachedRigElement> ParentCaches;
};

/**
* Constrains an item's scale to multiple items' scales
*/
USTRUCT(meta=(DisplayName="Scale Constraint", Category="Constraints", Keywords = "Parent, Size", Deprecated = "5.0"))
struct FRigUnit_ScaleConstraint : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ScaleConstraint()
        : Child(FRigElementKey(NAME_None, ERigElementType::Bone))
        , bMaintainOffset(true)
		, Weight(1.0f)
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
    virtual void Execute() override;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;
	
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FFilterOptionPerAxis Filter;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Constrains an item's scale to multiple items' scales
*/
USTRUCT(meta=(DisplayName="Scale Constraint", Category="Constraints", Keywords = "Parent, Size"))
struct FRigUnit_ScaleConstraintLocalSpaceOffset : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_ScaleConstraintLocalSpaceOffset()
		: Child(FRigElementKey(NAME_None, ERigElementType::Bone))
		, bMaintainOffset(true)
		, Weight(1.0f)
		, ChildCache()
		, ParentCaches()
	{
		Parents.Add(FConstraintParent());
	}
	
	RIGVM_METHOD()
	virtual void Execute() override;

	// The child to constrain
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault))
	FRigElementKey Child;
	
	// If True the offset will be computed based on the current delta between each child and parent
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	bool bMaintainOffset;

	// The filter to apply on the transform
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input)) 
	FFilterOptionPerAxis Filter;

	// The array of parents to constrain the child to
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input, ExpandByDefault, DefaultArraySize = 1))
	TArray<FConstraintParent> Parents;

	// The weight to use when applying the resulting transform to the child
	UPROPERTY(EditAnywhere, Category = "Constraint", meta = (Input))
	float Weight;

	UPROPERTY()
	FCachedRigElement ChildCache;

	UPROPERTY()
	TArray<FCachedRigElement> ParentCaches;
};

#undef UE_API
