// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigUnit_TwistBones.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_TwistBones_WorkData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FCachedRigElement> CachedItems;

	UPROPERTY()
	TArray<float> ItemRatios;

	UPROPERTY()
	TArray<FTransform> ItemTransforms;
};


/**
 * Creates a gradient of twist rotation along a chain.
 */
USTRUCT(meta=(DisplayName="Twist Bones", Category="Hierarchy", Keywords="TwistBones", Deprecated = "4.25"))
struct FRigUnit_TwistBones : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TwistBones()
	{
		StartBone = EndBone = NAME_None;
		TwistAxis = FVector(1.f, 0.f, 0.f);
		PoleAxis = FVector(0.f, 1.f, 0.f);
		TwistEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of the first bone to twist
	 */
	UPROPERTY(meta = (Input))
	FName StartBone;

	/** 
	 * The name of the last bone to twist
	 */
	UPROPERTY(meta = (Input))
	FName EndBone;

	/**
	 * The axis to twist the bones around
	 */
	UPROPERTY(meta = (Input, Constant))
	FVector TwistAxis;

	/**
	 * The axis to use for the pole vector for each bone
	 */
	UPROPERTY(meta = (Input, Constant))
	FVector PoleAxis;

	/**
	 * The easing to use between two twists.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType TwistEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */	
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_TwistBones_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Creates a gradient of twist rotation along a collection of items.
 */
USTRUCT(meta=(DisplayName="Twist Bones", Category="Hierarchy", Keywords="TwistBones", Deprecated = "5.0"))
struct FRigUnit_TwistBonesPerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TwistBonesPerItem()
	{
		TwistAxis = FVector(1.f, 0.f, 0.f);
		PoleAxis = FVector(0.f, 1.f, 0.f);
		TwistEaseType = ERigVMAnimEasingType::Linear;
		Weight = 1.f;
		bPropagateToChildren = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The items to twist
	 */
	UPROPERTY(meta = (Input))
	FRigElementKeyCollection Items;

	/**
	 * The axis to twist the bones around
	 */
	UPROPERTY(meta = (Input, Constant))
	FVector TwistAxis;

	/**
	 * The axis to use for the pole vector for each bone
	 */
	UPROPERTY(meta = (Input, Constant))
	FVector PoleAxis;

	/**
	 * The easing to use between two twists.
	 */
	UPROPERTY(meta = (Input, Constant))
	ERigVMAnimEasingType TwistEaseType;

	/**
	 * The weight of the solver - how much the rotation should be applied
	 */	
	UPROPERTY(meta = (Input))
	float Weight;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	UPROPERTY(transient)
	FRigUnit_TwistBones_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
