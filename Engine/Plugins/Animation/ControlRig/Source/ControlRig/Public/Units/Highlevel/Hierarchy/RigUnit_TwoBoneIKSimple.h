// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Highlevel/RigUnit_HighlevelBase.h"
#include "RigUnit_TwoBoneIKSimple.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_TwoBoneIKSimple_DebugSettings
{
	GENERATED_BODY()

	FRigUnit_TwoBoneIKSimple_DebugSettings()
	{
		bEnabled = false;
		Scale = 10.f;
		WorldOffset = FTransform::Identity;
	}

	/**
	 * If enabled debug information will be drawn 
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "DebugSettings")
	bool bEnabled;

	/**
	 * The size of the debug drawing information
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	float Scale;

	/**
	 * The offset at which to draw the debug information in the world
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FTransform WorldOffset;
};

/**
 * Solves the two bone IK given two bones.
 * Note: This node operates in world space!
 */
USTRUCT(meta=(DisplayName="Basic IK", Category="Hierarchy", Keywords="TwoBone,IK", Deprecated = "4.25"))
struct FRigUnit_TwoBoneIKSimple : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TwoBoneIKSimple()
	{
		BoneA = BoneB = EffectorBone = PoleVectorSpace = NAME_None;
		Effector = FTransform::Identity;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 1.f, 0.f);
		SecondaryAxisWeight = 1.f;
		PoleVector = FVector(0.f, 0.f, 1.f);
		PoleVectorKind = EControlRigVectorKind::Direction;
		bEnableStretch = false;
		StretchStartRatio = 0.75f;
		StretchMaximumRatio = 1.25f;
		Weight = 1.f;
		BoneALength = BoneBLength = 0.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_TwoBoneIKSimple_DebugSettings();

		CachedBoneAIndex = FCachedRigElement();
		CachedBoneBIndex = FCachedRigElement();
		CachedEffectorBoneIndex = FCachedRigElement();
		CachedPoleVectorSpaceIndex = FCachedRigElement();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of first bone
	 */
	UPROPERTY(meta = (Input))
	FName BoneA;

	/** 
	 * The name of second bone
	 */
	UPROPERTY(meta = (Input))
	FName BoneB;

	/** 
	 * The name of the effector bone (if exists)
	 */
	UPROPERTY(meta = (Input))
	FName EffectorBone;

	/** 
	 * The transform of the effector
	 */
	UPROPERTY(meta = (Input))
	FTransform Effector;

	/** 
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/** 
	 * The minor axis being aligned - towards the polevector
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * Determines how much the secondary axis roll is being applied
	 */
	UPROPERTY(meta = (Input))
	float SecondaryAxisWeight;

	/** 
	 * The polevector to use for the IK solver
	 * This can be a location or direction.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVector;

	/**
	 * The kind of pole vector this is representing - can be a direction or a location
	 */
	UPROPERTY(meta = (Input))
	EControlRigVectorKind PoleVectorKind;

	/** 
	 * The space in which the pole vector is expressed
	 */
	UPROPERTY(meta = (Input))
	FName PoleVectorSpace;

	/**
	 * If set to true the stretch feature of the solver will be enabled
	 */
	UPROPERTY(meta = (Input))
	bool bEnableStretch;

	/**
	 * The ratio where the stretch starts
	 */
	UPROPERTY(meta = (Input))
	float StretchStartRatio;

	/**
     * The maximum allowed stretch ratio
	 */
	UPROPERTY(meta = (Input))
	float StretchMaximumRatio;

	/** 
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/** 
	 * The length of the first bone.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float BoneALength;

	/** 
	 * The length of the second  bone.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float BoneBLength;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/** 
	 * The settings for debug drawing
	 */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_TwoBoneIKSimple_DebugSettings DebugSettings;

	UPROPERTY()
	FCachedRigElement CachedBoneAIndex;
	UPROPERTY()
	FCachedRigElement CachedBoneBIndex;
	UPROPERTY()
	FCachedRigElement CachedEffectorBoneIndex;
	UPROPERTY()
	FCachedRigElement CachedPoleVectorSpaceIndex;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Solves the two bone IK given two bones.
 * Note: This node operates in world space!
 */
USTRUCT(meta=(DisplayName="Basic IK", Category="Hierarchy", Keywords="TwoBone,IK", NodeColor="0 1 1"))
struct FRigUnit_TwoBoneIKSimplePerItem : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TwoBoneIKSimplePerItem()
	{
		ItemA = ItemB = EffectorItem = PoleVectorSpace = FRigElementKey(NAME_None, ERigElementType::Bone);
		Effector = FTransform::Identity;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 1.f, 0.f);
		SecondaryAxisWeight = 1.f;
		PoleVector = FVector(0.f, 0.f, 1.f);
		PoleVectorKind = EControlRigVectorKind::Direction;
		bEnableStretch = false;
		StretchStartRatio = 0.75f;
		StretchMaximumRatio = 1.25f;
		Weight = 1.f;
		ItemALength = ItemBLength = 0.f;
		bPropagateToChildren = true;
		DebugSettings = FRigUnit_TwoBoneIKSimple_DebugSettings();

		CachedItemAIndex = FCachedRigElement();
		CachedItemBIndex = FCachedRigElement();
		CachedEffectorItemIndex = FCachedRigElement();
		CachedPoleVectorSpaceIndex = FCachedRigElement();
	}

#if WITH_EDITOR
	UE_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
	UE_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) override;
#endif

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** 
	 * The name of first item
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey ItemA;

	/** 
	 * The name of second item
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey ItemB;

	/** 
	 * The name of the effector item (if exists)
	 */
	UPROPERTY(meta = (Input, ExpandByDefault))
	FRigElementKey EffectorItem;

	/** 
	 * The transform of the effector
	 */
	UPROPERTY(meta = (Input))
	FTransform Effector;

	/** 
	 * The major axis being aligned - along the item
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/** 
	 * The minor axis being aligned - towards the polevector
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * Determines how much the secondary axis roll is being applied
	 */
	UPROPERTY(meta = (Input))
	float SecondaryAxisWeight;

	/** 
	 * The polevector to use for the IK solver
	 * This can be a location or direction.
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVector;

	/**
	 * The kind of pole vector this is representing - can be a direction or a location
	 */
	UPROPERTY(meta = (Input))
	EControlRigVectorKind PoleVectorKind;

	/** 
	 * The space in which the pole vector is expressed
	 */
	UPROPERTY(meta = (Input))
	FRigElementKey PoleVectorSpace;

	/**
	 * If set to true the stretch feature of the solver will be enabled
	 */
	UPROPERTY(meta = (Input))
	bool bEnableStretch;

	/**
	 * The ratio where the stretch starts
	 */
	UPROPERTY(meta = (Input))
	float StretchStartRatio;

	/**
     * The maximum allowed stretch ratio
	 */
	UPROPERTY(meta = (Input))
	float StretchMaximumRatio;

	/** 
	 * The weight of the solver - how much the IK should be applied.
	 */
	UPROPERTY(meta = (Input))
	float Weight;

	/** 
	 * The length of the first item.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float ItemALength;

	/** 
	 * The length of the second item.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float ItemBLength;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/** 
	 * The settings for debug drawing
	 */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_TwoBoneIKSimple_DebugSettings DebugSettings;

	UPROPERTY()
	FCachedRigElement CachedItemAIndex;
	UPROPERTY()
	FCachedRigElement CachedItemBIndex;
	UPROPERTY()
	FCachedRigElement CachedEffectorItemIndex;
	UPROPERTY()
	FCachedRigElement CachedPoleVectorSpaceIndex;
};

/**
 * Solves the two bone IK given positions
 * Note: This node operates in world space!
 */
USTRUCT(meta = (DisplayName = "Basic IK Positions", Category = "Hierarchy", Keywords = "TwoBone,IK", NodeColor = "0 1 1"))
struct FRigUnit_TwoBoneIKSimpleVectors : public FRigUnit_HighlevelBase
{
	GENERATED_BODY()

	FRigUnit_TwoBoneIKSimpleVectors()
	{
		Root = PoleVector = Effector = Elbow = FVector::ZeroVector;
		bEnableStretch = false;
		StretchStartRatio = 0.75f;
		StretchMaximumRatio = 1.25f;
		BoneALength = BoneBLength = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The position of the root of the triangle
	 */
	UPROPERTY(meta = (Input))
	FVector Root;

	/**
	 * The position of the pole of the triangle
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVector;

	/**
	 * The position of the effector
	 */
	UPROPERTY(meta = (Input, Output))
	FVector Effector;

	/**
	 * If set to true the stretch feature of the solver will be enabled
	 */
	UPROPERTY(meta = (Input))
	bool bEnableStretch;

	/**
	 * The ratio where the stretch starts
	 */
	UPROPERTY(meta = (Input))
	float StretchStartRatio;

	/**
	 * The maximum allowed stretch ratio
	 */
	UPROPERTY(meta = (Input))
	float StretchMaximumRatio;

	/**
	 * The length of the first bone.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float BoneALength;

	/**
	 * The length of the second  bone.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float BoneBLength;

	/**
	 * The resulting elbow position
	 */
	UPROPERTY(meta = (Output))
	FVector Elbow;
};

/**
 * Solves the two bone IK given transforms
 * Note: This node operates in world space!
 */
USTRUCT(meta = (DisplayName = "Basic IK Transforms", Category = "Hierarchy", Keywords = "TwoBone,IK", NodeColor = "0 1 1"))
struct FRigUnit_TwoBoneIKSimpleTransforms : public FRigUnit_HighlevelBase
{
	GENERATED_BODY()

	FRigUnit_TwoBoneIKSimpleTransforms()
	{
		Root = Effector = Elbow = FTransform::Identity;
		PoleVector = FVector::ZeroVector;
		PrimaryAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAxis = FVector(0.f, 1.f, 0.f);
		SecondaryAxisWeight = 1.f;
		bEnableStretch = false;
		StretchStartRatio = 0.75f;
		StretchMaximumRatio = 1.25f;
		BoneALength = BoneBLength = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/**
	 * The transform of the root of the triangle
	 */
	UPROPERTY(meta = (Input, Output))
	FTransform Root;

	/**
	 * The position of the pole of the triangle
	 */
	UPROPERTY(meta = (Input))
	FVector PoleVector;

	/**
	 * The transform of the effector
	 */
	UPROPERTY(meta = (Input, Output))
	FTransform Effector;
		/** 
	 * The major axis being aligned - along the bone
	 */
	UPROPERTY(meta = (Input))
	FVector PrimaryAxis;

	/** 
	 * The minor axis being aligned - towards the polevector
	 */
	UPROPERTY(meta = (Input))
	FVector SecondaryAxis;

	/**
	 * Determines how much the secondary axis roll is being applied
	 */
	UPROPERTY(meta = (Input))
	float SecondaryAxisWeight;

	/**
	 * If set to true the stretch feature of the solver will be enabled
	 */
	UPROPERTY(meta = (Input))
	bool bEnableStretch;

	/**
	 * The ratio where the stretch starts
	 */
	UPROPERTY(meta = (Input))
	float StretchStartRatio;

	/**
	 * The maximum allowed stretch ratio
	 */
	UPROPERTY(meta = (Input))
	float StretchMaximumRatio;

	/**
	 * The length of the first bone.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float BoneALength;

	/**
	 * The length of the second  bone.
	 * If set to 0.0 it will be determined by the hierarchy
	 */
	UPROPERTY(meta = (Input))
	float BoneBLength;

	/**
	 * The resulting elbow transform
	 */
	UPROPERTY(meta = (Input, Output))
	FTransform Elbow;
};

#undef UE_API
