// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "RigUnit_HighlevelBase.generated.h"

UENUM()
enum class EControlRigVectorKind : uint8
{
	Direction,
	Location
};

/*
 * Base class for all pure high level nodes
 */
USTRUCT(meta=(Abstract, Category="Debug", NodeColor = "0.462745 1.0 0.329412", DocumentationPolicy = "Strict"))
struct FRigUnit_HighlevelBase : public FRigUnit
{
	GENERATED_BODY()
};

/*
 * Base class for all mutable high level nodes
 */
USTRUCT(meta=(Abstract, Category="Debug", NodeColor = "0 0.364706 1.0", DocumentationPolicy = "Strict"))
struct FRigUnit_HighlevelBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};
