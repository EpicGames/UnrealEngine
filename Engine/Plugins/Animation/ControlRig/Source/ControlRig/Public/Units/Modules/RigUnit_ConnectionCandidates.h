// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_ConnectionCandidates.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Returns the connection candidates for one connector
 */
USTRUCT(meta=(DisplayName="Get Candidates", Category="Modules", TitleColor="1 0 0", NodeColor="1 1 1", Keywords="Connection,Resolve", Varying))
struct FRigUnit_GetCandidates : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The connector being resolved
	UPROPERTY(EditAnywhere, Transient, Category = "Modules", meta = (Output))
	FRigElementKey Connector;

	// The items being interacted on
	UPROPERTY(EditAnywhere, Transient, Category = "Modules", meta = (Output))
	TArray<FRigElementKey> Candidates;
};

/**
 * Discards matches during a connector event
 */
USTRUCT(meta=(DisplayName="Discard Matches", Category="Modules", TitleColor="1 0 0", NodeColor="1 1 1", Keywords="Connection,Resolve,Match", Varying))
struct FRigUnit_DiscardMatches : public FRigUnitMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The items being interacted on
	UPROPERTY(EditAnywhere, Transient, Category = "Modules", meta = (Input))
	TArray<FRigElementKey> Excluded;

	// The message to send to the user interface when discarding matches 
	UPROPERTY(EditAnywhere, Transient, Category = "Modules", meta = (Input))
	FString Message;
};

/**
 * Set default match during a connector event
 */
USTRUCT(meta=(DisplayName="Set Default Match", Category="Modules", TitleColor="1 0 0", NodeColor="1 1 1", TemplateName = "Set Default Match", Keywords="Connection,Resolve,Match,Default"))
struct FRigUnit_SetDefaultMatch : public FRigUnitMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The item being interacted on
	UPROPERTY(EditAnywhere, Transient, Category = "Modules", meta = (Input))
	FRigElementKey Default;
};

/**
 * Set default matches for an array of connectors during a connector event
 */
USTRUCT(meta = (DisplayName = "Set Default Match (Array)", Category = "Modules", TitleColor = "1 0 0", NodeColor = "1 1 1", TemplateName = "Set Default Match", Keywords = "Connection,Resolve,Match,Array,Default"))
struct FRigUnit_SetDefaultMatchArray : public FRigUnitMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The items being interacted on
	UPROPERTY(EditAnywhere, Transient, Category = "Modules", meta = (Input, ExpandByDefault))
	TArray<FRigElementKey> Default;
};

#undef UE_API
