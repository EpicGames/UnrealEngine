// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/RigUnit.h"
#include "ControlRigDefines.h"
#include "RigUnit_ConnectorExecution.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Event for filtering connection candidates
 */
USTRUCT(meta=(DisplayName="Connector", Category="Events", NodeColor="1, 0, 0", Keywords="Event,During,Resolve,Connect,Filter"))
struct FRigUnit_ConnectorExecution : public FRigUnit
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	virtual FName GetEventName() const override { return EventName; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "BeginExecution", meta = (Output))
	FControlRigExecuteContext ExecuteContext;

	static inline const FLazyName EventName = FLazyName(TEXT("Connector"));
};

#undef UE_API
