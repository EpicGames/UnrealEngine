// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigDefines.h"
#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigUnitContext.h"
#include "RigDispatchFactory.generated.h"

#define UE_API CONTROLRIG_API

/** Base class for all rig dispatch factories */
USTRUCT(BlueprintType, meta=(Abstract, ExecuteContextType=FControlRigExecuteContext))
struct FRigDispatchFactory : public FRigVMDispatchFactory
{
	GENERATED_BODY()

	virtual UScriptStruct* GetExecuteContextStruct() const override
	{
		return FControlRigExecuteContext::StaticStruct();
	}

	virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const override
	{
		InRegistry->FindOrAddType_NoLock(FControlRigExecuteContext::StaticStruct());
		InRegistry->FindOrAddType_NoLock(FRigElementKey::StaticStruct());
    	InRegistry->FindOrAddType_NoLock(FCachedRigElement::StaticStruct());
	}

#if WITH_EDITOR

	UE_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;

#endif

	static const FRigUnitContext& GetRigUnitContext(const FRigVMExtendedExecuteContext& InContext)
	{
		return InContext.GetPublicData<FControlRigExecuteContext>().UnitContext;
	}
};

#undef UE_API
