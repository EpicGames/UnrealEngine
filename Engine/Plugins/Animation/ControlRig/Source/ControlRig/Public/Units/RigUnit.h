// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigDefines.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigUnitContext.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"

#if WITH_EDITOR
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#endif

#include "RigUnit.generated.h"

struct FRigUnitContext;
struct FRigDirectManipulationInfo;

#if WITH_EDITOR

struct FRigDirectManipulationTarget
{
	FRigDirectManipulationTarget()
		: Name()
		, ControlType(ERigControlType::EulerTransform)
	{
	}

	FRigDirectManipulationTarget(const FString& InName, ERigControlType InControlType)
		: Name(InName)
		, ControlType(InControlType)
	{
	}

	bool operator == (const FRigDirectManipulationTarget& InOther) const
	{
		return Name.Equals(InOther.Name, ESearchCase::CaseSensitive);
	}

	bool operator > (const FRigDirectManipulationTarget& InOther) const
	{
		return Name > InOther.Name;
	}

	bool operator < (const FRigDirectManipulationTarget& InOther) const
	{
		return Name < InOther.Name;
	}

	FString Name;
	ERigControlType ControlType;
};

#endif

/** Base class for all rig units */
USTRUCT(BlueprintType, meta=(Abstract, NodeColor = "0.1 0.1 0.1", ExecuteContext="FControlRigExecuteContext"))
struct FRigUnit : public FRigVMStruct
{
	GENERATED_BODY()

public:

	FRigUnit()
	{}

	/** Virtual destructor */
	virtual ~FRigUnit() = default;

	virtual FRigElementKey DetermineSpaceForPin(const FString& InPinPath, void* InUserContext) const { return FRigElementKey(); }
	
	virtual FTransform DetermineOffsetTransformForPin(const FString& InPinPath, void* InUserContext) const { return FTransform::Identity; }
	
	/** The name of the method used within each rig unit */
	static FName GetMethodName()
	{
		static const FLazyName MethodName = FRigVMStruct::ExecuteName;
		return MethodName;
	}

#if WITH_EDITOR
	/** Returns the targets for viewport posing */
	CONTROLRIG_API virtual bool GetDirectManipulationTargets(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, URigHierarchy* InHierarchy, TArray<FRigDirectManipulationTarget>& InOutTargets, FString* OutFailureReason) const;

	/** Optionally configures a control's settings and value for a given target */
	CONTROLRIG_API virtual void ConfigureDirectManipulationControl(const URigVMUnitNode* InNode, TSharedPtr<FRigDirectManipulationInfo> InInfo, FRigControlSettings& InOutSettings, FRigControlValue& InOutValue) const;

	/** Sets a control's pose to represent this viewport pose target */ 
	CONTROLRIG_API virtual bool UpdateHierarchyForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo);

	/** Sets the values on this node based on a viewport pose */
	CONTROLRIG_API virtual bool UpdateDirectManipulationFromHierarchy(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo);

	/** returns a list of pins affected by the viewport pose */
	CONTROLRIG_API virtual TArray<const URigVMPin*> GetPinsForDirectManipulation(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget) const;

	/** Allows the node to draw debug drawing during a manipulation */
	CONTROLRIG_API virtual void PerformDebugDrawingForDirectManipulation(const URigVMUnitNode* InNode, TSharedPtr<FStructOnScope> InInstance, FControlRigExecuteContext& InContext, TSharedPtr<FRigDirectManipulationInfo> InInfo) const;

private:
	
	static CONTROLRIG_API bool AddDirectManipulationTarget_Internal(TArray<FRigDirectManipulationTarget>& InOutTargets, const URigVMPin* InPin, const UScriptStruct* InScriptStruct);
	static CONTROLRIG_API TTuple<const FStructProperty*, uint8*> FindStructPropertyAndTargetMemory(TSharedPtr<FStructOnScope> InInstance, const UScriptStruct* InStruct, const FString& InPinPath); 
#endif
};

/** Base class for all rig units that can change data */
USTRUCT(BlueprintType, meta = (Abstract))
struct FRigUnitMutable : public FRigUnit
{
	GENERATED_BODY()

	FRigUnitMutable()
	: FRigUnit()
	{}

	/*
	 * This property is used to chain multiple mutable units together
	 */
	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;
};

#if WITH_EDITOR

struct FRigDirectManipulationInfo
{
	FRigDirectManipulationInfo()
		: bInitialized(false)
		, Target()
		, ControlKey(NAME_None, ERigElementType::Control)
		, OffsetTransform(FTransform::Identity)
	{
		Reset();
	}

	void Reset()
	{
		bInitialized = false;
		OffsetTransform = FTransform::Identity;
		Transforms.Reset();
		Transforms.Add(FTransform::Identity);
	}

	bool bInitialized;
	FRigDirectManipulationTarget Target;
	FRigElementKey ControlKey;
	FTransform OffsetTransform;
	TArray<FTransform> Transforms;
	TWeakObjectPtr<const URigVMUnitNode> Node;
};

#endif
