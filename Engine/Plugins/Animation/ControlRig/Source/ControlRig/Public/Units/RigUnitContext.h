// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Rigs/RigCurveContainer.h"
#include "AnimationDataSource.h"
#include "Animation/AttributesRuntime.h"
#include "ControlRigAssetUserData.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Rigs/RigDependencyRecords.h"
#include "RigUnitContext.generated.h"

#define UE_API CONTROLRIG_API

class UControlRig;
class UControlRigShapeLibrary;
struct FRigModuleInstance;

/**
 * The type of interaction happening on a rig
 */
UENUM()
enum class EControlRigInteractionType : uint8
{
	None = 0,
	Translate = (1 << 0),
	Rotate = (1 << 1),
	Scale = (1 << 2),
	All = Translate | Rotate | Scale
};

UENUM(BlueprintType, meta = (RigVMTypeAllowed))
enum class ERigMetaDataNameSpace : uint8
{
	// Use no namespace - store the metadata directly on the item
	None,
	// Store the metadata for item relative to its module
	Self,
	// Store the metadata relative to its parent model
	Parent,
	// Store the metadata under the root module
	Root,
	Last UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FRigHierarchySettings
{
	GENERATED_BODY();

	FRigHierarchySettings()
		: ElementNameDisplayMode(EElementNameDisplayMode::Auto)
		, ProceduralElementLimit(2000)
	{
	}
	
	bool operator==(const FRigHierarchySettings& InOther) const
	{
		return ElementNameDisplayMode == InOther.ElementNameDisplayMode &&
			ProceduralElementLimit == InOther.ProceduralElementLimit;
	}

	// The way to display this hierarchy's element names in the user interface
	UPROPERTY(EditAnywhere, Category = "Hierarchy Settings")
	EElementNameDisplayMode ElementNameDisplayMode;

	// Sets the limit for the number of elements to create procedurally
	UPROPERTY(EditAnywhere, Category = "Hierarchy Settings")
	int32 ProceduralElementLimit;
};

/** Execution context that rig units use */
struct FRigUnitContext
{
	/** default constructor */
	FRigUnitContext()
		: AnimAttributeContainer(nullptr)
		, DataSourceRegistry(nullptr)
		, InteractionType((uint8)EControlRigInteractionType::None)
		, ElementsBeingInteracted()
	{
	}

	/** An external anim attribute container */
	UE::Anim::FMeshAttributeContainer* AnimAttributeContainer;
	
	/** The registry to access data source */
	const UAnimationDataSourceRegistry* DataSourceRegistry;

	/** The current hierarchy settings */
	FRigHierarchySettings HierarchySettings;

	/** The type of interaction currently happening on the rig (0 == None) */
	uint8 InteractionType;

	/** The elements being interacted with. */
	TArray<FRigElementKey> ElementsBeingInteracted;

	/** Acceptable subset of connection matches */
	FModularRigResolveResult ConnectionResolve;

	/**
	 * Returns a given data source and cast it to the expected class.
	 *
	 * @param InName The name of the data source to look up.
	 * @return The requested data source
	 */
	template<class T>
	T* RequestDataSource(const FName& InName) const
	{
		if (DataSourceRegistry == nullptr)
		{
			return nullptr;
		}
		return DataSourceRegistry->RequestSource<T>(InName);
	}

	/**
	 * Returns true if this context is currently being interacted on
	 */
	bool IsInteracting() const
	{
		return InteractionType != (uint8)EControlRigInteractionType::None;
	}
};

USTRUCT(BlueprintType)
struct FControlRigExecuteContext : public FRigVMExecuteContext
{
	GENERATED_BODY()

public:
	
	FControlRigExecuteContext()
		: FRigVMExecuteContext()
		, Hierarchy(nullptr)
		, ControlRig(nullptr)
		, RigModulePrefix()
		, RigModulePrefixHash(0)
		, RigModuleInstance(nullptr)
		, Records(&RecordsStorage)
	{
	}

	virtual void Copy(const FRigVMExecuteContext* InOtherContext) override
	{
		Super::Copy(InOtherContext);

		const FControlRigExecuteContext* OtherContext = (const FControlRigExecuteContext*)InOtherContext; 
		Hierarchy = OtherContext->Hierarchy;
		ControlRig = OtherContext->ControlRig;
	}

	/**
     * Finds a name spaced user data object
     */
	const UNameSpacedUserData* FindUserData(const FString& InNameSpace) const
	{
		// walk in reverse since asset user data at the end with the same
		// namespace overrides previously listed user data
		for(int32 Index = AssetUserData.Num() - 1; AssetUserData.IsValidIndex(Index); Index--)
		{
			if(!IsValid(AssetUserData[Index]))
			{
				continue;
			}
			if(const UNameSpacedUserData* NameSpacedUserData = Cast<UNameSpacedUserData>(AssetUserData[Index]))
			{
				if(NameSpacedUserData->NameSpace.Equals(InNameSpace, ESearchCase::CaseSensitive))
				{
					return NameSpacedUserData;
				}
			}
		}
		return nullptr;
	}

	/**
	 * Returns true if the event currently running is considered a construction or post-construction event
	 */
	bool IsRunningAConstructionEvent() const;

	/**
	 * Returns true if this context is used on a module currently
	 */
	bool IsRigModule() const
	{
		return !GetRigModulePrefix().IsEmpty();
	}

	/**
	 * Returns the rig module prefix
	 */
	const FString& GetRigModulePrefix() const
	{
		return RigModulePrefix;
	}

	/**
	 * Returns the prefix of the parent rig module
	 */
	const FString& GetRigRootModulePrefix() const
	{
		return RigRootModulePrefix;
	}

	/**
	 * Returns the prefix of the root rig module
	 */
	const FString& GetRigParentModulePrefix() const
	{
		return RigParentModulePrefix;
	}

	/**
	 * Returns the namespace given a namespace type
	 */
	const FString& GetElementModulePrefix(ERigMetaDataNameSpace InNameSpaceType) const;
	
	/**
	 * Returns the module this unit is running inside of (or nullptr)
	 */
	const FRigModuleInstance* GetRigModuleInstance() const
	{
		return RigModuleInstance;
	}
	
	/**
	 * Returns the module this unit is running inside of (or nullptr)
	 */
	const FRigModuleInstance* GetRigModuleInstance(ERigMetaDataNameSpace InNameSpaceType) const;

	/**
	 * Adapts a metadata name according to rig module namespace.
	 */
	FName AdaptMetadataName(ERigMetaDataNameSpace InNameSpaceType, const FName& InMetadataName) const;

	/** The list of available asset user data object */
	TArray<const UAssetUserData*> AssetUserData;

	DECLARE_DELEGATE_FourParams(FOnAddShapeLibrary, const FControlRigExecuteContext* InContext, const FString&, UControlRigShapeLibrary*, bool /* log results */);
	FOnAddShapeLibrary OnAddShapeLibraryDelegate;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShapeExists, const FName&);
	FOnShapeExists OnShapeExistsDelegate;
	
	FRigUnitContext UnitContext;
	URigHierarchy* Hierarchy;
	UControlRig* ControlRig;
	
	void AddReadInstructionRecord(const FInstructionRecord& InRecord) const
	{
		check(Records);
		Records->ReadRecords.Add(InRecord);
		Records->ReadRecordsHash = HashCombine(Records->ReadRecordsHash, GetTypeHash(InRecord));
	}

	void AddWrittenInstructionRecord(const FInstructionRecord& InRecord) const
	{
		check(Records);
		Records->WrittenRecords.Add(InRecord);
		Records->WrittenRecordsHash = HashCombine(Records->WrittenRecordsHash, GetTypeHash(InRecord));
	}

	void ResetInstructionRecords() const
	{
		// only reset the memory we own.
		// (ignore the Records pointer since it may point to unowned memory)
		RecordsStorage.ReadRecords.Reset();
		RecordsStorage.WrittenRecords.Reset();
		RecordsStorage.ReadRecordsHash = RecordsStorage.WrittenRecordsHash = 0;
	}

	uint32 GetInstructionRecordsHash() const
	{
		check(Records);
		return HashCombine(Records->ReadRecordsHash, Records->WrittenRecordsHash);
	}

	const FInstructionRecordContainer& GetInstructionRecords() const
	{
		check(Records);
		return *Records;
	}

#if WITH_EDITOR
	virtual void Report(const FRigVMLogSettings& InLogSettings, const FName& InFunctionName, int32 InInstructionIndex, const FString& InMessage) const override
	{
		const FString& Prefix = GetRigModulePrefix();
		if (!Prefix.IsEmpty())
		{
			const FString Name = FString::Printf(TEXT("%s %s"), *Prefix, *InFunctionName.ToString());
			FRigVMExecuteContext::Report(InLogSettings, *Name, InInstructionIndex, InMessage);
		}
		else
		{
			FRigVMExecuteContext::Report(InLogSettings, InFunctionName, InInstructionIndex, InMessage);
		}
	}
#endif

private:
	FString RigModulePrefix;
	FString RigParentModulePrefix;
	FString RigRootModulePrefix;
	uint32 RigModulePrefixHash;
	const FRigModuleInstance* RigModuleInstance;

	mutable FInstructionRecordContainer RecordsStorage;
	mutable FInstructionRecordContainer* Records;

	friend class FControlRigExecuteContextRigModuleGuard;
	friend class UModularRig;
};

class FControlRigExecuteContextRigModuleGuard
{
public:
	UE_API FControlRigExecuteContextRigModuleGuard(FControlRigExecuteContext& InContext, const UControlRig* InControlRig);
	UE_API FControlRigExecuteContextRigModuleGuard(FControlRigExecuteContext& InContext, const FString& InNewModulePrefix, const FString& InNewParentModulePrefix, const FString& InNewRootModulePrefix);
	UE_API ~FControlRigExecuteContextRigModuleGuard();

private:

	FControlRigExecuteContext& Context;
	FString PreviousRigModulePrefix;
	FString PreviousRigParentModulePrefix;
	FString PreviousRigRootModulePrefix;
	uint32 PreviousRigModulePrefixHash;
};

#if WITH_EDITOR
#define UE_CONTROLRIG_RIGUNIT_REPORT(Severity, Format, ...) \
ExecuteContext.Report(EMessageSeverity::Severity, ExecuteContext.GetFunctionName(), ExecuteContext.GetInstructionIndex(), FString::Printf((Format), ##__VA_ARGS__)); 

#define UE_CONTROLRIG_RIGUNIT_LOG_MESSAGE(Format, ...) UE_CONTROLRIG_RIGUNIT_REPORT(Info, (Format), ##__VA_ARGS__)
#define UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(Format, ...) UE_CONTROLRIG_RIGUNIT_REPORT(Warning, (Format), ##__VA_ARGS__)
#define UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(Format, ...) UE_CONTROLRIG_RIGUNIT_REPORT(Error, (Format), ##__VA_ARGS__)
#else
#define UE_CONTROLRIG_RIGUNIT_REPORT(Severity, Format, ...)
#define UE_CONTROLRIG_RIGUNIT_LOG_MESSAGE(Format, ...)
#define UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(Format, ...)
#define UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(Format, ...)
#endif

#undef UE_API
