// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "RigUnit.h"
#include "RigUnitContext.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "RigVMCore/RigVMStructTest.h"

class FControlRigUnitTestBase : public FRigVMStructTestBase<FControlRigExecuteContext>
{
public:
	FControlRigUnitTestBase(const FString& InName, bool bIsComplex)
		: FRigVMStructTestBase<FControlRigExecuteContext>(InName, bIsComplex)
		, Hierarchy(nullptr)
		, Controller(nullptr)
	{
	}

	~FControlRigUnitTestBase()
	{
		if (Hierarchy.IsValid())
		{
			// we no longer add/remove the controller to/from root since controller is now part of the hierarchy
			Hierarchy->RemoveFromRoot();
		}
	}

	virtual void Initialize() override
	{
		FRigVMStructTestBase<FControlRigExecuteContext>::Initialize();
		
		if (!Hierarchy)
		{
			Hierarchy = NewObject<URigHierarchy>();
			Controller = Hierarchy->GetController(true);

			Hierarchy->AddToRoot();
			// we no longer add the controller to root since controller is now part of the hierarchy
		}
		ExecuteContext.Hierarchy = Hierarchy.Get();
	}

	TSoftObjectPtr<URigHierarchy> Hierarchy;
	TSoftObjectPtr<URigHierarchyController> Controller;
};

#define CONTROLRIG_RIGUNIT_STRINGIFY(Content) #Content
#define IMPLEMENT_RIGUNIT_AUTOMATION_TEST(TUnitStruct) \
	class TUnitStruct##Test : public FControlRigUnitTestBase \
	{ \
	public: \
		TUnitStruct##Test( const FString& InName ) \
		:FControlRigUnitTestBase( InName, false ) {\
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter; } \
		virtual bool IsStressTest() const { return false; } \
		virtual uint32 GetRequiredDeviceNum() const override { return 1; } \
		virtual FString GetTestSourceFileName() const override { return __FILE__; } \
		virtual int32 GetTestSourceFileLine() const override { return __LINE__; } \
	protected: \
		virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override \
		{ \
			OutBeautifiedNames.Add(TEXT(CONTROLRIG_RIGUNIT_STRINGIFY(ControlRig.Units.TUnitStruct))); \
			OutTestCommands.Add(FString()); \
		} \
		TUnitStruct Unit; \
		virtual bool RunTest(const FString& Parameters) override \
		{ \
			Initialize(); \
			Hierarchy->Reset(); \
			Unit = TUnitStruct(); \
			return RunControlRigUnitTest(Parameters); \
		} \
		virtual bool RunControlRigUnitTest(const FString& Parameters); \
		virtual FString GetBeautifiedTestName() const override { return TEXT(CONTROLRIG_RIGUNIT_STRINGIFY(ControlRig.Units.TUnitStruct)); } \
		void InitAndExecute() \
		{ \
			Unit.Initialize(); \
			Unit.Execute(ExecuteContext); \
		} \
		void Execute() \
		{ \
			Unit.Execute(ExecuteContext); \
		} \
	}; \
	namespace\
	{\
		TUnitStruct##Test TUnitStruct##AutomationTestInstance(TEXT(CONTROLRIG_RIGUNIT_STRINGIFY(TUnitStruct##Test))); \
	} \
	bool TUnitStruct##Test::RunControlRigUnitTest(const FString& Parameters)