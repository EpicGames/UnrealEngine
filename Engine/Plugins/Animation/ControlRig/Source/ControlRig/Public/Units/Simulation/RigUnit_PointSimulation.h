// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "Math/ControlRigMathLibrary.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "Math/Simulation/CRSimLinearSpring.h"
#include "Math/Simulation/CRSimPointForce.h"
#include "Math/Simulation/CRSimSoftCollision.h"
#include "Math/Simulation/CRSimPointContainer.h"
#include "RigUnit_PointSimulation.generated.h"

#define UE_API CONTROLRIG_API

USTRUCT()
struct FRigUnit_PointSimulation_DebugSettings
{
	GENERATED_BODY()

	FRigUnit_PointSimulation_DebugSettings()
	{
		bEnabled = false;
		Scale = 1.f;
		CollisionScale = 50.f;
		bDrawPointsAsSpheres = false;
		Color = FLinearColor::Blue;
		WorldOffset = FTransform::Identity;
	}

	/**
	 * If enabled debug information will be drawn 
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "DebugSettings")
	bool bEnabled;

	/**
	 * The size of the debug drawing information
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	float Scale;

	/**
     * The size of the debug drawing information
     */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	float CollisionScale;

	/**
	 * If set to true points will be drawn as spheres with their sizes reflected
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	bool bDrawPointsAsSpheres;

	/**
	 * The color to use for debug drawing
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FLinearColor Color;

	/**
	 * The offset at which to draw the debug information in the world
	 */
	UPROPERTY(EditAnywhere, meta = (Input, EditCondition = "bEnabled"), Category = "DebugSettings")
	FTransform WorldOffset;
};

USTRUCT()
struct FRigUnit_PointSimulation_BoneTarget
{
	GENERATED_BODY()

	FRigUnit_PointSimulation_BoneTarget()
	{
		Bone = NAME_None;
		TranslationPoint = PrimaryAimPoint = SecondaryAimPoint = INDEX_NONE;
	}

	/**
	 * The name of the bone to map
	 */
	UPROPERTY(EditAnywhere, meta = (Input), Category = "BoneTarget")
	FName Bone;

	/**
	 * The index of the point to use for translation
	 */
	UPROPERTY(EditAnywhere, meta = (Input, Constant), Category = "BoneTarget")
	int32 TranslationPoint;

	/**
	 * The index of the point to use for aiming the primary axis.
	 * Use -1 to indicate that you don't want to aim the bone.
	 */
	UPROPERTY(EditAnywhere, meta = (Input, Constant), Category = "BoneTarget")
	int32 PrimaryAimPoint;

	/**
	 * The index of the point to use for aiming the secondary axis.
	 * Use -1 to indicate that you don't want to aim the bone.
	 */
	UPROPERTY(EditAnywhere, meta = (Input, Constant), Category = "BoneTarget")
	int32 SecondaryAimPoint;
};

USTRUCT()
struct FRigUnit_PointSimulation_WorkData
{
	GENERATED_BODY()

	UPROPERTY()
	FCRSimPointContainer Simulation;

	UPROPERTY()
	TArray<FCachedRigElement> BoneIndices;
};

/**
 * Performs point based simulation
 * Note: Disabled for now.
 */
USTRUCT(meta=(DisplayName="Point Simulation", Keywords="Simulate,Verlet,Springs", Deprecated="4.25"))
struct FRigUnit_PointSimulation : public FRigVMFunction_SimBaseMutable
{
	GENERATED_BODY()
	
	FRigUnit_PointSimulation()
	{
		SimulatedStepsPerSecond = 60.f;
		IntegratorType = ERigVMSimPointIntegrateType::Verlet;
		VerletBlend = 4.f;
		bLimitLocalPosition = true;
		bPropagateToChildren = true;
		PrimaryAimAxis = FVector(1.f, 0.f, 0.f);
		SecondaryAimAxis = FVector(0.f, 1.f, 0.f);
		DebugSettings = FRigUnit_PointSimulation_DebugSettings();
		Bezier = FRigVMFourPointBezier();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** The points to simulate */
	UPROPERTY(meta = (Input))
	TArray<FRigVMSimPoint> Points;

	/** The links to connect the points with */
	UPROPERTY(meta = (Input))
	TArray<FCRSimLinearSpring> Links;

	/** The forces to apply */
	UPROPERTY(meta = (Input))
	TArray<FCRSimPointForce> Forces;

	/** The collision volumes to define */
	UPROPERTY(meta = (Input))
	TArray<FCRSimSoftCollision> CollisionVolumes;

	/** The frame rate of the simulation */
	UPROPERTY(meta = (Input, Constant))
	float SimulatedStepsPerSecond;

	/** The type of integrator to use */
	UPROPERTY(meta = (Input, Constant))
	ERigVMSimPointIntegrateType IntegratorType;

	/** The amount of blending to apply per second ( only for verlet integrations )*/
	UPROPERTY(meta = (Input))
	float VerletBlend;

	/** The bones to map to the simulated points. */
	UPROPERTY(meta = (Input, Constant))
	TArray<FRigUnit_PointSimulation_BoneTarget> BoneTargets;

	/**
	 * If set to true bones are placed within the original distance of
	 * the previous local transform. This can be used to avoid stretch.
	 */
	UPROPERTY(meta = (Input))
	bool bLimitLocalPosition;

	/**
	 * If set to true all of the global transforms of the children
	 * of this bone will be recalculated based on their local transforms.
	 * Note: This is computationally more expensive than turning it off.
	 */
	UPROPERTY(meta = (Input, Constant))
	bool bPropagateToChildren;

	/** The primary axis to use for the aim */
	UPROPERTY(meta = (Input))
	FVector PrimaryAimAxis;

	/** The secondary axis to use for the aim */
	UPROPERTY(meta = (Input))
	FVector SecondaryAimAxis;

	/** Debug draw settings for this simulation */
	UPROPERTY(meta = (Input, DetailsOnly))
	FRigUnit_PointSimulation_DebugSettings DebugSettings;

	/** If the simulation has at least four points they will be stored in here. */
	UPROPERTY(meta = (Output))
	FRigVMFourPointBezier Bezier;

	UPROPERTY(transient)
	FRigUnit_PointSimulation_WorkData WorkData;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

#undef UE_API
