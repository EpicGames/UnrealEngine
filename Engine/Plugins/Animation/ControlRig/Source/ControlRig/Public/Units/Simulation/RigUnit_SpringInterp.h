// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "Kismet/KismetMathLibrary.h"
#include "RigUnit_SpringInterp.generated.h"

#define UE_API CONTROLRIG_API

/**
 * Uses a simple spring model to interpolate a float from Current to Target.
 */
USTRUCT(meta=(DisplayName="Spring Interpolate", Keywords="Alpha,SpringInterpolate", TemplateName="SpringInterp", Deprecated="5.0"))
struct FRigUnit_SpringInterp : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
 
	FRigUnit_SpringInterp()
	{
		Current = Target = Result = 0.0f;
		Stiffness = 10.0f;
		CriticalDamping = 2.0f;
		Mass = 10.0f;
		SpringState = FFloatSpringState();
	}
 
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
 
	UPROPERTY(meta = (Input))
	float Current;
 
	UPROPERTY(meta=(Input))
	float Target;
 
	UPROPERTY(meta=(Input))
	float Stiffness;
 
	UPROPERTY(meta=(Input))
	float CriticalDamping;
 
	UPROPERTY(meta=(Input))
	float Mass;
 
	UPROPERTY(meta=(Output))
	float Result;
 
	UPROPERTY()
	FFloatSpringState SpringState;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};
 
/**
 * Uses a simple spring model to interpolate a vector from Current to Target.
 */
USTRUCT(meta=(DisplayName="Spring Interpolate", Keywords="Alpha,SpringInterpolate", TemplateName="SpringInterp", MenuDescSuffix = "(Vector)", Deprecated="5.0"))
struct FRigUnit_SpringInterpVector : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigUnit_SpringInterpVector()
	{
		Current = Target = Result = FVector::ZeroVector;
		Stiffness = 10.0f;
		CriticalDamping = 2.0f;
		Mass = 10.0f;
		SpringState = FVectorSpringState();
	}
 
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
 
	UPROPERTY(meta = (Input))
	FVector Current;
 
	UPROPERTY(meta=(Input))
	FVector Target;
 
	UPROPERTY(meta=(Input))
	float Stiffness;
 
	UPROPERTY(meta=(Input))
	float CriticalDamping;
 
	UPROPERTY(meta=(Input))
	float Mass;
 
	UPROPERTY(meta=(Output))
	FVector Result;
 
	UPROPERTY()
	FVectorSpringState SpringState;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Uses a simple spring model to interpolate a float from Current to Target.
 */
USTRUCT(meta=(DisplayName="Spring Interpolate", Keywords="Alpha,SpringInterpolate,Verlet", Category = "Simulation|Springs", TemplateName="SpringInterp"))
struct FRigUnit_SpringInterpV2 : public FRigVMFunction_SimBase
{
	GENERATED_BODY()

	FRigUnit_SpringInterpV2()
	{
		bUseCurrentInput = false;
		Current = Target = Velocity = Result = SimulatedResult = 0.0f;
		Strength = 4.0f;
		CriticalDamping = 1.0f;
		TargetVelocityAmount = 0.0f;
		bInitializeFromTarget = false;
		Force = 0.0f;
		SpringState = FFloatSpringState();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Rest/target position of the spring. */
	UPROPERTY(meta=(Input))
	float Target;

	/**
	 * The spring strength determines how hard it will pull towards the target. The value is the frequency
	 * at which it will oscillate when there is no damping.
	 */
	UPROPERTY(meta=(Input, ClampMin="0"))
	float Strength;

	/** 
	 * The amount of damping in the spring.
	 * Set it smaller than 1 to make the spring oscillate before stabilizing on the target. 
	 * Set it equal to 1 to reach the target without overshooting. 
	 * Set it higher than one to make the spring take longer to reach the target.
	 */
	UPROPERTY(meta=(Input, ClampMin="0"))
	float CriticalDamping;

	/**
	 * Extra force to apply (since the mass is 1, this is also the acceleration).
	 */
	UPROPERTY(meta = (Input))
	float Force;

	/**
	 * If true, then the Current input will be used to initialize the state, and is required to be a variable that 
	 * holds the current state. If false then the Target value will be used to initialize the state and the Current 
	 * input will be ignored/unnecessary as a state will be maintained by this node.
	 */
	UPROPERTY(meta=(Input, Constant))
	bool bUseCurrentInput;

	/** Current position of the spring. */
	UPROPERTY(meta = (Input, EditCondition = "bUseCurrentInput"))
	float Current;

	/**
	 * The amount that the velocity should be passed through to the spring. A value of 1 will result in more
	 * responsive output, but if the input is noisy or has step changes, these discontinuities will be passed 
	 * through to the output much more than if a smaller value such as 0 is used.
	 */
	UPROPERTY(meta = (Input, ClampMin="0", ClampMax="1"))
	float TargetVelocityAmount;

	/**
	 * If true, then the initial value will be taken from the target value, and not from the current value.
	 */
	UPROPERTY(meta = (Input, Constant, EditCondition = "bUseCurrentInput"))
	bool bInitializeFromTarget;

	/** New position of the spring after delta time. */
	UPROPERTY(meta=(Output))
	float Result;

	/** The resulting velocity of this spring */
	UPROPERTY(meta = (Output))
	float Velocity;

	UPROPERTY()
	float SimulatedResult;

	UPROPERTY()
	FFloatSpringState SpringState;
};

/**
 * Uses a simple spring model to interpolate a vector from Current to Target.
 */
USTRUCT(meta=(DisplayName="Spring Interpolate", Keywords="Alpha,SpringInterpolate,Verlet", Category = "Simulation|Springs", TemplateName="SpringInterp", MenuDescSuffix = "(Vector)"))
struct FRigUnit_SpringInterpVectorV2 : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigUnit_SpringInterpVectorV2()
	{
		bUseCurrentInput = false;
		Current = Target = Velocity = Result = SimulatedResult = FVector::ZeroVector;
		Strength = 4.0f;
		CriticalDamping = 1.0f;
		TargetVelocityAmount = 0.0f;
		bInitializeFromTarget = false;
		Force = FVector::ZeroVector;
		SpringState = FVectorSpringState();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Rest/target position of the spring. */
	UPROPERTY(meta=(Input))
	FVector Target;

	/**
	 * The spring strength determines how hard it will pull towards the target. The value is the frequency
	 * at which it will oscillate when there is no damping.
	 */
	UPROPERTY(meta=(Input, ClampMin="0"))
	float Strength;

	/** 
	 * The amount of damping in the spring.
	 * Set it smaller than 1 to make the spring oscillate before stabilizing on the target. 
	 * Set it equal to 1 to reach the target without overshooting. 
	 * Set it higher than one to make the spring take longer to reach the target.
	 */
	UPROPERTY(meta=(Input, ClampMin="0"))
	float CriticalDamping;

	/**
	 * Extra force to apply (since the mass is 1, this is also the acceleration).
	 */
	UPROPERTY(meta = (Input))
	FVector Force;

	/**
	 * If true, then the Current input will be used to initialize the state, and is required to be a variable that 
	 * holds the current state. If false then the Target value will be used to initialize the state and the Current 
	 * input will be ignored/unnecessary as a state will be maintained by this node.
	 */
	UPROPERTY(meta=(Input, Constant))
	bool bUseCurrentInput;

	/** Current position of the spring. */
	UPROPERTY(meta = (Input, EditCondition = "bUseCurrentInput"))
	FVector Current;

	/**
	 * The amount that the velocity should be passed through to the spring. A value of 1 will result in more
	 * responsive output, but if the input is noisy or has step changes, these discontinuities will be passed 
	 * through to the output much more than if a smaller value such as 0 is used.
	 */
	UPROPERTY(meta = (Input, ClampMin="0", ClampMax="1"))
	float TargetVelocityAmount;

	/**
	 * If true, then the initial value will be taken from the target value, and not from the current value. 
	 */
	UPROPERTY(meta = (Input, Constant, EditCondition = "bUseCurrentInput"))
	bool bInitializeFromTarget;

	/** New position of the spring after delta time. */
	UPROPERTY(meta=(Output))
	FVector Result;

	/** The resulting velocity of this spring */
	UPROPERTY(meta = (Output))
	FVector Velocity;

	UPROPERTY()
	FVector SimulatedResult;

	UPROPERTY()
	FVectorSpringState SpringState;
};

/**
 * Uses a simple spring model to interpolate a quaternion from Current to Target.
 */
USTRUCT(meta=(DisplayName="Spring Interpolate", Keywords="Alpha,SpringInterpolate,Verlet", Category = "Simulation|Springs", TemplateName="SpringInterp", MenuDescSuffix = "(Quaternion)"))
struct FRigUnit_SpringInterpQuaternionV2 : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	// RigUnit_SpringInterpQuaternion is redirected to RigUnit_SpringInterpQuaternionV2
	FRigUnit_SpringInterpQuaternionV2()
	{
		bUseCurrentInput = false;
		Current = Target = Result = SimulatedResult = FQuat::Identity;
		AngularVelocity = FVector::ZeroVector;
		Strength = 4.0f;
		CriticalDamping = 1.0f;
		TargetVelocityAmount = 0.0f;
		bInitializeFromTarget = false;
		Torque = FVector::ZeroVector;
		SpringState = FQuaternionSpringState();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** Rest/target position of the spring. */
	UPROPERTY(meta=(Input))
	FQuat Target;

	/**
	 * The spring strength determines how hard it will pull towards the target. The value is the frequency
	 * at which it will oscillate when there is no damping.
	 */
	UPROPERTY(meta=(Input, ClampMin="0"))
	float Strength;

	/** 
	 * The amount of damping in the spring.
	 * Set it smaller than 1 to make the spring oscillate before stabilizing on the target. 
	 * Set it equal to 1 to reach the target without overshooting. 
	 * Set it higher than one to make the spring take longer to reach the target.
	 */
	UPROPERTY(meta=(Input, ClampMin="0"))
	float CriticalDamping;

	/**
	 * Extra torque to apply (since the moment of inertia is 1, this is also the angular acceleration).
	 */
	UPROPERTY(meta = (Input))
	FVector Torque;

	/**
	 * If true, then the Current input will be used to initialize the state, and is required to be a variable that 
	 * holds the current state. If false then the Target value will be used to initialize the state and the Current 
	 * input will be ignored/unnecessary as a state will be maintained by this node.
	 */
	UPROPERTY(meta=(Input, Constant))
	bool bUseCurrentInput;

	/** Current position of the spring. */
	UPROPERTY(meta = (Input, EditCondition = "bUseCurrentInput"))
	FQuat Current;

	/**
	 * The amount that the velocity should be passed through to the spring. A value of 1 will result in more
	 * responsive output, but if the input is noisy or has step changes, these discontinuities will be passed 
	 * through to the output much more than if a smaller value such as 0 is used.
	 */
	UPROPERTY(meta=(Input, ClampMin="0", ClampMax="1"))
	float TargetVelocityAmount;

	/**
	 * If true, then the initial value will be taken from the target value, and not from the current value. 
	 */
	UPROPERTY(meta = (Input, Constant, EditCondition = "bUseCurrentInput"))
	bool bInitializeFromTarget;

	/** New position of the spring after delta time. */
	UPROPERTY(meta=(Output))
	FQuat Result;

	/** Angular velocity */
	UPROPERTY(meta = (Output))
	FVector AngularVelocity;

	UPROPERTY()
	FQuat SimulatedResult;

	UPROPERTY()
	FQuaternionSpringState SpringState;
};

#undef UE_API
