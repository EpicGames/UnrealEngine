// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ControlRigDeveloper : ModuleRules
    {
        public ControlRigDeveloper(ReadOnlyTargetRules Target) : base(Target)
        {
			// Copying some these from ControlRig.build.cs, our deps are likely leaner
			// and therefore these could be pruned if needed:
			PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "AnimGraphRuntime",
                    "ControlRig",
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "KismetCompiler",
                    "MovieScene",
                    "MovieSceneTracks",
                    "PropertyPath",
                    "Slate",
                    "SlateCore",
                    "InputCore",
                    "TimeManagement",
					"EditorWidgets",
					"MessageLog",
                    "RigVM",
					"ToolWidgets",
                }
            );

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "AnimationCore",
                    "VisualGraphUtils",
                }
            );

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
						"EditorFramework",
                        "UnrealEd",
						"Kismet",
                        "AnimGraph",
                        "BlueprintGraph",
                        "PropertyEditor",
                        "RigVMDeveloper",
						"GraphEditor",
						"ApplicationCore",
						"RigVMEditor",
					}
                );

                PublicDependencyModuleNames.AddRange(
	                new string[]
	                {
		                "AnimationEditorWidgets",
	                }
                );

                PrivateIncludePathModuleNames.Add("ControlRigEditor");
                DynamicallyLoadedModuleNames.Add("ControlRigEditor");
            }
        }
    }
}
