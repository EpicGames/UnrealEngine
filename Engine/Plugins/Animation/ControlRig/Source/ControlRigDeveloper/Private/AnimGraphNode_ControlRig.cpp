// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_ControlRig.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimationGraphSchema.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigObjectBinding.h"
#include "ModularRig.h"
#include "RigVMDeveloperTypeUtils.h"
#include "Misc/DefaultValueHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ControlRig)

#define LOCTEXT_NAMESPACE "AnimGraphNode_ControlRig"

UAnimGraphNode_ControlRig::UAnimGraphNode_ControlRig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ControlRigIOMapping(MakeShared<FControlRigIOMapping>(Node.InputMapping, Node.OutputMapping, CustomPinProperties))
{
	ControlRigIOMapping->GetOnPinCheckStateChangedDelegate().BindUObject(this, &UAnimGraphNode_ControlRig::OnPropertyExposeCheckboxChanged);
	ControlRigIOMapping->GetOnVariableMappingChanged().BindUObject(this, &UAnimGraphNode_ControlRig::OnVariableMappingChanged);
	ControlRigIOMapping->GetOnGetTargetSkeletonDelegate().BindUObject(this, &UAnimGraphNode_ControlRig::GetTargetSkeleton);
	ControlRigIOMapping->GetOnGetTargetAssetReferenceDelegate().BindUObject(this, &UAnimGraphNode_ControlRig::GetTargetAssetReference);
}

FText UAnimGraphNode_ControlRig::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// display control rig here
	return LOCTEXT("AnimGraphNode_ControlRig_Title", "Control Rig");
}

FText UAnimGraphNode_ControlRig::GetTooltipText() const
{
	// display control rig here
	return LOCTEXT("AnimGraphNode_ControlRig_Tooltip", "Evaluates a control rig");
}

void UAnimGraphNode_ControlRig::CreateCustomPins(TArray<UEdGraphPin*>* OldPins)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	
	// we do this to refresh input variables
	ControlRigIOMapping->RebuildExposedProperties();
	
	// Grab the SKELETON class here as when we are reconstructed during during BP compilation
	// the full generated class is not yet present built.
	UClass* TargetClass = GetTargetSkeletonClass();

	if (!TargetClass)
	{
		// Nothing to search for properties
		return;
	}

	// Need the schema to extract pin types
	const UEdGraphSchema_K2* Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	// Default anim schema for util funcions
	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();

#if WITH_EDITOR

	// sustain the current set of custom pins - we'll refrain from changing the node until post load is complete
	FControlRigAssetStrongReference AssetReference = GetTargetAssetReference();
	if (AssetReference.IsValid())
	{
		if (UObject* ReferencedObj = AssetReference.GetEditorAsset())
		{
			if (ReferencedObj->HasAllFlags(RF_NeedPostLoad))
			{
				if(OldPins)
				{
					for (UEdGraphPin* OldPin : *OldPins)
					{
						// do not rebuild sub pins as they will be treated after in UK2Node::RestoreSplitPins
						const bool bIsSubPin = OldPin->ParentPin && OldPin->ParentPin->bHidden;
						if (bIsSubPin)
						{
							continue;
						}
						
						bool bFound = false;
						for (UEdGraphPin* CurrentPin : Pins)
						{
							if (CurrentPin->GetFName() == OldPin->GetFName())
							{
								if (CurrentPin->PinType == OldPin->PinType ||
									AnimGraphDefaultSchema->ArePinTypesCompatible(CurrentPin->PinType, OldPin->PinType))
								{
									bFound = true;
									break;
								}
								
							}							
						}

						if (!bFound)
						{
							FName PropertyName = OldPin->GetFName();
							UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Input, OldPin->PinType, PropertyName);
							NewPin->PinFriendlyName = OldPin->PinFriendlyName;
							
							// Newly created pin does not have an auto generated default value, so we need to generate one here
							// Missing the auto-gen default would cause UEdGraphPin::MovePersistentDataFromOldPin to override
							// the actual default value with the empty auto-gen default, causing BP compiler to complain
							// This is similar to how the following two functions create and initialize new pins, create first 
							// and then set an auto-gen default
							// FOptionalPinManager::CreateVisiblePins()
							// FAnimBlueprintNodeOptionalPinManager::PostInitNewPin()
							AnimGraphDefaultSchema->SetPinAutogeneratedDefaultValueBasedOnType(NewPin);

							AnimGraphDefaultSchema->ResetPinToAutogeneratedDefaultValue(NewPin, false);

							CustomizePinData(NewPin, PropertyName, INDEX_NONE);
						}
					}
				}
				return;
			}
		}
	}
#endif

	// We'll track the names we encounter by removing from this list, if anything remains the properties
	// have been removed from the target class and we should remove them too
	TSet<FName> BeginExposableNames;
	TSet<FName> CurrentlyExposedNames;
	for(const FOptionalPinFromProperty& OptionalPin : CustomPinProperties)
	{
		BeginExposableNames.Add(OptionalPin.PropertyName);

		if(OptionalPin.bShowPin)
		{
			CurrentlyExposedNames.Add(OptionalPin.PropertyName);
		}
	}

	for (const TPair<FName, FRigVMExternalVariable>& VariablePair : ControlRigIOMapping->GetInputVariables())
	{
		FName PropertyName = VariablePair.Key;
		BeginExposableNames.Remove(PropertyName);

		if (CurrentlyExposedNames.Contains(PropertyName))
		{
			const FRigVMExternalVariable& Variable = VariablePair.Value;

			FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(Variable);
			if (!PinType.PinCategory.IsValid())
			{
				continue;
			}

			UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Input, PinType, PropertyName);
			NewPin->PinFriendlyName = FText::FromName(PropertyName);

			// Newly created pin does not have an auto generated default value, so we need to generate one here
			// Missing the auto-gen default would cause UEdGraphPin::MovePersistentDataFromOldPin to override
			// the actual default value with the empty auto-gen default, causing BP compiler to complain
			// This is similar to how the following two functions create and initialize new pins, create first 
			// and then set an auto-gen default
			// FOptionalPinManager::CreateVisiblePins()
			// FAnimBlueprintNodeOptionalPinManager::PostInitNewPin()
			AnimGraphDefaultSchema->SetPinAutogeneratedDefaultValueBasedOnType(NewPin);

			// We cant interrogate CDO here as we may be mid-compile, so we can only really
			// reset to the autogenerated default.
			AnimGraphDefaultSchema->ResetPinToAutogeneratedDefaultValue(NewPin, false);

			// Extract default values from the Target Control Rig if possible
			// Memory could be null if Control Rig is compiling, so only do it if Memory is not null
			if (Variable.IsValid(false))
			{ 
				TArray<FRigVMExternalVariable> ExternalVariables = AssetReference.GetExternalVariables();
				for (FRigVMExternalVariable& ExternalVariable : ExternalVariables)
				{
					const FProperty* Property = ExternalVariable.GetProperty();
					if (Property->GetFName() == PropertyName)
					{
						FString DefaultValue;
							
						// The format the graph pins editor use is different of what property exporter produces, so we use BlueprintEditorUtils to generate the default string
						// Variable.Memory here points to the corresponding property in the Control Rig BP CDO, it was initialized in UAnimGraphNode_ControlRig::RebuildExposedProperties 
						FBlueprintEditorUtils::PropertyValueToString_Direct(Property, Variable.GetMemory(), DefaultValue, this);

						if (!DefaultValue.IsEmpty())
						{
							AnimGraphDefaultSchema->TrySetDefaultValue(*NewPin, DefaultValue);
						}
					}
				}
			} 
			
			// sustain the current set of custom pins - we'll refrain from changing the node until post load is complete 
			CustomizePinData(NewPin, PropertyName, INDEX_NONE);
		}
	}

	const TArray<FControlRigIOMapping::FControlsInfo>& Controls = ControlRigIOMapping->GetControls();
	for (const FControlRigIOMapping::FControlsInfo& ControlInfo : Controls)
	{
		const FName ControlName = ControlInfo.Name;
		BeginExposableNames.Remove(ControlName);

		if (CurrentlyExposedNames.Contains(ControlName))
		{
			const FEdGraphPinType PinType = ControlInfo.PinType;
			if (!PinType.PinCategory.IsValid())
			{
				continue;
			}

			UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Input, PinType, ControlName);
			NewPin->PinFriendlyName = FText::FromName(ControlInfo.Name);

			// Newly created pin does not have an auto generated default value, so we need to generate one here
			// Missing the auto-gen default would cause UEdGraphPin::MovePersistentDataFromOldPin to override
			// the actual default value with the empty auto-gen default, causing BP compiler to complain
			// This is similar to how the following two functions create and initialize new pins, create first 
			// and then set an auto-gen default
			// FOptionalPinManager::CreateVisiblePins()
			// FAnimBlueprintNodeOptionalPinManager::PostInitNewPin()
			AnimGraphDefaultSchema->SetPinAutogeneratedDefaultValueBasedOnType(NewPin);

			// We cant interrogate CDO here as we may be mid-compile, so we can only really
			// reset to the autogenerated default.
			AnimGraphDefaultSchema->ResetPinToAutogeneratedDefaultValue(NewPin, false);

			// Extract default values from the Target Control Rig if possible
			const FString DefaultValue = ControlInfo.DefaultValue;
			if(!DefaultValue.IsEmpty())
			{
				AnimGraphDefaultSchema->TrySetDefaultValue(*NewPin, DefaultValue);
			}
			
			// sustain the current set of custom pins - we'll refrain from changing the node until post load is complete 
			CustomizePinData(NewPin, ControlName, INDEX_NONE);
		}
	}

	// Remove any properties that no longer exist on the target class
	for (FName& RemovedPropertyName : BeginExposableNames)
	{
		CustomPinProperties.RemoveAll([RemovedPropertyName](const FOptionalPinFromProperty& InOptionalPin) { return InOptionalPin.PropertyName == RemovedPropertyName; });
	}
}

void UAnimGraphNode_ControlRig::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	FControlRigAssetStrongReference AssetReference = GetTargetAssetReference();
	if (AssetReference.IsValid())
	{
		URigHierarchy* Hierarchy = nullptr;
		if (FControlRigAssetInterfacePtr EditorAsset = AssetReference.GetEditorAsset())
		{
			Hierarchy = EditorAsset->GetHierarchy();
		}
		if (Hierarchy)
		{
			if (ForSkeleton)
			{
				const FReferenceSkeleton& ReferenceSkeleton = ForSkeleton->GetReferenceSkeleton();
				const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRefBoneInfo();

				for (const FMeshBoneInfo& BoneInfo : BoneInfos)
				{
					const FRigElementKey BoneKey(BoneInfo.Name, ERigElementType::Bone);
					if (Hierarchy->Contains(BoneKey))
					{
						const FRigElementKey ParentKey = Hierarchy->GetFirstParent(BoneKey);

						FName DesiredParentName = NAME_None;
						if (BoneInfo.ParentIndex != INDEX_NONE)
						{
							DesiredParentName = BoneInfos[BoneInfo.ParentIndex].Name;
						}

						if (DesiredParentName != ParentKey.Name)
						{
							FString Message = FString::Printf(TEXT("@@ - Hierarchy discrepancy for bone '%s' - different parents on Control Rig vs SkeletalMesh."), *BoneInfo.Name.ToString());
							MessageLog.Warning(*Message, this);
						}
					}
				}
			}
		}
	}

	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

void UAnimGraphNode_ControlRig::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

void UAnimGraphNode_ControlRig::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Super::CustomizeDetails(DetailBuilder);

	if (!ControlRigIOMapping->CreateVariableMappingWidget(DetailBuilder))
	{
		return;
	}

	TSharedRef<IPropertyHandle> ClassHandle = DetailBuilder.GetProperty(TEXT("Node.ControlRigAssetReference"), GetClass());
	if (ClassHandle->IsValidHandle())
	{
		ClassHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateUObject(this, &UAnimGraphNode_ControlRig::OnInstanceClassChanged, &DetailBuilder));
	}

	// input/output exposure feature END

	// alpha property blending support START
	TSharedRef<IPropertyHandle> NodeHandle = DetailBuilder.GetProperty(FName(TEXT("Node")), GetClass());

	if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, AlphaBoolBlend)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, Alpha)));
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, AlphaScaleBias)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, AlphaCurveName)));
	}

	if ((Node.AlphaInputType != EAnimAlphaInputType::Float)
		&& (Node.AlphaInputType != EAnimAlphaInputType::Curve))
	{
		DetailBuilder.HideProperty(NodeHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, AlphaScaleBiasClamp)));
	}
	// alpha property blending support END
}

void UAnimGraphNode_ControlRig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRequiresNodeReconstruct = false;
	FProperty* ChangedProperty = PropertyChangedEvent.Property;

	if (ChangedProperty)
	{
		if (ChangedProperty->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimNode_ControlRig, ControlRigAssetReference))
		{
			bRequiresNodeReconstruct = true;
			ControlRigIOMapping->RebuildExposedProperties();
		}

		if (ChangedProperty->GetFName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, AlphaInputType))
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
			Modify();

			// Break links to pins going away
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = Pins[PinIndex];
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, Alpha))
				{
					if (Node.AlphaInputType != EAnimAlphaInputType::Float)
					{
						Pin->BreakAllPinLinks();
						RemoveBindings(Pin->PinName);
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, bAlphaBoolEnabled))
				{
					if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
					{
						Pin->BreakAllPinLinks();
						RemoveBindings(Pin->PinName);
					}
				}
				else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, AlphaCurveName))
				{
					if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
					{
						Pin->BreakAllPinLinks();
						RemoveBindings(Pin->PinName);
					}
				}
			}

			bRequiresNodeReconstruct = true;
		}
	}

	if (bRequiresNodeReconstruct)
	{
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UAnimGraphNode_ControlRig::PostReconstructNode()
{
	// Fix default values that were serialized directly with property exported strings (not valid for pin editors)
#if WITH_EDITOR
	if (!IsTemplate())
	{
		static UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		static UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();

		// fix up any pin data if it needs to 
		for (UEdGraphPin* CurrentPin : Pins)
		{
			const FName& PinCategory = CurrentPin->PinType.PinCategory;

			// Target only struct types vector and transform 
			// For rotator, if it was created with incorect format, it was serialized with DefaultValue = L"0, 0, 0", so it can not be corrected without risking changing users value
			if (PinCategory == UEdGraphSchema_K2::PC_Struct)
			{
				const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();
				const FName& PinName = CurrentPin->GetFName();

				if (CurrentPin->PinType.PinSubCategoryObject == VectorStruct)
				{
					// If InitFromString succeeds, it has bad format, re-encode
					FVector FixVector = FVector::ZeroVector;
					if (FixVector.InitFromString(CurrentPin->DefaultValue))
					{
						const FString DefaultValue = FString::Printf(TEXT("%f,%f,%f"), FixVector.X, FixVector.Y, FixVector.Z);
						AnimGraphDefaultSchema->TrySetDefaultValue(*CurrentPin, DefaultValue);
					}
				}
				else if (CurrentPin->PinType.PinSubCategoryObject == TransformStruct)
				{
					// If pin was serialized with bad format, transforms would stay empty, so try to get the value from ControlRig or set to Identity as last resort
					if (CurrentPin->DefaultValue.IsEmpty())
					{
						FTransform FixTransform = FTransform::Identity;
						FString DefaultValue = FixTransform.ToString();

						const FRigVMExternalVariable* RigVMExternalVariable = (CurrentPin->Direction == EGPD_Input) ? 
							ControlRigIOMapping->GetInputVariables().Find(CurrentPin->GetFName()) : (CurrentPin->Direction == EGPD_Output) ?
							ControlRigIOMapping->GetOutputVariables().Find(CurrentPin->GetFName()) : nullptr;

						if (RigVMExternalVariable != nullptr && RigVMExternalVariable->IsValid())
						{
							FControlRigAssetStrongReference AssetReference = GetTargetAssetReference();
							if (AssetReference.IsValid())
							{
								const FName& PropertyName = CurrentPin->GetFName();
								TArray<FRigVMExternalVariable> ExternalVariables = AssetReference.GetExternalVariables();
								for (FRigVMExternalVariable& ExternalVariable : ExternalVariables)
								{
									const FProperty* Property = ExternalVariable.GetProperty();
									if (Property->GetFName() == PropertyName)
									{
										// The format the graph pins editor use is different of what property exporter produces, so we use BlueprintEditorUtils to generate the default string
										// Variable.Memory here points to the corresponding property in the Control Rig BP CDO, it was initialized in UAnimGraphNode_ControlRig::RebuildExposedProperties 
										FBlueprintEditorUtils::PropertyValueToString_Direct(Property, RigVMExternalVariable->GetMemory(), DefaultValue, this);
										break;
									}
								}
							}
						}

						AnimGraphDefaultSchema->SetPinAutogeneratedDefaultValue(CurrentPin, DefaultValue);
						AnimGraphDefaultSchema->TrySetDefaultValue(*CurrentPin, DefaultValue);
					}
				}
			}
		}
	}
#endif // WITH_EDITOR

	Super::PostReconstructNode();
}

void UAnimGraphNode_ControlRig::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();

	if (UObject* CDO = Node.ControlRigAssetReference.Get())
	{
		PreloadObject(CDO);
		const FModularRigModel& Model = Node.ControlRigAssetReference.GetModularRigModel();
		Model.ForEachModule([this](const FRigModuleReference* Module) -> bool
		{
				// Need to make sure rig modules are loaded before compiling the AnimBP
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (!Module->Class_DEPRECATED.IsNull())
				{
					(void)Module->Class_DEPRECATED.LoadSynchronous();
				}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				else if (Module->ControlRigAssetReference.IsValid())
				{
					constexpr bool bForceLoad = true;
					(void)Module->ControlRigAssetReference.LoadSynchronous(bForceLoad);
				}
			return true;
		});
	}
}

UObject* UAnimGraphNode_ControlRig::GetJumpTargetForDoubleClick() const
{
	const FControlRigAssetStrongReference& RigSource = GetTargetAssetReference();
	if (RigSource.IsValid())
	{
		return RigSource.GetEditorAsset();
	}

	return nullptr;
}

UStruct* UAnimGraphNode_ControlRig::GetTargetSkeletonStruct() const
{
	FControlRigAssetStrongReference RigSource = GetTargetAssetReference();
	if (RigSource.IsValid())
	{
		if (RigSource.GetBlueprintClass())
		{
			UClass* GeneratedClass = RigSource.GetBlueprintClass();
			if(GeneratedClass && GeneratedClass->ClassGeneratedBy)
			{
				UBlueprint* Blueprint = CastChecked<UBlueprint>(GeneratedClass->ClassGeneratedBy);
				if(Blueprint)
				{
					if (Blueprint->SkeletonGeneratedClass)
					{
						return Blueprint->SkeletonGeneratedClass;
					}
				}
			}
		}
		if (RigSource.GetControlRigAsset())
		{
			return RigSource.GetVariablesStruct();
		}
	}

	return nullptr;
}

UStruct* UAnimGraphNode_ControlRig::GetTargetStruct() const
{
	const FControlRigAssetStrongReference& RigSource = GetTargetAssetReference();
	if (RigSource.IsValid())
	{
		return RigSource.GetVariablesStruct();
	}

	return nullptr;
}

FControlRigAssetStrongReference UAnimGraphNode_ControlRig::GetTargetAssetReference() const
{
	return Node.GetControlRigAssetReference();
}

void UAnimGraphNode_ControlRig::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, Alpha))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Float);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBias.GetFriendlyName(Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName));
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, bAlphaBoolEnabled))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Bool);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_ControlRig, AlphaCurveName))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Curve);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_ControlRig::OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FGuardValue_Bitfield(bDisableOrphanPinSaving, true);
	ReconstructNode();

	// see if any of my child has the mapping, and clear them
	if (NewState == ECheckBoxState::Checked)
	{
		FScopedTransaction Transaction(LOCTEXT("PropertyExposedChanged", "Expose Property to Pin"));
		Modify();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());

		bool bInput = ControlRigIOMapping->IsInputProperty(PropertyName);
		// if checked, we clear mapping
		// and unclear all children
		ControlRigIOMapping->SetIOMapping(bInput, PropertyName, NAME_None);
	}
}

void UAnimGraphNode_ControlRig::OnVariableMappingChanged(const FName& PathName, const FName& Curve, bool bInput)
{
	FScopedTransaction Transaction(LOCTEXT("VariableMappingChanged", "Change Variable Mapping"));
	Modify();

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());

	// @todo: this is not enough when we start breaking down struct
	ControlRigIOMapping->SetIOMapping(bInput, PathName, Curve);
}

USkeleton* UAnimGraphNode_ControlRig::GetTargetSkeleton() const
{
	USkeleton* TargetSkeleton = nullptr;
	if (UAnimBlueprint* AnimBP = CastChecked<UAnimBlueprint>(GetBlueprint()))
	{
		TargetSkeleton = AnimBP->TargetSkeleton;
	}
	return TargetSkeleton;
}
#undef LOCTEXT_NAMESPACE

