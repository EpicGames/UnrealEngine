// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigBlueprintCompiler.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "Kismet2/KismetReinstanceUtilities.h"

bool FControlRigBlueprintCompiler::CanCompile(const UBlueprint* Blueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Blueprint && Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(UControlRig::StaticClass()))
	{
		return true;
	}

	return false;
}

void FControlRigBlueprintCompiler::Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FControlRigBlueprintCompilerContext Compiler(Blueprint, Results, CompileOptions);
	Compiler.Compile();
}

void FControlRigBlueprintCompilerContext::SpawnNewClass(const FString& NewClassName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	NewRigVMBlueprintGeneratedClass = FindObject<UControlRigBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);

	if (NewRigVMBlueprintGeneratedClass == nullptr)
	{
		NewRigVMBlueprintGeneratedClass = NewObject<UControlRigBlueprintGeneratedClass>(Blueprint->GetOutermost(), FName(*NewClassName), RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(NewRigVMBlueprintGeneratedClass);
	}
	NewClass = NewRigVMBlueprintGeneratedClass;
}

void FControlRigBlueprintCompilerContext::CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMBlueprintCompilerContext::CopyTermDefaultsToDefaultObject(DefaultObject);

	UControlRigBlueprint* ControlRigBlueprint = Cast<UControlRigBlueprint>(Blueprint);
	if (ControlRigBlueprint)
	{
		// here, CDO is initialized from BP,
		// and in UControlRig::InitializeFromCDO,
		// other Control Rig Instances are then initialized From the CDO 
		UControlRig* ControlRig = CastChecked<UControlRig>(DefaultObject);
		
		// copy hierarchy
		{
			TGuardValue<bool> Guard(ControlRig->GetHierarchy()->GetSuspendNotificationsFlag(), false);
			ControlRig->GetHierarchy()->CopyHierarchy(ControlRigBlueprint->Hierarchy);
			ControlRig->GetHierarchy()->ResetPoseToInitial(ERigElementType::All);

#if WITH_EDITOR
			// link CDO hierarchy to BP's hierarchy
			// so that whenever BP's hierarchy is modified, CDO's hierarchy is kept in sync
			// Other instances of Control Rig can make use of CDO to reset to the initial state
			ControlRigBlueprint->Hierarchy->ClearListeningHierarchy();
			ControlRigBlueprint->Hierarchy->RegisterListeningHierarchy(ControlRig->GetHierarchy());
#endif
		}

		// notify clients that the hierarchy has changed
		ControlRig->GetHierarchy()->Notify(ERigHierarchyNotification::HierarchyReset, {});

		ControlRig->DrawContainer = ControlRigBlueprint->URigVMBlueprint::GetDrawContainer();
		ControlRig->Influences = ControlRigBlueprint->Influences;
		ControlRig->RigModuleSettings = ControlRigBlueprint->RigModuleSettings;
	}
}
