// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigBlueprintLegacy.h"

#include "RigVMBlueprintGeneratedClass.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphNode_Comment.h"
#include "Engine/SkeletalMesh.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "ControlRig.h"
#include "Graph/ControlRigGraph.h"
#include "Graph/ControlRigGraphSchema.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectGlobals.h"
#include "ControlRigObjectVersion.h"
#include "BlueprintCompilationManager.h"
#include "ModularRig.h"
#include "ModularRigController.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Hierarchy/RigUnit_SetBoneTransform.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "RigVMPythonUtils.h"
#include "RigVMTypeUtils.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "Rigs/RigControlHierarchy.h"
#include "Settings/ControlRigSettings.h"
#include "Units/ControlRigNodeWorkflow.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_DynamicHierarchy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigBlueprintLegacy)

#if WITH_EDITOR
#include "IControlRigEditorModule.h"
#include "Kismet2/WatchedPin.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/Transactor.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "ScopedTransaction.h"
#include "Algo/Count.h"
#endif//WITH_EDITOR

#define LOCTEXT_NAMESPACE "ControlRigBlueprint"

UControlRigBlueprint::UControlRigBlueprint(const FObjectInitializer& ObjectInitializer)
	: URigVMBlueprint(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	GizmoLibrary_DEPRECATED = nullptr;
#endif

	if(GetClass() == UControlRigBlueprint::StaticClass())
	{
		IControlRigEditorAssetInterface::CommonInitialization(ObjectInitializer);
	}
}

UControlRigBlueprint::UControlRigBlueprint()
{
	ModulesRecompilationBracket = 0;
}

void UControlRigBlueprint::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	URigVMBlueprint::GetPreloadDependencies(OutDeps);
	IControlRigEditorAssetInterface::GetPreloadDependencies(OutDeps);
}

void UControlRigBlueprint::Serialize(FArchive& Ar)
{
	IControlRigEditorAssetInterface::Serialize(Ar);

	if(Ar.IsLoading())
	{
		if(Model_DEPRECATED || FunctionLibrary_DEPRECATED)
		{
			TGuardValue<bool> DisableClientNotifs(RigVMClient.bSuspendNotifications, true);
			RigVMClient.SetFromDeprecatedData(Model_DEPRECATED, FunctionLibrary_DEPRECATED);
		}
	}
}

UClass* UControlRigBlueprint::RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO)
{
	UClass* Result = URigVMBlueprint::RegenerateClass(ClassToRegenerate, PreviousCDO);
	OnRegeneratedClass(ClassToRegenerate, PreviousCDO);
	if (UControlRigBlueprintGeneratedClass* Generated = Cast<UControlRigBlueprintGeneratedClass>(Result))
	{
		Generated->PreviewSkeletalMesh = PreviewSkeletalMesh.ToSoftObjectPath();
		Generated->bAllowMultipleInstances = bAllowMultipleInstances;
		Generated->bExposesAnimatableControls = bExposesAnimatableControls;
		Generated->ControlRigType = ControlRigType;
		Generated->ItemTypeDisplayName = ItemTypeDisplayName;
		Generated->RigModuleSettings = RigModuleSettings;
		Generated->ModuleReferenceData = ModuleReferenceData;
		Generated->CustomThumbnail = CustomThumbnail;
		Generated->bSupportsInversion = bSupportsInversion;
		Generated->bSupportsControls = bSupportsControls;
	}
	return Result;
}

void UControlRigBlueprint::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	URigVMBlueprint::PreSave(ObjectSaveContext);
	IControlRigEditorAssetInterface::PreSave(ObjectSaveContext);

	if (UControlRigBlueprintGeneratedClass* Generated = Cast<UControlRigBlueprintGeneratedClass>(GeneratedClass))
	{
		Generated->PreviewSkeletalMesh = PreviewSkeletalMesh.ToSoftObjectPath();
		Generated->bAllowMultipleInstances = bAllowMultipleInstances;
		Generated->bExposesAnimatableControls = bExposesAnimatableControls;
		Generated->ControlRigType = ControlRigType;
		Generated->ItemTypeDisplayName = ItemTypeDisplayName;
		Generated->RigModuleSettings = RigModuleSettings;
		Generated->ModuleReferenceData = ModuleReferenceData;
		Generated->CustomThumbnail = CustomThumbnail;
		Generated->bSupportsInversion = bSupportsInversion;
		Generated->bSupportsControls = bSupportsControls;
	}
}

void UControlRigBlueprint::PostLoad()
{
	URigVMBlueprint::PostLoad();
	
	{
#if WITH_EDITOR
		
		// correct the offset transforms
		if (GetLinkerCustomVersion(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::ControlOffsetTransform)
		{
			HierarchyContainer_DEPRECATED.ControlHierarchy.PostLoad();
			if (HierarchyContainer_DEPRECATED.ControlHierarchy.Num() > 0)
			{
				MarkDirtyDuringLoad();
			}

			for (FRigControl& Control : HierarchyContainer_DEPRECATED.ControlHierarchy)
			{
				const FTransform PreviousOffsetTransform = Control.GetTransformFromValue(ERigControlValueType::Initial);
				Control.OffsetTransform = PreviousOffsetTransform;
				Control.InitialValue = Control.Value;

				if (Control.ControlType == ERigControlType::Transform)
				{
					Control.InitialValue = FRigControlValue::Make<FTransform>(FTransform::Identity);
				}
				else if (Control.ControlType == ERigControlType::TransformNoScale)
				{
					Control.InitialValue = FRigControlValue::Make<FTransformNoScale>(FTransformNoScale::Identity);
				}
				else if (Control.ControlType == ERigControlType::EulerTransform)
				{
					Control.InitialValue = FRigControlValue::Make<FEulerTransform>(FEulerTransform::Identity);
				}
			}
		}

		// convert the hierarchy from V1 to V2
		if (GetLinkerCustomVersion(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RigHierarchyV2)
		{
			UBlueprint::Modify();
			
			TGuardValue<bool> SuspendNotifGuard(GetHierarchy()->GetSuspendNotificationsFlag(), true);
			
			GetHierarchy()->Reset();
			GetHierarchyController()->ImportFromHierarchyContainer(HierarchyContainer_DEPRECATED, false);
		}

		// perform backwards compat value upgrades
		TArray<URigVMGraph*> GraphsToValidate = GetAllModels();
		for (int32 GraphIndex = 0; GraphIndex < GraphsToValidate.Num(); GraphIndex++)
		{
			URigVMGraph* GraphToValidate = GraphsToValidate[GraphIndex];
			if(GraphToValidate == nullptr)
			{
				continue;
			}

			for(URigVMNode* Node : GraphToValidate->GetNodes())
			{
				TArray<URigVMPin*> Pins = Node->GetAllPinsRecursively();
				for(URigVMPin* Pin : Pins)
				{
					if(Pin->GetCPPTypeObject() == StaticEnum<ERigElementType>())
					{
						if(Pin->GetDefaultValue() == TEXT("Space"))
						{
							if(URigVMController* Controller = GetController(GraphToValidate))
							{
								FRigVMControllerNotifGuard NotifGuard(Controller, true);
								FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Override);
								Controller->SetPinDefaultValue(Pin->GetPinPath(), TEXT("Null"), false, false, false);
							}
						}
					}
				}
			}
		}

#endif
	}

	// upgrade the gizmo libraries to shape libraries
	if(!GizmoLibrary_DEPRECATED.IsNull() || GetLinkerCustomVersion(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::RenameGizmoToShape)
	{
		// if it's an older file and it doesn't have the GizmoLibrary stored,
		// refer to the previous default.
		GetShapeLibraries().Reset();

		if(!GizmoLibrary_DEPRECATED.IsNull())
		{
			ShapeLibrariesToLoadOnPackageLoaded.Add(GizmoLibrary_DEPRECATED.ToString());
		}
		else
		{
			static const FString DefaultGizmoLibraryPath = TEXT("/ControlRig/Controls/DefaultGizmoLibrary.DefaultGizmoLibrary");
			ShapeLibrariesToLoadOnPackageLoaded.Add(DefaultGizmoLibraryPath);
		}

		TArray<UObject*> ArchetypeInstances = GetArchetypeInstances(true, true);
		for (UObject* Instance : ArchetypeInstances)
		{
			if (UControlRig* InstanceRig = Cast<UControlRig>(Instance))
			{
				InstanceRig->ShapeLibraries.Reset();
				InstanceRig->GizmoLibrary_DEPRECATED.Reset();
			}
		}
	}

	if(GetArrayConnectionMap().IsEmpty() && !ConnectionMap_DEPRECATED.IsEmpty())
	{
		for(const TPair<FRigElementKey, FRigElementKey>& Pair : ConnectionMap_DEPRECATED)
		{
			GetArrayConnectionMap().Add(Pair.Key, FRigElementKeyCollection({Pair.Value}));
		}
	}
	
	// Copy draw container to RigVMBlueprint
	if (DrawContainer_DEPRECATED.Num() > 0)
	{
		DrawContainer = DrawContainer_DEPRECATED;
	}
	
	IControlRigEditorAssetInterface::PostLoad();
}

void UControlRigBlueprint::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	URigVMBlueprint::PostTransacted(TransactionEvent);
	IControlRigEditorAssetInterface::PostTransacted(TransactionEvent);
}

void UControlRigBlueprint::PostDuplicate(bool bDuplicateForPIE)
{
	URigVMBlueprint::PostDuplicate(bDuplicateForPIE);
	return IControlRigEditorAssetInterface::PostDuplicate(bDuplicateForPIE);
}

void UControlRigBlueprint::PostRename(UObject* OldOuter, const FName OldName)
{
	URigVMBlueprint::PostRename(OldOuter, OldName);
	IControlRigEditorAssetInterface::PostRename(OldOuter, OldName);
}

void UControlRigBlueprint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	URigVMBlueprint::PostEditChangeProperty(PropertyChangedEvent);
	IControlRigEditorAssetInterface::PostEditChangeProperty(PropertyChangedEvent);
}

void UControlRigBlueprint::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	URigVMBlueprint::PostEditChangeChainProperty(PropertyChangedEvent);
	IControlRigEditorAssetInterface::PostEditChangeChainProperty(PropertyChangedEvent);
}

#if WITH_EDITOR
bool UControlRigBlueprint::CopyBlueprintToAsset(UControlRigBlueprint* InBlueprint, UControlRigRuntimeAsset* InAsset)
{
	Super::CopyBlueprintToAsset(InBlueprint, InAsset);
	
	InAsset->Hierarchy->CopyHierarchy(InBlueprint->Hierarchy);
	InAsset->HierarchySettings = InBlueprint->HierarchySettings;
	InAsset->ModularRigModel = InBlueprint->ModularRigModel;
	InAsset->ModularRigSettings = InBlueprint->ModularRigSettings;
	InAsset->RigModuleSettings = InBlueprint->RigModuleSettings;
	InAsset->ControlRigType = InBlueprint->ControlRigType;
	InAsset->ItemTypeDisplayName = InBlueprint->ItemTypeDisplayName;
	InAsset->SourceHierarchyImport = InBlueprint->SourceHierarchyImport;
	InAsset->SourceCurveImport = InBlueprint->SourceCurveImport;
	InAsset->PreviewSkeletalMesh = InBlueprint->PreviewSkeletalMesh;
	InAsset->ModuleReferenceData = InBlueprint->ModuleReferenceData;
	InAsset->ArrayConnectionMap = InBlueprint->ArrayConnectionMap;
	InAsset->Influences = InBlueprint->Influences;
	InAsset->bSupportsInversion = InBlueprint->bSupportsInversion;
	InAsset->bSupportsControls = InBlueprint->bSupportsControls;
	InAsset->bExposesAnimatableControls = InBlueprint->bExposesAnimatableControls; 
	InAsset->bAllowMultipleInstances = InBlueprint->bAllowMultipleInstances; 
	InAsset->CustomThumbnail = InBlueprint->CustomThumbnail; 
	InAsset->ShapeLibraries = InBlueprint->ShapeLibraries; 
	
	return true;
}
#endif

UClass* UControlRigBlueprint::GetControlRigClass() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return IControlRigEditorAssetInterface::GetControlRigClass();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FControlRigAssetStrongReference UControlRigBlueprint::GetControlRigAssetReference() const
{
	UClass* Class = GetRigVMGeneratedClass();
	FControlRigAssetStrongReference Source;
	Source.Set(TSubclassOf<UControlRig>(Class));
	return Source;
}

void UControlRigBlueprint::SetPreviewMeshImpl(USkeletalMesh* PreviewMesh)
{
	PreviewSkeletalMesh = PreviewMesh;
}

#if WITH_EDITOR
TArray<UControlRigBlueprint*> UControlRigBlueprint::GetCurrentlyOpenRigBlueprints()
{
	TArray<UControlRigBlueprint*> Blueprints;
	TArray<FControlRigAssetInterfacePtr> Assets = GetCurrentlyOpenRigAssets();
	Algo::Transform(Assets, Blueprints, [](const FControlRigAssetInterfacePtr& Asset)
		{
			return Cast<UControlRigBlueprint>(Asset.GetObject());
		});
	return Blueprints;
}

void UControlRigBlueprint::GetBackwardsCompatibilityPublicFunctions(TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders)
{
	if (GetLinkerCustomVersion(FControlRigObjectVersion::GUID) < FControlRigObjectVersion::StoreFunctionsInGeneratedClass)
	{
		for (const FRigVMOldPublicFunctionData& OldPublicFunction : PublicFunctions_DEPRECATED)
		{
			BackwardsCompatiblePublicFunctions.Add(OldPublicFunction.Name);
		}
	}
	else
	{
		if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RigVMSaveFunctionAccessInModel)
		{
			if(const TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = GetRigVMClientHost()->GetRigVMGraphFunctionHost())
			{
				FRigVMGraphFunctionStore* Store = FunctionHost->GetRigVMGraphFunctionStore();
				for (const FRigVMGraphFunctionData& FunctionData : Store->PublicFunctions)
				{
					BackwardsCompatiblePublicFunctions.Add(FunctionData.Header.Name);
					URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionData.Header.LibraryPointer.GetNodeSoftPath().ResolveObject());
					OldHeaders.Add(LibraryNode, FunctionData.Header);
				}
			}
		}
	}

	// Addressing issue where PublicGraphFunctions is populated, but the model PublicFunctionNames is not
	URigVMFunctionLibrary* FunctionLibrary = GetLocalFunctionLibrary();
	if (FunctionLibrary)
	{
		if (PublicGraphFunctions.Num() > FunctionLibrary->PublicFunctionNames.Num())
		{
			for (const FRigVMGraphFunctionHeader& PublicHeader : PublicGraphFunctions)
			{
				BackwardsCompatiblePublicFunctions.Add(PublicHeader.Name);
			}
		}
	}
}

void UControlRigBlueprint::PatchVariableNodesOnLoad()
{
	IControlRigEditorAssetInterface::PatchVariableNodesOnLoad();
	URigVMBlueprint::PatchVariableNodesOnLoad();
}

TArray<FString> UControlRigBlueprint::GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset)
{
	TArray<FString> InternalCommands;
	InternalCommands.Add(TEXT("import unreal"));
	InternalCommands.Add(TEXT("unreal.load_module('ControlRigDeveloper')"));

	if (bCreateAsset)
	{
		InternalCommands.Add(TEXT("factory = unreal.ControlRigBlueprintFactory"));
		InternalCommands.Add(FString::Printf(TEXT("asset = factory.create_new_control_rig_asset(desired_package_path = '%s')"), *InNewBlueprintName));
	}
	else
	{
		InternalCommands.Add(FString::Printf(TEXT("asset = unreal.load_object(name = '%s', outer = None)"), *InNewBlueprintName));
	}
	
	InternalCommands.Add(TEXT("hierarchy = asset.hierarchy"));
	InternalCommands.Add(TEXT("hierarchy_controller = hierarchy.get_controller()"));
	
	InternalCommands.Add(TEXT("library = asset.get_local_function_library()"));
	InternalCommands.Add(TEXT("library_controller = asset.get_controller(library)"));
	InternalCommands.Add(TEXT("asset.set_auto_vm_recompile(False)"));

	return InternalCommands;
}
#endif


#undef LOCTEXT_NAMESPACE


