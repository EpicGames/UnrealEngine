// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigDeveloper.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigBlueprintCompiler.h"
#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"

#define LOCTEXT_NAMESPACE "ControlRigDeveloperModule"

DEFINE_LOG_CATEGORY(LogControlRigDeveloper);

class FControlRigDeveloperModule : public IControlRigDeveloperModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Compiler customization for animation controllers */
	FControlRigBlueprintCompiler ControlRigBlueprintCompiler;

	static TSharedPtr<FKismetCompilerContext> GetControlRigCompiler(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions);

	virtual void RegisterPinTypeColor(UStruct* Struct, const FLinearColor Color) override;

	virtual void UnRegisterPinTypeColor(UStruct* Struct) override;
	
	virtual const FLinearColor* FindPinTypeColor(UStruct* Struct) const override;

private:

	TMap<UStruct*, FLinearColor> PinTypeColorMap;
};

void FControlRigDeveloperModule::StartupModule()
{
	// Register blueprint compiler
	FKismetCompilerContext::RegisterCompilerForBP(UControlRigBlueprint::StaticClass(), &FControlRigDeveloperModule::GetControlRigCompiler);
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetCompilers().Add(&ControlRigBlueprintCompiler);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = false;
	InitOptions.bAllowClear = true;
	MessageLogModule.RegisterLogListing("ControlRigLog", LOCTEXT("ControlRigLog", "Control Rig Log"), InitOptions);
}

void FControlRigDeveloperModule::ShutdownModule()
{
	IKismetCompilerInterface* KismetCompilerModule = FModuleManager::GetModulePtr<IKismetCompilerInterface>("KismetCompiler");
	if (KismetCompilerModule)
	{
		KismetCompilerModule->GetCompilers().Remove(&ControlRigBlueprintCompiler);
	}

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing("ControlRigLog");
}

TSharedPtr<FKismetCompilerContext> FControlRigDeveloperModule::GetControlRigCompiler(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
{
	return TSharedPtr<FKismetCompilerContext>(new FControlRigBlueprintCompilerContext(BP, InMessageLog, InCompileOptions));
}

void FControlRigDeveloperModule::RegisterPinTypeColor(UStruct* Struct, const FLinearColor Color)
{
	if (Struct)
	{ 
		PinTypeColorMap.FindOrAdd(Struct) = Color;
	}
}

void FControlRigDeveloperModule::UnRegisterPinTypeColor(UStruct* Struct)
{
	PinTypeColorMap.Remove(Struct);
}

const FLinearColor* FControlRigDeveloperModule::FindPinTypeColor(UStruct* Struct) const
{
	return PinTypeColorMap.Find(Struct);
}

IMPLEMENT_MODULE(FControlRigDeveloperModule, ControlRigDeveloper)

#undef LOCTEXT_NAMESPACE
