// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ControlRigEditorAssetUndoClient.h"

#include "ControlRigEditorAsset.h"

FControlRigEditorAssetUndoClient::FControlRigEditorAssetUndoClient(
	FPrivateToken, 
	const FControlRigAssetInterfacePtr& InOwner)
	: WeakOwner(InOwner)
{}

void FControlRigEditorAssetUndoClient::PostUndo(bool bSuccess)
{
	if (WeakOwner.IsValid())
	{
		WeakOwner->GetModularRigModel().PatchModelsOnLoad();
	}
}

void FControlRigEditorAssetUndoClient::PostRedo(bool bSuccess)
{
	// Same as PostUndo 
	PostUndo(bSuccess);
}

#endif // WITH_EDITOR
