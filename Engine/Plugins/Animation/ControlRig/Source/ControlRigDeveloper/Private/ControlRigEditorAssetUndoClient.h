// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "EditorUndoClient.h"
#include "UObject/WeakInterfacePtr.h"

class IControlRigEditorAssetInterface;
template <typename InInterfaceType> class TScriptInterface;

using FControlRigAssetInterfacePtr = TScriptInterface<IControlRigEditorAssetInterface>;

/** Undo redo handler for the control rig editor asset interface */
class FControlRigEditorAssetUndoClient final
	: public FSelfRegisteringEditorUndoClient 
	, FNoncopyable
{
	// Only IControlRigEditorAssetInterface should ever construct this
	friend class IControlRigEditorAssetInterface;

	/** Private token required to construct this class */
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FControlRigEditorAssetUndoClient(FPrivateToken, const FControlRigAssetInterfacePtr& InOwner);

private:
	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

	/** The control rig editor asset interface that owns this handler */
	TWeakInterfacePtr<IControlRigEditorAssetInterface> WeakOwner;
};

#endif // WITH_EDITOR
