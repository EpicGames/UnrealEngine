// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigIOMapping.h"
#include "DetailLayoutBuilder.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRig.h"
#include "ControlRigObjectBinding.h"
#include "DetailWidgetRow.h"
#include "Widgets/SRigVMVariableMappingWidget.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

FControlRigIOMapping::FControlRigIOMapping(TMap<FName, FName>& InInputMapping, TMap<FName, FName>& InOutputMapping, TArray<FOptionalPinFromProperty>& InCustomPinProperties)
	: InputMapping(InInputMapping)
	, OutputMapping(InOutputMapping)
	, CustomPinProperties(InCustomPinProperties)
{
}

FControlRigIOMapping::~FControlRigIOMapping()
{
}

bool FControlRigIOMapping::CreateVariableMappingWidget(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// We dont allow multi-select here
	if (DetailBuilder.GetSelectedObjects().Num() > 1)
	{
		return false;
	}
	
	if (!GetTargetAssetReference().IsValid())
	{
		return false;
	}

	// input/output exposure feature START
	RebuildExposedProperties();

	IDetailCategoryBuilder& InputCategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("Input")));

	FDetailWidgetRow& InputWidgetRow = InputCategoryBuilder.AddCustomRow(FText::FromString(TEXT("Input")));
#if WITH_EDITOR
	InputWidgetRow.WholeRowContent()
		[
			SNew(SRigVMVariableMappingWidget)
				.OnVariableMappingChanged(FOnRigVMVariableMappingChanged::CreateSP(this, &FControlRigIOMapping::OnVariableMappingChanged, true))
				.OnGetVariableMapping(FOnRigVMGetVariableMapping::CreateSP(this, &FControlRigIOMapping::GetVariableMapping, true))
				.OnGetAvailableMapping(FOnRigVMGetAvailableMapping::CreateSP(this, &FControlRigIOMapping::GetAvailableMapping, true))
				.OnCreateVariableMapping(FOnRigVMCreateVariableMapping::CreateSP(this, &FControlRigIOMapping::CreateVariableMapping, true))
				.OnVariableOptionAvailable(FOnRigVMVarOptionAvailable::CreateSP(this, &FControlRigIOMapping::IsAvailableToMapToCurve, true))
				.OnPinGetCheckState(FOnRigVMPinGetCheckState::CreateSP(this, &FControlRigIOMapping::IsPropertyExposed))
				.OnPinCheckStateChanged(FOnRigVMPinCheckStateChanged::CreateSP(this, &FControlRigIOMapping::OnPropertyExposeCheckboxChanged))
				.OnPinIsEnabledCheckState(FOnRigVMPinIsCheckEnabled::CreateSP(this, &FControlRigIOMapping::IsPropertyExposeEnabled))
		];
#endif

	IDetailCategoryBuilder& OutputCategoryBuilder = DetailBuilder.EditCategory(FName(TEXT("Output")));
	FDetailWidgetRow& OutputWidgetRow = OutputCategoryBuilder.AddCustomRow(FText::FromString(TEXT("Output")));
#if WITH_EDITOR
	OutputWidgetRow.WholeRowContent()
		[
			SNew(SRigVMVariableMappingWidget)
				.OnVariableMappingChanged(FOnRigVMVariableMappingChanged::CreateSP(this, &FControlRigIOMapping::OnVariableMappingChanged, false))
				.OnGetVariableMapping(FOnRigVMGetVariableMapping::CreateSP(this, &FControlRigIOMapping::GetVariableMapping, false))
				.OnGetAvailableMapping(FOnRigVMGetAvailableMapping::CreateSP(this, &FControlRigIOMapping::GetAvailableMapping, false))
				.OnCreateVariableMapping(FOnRigVMCreateVariableMapping::CreateSP(this, &FControlRigIOMapping::CreateVariableMapping, false))
				.OnVariableOptionAvailable(FOnRigVMVarOptionAvailable::CreateSP(this, &FControlRigIOMapping::IsAvailableToMapToCurve, false))
				.OnPinGetCheckState(FOnRigVMPinGetCheckState::CreateSP(this, &FControlRigIOMapping::IsPropertyExposed))
				.OnPinCheckStateChanged(FOnRigVMPinCheckStateChanged::CreateSP(this, &FControlRigIOMapping::OnPropertyExposeCheckboxChanged))
				.OnPinIsEnabledCheckState(FOnRigVMPinIsCheckEnabled::CreateSP(this, &FControlRigIOMapping::IsPropertyExposeEnabled))
		];
#endif

	return true;
}

void FControlRigIOMapping::RebuildExposedProperties()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	const FControlRigAssetStrongReference RigSource = GetTargetAssetReference();
	if (!RigSource.IsValid())
	{
		return;
	}
	
	if (UObject* EditorAsset = RigSource.GetEditorAsset())
	{
		if (EditorAsset->HasAllFlags(RF_NeedPostLoad))
		{
			return;
		}
	}

	TSet<FName> OldOptionalPinNames;
	TSet<FName> OldExposedPinNames;
	for (const FOptionalPinFromProperty& OptionalPin : CustomPinProperties)
	{
		OldOptionalPinNames.Add(OptionalPin.PropertyName);

		if (OptionalPin.bShowPin)
		{
			OldExposedPinNames.Add(OptionalPin.PropertyName);
		}
	}
	CustomPinProperties.Reset();

	// go through exposed properties, and clean up
	GetVariables(RigSource, true, bIgnoreVariablesWithNoMemory, InputVariables);
	// we still update OUtputvariables
	// we don't want output to be exposed
	GetVariables(RigSource, false, bIgnoreVariablesWithNoMemory, OutputVariables);

	// clear IO variables that don't exist anymore
	auto ClearInvalidMapping = [](const TMap<FName, FRigVMExternalVariable>& InVariables, TMap<FName, FName>& InOutMapping)
		{
			TArray<FName> KeyArray;
			InOutMapping.GenerateKeyArray(KeyArray);

			for (int32 Index = 0; Index < KeyArray.Num(); ++Index)
			{
				// if this input doesn't exist anymore
				if (!InVariables.Contains(KeyArray[Index]))
				{
					InOutMapping.Remove(KeyArray[Index]);
				}
			}
		};

	ClearInvalidMapping(InputVariables, InputMapping);
	ClearInvalidMapping(OutputVariables, OutputMapping);

	auto MakeOptionalPin = [&OldExposedPinNames](FName InPinName)-> FOptionalPinFromProperty
		{
			FOptionalPinFromProperty OptionalPin;
			OptionalPin.PropertyName = InPinName;
			OptionalPin.bShowPin = OldExposedPinNames.Contains(InPinName);
			OptionalPin.bCanToggleVisibility = true;
			OptionalPin.bIsOverrideEnabled = false;
			return OptionalPin;
		};

	for (auto Iter = InputVariables.CreateConstIterator(); Iter; ++Iter)
	{
		CustomPinProperties.Add(MakeOptionalPin(Iter.Key()));
	}

	// also add all of the controls
	const TArray<FControlsInfo>& Controls = GetControls();
	for (const FControlsInfo& ControlInfo : Controls)
	{
		CustomPinProperties.Add(MakeOptionalPin(ControlInfo.Name));
	}
}

bool FControlRigIOMapping::IsInputProperty(const FName& PropertyName) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// this is true for both input variables and controls
	return InputVariables.Contains(PropertyName) || !OutputVariables.Contains(PropertyName);
}

void FControlRigIOMapping::SetIOMapping(bool bInput, const FName& SourceProperty, const FName& TargetCurve)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	const FControlRigAssetStrongReference& TargetSource = GetTargetAssetReference();
	if (TargetSource.IsValid())
	{
		TMap<FName, FName>& MappingData = (bInput) ? InputMapping : OutputMapping;
		TArray<FRigVMExternalVariable> ExternalVariables = TargetSource.GetExternalVariables();
		FRigVMExternalVariable* Variable = ExternalVariables.FindByPredicate([SourceProperty](const FRigVMExternalVariable& Variable)
			{
				return Variable.GetName() == SourceProperty;
			});
		if (Variable)
		{
			// if it's valid as of now, we add it
			bool bIsReadOnly = Variable->IsReadOnly();
			if (!bInput || !bIsReadOnly)
			{
				if (TargetCurve == NAME_None)
				{
					MappingData.Remove(SourceProperty);
				}
				else
				{
					MappingData.FindOrAdd(SourceProperty) = TargetCurve;
				}
			}
		}
	}
}

FName FControlRigIOMapping::GetIOMapping(bool bInput, const FName& SourceProperty) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	const TMap<FName, FName>& MappingData = (bInput) ? InputMapping : OutputMapping;
	if (const FName* NameFound = MappingData.Find(SourceProperty))
	{
		return *NameFound;
	}

	return NAME_None;
}

const FControlRigIOMapping::FControlsInfo* FControlRigIOMapping::FindControlElement(const FName& InControlName) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	const TArray<FControlsInfo>& Controls = GetControls();
	return Controls.FindByPredicate([InControlName](const FControlsInfo& Info)
		{
			return Info.Name == InControlName;
		});
}

bool FControlRigIOMapping::IsAvailableToMapToCurve(const FName& PropertyName, bool bInput) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// find if input or output
	// ensure it could convert to float
	const FRigVMExternalVariable* Variable = (bInput) ? InputVariables.Find(PropertyName) : OutputVariables.Find(PropertyName);
	if (Variable)
	{
		return Variable->GetExtendedCPPType() == TEXT("float");
	}

	if (const FControlsInfo* ControlInfo = FindControlElement(PropertyName))
	{
		return (ControlInfo->ControlType == ERigControlType::Float || ControlInfo->ControlType == ERigControlType::ScaleFloat);
	}

	return ensure(false);
}

bool FControlRigIOMapping::IsPropertyExposeEnabled(FName PropertyName) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// if known exposable, and and if it hasn't been exposed yet
	if (CustomPinProperties.ContainsByPredicate([PropertyName](const FOptionalPinFromProperty& InOptionalPin) { return InOptionalPin.PropertyName == PropertyName; }))
	{
		return IsInputProperty(PropertyName);
	}

	return false;
}

ECheckBoxState FControlRigIOMapping::IsPropertyExposed(FName PropertyName) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (CustomPinProperties.ContainsByPredicate([PropertyName](const FOptionalPinFromProperty& InOptionalPin) { return InOptionalPin.bShowPin && InOptionalPin.PropertyName == PropertyName; }))
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void FControlRigIOMapping::OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FOptionalPinFromProperty* FoundPin = CustomPinProperties.FindByPredicate([PropertyName](const FOptionalPinFromProperty& InOptionalPin)
		{
			return InOptionalPin.PropertyName == PropertyName;
		});

	if (FoundPin)
	{
		FoundPin->bShowPin = !FoundPin->bShowPin;

		if (OnPinCheckStateChangedDelegate.IsBound())
		{
			OnPinCheckStateChangedDelegate.Execute(NewState, PropertyName);
		}
	}
}

const TArray<FControlRigIOMapping::FControlsInfo>& FControlRigIOMapping::GetControls() const
{
	const FControlRigAssetStrongReference ControlRigAssetReference = GetTargetAssetReference();
	USkeleton* TargetSkeleton = GetTargetSkeleton();
	return RigControlsData.GetControls(ControlRigAssetReference, TargetSkeleton);
}

const TArray<FControlRigIOMapping::FControlsInfo>& FControlRigIOMapping::FRigControlsData::GetControls(const FControlRigAssetStrongReference& ControlRigAssetReference, USkeleton* TargetSkeleton) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (ControlsInfoAssetReference != ControlRigAssetReference)
	{
		ControlsInfo.Reset();
		ControlsInfoAssetReference = ControlRigAssetReference;

		if (ControlRigAssetReference.IsValid())
		{
			if (TargetSkeleton != nullptr)
			{
				UControlRig* TemplateRig = ControlRigAssetReference.CreateInstance(GetTransientPackage(), NAME_None, RF_Transient);
				TemplateRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
				TemplateRig->GetObjectBinding()->BindToObject(TargetSkeleton);
				TemplateRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, TemplateRig->GetObjectBinding()->GetBoundObject());

				TemplateRig->Initialize();
				TemplateRig->SetBoneInitialTransformsFromRefSkeleton(TargetSkeleton->GetReferenceSkeleton());
				{
					// Empty event queue to evaluate only construction
					TGuardValue<TArray<FName>> EventQueueGuard(TemplateRig->EventQueue, {});
					TemplateRig->Evaluate_AnyThread();
				}

				if (const URigHierarchy* Hierarchy = TemplateRig->GetHierarchy())
				{
					Hierarchy->ForEach<FRigControlElement>([&](FRigControlElement* ControlElement) -> bool
						{
							if (Hierarchy->IsAnimatable(ControlElement))
							{
								FControlsInfo Info;
								Info.Name = ControlElement->GetFName();
								Info.DisplayName = ControlElement->GetName();
								Info.PinType = TemplateRig->GetHierarchy()->GetControlPinType(ControlElement);
								Info.ControlType = ControlElement->Settings.ControlType;
								Info.DefaultValue = TemplateRig->GetHierarchy()->GetControlPinDefaultValue(ControlElement, true);
								ControlsInfo.Add(Info);
							}
							return true;
						});
				}

				TemplateRig->MarkAsGarbage();
			}
		}
	}

	return ControlsInfo;
}

void FControlRigIOMapping::GetVariables(const FControlRigAssetStrongReference& TargetSource, bool bInput, bool bIgnoreVariablesWithNoMemory, TMap<FName, FRigVMExternalVariable>& OutVariables)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	OutVariables.Reset();

	if (TargetSource.IsValid())
	{
		TArray<FRigVMExternalVariable> ExternalVariables = TargetSource.GetExternalVariables();
		for (FRigVMExternalVariable& ExternalVariable : ExternalVariables)
		{
			if (ExternalVariable.IsPublic())
			{
				if (!bInput || !ExternalVariable.IsReadOnly())
				{
					if (bIgnoreVariablesWithNoMemory && ExternalVariable.GetMemory() == nullptr)
					{
						continue;
					}
					OutVariables.Add(ExternalVariable.GetName(), ExternalVariable);
				}
			}
		}
	}
}

#if WITH_EDITOR

void FControlRigIOMapping::OnVariableMappingChanged(const FName& PathName, const FName& Curve, bool bInput)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (OnVariableMappingChangedDelegate.IsBound())
	{
		OnVariableMappingChangedDelegate.Execute(PathName, Curve, bInput);
	}
}

FName FControlRigIOMapping::GetVariableMapping(const FName& PathName, bool bInput)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// @todo: this is not enough when we start breaking down struct
	return GetIOMapping(bInput, PathName);
}

void FControlRigIOMapping::GetAvailableMapping(const FName& PathName, TArray<FName>& OutArray, bool bInput)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	OutArray.Reset();

	if (USkeleton * TargetSkeleton = GetTargetSkeleton())
	{
		TargetSkeleton->GetCurveMetaDataNames(OutArray);

		// also add all controls
		const TArray<FControlsInfo>& Controls = GetControls();
		for (const FControlsInfo& ControlInfo : Controls)
		{
			OutArray.Add(ControlInfo.Name);
		}

		// we want to exclude anything that has been mapped already
		TArray<FName> TmpInputMapping, TmpOutputMapping;
		InputMapping.GenerateValueArray(TmpInputMapping);
		OutputMapping.GenerateValueArray(TmpOutputMapping);

		// I have to remove Input/Output Curves from OutArray
		for (int32 Index = 0; Index < OutArray.Num(); ++Index)
		{
			const FName& Item = OutArray[Index];

			if (TmpInputMapping.Contains(Item))
			{
				OutArray.RemoveAt(Index);
				--Index;
			}
			else if (TmpOutputMapping.Contains(Item))
			{
				OutArray.RemoveAt(Index);
				--Index;
			}
		}
	}
}

void FControlRigIOMapping::CreateVariableMapping(const FString& FilteredText, TArray< TSharedPtr<FRigVMVariableMappingInfo> >& OutArray, bool bInput)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// should have latest
	OutArray.Reset();

	bool bDoFiltering = !FilteredText.IsEmpty();
	const TMap<FName, FRigVMExternalVariable>& Variables = (bInput) ? InputVariables : OutputVariables;
	for (auto Iter = Variables.CreateConstIterator(); Iter; ++Iter)
	{
		const FName& Name = Iter.Key();
		const FString& DisplayName = Name.ToString();

		const FString MappedName = GetVariableMapping(Name, bInput).ToString();
		// make sure it doesn't fit any of them
		if (!bDoFiltering ||
			(DisplayName.Contains(FilteredText) || MappedName.Contains(FilteredText)))
		{
			TSharedRef<FRigVMVariableMappingInfo> Item = FRigVMVariableMappingInfo::Make(Iter.Key());

			FRigVMVariableMappingInfo& ItemRaw = Item.Get();
			FString PathString = ItemRaw.GetPathName().ToString();
			FString DisplayString = ItemRaw.GetDisplayName();

			OutArray.Add(Item);
		}
	}

	if (bInput)
	{
		// add all controls
		const TArray<FControlsInfo>& Controls = GetControls();
		for (const FControlsInfo& ControlInfo : Controls)
		{
			const FName ControlName = ControlInfo.Name;
			const FString& DisplayName = ControlInfo.DisplayName;

			if (!bDoFiltering || DisplayName.Contains(FilteredText))
			{
				TSharedRef<FRigVMVariableMappingInfo> Item = FRigVMVariableMappingInfo::Make(ControlName);
				OutArray.Add(Item);
			}
		}
	}
}

#endif

const UClass* FControlRigIOMapping::GetTargetClass() const
{
	return nullptr;
}

FControlRigAssetStrongReference FControlRigIOMapping::GetTargetAssetReference() const
{
	return (OnGetTargetAssetReferenceDelegate.IsBound()) ? OnGetTargetAssetReferenceDelegate.Execute() : nullptr;
}

USkeleton* FControlRigIOMapping::GetTargetSkeleton() const
{
	return (OnGetTargetSkeletonDelegate.IsBound()) ? OnGetTargetSkeletonDelegate.Execute() : nullptr;
}
