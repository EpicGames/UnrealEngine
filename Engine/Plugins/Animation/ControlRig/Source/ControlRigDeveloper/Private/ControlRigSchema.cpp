// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSchema.h"
#include "ControlRigBlueprintLegacy.h"
#include "RigVMModel/RigVMController.h"
#include "Rigs/RigHierarchyPose.h"
#include "Rigs/RigPhysics.h"
#include "Curves/CurveFloat.h"
#include "Units/RigUnitContext.h"
#include "Units/Modules/RigUnit_ConnectorExecution.h"
#include "Units/Modules/RigUnit_ConnectionCandidates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSchema)

UControlRigSchema::UControlRigSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetExecuteContextStruct(FControlRigExecuteContext::StaticStruct());
}

bool UControlRigSchema::ShouldUnfoldStruct(URigVMController* InController, const UStruct* InStruct) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY
	
	if(!Super::ShouldUnfoldStruct(InController, InStruct))
	{
		return false;
	}
	if (InStruct == TBaseStructure<FQuat>::Get())
	{
		return false;
	}
	if (InStruct == FRuntimeFloatCurve::StaticStruct())
	{
		return false;
	}
	if (InStruct == FRigPose::StaticStruct())
	{
		return false;
	}
	if (InStruct == FRigPhysicsSolverID::StaticStruct())
	{
		return false;
	}
	
	return true;
}

bool UControlRigSchema::SupportsUnitFunction_NoLock(URigVMController* InController, const FRigVMFunction* InUnitFunction, FRigVMRegistryHandle& InRegistry) const
{
	if (InUnitFunction->Struct == FRigUnit_ConnectorExecution::StaticStruct() ||
		InUnitFunction->Struct == FRigUnit_GetCandidates::StaticStruct() ||
		InUnitFunction->Struct == FRigUnit_DiscardMatches::StaticStruct())
	{
		if (InController)
		{
			if (UControlRigBlueprint* Blueprint = Cast<UControlRigBlueprint>(InController->GetOuter()))
			{
				return Blueprint->IsControlRigModule();
			}
		}
	}
	return Super::SupportsUnitFunction_NoLock(InController, InUnitFunction, InRegistry);
}
