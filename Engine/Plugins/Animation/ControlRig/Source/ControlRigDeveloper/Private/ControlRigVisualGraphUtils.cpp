// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigVisualGraphUtils.h"

#include "ControlRig.h"
#include "VisualGraphUtilsModule.h"

#if WITH_EDITOR

#include "HAL/PlatformApplicationMisc.h"
#include "RigVMModel/RigVMNode.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"

FAutoConsoleCommandWithWorldAndArgs FCmdControlRigVisualGraphUtilsDumpHierarchy
(
	TEXT("VisualGraphUtils.ControlRig.TraverseHierarchy"),
	TEXT("Traverses the hierarchy for a given control rig"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
	{
		const FName BeginExecuteEventName = FRigUnit_BeginExecution::EventName;
		const FName PrepareForExecuteEventName = FRigUnit_PrepareForExecution::EventName;
		const FName InverseExecuteEventName = FRigUnit_InverseExecution::EventName;

		if(InParams.Num() == 0)
		{
			UE_LOGF(LogVisualGraphUtils, Error, "Unsufficient parameters. Command usage:");
			UE_LOGF(LogVisualGraphUtils, Error, "Example: VisualGraphUtilsControlRig.TraverseHierarchy /Game/Animation/ControlRig/BasicControls_CtrlRig event=update");
			UE_LOGF(LogVisualGraphUtils, Error, "Provide one path name to an instance of a control rig.");
			UE_LOGF(LogVisualGraphUtils, Error, "Optionally provide the event name (%ls, %ls or %ls).", *BeginExecuteEventName.ToString(), *PrepareForExecuteEventName.ToString(), *InverseExecuteEventName.ToString());
			return;
		}

		TArray<FString> ObjectPathNames;
		FName EventName = BeginExecuteEventName;
		for(const FString& InParam : InParams)
		{
			static const FString ObjectPathToken = TEXT("path"); 
			static const FString EventPathToken = TEXT("event"); 
			FString Token = ObjectPathToken;
			FString Content = InParam;

			if(InParam.Contains(TEXT("=")))
			{
				const int32 Pos = InParam.Find(TEXT("="));
				Token = InParam.Left(Pos);
				Content = InParam.Mid(Pos+1);
				Token.TrimStartAndEndInline();
				Token.ToLowerInline();
				Content.TrimStartAndEndInline();
			}

			if(Token == ObjectPathToken)
			{
				ObjectPathNames.Add(Content);
			}
			else if(Token == EventPathToken)
			{
				EventName = *Content;
			}
		}

		TArray<URigHierarchy*> Hierarchies;
		for(const FString& ObjectPathName : ObjectPathNames)
		{
			if(UObject* Object = FindObject<UObject>(nullptr, *ObjectPathName, EFindObjectFlags::None))
			{
				if(UControlRig* CR = Cast<UControlRig>(Object))
				{
					Hierarchies.Add(CR->GetHierarchy());
				}
				else if(URigHierarchy* Hierarchy = Cast<URigHierarchy>(Object))
				{
					Hierarchies.Add(Hierarchy);
				}
				else
				{
					UE_LOGF(LogVisualGraphUtils, Error, "Object is not a hierarchy / nor a Control Rig / or short name was provided: '%ls'", *ObjectPathName);
					return;
				}
			}
			else
			{
				UE_LOGF(LogVisualGraphUtils, Error, "Object with pathname '%ls' not found.", *ObjectPathName);
				return;
			}
		}

		if(Hierarchies.Num() == 0)
		{
			UE_LOGF(LogVisualGraphUtils, Error, "No hierarchy found.");
			return;
		}

		const FString DotGraphContent = FControlRigVisualGraphUtils::DumpRigHierarchyToDotGraph(Hierarchies[0], EventName);
		FPlatformApplicationMisc::ClipboardCopy(*DotGraphContent);
		UE_LOGF(LogVisualGraphUtils, Display, "The content has been copied to the clipboard.");
	})
);

#endif

FString FControlRigVisualGraphUtils::DumpRigHierarchyToDotGraph(URigHierarchy* InHierarchy, const FName& InEventName)
{
	check(InHierarchy);

	FVisualGraph Graph(TEXT("Rig"));

	struct Local
	{
		static FName GetNodeNameForElement(const FRigBaseElement* InElement)
		{
			check(InElement);
			return GetNodeNameForElementIndex(InElement->GetIndex());
		}

		static FName GetNodeNameForElementIndex(int32 InElementIndex)
		{
			return *FString::Printf(TEXT("Element_%d"), InElementIndex);
		}

		static TArray<int32> VisitParents(const FRigBaseElement* InElement, URigHierarchy* InHierarchy, FVisualGraph& OutGraph)
		{
			TArray<int32> Result;
			FRigBaseElementParentArray Parents = InHierarchy->GetParents(InElement);

			for(int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
			{
				const int32 ParentNodeIndex = VisitElement(Parents[ParentIndex], InHierarchy, OutGraph);
				Result.Add(ParentNodeIndex);
			}

			return Result;
		}

		static int32 VisitElement(const FRigBaseElement* InElement, URigHierarchy* InHierarchy, FVisualGraph& OutGraph)
		{
			if(InElement->GetType() == ERigElementType::Curve)
			{
				return INDEX_NONE;
			}
			
			const FName NodeName = GetNodeNameForElement(InElement);
			int32 NodeIndex = OutGraph.FindNode(NodeName);
			if(NodeIndex != INDEX_NONE)
			{
				return NodeIndex;
			}

			EVisualGraphShape Shape = EVisualGraphShape::Ellipse;
			TOptional<FLinearColor> Color;

			switch(InElement->GetType())
			{
				case ERigElementType::Bone:
				{
					Shape = EVisualGraphShape::Box;
						
					if(Cast<FRigBoneElement>(InElement)->BoneType == ERigBoneType::User)
					{
						Color = FLinearColor::Green;
					}
					break;
				}
				case ERigElementType::Null:
				{
					Shape = EVisualGraphShape::Diamond;
					break;
				}
				case ERigElementType::Control:
				{
					FLinearColor ShapeColor = Cast<FRigControlElement>(InElement)->Settings.ShapeColor; 
					ShapeColor.A = 1.f;
					Color = ShapeColor;
					break;
				}
				default:
				{
					break;
				}
			}

			NodeIndex = OutGraph.AddNode(NodeName, InElement->GetFName(), Color, Shape);

			if(NodeIndex != INDEX_NONE)
			{
				TArray<int32> Parents = VisitParents(InElement, InHierarchy, OutGraph);
				TArray<FRigElementWeight> Weights = InHierarchy->GetParentWeightArray(InElement);
				for(int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
				{
					const int32 ParentNodeIndex = Parents[ParentIndex];
					if(ParentNodeIndex != INDEX_NONE)
					{
						const TOptional<FLinearColor> EdgeColor;
						TOptional<EVisualGraphStyle> Style;
						if(Weights.IsValidIndex(ParentIndex))
						{
							if(Weights[ParentIndex].IsAlmostZero())
							{
								Style = EVisualGraphStyle::Dotted;
							}
						}
						OutGraph.AddEdge(
							NodeIndex,
							ParentNodeIndex,
							EVisualGraphEdgeDirection::SourceToTarget,
							NAME_None,
							TOptional<FName>(),
							EdgeColor,
							Style);
					}
				}
			}

			return NodeIndex;
		}
	};

	InHierarchy->ForEach([InHierarchy, &Graph](const FRigBaseElement* InElement)
	{
		Local::VisitElement(InElement, InHierarchy, Graph);
		return true;
	});

#if WITH_EDITOR
	
	if(!InEventName.IsNone())
	{
		if(UControlRig* CR = InHierarchy->GetTypedOuter<UControlRig>())
		{
			const FRigDependenciesProviderForControlRig DependenciesProvider(CR, InEventName);
			TRigHierarchyDependencyMap Dependencies = DependenciesProvider.GetRigHierarchyDependencies();

			for(const TRigHierarchyDependencyMapPair& Dependency : Dependencies)
			{
				if (!Dependency.Key.IsElement())
				{
					continue;
				}
				
				const int32 TargetElementIndex = Dependency.Key.Index;
				
				for(const FRigHierarchyRecord SourceElementRecord : Dependency.Value)
				{
					if (!SourceElementRecord.IsElement())
					{
						continue;
					}
					
					FName EdgeName = *FString::Printf(TEXT("Dependency_%d_%d"), SourceElementRecord.Index, TargetElementIndex);
					if(Graph.FindEdge(EdgeName) != INDEX_NONE)
					{
						continue;
					}
					
					const TOptional<FLinearColor> EdgeColor = FLinearColor::Gray;
					TOptional<EVisualGraphStyle> Style = EVisualGraphStyle::Dashed;

					const FName SourceNodeName = Local::GetNodeNameForElementIndex(SourceElementRecord.Index);
					const FName TargetNodeName = Local::GetNodeNameForElementIndex(TargetElementIndex);
					const int32 SourceNodeIndex = Graph.FindNode(SourceNodeName);
					const int32 TargetNodeIndex = Graph.FindNode(TargetNodeName);

					if(SourceNodeIndex == INDEX_NONE || TargetNodeIndex == INDEX_NONE)
					{
						continue;
					}
					
					Graph.AddEdge(
						TargetNodeIndex,
						SourceNodeIndex,
						EVisualGraphEdgeDirection::SourceToTarget,
						EdgeName,
						TOptional<FName>(),
						EdgeColor,
						Style);
				}					
			}
		}
	}
	
#endif
	
	return Graph.DumpDot();
}
