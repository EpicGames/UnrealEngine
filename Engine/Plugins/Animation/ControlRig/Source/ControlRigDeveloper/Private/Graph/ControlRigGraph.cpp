// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/ControlRigGraph.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRig.h"
#include "RigVMModel/RigVMGraph.h"
#include "Units/RigUnit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigGraph)

#define LOCTEXT_NAMESPACE "ControlRigGraph"

UControlRigGraph::UControlRigGraph()
{
	bSuspendModelNotifications = false;
	bIsTemporaryGraphForCopyPaste = false;
	LastHierarchyTopologyVersion = INDEX_NONE;
	bIsFunctionDefinition = false;
}

void UControlRigGraph::InitializeFromAsset(FRigVMEditorAssetInterfacePtr InAsset)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Super::InitializeFromAsset(InAsset);

	const FControlRigAssetInterfacePtr ControlRigBlueprint(InAsset->GetObject());
	URigHierarchy* Hierarchy = ControlRigBlueprint->GetHierarchy();

	if(UControlRig* ControlRig = Cast<UControlRig>(InAsset->GetObjectBeingDebugged()))
	{
		Hierarchy = ControlRig->GetHierarchy();
	}

	if(Hierarchy)
	{
		CacheNameLists(Hierarchy, &ControlRigBlueprint->GetDrawContainer(), ControlRigBlueprint->GetShapeLibraries());
	}
}

#if WITH_EDITOR

TArray<TSharedPtr<FRigVMStringWithTag>> UControlRigGraph::EmptyElementNameList;

const TArray<TSharedPtr<FRigVMStringWithTag>>* UControlRigGraph::GetNameListForWidget(const FString& InWidgetName) const
{
	if (InWidgetName == TEXT("BoneName"))
	{
		return GetBoneNameList();
	}
	if (InWidgetName == TEXT("ControlName"))
	{
		return GetControlNameListWithoutAnimationChannels();
	}
	if (InWidgetName == TEXT("SpaceName"))
	{
		return GetNullNameList();
	}
	if (InWidgetName == TEXT("CurveName"))
	{
		return GetCurveNameList();
	}
	return nullptr;
}

void UControlRigGraph::CacheNameLists(URigHierarchy* InHierarchy, const FRigVMDrawContainer* DrawContainer, TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries)
{
	if (UControlRigGraph* OuterGraph = Cast<UControlRigGraph>(GetOuter()))
	{
		return;
	}

	check(InHierarchy);
	check(DrawContainer);

	UControlRig* ControlRig = InHierarchy->GetTypedOuter<UControlRig>();

	if(LastHierarchyTopologyVersion != InHierarchy->GetTopologyVersion())
	{
		ElementNameLists.FindOrAdd(ERigElementType::All);
		ElementNameLists.FindOrAdd(ERigElementType::Bone);
		ElementNameLists.FindOrAdd(ERigElementType::Null);
		ElementNameLists.FindOrAdd(ERigElementType::Control);
		ElementNameLists.FindOrAdd(ERigElementType::Curve);
		ElementNameLists.FindOrAdd(ERigElementType::Physics);
		ElementNameLists.FindOrAdd(ERigElementType::Reference);
		ElementNameLists.FindOrAdd(ERigElementType::Connector);
		ElementNameLists.FindOrAdd(ERigElementType::Socket);

		TArray<TSharedPtr<FRigVMStringWithTag>>& AllNameList = ElementNameLists.FindChecked(ERigElementType::All);
		TArray<TSharedPtr<FRigVMStringWithTag>>& BoneNameList = ElementNameLists.FindChecked(ERigElementType::Bone);
		TArray<TSharedPtr<FRigVMStringWithTag>>& NullNameList = ElementNameLists.FindChecked(ERigElementType::Null);
		TArray<TSharedPtr<FRigVMStringWithTag>>& ControlNameList = ElementNameLists.FindChecked(ERigElementType::Control);
		TArray<TSharedPtr<FRigVMStringWithTag>>& CurveNameList = ElementNameLists.FindChecked(ERigElementType::Curve);
		TArray<TSharedPtr<FRigVMStringWithTag>>& PhysicsNameList = ElementNameLists.FindChecked(ERigElementType::Physics);
		TArray<TSharedPtr<FRigVMStringWithTag>>& ReferenceNameList = ElementNameLists.FindChecked(ERigElementType::Reference);
		TArray<TSharedPtr<FRigVMStringWithTag>>& ConnectorNameList = ElementNameLists.FindChecked(ERigElementType::Connector);
		TArray<TSharedPtr<FRigVMStringWithTag>>& SocketNameList = ElementNameLists.FindChecked(ERigElementType::Socket);
		
		CacheNameListForHierarchy<FRigBaseElement>(ControlRig, InHierarchy, AllNameList, false);
		CacheNameListForHierarchy<FRigBoneElement>(ControlRig, InHierarchy, BoneNameList, false);
		CacheNameListForHierarchy<FRigNullElement>(ControlRig, InHierarchy, NullNameList, false);
		CacheNameListForHierarchy<FRigControlElement>(ControlRig, InHierarchy, ControlNameList, false);
		CacheNameListForHierarchy<FRigControlElement>(ControlRig, InHierarchy, ControlNameListWithoutAnimationChannels, true);
		CacheNameListForHierarchy<FRigCurveElement>(ControlRig, InHierarchy, CurveNameList, false);
		CacheNameListForHierarchy<FRigReferenceElement>(ControlRig, InHierarchy, ReferenceNameList, false);
		CacheNameListForHierarchy<FRigConnectorElement>(ControlRig, InHierarchy, ConnectorNameList, false);
		CacheNameListForHierarchy<FRigSocketElement>(ControlRig, InHierarchy, SocketNameList, false);

		LastHierarchyTopologyVersion = InHierarchy->GetTopologyVersion();
	}
	CacheNameList<FRigVMDrawContainer>(*DrawContainer, DrawingNameList);

	// always update the connector name list since the connector may have been re-resolved
	TArray<TSharedPtr<FRigVMStringWithTag>>& ConnectorNameList = ElementNameLists.FindChecked(ERigElementType::Connector);
	CacheNameListForHierarchy<FRigConnectorElement>(ControlRig, InHierarchy, ConnectorNameList, false);

	ShapeNameList.Reset();
	ShapeNameList.Add(MakeShared<FRigVMStringWithTag>(FName(NAME_None).ToString()));

	TMap<FString, FString> LibraryNameMap;
	if(ControlRig)
	{
		LibraryNameMap = ControlRig->ShapeLibraryNameMap;
	}

	for(const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrarySoftPtr : ShapeLibraries)
	{
		if(ShapeLibrarySoftPtr.IsNull() || !ShapeLibrarySoftPtr.IsValid())
		{
			(void)ShapeLibrarySoftPtr.LoadSynchronous();
		}

		UControlRigShapeLibrary* ShapeLibrary = ShapeLibrarySoftPtr.Get();
		if (!ShapeLibrary)
		{
			continue;
		}
		
		const bool bUseNameSpace = ShapeLibraries.Num() > 1;
		auto AddShapeDefinition = [&ShapeNameListRef = ShapeNameList, ShapeLibrary, bUseNameSpace, &LibraryNameMap](const FControlRigShapeDefinition& ShapeDefinition)
		{
			if (!ShapeDefinition.ShapeProxy.IsNull())
			{
				const FString ShapeName = UControlRigShapeLibrary::GetShapeName(ShapeLibrary, bUseNameSpace, LibraryNameMap, ShapeDefinition);
				ShapeNameListRef.Add(MakeShared<FRigVMStringWithTag>(ShapeName));
			}
		};
		
		AddShapeDefinition(ShapeLibrary->DefaultShape);		
		for (const FControlRigShapeDefinition& Shape : ShapeLibrary->Shapes)
		{
			AddShapeDefinition(Shape);
		}
	}
}

const TArray<TSharedPtr<FRigVMStringWithTag>>* UControlRigGraph::GetElementNameList(ERigElementType InElementType) const
{
	if (UControlRigGraph* OuterGraph = Cast<UControlRigGraph>(GetOuter()))
	{
		return OuterGraph->GetElementNameList(InElementType);
	}
	
	if(InElementType == ERigElementType::None)
	{
		return &EmptyElementNameList;
	}
	
	if(!ElementNameLists.Contains(InElementType))
	{
		const FControlRigAssetInterfacePtr Blueprint(GetAsset().GetObject());
		if(Blueprint == nullptr)
		{
			return &EmptyElementNameList;
		}

		UControlRigGraph* MutableThis = (UControlRigGraph*)this;
		URigHierarchy* Hierarchy = Blueprint->GetHierarchy();
		if(UControlRig* ControlRig = Cast<UControlRig>(Blueprint->GetObjectBeingDebugged()))
		{
			Hierarchy = ControlRig->GetHierarchy();
		}	
			
		MutableThis->CacheNameLists(Hierarchy, &Blueprint->GetDrawContainer(), Blueprint->GetShapeLibraries());
	}
	return &ElementNameLists.FindChecked(InElementType);
}

const TArray<TSharedPtr<FRigVMStringWithTag>>* UControlRigGraph::GetElementNameList(URigVMPin* InPin) const
{
	if (InPin)
	{
		if (URigVMPin* ParentPin = InPin->GetParentPin())
		{
			if (ParentPin->GetCPPTypeObject() == FRigElementKey::StaticStruct())
			{
				if (URigVMPin* TypePin = ParentPin->FindSubPin(TEXT("Type")))
				{
					FString DefaultValue = TypePin->GetDefaultValue();
					if (!DefaultValue.IsEmpty())
					{
						ERigElementType Type = (ERigElementType)StaticEnum<ERigElementType>()->GetValueByNameString(DefaultValue);
						return GetElementNameList(Type);
					}
				}
			}
		}
	}

	return GetBoneNameList(nullptr);
}

const TArray<TSharedPtr<FRigVMStringWithTag>> UControlRigGraph::GetSelectedElementsNameList() const
{
	TArray<TSharedPtr<FRigVMStringWithTag>> Result;
	if (UControlRigGraph* OuterGraph = Cast<UControlRigGraph>(GetOuter()))
	{
		return OuterGraph->GetSelectedElementsNameList();
	}
	
	const FControlRigAssetInterfacePtr Blueprint = GetAsset()->GetObject();
	if(Blueprint == nullptr)
	{
		return Result;
	}
	
	TArray<FRigElementKey> Keys = Blueprint->GetHierarchy()->GetSelectedKeys();
	for (const FRigElementKey& Key : Keys)
	{
		FString ValueStr;
		FRigElementKey::StaticStruct()->ExportText(ValueStr, &Key, nullptr, nullptr, PPF_None, nullptr);
		Result.Add(MakeShared<FRigVMStringWithTag>(ValueStr));
	}
	
	return Result;
}

const TArray<TSharedPtr<FRigVMStringWithTag>>* UControlRigGraph::GetDrawingNameList(URigVMPin* InPin) const
{
	if (UControlRigGraph* OuterGraph = Cast<UControlRigGraph>(GetOuter()))
	{
		return OuterGraph->GetDrawingNameList(InPin);
	}
	return &DrawingNameList;
}

const TArray<TSharedPtr<FRigVMStringWithTag>>* UControlRigGraph::GetShapeNameList(URigVMPin* InPin) const
{
	if (UControlRigGraph* OuterGraph = Cast<UControlRigGraph>(GetOuter()))
	{
		return OuterGraph->GetShapeNameList(InPin);
	}
	return &ShapeNameList;
}

FReply UControlRigGraph::HandleGetSelectedClicked(URigVMEdGraph* InEdGraph, URigVMPin* InPin, FString InDefaultValue)
{
	FControlRigAssetInterfacePtr RigVMBlueprint = InEdGraph->GetAsset()->GetObject();
	if(InPin->GetCustomWidgetName() == TEXT("ElementName"))
	{
		if (URigVMPin* ParentPin = InPin->GetParentPin())
		{
			InEdGraph->GetController()->SetPinDefaultValue(ParentPin->GetPinPath(), InDefaultValue, true, true, false, true);
			return FReply::Handled();
		}
	}

	else if (InPin->GetCustomWidgetName() == TEXT("BoneName"))
	{
		URigHierarchy* Hierarchy = RigVMBlueprint->GetHierarchy();
		TArray<FRigElementKey> Keys = Hierarchy->GetSelectedKeys();
		FRigBaseElement* Element = Hierarchy->FindChecked(Keys[0]);
		if (Element->GetType() == ERigElementType::Bone)
		{
			InEdGraph->GetController()->SetPinDefaultValue(InPin->GetPinPath(), Keys[0].Name.ToString(), true, true, false, true);
			return FReply::Handled();
		}
	}

	// if we don't have a key pin - this is just a plain name.
	// let's derive the type of element this node deals with from its name.
	// there's nothing better in place for now.
	else if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InPin->GetNode()))
	{
		const int32 LastIndex = StaticEnum<ERigElementType>()->GetIndexByName(TEXT("Last")); 
		const FString UnitName = UnitNode->GetScriptStruct()->GetStructCPPName();
		for(int32 EnumIndex = 0; EnumIndex < LastIndex; EnumIndex++)
		{
			const FString EnumDisplayName = StaticEnum<ERigElementType>()->GetDisplayNameTextByIndex(EnumIndex).ToString();
			if(UnitName.Contains(EnumDisplayName))
			{
				const ERigElementType ElementType = (ERigElementType)StaticEnum<ERigElementType>()->GetValueByIndex(EnumIndex);

				FRigElementKey Key;
				FRigElementKey::StaticStruct()->ImportText(*InDefaultValue, &Key, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);
				if (Key.IsValid())
				{
					if(Key.Type == ElementType)
					{
						InEdGraph->GetController()->SetPinDefaultValue(InPin->GetPinPath(), Key.Name.ToString(), true, true, false, true);
						return FReply::Handled();
					}
				}
				break;
			}
		}
	}
	return FReply::Unhandled();
}

FReply UControlRigGraph::HandleBrowseClicked(URigVMEdGraph* InEdGraph, URigVMPin* InPin, FString InDefaultValue)
{
	FControlRigAssetInterfacePtr RigVMBlueprint = InEdGraph->GetAsset()->GetObject();
	URigVMPin* KeyPin = InPin->GetParentPin();
	if(KeyPin && KeyPin->GetCPPTypeObject() == FRigElementKey::StaticStruct())
	{
		// browse to rig element key
		FString DefaultValue = InPin->GetParentPin()->GetDefaultValue();
		if (!DefaultValue.IsEmpty())
		{
			FRigElementKey Key;
			FRigElementKey::StaticStruct()->ImportText(*DefaultValue, &Key, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);
			if (Key.IsValid())
			{
				RigVMBlueprint->GetHierarchyController()->SetSelection({Key});
			}
		}
	}
	else if (InPin->GetCustomWidgetName() == TEXT("BoneName"))
	{
		// browse to named bone
		const FString DefaultValue = InPin->GetDefaultValue();
		FRigElementKey Key(*DefaultValue, ERigElementType::Bone);
		RigVMBlueprint->GetHierarchyController()->SetSelection({Key});
	}
	else
	{
		// if we don't have a key pin - this is just a plain name.
		// let's derive the type of element this node deals with from its name.
		// there's nothing better in place for now.
		if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InPin->GetNode()))
		{
			const int32 LastIndex = StaticEnum<ERigElementType>()->GetIndexByName(TEXT("Last")); 
			const FString UnitName = UnitNode->GetScriptStruct()->GetStructCPPName();
			for(int32 EnumIndex = 0; EnumIndex < LastIndex; EnumIndex++)
			{
				const FString EnumDisplayName = StaticEnum<ERigElementType>()->GetDisplayNameTextByIndex(EnumIndex).ToString();
				if(UnitName.Contains(EnumDisplayName))
				{
					const FString DefaultValue = InPin->GetDefaultValue();
					const ERigElementType ElementType = (ERigElementType)StaticEnum<ERigElementType>()->GetValueByIndex(EnumIndex);
					FRigElementKey Key(*DefaultValue, ElementType);
					RigVMBlueprint->GetHierarchyController()->SetSelection({Key});
					break;
				}
			}
		}
	}
	return FReply::Handled();
}

#endif

bool UControlRigGraph::HandleModifiedEvent_Internal(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if(!Super::HandleModifiedEvent_Internal(InNotifType, InGraph, InSubject))
	{
		return false;
	}

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::PinDefaultValueChanged:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if (UControlRigGraphNode* RigNode = Cast<UControlRigGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
				{
					if (Cast<URigVMUnitNode>(ModelPin->GetNode()))
					{
						// if the node contains a rig element key - invalidate the node
						if(RigNode->GetAllPins().ContainsByPredicate([](UEdGraphPin* Pin) -> bool
						{
							return Pin->PinType.PinSubCategoryObject == FRigElementKey::StaticStruct();
						}))
						{
							// we do this to enforce the refresh of the element name widgets
							RigNode->SynchronizeGraphPinValueWithModelPin(ModelPin);
						}
					}
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

