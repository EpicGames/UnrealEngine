// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/ControlRigGraphNode.h"
#include "Graph/ControlRigGraph.h"
#include "Graph/ControlRigGraphSchema.h"
#include "IControlRigEditorModule.h"
#if !WITH_RIGVMLEGACYEDITOR
#include "RigVMEditor/Private/Editor/Kismet/RigVMFindInBlueprintManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigGraphNode)

#define LOCTEXT_NAMESPACE "ControlRigGraphNode"

UControlRigGraphNode::UControlRigGraphNode()
: URigVMEdGraphNode()
{
}

#if WITH_EDITOR

#if WITH_RIGVMLEGACYEDITOR
void UControlRigGraphNode::AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(Pin, OutTaggedMetaData);
#else
void UControlRigGraphNode::AddRigVMPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddRigVMPinSearchMetaDataInfo(Pin, OutTaggedMetaData);
#endif

	if(const URigVMPin* ModelPin = FindModelPinFromGraphPin(Pin))
	{
		if(ModelPin->GetCPPTypeObject() == FRigElementKey::StaticStruct())
		{
			const FString DefaultValue = ModelPin->GetDefaultValue();
			if(!DefaultValue.IsEmpty())
			{
				FString RigElementKeys;
				if(ModelPin->IsArray())
				{
					RigElementKeys = DefaultValue;
				}
				else
				{
					RigElementKeys = FString::Printf(TEXT("(%s)"), *DefaultValue);
				}
				if(!RigElementKeys.IsEmpty())
				{
					RigElementKeys.ReplaceInline(TEXT("="), TEXT(","));
					RigElementKeys.ReplaceInline(TEXT("\""), TEXT(""));
					OutTaggedMetaData.Emplace(FText::FromString(TEXT("Rig Items")), FText::FromString(RigElementKeys));
				}
			}
		}
	}
}

#endif

#undef LOCTEXT_NAMESPACE

