// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/ControlRigGraphSchema.h"
#include "Graph/ControlRigGraph.h"
#include "Settings/ControlRigSettings.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Units/Execution/RigUnit_InteractionExecution.h"
#include "Units/Modules/RigUnit_ConnectorExecution.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigGraphSchema)



#define LOCTEXT_NAMESPACE "ControlRigGraphSchema"

UControlRigGraphSchema::UControlRigGraphSchema()
{
	EditorSettingsClass = UControlRigEditorSettings::StaticClass();
}

FLinearColor UControlRigGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	const FName& TypeName = PinType.PinCategory;
	if (TypeName == UEdGraphSchema_K2::PC_Struct)
	{
		if (UStruct* Struct = Cast<UStruct>(PinType.PinSubCategoryObject))
		{
			if (Struct == FRigElementKey::StaticStruct() || Struct == FRigElementKeyCollection::StaticStruct())
			{
				return FLinearColor(0.0f, 0.6588f, 0.9490f);
			}

			if (Struct == FRigComponentKey::StaticStruct())
			{
				return FLinearColor(0.0f, 0.6588f * 0.7f, 0.9490f * 0.7f);
			}

			if (Struct == FRigPose::StaticStruct())
			{
				return FLinearColor(0.0f, 0.3588f, 0.5490f);
			}
		}
	}
	
	return Super::GetPinTypeColor(PinType);
}

bool UControlRigGraphSchema::IsRigVMDefaultEvent(const FName& InEventName) const
{
	if(Super::IsRigVMDefaultEvent(InEventName))
	{
		return true;
	}
	
	return InEventName == FRigUnit_BeginExecution::EventName ||
		InEventName == FRigUnit_PreBeginExecution::EventName ||
		InEventName == FRigUnit_PostBeginExecution::EventName ||
		InEventName == FRigUnit_InverseExecution::EventName ||
		InEventName == FRigUnit_PrepareForExecution::EventName ||
		InEventName == FRigUnit_PostPrepareForExecution::EventName ||
		InEventName == FRigUnit_InteractionExecution::EventName ||
		InEventName == FRigUnit_ConnectorExecution::EventName;
}

#undef LOCTEXT_NAMESPACE

