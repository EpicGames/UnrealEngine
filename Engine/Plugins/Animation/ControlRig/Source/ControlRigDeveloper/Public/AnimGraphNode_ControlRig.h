// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_CustomProperty.h"
#include "AnimNode_ControlRig.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "ControlRigIOMapping.h"
#include "AnimGraphNode_ControlRig.generated.h"

struct FRigVMVariableMappingInfo;

UCLASS(MinimalAPI)
class UAnimGraphNode_ControlRig : public UAnimGraphNode_CustomProperty
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_ControlRig Node;
	
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostReconstructNode() override;
	virtual void PreloadRequiredAssets() override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual UStruct* GetTargetSkeletonStruct() const override;
	virtual UStruct* GetTargetStruct() const override;
	virtual FControlRigAssetStrongReference GetTargetAssetReference() const;

private:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual FText GetTooltipText() const override;

	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() override { return &Node; }
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const override { return &Node; }

	virtual void CreateCustomPins(TArray<UEdGraphPin*>* OldPins) override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;

	void OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName);
	void OnVariableMappingChanged(const FName& PathName, const FName& Curve, bool bInput);

	USkeleton* GetTargetSkeleton() const;

	TSharedPtr <FControlRigIOMapping> ControlRigIOMapping;
};
