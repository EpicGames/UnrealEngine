// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMBlueprintCompiler.h"

#define UE_API CONTROLRIGDEVELOPER_API

class FControlRigBlueprintCompiler : public FRigVMBlueprintCompiler
{
public:
	/** IBlueprintCompiler interface */
	UE_API virtual bool CanCompile(const UBlueprint* Blueprint) override;
	UE_API virtual void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) override;
};

class FControlRigBlueprintCompilerContext : public FRigVMBlueprintCompilerContext
{
public:
	FControlRigBlueprintCompilerContext(UBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
		: FRigVMBlueprintCompilerContext(SourceSketch, InMessageLog, InCompilerOptions)
	{
	}

	// FKismetCompilerContext interface
	UE_API virtual void SpawnNewClass(const FString& NewClassName) override;
	UE_API virtual void CopyTermDefaultsToDefaultObject(UObject* DefaultObject) override;
};

#undef UE_API
