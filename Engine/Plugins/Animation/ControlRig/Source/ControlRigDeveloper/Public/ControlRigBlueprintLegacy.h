// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigBlueprintGeneratedClass.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Blueprint.h"
#include "Engine/Texture2D.h"
#include "ControlRigDefines.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Rigs/RigHierarchy.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "ControlRigGizmoLibrary.h"
#include "ControlRigSchema.h"
#include "RigVMCore/RigVMStatistics.h"
#include "RigVMModel/RigVMClient.h"
#include "ControlRigValidationPass.h"
#include "RigVMBlueprintLegacy.h"
#include "Rigs/RigModuleDefines.h"
#include "ModularRigModel.h"
#include "ControlRigEditorAsset.h"

#if WITH_EDITOR
#include "Kismet2/CompilerResultsLog.h"
#include "Overrides/SOverrideListWidget.h"
#endif

#include "ControlRigBlueprintLegacy.generated.h"

class URigVMBlueprintGeneratedClass;

UCLASS(MinimalAPI, BlueprintType, meta=(IgnoreClassThumbnail))
class UControlRigBlueprint : public URigVMBlueprint, public IControlRigEditorAssetInterface, public IInterface_PreviewMeshProvider, public IRigHierarchyProvider
{
	GENERATED_UCLASS_BODY()

public:
	CONTROLRIGDEVELOPER_API UControlRigBlueprint();

	// --- IControlRigEditorAssetInterface interface ---
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() override { return FRigVMEditorAssetInterfacePtr(this); }
	virtual FModularRigModel& GetModularRigModel() override { return ModularRigModel; }
	virtual URigHierarchy* GetHierarchy() const override { return Hierarchy; }
	virtual TSoftObjectPtr<USkeletalMesh> GetPreviewSkeletalMesh() const override { return PreviewSkeletalMesh; }
	virtual bool& GetAllowMultipleInstances() override { return bAllowMultipleInstances; }
	virtual TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() override { return ArrayConnectionMap; }
	virtual void SetControlRigType(EControlRigType InType) override { ControlRigType = InType; }
	virtual EControlRigType GetControlRigType() const override { return ControlRigType; }
	virtual void SetCustomThumbnail(FString InThumbnail) override { CustomThumbnail = InThumbnail; }
	virtual FString GetCustomThumbnail() const override { return CustomThumbnail; }
	virtual float& GetDebugBoneRadius() override { return DebugBoneRadius; }
	virtual bool& GetExposesAnimatableControls() override { return bExposesAnimatableControls; }
	virtual FRigHierarchySettings& GetHierarchySettings() override { return HierarchySettings; }
	virtual FRigInfluenceMapPerEvent& GetInfluences() override { return Influences; }
	virtual void SetItemTypeDisplayName(FName InName) override { ItemTypeDisplayName = InName; }
	virtual FName GetItemTypeDisplayName() const override { return ItemTypeDisplayName; }
	virtual FModularRigSettings& GetModularRigSettings() override { return ModularRigSettings; }
	virtual FRigModuleSettings& GetRigModuleSettings() override { return RigModuleSettings; }
	virtual TArray<FModuleReferenceData>& GetModuleReferenceData() override { return ModuleReferenceData; }
	virtual TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() override { return ShapeLibraries; }
	virtual TSoftObjectPtr<UObject>& GetSourceCurveImport() override { return SourceCurveImport; }
	virtual TSoftObjectPtr<UObject>& GetSourceHierarchyImport() override { return SourceHierarchyImport; }
	virtual bool& GetSupportsControls() override { return bSupportsControls; }
	virtual bool& GetSupportsInversion() override { return bSupportsInversion; }
	virtual TObjectPtr<UControlRigValidator>& GetValidator() override { return Validator; }
	virtual UObject* GetDefaultHierarchyOuter() override { return this; }

	//  --- IRigVMClientHost interface ---
	virtual UClass* GetRigVMSchemaClass() const override { return IControlRigEditorAssetInterface::GetRigVMSchemaClass(); }
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const override { return IControlRigEditorAssetInterface::GetRigVMExecuteContextStruct(); }
	virtual UClass* GetRigVMEdGraphClass() const override { return IControlRigEditorAssetInterface::GetRigVMEdGraphClass(); }
	virtual UClass* GetRigVMEdGraphNodeClass() const override { return IControlRigEditorAssetInterface::GetRigVMEdGraphNodeClass(); }
	virtual UClass* GetRigVMEdGraphSchemaClass() const override { return IControlRigEditorAssetInterface::GetRigVMEdGraphSchemaClass(); }
	virtual UClass* GetRigVMEditorSettingsClass() const override { return IControlRigEditorAssetInterface::GetRigVMEditorSettingsClass(); }

	// URigVMBlueprint interface
	UE_DEPRECATED(5.7, "Please use GetRigVMGeneratedClassPrototype")
	virtual UClass* GetRigVMBlueprintGeneratedClassPrototype() const { return UControlRigBlueprintGeneratedClass::StaticClass(); }
	virtual UClass* GetRigVMGeneratedClassPrototype() const override { return UControlRigBlueprintGeneratedClass::StaticClass(); }
	
#if WITH_EDITOR
	UE_DEPRECATED(5.8, "Use TArray<FString> GeneratePythonCommands() instead.")
	virtual TArray<FString> GeneratePythonCommands(const FString InNewBlueprintName) override { return TArray<FString>(); }
	virtual TArray<FString> GeneratePythonCommands() override { return IControlRigEditorAssetInterface::GeneratePythonCommands(); }
	CONTROLRIGDEVELOPER_API virtual TArray<FString> GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset) override;
#endif
	
	CONTROLRIGDEVELOPER_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if WITH_EDITOR
	virtual const FLazyName& GetPanelPinFactoryName() const override { return IControlRigEditorAssetInterface::GetPanelPinFactoryName(); }
	virtual IRigVMEditorModule* GetEditorModule() const override { return IControlRigEditorAssetInterface::GetEditorModule(); }
#endif

	CONTROLRIGDEVELOPER_API virtual void Serialize(FArchive& Ar) override;
	virtual void SerializeSuper(FArchive& Ar) override { return URigVMBlueprint::SerializeSuper(Ar); }

#if WITH_EDITOR

	// UBlueprint interface
	virtual UClass* GetBlueprintClass() const override { return IControlRigEditorAssetInterface::GetBlueprintClass(); }
	CONTROLRIGDEVELOPER_API virtual UClass* RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO) override;
	virtual bool RequiresForceLoadMembersSuper(UObject* InObject) const override { return URigVMBlueprint::RequiresForceLoadMembersSuper(InObject); }
	virtual bool SupportedByDefaultBlueprintFactory() const override { return IControlRigEditorAssetInterface::SupportedByDefaultBlueprintFactory(); }
	virtual bool IsValidForBytecodeOnlyRecompile() const override { return IControlRigEditorAssetInterface::IsValidForBytecodeOnlyRecompile(); }
	virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override { return IControlRigEditorAssetInterface::GetTypeActions(ActionRegistrar); }
	virtual void GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override { return IControlRigEditorAssetInterface::GetInstanceActions(ActionRegistrar); }
	CONTROLRIGDEVELOPER_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	CONTROLRIGDEVELOPER_API virtual void PostLoad() override;
	CONTROLRIGDEVELOPER_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	CONTROLRIGDEVELOPER_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	CONTROLRIGDEVELOPER_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual bool RequiresForceLoadMembers(UObject* InObject) const override { return IControlRigEditorAssetInterface::RequiresForceLoadMembers(InObject); }

	virtual bool SupportsGlobalVariables() const override { return IControlRigEditorAssetInterface::SupportsGlobalVariables(); }
	virtual bool SupportsLocalVariables() const override { return IControlRigEditorAssetInterface::SupportsLocalVariables(); }
	virtual bool SupportsFunctions() const override { return IControlRigEditorAssetInterface::SupportsFunctions(); }
	virtual bool SupportsEventGraphs() const override { return IControlRigEditorAssetInterface::SupportsEventGraphs(); }


	// UObject interface
	CONTROLRIGDEVELOPER_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	CONTROLRIGDEVELOPER_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	
	CONTROLRIGDEVELOPER_API static bool CopyBlueprintToAsset(UControlRigBlueprint* InBlueprint, UControlRigRuntimeAsset* InAsset);

#endif	// #if WITH_EDITOR

	UE_DEPRECATED(5.8, "Use GetControlRigAssetReference instead")
	UFUNCTION(BlueprintCallable, Category = "VM", meta = (DeprecatedFunction, DeprecationMessage = "Use GetControlRigAssetReference instead"))
	CONTROLRIGDEVELOPER_API virtual UClass* GetControlRigClass() const override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual FControlRigAssetStrongReference GetControlRigAssetReference() const override;

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	UControlRig* CreateControlRig() { return IControlRigEditorAssetInterface::CreateControlRig(); }

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	UControlRig* GetDebuggedControlRig() { return IControlRigEditorAssetInterface::GetDebuggedControlRig(); } 

	/** IInterface_PreviewMeshProvider interface */
	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override { return IControlRigEditorAssetInterface::SetPreviewMesh(PreviewMesh, bMarkAsDirty); }
	virtual void SetPreviewMeshImpl(USkeletalMesh* PreviewMesh) override;
	
	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	virtual USkeletalMesh* GetPreviewMesh() const override { return IControlRigEditorAssetInterface::GetPreviewMesh(); }

	UFUNCTION(BlueprintPure, Category = "Control Rig Blueprint")
	virtual bool IsControlRigModule() const { return IControlRigEditorAssetInterface::IsControlRigModule(); }

#if WITH_EDITORONLY_DATA
	
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "TurnIntoControlRigModule", ScriptName = "TurnIntoControlRigModule"), Category = "Control Rig Blueprint")
	bool TurnIntoControlRigModule_Blueprint() { return IControlRigEditorAssetInterface::TurnIntoControlRigModule(); }

	UFUNCTION(BlueprintPure, meta = (DisplayName = "CanTurnIntoStandaloneRig", ScriptName = "CanTurnIntoStandaloneRig"), Category = "Control Rig Blueprint")
	bool CanTurnIntoStandaloneRig_Blueprint() const { return IControlRigEditorAssetInterface::CanTurnIntoStandaloneRig(); }

	UFUNCTION(BlueprintCallable, meta = (DisplayName = "TurnIntoStandaloneRig", ScriptName = "TurnIntoStandaloneRig"), Category = "Control Rig Blueprint")
	bool TurnIntoStandaloneRig_Blueprint() { return IControlRigEditorAssetInterface::TurnIntoStandaloneRig(); }

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	TArray<URigVMNode*> ConvertHierarchyElementsToSpawnerNodes(URigHierarchy* InHierarchy, TArray<FRigElementKey> InKeys, bool bRemoveElements = true) { return IControlRigEditorAssetInterface::ConvertHierarchyElementsToSpawnerNodes(InHierarchy, InKeys, bRemoveElements); }

#endif // WITH_EDITORONLY_DATA
	
	virtual UObject* GetOuterForModularModel() override { return this; }

	UFUNCTION(BlueprintPure, Category = "Control Rig Blueprint")
	UTexture2D* GetRigModuleIcon() const { return IControlRigEditorAssetInterface::GetRigModuleIcon(); }

	UPROPERTY(EditAnywhere, Category = "Modular Rig")
	FModularRigSettings ModularRigSettings;

	UPROPERTY(EditAnywhere, Category = "Hierarchy")
	FRigHierarchySettings HierarchySettings;

	UPROPERTY(EditAnywhere, Category = "Hierarchy", AssetRegistrySearchable)
	FRigModuleSettings RigModuleSettings;

	// This relates to FAssetThumbnailPool::CustomThumbnailTagName and allows
	// the thumbnail pool to show the thumbnail of the icon rather than the
	// rig itself to avoid deploying the 3D renderer.
	UPROPERTY(EditAnywhere, Category = "Hierarchy", AssetRegistrySearchable)
	FString CustomThumbnail;

	/** Asset searchable information module references in this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FModuleReferenceData> ModuleReferenceData;

	UPROPERTY()
	TMap<FRigElementKey, FRigElementKey> ConnectionMap_DEPRECATED;

	UPROPERTY()
	TMap<FRigElementKey, FRigElementKeyCollection> ArrayConnectionMap;

	UFUNCTION(BlueprintPure, Category = "Control Rig Blueprint")
	TArray<FModuleReferenceData> FindReferencesToModule() const { return IControlRigEditorAssetInterface::FindReferencesToModule(); }

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	virtual void UpdateExposedModuleConnectors() const { return IControlRigEditorAssetInterface::UpdateExposedModuleConnectors(); }
	
protected:


	/** Asset searchable information about exposed public functions on this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FRigVMOldPublicFunctionData> PublicFunctions_DEPRECATED;

	virtual void SetupDefaultObjectDuringCompilation(URigVMHost* InCDO) override { return IControlRigEditorAssetInterface::SetupDefaultObjectDuringCompilation(InCDO); }

public:

	virtual void SetupPinRedirectorsForBackwardsCompatibility() override { return IControlRigEditorAssetInterface::SetupPinRedirectorsForBackwardsCompatibility(); }

	UFUNCTION(BlueprintCallable, Category = "VM")
	static CONTROLRIGDEVELOPER_API TArray<UControlRigBlueprint*> GetCurrentlyOpenRigBlueprints();

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TSoftObjectPtr<UControlRigShapeLibrary> GizmoLibrary_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries;

	UPROPERTY(transient, DuplicateTransient, meta = (DisplayName = "VM Statistics", DisplayAfter = "VMCompileSettings"))
	FRigVMStatistics Statistics_DEPRECATED;
#endif

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage = "Use RigVMBlueprint::DrawContainer instead"))
	FRigVMDrawContainer DrawContainer_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = "Influence Map")
	FRigInfluenceMapPerEvent Influences;

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FRigHierarchyContainer HierarchyContainer_DEPRECATED;
#endif

	UPROPERTY(BlueprintReadOnly, Category = "Hierarchy")
	TObjectPtr<URigHierarchy> Hierarchy;

	UFUNCTION(BlueprintCallable, Category = "Hierarchy")
	URigHierarchyController* GetHierarchyController() { return IControlRigEditorAssetInterface::GetHierarchyController(); }

	UPROPERTY(BlueprintReadOnly, Category = "Modules")
	FModularRigModel ModularRigModel;

	UFUNCTION(BlueprintCallable, Category = "Modules")
	virtual UModularRigController* GetModularRigController() { return IControlRigEditorAssetInterface::GetModularRigController(); }

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	virtual void RecompileModularRig() { return IControlRigEditorAssetInterface::RecompileModularRig(); }

	UPROPERTY(AssetRegistrySearchable)
	EControlRigType ControlRigType;

	UPROPERTY(AssetRegistrySearchable)
	FName ItemTypeDisplayName = TEXT("Control Rig");

private:

	/** Whether or not this rig has an Inversion Event */
	UPROPERTY(AssetRegistrySearchable)
	bool bSupportsInversion;

	/** Whether or not this rig has Controls on It */
	UPROPERTY(AssetRegistrySearchable)
	bool bSupportsControls;

	/** The default skeletal mesh to use when previewing this asset */
#if WITH_EDITORONLY_DATA
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category="Control Rig Blueprint")
	TSoftObjectPtr<USkeletalMesh> PreviewSkeletalMesh;
#endif

	/** The skeleton from import into a hierarchy */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable, EditAnywhere, Category="Control Rig Blueprint")
	TSoftObjectPtr<UObject> SourceHierarchyImport;

	/** The skeleton from import into a curve */
	UPROPERTY(DuplicateTransient, AssetRegistrySearchable, EditAnywhere, Category="Control Rig Blueprint")
	TSoftObjectPtr<UObject> SourceCurveImport;

	/** If set to true, this control rig has animatable controls */
	UPROPERTY(AssetRegistrySearchable)
	bool bExposesAnimatableControls;
public:

	/** If set to true, multiple control rig tracks can be created for the same rig in sequencer*/
	UPROPERTY(EditAnywhere, Category="Sequencer", AssetRegistrySearchable)
	bool bAllowMultipleInstances = false;

private:

	virtual void PathDomainSpecificContentOnLoad() override { return IControlRigEditorAssetInterface::PathDomainSpecificContentOnLoad(); }
	CONTROLRIGDEVELOPER_API virtual void GetBackwardsCompatibilityPublicFunctions(TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders) override;

protected:
	virtual void CreateMemberVariablesOnLoad() override { return IControlRigEditorAssetInterface::CreateMemberVariablesOnLoad(); }
	CONTROLRIGDEVELOPER_API virtual void PatchVariableNodesOnLoad() override;
	virtual void SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy) override { Hierarchy = InHierarchy; }
	virtual FCompilerResultsLog CompileBlueprint() override { return Super::CompileBlueprint();}
	
private:

	UPROPERTY()
	TObjectPtr<UControlRigValidator> Validator;

	FRigHierarchyModifiedEvent	HierarchyModifiedEvent;
	FOnRigVMRefreshEditorEvent ModularRigPreCompiled;
	FOnRigVMRefreshEditorEvent ModularRigCompiled;

	UPROPERTY(transient, DuplicateTransient)
	int32 ModulesRecompilationBracket = 0;


#if WITH_EDITOR
	virtual void HandlePackageDone() override { return IControlRigEditorAssetInterface::HandlePackageDone(); }
	virtual void HandlePackageDoneSuper() override { return URigVMBlueprint::HandlePackageDone(); }
	virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure) override { return IControlRigEditorAssetInterface::HandleConfigureRigVMController(InClient, InControllerToConfigure); }
#endif

	UPROPERTY()
	float DebugBoneRadius;

	friend class FControlRigBlueprintActions;
	friend class FControlRigBasicsConvertToAssetTest;
};
