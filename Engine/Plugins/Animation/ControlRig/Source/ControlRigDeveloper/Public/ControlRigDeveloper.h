// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogControlRigDeveloper, Log, All);

class IControlRigDeveloperModule : public IModuleInterface
{
public:
	virtual void RegisterPinTypeColor(UStruct* Struct, const FLinearColor Color) = 0;
	virtual void UnRegisterPinTypeColor(UStruct* Struct) = 0;
	virtual const FLinearColor* FindPinTypeColor(UStruct* Struct) const = 0;
};

