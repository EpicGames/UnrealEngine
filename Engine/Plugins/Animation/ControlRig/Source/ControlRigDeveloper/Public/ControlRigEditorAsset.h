// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ControlRigDefines.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Rigs/RigHierarchy.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "ControlRigGizmoLibrary.h"
#include "ControlRigSchema.h"
#include "RigVMCore/RigVMStatistics.h"
#include "RigVMModel/RigVMClient.h"
#include "ControlRigValidationPass.h"
#include "RigVMEditorAsset.h"
#include "Rigs/RigModuleDefines.h"
#include "ModularRigModel.h"

#if WITH_EDITOR
#include "Overrides/SOverrideListWidget.h"
#endif

#include "ControlRigEditorAsset.generated.h"

#define UE_API CONTROLRIGDEVELOPER_API

class IControlRigRuntimeAssetInterface;
class FRigVMNameValidator;
struct FRigVMOldPublicFunctionData;
class IControlRigEditorAssetInterface;
typedef TScriptInterface<IControlRigEditorAssetInterface> FControlRigAssetInterfacePtr;

class USkeletalMesh;
class UControlRigGraph;
struct FEndLoadPackageContext;

#if WITH_EDITOR
class FControlRigEditorAssetUndoClient;
#endif 

UINTERFACE(MinimalAPI, NotBlueprintable)
class UControlRigEditorAssetInterface : public UInterface
{
	GENERATED_BODY()
};

class IControlRigEditorAssetInterface 
{
	GENERATED_BODY()

public:
	UE_API static FControlRigAssetInterfacePtr GetInterfaceOuter(const UObject* InObject);
	static TArray<UClass*> FindAllImplementingClasses();
	UE_API static bool IsAControlRigAsset(UObject* InObject);
	
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() = 0;
	virtual const FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetRigVMAssetInterface(); }
	virtual TScriptInterface<IControlRigRuntimeAssetInterface> GetRuntimeAssetInterface() const;

	UFUNCTION(BlueprintCallable, Category = ControlRigAsset)
	virtual URigHierarchy* GetHierarchy() const = 0;
	UE_API virtual FRigVMClient* GetRigVMClient();
	virtual UObject* GetDefaultHierarchyOuter() = 0;

protected:

	virtual bool RequiresForceLoadMembersSuper(UObject* InObject) const = 0;
	virtual void SerializeSuper(FArchive& Ar) = 0;
	virtual void HandlePackageDoneSuper() = 0;
	virtual void SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy) = 0;
	virtual FCompilerResultsLog CompileBlueprint() = 0;

	UObject* GetObject() { return GetRigVMAssetInterface()->GetObject(); }
	const UObject* GetObject() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetObject(); }
	FControlRigAssetInterfacePtr GetControlRigAssetInterface() { return FControlRigAssetInterfacePtr(GetObject()); }

public:
	void Modify() { GetObject()->Modify(); }
	UObject* GetObjectBeingDebugged() { return GetRigVMAssetInterface()->GetObjectBeingDebugged(); }
	const UObject* GetObjectBeingDebugged() const { return GetRigVMAssetInterface()->GetObjectBeingDebugged(); }

	static void CommonInitialization(const FObjectInitializer& ObjectInitializer);
	IControlRigEditorAssetInterface();

	//  --- IRigVMClientHost interface ---
	virtual UClass* GetRigVMSchemaClass() const { return UControlRigSchema::StaticClass(); }
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const { return FControlRigExecuteContext::StaticStruct(); }
	virtual UClass* GetRigVMEdGraphClass() const;
	virtual UClass* GetRigVMEdGraphNodeClass() const;
	virtual UClass* GetRigVMEdGraphSchemaClass() const;
	virtual UClass* GetRigVMEditorSettingsClass() const;

	// URigVMBlueprint interface
	//virtual UClass* GetRigVMGeneratedClassPrototype() const override { return UControlRigBlueprintGeneratedClass::StaticClass(); }
	virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps);
#if WITH_EDITOR
	virtual TArray<FString> GeneratePythonCommands();
	virtual TArray<FString> GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset) = 0;
	virtual const FLazyName& GetPanelPinFactoryName() const;
	static inline const FLazyName ControlRigPanelNodeFactoryName = FLazyName(TEXT("FControlRigGraphPanelPinFactory"));
	virtual IRigVMEditorModule* GetEditorModule() const;
#endif

	virtual void Serialize(FArchive& Ar);

#if WITH_EDITOR

	// UBlueprint interface
	virtual UClass* GetBlueprintClass() const;
	void OnRegeneratedClass(UClass* ClassToRegenerate, UObject* PreviousCDO);
	virtual bool SupportedByDefaultBlueprintFactory() const { return false; }
	virtual bool IsValidForBytecodeOnlyRecompile() const { return false; }
	virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const;
	virtual void GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext);
	virtual void PostLoad();
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent);
	virtual void PostDuplicate(bool bDuplicateForPIE);
	virtual void PostRename(UObject* OldOuter, const FName OldName);
	virtual bool RequiresForceLoadMembers(UObject* InObject) const;

	virtual bool SupportsGlobalVariables() const { return true; }
	virtual bool SupportsLocalVariables() const { return !IsModularRig(); }
	virtual bool SupportsFunctions() const { return !IsModularRig(); }
	virtual bool SupportsEventGraphs() const { return !IsModularRig(); }


	// UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent);

#endif	// #if WITH_EDITOR

	UE_DEPRECATED(5.8, "Use GetControlRigAssetReference instead")
	UE_API virtual UClass* GetControlRigClass() const;
	
	virtual FControlRigAssetStrongReference GetControlRigAssetReference() const = 0;

	UE_API bool IsModularRig() const;

	UE_API virtual UControlRig* CreateControlRig();

	UE_API virtual UControlRig* GetDebuggedControlRig();

	/** IInterface_PreviewMeshProvider interface */
	virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true);
	virtual void SetPreviewMeshImpl(USkeletalMesh* PreviewMesh) = 0;
	
	virtual USkeletalMesh* GetPreviewMesh() const;

	UE_API virtual bool IsControlRigModule() const;

#if WITH_EDITORONLY_DATA
	
	UE_API bool CanTurnIntoControlRigModule(bool InAutoConvertHierarchy, FString* OutErrorMessage = nullptr) const;
	UE_API bool TurnIntoControlRigModule(bool InAutoConvertHierarchy = false, FString* OutErrorMessage = nullptr);
	UE_API bool CanTurnIntoStandaloneRig(FString* OutErrorMessage = nullptr) const;
	UE_API bool TurnIntoStandaloneRig(FString* OutErrorMessage = nullptr);

	TArray<URigVMNode*> ConvertHierarchyElementsToSpawnerNodes(URigHierarchy* InHierarchy, TArray<FRigElementKey> InKeys, bool bRemoveElements = true);

#endif // WITH_EDITORONLY_DATA

	UE_API UTexture2D* GetRigModuleIcon() const;

	DECLARE_EVENT_OneParam(IControlRigEditorAssetInterface, FOnRigTypeChanged, FControlRigAssetInterfacePtr);

	FOnRigTypeChanged& OnRigTypeChanged() { return OnRigTypeChangedDelegate; }

	virtual FModularRigSettings& GetModularRigSettings() = 0;
	const FModularRigSettings& GetModularRigSettings() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetModularRigSettings(); }

	virtual FRigHierarchySettings& GetHierarchySettings() = 0;
	const FRigHierarchySettings& GetHierarchySettings() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetHierarchySettings(); }

	virtual FRigModuleSettings& GetRigModuleSettings() = 0;
	virtual const FRigModuleSettings& GetRigModuleSettings() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetRigModuleSettings(); }

	// This relates to FAssetThumbnailPool::CustomThumbnailTagName and allows
	// the thumbnail pool to show the thumbnail of the icon rather than the
	// rig itself to avoid deploying the 3D renderer.
	virtual void SetCustomThumbnail(FString InThumbnail) = 0;
	virtual FString GetCustomThumbnail() const = 0;

	/** Asset searchable information module references in this rig */
	virtual TArray<FModuleReferenceData>& GetModuleReferenceData() = 0;
	
	virtual UObject* GetOuterForModularModel() = 0;

	virtual TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() = 0;
	const TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetArrayConnectionMap(); }

	TArray<FModuleReferenceData> FindReferencesToModule() const;

	UE_API static EControlRigType GetRigType(const FAssetData& InAsset);
	UE_API static TArray<FSoftObjectPath> GetReferencesToRigModule(const FAssetData& InModuleAsset);

	UE_API virtual void UpdateExposedModuleConnectors() const;

#if WITH_EDITOR
	UE_API TArray<FOverrideStatusSubject> GetOverrideSubjects() const;
	UE_API uint32 GetOverrideSubjectsHash() const;
#endif
	
	FRigVMDrawContainer& GetDrawContainer() const { return GetRigVMAssetInterface()->GetDrawContainer(); }
	
protected:

	FName FindHostMemberVariableUniqueName(TSharedPtr<FRigVMNameValidator> InNameValidator, const FString& InBaseName);

	TArray<FModuleReferenceData> GetModuleReferenceDataImpl() const;

	FOnRigTypeChanged OnRigTypeChangedDelegate;
	
	UE_API bool ResolveConnector(const FRigElementKey& DraggedKey, const FRigElementKey& TargetKey, bool bSetupUndoRedo = true);
	UE_API bool ResolveConnectorToArray(const FRigElementKey& DraggedKey, const TArray<FRigElementKey>& TargetKeys, bool bSetupUndoRedo = true);

	void UpdateConnectionMapFromModel();

	virtual void SetupDefaultObjectDuringCompilation(URigVMHost* InCDO);

public:

	virtual void SetupPinRedirectorsForBackwardsCompatibility();

	static TArray<FControlRigAssetInterfacePtr> GetCurrentlyOpenRigAssets();

#if WITH_EDITORONLY_DATA
	virtual TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() = 0;
	const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetShapeLibraries(); }

	UE_API const FControlRigShapeDefinition* GetControlShapeByName(const FName& InName) const;

#endif

#if WITH_EDITOR
	/** Remove a transient / temporary control used to interact with a pin */
	UE_API FName AddTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget);

	/** Remove a transient / temporary control used to interact with a pin */
	UE_API FName RemoveTransientControl(const URigVMUnitNode* InNode, const FRigDirectManipulationTarget& InTarget);

	/** Remove a transient / temporary control used to interact with a bone */
	UE_API FName AddTransientControl(const FRigElementKey& InElement);

	/** Remove a transient / temporary control used to interact with a bone */
	UE_API FName RemoveTransientControl(const FRigElementKey& InElement);

	/** Removes all  transient / temporary control used to interact with pins */
	UE_API void ClearTransientControls();

#endif

	virtual FRigInfluenceMapPerEvent& GetInfluences() = 0;

public:

	UFUNCTION(BlueprintCallable, Category = ControlRigAsset)
	virtual URigHierarchyController* GetHierarchyController() { return GetHierarchy()->GetController(true); }

	virtual FModularRigModel& GetModularRigModel() = 0;
	const FModularRigModel& GetModularRigModel() const { return const_cast<IControlRigEditorAssetInterface*>(this)->GetModularRigModel(); }

	UE_API virtual UModularRigController* GetModularRigController();

	UE_API virtual void RecompileModularRig();

	
	virtual void SetControlRigType(EControlRigType InType) = 0;
	virtual EControlRigType GetControlRigType() const = 0;

	virtual void SetItemTypeDisplayName(FName InName) = 0;
	virtual FName GetItemTypeDisplayName() const = 0;

protected:

	/** Whether or not this rig has an Inversion Event */
	virtual bool& GetSupportsInversion() = 0;

	/** Whether or not this rig has Controls on It */
	virtual bool& GetSupportsControls() = 0;

	/** The default skeletal mesh to use when previewing this asset */
#if WITH_EDITORONLY_DATA
	virtual TSoftObjectPtr<USkeletalMesh> GetPreviewSkeletalMesh() const = 0;
#endif

	/** The skeleton from import into a hierarchy */
	virtual TSoftObjectPtr<UObject>& GetSourceHierarchyImport() = 0;

	/** The skeleton from import into a curve */
	virtual TSoftObjectPtr<UObject>& GetSourceCurveImport() = 0;

	/** If set to true, this control rig has animatable controls */
	virtual bool& GetExposesAnimatableControls() = 0;
public:

	/** If set to true, multiple control rig tracks can be created for the same rig in sequencer*/
	virtual bool& GetAllowMultipleInstances() = 0;

protected:

	UE_API static TArray<FControlRigAssetInterfacePtr> sCurrentlyOpenedRigBlueprints;

	virtual void PathDomainSpecificContentOnLoad();
	virtual void GetBackwardsCompatibilityPublicFunctions(TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders) = 0;
	void PatchRigElementKeyCacheOnLoad();
	void PatchPropagateToChildren();

protected:
	virtual void CreateMemberVariablesOnLoad();
	virtual void PatchVariableNodesOnLoad();

public:
	UE_API void UpdateElementKeyRedirector(UControlRig* InControlRig) const;
	UE_API void PropagatePoseFromInstanceToBP(UControlRig* InControlRig) const;
	UE_API void PropagatePoseFromBPToInstances() const;
	UE_API void PropagateHierarchyFromBPToInstances() const;
	UE_API void PropagateDrawInstructionsFromBPToInstances() const;
	void PropagatePropertyFromBPToInstances(FRigElementKey InRigElement, const FProperty* InProperty) const;
	void PropagatePropertyFromInstanceToBP(FRigElementKey InRigElement, const FProperty* InProperty, UControlRig* InInstance) const;
	UE_API void PropagateModuleHierarchyFromBPToInstances() const;
	void UpdateModularDependencyDelegates();
	void OnModularDependencyVMCompiled(UObject* InBlueprint, URigVM* InVM, FRigVMExtendedExecuteContext& InExecuteContext);
	void OnModularDependencyChanged(FRigVMEditorAssetInterfacePtr InBlueprint);
	void RequestConstructionOnAllModules();
	UE_API void RefreshModuleVariables();
	UE_API void RefreshModuleVariables(const FRigModuleReference* InModule);
	UE_API void RefreshModuleConnectors();
	UE_API void RefreshModuleConnectors(const FRigModuleReference* InModule, bool bPropagateHierarchy = true);

	/**
	* Returns the modified event, which can be used to 
	* subscribe to topological changes happening within the hierarchy. The event is broadcast only after all hierarchy instances are up to date
	* @return The event used for subscription.
	*/
	FRigHierarchyModifiedEvent& OnHierarchyModified() { return HierarchyModifiedEvent; }

	FOnRigVMRefreshEditorEvent& OnModularRigPreCompiled() { return ModularRigPreCompiled; }
	FOnRigVMRefreshEditorEvent& OnModularRigCompiled() { return ModularRigCompiled; }

protected:

	virtual TObjectPtr<UControlRigValidator>& GetValidator() = 0;

	FRigHierarchyModifiedEvent	HierarchyModifiedEvent;
	FOnRigVMRefreshEditorEvent ModularRigPreCompiled;
	FOnRigVMRefreshEditorEvent ModularRigCompiled;

	int32 ModulesRecompilationBracket = 0;


	void HandleHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void HandleHierarchyElementKeyChanged(const FRigElementKey& InOldKey, const FRigElementKey& InNewKey);
	void HandleHierarchyComponentKeyChanged(const FRigComponentKey& InOldKey, const FRigComponentKey& InNewKey);

	void HandleRigModulesModified(EModularRigNotification InNotification, const FRigModuleReference* InModule);

#if WITH_EDITOR
	virtual void HandlePackageDone();
	virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure);
#endif

	void UpdateConnectionMapAfterRename(const FString& InOldModuleName);

	// Class used to temporarily cache all 
	// current control values and reapply them
	// on destruction
	class FControlValueScope
	{
	public: 
		UE_API FControlValueScope(FControlRigAssetInterfacePtr InBlueprint);
		UE_API ~FControlValueScope();

	protected:

		FControlRigAssetInterfacePtr Blueprint;
		TMap<FName, FRigControlValue> ControlValues;
	};

	virtual float& GetDebugBoneRadius() = 0;

#if WITH_EDITOR
private:
	/** Editor undo client for this interface */
	TPimplPtr<FControlRigEditorAssetUndoClient> EditorUndoClient;

public:

	/** Shape libraries to load during package load completed */ 
	TArray<FString> ShapeLibrariesToLoadOnPackageLoaded;

#endif

	friend class FControlRigBlueprintCompilerContext;
	friend class SRigHierarchy;
	friend class SRigCurveContainer;
	friend class FControlRigBaseEditor;
#if WITH_RIGVMLEGACYEDITOR
	friend class FControlRigLegacyEditor;
#endif
	friend class FControlRigEditor;
	friend class UEngineTestControlRig;
	friend class FControlRigEditMode;
	friend class FControlRigBlueprintActions;
	friend class FControlRigDrawContainerDetails;
	friend class UDefaultControlRigManipulationLayer;
	friend struct FRigValidationTabSummoner;
	friend class UAnimGraphNode_ControlRig;
	friend class UControlRigThumbnailRenderer;
	friend class FControlRigGraphDetails;
	friend class FControlRigEditorModule;
	friend class UControlRigComponent;
	friend struct FControlRigGraphSchemaAction_PromoteToVariable;
	friend class UControlRigGraphSchema;
	friend class FControlRigBlueprintDetails;
	friend class FRigConnectorElementDetails;
};

UCLASS(MinimalAPI, BlueprintType)
class UControlRigEditorAsset : public URigVMEditorAsset, public IControlRigEditorAssetInterface, public IInterface_PreviewMeshProvider, public IRigHierarchyProvider
{
	GENERATED_UCLASS_BODY()
	
	
	UPROPERTY()
	TObjectPtr<UControlRigValidator> Validator;
	
	UPROPERTY()
	float DebugBoneRadius;
	
public:
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() override { return this; }
	
	//  --- IRigVMClientHost interface ---
	virtual UClass* GetRigVMSchemaClass() const override { return IControlRigEditorAssetInterface::GetRigVMSchemaClass(); }
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const override { return IControlRigEditorAssetInterface::GetRigVMExecuteContextStruct(); }
	virtual UClass* GetRigVMEdGraphClass() const override { return IControlRigEditorAssetInterface::GetRigVMEdGraphClass(); }
	virtual UClass* GetRigVMEdGraphNodeClass() const override { return IControlRigEditorAssetInterface::GetRigVMEdGraphNodeClass(); }
	virtual UClass* GetRigVMEdGraphSchemaClass() const override { return IControlRigEditorAssetInterface::GetRigVMEdGraphSchemaClass(); }
	virtual UClass* GetRigVMEditorSettingsClass() const override { return IControlRigEditorAssetInterface::GetRigVMEditorSettingsClass(); }

	
	// IInterface_PreviewMeshProvider Overrides Begins
	UFUNCTION(BlueprintCallable, Category = "Control Rig Asset")
	virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override { return IControlRigEditorAssetInterface::SetPreviewMesh(PreviewMesh, bMarkAsDirty); }
	virtual USkeletalMesh* GetPreviewMesh() const override { return IControlRigEditorAssetInterface::GetPreviewMesh(); };
	// IInterface_PreviewMeshProvider Overrides Ends
	
	// IRigHierarchyProvider Overrides Begins
	UE_API virtual URigHierarchy* GetHierarchy() const override;
	// IRigHierarchyProvider Overrides Ends
protected:
	
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	
	// IControlRigEditorAssetInterface Overrides Begins
	virtual FControlRigAssetStrongReference GetControlRigAssetReference() const override { return FControlRigAssetStrongReference(GetRuntimeAsset()); }
	virtual bool RequiresForceLoadMembersSuper(UObject* InObject) const override { return true; } 
	virtual void SerializeSuper(FArchive& Ar) override { UObject::Serialize(Ar);  } 
	virtual void SetHierarchy(TObjectPtr<URigHierarchy> InHierarchy) override; 
	virtual FCompilerResultsLog CompileBlueprint() override;
	virtual FModularRigSettings& GetModularRigSettings() override;
	virtual FRigHierarchySettings& GetHierarchySettings() override;
	virtual FRigModuleSettings& GetRigModuleSettings() override;
	virtual void SetCustomThumbnail(FString InThumbnail) override;
	virtual FString GetCustomThumbnail() const override;
	virtual TArray<FModuleReferenceData>& GetModuleReferenceData() override;
	virtual TMap<FRigElementKey, FRigElementKeyCollection>& GetArrayConnectionMap() override;
	virtual FRigInfluenceMapPerEvent& GetInfluences() override;
	virtual FModularRigModel& GetModularRigModel() override;
	virtual void SetControlRigType(EControlRigType InType) override;
	virtual EControlRigType GetControlRigType() const override;
	virtual void SetItemTypeDisplayName(FName InName) override;
	virtual FName GetItemTypeDisplayName() const override;
	virtual bool& GetSupportsInversion() override;
	virtual bool& GetSupportsControls() override;
	virtual TSoftObjectPtr<USkeletalMesh> GetPreviewSkeletalMesh() const override;
	virtual TSoftObjectPtr<UObject>& GetSourceHierarchyImport() override;
	virtual TSoftObjectPtr<UObject>& GetSourceCurveImport() override;
	virtual bool& GetExposesAnimatableControls() override;
	virtual bool& GetAllowMultipleInstances() override;
	virtual void GetBackwardsCompatibilityPublicFunctions(TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders) override {  } 
	virtual TObjectPtr<UControlRigValidator>& GetValidator() override;
	virtual float& GetDebugBoneRadius() override;
	virtual TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& GetShapeLibraries() override;
	virtual UObject* GetDefaultHierarchyOuter() override;
	virtual const FLazyName& GetPanelPinFactoryName() const override { return IControlRigEditorAssetInterface::GetPanelPinFactoryName(); }
	virtual void SetPreviewMeshImpl(USkeletalMesh* PreviewMesh) override;
	virtual UObject* GetOuterForModularModel() override { return GetOuter(); }
	
	virtual TArray<FString> GeneratePythonCommands() override { return IControlRigEditorAssetInterface::GeneratePythonCommands(); }
	virtual TArray<FString> GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset) override;
	
#if WITH_EDITOR
	virtual void HandlePackageDone() override { return IControlRigEditorAssetInterface::HandlePackageDone(); }
	virtual void HandlePackageDoneSuper() override { return URigVMEditorAsset::HandlePackageDone(); }
	virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure) override { IControlRigEditorAssetInterface::HandleConfigureRigVMController(InClient, InControllerToConfigure); }
#endif
	// IControlRigEditorAssetInterface Overrides Ends
	
#if WITH_EDITOR
	virtual IRigVMEditorModule* GetEditorModule() const override;
#endif
};

#undef UE_API