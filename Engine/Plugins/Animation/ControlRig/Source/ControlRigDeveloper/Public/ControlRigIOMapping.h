// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigAssetReference.h"
#include "EdGraph/EdGraphPin.h"
#include "Rigs/RigHierarchyDefines.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "Styling/SlateTypes.h"

#define UE_API CONTROLRIGDEVELOPER_API

struct FControlRigAssetStrongReference;
struct FRigVMVariableMappingInfo;
struct FOptionalPinFromProperty;
class IDetailLayoutBuilder;
class UClass;
class USkeleton;
class UControlRig;

struct FControlRigIOMapping : public TSharedFromThis<FControlRigIOMapping>
{
	DECLARE_DELEGATE_RetVal(USkeleton*, FOnGetTargetSkeleton);
	DECLARE_DELEGATE_RetVal(FControlRigAssetStrongReference, FOnGetTargetAssetReference);
	DECLARE_DELEGATE_TwoParams(FOnPinCheckStateChanged, ECheckBoxState /*NewState*/, FName /** Property Name */);
	DECLARE_DELEGATE_ThreeParams(FOnVariableMappingChanged, const FName& /*PathName*/, const FName& /*Curve*/, bool /*bInput*/);

	FControlRigIOMapping() = delete;
	UE_API FControlRigIOMapping(TMap<FName, FName>& InInputMapping, TMap<FName, FName>& InOutputMapping, TArray<FOptionalPinFromProperty>& InCustomPinProperties);
	UE_API virtual ~FControlRigIOMapping();

	struct FControlsInfo
	{
		FName Name;
		FString DisplayName;
		FEdGraphPinType PinType;
		ERigControlType ControlType;
		FString DefaultValue;
	};

	// Helper enabling quering controls in editor from a Control Rig class
	struct FRigControlsData
	{
		FRigControlsData() = default;

		UE_API const TArray<FControlsInfo>& GetControls(const FControlRigAssetStrongReference& ControlRigAssetReference, USkeleton* TargetSkeleton) const;

	private:
		// Controls data
		mutable FControlRigAssetStrongReference ControlsInfoAssetReference;
		mutable TArray<FControlsInfo> ControlsInfo;
	};

	UE_API bool CreateVariableMappingWidget(IDetailLayoutBuilder& DetailBuilder);

	const TMap<FName, FRigVMExternalVariable>& GetInputVariables() const
	{
		return InputVariables;
	}
	const TMap<FName, FRigVMExternalVariable>& GetOutputVariables() const
	{
		return OutputVariables;
	}

	UE_API const TArray<FControlRigIOMapping::FControlsInfo>& GetControls() const;

	UE_API void RebuildExposedProperties();

	FOnGetTargetSkeleton& GetOnGetTargetSkeletonDelegate()
	{
		return OnGetTargetSkeletonDelegate;
	}

	FOnGetTargetAssetReference& GetOnGetTargetAssetReferenceDelegate()
	{
		return OnGetTargetAssetReferenceDelegate;
	}

	FOnPinCheckStateChanged& GetOnPinCheckStateChangedDelegate()
	{
		return OnPinCheckStateChangedDelegate;
	}

	FOnVariableMappingChanged& GetOnVariableMappingChanged()
	{
		return OnVariableMappingChangedDelegate;
	}

	UE_API bool IsInputProperty(const FName& PropertyName) const;

	UE_API void SetIOMapping(bool bInput, const FName& SourceProperty, const FName& TargetCurve);
	UE_API FName GetIOMapping(bool bInput, const FName& SourceProperty) const;

	void SetIgnoreVariablesWithNoMemory(bool bIgnore) { bIgnoreVariablesWithNoMemory = bIgnore; }

private:
	FOnGetTargetSkeleton OnGetTargetSkeletonDelegate;
	FOnGetTargetAssetReference OnGetTargetAssetReferenceDelegate;
	FOnPinCheckStateChanged OnPinCheckStateChangedDelegate;
	FOnVariableMappingChanged OnVariableMappingChangedDelegate;

	TMap<FName, FName>& InputMapping;
	TMap<FName, FName>& OutputMapping;
	TArray<FOptionalPinFromProperty>& CustomPinProperties;

	// Controls data
	FRigControlsData RigControlsData;

	bool bIgnoreVariablesWithNoMemory = false;

	// property related things
	static UE_API void GetVariables(const FControlRigAssetStrongReference& TargetSource, bool bInput, bool bIgnoreVariablesWithNoMemory, TMap<FName, FRigVMExternalVariable>& OutParameters);

	TMap<FName, FRigVMExternalVariable> InputVariables;
	TMap<FName, FRigVMExternalVariable> OutputVariables;

	// pin option related
	UE_API bool IsPropertyExposeEnabled(FName PropertyName) const;
	UE_API ECheckBoxState IsPropertyExposed(FName PropertyName) const;
	UE_API void OnPropertyExposeCheckboxChanged(ECheckBoxState NewState, FName PropertyName);

#if WITH_EDITOR
	// SRigVMVariableMappingWidget related
	UE_API void OnVariableMappingChanged(const FName& PathName, const FName& Curve, bool bInput);
	UE_API FName GetVariableMapping(const FName& PathName, bool bInput);
	UE_API void GetAvailableMapping(const FName& PathName, TArray<FName>& OutArray, bool bInput);
	UE_API void CreateVariableMapping(const FString& FilteredText, TArray< TSharedPtr<FRigVMVariableMappingInfo> >& OutArray, bool bInput);
#endif

	UE_API bool IsAvailableToMapToCurve(const FName& PropertyName, bool bInput) const;
	UE_API const FControlsInfo* FindControlElement(const FName& InControlName) const;

	UE_DEPRECATED(5.8, "Use GetTargetAssetReference instead")
	UE_API const UClass* GetTargetClass() const;
	UE_API FControlRigAssetStrongReference GetTargetAssetReference() const;
	UE_API USkeleton* GetTargetSkeleton() const;
};

#undef UE_API
