// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/RigVMSchema.h"
#include "Graph/ControlRigGraphSchema.h"
#include "ControlRigSchema.generated.h"

#define UE_API CONTROLRIGDEVELOPER_API

UCLASS(MinimalAPI, BlueprintType)
class UControlRigSchema : public URigVMSchema
{
	GENERATED_UCLASS_BODY()

public:

	virtual UClass* GetEdGraphSchemaClass() const override
	{
		return UControlRigGraphSchema::StaticClass();
	}

	UE_API virtual bool ShouldUnfoldStruct(URigVMController* InController, const UStruct* InStruct) const override;

	UE_API virtual bool SupportsUnitFunction_NoLock(URigVMController* InController, const FRigVMFunction* InUnitFunction, FRigVMRegistryHandle& InRegistry) const override;
};

#undef UE_API
