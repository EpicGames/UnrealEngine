// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VisualGraphUtils.h"
#include "Rigs/RigHierarchy.h"

#define UE_API CONTROLRIGDEVELOPER_API

struct FControlRigVisualGraphUtils
{
	static UE_API FString DumpRigHierarchyToDotGraph(URigHierarchy* InHierarchy, const FName& InEventName = NAME_None);
};

#undef UE_API
