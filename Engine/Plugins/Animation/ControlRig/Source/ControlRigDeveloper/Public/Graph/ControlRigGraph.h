// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigBlueprintLegacy.h"
#include "EdGraph/EdGraph.h"
#include "Graph/ControlRigGraphNode.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMCore/RigVM.h"
#include "EdGraph/RigVMEdGraph.h"
#include "ControlRigGraph.generated.h"

#define UE_API CONTROLRIGDEVELOPER_API

class UControlRigBlueprint;
class UControlRigGraphSchema;
class UControlRig;
class URigVMController;
struct FRigCurveContainer;

UCLASS(MinimalAPI)
class UControlRigGraph : public URigVMEdGraph
{
	GENERATED_BODY()

public:
	UE_API UControlRigGraph();

	/** Set up this graph */
	UE_API virtual void InitializeFromAsset(FRigVMEditorAssetInterfacePtr InBlueprint) override;

	UE_API virtual bool HandleModifiedEvent_Internal(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) override;

#if WITH_EDITOR

	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetBoneNameList(URigVMPin* InPin = nullptr) const
	{
		return GetElementNameList(ERigElementType::Bone);
	}
	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetControlNameList(URigVMPin* InPin = nullptr) const
	{
		return GetElementNameList(ERigElementType::Control);
	}
	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetControlNameListWithoutAnimationChannels(URigVMPin* InPin = nullptr) const
	{
		return &ControlNameListWithoutAnimationChannels;
	}
	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetNullNameList(URigVMPin* InPin = nullptr) const
	{
		return GetElementNameList(ERigElementType::Null);
	}
	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetCurveNameList(URigVMPin* InPin = nullptr) const
	{
		return GetElementNameList(ERigElementType::Curve);
	}
	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetConnectorNameList(URigVMPin* InPin = nullptr) const
	{
		return GetElementNameList(ERigElementType::Connector);
	}
	const TArray<TSharedPtr<FRigVMStringWithTag>>* GetSocketNameList(URigVMPin* InPin = nullptr) const
	{
		return GetElementNameList(ERigElementType::Socket);
	}

	UE_API virtual const TArray<TSharedPtr<FRigVMStringWithTag>>* GetNameListForWidget(const FString& InWidgetName) const override;

	UE_API void CacheNameLists(URigHierarchy* InHierarchy, const FRigVMDrawContainer* DrawContainer, TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries);
	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>>* GetElementNameList(ERigElementType InElementType = ERigElementType::Bone) const;
	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>>* GetElementNameList(URigVMPin* InPin = nullptr) const;
	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>> GetSelectedElementsNameList() const;
	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>>* GetDrawingNameList(URigVMPin* InPin = nullptr) const;
	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>>* GetShapeNameList(URigVMPin* InPin = nullptr) const;

	UE_API FReply HandleGetSelectedClicked(URigVMEdGraph* InEdGraph, URigVMPin* InPin, FString InDefaultValue);
	UE_API FReply HandleBrowseClicked(URigVMEdGraph* InEdGraph, URigVMPin* InPin, FString InDefaultValue);

private:

	template<class T>
	static bool IncludeElementInNameList(const T* InElement)
	{
		return true;
	}

	template<class T>
	void CacheNameListForHierarchy(UControlRig* InControlRig, URigHierarchy* InHierarchy, TArray<TSharedPtr<FRigVMStringWithTag>>& OutNameList, bool bFilter = true)
	{
        TArray<FRigVMStringWithTag> Names;
		UE::TScopeLock Lock(InHierarchy->ElementsLock);
		for (auto Element : *InHierarchy)
		{
			if(Element->IsA<T>())
			{
				if(!bFilter || IncludeElementInNameList<T>(Cast<T>(Element)))
				{
					static const FLinearColor Color(FLinearColor(0.0, 112.f/255.f, 224.f/255.f));
					FRigVMStringTag Tag;
					
					if(InControlRig)
					{
						if(const FRigElementKey* SourceKey = InControlRig->GetElementKeyRedirector().FindReverse(Element->GetKey()))
						{
							Tag = FRigVMStringTag(Element->GetKey().Name, Color);
							Names.Emplace(SourceKey->Name.ToString(), Tag);
							continue;
						}

						// look up the resolved name
						if(Element->GetType() == ERigElementType::Connector)
						{
							if(const FRigElementKeyRedirector::FCachedKeyArray* Cache = InControlRig->GetElementKeyRedirector().Find(Element->GetKey()))
							{
								if(!Cache->IsEmpty())
								{
									Tag = FRigVMStringTag((*Cache)[0].GetKey().Name, Color);
								}
							}
						}
					}

					Names.Emplace(Element->GetName(), Tag);
				}
			}
		}
		Names.Sort();

		OutNameList.Reset();
		OutNameList.Add(MakeShared<FRigVMStringWithTag>(FName(NAME_None).ToString()));
		for (const FRigVMStringWithTag& Name : Names)
		{
			OutNameList.Add(MakeShared<FRigVMStringWithTag>(Name));
		}
	}

	template<class T>
	void CacheNameList(const T& ElementList, TArray<TSharedPtr<FRigVMStringWithTag>>& OutNameList)
	{
		TArray<FString> Names;
		for (auto Element : ElementList)
		{
			Names.Add(Element.Name.ToString());
		}
		Names.Sort();

		OutNameList.Reset();
		OutNameList.Add(MakeShared<FRigVMStringWithTag>(FName(NAME_None).ToString()));
		for (const FString& Name : Names)
		{
			OutNameList.Add(MakeShared<FRigVMStringWithTag>(Name));
		}
	}

	TMap<ERigElementType, TArray<TSharedPtr<FRigVMStringWithTag>>> ElementNameLists;
	TArray<TSharedPtr<FRigVMStringWithTag>>	ControlNameListWithoutAnimationChannels;
	TArray<TSharedPtr<FRigVMStringWithTag>> DrawingNameList;
	TArray<TSharedPtr<FRigVMStringWithTag>> ShapeNameList;
	int32 LastHierarchyTopologyVersion;

	static UE_API TArray<TSharedPtr<FRigVMStringWithTag>> EmptyElementNameList;

#endif

	friend class UControlRigGraphNode;
	friend class SRigVMGraphNode;
	friend class UControlRigBlueprint;
};

template<>
inline bool UControlRigGraph::IncludeElementInNameList<FRigControlElement>(const FRigControlElement* InElement)
{
	return !InElement->IsAnimationChannel();
}

#undef UE_API
