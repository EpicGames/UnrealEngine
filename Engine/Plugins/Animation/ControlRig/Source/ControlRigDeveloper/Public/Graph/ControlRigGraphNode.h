// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "K2Node.h"
#include "UObject/SoftObjectPath.h"
#include "EdGraphSchema_K2.h"
#include "RigVMModel/RigVMGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "ControlRigGraphNode.generated.h"

#define UE_API CONTROLRIGDEVELOPER_API

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
struct FSlateIcon;
class UControlRigBlueprint;
#if WITH_RIGVMLEGACYEDITOR
struct FSearchTagDataPair;
#endif

/** Base class for animation ControlRig-related nodes */
UCLASS(MinimalAPI)
class UControlRigGraphNode : public URigVMEdGraphNode
{
	GENERATED_BODY()

	friend class FControlRigGraphNodeDetailsCustomization;
	friend class FControlRigBlueprintCompilerContext;
	friend class UControlRigGraph;
	friend class UControlRigGraphSchema;
	friend class UControlRigBlueprint;
	friend class FControlRigGraphTraverser;
	friend class FControlRigGraphPanelPinFactory;
	friend class SControlRigGraphPinCurveFloat;

public:

	UE_API UControlRigGraphNode();

#if WITH_EDITOR
#if WITH_RIGVMLEGACYEDITOR
	UE_API void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const;
#else
	UE_API void AddRigVMPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const;
#endif
#endif

private:

	friend class SRigVMGraphNode;
	friend class FControlRigArgumentLayout;
	friend class FControlRigGraphDetails;
	friend class UControlRigTemplateNodeSpawner;
	friend class UControlRigArrayNodeSpawner;
	friend class UControlRigIfNodeSpawner;
	friend class UControlRigSelectNodeSpawner;
};

#undef UE_API
