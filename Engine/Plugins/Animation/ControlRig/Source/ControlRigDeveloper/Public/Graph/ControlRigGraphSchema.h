// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigGraphNode.h"
#include "GraphEditorDragDropAction.h"
#include "RigVMModel/RigVMGraph.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EdGraph/RigVMEdGraphSchema.h"

#include "ControlRigGraphSchema.generated.h"

#define UE_API CONTROLRIGDEVELOPER_API

class UControlRigBlueprint;
class UControlRigGraph;
class UControlRigGraphNode;
class UControlRigGraphNode_Unit;
class UControlRigGraphNode_Property;

UCLASS(MinimalAPI)
class UControlRigGraphSchema : public URigVMEdGraphSchema
{
	GENERATED_BODY()

public:
	/** Name constants */
	static inline const FLazyName GraphName_ControlRig = FLazyName(TEXT("Rig"));

public:
	UE_API UControlRigGraphSchema();

	// UEdGraphSchema interface
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;

	// URigVMEdGraphSchema interface
	virtual const FLazyName& GetRootGraphName() const override { return GraphName_ControlRig; }
	UE_API virtual bool IsRigVMDefaultEvent(const FName& InEventName) const override;
	virtual bool SupportsPreviewFromHereOnNode(const UEdGraphNode* InNode) const override { return true; }
};

#undef UE_API
