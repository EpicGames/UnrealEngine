// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ControlRigEditor : ModuleRules
    {
        public ControlRigEditor(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"MainFrame",
					"AppFramework",
                    "RigVMEditor",
					"CurveEditor",
                    "InteractiveToolsFramework", 
                    "TweeningUtilsEditor"
				}
			);

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"AssetDefinition",
                    "Core",
                    "CoreUObject",
					"DetailCustomizations",
					"Slate",
                    "SlateCore",
                    "InputCore",
					"Engine",
					"EditorFramework",
					"UnrealEd",
                    "KismetCompiler",
                    "BlueprintGraph",
                    "ControlRig",
                    "ControlRigDeveloper",
					"KismetCompiler",
                    "EditorStyle",
					"EditorWidgets",
                    "ApplicationCore",
                    "AnimationCore",
                    "PropertyEditor",
                    "AnimGraph",
                    "AnimGraphRuntime",
                    "MovieScene",
                    "MovieSceneTracks",
                    "MovieSceneTools",
                    "SequencerCore",
                    "Sequencer",
					"LevelSequenceEditor",
                    "ClassViewer",
                    "AssetTools",
                    "ContentBrowser",
					"ContentBrowserData",
                    "LevelEditor",
                    "SceneOutliner",
                    "EditorInteractiveToolsFramework",
                    "LevelSequence",
                    "GraphEditor",
                    "PropertyPath",
                    "Persona",
                    "UMG",
					"TimeManagement",
                    "PropertyPath",
					"WorkspaceMenuStructure",
					"Json",
					"DesktopPlatform",
					"ToolMenus",
                    "RigVM",
                    "RigVMDeveloper",
                    "RigVMEditor",
					"AnimationEditor",
					"MessageLog",
                    "SequencerScripting",
					"SequencerScriptingEditor",
					"PropertyAccessEditor",
					"KismetWidgets",
					"PythonScriptPlugin",
					"AdvancedPreviewScene",
					"ToolWidgets",
                    "AnimationWidgets",
                    "AnimationEditorWidgets",
                    "ActorPickerMode",
                    "Constraints",
                    "AnimationEditMode",
					"SequencerWidgets",
					"DeveloperSettings",
					"TweeningUtils",
					"Blutility",
					"UMGEditor",
					"TypedElementFramework"
                }
            );

            bool bAddBlueprintEditorDependency = true;
            if (bAddBlueprintEditorDependency)
            {
	            PublicDependencyModuleNames.Add("Kismet");
            }

            AddEngineThirdPartyPrivateStaticDependencies(Target, "FBX");
        }
    }
}
