// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsFilter.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/Transform.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "AnimDetails/Proxies/AnimDetailsProxyTransform.h"
#include "AnimDetailsProxyManager.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "PropertyHandle.h"

namespace UE::ControlRigEditor
{
	void FAnimDetailsFilter::Update(const FText& FilterText, const TArray<TObjectPtr<UAnimDetailsProxyBase>>& Proxies)
	{
		TArray<FString> NewSearchFragments;

		constexpr bool bCullEmpty = true;
		FilterText.ToString().ParseIntoArrayWS(NewSearchFragments, TEXT(","), bCullEmpty);

		TArray<FAnimDetailsFilteredProperty> NewFilteredProperties;
		for (UAnimDetailsProxyBase* Proxy : Proxies)
		{
			if (!Proxy)
			{
				continue;
			}

			const TArray<FName> PropertyNames = Proxy->GetPropertyNames();
			for (const FName& PropertyName : PropertyNames)
			{
				const TOptional<FAnimDetailsFilteredProperty> FilteredProperty = FAnimDetailsFilteredProperty::TryCreate(NewSearchFragments, Proxy, PropertyName);

				if (FilteredProperty.IsSet())
				{
					NewFilteredProperties.Add(FilteredProperty.GetValue());
				}
			}
		}

		// Only update when data changed
		if (SearchFragments != NewSearchFragments ||
			FilteredProperties != NewFilteredProperties)
		{
			SearchFragments = NewSearchFragments;
			FilteredProperties = NewFilteredProperties;

			OnFilterChangedDelegate.Broadcast();
		}
	}

	TArray<UAnimDetailsProxyBase*> FAnimDetailsFilter::GetFilteredProxies() const
	{
		TArray<UAnimDetailsProxyBase*> Result;

		for (const FAnimDetailsFilteredProperty& FilteredProperty : FilteredProperties)
		{
			UAnimDetailsProxyBase* Proxy = FilteredProperty.WeakProxy.Get();
			if (Proxy)
			{
				Result.AddUnique(Proxy);
			}
		}

		return Result;
	}

	bool FAnimDetailsFilter::ContainsProperty(const UAnimDetailsProxyBase& Proxy, const FName& PropertyName) const
	{
		return Algo::AnyOf(FilteredProperties,
			[&Proxy, &PropertyName](const FAnimDetailsFilteredProperty& FilteredProperty)
			{		
				return
					FilteredProperty.WeakProxy.IsValid() &&
					FilteredProperty.WeakProxy == &Proxy &&
					FilteredProperty.PropertyName == PropertyName;
			});
	}

	bool FAnimDetailsFilter::ContainsProperty(const TSharedRef<IPropertyHandle>& PropertyHandle) const
	{
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		return OuterObjects.ContainsByPredicate(
			[&PropertyHandle, this](const UObject* ProxyObject)
			{
				if (!ProxyObject)
				{
					return false;
				}

				return Algo::AnyOf(FilteredProperties,
					[&PropertyHandle, &ProxyObject](const FAnimDetailsFilteredProperty& FilteredProperty)
					{
						return
							FilteredProperty.WeakProxy.IsValid() &&
							FilteredProperty.WeakProxy == ProxyObject &&
							FilteredProperty.PropertyName == PropertyHandle->GetProperty()->GetFName();
					});
			});
	}

	bool FAnimDetailsFilter::ContainsStructProperty(const TSharedRef<IPropertyHandle>& StructPropertyHandle) const
	{
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);

		// Proxies with only one property are filtered out on the object level,
		// getting customized here means they're visible.
		if (NumChildren == 1)
		{
			return true;
		}

		// Test structs with more than one child
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			if (ContainsProperty(StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef()))
			{
				return true;
			}
		}

		return false;
	}

	bool FAnimDetailsFilter::FAnimDetailsFilteredProperty::operator==(const FAnimDetailsFilteredProperty& Other) const
	{
		return
			PropertyName == Other.PropertyName &&
			WeakProxy == Other.WeakProxy;
	}

	TOptional<FAnimDetailsFilter::FAnimDetailsFilteredProperty> FAnimDetailsFilter::FAnimDetailsFilteredProperty::TryCreate(
		const TArray<FString>& SearchFragments,
		UAnimDetailsProxyBase* InProxy,
		const FName& InPropertyName)
	{
		if (!InProxy)
		{
			return TOptional<FAnimDetailsFilteredProperty>();
		}

		bool bInFilter = SearchFragments.IsEmpty();
		if (bInFilter)
		{
			return FAnimDetailsFilteredProperty(InProxy, InPropertyName);
		}

		// Test the proxy name
		const FString ProxyDisplayName = InProxy->GetDisplayNameText().ToString();
		bInFilter = Algo::AllOf(SearchFragments,
			[&ProxyDisplayName](const FString& SearchFragment)
			{
				return ProxyDisplayName.Contains(SearchFragment);
			});

		if (bInFilter)
		{
			return FAnimDetailsFilteredProperty(InProxy, InPropertyName);
		}

		TOptional<FText> OptionalStructDisplayNameText;
		FText PropertyDisplayNameText;
		InProxy->GetLocalizedPropertyName(InPropertyName, PropertyDisplayNameText, OptionalStructDisplayNameText);

		// Test the struct name
		if (OptionalStructDisplayNameText.IsSet())
		{
			const FString StructDisplayName = OptionalStructDisplayNameText.GetValue().ToString();

			bInFilter = Algo::AllOf(SearchFragments,
				[&StructDisplayName](const FString& SearchFragment)
				{
					return StructDisplayName.Contains(SearchFragment);
				});

			if (bInFilter)
			{
				return FAnimDetailsFilteredProperty(InProxy, InPropertyName);
			}
		}

		// Test the property name
		const FString PropertyDisplayName = PropertyDisplayNameText.ToString();

		bInFilter = Algo::AllOf(SearchFragments,
			[&PropertyDisplayName](const FString& SearchFragment)
			{
				return PropertyDisplayName.Contains(SearchFragment);
			});

		if (bInFilter)
		{
			return FAnimDetailsFilteredProperty(InProxy, InPropertyName);
		}

		return TOptional<FAnimDetailsFilteredProperty>();
	}

	FAnimDetailsFilter::FAnimDetailsFilteredProperty::FAnimDetailsFilteredProperty(UAnimDetailsProxyBase* InProxy, const FName& InPropertyName)
		: WeakProxy(InProxy)
		, PropertyName(InPropertyName)
	{}
}
