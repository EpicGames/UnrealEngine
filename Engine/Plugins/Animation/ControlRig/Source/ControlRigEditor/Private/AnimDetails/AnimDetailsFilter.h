// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyHandle;
class UAnimDetailsProxyBase;

namespace UE::ControlRigEditor
{
	/** Filters the details view used in anim details. */
	struct FAnimDetailsFilter
	{
	public:
		/** Filters proxies given the filter text. */
		void Update(const FText& FilterText, const TArray<TObjectPtr<UAnimDetailsProxyBase>>& Proxies);

		/** Returns proxies that match the current filter */
		TArray<UAnimDetailsProxyBase*> GetFilteredProxies() const;

		/** Returns true if the property is contained in the filter */
		bool ContainsProperty(const UAnimDetailsProxyBase& Proxy, const FName& PropertyName) const;

		/** Returns true if the property is contained in the filter */
		bool ContainsProperty(const TSharedRef<IPropertyHandle>& PropertyHandle) const;

		/** Returns true if the struct is contained in the filter */
		bool ContainsStructProperty(const TSharedRef<IPropertyHandle>& StructPropertyHandle) const;

		/** Returns a delegate broadcast when the filtered proxies changed. */
		FSimpleMulticastDelegate& GetOnFilterChanged() { return OnFilterChangedDelegate; }

	private:
		/** Fragments of the search string, trimmed, fragmented by whitespace */
		TArray<FString> SearchFragments;

		/** Describes a property that is matching the current filter */
		struct FAnimDetailsFilteredProperty
		{
			/** Tries to create an instance. The returned optional is set if creation succeeded */
			static TOptional<FAnimDetailsFilteredProperty> TryCreate(
				const TArray<FString>& SearchFragments,
				UAnimDetailsProxyBase* InProxy, 
				const FName& InPropertyName);

			bool operator==(const FAnimDetailsFilteredProperty& Other) const;

			/** The proxy */
			const TWeakObjectPtr<UAnimDetailsProxyBase> WeakProxy;

			/** The property name */
			const FName PropertyName;

		private:
			FAnimDetailsFilteredProperty() = default;
			FAnimDetailsFilteredProperty(UAnimDetailsProxyBase* InProxy, const FName& InPropertyName);
		};

		/** Properties matching the filter */
		TArray<FAnimDetailsFilteredProperty> FilteredProperties;

		/** Called when the filtered proxies changed */
		FSimpleMulticastDelegate OnFilterChangedDelegate;
	};
}
