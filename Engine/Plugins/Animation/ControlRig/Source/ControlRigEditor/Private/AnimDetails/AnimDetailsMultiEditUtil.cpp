// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsMultiEditUtil.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "Editor.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AnimDetailsMultiEditUtil"

namespace UE::ControlRigEditor
{
	namespace PropertyUtils
	{
		/** Adjusts the property values of a property by delta */
		template <typename ValueType>
		static void Adjust(const TSharedRef<IPropertyHandle>& PropertyHandle, const ValueType Delta, const bool bInteractive)
		{
			TArray<FString> PerObjectValues;
			if (PropertyHandle->GetPerObjectValues(PerObjectValues) == FPropertyAccess::Success)
			{
				for (int32 ValueIndex = 0; ValueIndex < PerObjectValues.Num(); ++ValueIndex)
				{
					ValueType OldValue;
					TTypeFromString<ValueType>::FromString(OldValue, *PerObjectValues[ValueIndex]);

					const ValueType NewValue = OldValue + Delta;
					PerObjectValues[ValueIndex] = TTypeToString<ValueType>::ToSanitizedString(NewValue);
				}

				PropertyHandle->NotifyPreChange();

				// Mind this still will modify the outer objects, work around exists in UAnimDetailsProxyBase::Modify
				const EPropertyValueSetFlags::Type ValueSetFlags = bInteractive ? 
					EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable : 
					EPropertyValueSetFlags::DefaultFlags;

				PropertyHandle->SetPerObjectValues(PerObjectValues, ValueSetFlags);

				const EPropertyChangeType::Type ChangeType = bInteractive ? 
					EPropertyChangeType::Interactive : 
					EPropertyChangeType::ValueSet;

				 PropertyHandle->NotifyPostChange(ChangeType);
			}
		}
	}

	TUniquePtr<FAnimDetailsMultiEditUtil> FAnimDetailsMultiEditUtil::Instance;

	FAnimDetailsMultiEditUtil& FAnimDetailsMultiEditUtil::Get()
	{
		if (!Instance.IsValid())
		{
			Instance = MakeUnique<FAnimDetailsMultiEditUtil>();
		}

		return *Instance;
	}

	void FAnimDetailsMultiEditUtil::Join(UAnimDetailsProxyManager* ProxyManager, const TSharedRef<IPropertyHandle>& PropertyHandle)
	{
		if (!ProxyManager)
		{
			return;
		}

		TArray<TWeakPtr<IPropertyHandle>>& Properties = ProxyManagerToPropertiesMap.FindOrAdd(ProxyManager);

		const bool bAlreadyJoined = Algo::AnyOf(Properties,
			[&PropertyHandle](const TWeakPtr<IPropertyHandle>& WeakPropertyHandle)
			{
				return PropertyHandle == WeakPropertyHandle;
			});

		if (!bAlreadyJoined)
		{
			Properties.Add(PropertyHandle);
		}
	}

	void FAnimDetailsMultiEditUtil::Leave(const TWeakPtr<IPropertyHandle>& WeakPropertyHandle)
	{
		for (auto It = ProxyManagerToPropertiesMap.CreateIterator(); It; ++It)
		{
			// Remove invalid proxy managers
			if (!(*It).Key.IsValid())
			{
				It.RemoveCurrent();
				continue;
			}

			TArray<TWeakPtr<IPropertyHandle>>& Properties = (*It).Value;

			// Remove Invalid properties
			Properties.SetNum(
				Algo::RemoveIf(Properties,
				[&WeakPropertyHandle](const TWeakPtr<IPropertyHandle>& OtherWeakProperty)
				{
					return
						!OtherWeakProperty.IsValid() ||
						!OtherWeakProperty.Pin()->IsValidHandle() ||
						OtherWeakProperty == WeakPropertyHandle;
				})
			);

			// If there are no properties for this proxy manager, remove the proxy manager
			if ((*It).Value.IsEmpty())
			{
				It.RemoveCurrent();
				continue;
			}
		}

		// Destroy self if there are no proxy managers left
		if (ProxyManagerToPropertiesMap.IsEmpty())
		{
			Instance.Reset();
		}
	}

	template <typename ValueType>
	void FAnimDetailsMultiEditUtil::MultiEditSet(
		const UAnimDetailsProxyManager& ProxyManager, 
		const ValueType Value, 
		const TSharedRef<IPropertyHandle>& InstigatorProperty)
	{
		// Don't set the same value again. This avoids issues where clearing focus on a property value widget
		// would multi set its value to other selected properties with possibly different values.
		ValueType CurrentValue;
		if (InstigatorProperty->GetValue(CurrentValue) == FPropertyAccess::Success &&
			Value == CurrentValue)
		{
			return;
		}

		const FScopedTransaction ScopedTransaction(LOCTEXT("MultiEditSetTransaction", "Set Property Value"));

		for (const TSharedRef<IPropertyHandle>& PropertyHandle : GetPropertiesBeingEdited<ValueType>(ProxyManager, InstigatorProperty))
		{
			PropertyHandle->NotifyPreChange();

			PropertyHandle->SetValue(Value);

			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}

	template void FAnimDetailsMultiEditUtil::MultiEditSet<double>(const UAnimDetailsProxyManager& ProxyManager, const double Value, const TSharedRef<IPropertyHandle>& InstigatorProperty);
	template void FAnimDetailsMultiEditUtil::MultiEditSet<int64>(const UAnimDetailsProxyManager& ProxyManager, const int64 Value, const TSharedRef<IPropertyHandle>& InstigatorProperty);
	template void FAnimDetailsMultiEditUtil::MultiEditSet<bool>(const UAnimDetailsProxyManager& ProxyManager, const bool Value, const TSharedRef<IPropertyHandle>& InstigatorProperty);

	template <typename ValueType>
	bool FAnimDetailsMultiEditUtil::MultiEditMath(
		const UAnimDetailsProxyManager& ProxyManager,
		FString Expression,
		const TSharedRef<IPropertyHandle>& InstigatorProperty)
	{
		// Plain numeric values are considered valid mathematical expressions by FBasicMathExpressionEvaluator.
		// Don't handle these here. Instead these should be set with MultiEditSet.
		if (Expression.IsNumeric())
		{
			return false;
		}

		TMap<TSharedRef<IPropertyHandle>, TArray<double>> PropertyToPerObjectValuesMap;

		const FBasicMathExpressionEvaluator Parser;

		for (const TSharedRef<IPropertyHandle>& PropertyHandle : GetPropertiesBeingEdited<ValueType>(ProxyManager, InstigatorProperty))
		{
			TArray<double>& PerObjectValues = PropertyToPerObjectValuesMap.Add(PropertyHandle);
			TArray<FString> PerObjectValueStrings;
			if (PropertyHandle->GetPerObjectValues(PerObjectValueStrings) == FPropertyAccess::Success)
			{
				for (int32 ValueIndex = 0; ValueIndex < PerObjectValueStrings.Num(); ++ValueIndex)
				{
					ValueType OldValue;
					TTypeFromString<ValueType>::FromString(OldValue, *PerObjectValueStrings[ValueIndex]);

					const TValueOrError<double, FExpressionError> ValueOrError = Parser.Evaluate(*Expression, OldValue);
					if (const double* ValuePtr = ValueOrError.TryGetValue())
					{
						PerObjectValues.Add(*ValuePtr);
					}
					else
					{
						// Does not evaulate to a math expression
						return false;
					}
				}
			}
		}

		// Apply a math operation if the string could be parsed as such
		for (const TTuple<TSharedRef<IPropertyHandle>, TArray<double>>& PropertyToPerObjectValuesPair : PropertyToPerObjectValuesMap)
		{
			TArray<FString> PerObjectValueStrings;
			Algo::Transform(PropertyToPerObjectValuesPair.Value, PerObjectValueStrings,
				[](const double& Value)
				{
					return TTypeToString<ValueType>::ToSanitizedString(Value);
				});

			PropertyToPerObjectValuesPair.Key->NotifyPreChange();

			PropertyToPerObjectValuesPair.Key->SetPerObjectValues(PerObjectValueStrings);

			PropertyToPerObjectValuesPair.Key->NotifyPostChange(EPropertyChangeType::ValueSet);
		}

		return true;
	}
	
	template bool FAnimDetailsMultiEditUtil::MultiEditMath<double>(const UAnimDetailsProxyManager& ProxyManager, FString StringValue, const TSharedRef<IPropertyHandle>& InstigatorProperty);
	template bool FAnimDetailsMultiEditUtil::MultiEditMath<int64>(const UAnimDetailsProxyManager& ProxyManager, FString StringValue, const TSharedRef<IPropertyHandle>& InstigatorProperty);

	template <typename ValueType>
	void FAnimDetailsMultiEditUtil::MultiEditChange(
		const UAnimDetailsProxyManager& ProxyManager, 
		const ValueType DesiredDelta, 
		const TSharedRef<IPropertyHandle>& InstigatorProperty, 
		const bool bInteractive)
	{
		if (bInteractive && !IsInteractive())
		{
			GEditor->BeginTransaction(LOCTEXT("MultiEditSetPropertyValues", "Set Property Value"));
		}
		bIsInteractiveChangeOngoing = bInteractive;

		PropertiesBeingEditedInteractively = GetPropertiesBeingEdited<ValueType>(ProxyManager, InstigatorProperty);

		const ValueType Delta = DesiredDelta;
		for (const TSharedRef<IPropertyHandle>& PropertyHandle : PropertiesBeingEditedInteractively)
		{
			const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();

			if constexpr (std::is_floating_point_v<ValueType>)
			{
				if (PropertyClass == FDoubleProperty::StaticClass())
				{
					PropertyUtils::Adjust<double>(PropertyHandle, Delta, bInteractive);
				}
			}
			else if constexpr (std::is_integral_v<ValueType>)
			{
				if (PropertyClass == FInt64Property::StaticClass())
				{
					PropertyUtils::Adjust<int64>(PropertyHandle, Delta, bInteractive);
				}
			}
			else
			{
				[] <bool SupportedType = false>()
				{
					static_assert(SupportedType, "Unsupported type in FAnimDetailsMultiEditUtil.");
				}();
			}
		}

		if (IsInteractive())
		{
			AccumulatedDelta = AccumulatedDelta.Get<ValueType>() + Delta;
		}
		else
		{
			GEditor->EndTransaction();

			// Don't remember edited properties and accumulated delta if this is not an interactive change
			PropertiesBeingEditedInteractively.Reset();
			AccumulatedDelta.Set<ValueType>(0);
		}
	}

	template void FAnimDetailsMultiEditUtil::MultiEditChange<double>(const UAnimDetailsProxyManager& ProxyManager, double DesiredDelta, const TSharedRef<IPropertyHandle>& InstigatorProperty, bool bInteractive);
	template void FAnimDetailsMultiEditUtil::MultiEditChange<int64>(const UAnimDetailsProxyManager& ProxyManager, int64 DesiredDelta,const TSharedRef<IPropertyHandle>& InstigatorProperty, bool bInteractive);

	template <typename ValueType>
	bool FAnimDetailsMultiEditUtil::GetInteractiveDelta(const TSharedRef<IPropertyHandle>& Property, ValueType& OutValue) const
	{
		if (PropertiesBeingEditedInteractively.Contains(Property))
		{
			OutValue = AccumulatedDelta.Get<ValueType>();
			return true;
		}

		return false;
	}

	template bool FAnimDetailsMultiEditUtil::GetInteractiveDelta<double>(const TSharedRef<IPropertyHandle>& Property, double& OutValue) const;
	template bool FAnimDetailsMultiEditUtil::GetInteractiveDelta<int64>(const TSharedRef<IPropertyHandle>& Property, int64& OutValue) const;

	template <typename ValueType>
	TArray<TSharedRef<IPropertyHandle>> FAnimDetailsMultiEditUtil::GetPropertiesBeingEdited(const UAnimDetailsProxyManager& ProxyManager, const TSharedRef<IPropertyHandle>& InstigatorProperty)
	{
		const TArray<TWeakPtr<IPropertyHandle>>* PropertyHandlesPtr = ProxyManagerToPropertiesMap.Find(&ProxyManager);
		if (!PropertyHandlesPtr)
		{
			return {};
		}

		TArray<TSharedRef<IPropertyHandle>> PropertiesBeingEdited;

		Algo::TransformIf(*PropertyHandlesPtr, PropertiesBeingEdited,
			[&ProxyManager](const TWeakPtr<IPropertyHandle>& WeakPropertyHandle)
			{
				const UAnimDetailsSelection* Selection = ProxyManager.GetAnimDetailsSelection();
				const TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
				if (!Selection ||
					!PropertyHandle.IsValid() ||
					!PropertyHandle->IsValidHandle())
				{
					return false;
				}

				const bool bIsSameType = [&PropertyHandle]()
					{
						if constexpr (std::is_same_v<ValueType, bool>)
						{
							return PropertyHandle->GetPropertyClass() == FBoolProperty::StaticClass();
						}
						else if constexpr (std::is_floating_point_v<ValueType>)
						{
							return PropertyHandle->GetPropertyClass() == FDoubleProperty::StaticClass();
						}
						else if constexpr (std::is_integral_v<ValueType>)
						{
							return PropertyHandle->GetPropertyClass() == FInt64Property::StaticClass();
						}
						else
						{
							return false;
						}
					}();

				return bIsSameType && Selection->IsPropertySelected(PropertyHandle.ToSharedRef());
			},
			[](const TWeakPtr<IPropertyHandle>& WeakPropertyHandle)
			{
				return WeakPropertyHandle.Pin().ToSharedRef();
			});

		// If the instigator is not selected, edit the instigator instead of the currently selected properties.
		const bool bInstigatorIsSelected = Algo::Find(PropertiesBeingEdited, InstigatorProperty) != nullptr;
		if (!bInstigatorIsSelected)
		{
			PropertiesBeingEdited = { InstigatorProperty };
		}

		return PropertiesBeingEdited;
	}

	template <typename ValueType>
	FAnimDetailsMultiEditUtil::FAnimDetailsVariantValue::FAnimDetailsVariantValue(const ValueType Value)
	{
		Set(Value);
	}

	template <typename ValueType>
	void FAnimDetailsMultiEditUtil::FAnimDetailsVariantValue::Set(ValueType Value)
	{
		VariantValue.Set<ValueType>(Value);
	}

	template <typename ValueType>
	ValueType FAnimDetailsMultiEditUtil::FAnimDetailsVariantValue::Get() const
	{
		const ValueType* ValuePtr = VariantValue.TryGet<ValueType>();
		return ValuePtr ? *ValuePtr : 0;
	}
}

#undef LOCTEXT_NAMESPACE
