// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class IPropertyHandle;
class UAnimDetailsProxyManager;
namespace UE::ControlRigEditor { template <typename NumericType> struct FAnimDetailsMathOperation; }

namespace UE::ControlRigEditor
{
	/** Utility to edit many possibly unrelated properties of control proxies */
	class FAnimDetailsMultiEditUtil
	{
	public:
		/** Returns the default multi edit util instance */
		static FAnimDetailsMultiEditUtil& Get();

		/** 
		 * Lets a single proxy join the util so it can be multi edited along with other proxies 
		 *
		 * @param ProxyManager			The proxy manager that owns the proxies being edited
		 * @param PropertyHandle		Property handle for the properties being edited
		 */
		void Join(UAnimDetailsProxyManager* ProxyManager, const TSharedRef<IPropertyHandle>& PropertyHandle);

		/** Lets a proxy leave the util, e.g. on destruction */
		void Leave(const TWeakPtr<IPropertyHandle>& WeakPropertyHandle);

		/** 
		 * Sets a value to all selected properties that joined this multi edit util. Supports math operations.
		 * 
		 * @param ProxyManager			The proxy manager that owns the proxies and their properties
		 * @param StringValue			The value to set for the properties
		 * @param InstigatorProperty	The property that instigated this change
		¨*/
		template <typename ValueType> 
		void MultiEditSet(const UAnimDetailsProxyManager& ProxyManager, const ValueType Value, const TSharedRef<IPropertyHandle>& InstigatorProperty);

		/**
		 * Sets a value to all selected properties that joined this multi edit util. Supports math operations.
		 *
		 * @param ProxyManager			The proxy manager that owns the proxies and their properties
		 * @param Expression			The mathematical expression to apply for the properties
		 * @param InstigatorProperty	The property that instigated this change
		 * 
		 * @return						True if a math operation was successfully applied.
		¨*/
		template <typename ValueType>
		bool MultiEditMath(const UAnimDetailsProxyManager& ProxyManager, FString Expression, const TSharedRef<IPropertyHandle>& InstigatorProperty);

		/** 
		 * Applies a delta to all selected properties that joined this multi edit util. 
		 * 
		 * @param ProxyManager			The proxy manager that owns the proxies and their properties
		 * @param Delta					The value to set for the properties
		 * @param InstigatorProperty	The property that instigated this change
		 * @param bInteractive			If true, the change is carried with interactive flags
		 */
		template <typename ValueType>
		void MultiEditChange(const UAnimDetailsProxyManager& ProxyManager, const ValueType Delta, const TSharedRef<IPropertyHandle>& InstigatorProperty, const bool bInteractive);

		/** True if the multi edit util is interactively changing values */
		bool IsInteractive() const { return bIsInteractiveChangeOngoing; }

		/**  
		 * Gets the interactive delta. Returns false if there is no interactive change on-going or the property is not being edited. 
		 * This is only available to MultiEditChange during interactive changes, always fails otherwise.
		 */
		template <typename ValueType>
		[[nodiscard]] bool GetInteractiveDelta(const TSharedRef<IPropertyHandle>& Property, ValueType& OutValue) const;

	private:
		/** Updates the properties that are currently being edited */
		template <typename ValueType>
		TArray<TSharedRef<IPropertyHandle>> GetPropertiesBeingEdited(const UAnimDetailsProxyManager& ProxyManager,  const TSharedRef<IPropertyHandle>& InstigatorProperty);

		/** True while PerObjectChange is called interactively */
		bool bIsInteractiveChangeOngoing = false;

		/** Map of proxy managers with those proxies that joined this util */
		TMap<TWeakObjectPtr<UAnimDetailsProxyManager>, TArray<TWeakPtr<IPropertyHandle>>> ProxyManagerToPropertiesMap;

		/** Struct that allows to store and retrive values of different types */
		struct FAnimDetailsVariantValue
		{
			FAnimDetailsVariantValue() = default;

			template <typename ValueType>
			FAnimDetailsVariantValue(const ValueType Value);

			/** Sets the value */
			template <typename ValueType>
			void Set(ValueType Value);

			/** Gets the value. Returns 0 if it was not previously set */
			template <typename ValueType>
			ValueType Get() const;

		private:
			TVariant<double, int64, bool> VariantValue;
		};

		/** The accumulated delta from an interactive change */
		FAnimDetailsVariantValue AccumulatedDelta;

		/** Stores the properties that are being edited while interactively changing values */
		TArray<TSharedRef<IPropertyHandle>> PropertiesBeingEditedInteractively;

		/** The proxy instance */
		static TUniquePtr<FAnimDetailsMultiEditUtil> Instance;
	};
}
