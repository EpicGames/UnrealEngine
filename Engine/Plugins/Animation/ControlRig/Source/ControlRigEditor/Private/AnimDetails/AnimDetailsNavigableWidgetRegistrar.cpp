// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsNavigableWidgetRegistrar.h"

#include "AnimDetailsProxyManager.h"

namespace UE::ControlRigEditor
{
	void FAnimDetailsNavigableWidgetRegistrar::RegisterAsNavigable(
		UAnimDetailsProxyManager& ProxyManager, 
		const TSharedRef<SWidget>& OwnerWidget,
		const TSharedRef<SWidget>& NavigableWidget)
	{
		ProxyManager.GetNavigableWidgetRegistry().RegisterNavigableWidget(OwnerWidget, NavigableWidget);

		WeakProxyManager = &ProxyManager;
		WeakOwnerWidget = NavigableWidget;
	}

	void FAnimDetailsNavigableWidgetRegistrar::RegisterAsNavigator(
		UAnimDetailsProxyManager& ProxyManager, 
		const TSharedRef<SWidget>& NavigatorWidget, 
		const TSharedRef<SWidget>& NavigateToOwner)
	{
		ProxyManager.GetNavigableWidgetRegistry().RegisterNavigatorWidget(NavigatorWidget, NavigateToOwner);

		WeakProxyManager = &ProxyManager;
		WeakOwnerWidget = NavigatorWidget;
	}
}
