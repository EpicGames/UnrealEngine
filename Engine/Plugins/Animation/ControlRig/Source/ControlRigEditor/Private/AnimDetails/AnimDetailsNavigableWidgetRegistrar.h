// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class SWidget;
class UAnimDetailsProxyManager;

namespace UE::ControlRigEditor
{
	/** Responsible to handle registration of navigable widgets with the navigable widget registry */
	struct FAnimDetailsNavigableWidgetRegistrar
	{
		/** 
		 * Registers a widget that can be navigated to with the navigable widget registry 
		 * 
		 * @param ProxyManager		The proxy manager that owns the registry
		 * @param OwnerWidget		The widget that owns the navigable widget
		 * @param NavigableWidget	The widget to navigate to
		 */
		void RegisterAsNavigable(
			UAnimDetailsProxyManager& ProxyManager, 
			const TSharedRef<SWidget>& OwnerWidget,  
			const TSharedRef<SWidget>& NavigableWidget);

		/** Registers a widget that can navigate to an owner with the navigable widget registry. 
		 * @param ProxyManager		The proxy manager that owns the registry
		 * @param NavigatorWidget	The widget that invokes the navigation
		 * @param NavigateToOwner	The owner widget to navigate to.
		 */
		void RegisterAsNavigator(
			UAnimDetailsProxyManager& ProxyManager, 
			const TSharedRef<SWidget>& NavigatorWidget, 
			const TSharedRef<SWidget>& NavigateToOwner);

	private:
		/** The proxy manager the widgets are registered with */
		TWeakObjectPtr<UAnimDetailsProxyManager> WeakProxyManager;

		/** The widget that owns this registrar */
		TWeakPtr<SWidget> WeakOwnerWidget;
	};
}
