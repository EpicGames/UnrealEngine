// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsNavigableWidgetRegistry.h"

#include "Algo/AnyOf.h"
#include "Framework/Application/SlateApplication.h"

namespace UE::ControlRigEditor
{
	const TSharedPtr<SWidget> FAnimDetailsNavigableWidgetRegistry::GetNext(const TSharedRef<SWidget>& OwnerWidget) const
	{
		for (auto It = WeakNavigatorToNavigableWidgetMap.CreateIterator(); It; ++It)
		{
			if (!(*It).Key.IsValid() || !(*It).Value.IsValid())
			{
				It.RemoveCurrent();
			}
		}

		// If the widget is a navigator, navigate to its navigable widget
		const TWeakPtr<SWidget>* OwnerWidgetPtr = WeakNavigatorToNavigableWidgetMap.Find(OwnerWidget);
		if (OwnerWidgetPtr)
		{
			const FNavigableWidget* NavigableWidgetPtr = NavigableWidgets.FindByPredicate(
				[OwnerWidgetPtr](const FNavigableWidget& NavigableWidget)
				{
					return NavigableWidget.WeakOwner == *OwnerWidgetPtr;
				});
			
			const TSharedPtr<SWidget> NavigateToWidget = NavigableWidgetPtr && NavigableWidgetPtr->WeakNavigableWidget.IsValid() ?
				NavigableWidgetPtr->WeakNavigableWidget.Pin() :
				nullptr;

			if (NavigateToWidget.IsValid() &&
				NavigateToWidget->IsEnabled() &&
				IsWidgetVisible(NavigateToWidget.ToSharedRef()))
			{
				return NavigateToWidget;
			}
		}

		// If the widget is navigable, navigate to the next index
		const int32 CurrentIndex = NavigableWidgets.IndexOfByPredicate(
			[&OwnerWidget](const FNavigableWidget& NavigableWidget)
			{
				return NavigableWidget.WeakOwner == OwnerWidget;
			});

		if (CurrentIndex == INDEX_NONE)
		{
			return nullptr;
		}

		for (int32 NextIndex = CurrentIndex + 1; NextIndex < NavigableWidgets.Num(); NextIndex++)
		{
			if (!NavigableWidgets[NextIndex].WeakNavigableWidget.IsValid())
			{
				continue;
			}

			const TSharedRef<SWidget> NextWidget = NavigableWidgets[NextIndex].WeakNavigableWidget.Pin().ToSharedRef();
			if (NextWidget->IsEnabled()) 
			{
				if (IsWidgetVisible(NextWidget))
				{
					return NextWidget;
				}
			}
		}

		return nullptr;
	}

	bool FAnimDetailsNavigableWidgetRegistry::IsWidgetVisible(const TSharedRef<SWidget>& Widget) const
	{
		TSharedPtr<SWidget> VisibilityTestWidget = Widget;
		while (VisibilityTestWidget.IsValid())
		{
			if (VisibilityTestWidget->GetVisibility() == EVisibility::Collapsed ||
				VisibilityTestWidget->GetVisibility() == EVisibility::Hidden)
			{
				return false;
			}

			VisibilityTestWidget = VisibilityTestWidget->GetParentWidget();
		}

		return true;
	}

	void FAnimDetailsNavigableWidgetRegistry::RegisterNavigableWidget(
		const TSharedRef<SWidget>& InOwnerWidget,
		const TSharedRef<SWidget>& InNavigableWidget)
	{		
		// Lazily cleanup
		NavigableWidgets.RemoveAll(
			[](const FNavigableWidget& NavigableWidget)
			{
				return
					!NavigableWidget.WeakOwner.IsValid() ||
					!NavigableWidget.WeakNavigableWidget.IsValid();
			});

		NavigableWidgets.Emplace(InOwnerWidget, InNavigableWidget);
	}

	void FAnimDetailsNavigableWidgetRegistry::RegisterNavigatorWidget(
		const TSharedRef<SWidget>& InNavigatorWidget, 
		const TSharedRef<SWidget>& InNavigateToWidget)
	{		
		// Lazily cleanup
		NavigableWidgets.RemoveAll(
			[](const FNavigableWidget& NavigableWidget)
			{
				return
					!NavigableWidget.WeakOwner.IsValid() ||
					!NavigableWidget.WeakNavigableWidget.IsValid();
			});

		WeakNavigatorToNavigableWidgetMap.Add(InNavigatorWidget, InNavigateToWidget);
	}

	FAnimDetailsNavigableWidgetRegistry::FNavigableWidget::FNavigableWidget(
		const TSharedRef<SWidget>& InWeakOwner, 
		const TSharedRef<SWidget>& InWeakNavigableWidget)
		: WeakOwner(InWeakOwner)
		, WeakNavigableWidget(InWeakNavigableWidget)
	{}
}
