// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWidget;

namespace UE::ControlRigEditor
{
	/** 
	 * Registry for navigable widgets, assuming an owner widget and an inner widget that actually can receive navigation.
	 * Used to navigate from property names or value wrappers (e.g. SAnimDetailsValueNumeric) to the actual value widgets. 
	 * Given such an owner widget, GetNext will return the correct value widget to navigate to.
	 */
	class FAnimDetailsNavigableWidgetRegistry
	{
		// Let the registrar register and unregister widgets
		friend struct FAnimDetailsNavigableWidgetRegistrar;

	public:
		/**
		 * Returns the next widget given the owner widget, whereas the owner widget 
		 * can be either registered as navigator or as navigable owner.
		 * 
		 * Returns nullptr if the widget is not registered with this registry. 
		 */
		const TSharedPtr<SWidget> GetNext(const TSharedRef<SWidget>& OwnerWidget) const;

	private:
		/** Does a deep test if a widget is truly visible. */
		bool IsWidgetVisible(const TSharedRef<SWidget>& Widget) const;

		/** Registers a widget that can be navigated to */
		void RegisterNavigableWidget(
			const TSharedRef<SWidget>& InOwnerWidget,
			const TSharedRef<SWidget>& InNavigableWidget);

		/** Registers a widget that can be navigated to a navigable widget. Requires the navigable widget to be registered. */
		void RegisterNavigatorWidget(
			const TSharedRef<SWidget>& InNavigatorWidget, 
			const TSharedRef<SWidget>& InNavigateToWidget);

		/** A widget that can be navigated to */
		struct FNavigableWidget
		{
			FNavigableWidget(const TSharedRef<SWidget>& InWeakOwner, const TSharedRef<SWidget>& InWeakNavigableWidget);

			const TWeakPtr<SWidget> WeakOwner;
			const TWeakPtr<SWidget> WeakNavigableWidget;
		};

		/** The current navigable widgets */
		mutable TArray<FNavigableWidget> NavigableWidgets;

		/** The current navigator widgets with their navigate to widget */
		mutable TMap<TWeakPtr<SWidget>, TWeakPtr<SWidget>> WeakNavigatorToNavigableWidgetMap;
	};
}
