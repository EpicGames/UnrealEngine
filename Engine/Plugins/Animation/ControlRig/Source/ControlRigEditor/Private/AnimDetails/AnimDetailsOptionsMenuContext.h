// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsOptionsMenuContext.generated.h"

namespace UE::ControlRigEditor { class SAnimDetailsOptions; }

/** Menu context for the anim details options menu */
UCLASS()
class UAnimDetailsOptionsMenuContext
	: public UObject
{
	GENERATED_BODY()

public:
	/** The anim details options widget that uses this menu context */
	TWeakPtr<UE::ControlRigEditor::SAnimDetailsOptions> WeakAnimDetailsOptions;
};
