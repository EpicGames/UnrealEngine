// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyManager.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "AnimDetails/AnimDetailsMultiEditUtil.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Customizations/AnimDetailsPropertyRowExtensions.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBool.h"
#include "AnimDetails/Proxies/AnimDetailsProxyEnum.h"
#include "AnimDetails/Proxies/AnimDetailsProxyFloat.h"
#include "AnimDetails/Proxies/AnimDetailsProxyInteger.h"
#include "AnimDetails/Proxies/AnimDetailsProxyLocation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyRotation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyScale.h"
#include "AnimDetails/Proxies/AnimDetailsProxyTransform.h"
#include "AnimDetails/Proxies/AnimDetailsProxyVector2D.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "ISequencer.h"
#include "LevelEditorViewport.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Selection/SequencerOutlinerSelection.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneIntegerTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyManager)

UAnimDetailsProxyManager::UAnimDetailsProxyManager()
{
	if (!IsTemplate())
	{
		AnimDetailsSelection = NewObject<UAnimDetailsSelection>(this, "AnimDetailsSelection", RF_Transactional);

		UE::ControlRigEditor::FAnimDetailsPropertyRowExtensions::Get().RegisterRowExtensions();
	}
}

void UAnimDetailsProxyManager::BeginDestroy()
{
	Super::BeginDestroy();

	if (!IsTemplate())
	{
		UE::ControlRigEditor::FAnimDetailsPropertyRowExtensions::Get().UnregisterRowExtensions();
	}
}

void UAnimDetailsProxyManager::SetSuspended(const bool bInSuspended)
{
	if (bSuspended == bInSuspended)
	{
		return;
	}

	bSuspended = bInSuspended;

	if (bSuspended)
	{
		ClearSequencer();

		// Clear out selection so it won't update itself
		AnimDetailsSelection = nullptr;

	}
	else
	{
		const bool bSuccess = UpdateSequencer();

		// Reinstall selection
		AnimDetailsSelection = NewObject<UAnimDetailsSelection>(this, "AnimDetailsSelection", RF_Transactional);

	}
}

TSharedPtr<ISequencer> UAnimDetailsProxyManager::GetSequencer() const
{
	return WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
}

void UAnimDetailsProxyManager::RequestUpdateProxies()
{
	if (bSuspended)
	{
		return;
	}

	if (ensureMsgf(IsInGameThread(), TEXT("Anim Details proxies can only be updated in game thread. Ignoring call")) &&
		!RequestUpdateProxiesTickerHandle.IsValid())
	{
		const TWeakObjectPtr<UAnimDetailsProxyManager> WeakThis = this;
		RequestUpdateProxiesTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThis](float DeltaTime)
			{
				if (!WeakThis.IsValid())
				{
					return false;
				}

				// Never update proxies while anim details self is editing interactively, but handle the request after
				if (UE::ControlRigEditor::FAnimDetailsMultiEditUtil::Get().IsInteractive())
				{
					return true;
				}

				WeakThis->ForceUpdateProxies();
				
				return false;
			})
		);
	}
}

void UAnimDetailsProxyManager::RequestUpdateProxyValues()
{
	if (bSuspended)
	{
		return;
	}

	if (ensureMsgf(IsInGameThread(), TEXT("Anim Details proxy values can only be updated in game thread. Ignoring call")) &&
		!RequestUpdateProxyValuesTimerHandle.IsValid())
	{
		RequestUpdateProxyValuesTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(
			FTimerDelegate::CreateUObject(this, &UAnimDetailsProxyManager::ForceUpdateProxyValues));
	}
}

UAnimDetailsSelection* UAnimDetailsProxyManager::GetAnimDetailsSelection()
{
	return AnimDetailsSelection;
}

const UAnimDetailsSelection* UAnimDetailsProxyManager::GetAnimDetailsSelection() const
{
	return AnimDetailsSelection;
}

void UAnimDetailsProxyManager::PostUndo(bool bSuccess)
{
	// Update proxies to reflect any external changes
	RequestUpdateProxies();
}

void UAnimDetailsProxyManager::PostRedo(bool bSuccess) 
{
	// Same as undo
	PostUndo(bSuccess);
}

void UAnimDetailsProxyManager::ClearSequencer()
{
	WeakSequencer.Reset();

	ReleaseBindings();
}

bool UAnimDetailsProxyManager::UpdateSequencer()
{
	if (bSuspended)
	{
		WeakSequencer.Reset();

		return false;
	}

	const TWeakPtr<ISequencer> NewWeakSequencer = UE::AnimationEditMode::GetSequencer();
	
	if (WeakSequencer.IsValid() && WeakSequencer == NewWeakSequencer)
	{
		// Nothing changed
		return true;
	}

	if (!NewWeakSequencer.IsValid())
	{
		// No new sequencer
		ReleaseBindings();

		return false;
	}

	// Reset the previous proxies
	Proxies.Reset();
	ExternalSelection.Reset();

	// Set the new sequencer and refresh from that
	WeakSequencer = NewWeakSequencer;

	SetupBingings();
	RequestUpdateProxies();

	return true;
}

void UAnimDetailsProxyManager::SetupBingings()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	if (Sequencer.IsValid())
	{
		if (!Sequencer->OnMovieSceneBindingsChanged().IsBoundToObject(this))
		{
			Sequencer->OnMovieSceneBindingsChanged().AddUObject(this, &UAnimDetailsProxyManager::RequestUpdateProxies);
		}

		if (!Sequencer->GetSelectionChangedObjectGuids().IsBoundToObject(this))
		{
			Sequencer->GetSelectionChangedObjectGuids().AddUObject(this, &UAnimDetailsProxyManager::OnSequencerSelectionChanged);
		}
	}

	// Listen to the control rig edit mode
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode &&
		!EditMode->OnControlRigAddedOrRemoved().IsBoundToObject(this) &&
		!EditMode->OnControlRigShapeActorsRecreated().IsBoundToObject(this) &&
		!EditMode->OnControlRigSelected().IsBoundToObject(this))
	{
		EditMode->OnControlRigAddedOrRemoved().AddUObject(this, &UAnimDetailsProxyManager::OnControlRigControlAdded);
		EditMode->OnControlRigShapeActorsRecreated().AddUObject(this, &UAnimDetailsProxyManager::OnControlRigShapeActorsRecreated);
		EditMode->OnControlRigSelected().AddUObject(this, &UAnimDetailsProxyManager::OnControlRigSelectionChanged);
	}

	// Listen to objects being replaced
	if (!FCoreUObjectDelegates::OnObjectsReplaced.IsBoundToObject(this))
	{
		FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UAnimDetailsProxyManager::OnObjectsReplaced);
	}
}

void UAnimDetailsProxyManager::ReleaseBindings()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	if (Sequencer.IsValid())
	{
		Sequencer->OnMovieSceneBindingsChanged().RemoveAll(this);
		Sequencer->GetSelectionChangedObjectGuids().RemoveAll(this);
	}

	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		EditMode->OnControlRigAddedOrRemoved().RemoveAll(this);
		EditMode->OnControlRigShapeActorsRecreated().RemoveAll(this);
		EditMode->OnControlRigSelected().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

void UAnimDetailsProxyManager::OnControlRigShapeActorsRecreated()
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode && !EditMode->AreEditingControlRigDirectly())
	{
		RequestUpdateProxies();
	}
}

void UAnimDetailsProxyManager::OnControlRigControlAdded(UControlRig* ControlRig, bool bIsAdded)
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode && !EditMode->AreEditingControlRigDirectly())
	{
		RequestUpdateProxies();
	}
}

void UAnimDetailsProxyManager::OnControlRigSelectionChanged(UControlRig* ControlRig, const FRigElementKey& RigElementKey, const bool bIsSelected)
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	const bool bInNotChangingSelection = AnimDetailsSelection && !AnimDetailsSelection->IsAnimLayersChangingSelection();
	if (EditMode && !EditMode->AreEditingControlRigDirectly() && bInNotChangingSelection)
	{
		RequestUpdateProxies();
	}
}

void UAnimDetailsProxyManager::OnSequencerSelectionChanged(TArray<FGuid> ObjectGuids)
{
	// Don't update proxies if anim layers is changing selection. Instead let the selection object override its selection.
	const bool bInNotChangingSelection = AnimDetailsSelection && !AnimDetailsSelection->IsAnimLayersChangingSelection();
	if (bInNotChangingSelection)
	{
		RequestUpdateProxies();
	}
}

void UAnimDetailsProxyManager::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	RequestUpdateProxies();
}

void UAnimDetailsProxyManager::ForceUpdateProxies()
{
	if (!ensureMsgf(IsInGameThread(), TEXT("Anim Details proxies can only be updated in game thread. Ignoring call")))
	{
		return;
	}

	FTSTicker::RemoveTicker(RequestUpdateProxiesTickerHandle);
	RequestUpdateProxiesTickerHandle.Reset();

	const FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (!EditMode || !UpdateSequencer())
	{
		Proxies.Reset();
		return;
	}

	RemoveInvalidProxies();
	ExternalSelection.Reset();

	if (!WeakSequencer.IsValid())
	{
		return;
	}
	const TSharedRef<ISequencer> Sequencer = WeakSequencer.Pin().ToSharedRef();

	// Update control rig proxies
	{
		if (!EditMode->AreEditingControlRigDirectly())
		{
			TMap<UControlRig*, TArray<FRigElementKey>> ControlRigToSelectedKeysMap;
			EditMode->GetAllSelectedControls(ControlRigToSelectedKeysMap);
			
			// Add grouped children to selected controls
			for (TPair <UControlRig*, TArray<FRigElementKey>>& SelectedKeys : ControlRigToSelectedKeysMap)
			{
				if (SelectedKeys.Key != nullptr)
				{
					if (URigHierarchy* Hierarchy = SelectedKeys.Key->GetHierarchy())
					{
						for (int32 Index = 0; Index < SelectedKeys.Value.Num(); ++Index)
						{
							if (FRigControlElement* ParentControl = Hierarchy->Find<FRigControlElement>(SelectedKeys.Value[Index]))
							{
								TArrayView<FRigBaseElement*> Children = Hierarchy->GetChildren(ParentControl);
								for (FRigBaseElement* BaseElement : Children)
								{
									if (FRigControlElement* Control = Cast<FRigControlElement>(BaseElement))
									{
										//if groupable and not contained/selected add it to the array
										FRigElementKey ChildKey = Control->GetKey();
										if (SelectedKeys.Value.Contains(ChildKey) == false && Hierarchy->ShouldBeGrouped(Control))
										{
											Index = SelectedKeys.Value.Insert(ChildKey, Index);
										}
									}
								}
								//now also check the WHOLE hierarchy to see if it's a hosted or whatever channel
								//similar to what's done in the track editor

								Hierarchy->ForEach<FRigControlElement>([Hierarchy, ParentControl, &SelectedKeys,&Index](FRigControlElement* OtherControlElement) -> bool
								{
									FRigElementKey ChildKey = OtherControlElement->GetKey();
									if (SelectedKeys.Value.Contains(ChildKey) == false && FControlRigParameterTrackEditor::ShouldBeSelected(Hierarchy, ParentControl, OtherControlElement))
									{
										Index = SelectedKeys.Value.Insert(ChildKey, Index);
									}
									return true;
								});
							}
						}
					}
				}
			}

			for (const TTuple<UControlRig*, TArray<FRigElementKey>>& ControlRigToSelectedKeysPair : ControlRigToSelectedKeysMap)
			{
				UControlRig* ControlRig = ControlRigToSelectedKeysPair.Key;
				URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
				if (!ControlRig || !Hierarchy)
				{
					continue;
				}

				// In order of the hierarchy
				TArray<FRigControlElement*> OrderedControlElements;
				ControlRig->GetControlsInOrder(OrderedControlElements);

				Algo::SortBy(ControlRigToSelectedKeysPair.Value,
					[&OrderedControlElements](const FRigElementKey& ElementKey)
					{
						return OrderedControlElements.IndexOfByPredicate(
							[&ElementKey](const FRigControlElement* ControlElement)
							{
								return
									ControlElement &&
									ControlElement->GetKey() == ElementKey;
							});
					});

				for (const FRigElementKey& ElementKey : ControlRigToSelectedKeysPair.Value)
				{
					FRigControlElement* ControlElement = ControlRig->FindControl(ElementKey.Name);
					if (!ControlElement)
					{
						continue;
					}

					UAnimDetailsProxyBase* Proxy = GetOrCreateControlRigProxy(ControlRigToSelectedKeysPair.Key, ControlElement);
					AddProxyToExternalSelectionIfValid(Proxy);

					// Also add children of the selected elements
					if (ElementKey.Type != ERigElementType::Control ||
						!ControlElement->CanDriveControls())
					{
						continue;
					}

					const TArray<FRigElementKey>& DrivenKeys = ControlElement->Settings.DrivenControls;
					for (const FRigElementKey& DrivenKey : DrivenKeys)
					{
						if (FRigControlElement* DrivenControlElement = Hierarchy->Find<FRigControlElement>(DrivenKey))
						{
							UAnimDetailsProxyBase* DrivenProxy = GetOrCreateControlRigProxy(ControlRig, DrivenControlElement);
							AddProxyToExternalSelectionIfValid(DrivenProxy);
						}
					}
				}
			}
		}
	}

	// Update sequencer proxies
	{
		// Just care about the outliner selection
		TArray<FGuid> ObjectGuids;
		TSet<FGuid> ObjectGuidsSet;
		using namespace UE::Sequencer;
		const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
		if (TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
		{
			if (SequencerSelection->Outliner.Num() > 0)
			{
				for (FViewModelPtr Model : SequencerSelection->Outliner)
				{
					TSharedPtr<IObjectBindingExtension> ObjectBinding = Model->FindAncestorOfType<IObjectBindingExtension>(true);
					if (ObjectBinding)
					{
						ObjectGuidsSet.Add(ObjectBinding->GetObjectGuid());
					}
				}
			}
		}
		ObjectGuids = ObjectGuidsSet.Array();

		const TMap<FGuid, TArray<UMovieScenePropertyTrack*>> ObjectGuidToPropertyTracksMap = GetPropertyTracks(Sequencer, ObjectGuids);

		for (const TTuple<FGuid, TArray<UMovieScenePropertyTrack*>>& ObjectGuidToPropertyTracksPair : ObjectGuidToPropertyTracksMap)
		{
			const FGuid& ID = ObjectGuidToPropertyTracksPair.Key;
			const TArray<UMovieScenePropertyTrack*> PropertyTracks = ObjectGuidToPropertyTracksPair.Value;
			if (PropertyTracks.IsEmpty())
			{
				continue;
			}

			const TArray<UObject*> BoundObjects = GetBoundObjectsFromTrack(Sequencer, ID);

			for (UObject* BoundObject : BoundObjects)
			{
				if (!BoundObject)
				{
					continue;
				}

				for (UMovieScenePropertyTrack* PropertyTrack : PropertyTracks)
				{
					if (!PropertyTrack)
					{
						continue;
					}

					//  Only get or create proxies that are not added via their control rig already
					const bool bAlreadyAddedAsControlRig = Algo::AnyOf(ExternalSelection,
						[BoundObject](const UAnimDetailsProxyBase* Proxy)
						{
							const UControlRig* ControlRig = Proxy ? Proxy->GetControlRig() : nullptr;
							const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig ? ControlRig->GetObjectBinding() : nullptr;
							const USceneComponent* SceneComponent = ObjectBinding ? Cast<USceneComponent>(ObjectBinding->GetBoundObject()) : nullptr;

							// In control rig the object binding is always a scene component
							const AActor* ParentActor = SceneComponent ? SceneComponent->GetOwner() : nullptr;

							return ParentActor && ParentActor == BoundObject;
						});

					if (!bAlreadyAddedAsControlRig)
					{
						const TSharedRef<FTrackInstancePropertyBindings> Bindings = MakeShared<FTrackInstancePropertyBindings>(PropertyTrack->GetPropertyName(), PropertyTrack->GetPropertyPath().ToString());
						UAnimDetailsProxyBase* Proxy = GetOrCreateSequencerProxy(BoundObject, PropertyTrack, Bindings);

						AddProxyToExternalSelectionIfValid(Proxy);
					}
				}
			}
		}
	}

	ForceUpdateProxyValues();

	// Grouped proxies are displayed first, reflect that in the external selection so that anim details and its selection use the same order
	ExternalSelection.StableSort(
		[](const UAnimDetailsProxyBase& ProxyA, UAnimDetailsProxyBase& ProxyB)
		{
			const bool bIsIndividualA = ProxyA.bIsIndividual;
			const bool bIsIndividualB = ProxyB.bIsIndividual;
			if (bIsIndividualA != bIsIndividualB)
			{
				return !bIsIndividualA;
			}

			return false;
		});

	OnProxiesChangedDelegate.Broadcast();
}

void UAnimDetailsProxyManager::ForceUpdateProxyValues()
{
	if (!ensureMsgf(IsInGameThread(), TEXT("Anim Details proxies can only be updated in game thread. Ignoring call")))
	{
		return;
	}

	// FTimerHandle has to be invalidated on the game thread
	RequestUpdateProxyValuesTimerHandle.Invalidate();

	if (bSuspended)
	{
		return;
	}

	if (ExternalSelection.IsEmpty())
	{
		return;
	}

	if (UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr)
	{
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.EvaluateAllConstraints();
	}

	for (UAnimDetailsProxyBase* Proxy : ExternalSelection)
	{
		Proxy->AdoptValues();
	}
}

void UAnimDetailsProxyManager::RemoveInvalidProxies()
{
	Proxies.SetNum(Algo::RemoveIf(Proxies,
		[](UAnimDetailsProxyBase* Proxy)
		{
			const bool bValidControlRig = Proxy->GetControlRig() && Proxy->GetControlElement();
			const bool bValidSequenderTrack = Proxy->GetSequencerItem().IsValid();
			
			return !bValidControlRig || !bValidSequenderTrack;
		})
	);
}

UAnimDetailsProxyBase* UAnimDetailsProxyManager::GetOrCreateControlRigProxy(UControlRig* ControlRig, FRigControlElement* ControlElement)
{
	if (!ControlRig || !ControlElement)
	{
		return nullptr;
	}

	const TObjectPtr<UAnimDetailsProxyBase>* ProxyPtr = Algo::FindByPredicate(Proxies,
		[&ControlRig, &ControlElement](const TObjectPtr<UAnimDetailsProxyBase> Proxy)
		{
			return
				Proxy &&
				Proxy->GetControlRig() == ControlRig &&
				Proxy->GetControlElement() == ControlElement;
		});

	UAnimDetailsProxyBase* Proxy = nullptr;
	if (ProxyPtr)
	{
		Proxy = *ProxyPtr;
		Proxy->SetControlFromControlRig(ControlRig, ControlElement->GetKey().Name);
	}
	else
	{
		const FName ProxyName = ControlElement->GetKey().Name;

		const FRigControlSettings& Settings = ControlElement->Settings;
		Proxy = NewProxyFromType(this, Settings.ControlType, Settings.ControlEnum, ProxyName);
		if (!Proxy)
		{
			return nullptr;
		}

		Proxy->Type = Settings.ControlType;
		Proxy->SetControlFromControlRig(ControlRig, ProxyName);
		Proxy->bIsIndividual = (ControlElement->IsAnimationChannel()) || (Settings.AnimationType == ERigControlAnimationType::ProxyControl);
		Proxy->Modify();

		Proxies.Add(Proxy);
	}

	return Proxy;
}

UAnimDetailsProxyBase* UAnimDetailsProxyManager::GetOrCreateSequencerProxy(UObject* BoundObject, UMovieScenePropertyTrack* PropertyTrack, const TSharedPtr<FTrackInstancePropertyBindings>& Binding)
{
	if (!BoundObject || !PropertyTrack || !Binding.IsValid())
	{
		return nullptr;
	}

	const TObjectPtr<UAnimDetailsProxyBase>* ProxyPtr = Algo::FindByPredicate(Proxies, 
		[&BoundObject, &PropertyTrack, &Binding](const TObjectPtr<UAnimDetailsProxyBase> Proxy)
		{
			return 
				Proxy && 
				Proxy->GetSequencerItem().GetBoundObject() == BoundObject &&
				Proxy->GetSequencerItem().GetMovieSceneTrack() == PropertyTrack &&
				Proxy->GetSequencerItem().GetBinding() == Binding;
		});

	UAnimDetailsProxyBase* Proxy = nullptr;
	if (ProxyPtr)
	{
		// Existing proxies do not need any updates
		Proxy = *ProxyPtr;
	}
	else
	{
		// Find the type of proxy to create
		ERigControlType ControlType;
		if (!TryGetControlTypeFromTrackType(*PropertyTrack, ControlType))
		{
			// Unsupported control type, quietly fail
			return nullptr;
		}

		UEnum* Enum = nullptr; // Enums are not supported
		const FName ProxyName = *(PropertyTrack->GetFName().ToString() + TEXT(".") + Binding->GetPropertyName().ToString());
		Proxy = NewProxyFromType(this, ControlType, Enum, ProxyName);
		if (!Proxy)
		{
			return nullptr;
		}

		Proxy->SetControlFromSequencerBinding(BoundObject, PropertyTrack, Binding);
		Proxy->bIsIndividual = !(
			ControlType == ERigControlType::Transform ||
			ControlType == ERigControlType::TransformNoScale ||
			ControlType == ERigControlType::EulerTransform);
		Proxy->Modify();

		Proxies.Add(Proxy);
	}

	return Proxy;
}

UAnimDetailsProxyBase* UAnimDetailsProxyManager::NewProxyFromType(UAnimDetailsProxyManager* Owner, ERigControlType ControlType, const TObjectPtr<UEnum>& InEnumPtr, const FName& ProxyName) const
{
	const FName UniqueName = MakeUniqueObjectName(Owner, UAnimDetailsProxyBase::StaticClass(), ProxyName);

	UAnimDetailsProxyBase* Proxy = nullptr;
	switch (ControlType)
	{
	case ERigControlType::Transform:
	case ERigControlType::TransformNoScale:
	case ERigControlType::EulerTransform:
	{
		Proxy = NewObject<UAnimDetailsProxyTransform>(Owner, UniqueName, RF_Transactional);
		break;
	}
	case ERigControlType::Float:
	case ERigControlType::ScaleFloat:
	{
		Proxy = NewObject<UAnimDetailsProxyFloat>(Owner, UniqueName, RF_Transactional);
		break;
	}
	case ERigControlType::Integer:
	{
		if (InEnumPtr == nullptr)
		{
			Proxy = NewObject<UAnimDetailsProxyInteger>(Owner, UniqueName, RF_Transactional);
		}
		else
		{
			UAnimDetailsProxyEnum* EnumProxy = NewObject<UAnimDetailsProxyEnum>(Owner, UniqueName, RF_Transactional);
			EnumProxy->Enum.EnumType = InEnumPtr;
			Proxy = EnumProxy;
		}
		break;

	}
	case ERigControlType::Position:
	{
		Proxy = NewObject<UAnimDetailsProxyLocation>(Owner, UniqueName, RF_Transactional);
		break;
	}
	case ERigControlType::Rotator:
	{
		Proxy = NewObject<UAnimDetailsProxyRotation>(Owner, UniqueName, RF_Transactional);
		break;
	}
	case ERigControlType::Scale:
	{
		Proxy = NewObject<UAnimDetailsProxyScale>(Owner, UniqueName, RF_Transactional);
		break;
	}
	case ERigControlType::Vector2D:
	{
		Proxy = NewObject<UAnimDetailsProxyVector2D>(Owner, UniqueName, RF_Transactional);
		break;
	}
	case ERigControlType::Bool:
	{
		Proxy = NewObject<UAnimDetailsProxyBool>(Owner, UniqueName, RF_Transactional);
		break;
	}
	default:
		break;
	}

	if (Proxy)
	{
		Proxy->Type = ControlType;
	}

	return Proxy;
}

void UAnimDetailsProxyManager::ReevaluateConstraints()
{
	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);

	for (const TObjectPtr<UAnimDetailsProxyBase>& Proxy : Proxies)
	{
		if (!Proxy)
		{
			continue;
		}

		Proxy->Modify();

		Controller.EvaluateAllConstraints();
		Proxy->AdoptValues();
	}
}

bool UAnimDetailsProxyManager::TryGetControlTypeFromTrackType(const UMovieScenePropertyTrack& InPropertyTrack, ERigControlType& OutControlRigType) const
{
	if (InPropertyTrack.IsA<UMovieScene3DTransformTrack>())
	{
		OutControlRigType = ERigControlType::Transform;
		return true;
	}
	else if (InPropertyTrack.IsA<UMovieSceneBoolTrack>())
	{
		OutControlRigType = ERigControlType::Bool;
		return true;
	}
	else if (InPropertyTrack.IsA<UMovieSceneIntegerTrack>())
	{
		OutControlRigType = ERigControlType::Integer;
		return true;
	}
	else if (InPropertyTrack.IsA<UMovieSceneDoubleTrack>() ||
		InPropertyTrack.IsA<UMovieSceneFloatTrack>())
	{
		OutControlRigType = ERigControlType::Float;
		return true;
	}

	return false;
}

TMap<FGuid, TArray<UMovieScenePropertyTrack*>> UAnimDetailsProxyManager::GetPropertyTracks(const TSharedRef<ISequencer>& Sequencer, const TArray<FGuid>& ObjectGuids) const
{
	using namespace UE::Sequencer;

	TMap<FGuid, TArray<UMovieScenePropertyTrack*>> ObjectGuidToPropertyTracksMap;

	const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
	const FViewModelPtr RootModel = SequencerViewModel.IsValid() ? SequencerViewModel->GetRootModel() : nullptr;
	const FTrackModelStorageExtension* TrackStorage = RootModel.IsValid() ? RootModel->CastDynamic<FTrackModelStorageExtension>() : nullptr;
	if (!SequencerViewModel.IsValid() ||
		!RootModel.IsValid() ||
		!TrackStorage)
	{
		return ObjectGuidToPropertyTracksMap;
	}

	const auto FindPropertyTracks = [&](const UMovieScene* MovieScene)
		{
			if (!MovieScene)
			{
				return;
			}

			for (const FGuid& ObjectGuid : ObjectGuids)
			{
				const TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieScenePropertyTrack::StaticClass(), ObjectGuid);

				for (UMovieSceneTrack* Track : Tracks)
				{
					if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(Track))
					{
						ObjectGuidToPropertyTracksMap.FindOrAdd(ObjectGuid).Add(PropertyTrack);
					}
				}

				// Sort in sequencer order
				for (TTuple<FGuid, TArray<UMovieScenePropertyTrack*>>& ObjectGuidToPropertyTracksPair : ObjectGuidToPropertyTracksMap)
				{
					ObjectGuidToPropertyTracksPair.Value.Sort(
						[&](UMovieScenePropertyTrack& TrackA, UMovieScenePropertyTrack& TrackB)
						{
							const TSharedPtr<FTrackModel> TrackModelA = TrackStorage->FindModelForTrack(&TrackA);
							const TSharedPtr<FTrackModel> TrackModelB = TrackStorage->FindModelForTrack(&TrackB);

							if (TrackModelA && TrackModelB)
							{
								return FSortingKey::CompareCustomOrderFirst(TrackModelA->GetSortingKey(), TrackModelB->GetSortingKey());
							}

							return false;
						});
				}
			}
		};

	// Find property tracks of the movie scene in the root sequence
	const UMovieSceneSequence* RootSequence = Sequencer->GetRootMovieSceneSequence();
	const UMovieScene* RootMovieScene = RootSequence ? RootSequence->GetMovieScene() : nullptr;
	if (RootMovieScene)
	{
		FindPropertyTracks(RootMovieScene);
	}

	// Find property tracks of movie scenes in sub sequences
	const FMovieSceneRootEvaluationTemplateInstance& EvaluationTemplate = Sequencer->GetEvaluationTemplate();
	const UMovieSceneCompiledDataManager* CompiledDataManager = EvaluationTemplate.GetCompiledDataManager();
	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager ? CompiledDataManager->FindHierarchy(EvaluationTemplate.GetCompiledDataID()) : nullptr;
	if (!Hierarchy)
	{
		return ObjectGuidToPropertyTracksMap;
	}

	for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& SequenceIDToSubsequenceDataPair : Hierarchy->AllSubSequenceData())
	{
		const UMovieSceneSequence* SubSequence = SequenceIDToSubsequenceDataPair.Value.GetSequence();
		const UMovieScene* InnerMovieScene = SubSequence ? SubSequence->GetMovieScene() : nullptr;
		if (!SubSequence || !InnerMovieScene)
		{
			continue;
		}

		FindPropertyTracks(InnerMovieScene);
	}

	return ObjectGuidToPropertyTracksMap;
}

TArray<UObject*> UAnimDetailsProxyManager::GetBoundObjectsFromTrack(const TSharedRef<ISequencer>& Sequencer, const FGuid& ObjectGuid) const
{
	TArray<UObject*> BoundObjects;
	for (const TWeakObjectPtr<UObject>& BoundObject : Sequencer->FindBoundObjects(ObjectGuid, Sequencer->GetFocusedTemplateID()))
	{
		const bool bValidObject = BoundObject.IsValid() ? BoundObject->IsA<AActor>() || BoundObject->IsA<UActorComponent>() : false;
		if (!bValidObject)
		{
			continue;
		}

		BoundObjects.Add(BoundObject.Get());
	}

	return BoundObjects;
}

void UAnimDetailsProxyManager::AddProxyToExternalSelectionIfValid(UAnimDetailsProxyBase* Proxy)
{
	if (Proxy)
	{
		check(Proxy->GetSequencerItem().IsValid() || Proxy->GetControlRig());
		ExternalSelection.Add(Proxy);
	}
}
