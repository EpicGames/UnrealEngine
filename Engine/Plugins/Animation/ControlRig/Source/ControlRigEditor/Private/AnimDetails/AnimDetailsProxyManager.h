// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsFilter.h"
#include "AnimDetailsNavigableWidgetRegistry.h"
#include "Containers/Ticker.h"
#include "EditorUndoClient.h"
#include "Engine/TimerHandle.h"
#include "TimerManager.h"

#include "AnimDetailsProxyManager.generated.h"

class FTrackInstancePropertyBindings;
class ISequencer;
class UControlRig;
class UAnimDetailsProxyBase;
class UAnimDetailsSelection;
class UMovieScene;
class UMovieScenePropertyTrack;
class UMovieSceneSection;
enum class ERigControlType : uint8;
struct FRigControlElement;
struct FRigElementKey;

/** 
 * Manages the instances of UAnimDetailsProxyBase for anim details.
 * 
 * This is a rewrite of what was previously UControlRigDetailPanelControlProxies in ControlRigEditor/Private/EditMode/ControlRigControlsProxy.h.
 */
UCLASS()
class UAnimDetailsProxyManager 
	: public UObject
	, public FSelfRegisteringEditorUndoClient
{
	GENERATED_BODY()

public:
	UAnimDetailsProxyManager();

	//~ Begin UObject interface
	virtual void BeginDestroy() override;
	//~ End UObject interface

	/** Suspends or resumes this manager. While the manager is suspended, its proxies are no longer updated. */
	void SetSuspended(const bool bEnabled);

	/** Returns the sequencer this proxy manager handles or nullptr if not currently assigned to a sequencer */
	TSharedPtr<ISequencer> GetSequencer() const;

	/** Updates proxies on the next tick. Ignored if the manager is suspended. */
	void RequestUpdateProxies();

	/** 
	 * Updates the proxy values on the next tick. Useful to let proxies know that values changed externally.
	 * Ignored if the manager is suspended. 
	 */
	void RequestUpdateProxyValues();

	/** Delegate broadcase when a displayed proxies changed */
	FSimpleMulticastDelegate& GetOnProxiesChanged() { return OnProxiesChangedDelegate; }

	/** Returns the proxies currently selected in anim outliner, control rig and sequencer. */
	const TArray<TObjectPtr<UAnimDetailsProxyBase>>& GetExternalSelection() const { return ExternalSelection; }

	/** Returns the anim details selection object */
	UAnimDetailsSelection* GetAnimDetailsSelection();

	/** Returns the anim details selection object, const */
	const UAnimDetailsSelection* GetAnimDetailsSelection() const;

	/** Returns the filter for anim details */
	UE::ControlRigEditor::FAnimDetailsFilter& GetAnimDetailsFilter() { return AnimDetailsFilter; }
	
	/** Returns the filter for anim details, const */
	const UE::ControlRigEditor::FAnimDetailsFilter& GetAnimDetailsFilter() const { return AnimDetailsFilter; }

	/** Returns the member name of the proxies member */
	static const FName GetProxiesMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyManager, Proxies); }

	/** Returns the navigable widget registry that belongs to this proxy manager. Useful for navigation */
	UE::ControlRigEditor::FAnimDetailsNavigableWidgetRegistry& GetNavigableWidgetRegistry() { return NavigableWidgetRegistry; }

private:
	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

	/** Clears the sequencer. This is useful to suspend this proxy manager. */
	void ClearSequencer();

	/** Updates the sequencer this manager uses. Returns true if a sequencer was set successfully */
	[[nodiscard]] bool UpdateSequencer();

	/** Binds to other objects' events to update on their changes */
	void SetupBingings();

	/** Releases bindings to other objects' events */
	void ReleaseBindings();

	/** Called when control rig shape actors were recreated  */
	void OnControlRigShapeActorsRecreated();

	/** Called when a control rig element was selected */
	void OnControlRigSelectionChanged(UControlRig* ControlRig, const FRigElementKey& RigElementKey, const bool bIsSelected);

	/** Called when the sequencer selection changed */
	void OnSequencerSelectionChanged(TArray<FGuid> ObjectGuids);

	/** Called when a control rig was added */
	void OnControlRigControlAdded(UControlRig* ControlRig, bool bIsAdded);

	/** Called from the editor when a blueprint object replacement has occurred. Useful to restore the control rig for proxies when it was replaced. */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Updates all proxies. */
	void ForceUpdateProxies();

	/** Updates the proxy values. */
	void ForceUpdateProxyValues();

	/** Removes any invalid proxy */
	void RemoveInvalidProxies();

	/** Creates or updates a control rig proxy. Will reuse and update sequencer proxies with their related control rig element. */
	[[nodiscard]] UAnimDetailsProxyBase* GetOrCreateControlRigProxy(UControlRig* ControlRig, FRigControlElement* ControlElement);

	/** Creates or updates a sequencer proxy. */
	[[nodiscard]] UAnimDetailsProxyBase* GetOrCreateSequencerProxy(UObject* BoundObject, UMovieScenePropertyTrack* PropertyTrack, const TSharedPtr<FTrackInstancePropertyBindings>& Binding);

	/** Creates a new anim details proxy */
	UAnimDetailsProxyBase* NewProxyFromType(UAnimDetailsProxyManager* Owner, ERigControlType ControlType, const TObjectPtr<UEnum>& InEnumPtr, const FName& ProxyName) const;

	/** Reevaluates constraints */
	void ReevaluateConstraints();

	/** Tries to get a supported control type that is used for the specified property track. */
	[[nodiscard]] bool TryGetControlTypeFromTrackType(const UMovieScenePropertyTrack& InPropertyTrack, ERigControlType& OutControlRigType) const;

	/** Returns a map of object guids with their sequencer property tracks */
	TMap<FGuid, TArray<UMovieScenePropertyTrack*>> GetPropertyTracks(const TSharedRef<ISequencer>& Sequencer, const TArray<FGuid>& ObjectGuids) const;

	/** Returns a an array of objects bound in sequencer which match provided object id */
	TArray<UObject*> GetBoundObjectsFromTrack(const TSharedRef<ISequencer>& Sequencer, const FGuid& ObjectGuid) const;

	/** Convenience to add a proxy to ExternalSelection, only if the proxy is valid */
	void AddProxyToExternalSelectionIfValid(UAnimDetailsProxyBase* Proxy);

	/** While true, the manager is not updated */
	bool bSuspended = true;

	/** The current proxies */
	UPROPERTY()
	TArray<TObjectPtr<UAnimDetailsProxyBase>> Proxies; 

	/** The proxies that are currently selected in anim outliner, control rig and sequencer */
	UPROPERTY()
	TArray<TObjectPtr<UAnimDetailsProxyBase>> ExternalSelection;

	/** The anim details selection */
	UPROPERTY()
	TObjectPtr<UAnimDetailsSelection> AnimDetailsSelection;

	/** The filter used for anim details */
	UE::ControlRigEditor::FAnimDetailsFilter AnimDetailsFilter;

	/** Registry for value widgets, useful for navigation */
	UE::ControlRigEditor::FAnimDetailsNavigableWidgetRegistry NavigableWidgetRegistry;

	/** The sequencer object this manager currently handles */
	TWeakPtr<ISequencer> WeakSequencer;

	/** Delegate broadcase when the selection changed */
	FSimpleMulticastDelegate OnProxiesChangedDelegate;

	/** Ticker handle for the RequestUpdateProxies member function */
	FTSTicker::FDelegateHandle RequestUpdateProxiesTickerHandle;

	/** Timer handle for the RequestUpdateProxyValues member function */
	FTimerHandle RequestUpdateProxyValuesTimerHandle;
};
