// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsSelection.h"

#include "Algo/AnyOf.h"
#include "Algo/Count.h" 
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "AnimDetailsProxyManager.h"
#include "CurveEditor.h"
#include "ISequencer.h"
#include "Misc/Optional.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "PropertyHandle.h"
#include "Sequencer/AnimLayers/AnimLayers.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tree/SCurveEditorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsSelection)

namespace UE::ControlRigEditor::SelectionUtils
{
	/** Utility to get the control element channel name if it matches curve name fragments */
	static TOptional<FString> GetControlElementChannelName(const TArray<FString>& CurveNameFragments, const FRigControlElement* ControlElement)
	{
		//if single channel expect one item and the name will match
		if (ControlElement->Settings.ControlType == ERigControlType::ScaleFloat ||
			ControlElement->Settings.ControlType == ERigControlType::Float ||
			ControlElement->Settings.ControlType == ERigControlType::Bool ||
			ControlElement->Settings.ControlType == ERigControlType::Integer)
		{
			if (CurveNameFragments[0] == ControlElement->GetKey().Name)
			{
				if (ControlElement->Settings.ControlType == ERigControlType::ScaleFloat ||
					ControlElement->Settings.ControlType == ERigControlType::Float)
				{
					return FString(TEXT("Float"));
				}
				if (ControlElement->Settings.ControlType == ERigControlType::Bool)
				{
					return FString(TEXT("Bool"));
				}
				if (ControlElement->Settings.ControlType == ERigControlType::Integer)
				{
					return FString(TEXT("Integer"));
				}
			}
		}
		else if (CurveNameFragments.Num() > 1)
		{
			if (CurveNameFragments[0] == ControlElement->GetKey().Name)
			{
				if (CurveNameFragments.Num() == 3)
				{
					return CurveNameFragments[1] + "." + CurveNameFragments[2];
				}
				else if (CurveNameFragments.Num() == 2)
				{
					return CurveNameFragments[1];
				}
			}
		}
		return TOptional<FString>();
	}

	/** Returns the control rig sections selected in sequencer */
	void GetSelectedControlRigSections(UMovieSceneSection* InSection, const FName& ControlName, TArray<UMovieSceneSection*>& OutSections)
	{
		UMovieSceneTrack* OwnerTrack = InSection ? InSection->GetTypedOuter<UMovieSceneTrack>() : nullptr;
		if (OwnerTrack)
		{
			TArray<UMovieSceneSection*> TrackSections = OwnerTrack->GetAllSections();
			for (UMovieSceneSection* Section : TrackSections)
			{
				if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
				{
					if ((Section->IsActive() || Section == InSection) &&
						CRSection->GetControlNameMask(ControlName))
					{
						OutSections.Add(Section);
					}
				}
			}
		}
	}
}

FAnimDetailsSelectionPropertyData::FAnimDetailsSelectionPropertyData(const FName& InPropertyName)
	: PropertyName(InPropertyName)
{}

void FAnimDetailsSelectionPropertyData::AddProxy(UAnimDetailsProxyBase* Proxy)
{
	if (Proxy)
	{
		WeakProxies.Add(Proxy);
	}
}

UAnimDetailsSelection::UAnimDetailsSelection()
{
	if (UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>())
	{
		ProxyManager->GetOnProxiesChanged().AddUObject(this, &UAnimDetailsSelection::OnProxiesChanged);
		ProxyManager->GetAnimDetailsFilter().GetOnFilterChanged().AddUObject(this, &UAnimDetailsSelection::OnFilterChanged);
	}
}

void UAnimDetailsSelection::SelectPropertyInProxies(const TArray<UAnimDetailsProxyBase*>& Proxies, const FName& PropertyName, const EAnimDetailsSelectionType SelectionType)
{
	if (bIsChangingSelection)
	{
		return;
	}
	const TGuardValue<bool> SelectionGuard(bIsChangingSelection, true);

	// Find the selected property
	const FName PropertyID = MakeCommonPropertyID(Proxies, PropertyName);
	FAnimDetailsSelectionPropertyData* PropertyDataPtr = PropertyIDToPropertyDataMap.Find(PropertyID);
	if (Proxies.IsEmpty() || !PropertyDataPtr)
	{
		return;
	}

	// Select as per selection type, ignoring visibility
	if (SelectionType == EAnimDetailsSelectionType::Select ||
		SelectionType == EAnimDetailsSelectionType::Toggle)
	{
		const bool bSelect = !PropertyDataPtr->IsSelected();
		PropertyDataPtr->SetSelected(bSelect);

		AnchorPropertyID = PropertyID;
	}
	else if (SelectionType == EAnimDetailsSelectionType::SelectRange)
	{
		const FAnimDetailsSelectionPropertyData* AnchorPropertyDataPtr = PropertyIDToPropertyDataMap.Find(AnchorPropertyID);
		const bool bAnchorSelected = AnchorPropertyDataPtr && AnchorPropertyDataPtr->IsSelected();

		if (bAnchorSelected)
		{
			// Range select
			bool bSelect = false;
			for (TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
			{
				if (PropertyIDToPropertyDataPair.Key == PropertyID ||
					PropertyIDToPropertyDataPair.Key == AnchorPropertyID)
				{
					PropertyIDToPropertyDataPair.Value.SetSelected(true);
					bSelect = PropertyID == AnchorPropertyID ? !PropertyIDToPropertyDataPair.Value.IsSelected() : !bSelect;
				}
				else
				{
					PropertyIDToPropertyDataPair.Value.SetSelected(bSelect);
				}
			}
		}
		else
		{
			// Range unselect
			bool bUnselect = false;
			for (TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
			{
				if (PropertyIDToPropertyDataPair.Key == PropertyID ||
					PropertyIDToPropertyDataPair.Key == AnchorPropertyID)
				{
					PropertyIDToPropertyDataPair.Value.SetSelected(false);

					bUnselect = PropertyID == AnchorPropertyID ? false : !bUnselect;
				}
				else if (bUnselect)
				{
					PropertyIDToPropertyDataPair.Value.SetSelected(false);
				}
			}
		}
	}
	else
	{
		ensureMsgf(0, TEXT("Unhandled enum value"));
	}

	// Unselect any hidden property
	for (TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
	{
		if (!PropertyIDToPropertyDataPair.Value.IsVisible())
		{
			PropertyIDToPropertyDataPair.Value.SetSelected(false);
		}
	}

	RequestPropagateSelectionToCurveEditor();
}

void UAnimDetailsSelection::ClearSelection()
{
	for (TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
	{
		PropertyIDToPropertyDataPair.Value.SetSelected(false);
	}

	RequestPropagateSelectionToCurveEditor();
}

bool UAnimDetailsSelection::IsPropertySelected(const UAnimDetailsProxyBase* Proxy, const FName& PropertyName) const
{
	if (Proxy)
	{
		const FName PropertyID = Proxy->GetPropertyID(PropertyName);
		const FAnimDetailsSelectionPropertyData* PropertyDataPtr = PropertyIDToPropertyDataMap.Find(PropertyID);

		return PropertyDataPtr ? PropertyDataPtr->IsVisible() && PropertyDataPtr->IsSelected() : false;
	}
		
	return false;
}

bool UAnimDetailsSelection::IsPropertySelected(const TSharedRef<IPropertyHandle>& PropertyHandle) const
{
	const FProperty* Property = PropertyHandle->IsValidHandle() ? PropertyHandle->GetProperty() : nullptr;
	if (Property)
	{
		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		// Consider all selected when one is selected
		return Algo::AnyOf(OuterObjects, [Property, this](const UObject* Object)
			{
				if (const UAnimDetailsProxyBase* Proxy = Cast<const UAnimDetailsProxyBase>(Object))
				{
					const FName PropertyID = Proxy->GetPropertyID(Property->GetFName());
					const FAnimDetailsSelectionPropertyData* PropertyDataPtr = PropertyIDToPropertyDataMap.Find(PropertyID);

					return PropertyDataPtr ? PropertyDataPtr->IsVisible() && PropertyDataPtr->IsSelected() : false;
				}

				return false;
			});
	}

	return false;
}

int32 UAnimDetailsSelection::GetNumSelectedProperties() const
{
	return Algo::CountIf(PropertyIDToPropertyDataMap,
		[](const TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair)
		{
			return 
				PropertyIDToPropertyDataPair.Value.IsVisible() && 
				PropertyIDToPropertyDataPair.Value.IsSelected();
		});
}

bool UAnimDetailsSelection::IsControlElementSelected(const UControlRig* ControlRig, const FRigControlElement* ControlElement) const
{
	return Algo::AnyOf(PropertyIDToPropertyDataMap, 
		[&ControlRig, &ControlElement](const TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair)
		{
			if (!PropertyIDToPropertyDataPair.Value.IsVisible() ||
				!PropertyIDToPropertyDataPair.Value.IsSelected())
			{
				return false;
			}

			return Algo::AnyOf(PropertyIDToPropertyDataPair.Value.GetProxiesBeingEdited(),
				[&ControlRig, &ControlElement](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
				{
					return
						WeakProxy.IsValid() &&
						WeakProxy->GetControlRig() == ControlRig &&
						WeakProxy->GetControlElement() == ControlElement;
				});
		});
}

TArray<UAnimDetailsProxyBase*> UAnimDetailsSelection::GetSelectedProxies() const
{
	TArray<UAnimDetailsProxyBase*> SelectedProxies;
	for (const TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
	{
		if (!PropertyIDToPropertyDataPair.Value.IsVisible() ||
			!PropertyIDToPropertyDataPair.Value.IsSelected())
		{
			continue;
		}

		for (const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy : PropertyIDToPropertyDataPair.Value.GetProxiesBeingEdited())
		{
			if (WeakProxy.IsValid())
			{
				SelectedProxies.Add(WeakProxy.Get());
			}
		}
	}

	return SelectedProxies;
}

void UAnimDetailsSelection::OnProxiesChanged()
{
	UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>();
	if (!ProxyManager)
	{
		return;
	}

	// Listen to anim layer selection changes
	const TSharedPtr<ISequencer> Sequencer = ProxyManager->GetSequencer();
	if (Sequencer.IsValid())
	{
		UAnimLayers* AnimLayers = UAnimLayers::HasAnimLayers(Sequencer.Get()) ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;
		if (AnimLayers != WeakAnimLayers)
		{
			if (WeakAnimLayers.IsValid())
			{
				WeakAnimLayers->GetOnSelectionChanged().RemoveAll(this);
			}

			if (AnimLayers)
			{
				AnimLayers->GetOnSelectionChanged().AddUObject(this, &UAnimDetailsSelection::OnAnimLayersSelected);
			}
		}
	}
	else if (WeakAnimLayers.IsValid())
	{
		WeakAnimLayers->GetOnSelectionChanged().RemoveAll(this);
	}

	PropertyIDToPropertyDataMap.Reset();

	for (const TObjectPtr<UAnimDetailsProxyBase>& Proxy : ProxyManager->GetExternalSelection())
	{
		if (!Proxy)
		{
			continue;
		}

		for (const FName& PropertyName : Proxy->GetPropertyNames())
		{
			const FName PropertyID = Proxy->GetPropertyID(PropertyName);

			FAnimDetailsSelectionPropertyData& PropertyData = PropertyIDToPropertyDataMap.FindOrAdd(PropertyID, FAnimDetailsSelectionPropertyData(PropertyName));
			PropertyData.AddProxy(Proxy);

			AnchorPropertyID = NAME_None;
		}
	}

}

void UAnimDetailsSelection::OnAnimLayersSelected()
{
	if (OnAnimLayersSelectedTimerHandle.IsValid())
	{
		return;
	}

	RestorePropertyIDToPropertyDataMap = PropertyIDToPropertyDataMap;

	bIsAnimLayersChangingSelection = true;

	const TWeakObjectPtr<UAnimDetailsSelection> WeakThis = this;
	OnAnimLayersSelectedTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([WeakThis, this]()
		{
			if (WeakThis.IsValid())
			{
				bIsAnimLayersChangingSelection = false;

				// Restore selection from RestorePropertyIDToPropertyDataMap 
				for (const TTuple<FName, FAnimDetailsSelectionPropertyData>& RestoreProperyNameToIndexPair : RestorePropertyIDToPropertyDataMap)
				{
					if (FAnimDetailsSelectionPropertyData* NewPropertyDataPtr = PropertyIDToPropertyDataMap.Find(RestoreProperyNameToIndexPair.Key))
					{
						NewPropertyDataPtr->SetSelected(RestoreProperyNameToIndexPair.Value.IsSelected());
					}
				}

				PropagateSelectionToCurveEditor();

				OnAnimLayersSelectedTimerHandle.Invalidate();
			}
		}));
}

void UAnimDetailsSelection::OnFilterChanged()
{
	UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>();
	if (!ProxyManager)
	{
		return;
	}
	const UE::ControlRigEditor::FAnimDetailsFilter& Filter = ProxyManager->GetAnimDetailsFilter();

	for (TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
	{
		const FName& PropertyName = PropertyIDToPropertyDataPair.Value.GetProperyName();
		const TArray<TWeakObjectPtr<UAnimDetailsProxyBase>>& WeakProxies = PropertyIDToPropertyDataPair.Value.GetProxiesBeingEdited();
		
		const bool bVisible = Algo::AnyOf(WeakProxies,
			[&Filter, &PropertyName](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
			{
				return 
					WeakProxy.IsValid() &&
					Filter.ContainsProperty(*WeakProxy.Get(), PropertyName);
			});

		PropertyIDToPropertyDataPair.Value.SetVisible(bVisible);
	}
}

void UAnimDetailsSelection::RequestPropagateSelectionToCurveEditor()
{
	if (ensureMsgf(IsInGameThread(), TEXT("Anim Details selection can only be updated in game thread. Ignoring call")) &&
		!PropagateSelectionToCurveEditorTimerHandle.IsValid())
	{
		PropagateSelectionToCurveEditorTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(
			FTimerDelegate::CreateUObject(this, &UAnimDetailsSelection::PropagateSelectionToCurveEditor));
	}
}

void UAnimDetailsSelection::PropagateSelectionToCurveEditor()
{
	if (!ensureMsgf(IsInGameThread(), TEXT("Anim Details selection can only be updated in game thread. Ignoring call")))
	{
		return;
	}

	using namespace UE::Sequencer;
	using namespace UE::ControlRigEditor;

	PropagateSelectionToCurveEditorTimerHandle.Invalidate();

	UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>();
	const TSharedPtr<ISequencer> Sequencer = ProxyManager ? ProxyManager->GetSequencer() : nullptr;
	const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer.IsValid() ? Sequencer->GetViewModel() : nullptr;
	const FCurveEditorExtension* CurveEditorExtension = SequencerViewModel.IsValid() ? SequencerViewModel->CastDynamic<FCurveEditorExtension>() : nullptr;	
	const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorExtension ? CurveEditorExtension->GetCurveEditor() : nullptr;
	const TSharedPtr<SCurveEditorTree>  CurveEditorTreeView = CurveEditorExtension ? CurveEditorExtension->GetCurveEditorTreeView() : nullptr;
	const TSharedPtr<FOutlinerViewModel> OutlinerViewModel = SequencerViewModel ? SequencerViewModel->GetOutliner() : nullptr;
	const FViewModelPtr OutlinerRootItem = OutlinerViewModel.IsValid() ? OutlinerViewModel->GetRootItem() : nullptr;
	if (!ProxyManager ||
		!Sequencer.IsValid() ||
		!SequencerViewModel.IsValid() ||
		!CurveEditorExtension ||
		!CurveEditor.IsValid() ||
		!CurveEditorTreeView.IsValid() ||
		!OutlinerViewModel.IsValid() ||
		!OutlinerRootItem.IsValid())
	{
		return;
	}

	// SetTreeSelection causes FSequencerSelectionSyncer::OnCurveEditorSelectionChanged to propagate the selection to Sequencer.
	// That invokes UAnimDetailsProxyManager::OnSequencerSelectionChanged, which in turn invokes us recursively. Prevent that.
	const TGuardValue<bool> Guard(bIsAnimLayersChangingSelection, true);
	
	// The user is doing an explicit selection on channels, so we want to clear any previous implicity child selection
	CurveEditor->SetTreeSelection({});

	// Cache the curve editor data to avoid redundant lookups
	struct FOutlinerExtensionData
	{
		UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = nullptr;
		TViewModelPtr<ITrackExtension> TrackModel;
		TViewModelPtr<FChannelGroupOutlinerModel> ChannelModel;
		TViewModelPtr<ICurveEditorTreeItemExtension> CurveEditorItem;

		TArray<FString> IdentifierFragments;
	};

	TArray<FOutlinerExtensionData> OutlinerExtensionDatas;
	TParentFirstChildIterator<IOutlinerExtension> CachedOutlinerExtenstionIt = OutlinerRootItem->GetDescendantsOfType<IOutlinerExtension>();
	for (TParentFirstChildIterator<IOutlinerExtension> OutlinerExtenstionIt = CachedOutlinerExtenstionIt; OutlinerExtenstionIt; ++OutlinerExtenstionIt)
	{
		if (const TViewModelPtr<ITrackExtension> TrackModel = OutlinerExtenstionIt.GetCurrentItem()->FindAncestorOfType<ITrackExtension>())
		{
			FOutlinerExtensionData ExtensionData;
			ExtensionData.TrackModel = TrackModel;
			ExtensionData.ChannelModel = CastViewModel<FChannelGroupOutlinerModel>(OutlinerExtenstionIt.GetCurrentItem());
			ExtensionData.CurveEditorItem = CastViewModel<FChannelGroupOutlinerModel>(OutlinerExtenstionIt.GetCurrentItem());

			if (!ExtensionData.TrackModel.IsValid() || 
				!ExtensionData.ChannelModel.IsValid() || 
				!ExtensionData.CurveEditorItem.IsValid())
			{
				continue;
			}

			const FName ID = OutlinerExtenstionIt->GetIdentifier();
			const FString Name = ID.ToString();
			Name.ParseIntoArray(ExtensionData.IdentifierFragments, TEXT("."));

			OutlinerExtensionDatas.Add(ExtensionData);
		}
	}

	// Propagate selection
	const FAnimDetailsFilter& Filter = ProxyManager->GetAnimDetailsFilter();
	const bool bAnimDetailsHasSelection = GetNumSelectedProperties() > 0;
	for (const TTuple<FName, FAnimDetailsSelectionPropertyData>& PropertyIDToPropertyDataPair : PropertyIDToPropertyDataMap)
	{
		const FAnimDetailsSelectionPropertyData& PropertyData = PropertyIDToPropertyDataPair.Value;
		for (const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy : PropertyIDToPropertyDataPair.Value.GetProxiesBeingEdited())
		{
			const UAnimDetailsProxyBase* Proxy = WeakProxy.Get();
			if (!Proxy)
			{
				continue;
			}

			const EControlRigContextChannelToKey ChannelToKeyContext = Proxy->GetChannelToKeyFromPropertyName(PropertyData.GetProperyName());
			for (const FOutlinerExtensionData& OutlinerExtensionData : OutlinerExtensionDatas)
			{
				if (UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(OutlinerExtensionData.TrackModel->GetTrack()))
				{
					if (ControlRigParameterTrack->GetControlRig() != Proxy->GetControlRig() || 
						ControlRigParameterTrack->GetAllSections().IsEmpty())
					{
						continue;
					}

					const FRigControlElement* ControlElement = Proxy->GetControlElement();
					if (!ControlElement)
					{
						continue;
					}

					const TArray<FString>& IdentifierFragments = OutlinerExtensionData.IdentifierFragments;

					const TOptional<FString> ChannelName = SelectionUtils::GetControlElementChannelName(IdentifierFragments, ControlElement);
					if (!ChannelName.IsSet())
					{
						continue;
					}
				
					const EControlRigContextChannelToKey ChannelToKeyFromCurve = Proxy->GetChannelToKeyFromChannelName(ChannelName.GetValue());
					const TViewModelPtr<ICurveEditorTreeItemExtension>& CurveEditorItem = OutlinerExtensionData.CurveEditorItem;
					if (ChannelToKeyContext != ChannelToKeyFromCurve || !CurveEditorItem.IsValid())
					{
						continue;
					}

					const FCurveEditorTreeItemID CurveEditorTreeItem = CurveEditorItem->GetCurveEditorItemID();
					if (!CurveEditorTreeItem.IsValid())
					{
						continue;
					}

					UMovieSceneSection* BaseSection = [ControlRigParameterTrack, ControlElement]()
						{
							UMovieSceneSection* Section = ControlRigParameterTrack->GetSectionToKey(ControlElement->GetFName());
							if (Section)
							{
								return Section;
							}

							const TArray<UMovieSceneSection*> AllSections = ControlRigParameterTrack->GetAllSections();
							return AllSections.IsEmpty() ? nullptr : AllSections[0];
						}();

					TArray<UMovieSceneSection*> ExistingSections;
					SelectionUtils::GetSelectedControlRigSections(BaseSection, ControlElement->GetFName(), ExistingSections);

					// GetSectionToKey already captures per-control routing set by ActivateLayerForKeying.
					// Delegate section ownership to the model rather than re-deriving it here.
					UMovieSceneSection* SectionToKey = nullptr;
					if (ExistingSections.Contains(BaseSection))
					{
						SectionToKey = BaseSection;
					}
					else if (!ExistingSections.IsEmpty())
					{
						SectionToKey = ExistingSections[0];
					}

					if (!SectionToKey)
					{
						continue;
					}

					// If there's no section to key we also don't select it
					const TViewModelPtr<FChannelGroupOutlinerModel>& ChannelModel = OutlinerExtensionData.ChannelModel;
					if (!ChannelModel.IsValid() ||
						!ChannelModel->GetChannel(SectionToKey))
					{
						continue;
					}

					const bool bPropertyVisible = PropertyData.IsVisible();
					const bool bPropertySelected = PropertyData.IsSelected();

					if (bAnimDetailsHasSelection)
					{
						const bool bSelected = bAnimDetailsHasSelection ? bPropertyVisible && bPropertySelected : true;

						CurveEditorTreeView->SetItemSelection(CurveEditorTreeItem, bSelected);
					}
					else if (const TSharedPtr<FCategoryGroupModel>& CategoryGroupModel = CastViewModel<FCategoryGroupModel>(ChannelModel->GetParent()))
					{
						// If there is no selection in anim details, select the parent category instead
						const FCurveEditorTreeItemID ParentCurveEditorTreeItem = CategoryGroupModel->GetCurveEditorItemID();
						CurveEditorTreeView->SetItemSelection(ParentCurveEditorTreeItem, true);
					}
				}
				else if (OutlinerExtensionData.TrackModel->GetTrack() == Proxy->GetSequencerItem().GetMovieSceneTrack())
				{
					if (OutlinerExtensionData.TrackModel->GetTrack()->GetAllSections().IsEmpty())
					{
						continue;
					}

					const UMovieSceneSection* SectionToKey = OutlinerExtensionData.TrackModel->GetTrack()->GetSectionToKey() ? 
						OutlinerExtensionData.TrackModel->GetTrack()->GetSectionToKey() : 
						OutlinerExtensionData.TrackModel->GetTrack()->GetAllSections()[0];
						
					// If there's no section to key we also don't select it
					const TViewModelPtr<FChannelGroupOutlinerModel>& ChannelModel = OutlinerExtensionData.ChannelModel;
					if (!ChannelModel.IsValid() || !ChannelModel->GetChannel(SectionToKey))
					{
						continue;
					}

					const TArray<FString>& IdentifierFragments = OutlinerExtensionData.IdentifierFragments;

					FString ChannelName;
					if (IdentifierFragments.Num() == 2)
					{
						ChannelName = IdentifierFragments[0] + "." + IdentifierFragments[1];
					}
					else if (IdentifierFragments.Num() == 1)
					{
						ChannelName = IdentifierFragments[0];
					}

					const EControlRigContextChannelToKey ChannelToKeyFromCurve = Proxy->GetChannelToKeyFromChannelName(ChannelName);
					const TViewModelPtr<ICurveEditorTreeItemExtension>& CurveEditorItem = OutlinerExtensionData.CurveEditorItem;
					if (ChannelToKeyContext != ChannelToKeyFromCurve || !CurveEditorItem.IsValid())
					{
						continue;
					}

					const FCurveEditorTreeItemID CurveEditorTreeItem = CurveEditorItem->GetCurveEditorItemID();
					if (CurveEditorTreeItem.IsValid())
					{
						const bool bPropertyVisible = PropertyData.IsVisible();
						const bool bPropertySelected = PropertyData.IsSelected();

						if (bAnimDetailsHasSelection)
						{
							const bool bSelected = bAnimDetailsHasSelection ? bPropertyVisible && bPropertySelected : true;

							CurveEditorTreeView->SetItemSelection(CurveEditorTreeItem, bSelected);
						}
						else if (const TSharedPtr<FCategoryGroupModel>& CategoryGroupModel = CastViewModel<FCategoryGroupModel>(ChannelModel->GetParent()))
						{
							// If there is no selection in anim details, select the parent category instead
							const FCurveEditorTreeItemID ParentCurveEditorTreeItem = CategoryGroupModel->GetCurveEditorItemID();
							CurveEditorTreeView->SetItemSelection(ParentCurveEditorTreeItem, true);
						}
					}
				}
			}
		}
	}
}

FName UAnimDetailsSelection::MakeCommonPropertyID(const TArray<UAnimDetailsProxyBase*>& Proxies, const FName& PropertyName) const
{
	if (!ensureMsgf(!Proxies.IsEmpty(), TEXT("Unexpected trying to get the common property id for an array of zero anim details proxies")))
	{
		return NAME_None;
	}

	const FName FirstPropertyID = Proxies[0] ? Proxies[0]->GetPropertyID(PropertyName) : NAME_None;
	const bool bValidProxies = Algo::AllOf(Proxies,
		[&PropertyName, &FirstPropertyID, this](const UAnimDetailsProxyBase* Proxy)
		{
			if (ensureMsgf(Proxy, TEXT("Unexpected trying to get the common property id for proxies, but invalid proxies were provided.")))
			{
				return Proxy->GetPropertyID(PropertyName) == FirstPropertyID;
			}

			return false;
		});

	ensureMsgf(bValidProxies, TEXT("Cannot find common property ID for proxies. Using first one instead"));

	// In any case return the first property ID - They're all equal if bValidProxies, else we fall back to the first one.
	return FirstPropertyID;
}
