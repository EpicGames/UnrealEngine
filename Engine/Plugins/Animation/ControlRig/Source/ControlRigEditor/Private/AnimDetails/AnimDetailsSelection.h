// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "Templates/UnrealTemplate.h"

#include "AnimDetailsSelection.generated.h"

class IPropertyHandle;
class UAnimLayers;
class UControlRig;
class UAnimDetailsProxyBase;
struct FRigControlElement;
namespace UE::Sequencer { class FOutlinerSelection; }

namespace UE::ControlRigEditor
{
	/** Describes the type of selection */
	enum class EAnimDetailsSelectionType : uint8
	{
		Select,
		Toggle,
		SelectRange,
	};
}

/** Struct to describe a single property in an anim details proxy */
USTRUCT()
struct FAnimDetailsSelectionPropertyData
{
	GENERATED_BODY()

	FAnimDetailsSelectionPropertyData() = default;
	FAnimDetailsSelectionPropertyData(const FName& InPropertyName);

	/** Adds a proxy to this property data */
	void AddProxy(UAnimDetailsProxyBase* Proxy);

	/** Returns true if this property is selected */
	bool IsSelected() const { return bIsSelected; }

	/** Sets if the property is selected */
	void SetSelected(bool bSelected) { bIsSelected = bSelected; }

	/** Returns true if this property is visible */
	bool IsVisible() const { return bIsVisible; }

	/** Sets if the property is visible */
	void SetVisible(bool bVisible) { bIsVisible = bVisible; }

	/** Returns the proxies being edited */
	const TArray<TWeakObjectPtr<UAnimDetailsProxyBase>>& GetProxiesBeingEdited() const { return WeakProxies; }

	/** 
	 * Returns the property name that relates to this data.
	 * Note, this is not a unique identifier, other property datas may use the same property name from different property paths. 
	 */
	const FName& GetProperyName() const { return PropertyName; }

private:
	/** The currently selected proxies that own the property. Maybe be multiple if proxies are multi edited on a details row */
	UPROPERTY()
	TArray<TWeakObjectPtr<UAnimDetailsProxyBase>> WeakProxies;

	/** If true, the property is selected */
	UPROPERTY()
	bool bIsSelected = false;

	/** If true, the property is visible */
	bool bIsVisible = true;

	/** The property name for this data */
	UPROPERTY()
	FName PropertyName;
};

/** 
 * The selection in Anim Details.
 *
 * Note, the selection does not necessarily correspond to the selection in Anim Outliner or Sequencer/Curve Editor.
 * Use UAnimDetailsProxyManager::GetExternalSelection to get the external selection.
 */
UCLASS()
class UAnimDetailsSelection
	: public UObject
{
	GENERATED_BODY()

	using EAnimDetailsSelectionType = UE::ControlRigEditor::EAnimDetailsSelectionType;

public:
	UAnimDetailsSelection();

	/** Selects the property. Depending on selection type other properties may be selected. Note, all proxies need to share the same detail row ID (ensured). */
	void SelectPropertyInProxies(const TArray<UAnimDetailsProxyBase*>& Proxies, const FName& PropertyName, const EAnimDetailsSelectionType SelectionType);
	
	/** Clears the selection */
	void ClearSelection();

	/** Returns true if the property is selected. Note, hidden properties are never considered being selected. */
	bool IsPropertySelected(const UAnimDetailsProxyBase* Proxy, const FName& PropertyName) const;

	/** Returns true if the property is selected. Note, hidden properties are never considered being selected. */
	bool IsPropertySelected(const TSharedRef<IPropertyHandle>& PropertyHandle) const;

	/** Returns the number of selected properties */
	int32 GetNumSelectedProperties() const;

	/** Returns true if the control element is selected. Note, hidden properties are never considered being selected. */
	bool IsControlElementSelected(const UControlRig* ControlRig, const FRigControlElement* ControlElement) const;

	/** Returns the currently selected proxies */
	TArray<UAnimDetailsProxyBase*> GetSelectedProxies() const;

	/** Returns true while anim layers is changing selection. Useful to avoid updating anim details to override its selection */
	bool IsAnimLayersChangingSelection() const { return bIsAnimLayersChangingSelection; }

private:
	/** Called when proxies changed in the proxy manager */
	void OnProxiesChanged();

	/** Called when anim layers were selected */
	void OnAnimLayersSelected();

	/** Called when the filter changed */
	void OnFilterChanged();

	/** Propagates the selection to curve editor on the next tick */
	void RequestPropagateSelectionToCurveEditor();
		
	/** Propagates the selection to curve editor */
	void PropagateSelectionToCurveEditor();

	/** 
	 * Creates a property ID common to the proxies. Note the proxies are expected to share the same detail row ID (ensured).
	 * This property ID does not relate to any other engine implementation, it is specific to this selection class.
	 */
	FName MakeCommonPropertyID(const TArray<UAnimDetailsProxyBase*>& Proxies, const FName& PropertyName) const;

	/** A property ID with its property data. */
	UPROPERTY()
	TMap<FName, FAnimDetailsSelectionPropertyData> PropertyIDToPropertyDataMap;

	/** When updating, restores property data from this map and resets it */
	TMap<FName, FAnimDetailsSelectionPropertyData> RestorePropertyIDToPropertyDataMap;

	/** The anchor for selection when shift-multiselecting or name none if there is no anchor */
	FName AnchorPropertyID;

	/** True while changing selection */
	bool bIsChangingSelection = false;

	/** True while anim layers is changing selection. Useful to override its selection */
	bool bIsAnimLayersChangingSelection = false;

	/** The current anim layers */
	TWeakObjectPtr<UAnimLayers> WeakAnimLayers;

	/** Timer handle to propagate the selection to the curve editor */
	FTimerHandle PropagateSelectionToCurveEditorTimerHandle;

	/** Timer handle to used when anim layers were selected */
	FTimerHandle OnAnimLayersSelectedTimerHandle;
};
