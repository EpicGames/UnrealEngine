// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsSettings.generated.h"

/** Settings for Anim Details */
UCLASS(config = EditorPerProjectUserSettings)
class UAnimDetailsSettings
	: public UObject
{
	GENERATED_BODY()

public:	
	/** Expand transform properties in anim details */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bExpandTransformProperties = true;

	/** The number of fractional digits displayed in anim details */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	uint8 NumFractionalDigits = 2;

	/** If true, selects a range when the left mouse button is down */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bLMBSelectsRange = true;
};
