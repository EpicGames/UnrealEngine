// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsOverrideDetails.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "AnimDetails/Widgets/SAnimDetailsPropertySelectionBorder.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "IDetailChildrenBuilder.h"
#include "ControlRigShapeNameList.h"

#define LOCTEXT_NAMESPACE "AnimDetailsProxyDetails"

namespace UE::ControlRigEditor
{
	FAnimDetailsOverrideDetails::FAnimDetailsOverrideDetails()
		: ControlRigShapeNameList(MakeShared<FControlRigShapeNameList>())
	{
	}

	void FAnimDetailsOverrideDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		HeaderRow.NameContent()
		.MaxDesiredWidth(30)
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
	}
	
	void FAnimDetailsOverrideDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized = StructCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
		const UControlRig* ControlRig = nullptr;
		for(const TWeakObjectPtr<UObject>& ObjectBeingCustomized : ObjectsBeingCustomized)
		{
			if(const UAnimDetailsProxyBase* ControlsProxy = Cast<UAnimDetailsProxyBase>(ObjectBeingCustomized.Get()))
			{
				ControlRig = ControlsProxy->GetControlRig();
				if(ControlRig)
				{
					break;
				}
			}
		}
		if(ControlRig == nullptr)
		{
			return;
		}

		ControlRigShapeNameList->GenerateShapeLibraryList(ControlRig);

		if(ControlRigShapeNameList->ShapeNameList.IsEmpty())
		{
			return;
		}

		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigUnit_HierarchyAddControl_ShapeSettings, Color)).ToSharedRef());

		TSharedPtr<IPropertyHandle> Property = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigUnit_HierarchyAddControl_ShapeSettings, Name));
		if(Property.IsValid())
		{
			ControlRigShapeNameList->CreateShapeLibraryListWidget(StructBuilder, Property);
		}

		StructBuilder.AddProperty(StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigUnit_HierarchyAddControl_ShapeSettings, bVisible)).ToSharedRef());

		if(TSharedPtr<IPropertyHandle> TransformHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigUnit_HierarchyAddControl_ShapeSettings, Transform)))
		{
			StructBuilder.AddProperty(TransformHandle->GetChildHandle(TEXT("Translation")).ToSharedRef());
			StructBuilder.AddProperty(TransformHandle->GetChildHandle(TEXT("Rotation")).ToSharedRef());
			StructBuilder.AddProperty(TransformHandle->GetChildHandle(TEXT("Scale3D")).ToSharedRef());
		}
	}
	
} // end namespace UE::ControlRigEditor

#undef LOCTEXT_NAMESPACE
