// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SRigVMGraphPinNameList.h"

namespace UE::ControlRigEditor
{
	struct FControlRigShapeNameList;

	/** Details customization for Control Shape settings */
	class FAnimDetailsOverrideDetails : public IPropertyTypeCustomization
	{
	public:
		static TSharedRef<class IPropertyTypeCustomization> MakeInstance()
		{
			return MakeShareable(new FAnimDetailsOverrideDetails());
		}

		FAnimDetailsOverrideDetails();

		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	private:
		TSharedPtr<FControlRigShapeNameList> ControlRigShapeNameList;
	};
}
