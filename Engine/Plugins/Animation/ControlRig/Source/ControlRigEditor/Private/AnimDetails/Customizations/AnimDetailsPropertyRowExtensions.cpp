// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsPropertyRowExtensions.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "DetailRowMenuContext.h"
#include "IDetailKeyframeHandler.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AnimDetailsPropertyRowExtensions"

namespace UE::ControlRigEditor
{
	namespace AnimDetailsPropertyRowExtensionsDetails
	{
		constexpr const TCHAR* PropertyEditorModuleName = TEXT("PropertyEditor");
	}

	FAnimDetailsPropertyRowExtensions& FAnimDetailsPropertyRowExtensions::Get()
	{
		static FAnimDetailsPropertyRowExtensions Instance;
		return Instance;
	}

	FAnimDetailsPropertyRowExtensions::~FAnimDetailsPropertyRowExtensions()
	{
		UnregisterRowExtensions();
	}

	void FAnimDetailsPropertyRowExtensions::RegisterRowExtensions()
	{
		++RegistrationCount;
		if (RegistrationCount > 1)
		{
			return;
		}

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(AnimDetailsPropertyRowExtensionsDetails::PropertyEditorModuleName);
		FOnGenerateGlobalRowExtension& RowExtensionDelegate = PropertyEditorModule.GetGlobalRowExtensionDelegate();

		RowExtensionHandle = RowExtensionDelegate.AddStatic(&CreatePropertyRowExtension);
	}

	void FAnimDetailsPropertyRowExtensions::UnregisterRowExtensions()
	{
		if (RegistrationCount == 0)
		{
			return;
		}

		--RegistrationCount;
		if (RegistrationCount > 0)
		{
			return;
		}

		if (RowExtensionHandle.IsValid() && 
			FModuleManager::Get().IsModuleLoaded(AnimDetailsPropertyRowExtensionsDetails::PropertyEditorModuleName))
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(AnimDetailsPropertyRowExtensionsDetails::PropertyEditorModuleName);
			PropertyEditorModule.GetGlobalRowExtensionDelegate().Remove(RowExtensionHandle);
			RowExtensionHandle.Reset();
		}
	}

	void FAnimDetailsPropertyRowExtensions::CreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions)
	{
		if (!InArgs.Property && 
			!InArgs.PropertyHandle.IsValid())
		{
			return;
		}

		UToolMenus* ToolMenus = UToolMenus::Get();
		UToolMenu* ContextMenu = ToolMenus ? ToolMenus->FindMenu(UE::PropertyEditor::RowContextMenuName) : nullptr;

		if (ToolMenus &&
			ContextMenu)
		{
			static const TCHAR* DetailViewRowExtensionName = TEXT("AnimDetailsRowExtensionContextSection");
			if (ContextMenu->ContainsSection(DetailViewRowExtensionName))
			{
				return;
			}

			ContextMenu->AddDynamicSection(
				DetailViewRowExtensionName,
				FNewToolMenuDelegate::CreateStatic(&ExtendContextMenu)
			);
		}
	}

	void FAnimDetailsPropertyRowExtensions::ExtendContextMenu(UToolMenu* InToolMenu)
	{
		UDetailRowMenuContext* RowMenuContext = InToolMenu->FindContext<UDetailRowMenuContext>();

		if (!RowMenuContext)
		{
			return;
		}

		TSharedPtr<IPropertyHandle> PropertyHandle;

		for (const TSharedPtr<IPropertyHandle>& ContextPropertyHandle : RowMenuContext->PropertyHandles)
		{
			if (ContextPropertyHandle.IsValid())
			{
				PropertyHandle = ContextPropertyHandle;
				break;
			}
		}

		if (!PropertyHandle.IsValid())
		{
			return;
		}

		if (TSharedPtr<IDetailsView> DetailsView = RowMenuContext->DetailsView.Pin())
		{
			if (!DetailsView->IsPropertyEditingEnabled())
			{
				return;
			}
		}

		if (PropertyHandle->IsEditConst() || !PropertyHandle->IsEditable())
		{
			return;
		}

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.IsEmpty())
		{
			return;
		}
		
		UAnimDetailsProxyManager* ProxyManager = OuterObjects[0] ? OuterObjects[0]->GetTypedOuter<UAnimDetailsProxyManager>() : nullptr;
		UAnimDetailsSelection* AnimDetailsSelection = ProxyManager ? ProxyManager->GetAnimDetailsSelection() : nullptr;
		if (!ProxyManager ||
			!AnimDetailsSelection ||
			!AnimDetailsSelection->IsPropertySelected(PropertyHandle.ToSharedRef()))
		{
			return;
		}

		const FName AnimDetailsMenuName = TEXT("AnimDetails");
		const FText AnimDetailsSectionName = LOCTEXT("AnimDetailsSection", "Anim Details");

		FToolMenuSection& Section = InToolMenu->AddSection(AnimDetailsMenuName, AnimDetailsSectionName);

		TArray<UAnimDetailsProxyBase*> Proxies;
		for (UObject* OuterObject : OuterObjects)
		{			
			if (UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(OuterObject))
			{
				Proxies.Add(Proxy);
			}
		}

		const bool bAnyKeyed = Proxies.ContainsByPredicate(
			[&PropertyHandle](const UAnimDetailsProxyBase* Proxy)
			{
				return Proxy->GetPropertyKeyedStatus(*PropertyHandle) != EPropertyKeyedStatus::NotKeyed;
			});

		if (!Proxies.IsEmpty() && 
			bAnyKeyed)
		{

			ExtendContextMenu(Section, PropertyHandle.ToSharedRef(), Proxies);
		}
	}

	void FAnimDetailsPropertyRowExtensions::ExtendContextMenu(
		FToolMenuSection& InSection,
		const TSharedRef<IPropertyHandle>& InPropertyHandle,
		const TArray<UAnimDetailsProxyBase*>& InProxies)
	{
		TWeakPtr<IPropertyHandle> WeakPropertyHandle = InPropertyHandle;

		TArray<TWeakObjectPtr<UAnimDetailsProxyBase>> WeakProxies;
		WeakProxies.Reserve(InProxies.Num());
		for (UAnimDetailsProxyBase* Proxy : InProxies)
		{
			WeakProxies.Add(Proxy);
		}

		FUIAction RemoveAllKeysAction;
		RemoveAllKeysAction.ExecuteAction = FExecuteAction::CreateLambda(
			[WeakProxies, WeakPropertyHandle]()
			{
				if (!WeakPropertyHandle.IsValid())
				{
					return;
				}
				const TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
				FScopedTransaction Transaction(LOCTEXT("RemoveAllKeys", "Remove All Keys"));
				for (const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy : WeakProxies)
				{
					if (WeakProxy.IsValid())
					{
						WeakProxy->RemoveAllKeys(*PropertyHandle);
					}
				}
			});

		RemoveAllKeysAction.CanExecuteAction = FCanExecuteAction::CreateLambda(
			[WeakProxies, WeakPropertyHandle]()
			{
				if (!WeakPropertyHandle.IsValid())
				{
					return false;
				}
				const TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin();
				return WeakProxies.ContainsByPredicate(
					[&PropertyHandle](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
					{
						return WeakProxy.IsValid() &&
							WeakProxy->GetPropertyKeyedStatus(*PropertyHandle) != EPropertyKeyedStatus::NotKeyed;
					});
			});

		InSection.AddEntry(FToolMenuEntry::InitMenuEntry(
			"RemoveAllKeys",
			LOCTEXT("RemoveAllKeysLabel", "Remove All Keys"),
			LOCTEXT("RemoveAllKeysTooltip", "Removes all keys of this channel from the current sequence"),
			TAttribute<FSlateIcon>(),
			FToolUIActionChoice(RemoveAllKeysAction),
			EUserInterfaceActionType::Button
		));
	}
}

#undef LOCTEXT_NAMESPACE
