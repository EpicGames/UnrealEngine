// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class UAnimDetailsProxyBase;
class UToolMenu;
struct FOnGenerateGlobalRowExtensionArgs;
struct FPropertyRowExtensionButton;
struct FToolMenuSection;

namespace UE::ControlRigEditor
{
	/** Extends the property rows of anim details, inspired by Dynamic Material Editor */
	class FAnimDetailsPropertyRowExtensions
	{
	public:
		static FAnimDetailsPropertyRowExtensions& Get();

		~FAnimDetailsPropertyRowExtensions();

		/** Registers the extensions with the property editor module */
		void RegisterRowExtensions();

		/** Unregisters the extensions from the property editor module */
		void UnregisterRowExtensions();

	private:
		/** Creates the anim details row extension */
		static void CreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions);

		/** Extends the context menu */
		static void ExtendContextMenu(UToolMenu* InToolMenu);

		/** Extends the context menu for the specific section and property handle for proxies */
		static void ExtendContextMenu(FToolMenuSection& InSection, const TSharedRef<IPropertyHandle>& InPropertyHandle, const TArray<UAnimDetailsProxyBase*>& InProxies);

		/** Delegate handle to track registration of the customization */
		FDelegateHandle RowExtensionHandle;

		/** Number of active registrations — the extension is bound while this is > 0 */
		int32 RegistrationCount = 0;
	};
}

