// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyDetails.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "AnimDetails/Widgets/SAnimDetailsPropertySelectionBorder.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "IDetailGroup.h"

#define LOCTEXT_NAMESPACE "AnimDetailsProxyDetails"

namespace UE::ControlRigEditor
{
	FAnimDetailsProxyDetails::FAnimDetailsProxyDetails()
	{}

	TSharedRef<IDetailCustomization> FAnimDetailsProxyDetails::MakeInstance()
	{
		return MakeShared<FAnimDetailsProxyDetails>();
	}

	void FAnimDetailsProxyDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		TArray<TWeakObjectPtr<UObject>> EditedObjects;
		DetailBuilder.GetObjectsBeingCustomized(EditedObjects);

		bool bIsIndividual = false;
		TArray<UAnimDetailsProxyBase*> GroupedProxies;
		for (const TWeakObjectPtr<UObject> WeakProxyObject : EditedObjects)
		{
			UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(WeakProxyObject.Get());
			if (Proxy)
			{
				GroupedProxies.Add(Proxy);

				bIsIndividual |= Proxy->bIsIndividual;
			}
		}

		if (GroupedProxies.IsEmpty())
		{
			return;
		}
		const UAnimDetailsProxyBase* FirstProxy = GroupedProxies[0];

		// Find the display name text and tooltip for this group
		FText DisplayNameText;
		FText TooltipText;
		if (GroupedProxies.Num() == 1 && GroupedProxies[0])
		{
			// Always use the long name for the category
			DisplayNameText = GroupedProxies[0]->GetDisplayNameText(EElementNameDisplayMode::ForceLong);
		}
		else
		{
			constexpr int32 MaxNumProxiesInTooltip = 5;
			TArray<FString> TooltipStrings;
			TArray<FString> CommonNames;
			for (int32 ProxyIndex = 0; ProxyIndex < GroupedProxies.Num(); ProxyIndex++)
			{
				const FText LongProxyNameText = GroupedProxies[ProxyIndex]->GetDisplayNameText(EElementNameDisplayMode::ForceLong);
				if (TooltipStrings.Num() < MaxNumProxiesInTooltip)
				{
					TooltipStrings.Add(LongProxyNameText.ToString());
				}

				const FText ShortProxyNameText = GroupedProxies[ProxyIndex]->GetDisplayNameText(EElementNameDisplayMode::ForceShort);
				CommonNames.AddUnique(ShortProxyNameText.ToString());
			}

			if (GroupedProxies.Num() > MaxNumProxiesInTooltip)
			{
				const int32 ExcessAmount = GroupedProxies.Num() - MaxNumProxiesInTooltip;
				const FText ExcessAmountText = FText::FromString(FString::FromInt(ExcessAmount));
				const FText ExcessText = FText::Format(LOCTEXT("ExcessProxiesInTooltip", "and {0} more"), ExcessAmountText);
				TooltipStrings.Add(ExcessText.ToString());
			}

			TooltipText = FText::FromString(FString::Join(TooltipStrings, TEXT("\n")));

			if (CommonNames.Num() == 1)
			{
				DisplayNameText = FText::Format(LOCTEXT("MultipleProxiesWithCommonNameLabel", "{0} (Multiple)"), FText::FromString(CommonNames[0]));
			}
			else
			{
				DisplayNameText = LOCTEXT("MultipleProxiesLabel", "Multiple");
			}
		}

		// Create a custom row to display the header instead of using the category row, so it cannot be collapsed
		IDetailCategoryBuilder& NoCategory = DetailBuilder.EditCategory("NoCategory");
		NoCategory.InitiallyCollapsed(false);

		if (!bIsIndividual)
		{
			const FText& NoFilterText = FText::GetEmpty();
			NoCategory.AddCustomRow(NoFilterText)
				.WholeRowContent()
				[
					SNew(SBorder)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.ToolTipText(TooltipText)
					.OnMouseButtonDown(this, &FAnimDetailsProxyDetails::OnHeaderCategoryRowClicked)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
					[
						SNew(STextBlock)
						.Text(DisplayNameText)
						.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
						.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
					]
				];
		}

		// Add properties to anim details specific categories instead of the default category
		const FName CategoryName = FirstProxy->GetCategoryName();

		IDetailCategoryBuilder& DefaultCategory = DetailBuilder.EditCategory(CategoryName);
		TArray<TSharedRef<IPropertyHandle>> PropertyHandles;
		DefaultCategory.GetDefaultProperties(PropertyHandles);
		DefaultCategory.SetCategoryVisibility(false);

		IDetailGroup* PropertyGroupPtr = nullptr;
		for (const TSharedRef<IPropertyHandle>& PropertyHandle : PropertyHandles)
		{	
			uint32 NumChildren;
			PropertyHandle->GetNumChildren(NumChildren);

			if (bIsIndividual && NumChildren < 2)
			{
				NoCategory.AddProperty(PropertyHandle);
			}
			else if (bIsIndividual)
			{
				if (!PropertyGroupPtr)
				{
					constexpr bool bForAdvanced = false;
					constexpr bool bStartExpanded = true;
					PropertyGroupPtr = &NoCategory.AddGroup(CategoryName, DisplayNameText, bForAdvanced, bStartExpanded);
					PropertyGroupPtr->SetToolTip(TooltipText);
				}

				PropertyGroupPtr->AddPropertyRow(PropertyHandle);
			}			
			else
			{
				NoCategory.AddProperty(PropertyHandle);
			}
		}
	}

	FReply FAnimDetailsProxyDetails::OnHeaderCategoryRowClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		// Reset selection
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		UAnimDetailsSelection* Selection = ProxyManager ? ProxyManager->GetAnimDetailsSelection() : nullptr;
		if (Selection)
		{
			Selection->ClearSelection();
		}

		return FReply::Handled();
	}
}

#undef LOCTEXT_NAMESPACE
