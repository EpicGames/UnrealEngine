// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"

namespace UE::ControlRigEditor
{
	/** Details customization for classes that inherit UAnimDetailsProxyBase */
	class FAnimDetailsProxyDetails
		: public IDetailCustomization
	{
	public:
		FAnimDetailsProxyDetails();

		/** Creates an instance of this details customization */
		static TSharedRef<IDetailCustomization> MakeInstance();

	protected:
		//~ Begin IDetailCustomization interface
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
		//~ End IDetailCustomization interface

	private:
		/** Called when the header category row (the category row we use to mock up a header row) was clicked */
		FReply OnHeaderCategoryRowClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	};
}
