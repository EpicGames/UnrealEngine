// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyManagerDetails.h"

#include "Algo/AnyOf.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "ControlRig.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "ISequencer.h"

#define LOCTEXT_NAMESPACE "AnimDetailsProxyManagerDetails"

namespace UE::ControlRigEditor
{
	namespace Private
	{
		/** Util to group proxies and add them to details */
		struct FAnimDetailsProxyGroupBuilder
		{
			/** Constructs the builder from an array of proxies */
			FAnimDetailsProxyGroupBuilder(const TArray<UAnimDetailsProxyBase*>& Proxies)
				: TopLevelProxyGroup(FAnimDetailsGroupedProxies(NAME_None))
			{
				for (UAnimDetailsProxyBase* Proxy : Proxies)
				{
					AddProxy(Proxy);
				}
			}

			/** Adds grouped top level proxies to details */
			void AddTopLevelProxies(IDetailLayoutBuilder& DetailLayout) const
			{
				const TMap<FName, TArray<UObject*>>& DetailRowIDToProxiesMap = TopLevelProxyGroup.DetailRowIDToProxiesMap;

				IDetailCategoryBuilder& CategoryBuilder_None = DetailLayout.EditCategory("nocategory");
				for (const TTuple<FName, TArray<UObject*>>& TopLevelDetailRowIDToProxiesPair : DetailRowIDToProxiesMap)
				{
					CategoryBuilder_None.AddExternalObjects(
						TopLevelDetailRowIDToProxiesPair.Value,
						EPropertyLocation::Default,
						FAddPropertyParams()
						.HideRootObjectNode(true));
				}
			}

			/** Adds grouped attribute proxies to details */
			void AddAttributeProxies(IDetailLayoutBuilder& DetailLayout)
			{			
				// Mind attribute proxies are displayed individually, but are still multi-edited across multiple control rig

				const bool bOmitAttributeGroupName = 
					!TopLevelProxyGroup.DetailRowIDToProxiesMap.IsEmpty() && 
					AttributeProxyGroups.Num() == 1;

				if (bOmitAttributeGroupName)
				{
					const FText CategoryName = LOCTEXT("AttributeCategoryName", "Attributes");
					IDetailCategoryBuilder& CategoryBuilder_Attributes = DetailLayout.EditCategory("Attributes", CategoryName);
					const FAnimDetailsGroupedProxies& AttributeGroup = AttributeProxyGroups[0];

					for (const TTuple<FName, TArray<UObject*>>& DetailRowIDToProxiesPair : AttributeGroup.DetailRowIDToProxiesMap)
					{
						CategoryBuilder_Attributes.AddExternalObjects(
							DetailRowIDToProxiesPair.Value,
							EPropertyLocation::Default,
							FAddPropertyParams()
							.HideRootObjectNode(true)
						);
					}
				}
				else
				{
					for (const FAnimDetailsGroupedProxies& AttributeGroup : AttributeProxyGroups)
					{
						constexpr bool bForAdvanced = false;
						constexpr bool bStartExpanded = true;

						const FText ParentNameText = AttributeGroup.ParentName.IsNone() ?
							FText::GetEmpty() :
							FText::FromString(AttributeGroup.ParentName.ToString());

						const FText CategoryName = ParentNameText.IsEmpty() ?
							LOCTEXT("CommonAttributeCategoryName", "Attributes") :
							FText::Format(LOCTEXT("ParentedAttributeCategoryName", "{0} Attributes"), ParentNameText);

						const FName CategoryID = *(CategoryName.ToString() + FGuid::NewGuid().ToString());
						IDetailCategoryBuilder& CategoryBuilder = DetailLayout.EditCategory(CategoryID, CategoryName);

						for (const TTuple<FName, TArray<UObject*>>& DetailRowIDToProxiesPair : AttributeGroup.DetailRowIDToProxiesMap)
						{
							CategoryBuilder.AddExternalObjects(
								DetailRowIDToProxiesPair.Value,
								EPropertyLocation::Default,
								FAddPropertyParams()
								.HideRootObjectNode(true)
							);
						}
					}
				}
			}

		private:
			/** A group of attribute proxies, defined by their parent name */
			struct FAnimDetailsGroupedProxies
			{
				FAnimDetailsGroupedProxies(const FName& InParentName)
					: ParentName(InParentName)
				{
				}

				/** The parent name */
				const FName ParentName;

				/** The proxies in this group */
				TMap<FName, TArray<UObject*>> DetailRowIDToProxiesMap;
			};

			/** Adds a proxy to the respective group */
			void AddProxy(UAnimDetailsProxyBase* Proxy)
			{
				if (!Proxy)
				{
					return;
				}

				const FName DetailsRowID = Proxy->GetDetailRowID();
				const FName ParentName = GetParentName(Proxy);

				FAnimDetailsGroupedProxies& ProxyGroup = [Proxy, &ParentName, this]() -> FAnimDetailsGroupedProxies&
					{
						if (Proxy->bIsIndividual)
						{
							FAnimDetailsGroupedProxies* ProxyGroupPtr = AttributeProxyGroups.FindByPredicate(
								[&ParentName, Proxy](const FAnimDetailsGroupedProxies& AttributeProxies)
								{
									return AttributeProxies.ParentName == ParentName;
								});

							if (ProxyGroupPtr)
							{
								return *ProxyGroupPtr;
							}
							else
							{
								AttributeProxyGroups.Add(FAnimDetailsGroupedProxies(ParentName));
								return AttributeProxyGroups.Last();
							}
						}
						else
						{
							return TopLevelProxyGroup;
						}
					}();

				ProxyGroup.DetailRowIDToProxiesMap.FindOrAdd(Proxy->GetDetailRowID()).Add(Proxy);
			}

			FName GetParentName(UAnimDetailsProxyBase* Proxy) const
			{
				// Assume all control rig controls have the same parent so they always get grouped,
				// but group sequencer items per parent
				if (const UObject* BoundObject = Proxy->GetSequencerItem().GetBoundObject())
				{
					if (const AActor* Actor = Cast<AActor>(BoundObject))
					{
						return Actor->GetFName();
					}
					else if (const UActorComponent* Component = Cast<UActorComponent>(BoundObject))
					{
						return Component->GetFName();
					}
				}

				return NAME_None;
			}

			FAnimDetailsGroupedProxies TopLevelProxyGroup;

			TArray<FAnimDetailsGroupedProxies> AttributeProxyGroups;
		};
	}

	TSharedRef<IDetailCustomization> FAnimDetailProxyManagerDetails::MakeInstance()
	{
		return MakeShared<FAnimDetailProxyManagerDetails>();
	}

	void FAnimDetailProxyManagerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
	{
		TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
		DetailLayout.GetObjectsBeingCustomized(ObjectsBeingCustomized);

		for (const TWeakObjectPtr<UObject>& ObjectBeingCustomized : ObjectsBeingCustomized)
		{
			if (UAnimDetailsProxyManager* ProxyManager = Cast<UAnimDetailsProxyManager>(ObjectBeingCustomized.Get()))
			{
				FAnimDetailsFilter& Filter = ProxyManager->GetAnimDetailsFilter();
				const TArray<UAnimDetailsProxyBase*> FilteredProxies = Filter.GetFilteredProxies();

				Private::FAnimDetailsProxyGroupBuilder ProxyGroupBuilder(FilteredProxies);

				ProxyGroupBuilder.AddTopLevelProxies(DetailLayout);
				ProxyGroupBuilder.AddAttributeProxies(DetailLayout);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
