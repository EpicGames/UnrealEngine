// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Engine/TimerHandle.h"

namespace UE::ControlRigEditor
{
	class FAnimDetailProxyManagerDetails 
		: public IDetailCustomization
	{
	public:
		/** Creates an instance of this details customization */
		static TSharedRef<IDetailCustomization> MakeInstance();

	protected:
		//~ Begin IDetailCustomization interface
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
		//~ End IDetailCustomization interface
	};
}
