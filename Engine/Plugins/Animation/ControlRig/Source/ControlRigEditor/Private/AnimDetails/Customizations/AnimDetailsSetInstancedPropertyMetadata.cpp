// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsSetInstancedPropertyMetadata.h"

#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "ControlRig.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "PropertyHandle.h"

namespace UE::ControlRigEditor::AnimDetailsMetaDataUtil
{
	TAutoConsoleVariable<bool> CVarAnimDetailsUseExternalPropertyMetadata(
		TEXT("AnimDetails.UseExternalPropertyMetadata"),
		true,
		TEXT("When enabled, Anim Details uses metadata from the controls and properties, instead of its own")
	);

	namespace Private
	{
		/** Returns the value at the component index of the type */
		template <typename TValueType>
		void GetValueFromComponentIndex(const int32 InComponentIndex, const TValueType& InValue, TOptional<float>& OutOptionalValue)
		{
			if constexpr (std::is_same_v<TValueType, float> || std::is_same_v<TValueType, int32>)
			{
				OutOptionalValue = InValue;
			}
			else if constexpr (std::is_same_v<TValueType, FVector3f>)
			{
				OutOptionalValue = InValue.Component(InComponentIndex);
			}
			else if constexpr (
				std::is_same_v<TValueType, FRigControlValue::FTransform_Float> ||
				std::is_same_v<TValueType, FRigControlValue::FTransformNoScale_Float> ||
				std::is_same_v<TValueType, FRigControlValue::FEulerTransform_Float>)
			{
				const FTransform Transform = [&InValue]()
					{
						if constexpr (
							std::is_same_v<TValueType, FRigControlValue::FTransformNoScale_Float> ||
							std::is_same_v<TValueType, FRigControlValue::FEulerTransform_Float>)
						{
							return InValue.ToTransform().ToFTransform();
						}
						else
						{
							return InValue.ToTransform();
						}
					}();

				if (InComponentIndex == 0)
				{
					OutOptionalValue = Transform.GetTranslation().X;
				}
				else if (InComponentIndex == 1)
				{
					OutOptionalValue = Transform.GetTranslation().Y;
				}
				else if (InComponentIndex == 2)
				{
					OutOptionalValue = Transform.GetTranslation().Z;
				}
				else if (InComponentIndex == 3)
				{
					OutOptionalValue = Transform.GetRotation().Rotator().Roll;
				}
				else if (InComponentIndex == 4)
				{
					OutOptionalValue = Transform.GetRotation().Rotator().Pitch;
				}
				else if (InComponentIndex == 5)
				{
					OutOptionalValue = Transform.GetRotation().Rotator().Yaw;
				}

				// Scale only for transforms with scale
				if constexpr (
					std::is_same_v<TValueType, FRigControlValue::FTransform_Float> ||
					std::is_same_v<TValueType, FRigControlValue::FEulerTransform_Float>)
				{
					if (InComponentIndex == 6)
					{
						OutOptionalValue = Transform.GetScale3D().X;
					}
					else if (InComponentIndex == 7)
					{
						OutOptionalValue = Transform.GetScale3D().Y;
					}
					else if (InComponentIndex == 8)
					{
						OutOptionalValue = Transform.GetScale3D().Z;
					}
				}
			}
			else
			{
				[] <bool SupportedType = false>()
				{
					static_assert(SupportedType, "Unsupported type in GetValueFromComponentIndex");
				}();
			}
		}

		/** Gets the min and max value from the component index of the type. 
		 * When bInApplyLimits is true, each side is gated by the corresponding LimitEnabled flag;
		 * otherwise the raw min/max from the control settings is always returned. 
		 */
		template <typename TValueType>
		void GetMinMaxValueFromComponentIndex(
			const FRigControlElement& InControlElement,
			const int32 InComponentIndex,
			const bool bInApplyLimits,
			TOptional<float>& OutOptionalMin,
			TOptional<float>& OutOptionalMax)
		{
			const TArray<FRigControlLimitEnabled>& LimitEnabled = InControlElement.Settings.LimitEnabled;
			if (!ensureMsgf(LimitEnabled.IsValidIndex(InComponentIndex), TEXT("Invalid value index when trying to get min max from element")))
			{
				return;
			}

			const bool bWithMinimum = !bInApplyLimits || (!LimitEnabled.IsEmpty() && LimitEnabled[InComponentIndex].bMinimum);
			const TOptional<TValueType> OptionalMinValue = bWithMinimum ? InControlElement.Settings.MinimumValue.Get<TValueType>() : TOptional<TValueType>();

			const bool bWithMaxium = !bInApplyLimits || (!LimitEnabled.IsEmpty() && LimitEnabled[InComponentIndex].bMaximum);
			const TOptional<TValueType> OptionalMaxValue = bWithMaxium ? InControlElement.Settings.MaximumValue.Get<TValueType>() : TOptional<TValueType>();

			if (OptionalMinValue.IsSet())
			{
				GetValueFromComponentIndex(InComponentIndex, OptionalMinValue.GetValue(), OutOptionalMin);
			}

			if (OptionalMaxValue.IsSet())
			{
				GetValueFromComponentIndex(InComponentIndex, OptionalMaxValue.GetValue(), OutOptionalMax);
			}
		}

		/** Returns the min and max value at given value index in the control element value. 
		 * When bInApplyLimits is true, only returns values whose LimitEnabled flag is set; 
		 * otherwise returns the raw min/max from the control settings. 
		 */
		void GetMinMaxFromControlElement(const FRigControlElement& InControlElement, const int32 InComponentIndex, const bool bInApplyLimits, TOptional<float>& OutOptionalMin, TOptional<float>& OutOptionalMax)
		{
			switch (InControlElement.Settings.ControlType)
			{
			case ERigControlType::Bool:
			{
				// Bool has no min max
				break;
			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				GetMinMaxValueFromComponentIndex<float>(InControlElement, InComponentIndex, bInApplyLimits, OutOptionalMin, OutOptionalMax);
				break;
			}
			case ERigControlType::Integer:
			{
				GetMinMaxValueFromComponentIndex<int32>(InControlElement, InComponentIndex, bInApplyLimits, OutOptionalMin, OutOptionalMax);
				break;
			}
			case ERigControlType::Vector2D:
			case ERigControlType::Rotator:
			case ERigControlType::Position:
			case ERigControlType::Scale:
			{
				GetMinMaxValueFromComponentIndex<FVector3f>(InControlElement, InComponentIndex, bInApplyLimits, OutOptionalMin, OutOptionalMax);
				break;
			}
			case ERigControlType::Transform:
			{
				GetMinMaxValueFromComponentIndex<FRigControlValue::FTransform_Float>(InControlElement, InComponentIndex, bInApplyLimits, OutOptionalMin, OutOptionalMax);
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				GetMinMaxValueFromComponentIndex<FRigControlValue::FTransformNoScale_Float>(InControlElement, InComponentIndex, bInApplyLimits, OutOptionalMin, OutOptionalMax);
				break;
			}
			case ERigControlType::EulerTransform:
			{
				GetMinMaxValueFromComponentIndex<FRigControlValue::FEulerTransform_Float>(InControlElement, InComponentIndex, bInApplyLimits, OutOptionalMin, OutOptionalMax);
				break;
			}
			default:
			{
				break;
			}
			}
		}

		/** Returns the min and max value gated by the control's enabled limits. 
		 * Use for ClampMin / ClampMax property metadata. 
		 */
		void GetClampMinMaxFromControlElement(const FRigControlElement& InControlElement, const int32 InComponentIndex, TOptional<float>& OutOptionalMin, TOptional<float>& OutOptionalMax)
		{
			constexpr bool bApplyLimits = true;
			GetMinMaxFromControlElement(InControlElement, InComponentIndex, bApplyLimits, OutOptionalMin, OutOptionalMax);
		}

		/** Returns the raw min and max value from the control settings, 
		 * regardless of whether the control's limits are enabled. 
		 * Exclusively used by Animation Channels for UI Min/Max
		 */
		void GetUIMinMaxFromControlElement(const FRigControlElement& InControlElement, const int32 InComponentIndex, TOptional<float>& OutOptionalMin, TOptional<float>& OutOptionalMax)
		{
			constexpr bool bApplyLimits = false;
			GetMinMaxFromControlElement(InControlElement, InComponentIndex, bApplyLimits, OutOptionalMin, OutOptionalMax);
		}
	}

	void SetInstancedPropertyMetaData(const UAnimDetailsProxyBase& TemplateProxy, const TSharedRef<IPropertyHandle>& ValuePropertyHandle)
	{
		if (!CVarAnimDetailsUseExternalPropertyMetadata.GetValueOnGameThread())
		{
			return;
		}

		const FProperty* Property = ValuePropertyHandle->GetProperty();
		if (!Property)
		{
			return;
		}
		const FName PropertyName = Property->GetFName();

		const UControlRig* ControlRig = TemplateProxy.GetControlRig();
		const FRigControlElement* ControlElement = TemplateProxy.GetControlElement();
		if (ControlRig && ControlElement)
		{
			// Always use a linear delta sensitivity and a slider exponent of 1 for control rig controls
			ValuePropertyHandle->SetInstanceMetaData("LinearDeltaSensitivity", FString::Printf(TEXT("%f"), 1.0f));
			ValuePropertyHandle->SetInstanceMetaData("SliderExponent", FString::Printf(TEXT("%f"), 1.0f));

			// Adopt limits from the rig element
			const int32 ComponentIndex = TemplateProxy.GetPropertyNames().IndexOfByKey(PropertyName);

			TOptional<float> ClampMin;
			TOptional<float> ClampMax;
			Private::GetClampMinMaxFromControlElement(*ControlElement, ComponentIndex, ClampMin, ClampMax);

			TOptional<float> UIMin;
			TOptional<float> UIMax;
			
			// Special case Animation Channels, their raw Min/Max values should be used as UI Min/Max
			if (ControlElement->IsAnimationChannel())
			{
				Private::GetUIMinMaxFromControlElement(*ControlElement, ComponentIndex, UIMin, UIMax);
			}

			// If the UI range doesn't define a finite slider range, and the property has no delta, use a delta of 0.5
			const bool bWithRange = UIMin.IsSet() && UIMax.IsSet();
			const bool bWithDeltaMetadata = !ValuePropertyHandle->GetMetaData("Delta").IsEmpty();
			if (!bWithRange && !bWithDeltaMetadata)
			{
				ValuePropertyHandle->SetInstanceMetaData("Delta", FString::Printf(TEXT("%f"), .5f));
			}
			
			if (ClampMin.IsSet())
			{
				ValuePropertyHandle->SetInstanceMetaData("ClampMin", FString::Printf(TEXT("%f"), ClampMin.GetValue()));
			}

			if (UIMin.IsSet())
			{
				ValuePropertyHandle->SetInstanceMetaData("UIMin", FString::Printf(TEXT("%f"), UIMin.GetValue()));
			}

			if (ClampMax.IsSet())
			{
				ValuePropertyHandle->SetInstanceMetaData("ClampMax", FString::Printf(TEXT("%f"), ClampMax.GetValue()));
			}

			if (UIMax.IsSet())
			{
				ValuePropertyHandle->SetInstanceMetaData("UIMax", FString::Printf(TEXT("%f"), UIMax.GetValue()));
			}
		}
		else if (const FProperty* SequencerProperty = TemplateProxy.GetSequencerItem().GetProperty())
		{
			// Adopt supported metadata from the controlled property
			if (const TMap<FName, FString>* PropertyMetadatas = SequencerProperty->GetMetaDataMap())
			{
				static const TSet<FName> SupprtedMetadataNames =
				{
					 "DisplayName",
					 "Tooltip",
					 "SliderExponent",
					 "LinearDeltaSensitivity",
					 "Delta",
					 "ClampMin",
					 "ClampMax",
					 "UIMin",
					 "UIMax",
					 "Units"
				};

				for (const TTuple<FName, FString>& PropertyMetadata : *PropertyMetadatas)
				{
					if (SupprtedMetadataNames.Contains(PropertyMetadata.Key) &&
						!PropertyMetadata.Value.IsEmpty())
					{
						ValuePropertyHandle->SetInstanceMetaData(PropertyMetadata.Key, PropertyMetadata.Value);
					}
				}
			}
		}
	}
}
