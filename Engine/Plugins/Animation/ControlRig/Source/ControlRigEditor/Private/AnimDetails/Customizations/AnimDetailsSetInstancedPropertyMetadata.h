// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IPropertyHandle;
class UAnimDetailsProxyBase;

namespace UE::ControlRigEditor::AnimDetailsMetaDataUtil
{
	/** Sets property meta data for the property instance from the data in the template proxy, for example min and max values. */
	void SetInstancedPropertyMetaData(const UAnimDetailsProxyBase& TemplateProxy, const TSharedRef<IPropertyHandle>& ValuePropertyHandle);
}
