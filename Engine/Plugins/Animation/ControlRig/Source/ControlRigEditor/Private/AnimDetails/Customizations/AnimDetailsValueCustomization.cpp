// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsValueCustomization.h"

#include "Algo/AnyOf.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "AnimDetails/Customizations/AnimDetailsSetInstancedPropertyMetadata.h"
#include "AnimDetails/Proxies/AnimDetailsProxyLocation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyRotation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyScale.h"
#include "AnimDetails/Proxies/AnimDetailsProxyVector2D.h"
#include "AnimDetails/Widgets/SAnimDetailsPropertySelectionBorder.h"
#include "AnimDetails/Widgets/SAnimDetailsValueBoolean.h"
#include "AnimDetails/Widgets/SAnimDetailsValueNumeric.h"
#include "DetailWidgetRow.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyEditorClipboard.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "AnimDetailsValueCustomization"

namespace UE::ControlRigEditor
{
	namespace Private
	{
		FAnimDetailsChildWidgetVisiblityHandler::FAnimDetailsChildWidgetVisiblityHandler(const TSharedRef<IPropertyHandle>& InStructPropertyHandle, const TArray<TWeakPtr<SWidget>>& InWeakWidgets)
			: WeakStructPropertyHandle(InStructPropertyHandle)
			, WeakWidgets(InWeakWidgets)
		{
			if (WeakWidgets.IsEmpty())
			{
				return;
			}

			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[this](float DeltaTime)
				{
					check(!WeakWidgets.IsEmpty());

					const TSharedPtr<SWidget> FirstWidget = WeakWidgets[0].Pin();
					if (!FirstWidget.IsValid())
					{
						return false;
					}

					if (!WeakStructPropertyHandle.IsValid())
					{
						return false;
					}

					const EVisibility Visibility = WeakStructPropertyHandle.Pin()->IsExpanded() ?
						EVisibility::Visible :
						EVisibility::Collapsed;

					if (FirstWidget->GetVisibility() != Visibility)
					{
						for (const TWeakPtr<SWidget>& WeaWidget : WeakWidgets)
						{
							if (WeaWidget.IsValid())
							{
								WeaWidget.Pin()->SetVisibility(Visibility);
							}
						}
					}

					return true;
				}
			));
		}

		FAnimDetailsChildWidgetVisiblityHandler::~FAnimDetailsChildWidgetVisiblityHandler()
		{
			if (TickerHandle.IsValid())
			{
				FTSTicker::RemoveTicker(TickerHandle);
			}
		}
	}

	TSharedRef<IPropertyTypeCustomization> FAnimDetailsValueCustomization::MakeInstance()
	{
		return MakeShared<FAnimDetailsValueCustomization>();
	}

	void FAnimDetailsValueCustomization::CustomizeChildren(
		TSharedRef<IPropertyHandle> InStructPropertyHandle, 
		IDetailChildrenBuilder& InStructBuilder, 
		IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

		if (!ProxyManager ||
			IsStructPropertyHiddenByFilter(InStructPropertyHandle))
		{
			InStructPropertyHandle->MarkHiddenByCustomization();
			return;
		}

		IDetailCategoryBuilder& CategoryBuilder = InStructBuilder.GetParentCategory();
		DetailBuilder = &CategoryBuilder.GetParentLayout();

		StructPropertyHandle = InStructPropertyHandle;

		const bool bHasMultipleChildHandles = SortedChildHandles.Num() > 1;

		// Details view deferred actions execute before RestoreAllExpandedItems(), so use a one-shot
		// next-frame callback to collapse after the saved expansion state has been restored.
		if (bHasMultipleChildHandles
			&& !GetDefault<UAnimDetailsSettings>()->bExpandTransformProperties
			&& ShouldUseTransformPropertyExpansion(InStructPropertyHandle))
		{
			const TWeakPtr<IPropertyHandle> WeakStructPropertyHandle = InStructPropertyHandle;
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[WeakStructPropertyHandle](const float InDeltaTime)
				{
					const TSharedPtr<IPropertyHandle> StructPropertyHandle = WeakStructPropertyHandle.Pin();
					if (StructPropertyHandle.IsValid()
						&& StructPropertyHandle->IsValidHandle()
						&& StructPropertyHandle->IsExpanded())
					{
						StructPropertyHandle->SetExpanded(false);
					}
					return false;
				}));
		}

		// Show custom children if expanded, allowing to select individual controls in the property name row
		TArray<TWeakPtr<SWidget>> ChildContentWidgets;
		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];

			if (IsChildPropertyHiddenByFilter(ChildHandle))
			{
				continue;
			}

			TArray<UObject*> OuterObjects;
			ChildHandle->GetOuterObjects(OuterObjects);

			UAnimDetailsProxyBase* FirstProxy = OuterObjects.IsEmpty() ? nullptr : Cast<UAnimDetailsProxyBase>(OuterObjects[0]);
			if (!FirstProxy)
			{
				continue;
			}

			AnimDetailsMetaDataUtil::SetInstancedPropertyMetaData(*FirstProxy, ChildHandle);

			const FText PropertyDisplayText = [bHasMultipleChildHandles, FirstProxy, &ChildHandle, this]()
				{
					// Draw the proxy display name if the property has only one child.
					if (!bHasMultipleChildHandles)
					{
						if (FirstProxy)
						{
							const UControlRig* ControlRig = FirstProxy->GetControlRig();
							const URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
							const FRigControlElement* ControlElement = FirstProxy->GetControlElement();
							if (ControlRig && Hierarchy && ControlElement)
							{
								// For single control rig controls, show the show the short element name
								constexpr EElementNameDisplayMode NameDisplayMode = EElementNameDisplayMode::ForceShort;
								return Hierarchy->GetDisplayNameForUI(ControlElement, NameDisplayMode);
							}
							else if (const UMovieSceneTrack* Track = FirstProxy->GetSequencerItem().GetMovieSceneTrack())
							{
								// For single sequencer bindings, show the track name
								return Track->GetDisplayName();
							}
						}
					}
					
					return ChildHandle->GetPropertyDisplayName();
				}();

			// Handle visibilty for child content widgets. This is required so that  
			// FAnimDetailsNavigableWidgetRegistry can find the right widget to navigate to.
			const TSharedRef<SWidget> ValueWidget = MakeChildWidget(InStructPropertyHandle, ChildHandle);
			if (bHasMultipleChildHandles)
			{
				ChildContentWidgets.Add(ValueWidget);
			}

			InStructBuilder.AddProperty(ChildHandle)
				.CustomWidget()
				.NameContent()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SAnimDetailsPropertySelectionBorder, *ProxyManager, ChildHandle)
					.NavigateToWidget(ValueWidget)
					[
						SNew(STextBlock)
						.Text(PropertyDisplayText)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					]
				]
				.ValueContent()
				.HAlign(HAlign_Fill)
				[
					SNew(SAnimDetailsPropertySelectionBorder, *ProxyManager, ChildHandle)
					.NavigateToWidget(ValueWidget)
					.RequiresModifierKeys(true)				
					[
						ValueWidget
					]
				]
				.ExtensionContent()
				[
					ChildHandle->CreateDefaultPropertyButtonWidgets()
				]
				.OverrideResetToDefault(
					FResetToDefaultOverride::Create
					(
						TAttribute<bool>::CreateSP(this, &FAnimDetailsValueCustomization::IsResetToDefaultVisible, ChildHandle.ToWeakPtr()),
						FSimpleDelegate::CreateSP(this, &FAnimDetailsValueCustomization::OnResetToDefaultClicked, ChildHandle.ToWeakPtr())
					))
				.CopyAction(
					FUIAction(
						FExecuteAction::CreateLambda([WeakChildHandle = ChildHandle->AsWeak()]
							{
								const TSharedPtr<IPropertyHandle> PropertyHandle = WeakChildHandle.IsValid() ? WeakChildHandle.Pin() : nullptr;

								FString Value;
								if (PropertyHandle.IsValid() &&
									PropertyHandle->IsValidHandle() &&
									PropertyHandle->GetValueAsFormattedString(Value, PPF_Copy) == FPropertyAccess::Success)
								{
									FPropertyEditorClipboard::ClipboardCopy(*Value);
								}
							}),
						FCanExecuteAction::CreateLambda([]()
							{
								return true;
							})
					)
				)
				.PasteAction(
					FUIAction(
						FExecuteAction::CreateLambda([WeakChildHandle = ChildHandle->AsWeak()]
							{								
								const TSharedPtr<IPropertyHandle> PropertyHandle = WeakChildHandle.IsValid() ? WeakChildHandle.Pin() : nullptr;

								FString Value;
								if (PropertyHandle.IsValid() &&
									PropertyHandle->IsValidHandle())
								{
									FString ClipboardContent;
									FPropertyEditorClipboard::ClipboardPaste(ClipboardContent);

									PropertyHandle->SetValueFromFormattedString(ClipboardContent);
								}
							}),
						FCanExecuteAction::CreateLambda([]()
							{
								const bool bShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
								return !bShiftDown;
							})
					)
				);
		}

		ChildContentVisibilityHandler = MakeUnique<Private::FAnimDetailsChildWidgetVisiblityHandler>(InStructPropertyHandle, ChildContentWidgets);
	}

	void FAnimDetailsValueCustomization::MakeHeaderRow(TSharedRef<IPropertyHandle>& InStructPropertyHandle, FDetailWidgetRow& InRow)
	{
		constexpr TCHAR ShowOnlyInnerPropertiesMetaDataName[] = TEXT("ShowOnlyInnerProperties");
		bool bShowHeader = !InStructPropertyHandle->HasMetaData(ShowOnlyInnerPropertiesMetaDataName);
		if (!bShowHeader || 
			IsStructPropertyHiddenByFilter(InStructPropertyHandle))
		{
			return;
		}

		StructPropertyHandle = InStructPropertyHandle;

		const bool bHasMultipleChildHandles = SortedChildHandles.Num() > 1;

		if (bHasMultipleChildHandles
			&& GetDefault<UAnimDetailsSettings>()->bExpandTransformProperties
			&& ShouldUseTransformPropertyExpansion(InStructPropertyHandle))
		{
			InRow.ShouldAutoExpand(true);
		}

		// Make enough space for each child handle
		const float DesiredWidth = 125.f * SortedChildHandles.Num();

		TSharedPtr<SHorizontalBox> HorizontalBox;

		InRow.NameContent()
			[
				MakePropertyNameWidget(InStructPropertyHandle)
			]
			.PasteAction(FUIAction(
				FExecuteAction::CreateLambda([]()
					{
						checkNoEntry();
					}),
				FCanExecuteAction::CreateLambda([]() 
					{ 
						return false; 
					}))
			)
			.ValueContent()
			.MinDesiredWidth(DesiredWidth)
			.MaxDesiredWidth(DesiredWidth)
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
				.Visibility(this, &FAnimDetailsValueCustomization::GetVisibilityFromExpansionState)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructPropertyHandle.ToWeakPtr())
			]
			.OverrideResetToDefault(
				FResetToDefaultOverride::Create
				(
					TAttribute<bool>::CreateSP(this, &FAnimDetailsValueCustomization::IsResetToDefaultVisible, StructPropertyHandle.ToWeakPtr()),
					FSimpleDelegate::CreateSP(this, &FAnimDetailsValueCustomization::OnResetToDefaultClicked, StructPropertyHandle.ToWeakPtr())
				));

		// Create inline children if collapsed, the children can only be selected in the property value row
		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
			FProperty* Property = ChildHandle->GetProperty();
			if (!Property)
			{
				continue;
			}

			const TSharedRef<SWidget> ChildWidget = MakeChildWidget(InStructPropertyHandle, ChildHandle);

			// Always display childs in the struct row but disable them if they're filtered out
			if (IsChildPropertyHiddenByFilter(ChildHandle))
			{
				ChildWidget->SetEnabled(false);
				ChildWidget->SetToolTipText(LOCTEXT("PropertyNotInFilterTooltip", "Excluded by search"));
			}

			const bool bLastChild = SortedChildHandles.Num() - 1 == ChildIndex;
			if (ChildHandle->GetPropertyClass() == FBoolProperty::StaticClass())
			{
				HorizontalBox->AddSlot()
					.Padding(FMargin(0.f, 2.f, bLastChild ? 0.f : 3.f, 2.f))
					.AutoWidth()  // keep the check box slots small
					[
						ChildWidget
					];
			}
			else
			{
				if (ChildHandle->GetPropertyClass() == FDoubleProperty::StaticClass())
				{
					NumericEntryBoxWidgetList.Add(ChildWidget);
				}

				HorizontalBox->AddSlot()
					.Padding(FMargin(0.f, 2.f, bLastChild ? 0.f : 3.f, 2.f))
					[
						ChildWidget
					];
			}
		}
	}

	TSharedRef<SWidget> FAnimDetailsValueCustomization::MakePropertyNameWidget(const TSharedRef<IPropertyHandle>& InStructPropertyHandle) const
	{
		uint32 NumChildren;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		// For properties with only one child, display the control name instead of the struct property name
		if (NumChildren == 1)
		{
			TArray<UObject*> OuterObjects;
			InStructPropertyHandle->GetOuterObjects(OuterObjects);
			const UAnimDetailsProxyBase* OuterProxy = OuterObjects.IsEmpty() ? nullptr : Cast<UAnimDetailsProxyBase>(OuterObjects[0]);
			if (OuterProxy)
			{
				return
					SNew(STextBlock)
					.Text(OuterProxy->GetDisplayNameText())
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"));
			}
		}

		return InStructPropertyHandle->CreatePropertyNameWidget();
	}

	TSharedRef<SWidget> FAnimDetailsValueCustomization::MakeChildWidget(
		TSharedRef<IPropertyHandle>& StructurePropertyHandle,
		TSharedRef<IPropertyHandle>& PropertyHandle)
	{
		const FProperty* Property = PropertyHandle->GetProperty();

		if (!Property)
		{
			return SNullWidget::NullWidget;
		}

		const FName PropertyName = Property->GetFName();
		const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();
		const FLinearColor LabelColor = GetColorFromProperty(PropertyName);
		const TSharedRef<SWidget> Label = SAnimDetailsValueNumeric<double>::BuildNarrowColorLabel(LabelColor);

		if (PropertyClass == FDoubleProperty::StaticClass())
		{
			return
				SNew(SAnimDetailsValueNumeric<double>, PropertyHandle)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, PropertyHandle.ToWeakPtr())
				.LabelPadding(FMargin(3))
				.LabelLocation(SAnimDetailsValueNumeric<double>::ELabelLocation::Inside)
				.Label()
				[
					Label
				];
		}
		else if (PropertyClass == FInt64Property::StaticClass())
		{
			return
				SNew(SAnimDetailsValueNumeric<int64>, PropertyHandle)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, PropertyHandle.ToWeakPtr())
				.LabelPadding(FMargin(3))
				.LabelLocation(SAnimDetailsValueNumeric<int64>::ELabelLocation::Inside)
				.Label()
				[
					Label
				];
		}
		else if (PropertyClass == FBoolProperty::StaticClass())
		{
			return
				SNew(SAnimDetailsValueBoolean, PropertyHandle)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, PropertyHandle.ToWeakPtr());
		}

		ensureMsgf(false, TEXT("Unsupported property class, cannot create an Anim Detail Values customization."));
		return SNullWidget::NullWidget;
	}

	bool FAnimDetailsValueCustomization::IsStructPropertyHiddenByFilter(const TSharedRef<class IPropertyHandle>& InStructPropertyHandle) const
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

		return 
			ProxyManager && 
			!ProxyManager->GetAnimDetailsFilter().ContainsStructProperty(InStructPropertyHandle);
	}

	bool FAnimDetailsValueCustomization::IsChildPropertyHiddenByFilter(const TSharedRef<IPropertyHandle>& InPropertyHandle) const
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

		return
			ProxyManager &&
			!ProxyManager->GetAnimDetailsFilter().ContainsProperty(InPropertyHandle);
	}

	EVisibility FAnimDetailsValueCustomization::GetVisibilityFromExpansionState() const
	{
		const bool bExpanded = StructPropertyHandle.IsValid() && StructPropertyHandle->IsExpanded();
		return bExpanded ? EVisibility::Collapsed : EVisibility::Visible;
	}

	FLinearColor FAnimDetailsValueCustomization::GetColorFromProperty(const FName& PropertyName) const
	{
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, X) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))
		{
			return SNumericEntryBox<double>::RedLabelBackgroundColor;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, Y) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))
		{
			return SNumericEntryBox<double>::GreenLabelBackgroundColor;
		}
		else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ) ||
			PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))
		{
			return SNumericEntryBox<double>::BlueLabelBackgroundColor;
		}

		return FLinearColor::White;
	}

	bool FAnimDetailsValueCustomization::IsResetToDefaultVisible(const TWeakPtr<IPropertyHandle> WeakPropertyHandle) const
	{
		const TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.IsValid() ? WeakPropertyHandle.Pin() : nullptr;
		const FProperty* Property = PropertyHandle.IsValid() ? PropertyHandle->GetProperty() : nullptr;
		if (!PropertyHandle.IsValid() ||
			!Property)
		{
			return false;
		}

		const FName PropertyName = Property->GetFName();

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		return Algo::AnyOf(OuterObjects,
			[&PropertyName](const UObject* OuterObject)
			{
				if (const UAnimDetailsProxyBase* Proxy = Cast<const UAnimDetailsProxyBase>(OuterObject))
				{
					return Proxy && !Proxy->HasDefaultValue(PropertyName);
				}

				return false;
			});
	}

	void FAnimDetailsValueCustomization::OnResetToDefaultClicked(const TWeakPtr<IPropertyHandle> WeakPropertyHandle)
	{
		const TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.IsValid() ? WeakPropertyHandle.Pin() : nullptr;
		const FProperty* Property = PropertyHandle.IsValid() ? PropertyHandle->GetProperty() : nullptr;
		if (!PropertyHandle.IsValid() ||
			!Property)
		{
			return;
		}

		const FName PropertyName = Property->GetFName();

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		const FScopedTransaction ResetValueToDefaultTransaction(LOCTEXT("ResetValueToDefaultTransaction", "Reset Value To Default"));

		// Reset all then Propagate all values so all is evaluated in the new state
		for (UObject* OuterObject : OuterObjects)
		{
			if (UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(OuterObject))
			{
				Proxy->Modify();
				Proxy->ResetPropertyToDefault(PropertyName);
			}
		}

		for (UObject* OuterObject : OuterObjects)
		{
			if (UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(OuterObject))
			{
				Proxy->PropagateValues();
			}
		}
	}

	bool FAnimDetailsValueCustomization::ShouldUseTransformPropertyExpansion(const TSharedRef<IPropertyHandle>& InStructPropertyHandle)
	{
		const FStructProperty* const StructProperty = CastField<FStructProperty>(InStructPropertyHandle->GetProperty());
		if (StructProperty && StructProperty->Struct)
		{
			return StructProperty->Struct == FAnimDetailsLocation::StaticStruct()
				|| StructProperty->Struct == FAnimDetailsRotation::StaticStruct()
				|| StructProperty->Struct == FAnimDetailsScale::StaticStruct();
		}
		return false;
	}

}

#undef LOCTEXT_NAMESPACE
