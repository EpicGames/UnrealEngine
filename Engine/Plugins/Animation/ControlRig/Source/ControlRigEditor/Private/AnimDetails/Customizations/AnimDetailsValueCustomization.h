// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Customizations/MathStructCustomizations.h"

class IDetailLayoutBuilder;
class IDetailPropertyRow;
class IPropertyHandle;
class SWidget;

namespace UE::ControlRigEditor
{
	namespace Private
	{
		/**
		 * When structs are collapsed, the visibility of the widgets is not updated, and their visibility attribute is not executed.
		 *
		 * This struct serves to update the widget's visibility regardless of this state. Required to deduce if the widget is navigable.
		 */
		struct FAnimDetailsChildWidgetVisiblityHandler
		{
			FAnimDetailsChildWidgetVisiblityHandler() = default;
			FAnimDetailsChildWidgetVisiblityHandler(const TSharedRef<IPropertyHandle>& InStructPropertyHandle, const TArray<TWeakPtr<SWidget>>& InWeakWidgets);
			
			~FAnimDetailsChildWidgetVisiblityHandler();

		private:
			TWeakPtr<IPropertyHandle> WeakStructPropertyHandle;
			TArray<TWeakPtr<SWidget>> WeakWidgets;

			FTSTicker::FDelegateHandle TickerHandle;
		};
	}

	/** Property type customization for struct properties such as FAnimDetailsBool or FAnimDetailsTransform */
	class FAnimDetailsValueCustomization
		: public FMathStructCustomization
	{
	public:
		/** Creates an instance of this struct customization */
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization interface
		virtual void MakeHeaderRow(TSharedRef<IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
		virtual TSharedRef<SWidget> MakeChildWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle) override;
		//~ End IPropertyTypeCustomization interface

	private:
		/** Makes a widget to display the property name */
		TSharedRef<SWidget> MakePropertyNameWidget(const TSharedRef<IPropertyHandle>& InStructPropertyHandle) const;

		/** Returns true if this struct is hidden by the filter */
		bool IsStructPropertyHiddenByFilter(const TSharedRef<IPropertyHandle>& InStructPropertyHandle) const;

		/** Returns true if this struct is hidden by the filter */
		bool IsChildPropertyHiddenByFilter(const TSharedRef<IPropertyHandle>& InPropertyHandle) const;

		/** Returns visible if the value struct is expanded, collapsed otherwise */
		EVisibility GetVisibilityFromExpansionState() const;

		/** Gets a color from the property */
		FLinearColor GetColorFromProperty(const FName& PropertyName) const;

		/** Determines whether the transform property should use an expanded layout for its UI in property customizations */
		static bool ShouldUseTransformPropertyExpansion(const TSharedRef<IPropertyHandle>& InStructPropertyHandle);

		/** Returns true if the property handle's reset to default button is visible */
		bool IsResetToDefaultVisible(const TWeakPtr<IPropertyHandle> WeakPropertyHandle) const;

		/** Called when the property handle's reset to default button was clicked */
		void OnResetToDefaultClicked(const TWeakPtr<IPropertyHandle> WeakPropertyHandle);

		/** Handles visibility for child widgets in this customization */
		TUniquePtr<Private::FAnimDetailsChildWidgetVisiblityHandler> ChildContentVisibilityHandler;

		/** Pointer to the detail builder, or nullptr if not initialized */
		IDetailLayoutBuilder* DetailBuilder = nullptr;

		/** The customized struct */
		TSharedPtr<IPropertyHandle> StructPropertyHandle;

		/** The numeric entry box widget */
		TSharedPtr<SWidget> NumericEntryBox;
	};
}
