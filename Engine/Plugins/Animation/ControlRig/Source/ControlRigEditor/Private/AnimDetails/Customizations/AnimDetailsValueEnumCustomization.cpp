// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsValueEnumCustomization.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/Proxies/AnimDetailsProxyEnum.h"
#include "DetailWidgetRow.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "SEnumCombo.h"

namespace UE::ControlRigEditor
{
	TSharedRef<IPropertyTypeCustomization> FAnimDetailsValueEnumCustomization::MakeInstance()
	{
		return MakeShared<FAnimDetailsValueEnumCustomization>();
	}

	void FAnimDetailsValueEnumCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		if (IsStructPropertyHiddenByFilter(InStructPropertyHandle))
		{
			InStructPropertyHandle->MarkHiddenByCustomization();
			return;
		}

		EnumIndexPropertyHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimDetailsEnum, EnumIndex));
		if (!ensureMsgf(EnumIndexPropertyHandle.IsValid(), TEXT("Unexpected cannot get enum index property handle for FAnimDetailsEnum")))
		{
			return;
		}

		const UEnum* EnumType = GetEnumType();
		if (!EnumType)
		{
			return;
		}

		HeaderRow
			.NameContent()
			.VAlign(VAlign_Center)
			[
				MakePropertyNameWidget(InStructPropertyHandle)
			]
			.ValueContent()
			.VAlign(VAlign_Center)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SNew(SEnumComboBox, EnumType)
					.OnEnumSelectionChanged(this, &FAnimDetailsValueEnumCustomization::OnEnumValueChanged)
					.CurrentValue(this, &FAnimDetailsValueEnumCustomization::GetEnumIndex)
				]

				+ SOverlay::Slot()
				[
					SNew(SComboButton)
					.Visibility_Lambda([this]()
						{
							int32 EnumIndex = 0;
							if (EnumIndexPropertyHandle.IsValid() &&
								EnumIndexPropertyHandle->IsValidHandle() &&
								EnumIndexPropertyHandle->GetValue(EnumIndex) == FPropertyAccess::Success)
							{
								return EVisibility::Collapsed;
							}

							return EVisibility::HitTestInvisible;
						})
					.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
					.ButtonContent()
					[
						SNew(STextBlock)
						.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
						.Text(NSLOCTEXT("FAnimDetailsValueEnumCustomization", "MultipleValuesText", "Multiple Values"))
					]
				]
			];
	}

	void FAnimDetailsValueEnumCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{}

	TSharedRef<SWidget> FAnimDetailsValueEnumCustomization::MakePropertyNameWidget(const TSharedRef<IPropertyHandle>& InStructPropertyHandle) const
	{
		TArray<UObject*> OuterObjects;
		InStructPropertyHandle->GetOuterObjects(OuterObjects);

		const UAnimDetailsProxyBase* OuterProxy = OuterObjects.IsEmpty() ? nullptr : Cast<UAnimDetailsProxyBase>(OuterObjects[0]);
		if (OuterProxy)
		{
			return
				SNew(STextBlock)
				.Text(InStructPropertyHandle->GetPropertyDisplayName())
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"));
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}

	void FAnimDetailsValueEnumCustomization::OnEnumValueChanged(int32 InValue, ESelectInfo::Type InSelectInfo)
	{
		if (EnumIndexPropertyHandle.IsValid() && EnumIndexPropertyHandle->IsValidHandle())
		{
			EnumIndexPropertyHandle->SetValue(InValue);
		}
	}

	bool FAnimDetailsValueEnumCustomization::IsStructPropertyHiddenByFilter(const TSharedRef<class IPropertyHandle>& InStructPropertyHandle) const
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

		return
			ProxyManager &&
			!ProxyManager->GetAnimDetailsFilter().ContainsStructProperty(InStructPropertyHandle);
	}

	const UEnum* FAnimDetailsValueEnumCustomization::GetEnumType() const
	{
		if (EnumIndexPropertyHandle.IsValid() && EnumIndexPropertyHandle->IsValidHandle())
		{
			TArray<UObject*> OuterObjects;
			EnumIndexPropertyHandle->GetOuterObjects(OuterObjects);

			UAnimDetailsProxyEnum* EnumProxy = OuterObjects.IsEmpty() ? nullptr : Cast<UAnimDetailsProxyEnum>(OuterObjects[0]);
			return EnumProxy ? EnumProxy->Enum.EnumType : nullptr;
		}

		return nullptr;
	}

	int32 FAnimDetailsValueEnumCustomization::GetEnumIndex() const
	{
		int32 EnumIndex = 0;
		if (EnumIndexPropertyHandle.IsValid() &&
			EnumIndexPropertyHandle->IsValidHandle() &&
			EnumIndexPropertyHandle->GetValue(EnumIndex) == FPropertyAccess::Success)
		{
			return EnumIndex;
		}

		return 0;
	}
}
