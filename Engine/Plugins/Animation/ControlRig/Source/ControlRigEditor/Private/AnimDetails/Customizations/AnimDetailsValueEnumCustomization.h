// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SWidget;

namespace UE::ControlRigEditor
{
	/** Property type customization for FAnimDetailsEnum */
	class FAnimDetailsValueEnumCustomization
		: public IPropertyTypeCustomization
	{
	public:
		/** Creates an instance of this property type customization */
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	protected:
		//~ Begin IPropertyTypeCustomization interface
		virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		//~ End IPropertyTypeCustomization interface

	private:
		/** Makes a widget to display the property name */
		TSharedRef<SWidget> MakePropertyNameWidget(const TSharedRef<IPropertyHandle>& InStructPropertyHandle) const;

		/** Returns true if this struct is hidden by the filter */
		bool IsStructPropertyHiddenByFilter(const TSharedRef<IPropertyHandle>& InStructPropertyHandle) const;

		/** Called when the enum value changed */
		void OnEnumValueChanged(int32 InValue, ESelectInfo::Type InSelectInfo);

		/** Returns the enum type */
		const UEnum* GetEnumType() const;

		/** Returns the current enum index */
		int32 GetEnumIndex() const;

		/** Property handle for the EnumIndex property */
		TSharedPtr<IPropertyHandle> EnumIndexPropertyHandle;
	};
}
