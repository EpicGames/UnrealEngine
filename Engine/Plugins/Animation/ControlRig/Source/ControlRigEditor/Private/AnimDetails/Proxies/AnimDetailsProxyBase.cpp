// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyBase.h"

#include "AnimDetails/AnimDetailsMultiEditUtil.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetailsProxyTransform.h"
#include "AnimDetailsProxyVector2D.h"
#include "ConstraintsManager.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "IDetailKeyframeHandler.h"
#include "ISequencer.h"
#include "LevelEditorViewport.h"
#include "MovieSceneCommonHelpers.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "PropertyHandle.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ScopedTransaction.h"
#include "Sequencer/ControlRigSequencerHelpers.h"
#include "Sequencer/AnimLayers/AnimLayers.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "SequencerAddKeyOperation.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Units/RigUnitContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyBase)

namespace UE::ControlRigEditor
{
	namespace KeyUtils
	{
		/** 
		 * Maps EControlRigContextChannelToKey bits to channel indices using the canonical Transform layout: 
		 * 0-2 = Translation, 3-5 = Rotation, 6-8 = Scale
		 * Handles arbitrary bit combinations correctly, unlike a switch on compound values. 
		 */
		static TArray<int32, TFixedAllocator<9>> ChannelIndicesFromMask(EControlRigContextChannelToKey Mask)
		{
			TArray<int32, TFixedAllocator<9>> Indices;
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::TranslationX)) { Indices.Add(0); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::TranslationY)) { Indices.Add(1); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::TranslationZ)) { Indices.Add(2); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::RotationX)) { Indices.Add(3); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::RotationY)) { Indices.Add(4); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::RotationZ)) { Indices.Add(5); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::ScaleX)) { Indices.Add(6); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::ScaleY)) { Indices.Add(7); }
			if (EnumHasAnyFlags(Mask, EControlRigContextChannelToKey::ScaleZ)) { Indices.Add(8); }
			return Indices;
		}

		/** Runs function for each channel to key in given section */
		static void ForEachChannelToKey(
			const UControlRig* ControlRig,
			const FName& ControlName,
			const UMovieSceneControlRigParameterSection* Section,
			EControlRigContextChannelToKey ChannelToKey,
			const TFunctionRef<void(FMovieSceneFloatChannel*)> Function)
		{
			const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ControlName) : nullptr;
			if (!ControlElement)
			{
				return;
			}

			const TArrayView<FMovieSceneFloatChannel*> FloatChannels = FControlRigSequencerHelpers::GetFloatChannels(ControlRig, ControlName, Section);

			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			case ERigControlType::Vector2D:
			{
				for (FMovieSceneFloatChannel* Channel : FloatChannels)
				{
					if (Channel)
					{
						Function(Channel);
					}
				}
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Transform:
			case ERigControlType::TransformNoScale:
			case ERigControlType::EulerTransform:
			{
				const TArray<EControlRigContextChannelToKey, TFixedAllocator<9>> TransformLayout =
				{
					EControlRigContextChannelToKey::TranslationX,
					EControlRigContextChannelToKey::TranslationY,
					EControlRigContextChannelToKey::TranslationZ,
					EControlRigContextChannelToKey::RotationX,
					EControlRigContextChannelToKey::RotationY,
					EControlRigContextChannelToKey::RotationZ,
					EControlRigContextChannelToKey::ScaleX,
					EControlRigContextChannelToKey::ScaleY,
					EControlRigContextChannelToKey::ScaleZ
				};

				for (int32 ChannelIndex = 0; ChannelIndex < FloatChannels.Num() && ChannelIndex < TransformLayout.Num(); ChannelIndex++)
				{
					if (EnumHasAnyFlags(ChannelToKey, TransformLayout[ChannelIndex]) &&
						FloatChannels[ChannelIndex])
					{
						Function(FloatChannels[ChannelIndex]);
					}
				}
				break;
			}
			case ERigControlType::Rotator:
			{
				const TArray<EControlRigContextChannelToKey, TFixedAllocator<3>>  RotatorLayout =
				{
					EControlRigContextChannelToKey::RotationX,
					EControlRigContextChannelToKey::RotationY,
					EControlRigContextChannelToKey::RotationZ,
				};

				for (int32 ChannelIndex = 0; ChannelIndex < FloatChannels.Num() && ChannelIndex < RotatorLayout.Num(); ChannelIndex++)
				{
					if (EnumHasAnyFlags(ChannelToKey, RotatorLayout[ChannelIndex]) &&
						FloatChannels[ChannelIndex])
					{
						Function(FloatChannels[ChannelIndex]);
					}
				}
				break;
			}
			case ERigControlType::Scale:
			{
				const TArray<EControlRigContextChannelToKey, TFixedAllocator<3>> ScaleLayout =
				{
					EControlRigContextChannelToKey::ScaleX,
					EControlRigContextChannelToKey::ScaleY,
					EControlRigContextChannelToKey::ScaleZ,
				};

				for (int32 ChannelIndex = 0; ChannelIndex < FloatChannels.Num() && ChannelIndex < 3; ++ChannelIndex)
				{
					if (EnumHasAnyFlags(ChannelToKey, ScaleLayout[ChannelIndex]) &&
						FloatChannels[ChannelIndex])
					{
						Function(FloatChannels[ChannelIndex]);
					}
				}
				break;
			}
			default:
				break;
			}
		}

		/** Returns the keyed status of a channel */
		static EPropertyKeyedStatus GetChannelKeyStatus(
			FMovieSceneChannel* InChannel, 
			EPropertyKeyedStatus InSectionKeyedStatus, 
			const TRange<FFrameNumber>& InRange, 
			int32& OutEmptyChannelCount)
		{
			if (!InChannel)
			{
				return InSectionKeyedStatus;
			}

			if (InChannel->GetNumKeys() == 0)
			{
				++OutEmptyChannelCount;
				return InSectionKeyedStatus;
			}

			InSectionKeyedStatus = FMath::Max(InSectionKeyedStatus, EPropertyKeyedStatus::KeyedInOtherFrame);

			TArray<FFrameNumber> KeyTimes;
			InChannel->GetKeys(InRange, &KeyTimes, nullptr);
			if (KeyTimes.IsEmpty())
			{
				++OutEmptyChannelCount;
			}
			else
			{
				InSectionKeyedStatus = FMath::Max(InSectionKeyedStatus, EPropertyKeyedStatus::PartiallyKeyed);
			}

			return InSectionKeyedStatus;
		}

		/** Returns the value of a key in a movie scene channel, given the value type is the value */
		template<typename TValueType>
		static std::enable_if_t<!std::is_class_v<TValueType>, TValueType>
			UnwrapKeyElement(TValueType Elem)
		{
			return Elem;
		}

		/** Returns the value of a key in a movie scene channel, given the value type is a value struct */
		template<typename TValueStructType>
		static std::enable_if_t<std::is_class_v<TValueStructType>, decltype(std::declval<TValueStructType>().Value)>
			UnwrapKeyElement(const TValueStructType& Elem)
		{
			return Elem.Value;
		}

		/** Deduces value or value struct type of a movie scene channel */
		template<typename TMovieSceneChannel>
		using TValueOrStructType = decltype(UnwrapKeyElement(std::declval<const TMovieSceneChannel>().GetValues()[0]));

		/** Returns first key value of a movie scene channel */
		template<typename TMovieSceneChannel> requires std::is_base_of_v<FMovieSceneChannel, TMovieSceneChannel>
		static TValueOrStructType<TMovieSceneChannel> FirstKeyValue(const TMovieSceneChannel& Channel)
		{
			return Channel.GetValues().IsEmpty() ?
				TValueOrStructType<TMovieSceneChannel>{} :
				UnwrapKeyElement(Channel.GetValues()[0]);
		}

		/** Removes all keys from channel, deduces and preserves default value from keys or resets to 0 if no keys are present */
		template<typename TMovieSceneChannel>
		static void RemoveKeysAndPreserveDefault(TMovieSceneChannel& Channel)
		{
			TArray<FFrameNumber> Times;
			TArray<FKeyHandle> Handles;
			Channel.GetKeys(TRange<FFrameNumber>::All(), &Times, &Handles);
			if (Handles.IsEmpty())
			{
				return;
			}

			Channel.SetDefault(FirstKeyValue(Channel));
			Channel.DeleteKeys(Handles);
		}

		/** Returns the keyed status of channel to key in given control rig section */
		static EPropertyKeyedStatus GetKeyedStatusInControlRigSection(
			const UControlRig* ControlRig, 
			const FName& ControlName, 
			const UMovieSceneControlRigParameterSection* Section, 
			const TRange<FFrameNumber>& Range, 
			const EControlRigContextChannelToKey ChannelToKey)
		{
			int32 EmptyChannelCount = 0;
			EPropertyKeyedStatus SectionKeyedStatus = EPropertyKeyedStatus::NotKeyed;
			
			const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ControlName) : nullptr;
			if (!ControlElement)
			{
				return SectionKeyedStatus;
			}
			
			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Bool:
			{
				const TArrayView<FMovieSceneBoolChannel*> BoolChannels = 
					FControlRigSequencerHelpers::GetBoolChannels(ControlRig, ControlElement->GetKey().Name, Section);
				for (FMovieSceneChannel* Channel : BoolChannels)
				{
					SectionKeyedStatus = GetChannelKeyStatus(Channel, SectionKeyedStatus, Range, EmptyChannelCount);
				}

				break;
			}
			case ERigControlType::Integer:
			{
				const TArrayView<FMovieSceneIntegerChannel*> IntegerChannels = 
					FControlRigSequencerHelpers::GetIntegerChannels(ControlRig, ControlElement->GetKey().Name, Section);
				for (FMovieSceneChannel* Channel : IntegerChannels)
				{
					SectionKeyedStatus = GetChannelKeyStatus(Channel, SectionKeyedStatus, Range, EmptyChannelCount);
				}

				const TArrayView<FMovieSceneByteChannel*>  EnumChannels = 
					FControlRigSequencerHelpers::GetByteChannels(ControlRig, ControlElement->GetKey().Name, Section);
				for (FMovieSceneChannel* Channel : EnumChannels)
				{
					SectionKeyedStatus = GetChannelKeyStatus(Channel, SectionKeyedStatus, Range, EmptyChannelCount);
				}

				break;
			}
			default:
				ForEachChannelToKey(ControlRig, ControlName, Section, ChannelToKey,
					[&](FMovieSceneFloatChannel* Channel)
					{
						SectionKeyedStatus = GetChannelKeyStatus(Channel, SectionKeyedStatus, Range, EmptyChannelCount);
					});
				break;
			}

			if (EmptyChannelCount == 0 && 
				SectionKeyedStatus == EPropertyKeyedStatus::PartiallyKeyed)
			{
				SectionKeyedStatus = EPropertyKeyedStatus::KeyedInFrame;
			}

			return SectionKeyedStatus;
		}

		/** Returns the keyed status of channel to key in given control rig track */
		static EPropertyKeyedStatus GetKeyedStatusInControlRigTrack(
			ISequencer& Sequencer,
			const UControlRig* ControlRig, 
			const FName& ControlName, 
			const UMovieSceneControlRigParameterTrack* Track, 
			const TRange<FFrameNumber>& Range,
			const EControlRigContextChannelToKey ChannelToKey)
		{
			EPropertyKeyedStatus SectionKeyedStatus = EPropertyKeyedStatus::NotKeyed;

			const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ControlName) : nullptr;
			if (!ControlElement)
			{
				return SectionKeyedStatus;
			}

			// Calling GetAnimLayers without testing HasAnimLayers will create Anim Layers in current version, avoid this
			TArray<UMovieSceneSection*> SectionsToKey;
			UAnimLayers* AnimLayers = UAnimLayers::HasAnimLayers(&Sequencer) ?
				UAnimLayers::GetAnimLayers(&Sequencer) :
				nullptr;
			if (AnimLayers)
			{
				SectionsToKey = AnimLayers->GetSelectedLayerSections();
			}

			// If there's no layer selected, evaluate for the base layer only
			if (SectionsToKey.IsEmpty())
			{
				const TArray<UMovieSceneSection*>& AllSections = Track->GetAllSections();
				if (AllSections.IsEmpty())
				{
					return SectionKeyedStatus;
				}
				UMovieSceneSection* const SectionToKey = Track->GetSectionToKey(ControlElement->GetFName());
				UMovieSceneSection* const BaseSection = SectionToKey ? SectionToKey : AllSections[0];

				SectionsToKey.AddUnique(BaseSection);
			}

			for (const UMovieSceneSection* Section : SectionsToKey)
			{
				const UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(Section);
				if (!ParameterSection)
				{
					continue;
				}

				const EPropertyKeyedStatus NewSectionKeyedStatus = GetKeyedStatusInControlRigSection(ControlRig, ControlName, ParameterSection, Range, ChannelToKey);
				SectionKeyedStatus = FMath::Max(SectionKeyedStatus, NewSectionKeyedStatus);

				// Maximum Status Reached no need to iterate further
				if (SectionKeyedStatus == EPropertyKeyedStatus::KeyedInFrame)
				{
					return SectionKeyedStatus;
				}
			}

			return SectionKeyedStatus;
		}

		/** Removes all keys in given control rig section */
		static void RemoveAllKeysInControlRigSection(
			const UControlRig* ControlRig,
			const FName& ControlName,
			UMovieSceneControlRigParameterSection* Section,
			EControlRigContextChannelToKey ChannelToKey)
		{
			if (!Section)
			{
				return;
			}

			const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ControlName) : nullptr;
			if (!ControlElement)
			{
				return;
			}

			Section->Modify();

			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Bool:
			{
				for (FMovieSceneBoolChannel* Channel : FControlRigSequencerHelpers::GetBoolChannels(ControlRig, ControlName, Section))
				{
					if (Channel)
					{
						RemoveKeysAndPreserveDefault(*Channel);
					}
				}
				break;
			}
			case ERigControlType::Integer:
			{
				for (FMovieSceneIntegerChannel* Channel : FControlRigSequencerHelpers::GetIntegerChannels(ControlRig, ControlName, Section))
				{
					if (Channel)
					{
						RemoveKeysAndPreserveDefault(*Channel);
					}
				}
				for (FMovieSceneByteChannel* Channel : FControlRigSequencerHelpers::GetByteChannels(ControlRig, ControlName, Section))
				{
					if (Channel)
					{
						RemoveKeysAndPreserveDefault(*Channel);
					}
				}
				break;
			}
			default:
				ForEachChannelToKey(ControlRig, ControlName, Section, ChannelToKey,
					[](FMovieSceneFloatChannel* Channel)
					{
						if (Channel)
						{
							RemoveKeysAndPreserveDefault(*Channel);
						}
					});
				break;
			}
		}

		/** Removes all keys in given control rig track */
		static void RemoveAllKeysInControlRigTrack(
			ISequencer& Sequencer,
			const UControlRig* ControlRig,
			const FName& ControlName,
			const UMovieSceneControlRigParameterTrack* Track,
			EControlRigContextChannelToKey ChannelToKey)
		{
			const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ControlName) : nullptr;
			if (!ControlElement)
			{
				return;
			}

			TArray<UMovieSceneSection*> SectionsToReset;
			UAnimLayers* AnimLayers = UAnimLayers::HasAnimLayers(&Sequencer) ?
				UAnimLayers::GetAnimLayers(&Sequencer) :
				nullptr;
			if (AnimLayers)
			{
				SectionsToReset = AnimLayers->GetSelectedLayerSections();
			}
			if (SectionsToReset.IsEmpty())
			{
				const TArray<UMovieSceneSection*>& AllSections = Track->GetAllSections();
				if (AllSections.IsEmpty())
				{
					return;
				}
				UMovieSceneSection* const SectionToKey = Track->GetSectionToKey(ControlElement->GetFName());
				UMovieSceneSection* const BaseSection = SectionToKey ? SectionToKey : AllSections[0];
				SectionsToReset.AddUnique(BaseSection);
			}

			for (UMovieSceneSection* Section : SectionsToReset)
			{
				if (UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(Section))
				{
					RemoveAllKeysInControlRigSection(ControlRig, ControlName, ParameterSection, ChannelToKey);
				}
			}
		}

		/** Returns the keyed status of channel to key in section */
		static EPropertyKeyedStatus GetKeyedStatusInPropertySection(
			const UMovieSceneSection* Section, 
			const TRange<FFrameNumber>& Range, 
			const EControlRigContextChannelToKey ChannelToKey, 
			const int32 MaxNumIndices)
		{
			EPropertyKeyedStatus SectionKeyedStatus = EPropertyKeyedStatus::NotKeyed;

			FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();

			const TArray<int32, TFixedAllocator<9>> ChannelIndices = ChannelIndicesFromMask(ChannelToKey);

			TSet<EPropertyKeyedStatus> PerChannelStatuses;
			for (const FMovieSceneChannelEntry& ChannelEntry : ChannelProxy.GetAllEntries())
			{
				if (ChannelEntry.GetChannelTypeName() != FMovieSceneDoubleChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneFloatChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneBoolChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneIntegerChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneByteChannel::StaticStruct()->GetFName())
				{
					continue;
				}

				const TConstArrayView<FMovieSceneChannel*> Channels = ChannelEntry.GetChannels();

				int32 ChannelIndex = 0;
				for (FMovieSceneChannel* Channel : Channels)
				{
					if (ChannelIndex >= MaxNumIndices)
					{
						break;
					}
					else if (ChannelIndices.Contains(ChannelIndex++) == false)
					{
						continue;
					}

					const int32 NumKeys = Channel->GetNumKeys();
					if (NumKeys == 0)
					{
						PerChannelStatuses.Add(EPropertyKeyedStatus::NotKeyed);
						continue;
					}

					TArray<FFrameNumber> KeyTimesInRange;
					Channel->GetKeys(Range, &KeyTimesInRange, nullptr);

					if (KeyTimesInRange.IsEmpty() && NumKeys > 0)
					{
						PerChannelStatuses.Add(EPropertyKeyedStatus::KeyedInOtherFrame);

						SectionKeyedStatus = FMath::Max(SectionKeyedStatus, EPropertyKeyedStatus::KeyedInOtherFrame);
					}
					else
					{
						PerChannelStatuses.Add(EPropertyKeyedStatus::KeyedInFrame);

						SectionKeyedStatus = EPropertyKeyedStatus::KeyedInFrame;
					}
				}

				break; // just do it for one type
			}

			// For structs, show partially keyed when some but not all channels are keyed this frame
			const bool bStructWithPerChannelStatuses = PerChannelStatuses.Num() > 1;
			if (bStructWithPerChannelStatuses)
			{
				const bool bKeyedInFrame = PerChannelStatuses.Contains(EPropertyKeyedStatus::KeyedInFrame);
				const bool bKeyedInOtherFrame = PerChannelStatuses.Contains(EPropertyKeyedStatus::KeyedInOtherFrame);
				const bool bNotKeyed = PerChannelStatuses.Contains(EPropertyKeyedStatus::NotKeyed);

				if (bKeyedInFrame && (bKeyedInOtherFrame || bNotKeyed))
				{
					SectionKeyedStatus = EPropertyKeyedStatus::PartiallyKeyed;
				}
			}

			return SectionKeyedStatus;
		}

		/** Returns the keyed status in given property track (non control rig) */
		static EPropertyKeyedStatus GetKeyedStatusInPropertyTrack(
			ISequencer& Sequencer,
			const UMovieScenePropertyTrack* Track, 
			const TRange<FFrameNumber>& Range, 
			const EControlRigContextChannelToKey ChannelToKey, 
			const int32 MaxNumIndices)
		{
			EPropertyKeyedStatus SectionKeyedStatus = EPropertyKeyedStatus::NotKeyed;
			for (UMovieSceneSection* BaseSection : Track->GetAllSections())
			{
				if (!BaseSection)
				{
					continue;
				}
				const EPropertyKeyedStatus NewSectionKeyedStatus = GetKeyedStatusInPropertySection(BaseSection, Range, ChannelToKey, MaxNumIndices);
				SectionKeyedStatus = FMath::Max(SectionKeyedStatus, NewSectionKeyedStatus);

				// Maximum Status Reached no need to iterate further
				if (SectionKeyedStatus == EPropertyKeyedStatus::KeyedInFrame)
				{
					return SectionKeyedStatus;
				}
			}

			return SectionKeyedStatus;
		}

		/** Removes all keys in given property section (non control rig) */
		static void RemoveAllKeysInPropertySection(
			UMovieSceneSection* Section,
			EControlRigContextChannelToKey ChannelToKey)
		{
			if (!Section)
			{
				return;
			}

			const TArray<int32, TFixedAllocator<9>> ChannelIndices = ChannelIndicesFromMask(ChannelToKey);
			if (ChannelIndices.IsEmpty())
			{
				return;
			}

			Section->Modify();

			FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
			for (const FMovieSceneChannelEntry& ChannelEntry : ChannelProxy.GetAllEntries())
			{
				if (ChannelEntry.GetChannelTypeName() != FMovieSceneDoubleChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneFloatChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneBoolChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneIntegerChannel::StaticStruct()->GetFName() &&
					ChannelEntry.GetChannelTypeName() != FMovieSceneByteChannel::StaticStruct()->GetFName())
				{
					continue;
				}

				const FName TypeName = ChannelEntry.GetChannelTypeName();
				const TConstArrayView<FMovieSceneChannel*> Channels = ChannelEntry.GetChannels();
				for (int32 ChannelIndex = 0; ChannelIndex < Channels.Num(); ChannelIndex++)
				{
					if (!ChannelIndices.Contains(ChannelIndex))
					{
						continue;
					}

					FMovieSceneChannel* const Channel = Channels[ChannelIndex];
					if (!Channel)
					{
						continue;
					}

					if (TypeName == FMovieSceneFloatChannel::StaticStruct()->GetFName())
					{
						RemoveKeysAndPreserveDefault(static_cast<FMovieSceneFloatChannel&>(*Channel));
					}
					else if (TypeName == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
					{
						RemoveKeysAndPreserveDefault(static_cast<FMovieSceneDoubleChannel&>(*Channel));
					}
					else if (TypeName == FMovieSceneBoolChannel::StaticStruct()->GetFName())
					{
						RemoveKeysAndPreserveDefault(static_cast<FMovieSceneBoolChannel&>(*Channel));
					}
					else if (TypeName == FMovieSceneIntegerChannel::StaticStruct()->GetFName())
					{
						RemoveKeysAndPreserveDefault(static_cast<FMovieSceneIntegerChannel&>(*Channel));
					}
					else if (TypeName == FMovieSceneByteChannel::StaticStruct()->GetFName())
					{
						RemoveKeysAndPreserveDefault(static_cast<FMovieSceneByteChannel&>(*Channel));
					}
				}
			}
		}

		/** Removes all keys in given property track (non control rig) */
		static void RemoveAllKeysInPropertyTrack(
			UMovieScenePropertyTrack* Track,
			EControlRigContextChannelToKey ChannelToKey)
		{
			for (UMovieSceneSection* Section : Track->GetAllSections())
			{
				RemoveAllKeysInPropertySection(Section, ChannelToKey);
			}
		}

		/** Adds a key to the specified track */
		static void KeyTrack(const TSharedPtr<ISequencer>& Sequencer, UAnimDetailsProxyBase* Proxy, UMovieScenePropertyTrack* Track, EControlRigContextChannelToKey ChannelToKey)
		{
			using namespace UE::Sequencer;

			if (!Sequencer.IsValid() || !Proxy || !Track)
			{
				return;
			}

			const FFrameNumber Time = Sequencer->GetLocalTime().Time.FloorToFrame();
			float Weight = 0.0;

			UMovieSceneSection* Section = Track->FindOrExtendSection(Time, Weight);

			FScopedTransaction PropertyChangedTransaction(NSLOCTEXT("AnimDetailsProxyBase", "KeyProperty", "Key Property"), !GIsTransacting);
			if (!Section || !Section->TryModify())
			{
				PropertyChangedTransaction.Cancel();
				return;
			}

			const TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();
			const FViewModelPtr RootModel = EditorViewModel.IsValid() ? EditorViewModel->GetRootModel() : nullptr;
			const FSectionModelStorageExtension* SectionModelStorage = RootModel.IsValid() ? RootModel->CastDynamic<FSectionModelStorageExtension>() : nullptr;
			const TSharedPtr<FSectionModel> SectionHandle = SectionModelStorage ? SectionModelStorage->FindModelForSection(Section) : nullptr;
			const TSharedPtr<FViewModel> ViewModel = SectionHandle.IsValid() ? SectionHandle->GetParentSectionOwnerModel().AsModel() : nullptr;
			if (!EditorViewModel.IsValid() ||
				!RootModel.IsValid() ||
				!SectionModelStorage || 
				!SectionHandle.IsValid() || 
				!ViewModel.IsValid())
			{
				return;
			}

			TArray<TSharedRef<IKeyArea>> KeyAreas;
			const TParentFirstChildIterator<FChannelGroupModel> KeyAreaNodes = ViewModel->GetDescendantsOfType<FChannelGroupModel>();
			
			for (const TViewModelPtr<FChannelGroupModel>& KeyAreaNode : KeyAreaNodes)
			{
				for (const TWeakViewModelPtr<FChannelModel>& Channel : KeyAreaNode->GetChannels())
				{
					if (const TSharedPtr<FChannelModel> ChannelModel = Channel.Pin())
					{
						const EControlRigContextChannelToKey ThisChannelToKey = Proxy->GetChannelToKeyFromChannelName(ChannelModel->GetChannelName().ToString());
						if ((int32)ChannelToKey & (int32)ThisChannelToKey)
						{
							KeyAreas.Add(ChannelModel->GetKeyArea().ToSharedRef());
						}
					}
				}
			}

			const TViewModelPtr<ITrackExtension> TrackModel = SectionHandle->FindAncestorOfType<ITrackExtension>();
			FAddKeyOperation::FromKeyAreas(TrackModel->GetTrackEditor().Get(), KeyAreas).Commit(Time, *Sequencer);
		}
	}
}

void UAnimDetailsProxyBase::SetControlFromControlRig(UControlRig* InControlRig, const FName& InName)
{
	SequencerItem.Reset();

	URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (Hierarchy)
	{
		WeakControlRig = InControlRig;
		CachedRigElement = FCachedRigElement(FRigElementKey(InName, ERigElementType::Control), Hierarchy);

		// Test if this yields a valid proxy
		const FRigControlElement* ControlElement = GetControlElement();
		const bool bValidControlElement = ControlElement && GetSupportedControlTypes().Contains(ControlElement->Settings.ControlType);
		
		if (!ensureMsgf(bValidControlElement, TEXT("Created invalid anim details proxy, control element is invalid, or control type does not match proxy type")))
		{
			WeakControlRig.Reset();
			CachedRigElement.Reset();
		}
	}
}

void UAnimDetailsProxyBase::SetControlFromSequencerBinding(UObject* InObject, const TWeakObjectPtr<UMovieSceneTrack>& InTrack, const TSharedPtr<FTrackInstancePropertyBindings>& InBinding)
{
	WeakControlRig.Reset();
	CachedRigElement.Reset();

	if (InObject &&
		InTrack.IsValid() &&
		InBinding.IsValid())
	{
		SequencerItem = FAnimDetailsSequencerProxyItem(*InObject, *InTrack.Get(), InBinding.ToSharedRef());
	}
	else
	{
		SequencerItem.Reset();
	}
}

UControlRig* UAnimDetailsProxyBase::GetControlRig() const
{
	return WeakControlRig.Get();
}

FRigControlElement* UAnimDetailsProxyBase::GetControlElement() const
{
	const URigHierarchy* Hierarchy = WeakControlRig.IsValid() ? WeakControlRig->GetHierarchy() : nullptr;
	const FRigControlElement* ControlElement = Hierarchy ? Cast<FRigControlElement>(CachedRigElement.GetElement(Hierarchy)) : nullptr;

	// @todo There is no specific reason to prevent from getting a non const ptr from FCachedRigElement. 
	// The related change turns out relatively large, so delay it for later and const cast here.
	return const_cast<FRigControlElement*>(ControlElement);
}

const FRigElementKey& UAnimDetailsProxyBase::GetControlElementKey() const
{
	return CachedRigElement.GetKey();
}

const FName& UAnimDetailsProxyBase::GetControlName() const
{
	return GetControlElementKey().Name;
}

void UAnimDetailsProxyBase::PropagateValues()
{
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::DoNotCare;

	FRigControlModifiedContext NotifyDrivenContext;

	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	Controller.EvaluateAllConstraints();

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		SetControlRigElementValueFromCurrent(Context);
		FControlRigEditMode::NotifyDrivenControls(ControlRig, ControlElement->GetKey(), NotifyDrivenContext);
			
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		SetSequencerBindingValueFromCurrent(Context);
	}
}

FText UAnimDetailsProxyBase::GetDisplayNameText(const EElementNameDisplayMode ElementNameDisplayMode) const
{
	if(!DisplayName.IsEmpty())
	{
		return FText::FromString(DisplayName);
	}
	
	const URigHierarchy* Hierarchy = WeakControlRig.IsValid() ? WeakControlRig->GetHierarchy() : nullptr;
	const FRigControlElement* ControlElement = WeakControlRig.IsValid() ? GetControlElement() : nullptr;
	if (Hierarchy && ControlElement)
	{
		return Hierarchy->GetDisplayNameForUI(ControlElement, ElementNameDisplayMode);
	}
	else if (const UObject* BoundObject = SequencerItem.GetBoundObject())
	{
		if (const AActor* Actor = Cast<AActor>(BoundObject))
		{
			return FText::FromString(Actor->GetActorLabel());
		}
		else if (const UActorComponent* Component = Cast<UActorComponent>(BoundObject))
		{
			return FText::FromString(*Component->GetName());
		}
	}

	return FText::GetEmpty();
}

void UAnimDetailsProxyBase::SetKey(const IPropertyHandle& KeyedPropertyHandle)
{
	using namespace UE::ControlRigEditor;

	const UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>();
	const TSharedPtr<ISequencer> Sequencer = ProxyManager ? ProxyManager->GetSequencer() : nullptr;
	if (!Sequencer.IsValid() || !Sequencer->GetFocusedMovieSceneSequence())
	{
		return;
	}

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();

	if (ControlRig && ControlElement)
	{
		if (ControlRig->GetHierarchy()->Contains(FRigElementKey(ControlElement->GetKey().Name, ERigElementType::Control)))
		{
			const FName PropertyName = KeyedPropertyHandle.GetProperty()->GetFName();

			FRigControlModifiedContext Context;
			Context.SetKey = EControlRigSetKey::Always;
			Context.KeyMask = (uint32)GetChannelToKeyFromPropertyName(PropertyName);

			SetControlRigElementValueFromCurrent(Context);

			ControlRig->Evaluate_AnyThread();

			FRigControlModifiedContext NotifyDrivenContext; // always key ever
			NotifyDrivenContext.SetKey = EControlRigSetKey::Always;

			FControlRigEditMode::NotifyDrivenControls(ControlRig, ControlElement->GetKey(), NotifyDrivenContext);
		}
	}
	else if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(SequencerItem.GetMovieSceneTrack()))
	{
		const FName PropertyName = KeyedPropertyHandle.GetProperty()->GetFName();
		const EControlRigContextChannelToKey ChannelToKey = GetChannelToKeyFromPropertyName(PropertyName);

		KeyUtils::KeyTrack(Sequencer, this, PropertyTrack, ChannelToKey);
	}
}

void UAnimDetailsProxyBase::RemoveAllKeys(const IPropertyHandle& KeyedPropertyHandle)
{
	using namespace UE::ControlRigEditor;

	const UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>();
	const TSharedPtr<ISequencer> Sequencer = ProxyManager ? ProxyManager->GetSequencer() : nullptr;
	if (!Sequencer.IsValid() || !Sequencer->GetFocusedMovieSceneSequence())
	{
		return;
	}

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();

	if (ControlRig && ControlElement)
	{
		if (ControlRig->GetHierarchy()->Contains(FRigElementKey(ControlElement->GetKey().Name, ERigElementType::Control)))
		{
			const FName PropertyName = KeyedPropertyHandle.GetProperty()->GetFName();
			const EControlRigContextChannelToKey ChannelToKey = GetChannelToKeyFromPropertyName(PropertyName);

			UMovieSceneControlRigParameterTrack* CRTrack =
				FControlRigSequencerHelpers::FindControlRigTrack(Sequencer->GetFocusedMovieSceneSequence(), ControlRig);
			if (CRTrack)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AnimDetailsProxyBase", "RemoveAllKeys", "Remove All Keys"), !GIsTransacting);
				KeyUtils::RemoveAllKeysInControlRigTrack(*Sequencer, ControlRig, ControlElement->GetFName(), CRTrack, ChannelToKey);
			}
		}
	}
	else if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(SequencerItem.GetMovieSceneTrack()))
	{
		const FName PropertyName = KeyedPropertyHandle.GetProperty()->GetFName();
		const EControlRigContextChannelToKey ChannelToKey = GetChannelToKeyFromPropertyName(PropertyName);

		FScopedTransaction Transaction(NSLOCTEXT("AnimDetailsProxyBase", "RemoveAllKeys", "Remove All Keys"), !GIsTransacting);
		KeyUtils::RemoveAllKeysInPropertyTrack(PropertyTrack, ChannelToKey);
	}
}

EPropertyKeyedStatus UAnimDetailsProxyBase::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	using namespace UE::ControlRigEditor;

	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	const UAnimDetailsProxyManager* ProxyManager = GetTypedOuter<UAnimDetailsProxyManager>();
	const TSharedPtr<ISequencer> Sequencer = ProxyManager ? ProxyManager->GetSequencer() : nullptr;
	if (!Sequencer.IsValid() || !Sequencer->GetFocusedMovieSceneSequence())
	{
		return KeyedStatus;
	}

	const UControlRig* ControlRig = GetControlRig();
	const URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
	const FRigElementKey ControlElementKey = GetControlElementKey();

	const TRange<FFrameNumber> FrameRange = TRange<FFrameNumber>(Sequencer->GetLocalTime().Time.FrameNumber);
	const FName PropertyName = PropertyHandle.GetProperty()->GetFName();
	const EControlRigContextChannelToKey ChannelToKey = GetChannelToKeyFromPropertyName(PropertyName);

	UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = ControlRig ? FControlRigSequencerHelpers::FindControlRigTrack(Sequencer->GetFocusedMovieSceneSequence(), ControlRig) : nullptr;
	if (ControlRig && ControlElementKey.IsValid() && ControlRigParameterTrack)
	{
		const EPropertyKeyedStatus NewKeyedStatus = KeyUtils::GetKeyedStatusInControlRigTrack(*Sequencer.Get(), ControlRig, ControlElementKey.Name, ControlRigParameterTrack, FrameRange, ChannelToKey);
		KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);
	}
	else if (UMovieScenePropertyTrack* MovieScenePropertyTrack = Cast<UMovieScenePropertyTrack>(SequencerItem.GetMovieSceneTrack()))
	{
		int32 MaxNumIndices = 1;
		if (IsA<UAnimDetailsProxyTransform>())
		{
			MaxNumIndices = 9;
		}
		else if (IsA<UAnimDetailsProxyVector2D>())
		{
			MaxNumIndices = 2;
		}

		const EPropertyKeyedStatus NewKeyedStatus = KeyUtils::GetKeyedStatusInPropertyTrack(*Sequencer.Get(), MovieScenePropertyTrack, FrameRange, ChannelToKey, MaxNumIndices);
		KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);
	}

	return KeyedStatus;
}

FName UAnimDetailsProxyBase::GetDetailRowID() const
{
	if (bIsIndividual)
	{
		const UControlRig* ControlRig = GetControlRig();
		const URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
		const FRigControlElement* ControlElement = GetControlElement();
		if (ControlRig && Hierarchy && ControlElement)
		{
			return *Hierarchy->GetDisplayNameForUI(ControlElement->GetKey(), EElementNameDisplayMode::ForceShort).ToString();
		}
		else if (const FProperty* Property = SequencerItem.GetProperty())
		{
			return *Property->GetPathName();
		}
		else
		{
			return NAME_None;
		}
	}
	else
	{
		return GetClass()->GetFName();
	}
}

void UAnimDetailsProxyBase::GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const
{
	// Proxies with one member in their struct only show inner properties
	OutOptionalStructDisplayName.Reset();

	const URigHierarchy* Hierarchy = WeakControlRig.IsValid() ? WeakControlRig->GetHierarchy() : nullptr;
	const FRigControlElement* ControlElement = WeakControlRig.IsValid() ? GetControlElement() : nullptr;
	if (Hierarchy && ControlElement)
	{
		OutPropertyDisplayName = Hierarchy->GetDisplayNameForUI(ControlElement);
	}
	else if (const TSharedPtr<FTrackInstancePropertyBindings>& Binding = SequencerItem.GetBinding())
	{
		OutPropertyDisplayName = FText::FromName(SequencerItem.GetBinding()->GetPropertyName());
	}
}

void UAnimDetailsProxyBase::UpdateOverrideableProperties()
{
	if(const FRigControlElement* ControlElement = GetControlElement())
	{
		DisplayName = ControlElement->GetDisplayName().ToString();
		Shape.ConfigureFrom(ControlElement, ControlElement->Settings);
	}
}

FName UAnimDetailsProxyBase::GetPropertyID(const FName& PropertyName) const
{
	return *(GetDetailRowID().ToString() + TEXT(".") + PropertyName.ToString());
}

void UAnimDetailsProxyBase::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ToggleEditable) // hack so we can clear the reset cache for this property and not actually send this to our controls
	{
		return;
	}
	
	// Reset to default is handled in the anim details value customizations
	if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault)
	{
		return;
	}
	
	if (const FProperty* Property = PropertyChangedEvent.Property)
	{
		const FProperty* MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode() ? 
			PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue() : 
			nullptr;

		if (PropertyIsOnProxy(Property, MemberProperty))
		{

			// Let the control rig know it's interacted with
			EControlRigContextChannelToKey ChannelToKeyContext = GetChannelToKeyFromPropertyName(Property->GetFName());
			AddControlRigInteractionScope(ChannelToKeyContext, PropertyChangedEvent.ChangeType);

			// Propagate the values to the control rig
			PropagateValues();

			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				InteractionScopes.Reset();
			}

			// Adopt values from the rig or sequencer binding. 
			// They may be different than what was propagated, e.g. due to constraints.
			AdoptValues(ERigControlValueType::Current);
		}
	}
}

bool UAnimDetailsProxyBase::Modify(bool bAlwaysMarkDirty)
{
	// IPropertyHandle::SetPerObjectValues which the multi edit util uses always modifies the object, 
	// hence we avoid modificiation by testing for interactive changes here.
	if (!UE::ControlRigEditor::FAnimDetailsMultiEditUtil::Get().IsInteractive())
	{
		return Super::Modify(bAlwaysMarkDirty);
	}

	return true;
}

void UAnimDetailsProxyBase::AddControlRigInteractionScope(EControlRigContextChannelToKey ChannelsToKey, EPropertyChangeType::Type ChangeType)
{
	if (ChangeType == EPropertyChangeType::Interactive || ChangeType == EPropertyChangeType::ValueSet)
	{
		EControlRigInteractionType InteractionType = EControlRigInteractionType::None;
		if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX) ||
			EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY) ||
			EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ))
		{
			EnumAddFlags(InteractionType, EControlRigInteractionType::Translate);
		}
		if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX) ||
			EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY) ||
			EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ))
		{
			EnumAddFlags(InteractionType, EControlRigInteractionType::Rotate);
		}
		if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX) ||
			EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY) ||
			EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ))
		{
			EnumAddFlags(InteractionType, EControlRigInteractionType::Scale);
		}

		UControlRig* ControlRig = GetControlRig();
		FRigControlElement* ControlElement = GetControlElement();

		if (ControlRig && ControlElement && !InteractionScopes.Contains(ControlElement))
		{
			const TSharedRef<FControlRigInteractionScope> InteractionScope = MakeShared<FControlRigInteractionScope>(ControlRig, ControlElement->GetKey(), InteractionType);
			InteractionScopes.Add(ControlElement, InteractionScope);
		}
	}
}
