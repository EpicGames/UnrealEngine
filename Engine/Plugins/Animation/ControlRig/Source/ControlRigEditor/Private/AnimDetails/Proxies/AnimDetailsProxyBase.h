// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsSequencerProxyItem.h"
#include "Rigs/RigHierarchyCache.h"
#include "Misc/Optional.h"
#include "Units/Execution/RigUnit_DynamicHierarchy.h"

#include "AnimDetailsProxyBase.generated.h"

class FControlRigInteractionScope;
class FTrackInstancePropertyBindings;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UControlRig;
class UMovieSceneTrack;
enum class EPropertyKeyedStatus : uint8;

/** 
 * Base class for anim details proxies. 
 * Anim details proxies can handle a property bound in sequencer, and the related controls if the bound object uses a control rig.
 * 
 * This is a rewrite of what was previously UControlRigControlsProxy in ControlRigEditor/Private/EditMode/ControlRigControlsProxy.h.
 */
UCLASS(Abstract)
class UAnimDetailsProxyBase 
	: public UObject
{
	GENERATED_BODY()

	using FAnimDetailsSequencerProxyItem = UE::ControlRigEditor::FAnimDetailsSequencerProxyItem;

public:
	/** Sets the control for a control rig */
	void SetControlFromControlRig(UControlRig* InControlRig, const FName& InName);

	/** Sets the control for a sequencer binding */
	void SetControlFromSequencerBinding(UObject* InObject, const TWeakObjectPtr<UMovieSceneTrack>& InTrack, const TSharedPtr<FTrackInstancePropertyBindings>& InBinding);

	/** Returns the control rig this proxy handles, or nullptr if the control rig is invalid */
	UControlRig* GetControlRig() const;

	/** Returns the control element this proxy handles, or nullptr if the element is invalid. */
	FRigControlElement* GetControlElement() const;

	/** Returns a key to the control element this proxy handles. */
	const FRigElementKey& GetControlElementKey() const;

	/** Returns the name of the control element his proxy handles */
	const FName& GetControlName() const;

	/** Returns the sequencer binding, or nullptr if this proxy is not assigned to a seqeuncer object */
	const FAnimDetailsSequencerProxyItem& GetSequencerItem() { return SequencerItem; }

	/** Returns the sequencer binding, or nullptr if this proxy is not assigned to a seqeuncer object, const version */
	const FAnimDetailsSequencerProxyItem& GetSequencerItem() const { return SequencerItem; }

	/** Propagates the current proxy values to control rig or the bound sequencer object */
	void PropagateValues();

	/** Returns the display name for this proxy as text */
	FText GetDisplayNameText(const EElementNameDisplayMode ElementNameDisplayMode = EElementNameDisplayMode::Auto) const;

	/** Sets a key for this property in sequencer */
	void SetKey(const IPropertyHandle& KeyedPropertyHandle);

	/** Removes all keys for this property in sequencer */
	void RemoveAllKeys(const IPropertyHandle& KeyedPropertyHandle);

	/** Gets the keyed status of a property */
	EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const;

	/** Returns the category of the struct that holds the control type, for example float or transform */
	virtual FName GetCategoryName() const PURE_VIRTUAL(UAnimDetailsProxyBase::GetCategoryName, return NAME_None;)

	/**
	 * Returns an ID for the details row where this proxy should be presented. 
	 * Proxies that return the same detail row ID will be multi-edited.
	 *
	 * The detail row ID does not relate to any other engine logic, it is specific to anim details proxies.
	 */
	virtual FName GetDetailRowID() const;

	/**
	 * Returns an ID for property consisting of the detail row ID and the property name.
	 * Properties that share the same property ID will be multiedited on the same property row.
	 * 
	 * The property ID does not relate to any other engine logic, it is specific to anim details proxies.
	 */
	virtual FName GetPropertyID(const FName& PropertyName) const;

	/** Returns the property names in this proxy controls. Should be in order they're declared */
	virtual TArray<FName> GetPropertyNames() const PURE_VIRTUAL(UAnimDetailsProxyBase::GetPropertyNames, return {};)

	/** Returns the localized property name, useful for filtering. Currently transforms override this as they have more than one inner struct. */
	virtual void GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const;

	/** Returns the control types the proxy supports */
	virtual TSet<ERigControlType> GetSupportedControlTypes() const PURE_VIRTUAL(UAnimDetailsProxyBase::GetSupportedControlTypes, return {};);

	/** Returns true if the property is handled by this proxy */
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) PURE_VIRTUAL(UAnimDetailsProxyBase::PropertyIsOnProxy, return false;)

	/** Tries to adopt the current value(s) from the sequencer binding or the control in the control rig. */
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) PURE_VIRTUAL(UAnimDetailsProxyBase::AdoptValues, return;)

	/** Resets a property in the proxy to the initial value in the rig, or a zero value when the proxy relates to a sequencer binding. */
	virtual void ResetPropertyToDefault(const FName& PropertyName) PURE_VIRTUAL(UAnimDetailsProxyBase::ResetPropertyToDefault, return;)

	/** Returns true if the property value is the default value */
	virtual bool HasDefaultValue(const FName& PropertyName) const PURE_VIRTUAL(UAnimDetailsProxyBase::HasDefaultValue, return true;)

	/** Updates the proxy shape override properties such as DisplayName and shape */
	virtual void UpdateOverrideableProperties();

	/** Returns the channel to key flags from the property name */
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& PropertyName) const PURE_VIRTUAL(UAnimDetailsProxyBase::GetChannelToKeyFromPropertyName, return EControlRigContextChannelToKey::AllTransform;) 
	
	/** Returns the channel to key flags from the sequencer channel name */
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const PURE_VIRTUAL(UAnimDetailsProxyBase::GetChannelToKeyFromChannelName, return EControlRigContextChannelToKey::AllTransform;) 

	//~ Begin UObject interface
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	//~ End UObject interface

	/** If true shows on its own detail row, but is still multi edited with identical controls. If false, is multi-edited with proxies of same type */
	bool bIsIndividual = false;

	/** The control rig type to use for this proxy. Note this is used even if there is only a sequencer binding but no control rig */
	ERigControlType Type = ERigControlType::Transform;

	/** An overrideable display name for the control */
	UPROPERTY(EditAnywhere, Category = "Overrides")
	FString DisplayName;

	/** Overrideable shape settings for the control */
	UPROPERTY(EditAnywhere, Category = "Overrides", meta=(ExpandByDefault))
	FRigUnit_HierarchyAddControl_ShapeSettings Shape;	

protected:
	/** Sets the control rig element value from the current proxy value */
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) PURE_VIRTUAL(UAnimDetailsProxyBase::SetControlRigElementValueFromCurrent, return;)

	/** Sets the sequencer binding value from the current proxy value */
	virtual void SetSequencerBindingValueFromCurrent(const FRigControlModifiedContext & Context) {}

	/** The control rig that holds the control or nullptr if there is no control rig. */
	TWeakObjectPtr<UControlRig> WeakControlRig;

	/** The cached rig element that holds the control, or nullptr if there is no rig element. */
	mutable FCachedRigElement CachedRigElement;

	/** Item holding data about a sequencer binding */
	FAnimDetailsSequencerProxyItem SequencerItem;

private:
	/** Adds a control rig interaction scope to the interaction scopes array */
	void AddControlRigInteractionScope(EControlRigContextChannelToKey ChannelsToKey, EPropertyChangeType::Type ChangeType);

	/** Current interaction scopes */
	TMap<FRigControlElement*, TSharedRef<FControlRigInteractionScope>> InteractionScopes;
};
