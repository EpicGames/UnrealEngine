// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyBool.h"

#include "AnimDetailsScopedPropertyChangedEvent.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "Components/SceneComponent.h"
#include "ControlRig.h"
#include "DetailLayoutBuilder.h"
#include "GameFramework/Actor.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyBool)

namespace UE::ControlRigEditor::Private
{
	struct FBoolScopedPropertyPathChangedEvent
		: FNoncopyable
	{
		FBoolScopedPropertyPathChangedEvent(UObject& InObject, const FString& InPropertyPath, FProperty& InProperty)
			: Property(InProperty)
		{
			InitObjectsToNotify(InObject);
			InitPropertyChain(InObject, InPropertyPath);

			if (PropertyChain.Num() == 0)
			{
				PropertyChain.AddHead(&Property);
			}

			for (UObject* const ObjectToNotify : ObjectsToNotify)
			{
				ObjectToNotify->PreEditChange(PropertyChain);
			}
		}

		~FBoolScopedPropertyPathChangedEvent()
		{
			FPropertyChangedEvent PropertyChangedEvent(&Property, EPropertyChangeType::ValueSet, MakeArrayView(ObjectsToNotify));
			FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

			for (UObject* const ObjectToNotify : ObjectsToNotify)
			{
				ObjectToNotify->PostEditChangeChainProperty(PropertyChangedChainEvent);
			}
		}

	private:
		void InitObjectsToNotify(UObject& InObject)
		{
			ObjectsToNotify.Add(&InObject);

			AActor* const Actor = InObject.IsA(AActor::StaticClass()) ? Cast<AActor>(&InObject) : InObject.GetTypedOuter<AActor>();
			if (Actor)
			{
				ObjectsToNotify.AddUnique(Actor);

				if (USceneComponent* const RootComponent = Actor->GetRootComponent())
				{
					ObjectsToNotify.AddUnique(RootComponent);
				}
			}

			if (USceneComponent* const OuterSceneComponent = InObject.GetTypedOuter<USceneComponent>())
			{
				ObjectsToNotify.AddUnique(OuterSceneComponent);
			}
		}

		void InitPropertyChain(UObject& InObject, const FString& InPropertyPath)
		{
			TArray<FString> PropertyPathSegments;
			InPropertyPath.ParseIntoArray(PropertyPathSegments, TEXT("."), true);
			if (PropertyPathSegments.Num() <= 1)
			{
				return;
			}

			TArray<FProperty*> ResolvedProperties;
			ResolvedProperties.Reserve(PropertyPathSegments.Num());

			UStruct* Struct = InObject.GetClass();
			for (const FString& PropertyPathSegment : PropertyPathSegments)
			{
				if (!Struct)
				{
					return;
				}

				FString PropertyName = PropertyPathSegment;
				int32 ArrayIndexStart = INDEX_NONE;
				if (PropertyName.FindChar(TEXT('['), ArrayIndexStart))
				{
					PropertyName.LeftInline(ArrayIndexStart);
				}

				FProperty* const ResolvedProperty = FindFProperty<FProperty>(Struct, *PropertyName);
				if (!ResolvedProperty)
				{
					return;
				}

				ResolvedProperties.Add(ResolvedProperty);

				const FStructProperty* const StructProperty = CastField<FStructProperty>(ResolvedProperty);
				Struct = StructProperty ? StructProperty->Struct.Get() : nullptr;
			}

			if (ResolvedProperties.Last() != &Property)
			{
				return;
			}

			for (FProperty* const ResolvedProperty : ResolvedProperties)
			{
				PropertyChain.AddHead(ResolvedProperty);
			}
		}

		FProperty& Property;
		FEditPropertyChain PropertyChain;
		TArray<UObject*> ObjectsToNotify;
	};
}

FName UAnimDetailsProxyBool::GetCategoryName() const
{
	return "Bool";
}

TArray<FName> UAnimDetailsProxyBool::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsBool, Bool)
	};
}

TSet<ERigControlType> UAnimDetailsProxyBool::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Bool };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyBool::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimDetailsBool, Bool)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyBool, Bool));
}

void UAnimDetailsProxyBool::AdoptValues(const ERigControlValueType RigControlValueType)
{
	const bool Value = [RigControlValueType, this]()
		{
			UControlRig* ControlRig = GetControlRig();
			FRigControlElement* ControlElement = GetControlElement();
			if (ControlRig && ControlElement)
			{
				const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
				return ControlValue.Get<bool>();
			}
			else if (SequencerItem.IsValid())
			{
				const TOptional<bool> OptionalValue = SequencerItem.GetBinding()->GetOptionalValue<bool>(*SequencerItem.GetBoundObject());
				return OptionalValue.IsSet() ? OptionalValue.GetValue() : false;
			}

			return false;
		}();

	const FAnimDetailsBool AnimDetailProxyBool(Value);

	constexpr const TCHAR* PropertyName = TEXT("Bool");
	FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
	Binding.CallFunction<FAnimDetailsBool>(*this, AnimDetailProxyBool);
}

void UAnimDetailsProxyBool::ResetPropertyToDefault(const FName& PropertyName)
{
	if (const FRigControlElement* ControlElement = GetControlElement())
	{
		AdoptValues(ERigControlValueType::Initial);
	}
	else
	{
		Bool = FAnimDetailsBool();
	}
}

bool UAnimDetailsProxyBool::HasDefaultValue(const FName& PropertyName) const
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
		return Bool.Bool == ControlValue.Get<bool>();
	}

	return Bool.Bool == false;
}

EControlRigContextChannelToKey UAnimDetailsProxyBool::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsBool, Bool))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyBool::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Bool"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	const UControlRig* ControlRig = GetControlRig();
	const FRigControlElement* ControlElement = GetControlElement();
	if (ControlElement && ControlElement->GetDisplayName() == InChannelName)
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyBool::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlElement && ControlRig)
	{
		const bool Value = Bool.Bool;

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<bool>(ControlElement->GetKey().Name, Value, bNotify, Context, bSetupUndo);
	}
}

void UAnimDetailsProxyBool::SetSequencerBindingValueFromCurrent(const FRigControlModifiedContext& Context)
{
	const TSharedPtr<FTrackInstancePropertyBindings> Binding = SequencerItem.GetBinding();
	UObject* BoundObject = SequencerItem.GetBoundObject();
	if (Binding.IsValid() && BoundObject)
	{
		if (FProperty* Property = Binding->GetProperty(*BoundObject))
		{
			const UE::ControlRigEditor::Private::FBoolScopedPropertyPathChangedEvent ScopedPropertyChangedEvent(*BoundObject, Binding->GetPropertyPath(), *Property);

			Binding->SetCurrentValue<bool>(*BoundObject, Bool.Bool);
		}
	}
}
