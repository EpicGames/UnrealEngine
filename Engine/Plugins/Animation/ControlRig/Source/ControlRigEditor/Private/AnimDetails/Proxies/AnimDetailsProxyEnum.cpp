// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyEnum.h"

#include "AnimDetails/AnimDetailsSettings.h"
#include "ControlRig.h"
#include "DetailLayoutBuilder.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyEnum)

FName UAnimDetailsProxyEnum::GetCategoryName() const
{
	return "Enum";
}

FName UAnimDetailsProxyEnum::GetDetailRowID() const
{
	if (!bIsIndividual)
	{
		return Enum.EnumType ? Enum.EnumType->GetFName() : NAME_None;
	}

	return UAnimDetailsProxyBase::GetDetailRowID();
}

TArray<FName> UAnimDetailsProxyEnum::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsEnum, EnumIndex)
	};
}

TSet<ERigControlType> UAnimDetailsProxyEnum::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Integer };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyEnum::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimDetailsEnum, EnumIndex)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyEnum, Enum));
}

void UAnimDetailsProxyEnum::AdoptValues(const ERigControlValueType RigControlValueType)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig &&
		ControlElement &&
		ControlElement->Settings.ControlEnum)
	{
		UEnum& EnumType = *ControlElement->Settings.ControlEnum;

		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);

		FAnimDetailsEnum EnumValue;
		EnumValue.EnumType = &EnumType;
		EnumValue.EnumIndex = ControlValue.Get<int32>();

		constexpr const TCHAR* PropertyName = TEXT("Enum");
		FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
		Binding.CallFunction<FAnimDetailsEnum>(*this, EnumValue);
	}

	// Not implemented for sequencer items (altho they have an enum type @todo)
}

void UAnimDetailsProxyEnum::ResetPropertyToDefault(const FName& PropertyName)
{
	if (const FRigControlElement* ControlElement = GetControlElement())
	{
		AdoptValues(ERigControlValueType::Initial);
	}
	else
	{
		Enum.EnumIndex = 0;
	}
}

bool UAnimDetailsProxyEnum::HasDefaultValue(const FName& PropertyName) const
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
		return Enum.EnumIndex == ControlValue.Get<int32>();
	}

	return Enum.EnumIndex == 0;
}

EControlRigContextChannelToKey UAnimDetailsProxyEnum::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsEnum, EnumIndex))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyEnum::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("EnumIndex"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	const UControlRig* ControlRig = GetControlRig();
	const FRigElementKey ElementKey = GetControlElementKey();

	const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ElementKey.Name) : nullptr;
	if (ControlElement && ControlElement->GetDisplayName() == InChannelName)
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyEnum::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		int32 Val = Enum.EnumIndex;

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<int32>(ControlElement->GetKey().Name, Val, bNotify, Context, bSetupUndo);
	}
}
