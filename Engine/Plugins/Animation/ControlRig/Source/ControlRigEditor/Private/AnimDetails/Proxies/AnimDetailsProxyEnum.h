// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsProxyBase.h"

#include "AnimDetailsProxyEnum.generated.h"

/** An enum value in anim details */
USTRUCT(BlueprintType)
struct FAnimDetailsEnum
{
	GENERATED_USTRUCT_BODY()

	FAnimDetailsEnum()
	{
		EnumType = nullptr;
		EnumIndex = INDEX_NONE;
	}

	UPROPERTY()
	TObjectPtr<UEnum> EnumType;

	UPROPERTY(EditAnywhere, Category = Enum)
	int32 EnumIndex;
};

/** Handles an enum property bound in sequencer, and the related control if the bound object uses a control rig */
UCLASS(EditInlineNew, CollapseCategories)
class UAnimDetailsProxyEnum 
	: public UAnimDetailsProxyBase
{
	GENERATED_BODY()

public:
	//~ Begin UAnimDetailsProxyBase interface
	virtual FName GetCategoryName() const override;
	virtual FName GetDetailRowID() const override;
	virtual TArray<FName> GetPropertyNames() const override;
	virtual TSet<ERigControlType> GetSupportedControlTypes() const override;
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) override;
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) override;
	virtual void ResetPropertyToDefault(const FName& PropertyName) override;
	virtual bool HasDefaultValue(const FName& PropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& InPropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const override;
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) override;
	//~ End UAnimDetailsProxyBase interface

	UPROPERTY(EditAnywhere, Interp, Meta = (ShowOnlyInnerProperties), Category = "Enum")
	FAnimDetailsEnum Enum;
};
