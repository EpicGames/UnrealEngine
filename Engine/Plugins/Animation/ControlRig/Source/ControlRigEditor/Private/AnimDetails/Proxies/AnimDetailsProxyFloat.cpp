// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyFloat.h"

#include "AnimDetailsScopedPropertyChangedEvent.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "Components/SceneComponent.h"
#include "ControlRig.h"
#include "DetailLayoutBuilder.h"
#include "GameFramework/Actor.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyFloat)

namespace UE::ControlRigEditor::Private
{
	struct FFloatScopedPropertyPathChangedEvent
		: FNoncopyable
	{
		FFloatScopedPropertyPathChangedEvent(UObject& InObject, const FString& InPropertyPath, FProperty& InProperty)
			: Property(InProperty)
		{
			InitObjectsToNotify(InObject);
			InitPropertyChain(InObject, InPropertyPath);

			if (PropertyChain.Num() == 0)
			{
				PropertyChain.AddHead(&Property);
			}

			for (UObject* const ObjectToNotify : ObjectsToNotify)
			{
				ObjectToNotify->PreEditChange(PropertyChain);
			}
		}

		~FFloatScopedPropertyPathChangedEvent()
		{
			FPropertyChangedEvent PropertyChangedEvent(&Property, EPropertyChangeType::ValueSet, MakeArrayView(ObjectsToNotify));
			FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

			for (UObject* const ObjectToNotify : ObjectsToNotify)
			{
				ObjectToNotify->PostEditChangeChainProperty(PropertyChangedChainEvent);
			}
		}

	private:
		void InitObjectsToNotify(UObject& InObject)
		{
			ObjectsToNotify.Add(&InObject);

			AActor* const Actor = InObject.IsA(AActor::StaticClass()) ? Cast<AActor>(&InObject) : InObject.GetTypedOuter<AActor>();
			if (Actor)
			{
				ObjectsToNotify.AddUnique(Actor);

				if (USceneComponent* const RootComponent = Actor->GetRootComponent())
				{
					ObjectsToNotify.AddUnique(RootComponent);
				}
			}

			if (USceneComponent* const OuterSceneComponent = InObject.GetTypedOuter<USceneComponent>())
			{
				ObjectsToNotify.AddUnique(OuterSceneComponent);
			}
		}

		void InitPropertyChain(UObject& InObject, const FString& InPropertyPath)
		{
			TArray<FString> PropertyPathSegments;
			InPropertyPath.ParseIntoArray(PropertyPathSegments, TEXT("."), true);
			if (PropertyPathSegments.Num() <= 1)
			{
				return;
			}

			TArray<FProperty*> ResolvedProperties;
			ResolvedProperties.Reserve(PropertyPathSegments.Num());

			UStruct* Struct = InObject.GetClass();
			for (const FString& PropertyPathSegment : PropertyPathSegments)
			{
				if (!Struct)
				{
					return;
				}

				FString PropertyName = PropertyPathSegment;
				int32 ArrayIndexStart = INDEX_NONE;
				if (PropertyName.FindChar(TEXT('['), ArrayIndexStart))
				{
					PropertyName.LeftInline(ArrayIndexStart);
				}

				FProperty* const ResolvedProperty = FindFProperty<FProperty>(Struct, *PropertyName);
				if (!ResolvedProperty)
				{
					return;
				}

				ResolvedProperties.Add(ResolvedProperty);

				const FStructProperty* const StructProperty = CastField<FStructProperty>(ResolvedProperty);
				Struct = StructProperty ? StructProperty->Struct.Get() : nullptr;
			}

			if (ResolvedProperties.Last() != &Property)
			{
				return;
			}

			for (FProperty* const ResolvedProperty : ResolvedProperties)
			{
				PropertyChain.AddHead(ResolvedProperty);
			}
		}

		FProperty& Property;
		FEditPropertyChain PropertyChain;
		TArray<UObject*> ObjectsToNotify;
	};
}

FName UAnimDetailsProxyFloat::GetCategoryName() const
{
	return "Float";
}

TArray<FName> UAnimDetailsProxyFloat::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsFloat, Float)
	};
}

TSet<ERigControlType> UAnimDetailsProxyFloat::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = 
	{ 
		ERigControlType::Float,
		ERigControlType::ScaleFloat
	};

	return SupportedControlTypes;
}

bool UAnimDetailsProxyFloat::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return 
		(Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimDetailsFloat, Float)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyFloat, Float));
}

void UAnimDetailsProxyFloat::AdoptValues(const ERigControlValueType RigControlValueType)
{
	const double Value = [this, RigControlValueType]() -> double
		{
			UControlRig* ControlRig = GetControlRig();
			FRigControlElement* ControlElement = GetControlElement();
			if (ControlRig && ControlElement)
			{
				const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
				return ControlValue.Get<float>();
			}
			else if (SequencerItem.IsValid())
			{
				if (FProperty* Property = SequencerItem.GetProperty())
				{
					if (Property->IsA(FDoubleProperty::StaticClass()))
					{
						const TOptional<double> OptionalValue = SequencerItem.GetBinding()->GetOptionalValue<double>(*SequencerItem.GetBoundObject());
						if (OptionalValue.IsSet())
						{
							return OptionalValue.GetValue();
						}
					}
					else if (Property->IsA(FFloatProperty::StaticClass()))
					{
						const TOptional<float> OptionalValue = SequencerItem.GetBinding()->GetOptionalValue<float>(*SequencerItem.GetBoundObject());
						if (OptionalValue.IsSet())
						{
							return OptionalValue.GetValue();
						}
					}
				}
			}

			return 0.0;
		}();

	const FAnimDetailsFloat AnimDetailProxyFloat(Value);

	constexpr const TCHAR* PropertyName = TEXT("Float");
	FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
	Binding.CallFunction<FAnimDetailsFloat>(*this, AnimDetailProxyFloat);
}

void UAnimDetailsProxyFloat::ResetPropertyToDefault(const FName& PropertyName)
{
	if (const FRigControlElement* ControlElement = GetControlElement())
	{
		AdoptValues(ERigControlValueType::Initial);
	}
	else
	{
		Float = FAnimDetailsFloat();
	}
}

bool UAnimDetailsProxyFloat::HasDefaultValue(const FName& PropertyName) const
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
		return Float.Float == ControlValue.Get<float>();
	}

	return Float.Float == 0.0;
}

EControlRigContextChannelToKey UAnimDetailsProxyFloat::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsFloat, Float))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyFloat::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Float"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	const UControlRig* ControlRig = GetControlRig();
	const FRigElementKey ElementKey = GetControlElementKey();

	const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ElementKey.Name) : nullptr;
	if (ControlElement && ControlElement->GetDisplayName() == InChannelName)
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyFloat::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		float Value = Float.Float;

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<float>(ControlElement->GetKey().Name, Value, bNotify, Context, bSetupUndo);
	}
}

void UAnimDetailsProxyFloat::SetSequencerBindingValueFromCurrent(const FRigControlModifiedContext& Context)
{
	const TSharedPtr<FTrackInstancePropertyBindings> Binding = SequencerItem.GetBinding();
	UObject* BoundObject = SequencerItem.GetBoundObject();
	if (Binding.IsValid() && BoundObject)
	{
		if (FProperty* Property = Binding->GetProperty(*BoundObject))
		{
			const UE::ControlRigEditor::Private::FFloatScopedPropertyPathChangedEvent ScopedPropertyChangedEvent(*BoundObject, Binding->GetPropertyPath(), *Property);

			if (Property->IsA(FDoubleProperty::StaticClass()))
			{
				Binding->SetCurrentValue<double>(*BoundObject, Float.Float);
			}
			else if (Property->IsA(FFloatProperty::StaticClass()))
			{
				float FVal = (float)Float.Float;
				Binding->SetCurrentValue<float>(*BoundObject, FVal);
			}
		}
	}
}
