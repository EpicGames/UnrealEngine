// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsProxyBase.h"

#include "AnimDetailsProxyFloat.generated.h"

/** 
 * A floating point value in anim details  
 * 
 * Note, control rig uses 'float' controls so we call this float though it's a 
 * double internally, so we can use same for non-control rig parameters
 */
USTRUCT(BlueprintType)
struct FAnimDetailsFloat
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Interp, Category = "Float")
	double Float = 0.0;
};

/** Handles an floating point property bound in sequencer, and the related control if the bound object uses a control rig */
UCLASS(EditInlineNew, CollapseCategories)
class UAnimDetailsProxyFloat 
	: public UAnimDetailsProxyBase
{
	GENERATED_BODY()

public:
	//~ Begin UAnimDetailsProxyBase interface
	virtual FName GetCategoryName() const override;
	virtual TArray<FName> GetPropertyNames() const override;
	virtual TSet<ERigControlType> GetSupportedControlTypes() const override;
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) override;
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) override;
	virtual void ResetPropertyToDefault(const FName& PropertyName) override;
	virtual bool HasDefaultValue(const FName& PropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& InPropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const override;
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) override;
	virtual void SetSequencerBindingValueFromCurrent(const FRigControlModifiedContext& Context) override;
	//~ End UAnimDetailsProxyBase interface

	UPROPERTY(EditAnywhere, Interp, Category = "Float", meta = (ShowOnlyInnerProperties, Delta = "0.05", SliderExponent = "1", LinearDeltaSensitivity = "1"))
	FAnimDetailsFloat Float;
};
