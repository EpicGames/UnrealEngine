// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyInteger.h"

#include "AnimDetailsScopedPropertyChangedEvent.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "Components/SceneComponent.h"
#include "ControlRig.h"
#include "DetailLayoutBuilder.h"
#include "GameFramework/Actor.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyInteger)

namespace UE::ControlRigEditor::Private
{
	struct FIntegerScopedPropertyPathChangedEvent
		: FNoncopyable
	{
		FIntegerScopedPropertyPathChangedEvent(UObject& InObject, const FString& InPropertyPath, FProperty& InProperty)
			: Property(InProperty)
		{
			InitObjectsToNotify(InObject);
			InitPropertyChain(InObject, InPropertyPath);

			if (PropertyChain.Num() == 0)
			{
				PropertyChain.AddHead(&Property);
			}

			for (UObject* const ObjectToNotify : ObjectsToNotify)
			{
				ObjectToNotify->PreEditChange(PropertyChain);
			}
		}

		~FIntegerScopedPropertyPathChangedEvent()
		{
			FPropertyChangedEvent PropertyChangedEvent(&Property, EPropertyChangeType::ValueSet, MakeArrayView(ObjectsToNotify));
			FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

			for (UObject* const ObjectToNotify : ObjectsToNotify)
			{
				ObjectToNotify->PostEditChangeChainProperty(PropertyChangedChainEvent);
			}
		}

	private:
		void InitObjectsToNotify(UObject& InObject)
		{
			ObjectsToNotify.Add(&InObject);

			AActor* const Actor = InObject.IsA(AActor::StaticClass()) ? Cast<AActor>(&InObject) : InObject.GetTypedOuter<AActor>();
			if (Actor)
			{
				ObjectsToNotify.AddUnique(Actor);

				if (USceneComponent* const RootComponent = Actor->GetRootComponent())
				{
					ObjectsToNotify.AddUnique(RootComponent);
				}
			}

			if (USceneComponent* const OuterSceneComponent = InObject.GetTypedOuter<USceneComponent>())
			{
				ObjectsToNotify.AddUnique(OuterSceneComponent);
			}
		}

		void InitPropertyChain(UObject& InObject, const FString& InPropertyPath)
		{
			TArray<FString> PropertyPathSegments;
			InPropertyPath.ParseIntoArray(PropertyPathSegments, TEXT("."), true);
			if (PropertyPathSegments.Num() <= 1)
			{
				return;
			}

			TArray<FProperty*> ResolvedProperties;
			ResolvedProperties.Reserve(PropertyPathSegments.Num());

			UStruct* Struct = InObject.GetClass();
			for (const FString& PropertyPathSegment : PropertyPathSegments)
			{
				if (!Struct)
				{
					return;
				}

				FString PropertyName = PropertyPathSegment;
				int32 ArrayIndexStart = INDEX_NONE;
				if (PropertyName.FindChar(TEXT('['), ArrayIndexStart))
				{
					PropertyName.LeftInline(ArrayIndexStart);
				}

				FProperty* const ResolvedProperty = FindFProperty<FProperty>(Struct, *PropertyName);
				if (!ResolvedProperty)
				{
					return;
				}

				ResolvedProperties.Add(ResolvedProperty);

				const FStructProperty* const StructProperty = CastField<FStructProperty>(ResolvedProperty);
				Struct = StructProperty ? StructProperty->Struct.Get() : nullptr;
			}

			if (ResolvedProperties.Last() != &Property)
			{
				return;
			}

			for (FProperty* const ResolvedProperty : ResolvedProperties)
			{
				PropertyChain.AddHead(ResolvedProperty);
			}
		}

		FProperty& Property;
		FEditPropertyChain PropertyChain;
		TArray<UObject*> ObjectsToNotify;
	};
}

FName UAnimDetailsProxyInteger::GetCategoryName() const
{
	return "Integer";
}

TArray<FName> UAnimDetailsProxyInteger::GetPropertyNames() const
{
	return 
	{ 
		GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyInteger, Integer)
	};
}

TSet<ERigControlType> UAnimDetailsProxyInteger::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Integer };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyInteger::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(FAnimDetailsInteger, Integer)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyInteger, Integer));
}

void UAnimDetailsProxyInteger::AdoptValues(const ERigControlValueType RigControlValueType)
{
	const int64 Value = [this, RigControlValueType]() -> int64
		{
			UControlRig* ControlRig = GetControlRig();
			FRigControlElement* ControlElement = GetControlElement();
			if (ControlRig && ControlElement)
			{
				const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
				return ControlValue.Get<int32>();
			}
			else if (SequencerItem.IsValid())
			{
				if (const TOptional<int64> OptionalValue = SequencerItem.GetBinding()->GetOptionalValue<int64>(*SequencerItem.GetBoundObject()))
				{
					if (OptionalValue.IsSet())
					{
						return OptionalValue.GetValue();
					}
				}
			}

			return 0;
		}();

	const FAnimDetailsInteger AnimDetailProxyInteger(Value);

	constexpr const TCHAR* PropertyName = TEXT("Integer");
	FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
	Binding.CallFunction<FAnimDetailsInteger>(*this, AnimDetailProxyInteger);
}

void UAnimDetailsProxyInteger::ResetPropertyToDefault(const FName& PropertyName)
{
	if (const FRigControlElement* ControlElement = GetControlElement())
	{
		AdoptValues(ERigControlValueType::Initial);
	}
	else
	{
		Integer = FAnimDetailsInteger();
	}
}

bool UAnimDetailsProxyInteger::HasDefaultValue(const FName& PropertyName) const
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
		return Integer.Integer == ControlValue.Get<int32>();
	}

	return Integer.Integer == 0;
}

EControlRigContextChannelToKey UAnimDetailsProxyInteger::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyInteger, Integer))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyInteger::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Integer"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	const UControlRig* ControlRig = GetControlRig();
	const FRigElementKey ElementKey = GetControlElementKey();

	const FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ElementKey.Name) : nullptr;
	if (ControlElement && ControlElement->GetDisplayName() == InChannelName)
	{
		return EControlRigContextChannelToKey::TranslationX;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyInteger::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		int32 Val = (int32)Integer.Integer;

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<int32>(ControlElement->GetKey().Name, Val, bNotify, Context, bSetupUndo);
	}
}

void UAnimDetailsProxyInteger::SetSequencerBindingValueFromCurrent(const FRigControlModifiedContext& Context)
{
	const TSharedPtr<FTrackInstancePropertyBindings> Binding = SequencerItem.GetBinding();
	UObject* BoundObject = SequencerItem.GetBoundObject();
	if (Binding.IsValid() && BoundObject)
	{
		if (FProperty* Property = Binding->GetProperty(*BoundObject))
		{
			const UE::ControlRigEditor::Private::FIntegerScopedPropertyPathChangedEvent ScopedPropertyChangedEvent(*BoundObject, Binding->GetPropertyPath(), *Property);

			Binding->SetCurrentValue<int64>(*BoundObject, Integer.Integer);
		}
	}
}
