// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyLocation.h"

#include "ControlRig.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyLocation)

namespace UE::ControlRigEditor::LocationUtils
{
	static void SetLocationValuesFromContext(UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context, FVector3f& TLocation)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);
		const FVector3f Value = ControlValue.Get<FVector3f>();

		const EControlRigContextChannelToKey ChannelsToKey = (EControlRigContextChannelToKey)Context.KeyMask;
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX))
		{
			TLocation.X = Value.X;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY))
		{
			TLocation.Y = Value.Y;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ))
		{
			TLocation.Z = Value.Z;
		}
	}

	/** Returns the default location for this proxy */
	static FAnimDetailsLocation GetDefaultLocation(const UAnimDetailsProxyLocation& Proxy)
	{
		UControlRig* ControlRig = Proxy.GetControlRig();
		FRigControlElement* ControlElement = Proxy.GetControlElement();
		if (ControlRig && ControlElement)
		{
			const FRigControlValue InitialControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
			const FVector3f InitialValue = InitialControlValue.Get<FVector3f>();

			return FAnimDetailsLocation(InitialValue);
		}

		return FAnimDetailsLocation();
	}
}

FAnimDetailsLocation::FAnimDetailsLocation(const FVector& InVector)
	: LX(InVector.X)
	, LY(InVector.Y)
	, LZ(InVector.Z)
{}

FAnimDetailsLocation::FAnimDetailsLocation(const FVector3f& InVector)
	: LX(InVector.X)
	, LY(InVector.Y)
	, LZ(InVector.Z)
{}

FName UAnimDetailsProxyLocation::GetCategoryName() const
{
	return "Location";
}

TArray<FName> UAnimDetailsProxyLocation::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX),
		GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY),
		GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ)
	};
}

void UAnimDetailsProxyLocation::GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const
{
	OutOptionalStructDisplayName = UAnimDetailsProxyLocation::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyLocation, Location))->GetDisplayNameText();

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))
	{
		OutPropertyDisplayName = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))
	{
		OutPropertyDisplayName = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))
	{
		OutPropertyDisplayName = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))->GetDisplayNameText();
	}
	else
	{
		ensureMsgf(0, TEXT("Cannot find member property for anim details proxy, cannot get property name text"));
	}
}

TSet<ERigControlType> UAnimDetailsProxyLocation::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Position };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyLocation::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyLocation, Location)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyLocation, Location));
}

void UAnimDetailsProxyLocation::AdoptValues(const ERigControlValueType RigControlValueType)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
		const FVector3f Value = ControlValue.Get<FVector3f>();

		// Alwasys set the value, even if it's the default
		const FAnimDetailsLocation AnimDetailProxyLocation(Value);

		constexpr const TCHAR* PropertyName = TEXT("Location");
		FTrackInstancePropertyBindings LocationBinding(PropertyName, PropertyName);
		LocationBinding.CallFunction<FAnimDetailsLocation>(*this, AnimDetailProxyLocation);
	}

	// Not implemented for sequencer items, they don't have a Location type
}

void UAnimDetailsProxyLocation::ResetPropertyToDefault(const FName& PropertyName)
{
	using namespace UE::ControlRigEditor;
	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyLocation, Location);
	if (PropertyName == StructName)
	{
		Location = LocationUtils::GetDefaultLocation(*this);
	}
	else
	{
		FProperty* Property = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(PropertyName);
		if (!ensureMsgf(Property, TEXT("Cannot find property in struct, cannot reset anim details property to default")))
		{
			return;
		}

		// Reset the specific property name
		const FAnimDetailsLocation DefaultLocation = LocationUtils::GetDefaultLocation(*this);

		*Property->ContainerPtrToValuePtr<double>(&Location) = *Property->ContainerPtrToValuePtr<double>(&DefaultLocation);

		constexpr const TCHAR* StructPropertyName = TEXT("Location");
		FTrackInstancePropertyBindings Binding(StructPropertyName, StructPropertyName);
		Binding.CallFunction<FAnimDetailsLocation>(*this, Location);
	}
}

bool UAnimDetailsProxyLocation::HasDefaultValue(const FName& PropertyName) const
{
	using namespace UE::ControlRigEditor;

	const FAnimDetailsLocation DefaultLocation = LocationUtils::GetDefaultLocation(*this);

	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyLocation, Location);
	if (PropertyName == StructName)
	{
		return
			Location.LX == DefaultLocation.LX &&
			Location.LY == DefaultLocation.LY &&
			Location.LZ == DefaultLocation.LZ;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))
	{
		return Location.LX == DefaultLocation.LX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))
	{
		return Location.LY == DefaultLocation.LY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))
	{
		return Location.LZ == DefaultLocation.LZ;
	}

	ensureMsgf(0, TEXT("Failed finding property for anim details location. Cannot determine if property has its default value"));

	return true;
}

EControlRigContextChannelToKey UAnimDetailsProxyLocation::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))
	{
		return EControlRigContextChannelToKey::TranslationY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))
	{
		return EControlRigContextChannelToKey::TranslationZ;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyLocation::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Location.X"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}
	else if (InChannelName == TEXT("Location.Y"))
	{
		return EControlRigContextChannelToKey::TranslationY;
	}
	else if (InChannelName == TEXT("Location.Z"))
	{
		return EControlRigContextChannelToKey::TranslationZ;
	}
	
	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyLocation::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	using namespace UE::ControlRigEditor;

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		FVector3f TLocation = Location.ToVector3f();
		LocationUtils::SetLocationValuesFromContext(ControlRig, ControlElement, Context, TLocation);

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<FVector3f>(ControlElement->GetKey().Name, TLocation, bNotify, Context, bSetupUndo);
	}
}
