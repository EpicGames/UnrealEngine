// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsProxyBase.h"

#include "AnimDetailsProxyLocation.generated.h"

/** A location value in anim details */
USTRUCT(BlueprintType)
struct FAnimDetailsLocation
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Interp, Category = "Location")
	double LX = 0.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Location")
	double LY = 0.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Location")
	double LZ = 0.0;

	FAnimDetailsLocation() = default;
	FAnimDetailsLocation(const FVector& InVector);
	FAnimDetailsLocation(const FVector3f& InVector);

	FVector ToVector() const { return FVector(LX, LY, LZ); }
	FVector3f ToVector3f() const { return FVector3f(LX, LY, LZ); }
};

/** Handles a location property bound in sequencer, and the related control if the bound object uses a control rig */
UCLASS(EditInlineNew, CollapseCategories)
class UAnimDetailsProxyLocation 
	: public UAnimDetailsProxyBase
{
	GENERATED_BODY()

public:
	//~ Begin UAnimDetailsProxyBase interface
	virtual FName GetCategoryName() const override;
	virtual TArray<FName> GetPropertyNames() const override;
	virtual void GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const override;
	virtual TSet<ERigControlType> GetSupportedControlTypes() const override;
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) override;
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) override;
	virtual void ResetPropertyToDefault(const FName& PropertyName) override;
	virtual bool HasDefaultValue(const FName& PropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& InPropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const override;
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) override;
	//~ End UAnimDetailsProxyBase interface

	UPROPERTY(EditAnywhere, Interp, Category = Location)
	FAnimDetailsLocation Location;
};
