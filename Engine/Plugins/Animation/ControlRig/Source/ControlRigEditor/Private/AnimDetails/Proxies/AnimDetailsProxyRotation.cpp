// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyRotation.h"

#include "ControlRig.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyRotation)

namespace UE::ControlRigEditor::RotationUtils
{
	static void SetRotationValuesFromContext(UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context, FVector3f& Val)
	{
		FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);
		FVector3f Value = ControlValue.Get<FVector3f>();

		EControlRigContextChannelToKey ChannelsToKey = (EControlRigContextChannelToKey)Context.KeyMask;
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX))
		{
			Val.X = Value.X;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ))
		{
			Val.Y = Value.Y;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY))
		{
			Val.Z = Value.Z;
		}
	}

	/** Returns the default rotation for this proxy */
	static FAnimDetailsRotation GetDefaultRotation(const UAnimDetailsProxyRotation& Proxy)
	{
		UControlRig* ControlRig = Proxy.GetControlRig();
		FRigControlElement* ControlElement = Proxy.GetControlElement();
		if (ControlRig && ControlElement)
		{
			const FRigControlValue InitialControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
			return FAnimDetailsRotation(InitialControlValue.Get<FVector3f>());
		}

		return FAnimDetailsRotation();
	}
}

FAnimDetailsRotation::FAnimDetailsRotation(const FRotator& InRotator)
{ 
	FromRotator(InRotator); 
}

FAnimDetailsRotation::FAnimDetailsRotation(const FVector3f& InVector)
	: RX(InVector.X)
	, RY(InVector.Y)
	, RZ(InVector.Z)
{}

FVector FAnimDetailsRotation::ToVector() const
{
	return FVector(RX, RY, RZ);
}

FVector3f FAnimDetailsRotation::ToVector3f() const
{ 
	return FVector3f(RX, RY, RZ);
}

FRotator FAnimDetailsRotation::ToRotator() const
{ 
	FRotator Rot; 
	Rot = Rot.MakeFromEuler(ToVector()); 
	
	return Rot; 
}

void FAnimDetailsRotation::FromRotator(const FRotator& InRotator)
{ 
	FVector Vec(InRotator.Euler()); 
	RX = Vec.X; 
	RY = Vec.Y; 
	RZ = Vec.Z; 
}

FName UAnimDetailsProxyRotation::GetCategoryName() const
{
	return "Rotation";
}

TArray<FName> UAnimDetailsProxyRotation::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX),
		GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY), 
		GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ),
	};
}

void UAnimDetailsProxyRotation::GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const
{
	OutOptionalStructDisplayName = UAnimDetailsProxyRotation::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyRotation, Rotation))->GetDisplayNameText();

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))
	{
		OutPropertyDisplayName = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))
	{
		OutPropertyDisplayName = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))
	{
		OutPropertyDisplayName = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))->GetDisplayNameText();
	}
	else
	{
		ensureMsgf(0, TEXT("Cannot find member property for anim details proxy, cannot get property name text"));
	}
}

TSet<ERigControlType> UAnimDetailsProxyRotation::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Rotator };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyRotation::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyRotation, Rotation)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyRotation, Rotation));
}

void UAnimDetailsProxyRotation::AdoptValues(const ERigControlValueType RigControlValueType)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
		const FVector3f Value = ControlValue.Get<FVector3f>();

		const FAnimDetailsRotation AnimDetailProxyRotation(Value);

		constexpr const TCHAR* PropertyName = TEXT("Rotation");
		FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
		Binding.CallFunction<FAnimDetailsRotation>(*this, AnimDetailProxyRotation);
	}

	// Not implemented for sequencer items, they don't have a rotation type
}

void UAnimDetailsProxyRotation::ResetPropertyToDefault(const FName& PropertyName)
{
	using namespace UE::ControlRigEditor;

	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyRotation, Rotation);
	if (PropertyName == StructName)
	{
		// Reset the whole rotation struct
		Rotation = RotationUtils::GetDefaultRotation(*this);
	}
	else
	{
		FProperty* Property = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(PropertyName);
		if (!ensureMsgf(Property, TEXT("Cannot find property in struct, cannot reset anim details property to default")))
		{
			return;
		}

		// Reset the specific property name
		const FAnimDetailsRotation DefaultRotation = RotationUtils::GetDefaultRotation(*this);

		*Property->ContainerPtrToValuePtr<double>(&Rotation) = *Property->ContainerPtrToValuePtr<double>(&DefaultRotation);

		constexpr const TCHAR* StructPropertyName = TEXT("Rotation");
		FTrackInstancePropertyBindings Binding(StructPropertyName, StructPropertyName);
		Binding.CallFunction<FAnimDetailsRotation>(*this, Rotation);
	}
}
bool UAnimDetailsProxyRotation::HasDefaultValue(const FName& PropertyName) const
{
	using namespace UE::ControlRigEditor;

	const FAnimDetailsRotation DefaultRotation = RotationUtils::GetDefaultRotation(*this);
	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyRotation, Rotation);
	if (PropertyName == StructName)
	{
		return
			Rotation.RX == DefaultRotation.RX &&
			Rotation.RY == DefaultRotation.RY &&
			Rotation.RZ == DefaultRotation.RZ;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))
	{
		return Rotation.RX == DefaultRotation.RX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))
	{
		return Rotation.RY == DefaultRotation.RY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))
	{
		return Rotation.RZ == DefaultRotation.RZ;
	}

	ensureMsgf(0, TEXT("Failed finding property for anim details rotation. Cannot determine if property has its default value"));

	return true;
}
EControlRigContextChannelToKey UAnimDetailsProxyRotation::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))
	{
		return EControlRigContextChannelToKey::RotationX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))
	{
		return EControlRigContextChannelToKey::RotationY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))
	{
		return EControlRigContextChannelToKey::RotationZ;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyRotation::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Rotation.X") || InChannelName == TEXT("Rotation.Roll"))
	{
		return EControlRigContextChannelToKey::RotationX;
	}
	else if (InChannelName == TEXT("Rotation.Y") || InChannelName == TEXT("Rotation.Pitch"))
	{
		return EControlRigContextChannelToKey::RotationY;
	}
	else if (InChannelName == TEXT("Rotation.Z") || InChannelName == TEXT("Rotation.Yaw"))
	{
		return EControlRigContextChannelToKey::RotationZ;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyRotation::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	using namespace UE::ControlRigEditor;

	UControlRig* ControlRig = GetControlRig();
	URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && Hierarchy && ControlElement)
	{
		FVector3f Value = Rotation.ToVector3f();
		RotationUtils::SetRotationValuesFromContext(ControlRig, ControlElement, Context, Value);

		const FVector EulerAngle(Rotation.ToRotator().Roll, Rotation.ToRotator().Pitch, Rotation.ToRotator().Yaw);
		const FRotator Rotator(Hierarchy->GetControlQuaternion(ControlElement, EulerAngle));

		Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<FRotator>(ControlElement->GetKey().Name, Rotator, bNotify, Context, bSetupUndo);

		ControlRig->Evaluate_AnyThread();
	}
}
