// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsProxyBase.h"

#include "AnimDetailsProxyRotation.generated.h"

/** A rotation value in anim details */
USTRUCT(BlueprintType)
struct  FAnimDetailsRotation
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Interp, Category = "Rotation")
	double RX = 0.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Rotation")
	double RY = 0.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Rotation")
	double RZ = 0.0;

	FAnimDetailsRotation() = default;
	FAnimDetailsRotation(const FRotator& InRotator);
	FAnimDetailsRotation(const FVector3f& InVector);

	FVector ToVector() const;
	FVector3f ToVector3f() const;
	FRotator ToRotator() const;
	void FromRotator(const FRotator& InRotator);
};

/** Handles a rotation property bound in sequencer, and the related control if the bound object uses a control rig */
UCLASS(EditInlineNew, CollapseCategories)
class UAnimDetailsProxyRotation 
	: public UAnimDetailsProxyBase
{
	GENERATED_BODY()

public:
	//~ Begin UAnimDetailsProxyBase interface
	virtual FName GetCategoryName() const override;
	virtual TArray<FName> GetPropertyNames() const override;
	virtual void GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const override;
	virtual TSet<ERigControlType> GetSupportedControlTypes() const override;
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) override;
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) override;
	virtual void ResetPropertyToDefault(const FName& PropertyName) override;
	virtual bool HasDefaultValue(const FName& PropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& InPropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const override;
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) override;
	//~ End UAnimDetailsProxyBase interface

	UPROPERTY(EditAnywhere, Interp, Category = Rotation)
	FAnimDetailsRotation Rotation;
};
