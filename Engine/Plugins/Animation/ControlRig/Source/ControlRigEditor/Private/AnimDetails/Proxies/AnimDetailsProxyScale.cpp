// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyScale.h"

#include "ControlRig.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyScale)

namespace UE::ControlRigEditor::ScaleUtils
{
	static void SetScaleValuesFromContext(UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context, FVector3f& TScale)
	{
		FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);
		FVector3f Value = ControlValue.Get<FVector3f>();

		EControlRigContextChannelToKey ChannelsToKey = (EControlRigContextChannelToKey)Context.KeyMask;

		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX))
		{
			TScale.X = Value.X;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY))
		{
			TScale.Y = Value.Y;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ))
		{
			TScale.Z = Value.Z;
		}
	}

	/** Returns the default scale for this proxy */
	static FAnimDetailsScale GetDefaultScale(const UAnimDetailsProxyScale& Proxy)
	{
		UControlRig* ControlRig = Proxy.GetControlRig();
		FRigControlElement* ControlElement = Proxy.GetControlElement();
		if (ControlRig && ControlElement)
		{
			const FRigControlValue InitialControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
			return InitialControlValue.Get<FVector3f>();
		}

		return FAnimDetailsScale();
	}
}

FAnimDetailsScale::FAnimDetailsScale(const FVector& InVector)
	: SX(InVector.X)
	, SY(InVector.Y)
	, SZ(InVector.Z)
{}

FAnimDetailsScale::FAnimDetailsScale(const FVector3f& InVector)
	: SX(InVector.X)
	, SY(InVector.Y)
	, SZ(InVector.Z)
{}

FVector FAnimDetailsScale::ToVector() const
{ 
	return FVector(SX, SY, SZ); 
}

FVector3f FAnimDetailsScale::ToVector3f() const
{
	return FVector3f(SX, SY, SZ); 
}

FName UAnimDetailsProxyScale::GetCategoryName() const
{
	return "Scale";
}

TArray<FName> UAnimDetailsProxyScale::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX),
		GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY),
		GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ),
	};
}

void UAnimDetailsProxyScale::GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const
{
	OutOptionalStructDisplayName = UAnimDetailsProxyScale::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyScale, Scale))->GetDisplayNameText();

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))
	{
		OutPropertyDisplayName = FAnimDetailsScale::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))
	{
		OutPropertyDisplayName = FAnimDetailsScale::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))
	{
		OutPropertyDisplayName = FAnimDetailsScale::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))->GetDisplayNameText();
	}
	else
	{
		ensureMsgf(0, TEXT("Cannot find member property for anim details proxy, cannot get property name text"));
	}
}

TSet<ERigControlType> UAnimDetailsProxyScale::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Scale };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyScale::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyScale, Scale)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyScale, Scale));
}

void UAnimDetailsProxyScale::AdoptValues(const ERigControlValueType RigControlValueType)
{
	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
		const FVector3f Value = ControlValue.Get<FVector3f>();

		const FAnimDetailsScale AnimDetailProxyScale(Value);

		constexpr const TCHAR* PropertyName = TEXT("Scale");
		FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
		Binding.CallFunction<FAnimDetailsScale>(*this, AnimDetailProxyScale);
	}

	// Not implemented for sequencer items, they don't have a scale type
}

void UAnimDetailsProxyScale::ResetPropertyToDefault(const FName& PropertyName)
{
	using namespace UE::ControlRigEditor;
	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyScale, Scale);
	if (PropertyName == StructName)
	{
		// Reset the whole scale struct
		Scale = ScaleUtils::GetDefaultScale(*this);
	}
	else
	{
		FProperty* Property = FAnimDetailsScale::StaticStruct()->FindPropertyByName(PropertyName);
		if (!ensureMsgf(Property, TEXT("Cannot find property in struct, cannot reset anim details property to default")))
		{
			return;
		}

		// Reset the specific property name
		const FAnimDetailsScale DefaultScale = ScaleUtils::GetDefaultScale(*this);

		*Property->ContainerPtrToValuePtr<double>(&Scale) = *Property->ContainerPtrToValuePtr<double>(&DefaultScale);

		constexpr const TCHAR* StructPropertyName = TEXT("Scale");
		FTrackInstancePropertyBindings Binding(StructPropertyName, StructPropertyName);
		Binding.CallFunction<FAnimDetailsScale>(*this, Scale);
	}
}

bool UAnimDetailsProxyScale::HasDefaultValue(const FName& PropertyName) const
{
	using namespace UE::ControlRigEditor;

	const FAnimDetailsScale DefaultScale = ScaleUtils::GetDefaultScale(*this);

	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyScale, Scale);
	if (PropertyName == StructName)
	{
		return
			Scale.SX == DefaultScale.SX &&
			Scale.SY == DefaultScale.SY &&
			Scale.SZ == DefaultScale.SZ;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))
	{
		return Scale.SX == DefaultScale.SX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))
	{
		return Scale.SY == DefaultScale.SY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))
	{
		return Scale.SZ == DefaultScale.SZ;
	}

	ensureMsgf(0, TEXT("Failed finding property for anim details scale. Cannot determine if property has its default value"));

	return true;
}

EControlRigContextChannelToKey UAnimDetailsProxyScale::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))
	{
		return EControlRigContextChannelToKey::ScaleX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))
	{
		return EControlRigContextChannelToKey::ScaleY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))
	{
		return EControlRigContextChannelToKey::ScaleZ;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyScale::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Scale.X"))
	{
		return EControlRigContextChannelToKey::ScaleX;
	}
	else if (InChannelName == TEXT("Scale.Y"))
	{
		return EControlRigContextChannelToKey::ScaleY;
	}
	else if (InChannelName == TEXT("Scale.Z"))
	{
		return EControlRigContextChannelToKey::ScaleZ;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyScale::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	using namespace UE::ControlRigEditor;

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		FVector3f Val = Scale.ToVector3f();
		ScaleUtils::SetScaleValuesFromContext(ControlRig, ControlElement, Context, Val);

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<FVector3f>(ControlElement->GetKey().Name, Val, bNotify, Context, bSetupUndo);

		ControlRig->Evaluate_AnyThread();
	}
}
