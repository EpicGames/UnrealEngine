// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsProxyBase.h"

#include "AnimDetailsProxyScale.generated.h"

/** A scale value in anim details */
USTRUCT(BlueprintType)
struct FAnimDetailsScale
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Interp, Category = "Scale")
	double SX = 1.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Scale")
	double SY = 1.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Scale")
	double SZ = 1.0;

	FAnimDetailsScale() = default;
	FAnimDetailsScale(const FVector& InVector);
	FAnimDetailsScale(const FVector3f& InVector);

	FVector ToVector() const;
	FVector3f ToVector3f() const;
};

/** Handles a scale property bound in sequencer, and the related control if the bound object uses a control rig */
UCLASS(EditInlineNew, CollapseCategories)
class UAnimDetailsProxyScale 
	: public UAnimDetailsProxyBase
{
	GENERATED_BODY()

public:
	//~ Begin UAnimDetailsProxyBase interface
	virtual FName GetCategoryName() const override;
	virtual TArray<FName> GetPropertyNames() const override;
	virtual void GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const override;
	virtual TSet<ERigControlType> GetSupportedControlTypes() const override;
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) override;
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) override;
	virtual void ResetPropertyToDefault(const FName& PropertyName) override;
	virtual bool HasDefaultValue(const FName& PropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& InPropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const override;
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) override;
	//~ End UAnimDetailsProxyBase interface

	UPROPERTY(EditAnywhere, Interp, Category = Scale)
	FAnimDetailsScale Scale;
};
