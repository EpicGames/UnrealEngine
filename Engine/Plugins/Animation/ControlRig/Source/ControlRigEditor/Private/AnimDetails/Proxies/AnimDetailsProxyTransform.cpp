// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyTransform.h"

#include "AnimDetailsScopedPropertyChangedEvent.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "ConstraintsManager.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "Editor/UnrealEdEngine.h"
#include "IControlRigObjectBinding.h"
#include "MovieSceneCommonHelpers.h"
#include "Rigs/RigHierarchy.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "UnrealEdGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyTransform)

namespace UE::ControlRigEditor
{
	namespace TransformUtils
	{
		static void SetValuesFromContext(const FEulerTransform& EulerTransform, const FRigControlModifiedContext& Context, FVector& TLocation, FRotator& TRotation, FVector& TScale)
		{
			EControlRigContextChannelToKey ChannelsToKey = (EControlRigContextChannelToKey)Context.KeyMask;
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX))
			{
				TLocation.X = EulerTransform.Location.X;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY))
			{
				TLocation.Y = EulerTransform.Location.Y;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ))
			{
				TLocation.Z = EulerTransform.Location.Z;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX))
			{
				TRotation.Roll = EulerTransform.Rotation.Roll;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ))
			{
				TRotation.Yaw = EulerTransform.Rotation.Yaw;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY))
			{
				TRotation.Pitch = EulerTransform.Rotation.Pitch;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX))
			{
				TScale.X = EulerTransform.Scale.X;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY))
			{
				TScale.Y = EulerTransform.Scale.Y;
			}
			if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ))
			{
				TScale.Z = EulerTransform.Scale.Z;
			}
		}

		static FEulerTransform GetValue(UControlRig* ControlRig, FRigControlElement* ControlElement, const ERigControlValueType ValueType)
		{
			FEulerTransform EulerTransform = FEulerTransform::Identity;

			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Transform:
			{
				const FTransform NewTransform = ControlRig->GetControlValue(ControlElement, ValueType).Get<FRigControlValue::FTransform_Float>().ToTransform();
				EulerTransform = FEulerTransform(NewTransform);
				break;

			}
			case ERigControlType::TransformNoScale:
			{
				const FTransformNoScale NewTransform = ControlRig->GetControlValue(ControlElement, ValueType).Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
				EulerTransform.Location = NewTransform.Location;
				EulerTransform.Rotation = FRotator(NewTransform.Rotation);
				break;

			}
			case ERigControlType::EulerTransform:
			{
				EulerTransform = ControlRig->GetControlValue(ControlElement, ValueType).Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
				break;
			}
			};

			if (ControlRig->GetHierarchy()->UsesPreferredEulerAngles())
			{
				const bool bInitial = ValueType == ERigControlValueType::Initial;
				EulerTransform.Rotation = ControlRig->GetHierarchy()->GetControlPreferredRotator(ControlElement, bInitial);
			}

			return EulerTransform;
		}

		static void GetActorAndSceneComponentFromObject(UObject* Object, AActor*& OutActor, USceneComponent*& OutSceneComponent)
		{
			OutActor = Cast<AActor>(Object);
			if (OutActor && OutActor->GetRootComponent())
			{
				OutSceneComponent = OutActor->GetRootComponent();
			}
			else
			{
				// If the object wasn't an actor attempt to get it directly as a scene component and then get the actor from there.
				OutSceneComponent = Cast<USceneComponent>(Object);
				if (OutSceneComponent)
				{
					OutActor = Cast<AActor>(OutSceneComponent->GetOuter());
				}
			}
		}

		static FEulerTransform GetValue(UObject* InObject, const TSharedPtr<FTrackInstancePropertyBindings>& Binding)
		{
			const FStructProperty* TransformProperty = Binding.IsValid() ? CastField<FStructProperty>(Binding->GetProperty(*InObject)) : nullptr;
			if (TransformProperty && TransformProperty->Struct == TBaseStructure<FTransform>::Get())
			{
				if (const TOptional<FTransform> Transform = Binding->GetOptionalValue<FTransform>(*InObject))
				{
					return FEulerTransform(Transform.GetValue());
				}
			}
			else if (TransformProperty && TransformProperty->Struct == TBaseStructure<FEulerTransform>::Get())
			{
				if (const TOptional<FEulerTransform> EulerTransform = Binding->GetOptionalValue<FEulerTransform>(*InObject))
				{
					return EulerTransform.GetValue();
				}
			}
			
			AActor* ActorThatChanged = nullptr;
			USceneComponent* SceneComponentThatChanged = nullptr;
			GetActorAndSceneComponentFromObject(InObject, ActorThatChanged, SceneComponentThatChanged);

			FEulerTransform EulerTransform = FEulerTransform::Identity;
			if (SceneComponentThatChanged)
			{
				EulerTransform.Location = SceneComponentThatChanged->GetRelativeLocation();
				EulerTransform.Rotation = SceneComponentThatChanged->GetRelativeRotation();
				EulerTransform.Scale = SceneComponentThatChanged->GetRelativeScale3D();
			}

			return EulerTransform;
		}

		FTransform GetControlRigComponentTransform(UControlRig* ControlRig)
		{
			FTransform Transform = FTransform::Identity;
			TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding();
			if (ObjectBinding.IsValid())
			{
				if (USceneComponent* BoundSceneComponent = Cast<USceneComponent>(ObjectBinding->GetBoundObject()))
				{
					return BoundSceneComponent->GetComponentTransform();
				}
			}

			return Transform;
		}

		static bool SetConstrainedTransform(FTransform LocalTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& InContext)
		{
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(ControlRig->GetWorld());
			const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, ControlElement->GetFName());
			const TArray< TWeakObjectPtr<UTickableConstraint> > Constraints = Controller.GetParentConstraints(ControlHash, true);
			if (Constraints.IsEmpty())
			{
				return false;
			}

			const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(Constraints);
			const bool bNeedsConstraintPostProcess = Constraints.IsValidIndex(LastActiveIndex);

			if (!bNeedsConstraintPostProcess)
			{
				return false;
			}

			constexpr bool bNotify = true;
			constexpr bool bUndo = true;
			constexpr bool bFixEuler = true;

			FRigControlModifiedContext Context = InContext;
			Context.EventName = FRigUnit_BeginExecution::EventName;
			Context.bConstraintUpdate = true;
			Context.SetKey = EControlRigSetKey::Never;

			// set the global space, assumes it's attached to actor
			// no need to compensate for constraints here, this will be done after when setting the control in the constraint space
			{
				const TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
				ControlRig->SetControlLocalTransform(
					ControlElement->GetKey().Name, LocalTransform, bNotify, Context, bUndo, bFixEuler);
			}
			FTransform GlobalTransform = ControlRig->GetControlGlobalTransform(ControlElement->GetKey().Name);

			// switch to constraint space
			FTransform ToWorldTransform = GetControlRigComponentTransform(ControlRig);
			const FTransform WorldTransform = GlobalTransform * ToWorldTransform;

			const TOptional<FTransform> RelativeTransform = UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, LocalTransform, WorldTransform);
			if (RelativeTransform)
			{
				LocalTransform = *RelativeTransform;
			}

			Context.bConstraintUpdate = false;
			Context.SetKey = InContext.SetKey;
			ControlRig->SetControlLocalTransform(ControlElement->GetKey().Name, LocalTransform, bNotify, Context, bUndo, bFixEuler);
			ControlRig->Evaluate_AnyThread();
			Controller.EvaluateAllConstraints();

			return true;
		}

		/** Returns the default transfrom for this proxy */
		static FEulerTransform GetDefaultTransform(const UAnimDetailsProxyTransform& Proxy)
		{
			UControlRig* ControlRig = Proxy.GetControlRig();
			FRigControlElement* ControlElement = Proxy.GetControlElement();
			if (ControlRig && ControlElement)
			{
				return TransformUtils::GetValue(ControlRig, ControlElement, ERigControlValueType::Initial);
			}

			return FEulerTransform();
		}
	}
}

FName UAnimDetailsProxyTransform::GetCategoryName() const
{
	return "Transform";
}

TArray<FName> UAnimDetailsProxyTransform::GetPropertyNames() const
{
	return
		{
			GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX),
			GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY),
			GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ),

			GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX),
			GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY),
			GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ),

			GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX),
			GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY),
			GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ)
		};
}

void UAnimDetailsProxyTransform::GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const
{
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsScale::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsScale::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))
	{
		OutOptionalStructDisplayName = UAnimDetailsProxyTransform::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale))->GetDisplayNameText();
		OutPropertyDisplayName = FAnimDetailsScale::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))->GetDisplayNameText();
	}
	else
	{
		ensureMsgf(0, TEXT("Cannot find member property for anim details proxy, cannot get property name text"));
	}
}

TSet<ERigControlType> UAnimDetailsProxyTransform::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes =
	{
		ERigControlType::Transform,
		ERigControlType::EulerTransform,
		ERigControlType::TransformNoScale
	};

	return SupportedControlTypes;
}

bool UAnimDetailsProxyTransform::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return ((Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location)) ||
		(Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation)) ||
		(Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale)));
}

void UAnimDetailsProxyTransform::AdoptValues(const ERigControlValueType RigControlValueType)
{
	using namespace UE::ControlRigEditor;

	FEulerTransform Value;

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		Value = TransformUtils::GetValue(ControlRig, ControlElement, RigControlValueType);
	}
	else if (SequencerItem.IsValid())
	{
		Value = TransformUtils::GetValue(SequencerItem.GetBoundObject(), SequencerItem.GetBinding());
	}

	const FAnimDetailsLocation TLocation(Value.Location);
	const FAnimDetailsRotation TRotation(Value.Rotation);
	const FAnimDetailsScale TScale(Value.Scale);

	constexpr const TCHAR* LocationPropertyName = TEXT("Location");
	FTrackInstancePropertyBindings LocationBinding(LocationPropertyName, LocationPropertyName);
	LocationBinding.CallFunction<FAnimDetailsLocation>(*this, TLocation);

	constexpr const TCHAR* RotationPropertyName = TEXT("Rotation");
	FTrackInstancePropertyBindings RotationBinding(RotationPropertyName, RotationPropertyName);
	RotationBinding.CallFunction<FAnimDetailsRotation>(*this, TRotation);

	constexpr const TCHAR* ScalePropertyName = TEXT("Scale");
	FTrackInstancePropertyBindings ScaleBinding(ScalePropertyName, ScalePropertyName);
	ScaleBinding.CallFunction<FAnimDetailsScale>(*this, TScale);
}

void UAnimDetailsProxyTransform::ResetPropertyToDefault(const FName& PropertyName)
{
	using namespace UE::ControlRigEditor;

	FEulerTransform CurrentValue;
	CurrentValue.Location = Location.ToVector();
	CurrentValue.Rotation = Rotation.ToRotator();
	CurrentValue.Scale = Scale.ToVector();

	const FEulerTransform DefaulTransform = TransformUtils::GetDefaultTransform(*this);

	TOptional<FAnimDetailsLocation> NewLocation;
	TOptional<FAnimDetailsRotation> NewRotation;
	TOptional<FAnimDetailsScale> NewScale;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location))
	{
		NewLocation = FAnimDetailsLocation(DefaulTransform.Location);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation))
	{
		NewRotation = FAnimDetailsRotation(DefaulTransform.Rotation);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale))
	{
		NewScale = FAnimDetailsScale(DefaulTransform.Scale);
	}
	else if (FProperty* LocationProperty = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(PropertyName))
	{
		NewLocation = Location;
		*LocationProperty->ContainerPtrToValuePtr<double>(&NewLocation.GetValue()) = 
			*LocationProperty->ContainerPtrToValuePtr<double>(&DefaulTransform.Location);
	}
	else if (FProperty* RotationProperty = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(PropertyName))
	{
		NewRotation = Rotation;
		*RotationProperty->ContainerPtrToValuePtr<double>(&NewRotation.GetValue()) = 
			*RotationProperty->ContainerPtrToValuePtr<double>(&DefaulTransform.Rotation);
	}
	else if (FProperty* ScaleProperty = FAnimDetailsScale::StaticStruct()->FindPropertyByName(PropertyName))
	{
		NewScale = Scale;
		*ScaleProperty->ContainerPtrToValuePtr<double>(&NewScale.GetValue()) = 
			*ScaleProperty->ContainerPtrToValuePtr<double>(&DefaulTransform.Scale);
	}

	if (NewLocation.IsSet())
	{
		constexpr const TCHAR* LocationPropertyName = TEXT("Location");
		FTrackInstancePropertyBindings LocationBinding(LocationPropertyName, LocationPropertyName);
		LocationBinding.CallFunction<FAnimDetailsLocation>(*this, NewLocation.GetValue());
	}
	else if (NewRotation.IsSet())
	{
		constexpr const TCHAR* RotationPropertyName = TEXT("Rotation");
		FTrackInstancePropertyBindings RotationBinding(RotationPropertyName, RotationPropertyName);
		RotationBinding.CallFunction<FAnimDetailsRotation>(*this, NewRotation.GetValue());
	}
	else if (NewScale.IsSet())
	{
		constexpr const TCHAR* ScalePropertyName = TEXT("Scale");
		FTrackInstancePropertyBindings ScaleBinding(ScalePropertyName, ScalePropertyName);
		ScaleBinding.CallFunction<FAnimDetailsScale>(*this, NewScale.GetValue());
	}
}

bool UAnimDetailsProxyTransform::HasDefaultValue(const FName& PropertyName) const
{
	using namespace UE::ControlRigEditor;

	const FEulerTransform DefaulTransform = TransformUtils::GetDefaultTransform(*this);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location))
	{
		return Location.ToVector() == DefaulTransform.Location;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation))
	{
		return Rotation.ToRotator() == DefaulTransform.Rotation;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale))
	{
		return Scale.ToVector() == DefaulTransform.Scale;
	}
	else if (FProperty* LocationProperty = FAnimDetailsLocation::StaticStruct()->FindPropertyByName(PropertyName))
	{
		const FAnimDetailsLocation& DefaultLocation = DefaulTransform.Location;
		return (*LocationProperty->ContainerPtrToValuePtr<double>(&Location)) == 
			(*LocationProperty->ContainerPtrToValuePtr<double>(&DefaultLocation));
	}
	else if (FProperty* RotationProperty = FAnimDetailsRotation::StaticStruct()->FindPropertyByName(PropertyName))
	{
		const FAnimDetailsRotation& DefaultRotation = DefaulTransform.Rotation;
		return (*RotationProperty->ContainerPtrToValuePtr<double>(&Rotation)) == 
			(*RotationProperty->ContainerPtrToValuePtr<double>(&DefaultRotation));
	}
	else if (FProperty* ScaleProperty = FAnimDetailsScale::StaticStruct()->FindPropertyByName(PropertyName))
	{
		const FAnimDetailsScale& DefaultScale = DefaulTransform.Scale;
		return (*ScaleProperty->ContainerPtrToValuePtr<double>(&Scale)) == 
			(*ScaleProperty->ContainerPtrToValuePtr<double>(&DefaultScale));
	}

	ensureMsgf(0, TEXT("Failed finding property %s for anim details transform. Cannot determine if property has its default value"), *PropertyName.ToString());

	return true;
}

EControlRigContextChannelToKey UAnimDetailsProxyTransform::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location))
	{
		return EControlRigContextChannelToKey::Translation;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LX))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LY))
	{
		return EControlRigContextChannelToKey::TranslationY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsLocation, LZ))
	{
		return EControlRigContextChannelToKey::TranslationZ;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation))
	{
		return EControlRigContextChannelToKey::Rotation;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RX))
	{
		return EControlRigContextChannelToKey::RotationX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RY))
	{
		return EControlRigContextChannelToKey::RotationY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsRotation, RZ))
	{
		return EControlRigContextChannelToKey::RotationZ;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale))
	{
		return EControlRigContextChannelToKey::Scale;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SX))
	{
		return EControlRigContextChannelToKey::ScaleX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SY))
	{
		return EControlRigContextChannelToKey::ScaleY;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsScale, SZ))
	{
		return EControlRigContextChannelToKey::ScaleZ;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyTransform::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("Location.X"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}
	else if (InChannelName == TEXT("Location.Y"))
	{
		return EControlRigContextChannelToKey::TranslationY;
	}
	else if (InChannelName == TEXT("Location.Z"))
	{
		return EControlRigContextChannelToKey::TranslationZ;
	}
	else if (InChannelName == TEXT("Rotation.X") || InChannelName == TEXT("Rotation.Roll"))
	{
		return EControlRigContextChannelToKey::RotationX;
	}
	else if (InChannelName == TEXT("Rotation.Y") || InChannelName == TEXT("Rotation.Pitch"))
	{
		return EControlRigContextChannelToKey::RotationY;
	}
	else if (InChannelName == TEXT("Rotation.Z") || InChannelName == TEXT("Rotation.Yaw"))
	{
		return EControlRigContextChannelToKey::RotationZ;
	}
	else if (InChannelName == TEXT("Scale.X"))
	{
		return EControlRigContextChannelToKey::ScaleX;
	}
	else if (InChannelName == TEXT("Scale.Y"))
	{
		return EControlRigContextChannelToKey::ScaleY;
	}
	else if (InChannelName == TEXT("Scale.Z"))
	{
		return EControlRigContextChannelToKey::ScaleZ;
	}
	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyTransform::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	using namespace UE::ControlRigEditor;

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (!ControlElement || !ControlRig)
	{
		return;
	}

	constexpr bool bNotify = true;
	constexpr bool bSetupUndo = false;

	FVector TLocation = Location.ToVector();
	FRotator TRotation = Rotation.ToRotator();
	FVector TScale = Scale.ToVector();
	FEulerTransform EulerTransform = TransformUtils::GetValue(ControlRig, ControlElement, ERigControlValueType::Current);

	TransformUtils::SetValuesFromContext(EulerTransform, Context, TLocation, TRotation, TScale);

	//constraints we just deal with FTransforms unfortunately, need to figure out how to handle rotation orders
	const FTransform RealTransform(TRotation, TLocation, TScale);
	if (TransformUtils::SetConstrainedTransform(RealTransform, ControlRig, ControlElement, Context))
	{
		AdoptValues(ERigControlValueType::Current);
		return;
	}
	switch (ControlElement->Settings.ControlType)
	{
	case ERigControlType::Transform:
	{
		const FVector EulerAngle(TRotation.Roll, TRotation.Pitch, TRotation.Yaw);
		ControlRig->GetHierarchy()->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);

		ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(ControlElement->GetKey().Name, RealTransform, bNotify, Context, bSetupUndo);
		ControlRig->GetHierarchy()->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
		break;

	}
	case ERigControlType::TransformNoScale:
	{
		FTransformNoScale NoScale(TLocation, TRotation.Quaternion());

		ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(ControlElement->GetKey().Name, NoScale, bNotify, Context, bSetupUndo);
		break;

	}
	case ERigControlType::EulerTransform:
	{
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

		if (Hierarchy && Hierarchy->UsesPreferredEulerAngles())
		{
			const FVector EulerAngle(TRotation.Roll, TRotation.Pitch, TRotation.Yaw);
			const FQuat Quat = Hierarchy->GetControlQuaternion(ControlElement, EulerAngle);

			Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);

			const FRotator UERotator(Quat);
			FEulerTransform UETransform(UERotator, TLocation, TScale);
			UETransform.Rotation = UERotator;

			ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlElement->GetKey().Name, UETransform, bNotify, Context, bSetupUndo);
			Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
		}
		else
		{
			ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlElement->GetKey().Name, FEulerTransform(RealTransform), bNotify, Context, bSetupUndo);
		}
		break;
	}
	}

	ControlRig->Evaluate_AnyThread();
}

void UAnimDetailsProxyTransform::SetSequencerBindingValueFromCurrent(const FRigControlModifiedContext& Context)
{
	using namespace UE::ControlRigEditor;
	
	const TSharedPtr<FTrackInstancePropertyBindings> Binding = SequencerItem.GetBinding();
	UObject* BoundObject = SequencerItem.GetBoundObject();
	if (!Binding.IsValid() || !BoundObject)
	{
		return;
	}

	FVector TLocation = Location.ToVector();
	FRotator TRotation = Rotation.ToRotator();
	FVector TScale = Scale.ToVector();
	FEulerTransform EulerTransform = TransformUtils::GetValue(BoundObject, Binding);

	TransformUtils::SetValuesFromContext(EulerTransform, Context, TLocation, TRotation, TScale);

	const FTransform RealTransform(TRotation, TLocation, TScale);

	const FStructProperty* TransformProperty = Binding.IsValid() ? CastField<FStructProperty>(Binding->GetProperty(*BoundObject)) : nullptr;
	if (TransformProperty)
	{
		if (TransformProperty->Struct == TBaseStructure<FEulerTransform>::Get())
		{
			EulerTransform = FEulerTransform(TLocation, TRotation, TScale);
			Binding->SetCurrentValue<FTransform>(*BoundObject, RealTransform);
		}
	}

	AActor* ActorThatChanged = nullptr;
	USceneComponent* SceneComponentThatChanged = nullptr;
	TransformUtils::GetActorAndSceneComponentFromObject(BoundObject, ActorThatChanged, SceneComponentThatChanged);
	if (SceneComponentThatChanged)
	{
		FStructProperty* StructProperty = nullptr;
		FProperty* AxisProperty = nullptr;
		if (Context.SetKey != EControlRigSetKey::Never)
		{
			EControlRigContextChannelToKey ChannelsToKey = (EControlRigContextChannelToKey)Context.KeyMask;
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, X));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Y));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Z));
			}

			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Roll));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Pitch));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Yaw));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, X));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Y));
			}
			if (EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ))
			{
				StructProperty = CastField<FStructProperty>(FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName()));
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Z));
			}
		}
				
		if (StructProperty && AxisProperty)
		{
			const UE::ControlRigEditor::FAnimDetailsScopedPropertyChangedEvent ScopedPropertyChangedEvent(*BoundObject, *StructProperty, *AxisProperty);
			
			SceneComponentThatChanged->SetRelativeTransform(RealTransform, false, nullptr, ETeleportType::None);

			// Force the location and rotation values to avoid Rot->Quat->Rot conversions
			SceneComponentThatChanged->SetRelativeLocation_Direct(TLocation);
			SceneComponentThatChanged->SetRelativeRotationExact(TRotation);
		}

		GUnrealEd->UpdatePivotLocationForSelection();
	}
}
