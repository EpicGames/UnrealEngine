// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsProxyVector2D.h"

#include "ControlRig.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimDetailsProxyVector2D)

namespace UE::ControlRigEditor::Vector2DUtils
{
	static void SetVector2DValuesFromContext(UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context, FVector2D& Val)
	{
		FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);
		FVector3f Value = ControlValue.Get<FVector3f>();

		EControlRigContextChannelToKey ChannelsToKey = (EControlRigContextChannelToKey)Context.KeyMask;
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX))
		{
			Val.X = Value.X;
		}
		if (!EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY))
		{
			Val.Y = Value.Y;
		}
	}
}

FAnimDetailsVector2D::FAnimDetailsVector2D(const FVector2D& InVector)
	: X(InVector.X)
	, Y(InVector.Y)
{}

FVector2D FAnimDetailsVector2D::ToVector2D() const
{ 
	return FVector2D(X, Y); 
}

FName UAnimDetailsProxyVector2D::GetCategoryName() const
{
	return "Vector2D";
}

TArray<FName> UAnimDetailsProxyVector2D::GetPropertyNames() const
{
	return
	{
		GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, X),
		GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, Y)
	};
}

void UAnimDetailsProxyVector2D::GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const
{
	OutOptionalStructDisplayName = UAnimDetailsProxyVector2D::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyVector2D, Vector2D))->GetDisplayNameText();

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, X))
	{
		OutPropertyDisplayName = FAnimDetailsVector2D::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, X))->GetDisplayNameText();
	}
	else if (InPropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, Y))
	{
		OutPropertyDisplayName = FAnimDetailsVector2D::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, Y))->GetDisplayNameText();
	}
	else
	{
		ensureMsgf(0, TEXT("Cannot find member property for anim details proxy, cannot get property name text"));
	}
}

TSet<ERigControlType> UAnimDetailsProxyVector2D::GetSupportedControlTypes() const
{
	static const TSet<ERigControlType> SupportedControlTypes = { ERigControlType::Vector2D };
	return SupportedControlTypes;
}

bool UAnimDetailsProxyVector2D::PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty)
{
	return (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyVector2D, Vector2D)) ||
		(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyVector2D, Vector2D));
}

void UAnimDetailsProxyVector2D::AdoptValues(const ERigControlValueType RigControlValueType)
{
	FVector2D Value = FVector2D::ZeroVector;

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig &&
		ControlElement &&
		ControlElement->Settings.ControlType == ERigControlType::Vector2D)
	{
		const FRigControlValue ControlValue = ControlRig->GetControlValue(ControlElement, RigControlValueType);
		const FVector3f Vec3Value = ControlValue.Get<FVector3f>();
		Value = FVector2D(Vec3Value.X, Vec3Value.Y);
	}

	const FAnimDetailsVector2D AnimDetailProxyVector2D(Value);

	constexpr const TCHAR* PropertyName = TEXT("Vector2D");
	FTrackInstancePropertyBindings Binding(PropertyName, PropertyName);
	Binding.CallFunction<FAnimDetailsVector2D>(*this, AnimDetailProxyVector2D);
}

void UAnimDetailsProxyVector2D::ResetPropertyToDefault(const FName& PropertyName)
{
	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyVector2D, Vector2D);
	if (PropertyName == StructName)
	{
		// Reset the whole vector 2D struct
		AdoptValues(ERigControlValueType::Initial);
	}
	else
	{
		FProperty* Property = FAnimDetailsVector2D::StaticStruct()->FindPropertyByName(PropertyName);
		if (!ensureMsgf(Property, TEXT("Cannot find property in struct, cannot reset anim details property to default")))
		{
			return;
		}

		// Reset the specific property name
		UControlRig* ControlRig = GetControlRig();
		FRigControlElement* ControlElement = GetControlElement();
		if (ControlRig &&
			ControlElement &&
			ControlElement->Settings.ControlType == ERigControlType::Vector2D)
		{
			const FRigControlValue InitialControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
			const FVector3f InitialVec3Value = InitialControlValue.Get<FVector3f>();
			const FVector2D InitialValue = FVector2D(InitialVec3Value.X, InitialVec3Value.Y);
			const FAnimDetailsVector2D InitialVector2D(InitialValue);

			*Property->ContainerPtrToValuePtr<double>(&Vector2D) = *Property->ContainerPtrToValuePtr<double>(&InitialVector2D);

			constexpr const TCHAR* StructPropertName = TEXT("Vector2D");
			FTrackInstancePropertyBindings Binding(StructPropertName, StructPropertName);
			Binding.CallFunction<FAnimDetailsVector2D>(*this, InitialVector2D);
		}
	}
}

bool UAnimDetailsProxyVector2D::HasDefaultValue(const FName& PropertyName) const
{
	const FAnimDetailsVector2D DefaultVector2D = [this]() -> FAnimDetailsVector2D
		{

			UControlRig* ControlRig = GetControlRig();
			FRigControlElement* ControlElement = GetControlElement();
			if (ControlRig && ControlElement)
			{
				const FRigControlValue InitialControlValue = ControlRig->GetControlValue(ControlElement, ERigControlValueType::Initial);
				const FVector3f InitialVec3Value = InitialControlValue.Get<FVector3f>();
				return FVector2D(InitialVec3Value.X, InitialVec3Value.Y);
			}

			return FAnimDetailsVector2D();
		}();
	const FName StructName = GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyVector2D, Vector2D);

	if (PropertyName == StructName)
	{
		return
			Vector2D.X == DefaultVector2D.X &&
			Vector2D.Y == DefaultVector2D.Y;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, X))
	{
		return Vector2D.X == DefaultVector2D.X;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, Y))
	{
		return Vector2D.Y == DefaultVector2D.Y;
	}

	ensureMsgf(0, TEXT("Failed finding property for anim details vector 2D. Cannot determine if property has its default value"));

	return true;
}

EControlRigContextChannelToKey UAnimDetailsProxyVector2D::GetChannelToKeyFromPropertyName(const FName& PropertyName) const
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, X))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimDetailsVector2D, Y))
	{
		return EControlRigContextChannelToKey::TranslationY;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

EControlRigContextChannelToKey UAnimDetailsProxyVector2D::GetChannelToKeyFromChannelName(const FString& InChannelName) const
{
	if (InChannelName == TEXT("X"))
	{
		return EControlRigContextChannelToKey::TranslationX;
	}
	else if (InChannelName == TEXT("Y"))
	{
		return EControlRigContextChannelToKey::TranslationY;
	}

	return EControlRigContextChannelToKey::AllTransform;
}

void UAnimDetailsProxyVector2D::SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context)
{
	using namespace UE::ControlRigEditor;

	UControlRig* ControlRig = GetControlRig();
	FRigControlElement* ControlElement = GetControlElement();
	if (ControlRig && ControlElement)
	{
		FVector2D Value = Vector2D.ToVector2D();
		Vector2DUtils::SetVector2DValuesFromContext(ControlRig, ControlElement, Context, Value);

		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		ControlRig->SetControlValue<FVector2D>(ControlElement->GetKey().Name, Value, bNotify, Context, bSetupUndo);

		ControlRig->Evaluate_AnyThread();
	}
}
