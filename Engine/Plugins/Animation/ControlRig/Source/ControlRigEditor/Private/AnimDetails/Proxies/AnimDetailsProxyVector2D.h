// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetailsProxyBase.h"

#include "AnimDetailsProxyVector2D.generated.h"

/** A vector 2D value in anim details */
USTRUCT(BlueprintType)
struct FAnimDetailsVector2D
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Interp, Category = "Vector2D", meta = (Delta = "0.05", SliderExponent = "1", LinearDeltaSensitivity = "1"))
	double X = 0.0;

	UPROPERTY(EditAnywhere, Interp, Category = "Vector2D", meta = (Delta = "0.05", SliderExponent = "1", LinearDeltaSensitivity = "1"))
	double Y = 0.0;

	FAnimDetailsVector2D() = default;
	FAnimDetailsVector2D(const FVector2D& InVector);

	FVector2D ToVector2D() const;
};

/** Handles a vector 2D property bound in sequencer, and the related control if the bound object uses a control rig */
UCLASS(EditInlineNew, CollapseCategories)
class UAnimDetailsProxyVector2D 
	: public UAnimDetailsProxyBase
{
	GENERATED_BODY()

public:
	//~ Begin UAnimDetailsProxyBase interface
	virtual FName GetCategoryName() const override;
	virtual TArray<FName> GetPropertyNames() const override;
	virtual void GetLocalizedPropertyName(const FName& InPropertyName, FText& OutPropertyDisplayName, TOptional<FText>& OutOptionalStructDisplayName) const override;
	virtual TSet<ERigControlType> GetSupportedControlTypes() const override;
	virtual bool PropertyIsOnProxy(const FProperty* Property, const FProperty* MemberProperty) override;
	virtual void AdoptValues(const ERigControlValueType RigControlValueType = ERigControlValueType::Current) override;
	virtual void ResetPropertyToDefault(const FName& PropertyName) override;
	virtual bool HasDefaultValue(const FName& PropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromPropertyName(const FName& InPropertyName) const override;
	virtual EControlRigContextChannelToKey GetChannelToKeyFromChannelName(const FString& InChannelName) const override;
	virtual void SetControlRigElementValueFromCurrent(const FRigControlModifiedContext& Context) override;
	//~ End UAnimDetailsProxyBase interface

	UPROPERTY(EditAnywhere, Interp, Category = Vector2D)
	FAnimDetailsVector2D Vector2D;
};
