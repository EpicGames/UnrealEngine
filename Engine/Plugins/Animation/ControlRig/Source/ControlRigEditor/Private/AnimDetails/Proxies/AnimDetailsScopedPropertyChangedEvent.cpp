// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsScopedPropertyChangedEvent.h"

#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"

namespace UE::ControlRigEditor
{
	FAnimDetailsScopedPropertyChangedEvent::FAnimDetailsScopedPropertyChangedEvent(UObject& Object, FProperty& InProperty)
		: Property(InProperty)
	{
		InitObjectToNotify(Object);

		PropertyChain.AddHead(&Property);

		for (UObject* ObjectToNotify : ObjectsToNotify)
		{
			ObjectToNotify->PreEditChange(PropertyChain);
		}
	}

	FAnimDetailsScopedPropertyChangedEvent::FAnimDetailsScopedPropertyChangedEvent(UObject& Object, FStructProperty& InStructProperty, FProperty& InProperty)
		: Property(InProperty)
	{
		InitObjectToNotify(Object);

		PropertyChain.AddHead(&InStructProperty);
		PropertyChain.AddHead(&Property);

		for (UObject* ObjectToNotify : ObjectsToNotify)
		{
			ObjectToNotify->PreEditChange(PropertyChain);
		}
	}

	FAnimDetailsScopedPropertyChangedEvent::~FAnimDetailsScopedPropertyChangedEvent()
	{
		FPropertyChangedEvent PropertyChangedEvent(&Property, EPropertyChangeType::ValueSet, MakeArrayView(ObjectsToNotify));

		FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

		for (UObject* ObjectToNotify : ObjectsToNotify)
		{
			ObjectToNotify->PostEditChangeChainProperty(PropertyChangedChainEvent);
		}
	}

	void FAnimDetailsScopedPropertyChangedEvent::InitObjectToNotify(UObject& Object)
	{
		ObjectsToNotify.Add(&Object);

		// Also notify the the actor, its root component and the outer scene component (similar to LiveLink)
		AActor* Actor = Object.IsA(AActor::StaticClass()) ? Cast<AActor>(&Object) : Object.GetTypedOuter<AActor>();
		if (Actor)
		{
			ObjectsToNotify.AddUnique(Actor);

			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				ObjectsToNotify.AddUnique(Actor->GetRootComponent());
			}
		}

		if (USceneComponent* OuterSceneComponent = Object.GetTypedOuter<USceneComponent>())
		{
			ObjectsToNotify.AddUnique(OuterSceneComponent);
		}
	}
}
