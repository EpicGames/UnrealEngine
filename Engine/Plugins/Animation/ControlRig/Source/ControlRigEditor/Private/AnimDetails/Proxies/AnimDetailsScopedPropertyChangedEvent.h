// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealType.h"

class FProperty;
class FStructProperty;
class UObject;

namespace UE::ControlRigEditor
{
	/** Utility to raise out pre - / post property changed events for arbitrary properties */
	struct FAnimDetailsScopedPropertyChangedEvent
		: FNoncopyable
	{
		/** Instantiates a scoped property change for object properties */
		FAnimDetailsScopedPropertyChangedEvent(UObject& Object, FProperty& InProperty);

		///** Instantiates a scoped property change for struct properties */
		FAnimDetailsScopedPropertyChangedEvent(UObject& Object, FStructProperty& InStructProperty, FProperty& InProperty);

		~FAnimDetailsScopedPropertyChangedEvent();

	private:
		/** Initializes the ObjectsToNotify member */
		void InitObjectToNotify(UObject& Object);

		/** The property that is being changed */
		FProperty& Property;

		/** The property chain for which the event is raised */
		FEditPropertyChain PropertyChain;

		/** Objects that should be notified of the property change */
		TArray<UObject*> ObjectsToNotify;
	};
}
