// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimDetailsSequencerProxyItem.h"

#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTrack.h"

namespace UE::ControlRigEditor
{
	FAnimDetailsSequencerProxyItem::FAnimDetailsSequencerProxyItem(UObject& InBoundObject, UMovieSceneTrack& InMovieSceneTrack, const TSharedRef<FTrackInstancePropertyBindings>& InBinding)
		: WeakBoundObject(&InBoundObject)
		, WeakMovieSceneTrack(&InMovieSceneTrack)
		, Binding(InBinding)
	{}

	UObject* FAnimDetailsSequencerProxyItem::GetBoundObject() const
	{
		return WeakBoundObject.Get();
	}

	UMovieSceneTrack* FAnimDetailsSequencerProxyItem::GetMovieSceneTrack() const
	{
		return WeakMovieSceneTrack.Get();
	}

	FProperty* FAnimDetailsSequencerProxyItem::GetProperty() const
	{
		return IsValid() ? Binding->GetProperty(*WeakBoundObject.Get()) : nullptr;
	}

	bool FAnimDetailsSequencerProxyItem::IsValid() const
	{
		return
			WeakBoundObject.IsValid() &&
			WeakMovieSceneTrack.IsValid() &&
			Binding.IsValid();
	}

	void FAnimDetailsSequencerProxyItem::Reset()
	{
		WeakBoundObject.Reset();
		WeakMovieSceneTrack.Reset();
		Binding.Reset();
	}
}
