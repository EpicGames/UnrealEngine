// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyCache.h"

class FTrackInstancePropertyBindings;
class UMovieSceneTrack;

namespace UE::ControlRigEditor
{
	/** A sequencer binding in a anim details proxy */
	struct FAnimDetailsSequencerProxyItem
	{
		FAnimDetailsSequencerProxyItem() = default;

		/**
		 * Constructs a valid item
		 *
		 * @param InBoundObject			The object the sequencer track is bound to. Should be an actor or a scene component.
		 * @param InMovieSceneTrack		The movie scene track.
		 * @param InBinding				The property binding for this item.
		 */
		FAnimDetailsSequencerProxyItem(UObject& InBoundObject, UMovieSceneTrack& InMovieSceneTrack, const TSharedRef<FTrackInstancePropertyBindings>& InBinding);

		/** Returns the object the sequencer track is bound to or nullptr if the bound object is not valid. */
		UObject* GetBoundObject() const;

		/** Returns the movie scene track  or nullptr if the movie scene track is not valid. */
		UMovieSceneTrack* GetMovieSceneTrack() const;

		/** Returns the property binding for this item or nullptr if the binding is not valid */
		const TSharedPtr<FTrackInstancePropertyBindings>& GetBinding() const { return Binding; }

		/** Returns the bound property or nullptr if no property is bound */
		FProperty* GetProperty() const;

		/** Returns true if this points to a live bound object of a track */
		bool IsValid() const;

		/** Resets this item irrevocably */
		void Reset();

	private:
		/** The object the sequencer track is bound to. Should be an actor or a scene component. */
		TWeakObjectPtr<UObject> WeakBoundObject;

		/** The movie scene track. */
		TWeakObjectPtr<UMovieSceneTrack> WeakMovieSceneTrack;

		/** The property binding for this item. */
		TSharedPtr<FTrackInstancePropertyBindings> Binding;
	};
}
