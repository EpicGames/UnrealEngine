// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDetailsView.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Customizations/AnimDetailsProxyDetails.h"
#include "AnimDetails/Customizations/AnimDetailsProxyManagerDetails.h"
#include "AnimDetails/Customizations/AnimDetailsValueCustomization.h"
#include "AnimDetails/Customizations/AnimDetailsValueEnumCustomization.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBool.h"
#include "AnimDetails/Proxies/AnimDetailsProxyEnum.h"
#include "AnimDetails/Proxies/AnimDetailsProxyFloat.h"
#include "AnimDetails/Proxies/AnimDetailsProxyInteger.h"
#include "AnimDetails/Proxies/AnimDetailsProxyLocation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyRotation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyScale.h"
#include "AnimDetails/Proxies/AnimDetailsProxyVector2D.h"
#include "AnimDetails/Widgets/SAnimDetailsOptions.h"
#include "AnimDetails/Widgets/SAnimDetailsSearchBox.h"
#include "CurveEditor.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ISequencer.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SAnimOverrideDetailsView.h"
#include "Widgets/Layout/SScrollBox.h"

namespace UE::ControlRigEditor
{
	void SAnimDetailsView::Construct(const FArguments& InArgs)
	{
		const FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		if (!EditMode || !ProxyManager)
		{
			return;
		}

		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bShowPropertyMatrixButton = false;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = true;
			DetailsViewArgs.bCustomFilterAreaLocation = false;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
			DetailsViewArgs.bShowScrollBar = false;
		}

		const TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
		WeakDetailsView = DetailsView;

		DetailsView->SetKeyframeHandler(EditMode->DetailKeyFrameCache);
		DetailsView->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FAnimDetailProxyManagerDetails::MakeInstance));
		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SAnimDetailsView::ShouldDisplayProperty));
		DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SAnimDetailsView::IsReadOnlyProperty));
		DetailsView->RegisterInstancedCustomPropertyLayout(UAnimDetailsProxyBase::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FAnimDetailsProxyDetails::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsFloat::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsInteger::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsBool::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsVector2D::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsLocation::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsRotation::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsScale::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueCustomization::MakeInstance));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FAnimDetailsEnum::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsValueEnumCustomization::MakeInstance));

		DetailsView->SetObjects({ TWeakObjectPtr<UObject>(ProxyManager) });
		
		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
			
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(6.f)
				.FillWidth(1.f)
				[
					SAssignNew(SearchBox, SAnimDetailsSearchBox)
					.OnSearchTextChanged(this, &SAnimDetailsView::RefreshDetailsView)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SAnimDetailsOptions)
					.OnOptionsChanged(this, &SAnimDetailsView::RefreshDetailsView)
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SScrollBox)
				.ScrollBarRightClickDragAllowed(true)

				+ SScrollBox::Slot()
				[
					DetailsView
				]

				+ SScrollBox::Slot()
				[
					SNew(SAnimOverrideDetailsView)
					.OnRequestRefreshDetails(this, &SAnimDetailsView::RefreshDetailsView)
					.Visibility_Lambda([]()
					{
						return CVarControlRigEnableOverrides.GetValueOnAnyThread() ? EVisibility::Visible : EVisibility::Collapsed;
					})
				]
			]
		];

		ProxyManager->GetOnProxiesChanged().AddSP(this, &SAnimDetailsView::RefreshDetailsView);
	}

	FReply SAnimDetailsView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		const TSharedPtr<ISequencer> Sequencer = EditMode ? EditMode->GetWeakSequencer().Pin() : nullptr;
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

		if (EditMode && ProxyManager && Sequencer.IsValid())
		{
			using namespace UE::Sequencer;
			const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
			const FCurveEditorExtension* CurveEditorExtension = SequencerViewModel.IsValid() ? SequencerViewModel->CastDynamic<FCurveEditorExtension>() : nullptr;
			const TSharedPtr<FCurveEditor> CurveEditor = CurveEditorExtension ? CurveEditorExtension->GetCurveEditor() : nullptr;
			if (CurveEditor.IsValid() && 
				CurveEditor->GetCommands()->ProcessCommandBindings(InKeyEvent))
			{
				ProxyManager->RequestUpdateProxyValues();

				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}

	void SAnimDetailsView::RefreshDetailsView()
	{
		const FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		if (!EditMode || !ProxyManager)
		{
			return;
		}

		const FText SearchText = SearchBox->GetSearchText();
		ProxyManager->GetAnimDetailsFilter().Update(SearchText, ProxyManager->GetExternalSelection());

		OnOptionsChanged();
	}

	void SAnimDetailsView::OnOptionsChanged()
	{
		if (WeakDetailsView.IsValid())
		{
			WeakDetailsView.Pin()->ForceRefresh();
		}
	}

	bool SAnimDetailsView::ShouldDisplayProperty(const FPropertyAndParent& InPropertyAndParent) const
	{
		auto ShouldPropertyBeVisible = [](const FProperty& InProperty)
			{
				const bool bShow = InProperty.HasAnyPropertyFlags(CPF_Interp) || InProperty.HasMetaData(FRigVMStruct::InputMetaName) || InProperty.HasMetaData(FRigVMStruct::OutputMetaName);
				return bShow;
			};

		if (InPropertyAndParent.Property.IsA<FStructProperty>())
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(&InPropertyAndParent.Property);
			for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct); PropertyIt; ++PropertyIt)
			{
				if (ShouldPropertyBeVisible(**PropertyIt))
				{
					return true;
				}
			}
		}

		return ShouldPropertyBeVisible(InPropertyAndParent.Property) ||
			(InPropertyAndParent.ParentProperties.Num() > 0 && ShouldPropertyBeVisible(*InPropertyAndParent.ParentProperties[0]));
	}

	bool SAnimDetailsView::IsReadOnlyProperty(const FPropertyAndParent& InPropertyAndParent) const
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!EditMode || !EditMode->GetWeakSequencer().IsValid())
		{
			return true;
		}

		const TSharedRef<ISequencer> Sequencer = EditMode->GetWeakSequencer().Pin().ToSharedRef();
		const bool bIsPlaying = Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
		if (bIsPlaying)
		{
			return true;
		}

		auto ShouldPropertyBeEnabledLambda = [](const FProperty& InProperty)
			{
				const bool bShouldPropertyBeEnabled = InProperty.HasAnyPropertyFlags(CPF_Interp) || InProperty.HasMetaData(FRigVMStruct::InputMetaName);
				return bShouldPropertyBeEnabled;
			};

		if (InPropertyAndParent.Property.IsA<FStructProperty>())
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(&InPropertyAndParent.Property);
			for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct); PropertyIt; ++PropertyIt)
			{
				if (ShouldPropertyBeEnabledLambda(**PropertyIt))
				{
					return false;
				}
			}
		}

		const bool bReadOnlyProperty = !ShouldPropertyBeEnabledLambda(InPropertyAndParent.Property);

		return bReadOnlyProperty;
	}
}
