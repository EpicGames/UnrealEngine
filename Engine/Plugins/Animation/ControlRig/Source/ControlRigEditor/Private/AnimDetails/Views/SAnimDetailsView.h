// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IDetailsView;
struct FPropertyAndParent;

namespace UE::ControlRigEditor
{
	class SAnimDetailsSearchBox;

	class SAnimDetailsView
		: public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SAnimDetailsView)
			{}
		
		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs);

	private:
		//~ Begin SWidget interface
		virtual bool SupportsKeyboardFocus() const override { return true; }
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		//~ End SWidget interface

		/** Refreshes the details view */
		void RefreshDetailsView();

		/** Called when options changed */
		void OnOptionsChanged();

		/** Returns ture if the propery should be displayed */
		bool ShouldDisplayProperty(const FPropertyAndParent& InPropertyAndParent) const;
		
		/** Returns ture if the propery is ready-only */
		bool IsReadOnlyProperty(const FPropertyAndParent& InPropertyAndParent) const;

		/** The search box displayed in this details view */
		TSharedPtr<SAnimDetailsSearchBox> SearchBox;

		/** Weak ptr to the details view this widget is displaying */
		TWeakPtr<IDetailsView> WeakDetailsView;
	};
}
