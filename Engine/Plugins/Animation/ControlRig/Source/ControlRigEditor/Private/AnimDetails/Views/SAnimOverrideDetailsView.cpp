// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimOverrideDetailsView.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "AnimDetails/Customizations/AnimDetailsProxyDetails.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Customizations/AnimDetailsProxyManagerDetails.h"
#include "AnimDetails/Widgets/SAnimDetailsOptions.h"
#include "AnimDetails/Widgets/SAnimDetailsSearchBox.h"
#include "CurveEditor.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ISequencer.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Overrides/OverrideStatusDetailsObjectFilter.h"
#include "Rigs/RigHierarchyController.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AnimDetails/Customizations/AnimDetailsOverrideDetails.h"

namespace UE::ControlRigEditor
{
	SLATE_IMPLEMENT_WIDGET(SAnimOverrideDetailsView)
	void SAnimOverrideDetailsView::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
	{
	}

	void SAnimOverrideDetailsView::Construct(const FArguments& InArgs)
	{
		const FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		if (!EditMode || !ProxyManager)
		{
			return;
		}

		RequestRefreshDetailsDelegate = InArgs._OnRequestRefreshDetails;

		ProxyPropertyToControl = {
			{
				TEXT("DisplayName"),
				TEXT("Settings->DisplayName"),
			},
			{
				TEXT("Shape->bVisible"),
				TEXT("Settings->bShapeVisible"),
			},
			{
				TEXT("Shape->Name"),
				TEXT("Settings->ShapeName"),
			},
			{
				TEXT("Shape->Color"),
				TEXT("Settings->ShapeColor"),
			},
			{
				TEXT("Shape->Transform"),
				TEXT("Settings->ShapeTransform"),
			},
			{
				TEXT("Rotation"),
				TEXT("Settings->ShapeTransform->Rotation"),
			}
		};
		
		FDetailsViewArgs DetailsViewArgs;
		{
			DetailsViewArgs.bAllowSearch = false;
			DetailsViewArgs.bShowPropertyMatrixButton = false;
			DetailsViewArgs.bHideSelectionTip = true;
			DetailsViewArgs.bLockable = false;
			DetailsViewArgs.bUpdatesFromSelection = false;
			DetailsViewArgs.bShowOptions = false;
			DetailsViewArgs.bShowModifiedPropertiesOption = true;
			DetailsViewArgs.bCustomFilterAreaLocation = false;
			DetailsViewArgs.bCustomNameAreaLocation = true;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
			DetailsViewArgs.bShowScrollBar = false;
		}
		
		const TSharedPtr<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SAnimOverrideDetailsView::ShouldDisplayProperty));
		DetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SAnimOverrideDetailsView::IsReadOnlyProperty));
		DetailsView->RegisterInstancedCustomPropertyTypeLayout(FRigUnit_HierarchyAddControl_ShapeSettings::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAnimDetailsOverrideDetails::MakeInstance));
		DetailsView->OnFinishedChangingProperties().AddSP(this, &SAnimOverrideDetailsView::OnFinishedChangingOverride);
		WeakDetailsView = DetailsView;

		ObjectFilter = FOverrideStatusDetailsViewObjectFilter::Create();
		ObjectFilter->OnCanMergeObjects().BindSP(this, &SAnimOverrideDetailsView::CanMergeObjects);
		ObjectFilter->OnCanCreateWidget().BindSP(this, &SAnimOverrideDetailsView::CanCreateWidget);
		ObjectFilter->OnGetStatus().BindSP(this, &SAnimOverrideDetailsView::GetOverrideStatus);
		ObjectFilter->OnAddOverride().BindSP(this, &SAnimOverrideDetailsView::OnAddOverride);
		ObjectFilter->OnClearOverride().BindSP(this, &SAnimOverrideDetailsView::OnClearOverride);
		DetailsView->SetObjectFilter(ObjectFilter);
		
		ChildSlot
		[
			DetailsView.ToSharedRef()
		];

		ProxyManager->GetOnProxiesChanged().AddSP(this, &SAnimOverrideDetailsView::RefreshDetailsView);
	}

	void SAnimOverrideDetailsView::RefreshDetailsView()
	{
		if (WeakDetailsView.IsValid())
		{
			const FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
			const UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
			TArray<UObject*> OverrideableProxies;
			if (ProxyManager)
			{
				const TArray<TObjectPtr<UAnimDetailsProxyBase>>& SelectedProxies = ProxyManager->GetExternalSelection();
				for(const TObjectPtr<UAnimDetailsProxyBase>& SelectedProxy : SelectedProxies)
				{
					const FRigControlElement* ControlElement = SelectedProxy->GetControlElement();
					if(!ControlElement || ControlElement->IsAnimationChannel())
					{
						continue;
					}
					SelectedProxy->UpdateOverrideableProperties();
					OverrideableProxies.Add(SelectedProxy);
				}
			}
			WeakDetailsView.Pin()->SetObjects(OverrideableProxies);
		}
	}

	bool SAnimOverrideDetailsView::ShouldDisplayProperty(const FPropertyAndParent& InPropertyAndParent) const
	{
		static const FLazyName CategoryName = TEXT("Category");
		static const FString OverrideCategoryName = TEXT("Overrides");
		if(InPropertyAndParent.Property.GetMetaData(CategoryName) == OverrideCategoryName)
		{
			return true;
		}
		if(!InPropertyAndParent.ParentProperties.IsEmpty())
		{
			if(InPropertyAndParent.ParentProperties.Last()->GetMetaData(CategoryName) == OverrideCategoryName)
			{
				return true;
			}
		}
		return false;
	}

	bool SAnimOverrideDetailsView::IsReadOnlyProperty(const FPropertyAndParent& InPropertyAndParent) const
	{
		// only allow display name editing if we have a single control selected
		static const FProperty* DisplayNameProperty = UAnimDetailsProxyBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyBase, DisplayName));
		if(&InPropertyAndParent.Property == DisplayNameProperty)
		{
			return InPropertyAndParent.Objects.Num() > 1;
		}
	
		return false;
	}

	void SAnimOverrideDetailsView::OnFinishedChangingOverride(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if(!ensure(Property))
		{
			return;
		}
		FString PropertyPathString = Property->GetName();
		
		TMap<FString, int32> PropertyNameStack;
		PropertyChangedEvent.GetArrayIndicesPerObject(0, PropertyNameStack);
		if (PropertyNameStack.Num() > 0)
		{
			TArray<FString> PropertyNames;
			PropertyNameStack.GetKeys(PropertyNames);
			for(int32 Index = 0; Index < PropertyNames.Num(); Index++)
			{
				const int32 ArrayIndex = PropertyNameStack.FindChecked(PropertyNames[Index]);
				if(ArrayIndex != INDEX_NONE)
				{
					PropertyNames[Index] = FString::Printf(TEXT("%s[%d]"), *PropertyNames[Index], ArrayIndex);
				}
			}
			Algo::Reverse(PropertyNames);
			PropertyPathString = FString::Join(PropertyNames, TEXT("->"));
		}
		MapPropertyFromProxyToControl(PropertyPathString);

		TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>> PreviousSelection;
		if(PropertyPathString.Contains(TEXT("DisplayName")))
		{
			for(int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); Index++)
			{
				if(const UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(PropertyChangedEvent.GetObjectBeingEdited(Index)))
				{
					if(UControlRig* ControlRig = Proxy->GetControlRig())
					{
						if(const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
						{
							PreviousSelection.Add(ControlRig, Hierarchy->GetSelectedKeys());
						}
					}
				}
			}
		}

		{
			FScopedTransaction Transaction(NSLOCTEXT("SAnimOverrideDetailsView", "EditOverrideValue", "Edit Override Value"));
			for(int32 Index = 0; Index < PropertyChangedEvent.GetNumObjectsBeingEdited(); Index++)
			{
				if(const UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(PropertyChangedEvent.GetObjectBeingEdited(Index)))
				{
					if(UControlRig* ControlRig = Proxy->GetControlRig())
					{
						if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
						{
							if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Proxy->GetControlElementKey()))
							{
								if(const URigHierarchyController* Controller = Hierarchy->GetController())
								{
									// copy the settings from proxy to element
									FRigControlSettings Settings = ControlElement->Settings;
									const FText DisplayNameText = Proxy->GetDisplayNameText();
									Settings.DisplayName = DisplayNameText.IsEmpty() ? FName(NAME_None) : FName(*DisplayNameText.ToString());
									Proxy->Shape.Configure(Settings);
									(void)Controller->SetControlSettings(ControlElement->GetKey(), Settings);
									Hierarchy->SetControlShapeTransform(ControlElement->GetKey(), Proxy->Shape.Transform);
								}
								if(UControlRigOverrideAsset* DefaultOverrideAsset = GetOrCreateOverrideAsset(ControlRig))
								{
									DefaultOverrideAsset->Modify();
									if(FControlRigOverrideValue* Override = DefaultOverrideAsset->Overrides.Find(PropertyPathString, ControlElement->GetFName()))
									{
										(void)Override->SetFromSubject(ControlElement, FRigControlElement::StaticStruct());
									}
									else
									{
										DefaultOverrideAsset->Overrides.FindOrAdd(FControlRigOverrideValue(PropertyPathString, FRigControlElement::StaticStruct(), ControlElement, ControlElement->GetFName()));
									}
									
									DefaultOverrideAsset->BroadcastChanged();
								}
							}
						}
					}
				}
			}
		}
		if(PropertyPathString.Contains(TEXT("DisplayName")))
		{
			// refresh the build of this view so that the label of the category is up2date
			RefreshDetailsView();
			(void)RequestRefreshDetailsDelegate.ExecuteIfBound();
		}

		for(const TPair<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>>& Pair : PreviousSelection)
		{
			if(UControlRig* ControlRig = Pair.Key.Get())
			{
				if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
				{
					if(URigHierarchyController* Controller = Hierarchy->GetController())
					{
						Controller->SetSelection(Pair.Value);
					}
				}
			}
		}
	}

	bool SAnimOverrideDetailsView::CanMergeObjects(const UObject* InObjectA, const UObject* InObjectB) const
	{
		if(InObjectA && InObjectB)
		{
			return InObjectA->IsA<UAnimDetailsProxyBase>() && InObjectB->IsA<UAnimDetailsProxyBase>(); 
		}
		return false;
	}

	bool SAnimOverrideDetailsView::CanCreateWidget(const FOverrideStatusSubject& InSubject) const
	{
		return InSubject.Contains<UAnimDetailsProxyBase>();
	}

	EOverrideWidgetStatus::Type SAnimOverrideDetailsView::GetOverrideStatus(const FOverrideStatusSubject& InSubject) const
	{
		FString PropertyPathString = InSubject.GetPropertyPathString();
		MapPropertyFromProxyToControl(PropertyPathString);
		return InSubject.GetStatus<UAnimDetailsProxyBase>(
			[PropertyPathString](const FOverrideStatusObjectHandle<UAnimDetailsProxyBase>& InControlsProxy) -> TOptional<EOverrideWidgetStatus::Type>
			{
				const UControlRig* ControlRig = InControlsProxy->GetControlRig();
				if(ControlRig == nullptr)
				{
					return {};
				}
				for(int32 AssetIndex = 0; AssetIndex < ControlRig->NumOverrideAssets(); AssetIndex++)
				{
					const UControlRigOverrideAsset* Asset = ControlRig->GetOverrideAsset(AssetIndex);
					check(Asset);
					if(PropertyPathString.IsEmpty())
					{
						if(Asset->Overrides.ContainsAnyPathForSubject(InControlsProxy->GetControlName()))
						{
							return EOverrideWidgetStatus::ChangedInside;
						}
					}
					else if(Asset->Overrides.Contains(PropertyPathString, InControlsProxy->GetControlName()))
					{
						return EOverrideWidgetStatus::ChangedHere;
					}
					else if(Asset->Overrides.ContainsChildPathOf(PropertyPathString, InControlsProxy->GetControlName()))
					{
						return EOverrideWidgetStatus::ChangedInside;
					}
					else if(Asset->Overrides.ContainsParentPathOf(PropertyPathString, InControlsProxy->GetControlName()))
					{
						return EOverrideWidgetStatus::ChangedOutside;
					}
				}
				return {};
			}
		).Get(EOverrideWidgetStatus::None);
	}

	FReply SAnimOverrideDetailsView::OnAddOverride(const FOverrideStatusSubject& InSubject)
	{
		FScopedTransaction Transaction(NSLOCTEXT("SAnimOverrideDetailsView", "AddOverride", "Add Override"));
			
		FString PropertyPathString = InSubject.GetPropertyPathString();
		MapPropertyFromProxyToControl(PropertyPathString);
		TArray<UControlRig*> AffectedControlRigs;
		TArray<UControlRigOverrideAsset*> AffectedAssets;
		InSubject.ForEach<UAnimDetailsProxyBase>(
			[InSubject, PropertyPathString, &AffectedAssets, &AffectedControlRigs](const FOverrideStatusObjectHandle<UAnimDetailsProxyBase>& InControlsProxy)
			{
				if(InSubject.HasPropertyPath())
				{
					UControlRig* ControlRig = InControlsProxy->GetControlRig();
					if(ControlRig == nullptr)
					{
						return;
					}
					if(const FRigControlElement* ControlElement = ControlRig->FindControl(InControlsProxy->GetControlName()))
					{
						if(UControlRigOverrideAsset* DefaultOverrideAsset = GetOrCreateOverrideAsset(ControlRig))
						{
							const FControlRigOverrideValue Value(PropertyPathString, FRigControlElement::StaticStruct(), ControlElement, ControlElement->GetFName());
							if(Value.IsValid())
							{
								DefaultOverrideAsset->Modify();
								DefaultOverrideAsset->Overrides.Add(Value);
								AffectedControlRigs.AddUnique(ControlRig);
								AffectedAssets.AddUnique(DefaultOverrideAsset);
							}
						}
					}
				}
			}
		);
		for(UControlRig* AffectedControlRig : AffectedControlRigs)
		{
			AffectedControlRig->SetSuspendOverrideAssetChangedDelegate(true);
		}
		for(UControlRigOverrideAsset* AffectedAsset : AffectedAssets)
		{
			AffectedAsset->BroadcastChanged();
		}
		for(UControlRig* AffectedControlRig : AffectedControlRigs)
		{
			AffectedControlRig->SetSuspendOverrideAssetChangedDelegate(false);
		}
		return FReply::Handled();
	}

	FReply SAnimOverrideDetailsView::OnClearOverride(const FOverrideStatusSubject& InSubject)
	{
		FScopedTransaction Transaction(NSLOCTEXT("SAnimOverrideDetailsView", "ClearOverride", "Clear Override"));
			
		FString PropertyPathString = InSubject.GetPropertyPathString();
		MapPropertyFromProxyToControl(PropertyPathString);
		TArray<UControlRigOverrideAsset*> AffectedAssets;
		InSubject.ForEach<UAnimDetailsProxyBase>(
			[InSubject, PropertyPathString, &AffectedAssets](const FOverrideStatusObjectHandle<UAnimDetailsProxyBase>& InControlsProxy)
			{
				if(InSubject.HasPropertyPath())
				{
					UControlRig* ControlRig = InControlsProxy->GetControlRig();
					if(ControlRig == nullptr)
					{
						return;
					}
					if(UControlRigOverrideAsset* OverrideAsset = GetOrCreateOverrideAsset(ControlRig))
					{
						if(OverrideAsset->Overrides.Contains(PropertyPathString, InControlsProxy->GetControlName()))
						{
							OverrideAsset->Modify();
							OverrideAsset->Overrides.Remove(PropertyPathString, InControlsProxy->GetControlName());
							AffectedAssets.AddUnique(OverrideAsset);
						}
					}
				}
			}
		);
		for(UControlRigOverrideAsset* AffectedAsset : AffectedAssets)
		{
			AffectedAsset->BroadcastChanged();
		}
		return FReply::Handled();
	}

	void SAnimOverrideDetailsView::MapPropertyFromProxyToControl(FString& InOutPropertyPath) const
	{
		if(const FString* MappedPath = ProxyPropertyToControl.Find(InOutPropertyPath))
		{
			InOutPropertyPath= *MappedPath;
		}
		else
		{
			for(const TPair<FString,FString>& Pair : ProxyPropertyToControl)
			{
				if(InOutPropertyPath.StartsWith(Pair.Key))
				{
					InOutPropertyPath.ReplaceInline(*Pair.Key, *Pair.Value);
				}
			}
		}
	}

	UControlRigOverrideAsset* SAnimOverrideDetailsView::GetOrCreateOverrideAsset(UControlRig* InControlRig)
	{
		check(InControlRig);
		if(!CVarControlRigEnableOverrides.GetValueOnAnyThread())
		{
			return nullptr;
		}
	
		if(InControlRig->NumOverrideAssets() == 0)
		{
			if(UControlRigOverrideAsset* OverrideAsset = UControlRigOverrideAsset::CreateOverrideAssetInDeveloperFolder(InControlRig))
			{
				OverrideAsset->Overrides.SetUsesKeyForSubject(false);
				InControlRig->LinkOverrideAsset(OverrideAsset);
				const FSoftObjectPath SoftObjectPath(OverrideAsset->GetOutermost());
				static const FString Message = TEXT("An override asset has been created.");
				FNotificationInfo Info(FText::FromString(Message));
				Info.bUseSuccessFailIcons = true;
				Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Info"));
				Info.bFireAndForget = true;
				Info.bUseThrobber = true;
				Info.FadeOutDuration = 8.0f;
				Info.ExpireDuration = Info.FadeOutDuration;
				Info.Hyperlink = FSimpleDelegate::CreateLambda([SoftObjectPath] {
					// Select the cloud in Content Browser when the hyperlink is clicked
					TArray<FAssetData> AssetData;
					AssetData.Add(FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get().GetAssetByObjectPath(SoftObjectPath.GetWithoutSubPath()));
					FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get().SyncBrowserToAssets(AssetData);
					});
				Info.HyperlinkText = FText::FromString(FPaths::GetBaseFilename(SoftObjectPath.ToString()));
										
				TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
		return InControlRig->GetLastOverrideAsset();
	}
}
