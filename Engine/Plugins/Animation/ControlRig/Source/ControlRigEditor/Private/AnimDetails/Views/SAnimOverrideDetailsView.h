// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Overrides/OverrideStatusSubject.h"

class IDetailsView;
struct FPropertyAndParent;
class UControlRig;
class UControlRigOverrideAsset;
class FOverrideStatusDetailsViewObjectFilter;

namespace UE::ControlRigEditor
{
	class SAnimDetailsSearchBox;

	class SAnimOverrideDetailsView
		: public SCompoundWidget
	{
		SLATE_DECLARE_WIDGET(SAnimOverrideDetailsView, SCompoundWidget)
		
		SLATE_BEGIN_ARGS(SAnimOverrideDetailsView)
			{}
		SLATE_EVENT(FSimpleDelegate, OnRequestRefreshDetails)
		
		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs);

	private:
		/** Refreshes the details view */
		void RefreshDetailsView();

		/** Returns ture if the property should be displayed */
		bool ShouldDisplayProperty(const FPropertyAndParent& InPropertyAndParent) const;
		
		/** Returns ture if the property is ready-only */
		bool IsReadOnlyProperty(const FPropertyAndParent& InPropertyAndParent) const;

		/** Called when editing a property in the override details panel has finished */ 
		void OnFinishedChangingOverride(const FPropertyChangedEvent& PropertyChangedEvent);

		/** Returns true if two objects can be merged for display on the details panel */
		bool CanMergeObjects(const UObject* InObjectA, const UObject* InObjectB) const;

		/** Returns true if we can create the override widget for a given subject */
		bool CanCreateWidget(const FOverrideStatusSubject& InSubject) const;

		/** Returns the override status for a given subject */
		EOverrideWidgetStatus::Type GetOverrideStatus(const FOverrideStatusSubject& InSubject) const;

		/** React to the user interface request to add an override on a given subject */
		FReply OnAddOverride(const FOverrideStatusSubject& InSubject);

		/** React to the user interface request to clear all overrides on a given subject */
		FReply OnClearOverride(const FOverrideStatusSubject& InSubject);

		/** Updates a property path from what it is on the proxy to what it is on the control settings */
		void MapPropertyFromProxyToControl(FString& InOutPropertyPath) const;

		/** Returns the override asset for a given control rig to record changes to*/
		static UControlRigOverrideAsset* GetOrCreateOverrideAsset(UControlRig* InControlRig);
		
		/** Weak ptr to the details view this widget is displaying */
		TWeakPtr<IDetailsView> WeakDetailsView;

		/** The object filter used to show the objects in the override details */
		TSharedPtr<FOverrideStatusDetailsViewObjectFilter> ObjectFilter;
		
		/** A map from proxy property path to control settings property path */
		TMap<FString,FString> ProxyPropertyToControl;

		/** A delegate fired when we request to refresh the details */
		FSimpleDelegate RequestRefreshDetailsDelegate;
	};
}
