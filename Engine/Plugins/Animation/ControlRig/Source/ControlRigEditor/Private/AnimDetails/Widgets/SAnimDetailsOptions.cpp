// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDetailsOptions.h"

#include "AnimDetails/AnimDetailsOptionsMenuContext.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "LevelEditor.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SAnimDetailsOptions"

namespace UE::ControlRigEditor
{
	void SAnimDetailsOptions::Construct(const FArguments& InArgs)
	{
		OnOptionsChangedDelegate = InArgs._OnOptionsChanged;

		ChildSlot
		[
			SNew(SComboButton)
			.HasDownArrow(false)
			.ContentPadding(0.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
			.MenuContent()
			[
				MakeOptionsMenu()
			]
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("DetailsView.ViewOptions"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	TSharedRef<SWidget> SAnimDetailsOptions::MakeOptionsMenu()
	{
		constexpr const TCHAR* AnimDetailsOptionsMenuName = TEXT("AnimDetails.Options");

		if (!UToolMenus::Get()->IsMenuRegistered(AnimDetailsOptionsMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AnimDetailsOptionsMenuName);

			Menu->AddDynamicSection("PopulateAnimDetailsMenu", FNewToolMenuDelegate::CreateStatic(&SAnimDetailsOptions::PopulateMenu));
		}

		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();

		UAnimDetailsOptionsMenuContext* ContextObject = NewObject<UAnimDetailsOptionsMenuContext>();
		ContextObject->WeakAnimDetailsOptions = SharedThis(this);

		const FToolMenuContext MenuContext(CommandList, TSharedPtr<FExtender>(), ContextObject);
		UToolMenu* ToolMenu = UToolMenus::Get()->GenerateMenu(AnimDetailsOptionsMenuName, MenuContext);

		return UToolMenus::Get()->GenerateWidget(AnimDetailsOptionsMenuName, MenuContext);
	}

	void SAnimDetailsOptions::PopulateMenu(UToolMenu* InMenu)
	{
		constexpr const TCHAR* AnimDetailsOptionsMenuName = TEXT("AnimDetails.Options");

		UAnimDetailsOptionsMenuContext* ContextObject = InMenu ? InMenu->FindContext<UAnimDetailsOptionsMenuContext>() : nullptr;
		SAnimDetailsOptions* This = ContextObject && ContextObject->WeakAnimDetailsOptions.IsValid() ?
			ContextObject->WeakAnimDetailsOptions.Pin().Get() :
			nullptr;

		if (!InMenu ||
			!This)
		{
			return;
		}

		FToolMenuSection& OptionsSection = InMenu->AddSection("Options");
		{
			constexpr bool bIndent = false;
			constexpr bool bSearchable = false;
			constexpr bool bNoPadding = true;

			// Expand transform properties option
			{
				const TSharedRef<SWidget> ExpandTransformPropertiesOptionWidget =
					SNew(SBox)
					.MinDesiredWidth(40.f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Right)
						.IsChecked(This, &SAnimDetailsOptions::GetExpandTransformPropertiesCheckState)
						.OnCheckStateChanged(This, &SAnimDetailsOptions::OnExpandTransformPropertiesCheckStateChanged)
					];

				OptionsSection.AddEntry(
					FToolMenuEntry::InitWidget(
						"ExpandTransformPropertiesOptionWidget",
						ExpandTransformPropertiesOptionWidget,
						LOCTEXT("ExpandTransformPropertiesLabel", "Expand Transform Properties"),
						bIndent,
						bSearchable,
						bNoPadding,
						LOCTEXT("ExpandTransformPropertiesTooltip", "When checked, expands Location, Rotation, and Scale properties in anim details"))
					);
			}

			// Num fractional digits option
			{
				const TSharedRef<SWidget> NumFractionalDigitsOptionWidget =
					SNew(SEditableTextBox)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.Justification(ETextJustify::Right)
					.MinDesiredWidth(40.f)
					.SelectAllTextWhenFocused(true)
					.Text(This, &SAnimDetailsOptions::GetNumFractionalDigitsText)
					.OnTextCommitted(This, &SAnimDetailsOptions::OnNumFractionalDigitsCommitted);

				OptionsSection.AddEntry(
					FToolMenuEntry::InitWidget(
						"NumFractionalDigitsOptionWidget", 
						NumFractionalDigitsOptionWidget,
						LOCTEXT("NumFractionalDigitsLabel", "Num Fractional Digits"),
						bIndent,
						bSearchable,
						bNoPadding,
						LOCTEXT("NumFractionalDigitsTooltip", "Sets the displayed num fractional digits"))
				);
			}

			// LMB selects range option
			{
				const TSharedRef<SWidget> LMBSelectsRangeOptionWidget =
					SNew(SBox)
					.MinDesiredWidth(40.f)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Right)
						.IsChecked(This, &SAnimDetailsOptions::GetLMBSelectsRangeCheckState)
						.OnCheckStateChanged(This, &SAnimDetailsOptions::OnLMBSelectsRangeCheckStateChanged)
					];

				OptionsSection.AddEntry(
					FToolMenuEntry::InitWidget(
						"LMBSelectsRangeOptionWidget",
						LMBSelectsRangeOptionWidget,
						LOCTEXT("LMBSelectsRangeLabel", "LMB selects range"),
						bIndent,
						bSearchable,
						bNoPadding,
						LOCTEXT("LMBSelectsRangeTooltip", "When checked, selects a range when the left mouse button is down"))
					);
			}
		}
	}

	ECheckBoxState SAnimDetailsOptions::GetExpandTransformPropertiesCheckState() const
	{
		return GetDefault<UAnimDetailsSettings>()->bExpandTransformProperties ?
			ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SAnimDetailsOptions::OnExpandTransformPropertiesCheckStateChanged(const ECheckBoxState InCheckBoxState)
	{
		UAnimDetailsSettings* const Settings = GetMutableDefault<UAnimDetailsSettings>();
		Settings->bExpandTransformProperties = InCheckBoxState == ECheckBoxState::Checked;
		Settings->SaveConfig();

		OnOptionsChangedDelegate.ExecuteIfBound();
	}

	FText SAnimDetailsOptions::GetNumFractionalDigitsText() const
	{
		const int32 NumFractionalDigits = GetDefault<UAnimDetailsSettings>()->NumFractionalDigits;
		return FText::FromString(FString::FromInt(NumFractionalDigits));
	}

	void SAnimDetailsOptions::OnNumFractionalDigitsCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
	{
		uint8 Value = 0.f;
		if (LexTryParseString(Value, *NewText.ToString()))
		{
			UAnimDetailsSettings* Settings = GetMutableDefault<UAnimDetailsSettings>();
			Settings->NumFractionalDigits = Value;

			Settings->SaveConfig();

			OnOptionsChangedDelegate.ExecuteIfBound();
		}
	}

	ECheckBoxState SAnimDetailsOptions::GetLMBSelectsRangeCheckState() const
	{
		return GetDefault<UAnimDetailsSettings>()->bLMBSelectsRange ?
			ECheckBoxState::Checked :
			ECheckBoxState::Unchecked;
	}

	void SAnimDetailsOptions::OnLMBSelectsRangeCheckStateChanged(ECheckBoxState CheckBoxState)
	{
		UAnimDetailsSettings* Settings = GetMutableDefault<UAnimDetailsSettings>();
		Settings->bLMBSelectsRange = CheckBoxState == ECheckBoxState::Checked ? true : false;

		Settings->SaveConfig();

		OnOptionsChangedDelegate.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE
