// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UToolMenu;

namespace UE::ControlRigEditor
{
	/** The options menu for anim details */
	class SAnimDetailsOptions
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimDetailsOptions)
			{}

			/** Event raised when options changed */
			SLATE_EVENT(FSimpleDelegate, OnOptionsChanged)

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs);

	private:
		/** Creates the options menu */
		TSharedRef<SWidget> MakeOptionsMenu();

		/** Populates the menu */
		static void PopulateMenu(UToolMenu* InMenu);

		/** Returns the check box state for the expand transform properties option */
		ECheckBoxState GetExpandTransformPropertiesCheckState() const;

		/** Called when the check state for the expand transform properties option changed */
		void OnExpandTransformPropertiesCheckStateChanged(const ECheckBoxState InCheckBoxState);

		/** Returns text for the num fractional digits option */
		FText GetNumFractionalDigitsText() const;

		/** Called when num fractional digits were committed */
		void OnNumFractionalDigitsCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

		/** Returns the check box state for the LMB selects range option */
		ECheckBoxState GetLMBSelectsRangeCheckState() const;

		/** Called when the check state for the LMB selects range option changed */
		void OnLMBSelectsRangeCheckStateChanged(ECheckBoxState CheckBoxState);

		/** Delegate executed when options changed */
		FSimpleDelegate OnOptionsChangedDelegate;
	};
}
