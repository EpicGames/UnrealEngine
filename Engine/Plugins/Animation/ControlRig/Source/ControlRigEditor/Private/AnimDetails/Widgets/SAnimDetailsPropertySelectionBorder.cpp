// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDetailsPropertySelectionBorder.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "CurveEditor.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ISequencer.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "PropertyHandle.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tree/SCurveEditorTree.h"

namespace UE::ControlRigEditor
{
	namespace Private
	{
		/** 
		 * Handles the LMB pressed scope of an interactive selection. 
		 * 
		 * Sets keyboard focus on the instigator widget so it can block any concurring keyboard input 
		 * by testing IsChangingSelection OnKeyDown.
		 * 
		 * Releases the keyboard focus when the interaction ended.
		 */
		class FAnimDetailsInteractiveSelection
		{
		public:
			/** Creates an interactive anim details selection scope that is ongoing while LMB is down */
			static void LMBDownScopeInteractiveSelection(const TSharedRef<SWidget>& InstigatorWidget)
			{
				FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
				UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

				const bool bLMBDown = FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton);

				if (ProxyManager &&
					ensureMsgf(bLMBDown, TEXT("Trying to create an interactive selection, but LMB is not pressed. This is not supported.")))
				{
					Instance = MakeUnique<FAnimDetailsInteractiveSelection>();

					FCoreDelegates::OnEndFrame.AddRaw(Instance.Get(), &FAnimDetailsInteractiveSelection::TickInteractiveSelection);
				}
			}

			/** True while an interactive change is ongoing */
			static bool IsChangingSelection()
			{
				return Instance.IsValid();
			}

		private:
			/** Ticks the interactive selection */
			void TickInteractiveSelection()
			{
				// The scope ends when LMB was released
				if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
				{
					EndInteractiveSelection();
				}
			}

			/** Ends the interactive selection */
			static void EndInteractiveSelection()
			{
				if (Instance.IsValid())
				{
					FCoreDelegates::OnEndFrame.RemoveAll(Instance.Get());

					Instance.Reset();
				}
			}

			/** The instance, valid while an interactive change is ongoing */
			static TUniquePtr<FAnimDetailsInteractiveSelection> Instance;
		};

		TUniquePtr<FAnimDetailsInteractiveSelection> FAnimDetailsInteractiveSelection::Instance;
	}

	void SAnimDetailsPropertySelectionBorder::Construct(
		const FArguments& InArgs, 
		UAnimDetailsProxyManager& InProxyManager, 
		const TSharedRef<IPropertyHandle>& PropertyHandle)
	{
		SetCanTick(true);

		WeakProxyManager = &InProxyManager;
		PropertyName = PropertyHandle->GetProperty()->GetFName();
			
		if (InArgs._NavigateToWidget.IsValid())
		{
			NavigableWidgetRegistrar.RegisterAsNavigator(InProxyManager, AsShared(), InArgs._NavigateToWidget.ToSharedRef());
		}

		TArray<UObject*> OuterObjects;
		PropertyHandle->GetOuterObjects(OuterObjects);

		Algo::Transform(OuterObjects, WeakProxies,
			[](UObject* OuterObject)
			{
				return CastChecked<UAnimDetailsProxyBase>(OuterObject);
			});	

		bRequiresModifierKeys = InArgs._RequiresModifierKeys;

		ChildSlot
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
				.BorderBackgroundColor_Lambda([this]()
					{
						const UAnimDetailsProxyBase* AnyProxy = WeakProxies.IsEmpty() || !WeakProxies[0].IsValid() ? nullptr : WeakProxies[0].Get();
						if (AnyProxy && IsSelected(AnyProxy))
						{
							return  FStyleColors::Select;
						}
						else 
						{
							return FStyleColors::Transparent;
						}
					})
				.Content()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						InArgs._Content.Widget
					]
				]
			]

			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.Visibility_Lambda([this]()
					{
						if (bRequiresModifierKeys)
						{
							const bool bModifierKeyDown = FSlateApplication::Get().GetModifierKeys().IsControlDown() || FSlateApplication::Get().GetModifierKeys().IsShiftDown();
							return bModifierKeyDown ? EVisibility::Visible : EVisibility::Collapsed;
						}
						else
						{
							return EVisibility::Visible;
						}
					})
				.OnMouseButtonDown(this, &SAnimDetailsPropertySelectionBorder::OnBorderMouseButtonDown)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
			]
		];
	}

	void SAnimDetailsPropertySelectionBorder::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		using namespace UE::ControlRigEditor::Private;

		// Only process if bLMBSelectsRange is enabled in settings and an interactive selection is ongoing
		const UAnimDetailsSettings* Settings = GetDefault<UAnimDetailsSettings>();
		if (!Settings || !Settings->bLMBSelectsRange || !FAnimDetailsInteractiveSelection::IsChangingSelection())
		{
			return;
		}

		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		UAnimDetailsSelection* Selection = ProxyManager ? ProxyManager->GetAnimDetailsSelection() : nullptr;
		if (!EditMode || !ProxyManager || !Selection)
		{
			return;
		}

		const TArray<UAnimDetailsProxyBase*> Proxies = MakeProxyArray();
		if (!Proxies.IsEmpty() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			Selection->SelectPropertyInProxies(Proxies, PropertyName, EAnimDetailsSelectionType::SelectRange);
		}
	}

	FReply SAnimDetailsPropertySelectionBorder::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		using namespace UE::ControlRigEditor::Private;

		if (InKeyEvent.GetKey() == EKeys::Tab)
		{
			// Navigate to the value widget
			const TSharedPtr<SWidget> NavigateToWidget = WeakProxyManager.IsValid() ?
				WeakProxyManager->GetNavigableWidgetRegistry().GetNext(AsShared()) :
				nullptr;

			if (NavigateToWidget.IsValid())
			{
				return FReply::Handled().SetNavigation(NavigateToWidget.ToSharedRef(), ENavigationGenesis::Keyboard);
			}
		}
		
		// Block any keyboard besides tab-navigating to prevent from any keyboard shortcuts while interactively changing selection
		if (FAnimDetailsInteractiveSelection::IsChangingSelection())
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	FReply SAnimDetailsPropertySelectionBorder::OnBorderMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		using namespace UE::Sequencer;
		using namespace UE::ControlRigEditor::Private;

		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!EditMode)
		{
			return FReply::Handled();
		}

		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			UAnimDetailsProxyManager* ProxyManager = EditMode->GetAnimDetailsProxyManager();
			UAnimDetailsSelection* Selection = ProxyManager ? ProxyManager->GetAnimDetailsSelection() : nullptr;
			const TSharedPtr<ISequencer> ISequencer = ProxyManager ? ProxyManager->GetSequencer() : nullptr;
			if (!ProxyManager || !Selection || !ISequencer)
			{
				return FReply::Handled();
			}

			TArray<UAnimDetailsProxyBase*> Proxies;
			Algo::TransformIf(WeakProxies, Proxies,
				[](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
				{
					return WeakProxy.IsValid();
				},
				[](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
				{
					return WeakProxy.Get();
				});

			const bool bIsShiftDown = MouseEvent.GetModifierKeys().IsShiftDown();
			const bool bIsCtrlDown = MouseEvent.GetModifierKeys().IsControlDown();

			const EAnimDetailsSelectionType SelectionType = bIsShiftDown ?
				EAnimDetailsSelectionType::SelectRange :
				(bIsCtrlDown ? EAnimDetailsSelectionType::Toggle : EAnimDetailsSelectionType::Select);


			// Clear selection when not ctrl or shift selecting 
			const bool bClearSelection = SelectionType == EAnimDetailsSelectionType::Select && Selection->GetNumSelectedProperties() > 0;
			if (bClearSelection)
			{
				Selection->ClearSelection();
			}

			FAnimDetailsInteractiveSelection::LMBDownScopeInteractiveSelection(AsShared());
			Selection->SelectPropertyInProxies(Proxies, PropertyName, SelectionType);
		}

		// Always handle clicks, it should not get to the details row below.
		return FReply::Handled();
	}

	bool SAnimDetailsPropertySelectionBorder::IsSelected(const UAnimDetailsProxyBase* Proxy) const
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		UAnimDetailsSelection* Selection = ProxyManager ? ProxyManager->GetAnimDetailsSelection() : nullptr;

		return Selection ? Selection->IsPropertySelected(Proxy, PropertyName) : false;
	}

	const TArray<UAnimDetailsProxyBase*> SAnimDetailsPropertySelectionBorder::MakeProxyArray() const
	{
		TArray<UAnimDetailsProxyBase*> Proxies;
		Algo::TransformIf(WeakProxies, Proxies,
			[](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
			{
				return WeakProxy.IsValid();
			},
			[](const TWeakObjectPtr<UAnimDetailsProxyBase>& WeakProxy)
			{
				return WeakProxy.Get();
			});

		return Proxies;
	}
}
