// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetails/AnimDetailsNavigableWidgetRegistrar.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class UAnimDetailsProxyBase;
class UAnimDetailsProxyManager;

namespace UE::ControlRigEditor
{
	enum class EAnimDetailsSelectionType : uint8;

	/** Widget that can wrap a property with a border so it can be selected in anim details */
	class SAnimDetailsPropertySelectionBorder
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimDetailsPropertySelectionBorder)
			: _RequiresModifierKeys(false)
			{}

			/** The content that can be selected */
			SLATE_DEFAULT_SLOT(FArguments, Content)

			/** If set to true, selection only occurs when a modifier key is pressed */
			SLATE_ARGUMENT(bool, RequiresModifierKeys)

			/** A widget that is navigated when tab-clicking on this border */
			SLATE_ARGUMENT(TSharedPtr<SWidget>, NavigateToWidget)

		SLATE_END_ARGS()

		/** 
		 * Constructs this widget
		 * 
		 * @param InArgs					Slate arguments
		 * @param ProxyManager				The proxy manager that owns the proxies
		 * @param PropertyHandle			Property handle for the property that can be selected
		 */
		void Construct(
			const FArguments& InArgs, 
			UAnimDetailsProxyManager& InProxyManager, 
			const TSharedRef<IPropertyHandle>& PropertyHandle);

	protected:
		//~ Begin SWidget interface
		virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual bool SupportsKeyboardFocus() const override { return true; }
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		//~ End SWidget interface

	private:
		/** Called when the border receives a mouse button down event */
		FReply OnBorderMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

		/** Tests if the specified proxy is selected */
		bool IsSelected(const UAnimDetailsProxyBase* Proxy) const;

		/** Convenience function to create an array of proxy object ptrs */
		const TArray<UAnimDetailsProxyBase*> MakeProxyArray() const;

		/** The proxy manager that owns the proxies */
		TWeakObjectPtr<UAnimDetailsProxyManager> WeakProxyManager;

		/** Weak proxy objects the property edits */
		TArray<TWeakObjectPtr<UAnimDetailsProxyBase>> WeakProxies;

		/** The name of the property that can be selected */
		FName PropertyName;

		/** Registrar for the navigable widget registry */
		FAnimDetailsNavigableWidgetRegistrar NavigableWidgetRegistrar;

		/** If ture, selection should only occur when a modifier key is pressed */
		bool bRequiresModifierKeys = false;
	};
}
