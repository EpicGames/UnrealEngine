// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDetailsSearchBox.h"

#include "EditorModeManager.h"
#include "Widgets/Input/SSearchBox.h"

namespace UE::ControlRigEditor
{
	void SAnimDetailsSearchBox::Construct(const FArguments& InArgs)
	{
		OnSearchTextChangedDelegate = InArgs._OnSearchTextChanged;

		ChildSlot
		[
			SNew(SSearchBox)
			.OnTextChanged(this, &SAnimDetailsSearchBox::OnSearchTextChanged)
			.OnTextCommitted(this, &SAnimDetailsSearchBox::OnSearchTextCommitted)
		];

		// Reset the search text as if the user committed it
		OnSearchTextCommitted(FText::GetEmpty(), ETextCommit::Default);
	}

	void SAnimDetailsSearchBox::OnSearchTextChanged(const FText& NewText)
	{
		SearchText = NewText;

		OnSearchTextChangedDelegate.ExecuteIfBound();
	}

	void SAnimDetailsSearchBox::OnSearchTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
	{
		SearchText = NewText;

		OnSearchTextChangedDelegate.ExecuteIfBound();
	}
}
