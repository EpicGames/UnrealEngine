// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::ControlRigEditor
{
	/** The search widget to filter anim details */
	class SAnimDetailsSearchBox
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimDetailsSearchBox)
			{}

			/** Event raised when the search text changed */
			SLATE_EVENT(FSimpleDelegate, OnSearchTextChanged)

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs);

		/** Returns the current search text */
		const FText& GetSearchText() const { return SearchText; }

	private:
		/** Called when the search text changed */
		void OnSearchTextChanged(const FText& NewText);

		/** Called when the search text was committed */
		void OnSearchTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
		
		/** The current search text */
		FText SearchText;

		/** Delegate executed when the search text changed */
		FSimpleDelegate OnSearchTextChangedDelegate;
	};
}
