// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDetailsValueBoolean.h"

#include "AnimDetails/AnimDetailsMultiEditUtil.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "PropertyHandle.h"
#include "SAnimDetailsPropertySelectionBorder.h"
#include "Widgets/Input/SCheckBox.h"

namespace UE::ControlRigEditor
{
	SAnimDetailsValueBoolean::~SAnimDetailsValueBoolean()
	{
		FAnimDetailsMultiEditUtil::Get().Leave(WeakPropertyHandle);
	}

	void SAnimDetailsValueBoolean::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
		if (!ProxyManager)
		{
			return;
		}

		WeakProxyManager = ProxyManager;
		WeakPropertyHandle = InPropertyHandle;

		CheckBox = SNew(SCheckBox)
			.Type(ESlateCheckBoxType::CheckBox)
			.IsChecked(this, &SAnimDetailsValueBoolean::GetCheckState)
			.OnCheckStateChanged(this, &SAnimDetailsValueBoolean::OnCheckStateChanged);

		ChildSlot
		[
			SAssignNew(SelectionBorder, SAnimDetailsPropertySelectionBorder, *ProxyManager, InPropertyHandle)
			.RequiresModifierKeys(true)
			[
				CheckBox.ToSharedRef()
			]
		];

		NavigableWidgetRegistrar.RegisterAsNavigable(*ProxyManager, AsShared(), CheckBox.ToSharedRef());

		FAnimDetailsMultiEditUtil::Get().Join(ProxyManager, InPropertyHandle);
	}

	FNavigationReply SAnimDetailsValueBoolean::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
	{
		if (InNavigationEvent.GetNavigationType() != EUINavigation::Next)
		{
			return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
		}
		else
		{
			return FNavigationReply::Custom(FNavigationDelegate::CreateLambda([WeakThis = AsWeak(), this](EUINavigation UINaviagation)
				{
					if (!WeakThis.IsValid() ||
						!WeakProxyManager.IsValid() ||
						!CheckBox.IsValid())
					{
						return SNullWidget::NullWidget;
					}

					const TSharedPtr<SWidget> NextWidget = WeakProxyManager->GetNavigableWidgetRegistry().GetNext(AsShared());
					if (!NextWidget.IsValid())
					{
						return SNullWidget::NullWidget;
					}

					return NextWidget.ToSharedRef();
				}));
		}
	}

	ECheckBoxState SAnimDetailsValueBoolean::GetCheckState() const
	{
		bool bChecked{};
		if (WeakPropertyHandle.IsValid() && 
			WeakPropertyHandle.Pin()->GetValue(bChecked) == FPropertyAccess::Success)
		{
			return bChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Undetermined;
	}

	void SAnimDetailsValueBoolean::OnCheckStateChanged(ECheckBoxState CheckBoxState)
	{
		if (WeakProxyManager.IsValid() && WeakPropertyHandle.IsValid())
		{
			const bool bEnabled = CheckBoxState == ECheckBoxState::Checked;
			UE::ControlRigEditor::FAnimDetailsMultiEditUtil::Get().MultiEditSet(*WeakProxyManager.Get(), bEnabled, WeakPropertyHandle.Pin().ToSharedRef());
		}
	}
}
