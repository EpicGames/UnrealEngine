// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetails/AnimDetailsNavigableWidgetRegistrar.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class SCheckBox;
class UAnimDetailsProxyManager;

namespace UE::ControlRigEditor
{
	class SAnimDetailsPropertySelectionBorder;

	class SAnimDetailsValueBoolean
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimDetailsValueBoolean)
			{}

			/** The label color of the widget, only when displaying values */
			SLATE_ARGUMENT(FLinearColor, LabelColor)

		SLATE_END_ARGS()

		virtual ~SAnimDetailsValueBoolean();

		/** 
		 * Constructs this widget
		 * 
		 * @param ProxyManager					The proxy manager that holds the displayed property
		 * @param StructurePropertyHandle		PropertyHandle or the outer struct prooperty
		 * @param PropertyHandle				The property that will be displayed
 		 */
		void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& PropertyHandle);

	private:
		//~ Begin SWidget Interface
		virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;
		//~ End SWidget Interface

		/** Returns the current check box state */
		ECheckBoxState GetCheckState() const;

		/** Called when the check box state changed */
		void OnCheckStateChanged(ECheckBoxState CheckBoxState);

		/** The check box used to display the boolean state */
		TSharedPtr<SCheckBox> CheckBox;

		/** Border that handles the selction of the displayed property */
		TSharedPtr<SAnimDetailsPropertySelectionBorder> SelectionBorder;

		/** Handles registration of the inner widget that can be navigated to */
		FAnimDetailsNavigableWidgetRegistrar NavigableWidgetRegistrar;

		/** The proxy manager that holds the displayed property */
		TWeakObjectPtr<UAnimDetailsProxyManager> WeakProxyManager;

		/** The property handle that is being edited */
		TWeakPtr<IPropertyHandle> WeakPropertyHandle;
	};
}
