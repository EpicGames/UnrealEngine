// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimDetailsValueNumeric.h"

#include "AnimDetails/AnimDetailsMultiEditUtil.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSettings.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBase.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "SAnimDetailsValueNumeric"

namespace UE::ControlRigEditor
{
	namespace PropertyUtils
	{
		/** Struct to extract meta data from a property handle */
		template <typename NumericType>
		struct FAnimDetailsPropertyMetadata
		{
			FAnimDetailsPropertyMetadata(const TSharedRef<IPropertyHandle>& PropertyHandle);

			TOptional<NumericType> MinValue;
			TOptional<NumericType> MaxValue;
			TOptional<NumericType> SliderMinValue;
			TOptional<NumericType> SliderMaxValue;
			double SliderExponent = 1.0;
			double Delta = 0.0;
			float ShiftMultiplier = 10.f;
			float CtrlMultiplier = 0.1f;
			bool SupportDynamicSliderMaxValue = false;
			bool SupportDynamicSliderMinValue = false;
		};

		template <typename NumericType>
		FAnimDetailsPropertyMetadata<NumericType>::FAnimDetailsPropertyMetadata(const TSharedRef<IPropertyHandle>& PropertyHandle)
		{
			const FString& MetaUIMinString = PropertyHandle->GetMetaData(TEXT("UIMin"));
			const FString& MetaUIMaxString = PropertyHandle->GetMetaData(TEXT("UIMax"));
			const FString& SliderExponentString = PropertyHandle->GetMetaData(TEXT("SliderExponent"));
			const FString& DeltaString = PropertyHandle->GetMetaData(TEXT("Delta"));
			const FString& ShiftMultiplierString = PropertyHandle->GetMetaData(TEXT("ShiftMultiplier"));
			const FString& CtrlMultiplierString = PropertyHandle->GetMetaData(TEXT("CtrlMultiplier"));
			const FString& SupportDynamicSliderMaxValueString = PropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
			const FString& SupportDynamicSliderMinValueString = PropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
			const FString& ClampMinString = PropertyHandle->GetMetaData(TEXT("ClampMin"));
			const FString& ClampMaxString = PropertyHandle->GetMetaData(TEXT("ClampMax"));

			// If no UIMin/Max was specified then use the clamp string
			const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
			const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

			NumericType ClampMin = TNumericLimits<NumericType>::Lowest();
			NumericType ClampMax = TNumericLimits<NumericType>::Max();

			if (!ClampMinString.IsEmpty())
			{
				TTypeFromString<NumericType>::FromString(ClampMin, *ClampMinString);
			}

			if (!ClampMaxString.IsEmpty())
			{
				TTypeFromString<NumericType>::FromString(ClampMax, *ClampMaxString);
			}

			NumericType UIMin = TNumericLimits<NumericType>::Lowest();
			NumericType UIMax = TNumericLimits<NumericType>::Max();
			TTypeFromString<NumericType>::FromString(UIMin, *UIMinString);
			TTypeFromString<NumericType>::FromString(UIMax, *UIMaxString);


			if (SliderExponentString.Len())
			{
				TTypeFromString<double>::FromString(SliderExponent, *SliderExponentString);
			}

			if (DeltaString.Len())
			{
				TTypeFromString<double>::FromString(Delta, *DeltaString);
			}

			if (ShiftMultiplierString.Len())
			{
				TTypeFromString<float>::FromString(ShiftMultiplier, *ShiftMultiplierString);
			}

			if (CtrlMultiplierString.Len())
			{
				TTypeFromString<float>::FromString(CtrlMultiplier, *CtrlMultiplierString);
			}

			const double ActualUIMin = FMath::Max(UIMin, ClampMin);
			const double ActualUIMax = FMath::Min(UIMax, ClampMax);

			MinValue = ClampMinString.Len() ? ClampMin : TOptional<NumericType>();
			MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<NumericType>();
			SliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<NumericType>();
			SliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<NumericType>();

			SupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
			SupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
		}
	}

	/** Numeric type interface for anim details spin boxes */
	template <typename NumericType>
	class TAnimDetailsNumericTypeInterface
		: public TDefaultNumericTypeInterface<NumericType>
	{
	public:
		/** Constructs this type interface */
		TAnimDetailsNumericTypeInterface(const TSharedRef<SAnimDetailsValueNumeric<NumericType>>& InValueWidget)
			: WeakValueWidget(InValueWidget)
		{}

		//~ Begin TDefaultNumericTypeInterface interface
		virtual TOptional<NumericType> FromString(const FString& InString, const NumericType& InExistingValue) override
		{
			if (!WeakValueWidget.IsValid())
			{
				return TOptional<NumericType>();
			}
			WeakValueWidget.Pin()->SetDisplayText(FText::FromString(InString));
			
			return TDefaultNumericTypeInterface<NumericType>::FromString(InString, InExistingValue);
		}
		//~ End TDefaultNumericTypeInterface interface

	private:
		/** The weak property to operate on */
		const TWeakPtr<SAnimDetailsValueNumeric<NumericType>> WeakValueWidget;
	};


	template class SAnimDetailsValueNumeric<double>;
	template class SAnimDetailsValueNumeric<int64>;

	template <typename NumericType>
	SAnimDetailsValueNumeric<NumericType>::~SAnimDetailsValueNumeric()
	{
		FAnimDetailsMultiEditUtil::Get().Leave(WeakPropertyHandle);
	}

	template SAnimDetailsValueNumeric<double>::~SAnimDetailsValueNumeric();
	template SAnimDetailsValueNumeric<int64>::~SAnimDetailsValueNumeric();

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle)
	{
		FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;

		WeakProxyManager = ProxyManager;
		WeakPropertyHandle = InPropertyHandle;

		Label = InArgs._Label;
		LabelVAlign = InArgs._LabelVAlign;
		LabelLocation = InArgs._LabelLocation;
		LabelPadding = InArgs._LabelPadding;

		TypeInterface = MakeShared<TAnimDetailsNumericTypeInterface<NumericType>>(SharedThis(this));

		ForceRefresh();

		// Join multi editing functionality to make use of it
		FAnimDetailsMultiEditUtil::Get().Join(ProxyManager, InPropertyHandle);
	}

	template void SAnimDetailsValueNumeric<double>::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);
	template void SAnimDetailsValueNumeric<int64>::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::RequestRefresh()
	{
		if (!RefreshTimerHandle.IsValid())
		{
			RefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SAnimDetailsValueNumeric<NumericType>::ForceRefresh));
		}
	}

	template <typename NumericType>
	FNavigationReply SAnimDetailsValueNumeric<NumericType>::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
	{
		if (InNavigationEvent.GetNavigationType() != EUINavigation::Next)
		{
			return SCompoundWidget::OnNavigation(MyGeometry, InNavigationEvent);
		}
		else
		{
			return FNavigationReply::Custom(FNavigationDelegate::CreateLambda([WeakThis = AsWeak(), this](EUINavigation UINaviagation)
				{
					if (!WeakThis.IsValid() || 
						!WeakProxyManager.IsValid() || 
						!SpinBox.IsValid())
					{
						return SNullWidget::NullWidget;
					}

					const TSharedPtr<SWidget> NextWidget = WeakProxyManager->GetNavigableWidgetRegistry().GetNext(AsShared());
					if (!NextWidget.IsValid())
					{
						return SNullWidget::NullWidget;
					}

					return NextWidget.ToSharedRef();
				}));
		}
	}

	template FNavigationReply SAnimDetailsValueNumeric<double>::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent);
	template FNavigationReply SAnimDetailsValueNumeric<int64>::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent);

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::ForceRefresh()
	{
		RefreshTimerHandle.Invalidate();

		if (!WeakProxyManager.IsValid() ||
			!WeakPropertyHandle.IsValid())
		{
			ChildSlot
			[
				SNullWidget::NullWidget
			];

			return;
		}
		UAnimDetailsProxyManager* ProxyManager = WeakProxyManager.Get();

		const TSharedRef<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin().ToSharedRef();
		const PropertyUtils::FAnimDetailsPropertyMetadata<NumericType>& PropertyMetaData = PropertyUtils::FAnimDetailsPropertyMetadata<NumericType>(PropertyHandle);

		const uint8 NumFractionalDigits = GetDefault<UAnimDetailsSettings>()->NumFractionalDigits;

		SpinBox = SNew(SSpinBox<NumericType>)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Value(this, &SAnimDetailsValueNumeric<NumericType>::GetValue)
			.OnGetDisplayValue(this, &SAnimDetailsValueNumeric<NumericType>::OnGetDisplayValue)
			.OnEditEditableText(this, &SAnimDetailsValueNumeric<NumericType>::OnGetDisplayValue)
			.OnValueChanged(this, &SAnimDetailsValueNumeric<NumericType>::OnValueChanged)
			.OnValueCommitted(this, &SAnimDetailsValueNumeric<NumericType>::OnValueCommitted)
			.OnBeginSliderMovement(this, &SAnimDetailsValueNumeric<NumericType>::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &SAnimDetailsValueNumeric<NumericType>::OnEndSliderMovement)
			.MinFractionalDigits(NumFractionalDigits)
			.MaxFractionalDigits(NumFractionalDigits)
			.SupportDynamicSliderMinValue(PropertyMetaData.SupportDynamicSliderMinValue)
			.SupportDynamicSliderMaxValue(PropertyMetaData.SupportDynamicSliderMaxValue)
			.MinValue(PropertyMetaData.MinValue)
			.MaxValue(PropertyMetaData.MaxValue)
			.MinSliderValue(PropertyMetaData.SliderMinValue)
			.MaxSliderValue(PropertyMetaData.SliderMaxValue)
			.ShiftMultiplier(PropertyMetaData.ShiftMultiplier)
			.CtrlMultiplier(PropertyMetaData.CtrlMultiplier)
			.SliderExponent(PropertyMetaData.SliderExponent)
			.Delta(PropertyMetaData.Delta)
			.LinearDeltaSensitivity(1)
			.ClearKeyboardFocusOnCommit(false)
			.TypeInterface(TypeInterface);

		NavigableWidgetRegistrar.RegisterAsNavigable(*ProxyManager, AsShared(), SpinBox.ToSharedRef());

		const TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

		const bool bHasLabel = Label.Widget != SNullWidget::NullWidget;
		if (bHasLabel && LabelLocation == ELabelLocation::Inside)
		{
			ContentBox->AddSlot()
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					[
						SpinBox.ToSharedRef()
					]

					+ SOverlay::Slot()
					.HAlign(HAlign_Left)
					.VAlign(LabelVAlign)
					.Padding(LabelPadding)
					[
						Label.Widget
					]
				];
		}
		else if (bHasLabel && LabelLocation == ELabelLocation::Outside)
		{
			ContentBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(LabelVAlign)
				.Padding(LabelPadding)
				[
					Label.Widget
				];

			ContentBox->AddSlot()
				[
					SpinBox.ToSharedRef()
				];
		}
		else
		{
			ContentBox->AddSlot()
				[
					SpinBox.ToSharedRef()
				];
		}

		ChildSlot
		[
			ContentBox
		];
	}

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::SetDisplayText(const FText& Text)
	{
		DisplayText = Text;
	}

	template <typename NumericType>
	NumericType SAnimDetailsValueNumeric<NumericType>::GetValue() const
	{
		if (!WeakPropertyHandle.IsValid())
		{
			return NumericType();
		}
		const TSharedRef<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin().ToSharedRef();

		NumericType Value;
		if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
		{
			return Value;
		}
		else if (PropertyHandle->GetNumPerObjectValues() > 1)
		{				
			// Don't return an unset optional when multi editing.
			// Instead let the user edit the first object and let other objects and multi edited properties follow on value changes.

			FString FirstObjectValueString;
			WeakPropertyHandle.Pin()->GetPerObjectValue(0, FirstObjectValueString);

			if (!FirstObjectValueString.IsEmpty())
			{
				NumericType FirstObjectValue;
				TTypeFromString<NumericType>::FromString(FirstObjectValue, *FirstObjectValueString);

				return FirstObjectValue;
			}
		}

		return NumericType();
	}

	template <typename NumericType>
	TOptional<FText> SAnimDetailsValueNumeric<NumericType>::OnGetDisplayValue(NumericType SpinBoxValue)
	{
		NumericType Value;
		if (WeakPropertyHandle.IsValid() && 
			WeakPropertyHandle.Pin()->GetValue(Value) == FPropertyAccess::Success)
		{
			// Let the SAnimDetailsSpinBox decide how to draw the value
			return TOptional<FText>(); 
		}
		else
		{
			NumericType InteractiveDelta{};

			if constexpr (std::is_floating_point_v<NumericType>)
			{
				if (WeakPropertyHandle.IsValid() &&
					FAnimDetailsMultiEditUtil::Get().GetInteractiveDelta(WeakPropertyHandle.Pin().ToSharedRef(), InteractiveDelta))
				{
					const uint8 NumFractionalDigits = GetDefault<UAnimDetailsSettings>()->NumFractionalDigits;

					FNumberFormattingOptions Options;
					Options.MinimumFractionalDigits = NumFractionalDigits;
					Options.MaximumFractionalDigits = NumFractionalDigits;

					const bool bGreaterZero = InteractiveDelta >= 0.0;

					return bGreaterZero ?
						FText::Format(LOCTEXT("MultiEditAddToFloat", "Multiple Values: + {0}"), FText::AsNumber(InteractiveDelta, &Options)) :
						FText::Format(LOCTEXT("MultiEditSubstractFromFloat", "Multiple Values: - {0}"), FText::AsNumber(FMath::Abs(InteractiveDelta), &Options));
				}
			}
			else if constexpr (std::is_integral_v<NumericType>)
			{
				if (WeakPropertyHandle.IsValid() &&
					FAnimDetailsMultiEditUtil::Get().GetInteractiveDelta(WeakPropertyHandle.Pin().ToSharedRef(), InteractiveDelta))
				{
					const bool bGreaterZero = InteractiveDelta >= 0.0;

					return bGreaterZero ?
						FText::Format(LOCTEXT("MultiEditAddToInt", "Multiple Values: + {0}"), FText::AsNumber(InteractiveDelta)) :
						FText::Format(LOCTEXT("MultiEditSubstractFromInt", "Multiple Values: - {0}"), FText::AsNumber(FMath::Abs(InteractiveDelta)));
				}
			}
		}

		return LOCTEXT("MultipleValuesInfo", "Multiple Values");
	}

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::OnValueChanged(NumericType Value)
	{
		if (bIsUsingSlider)
		{
			constexpr bool bInteractive = true;
			MultiEditChangePropertyValue(Value, bInteractive);
		}
	}

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::OnValueCommitted(NumericType Value, ETextCommit::Type CommitType)
	{
		if (bIsUsingSlider)
		{
			constexpr bool bInteractive = false;
			MultiEditChangePropertyValue(Value, bInteractive);
		}
		else if (WeakProxyManager.IsValid() &&
			WeakPropertyHandle.IsValid())
		{
			const TSharedRef<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin().ToSharedRef();

			if (!FAnimDetailsMultiEditUtil::Get().MultiEditMath<NumericType>(*WeakProxyManager.Get(), DisplayText.ToString(), PropertyHandle))
			{
				FAnimDetailsMultiEditUtil::Get().MultiEditSet(*WeakProxyManager.Get(), Value, PropertyHandle);
			}
		}
	}

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::OnBeginSliderMovement()
	{
		bIsUsingSlider = true;
	}

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::OnEndSliderMovement(NumericType Value)
	{		
		bIsUsingSlider = false;
	}

	template <typename NumericType>
	void SAnimDetailsValueNumeric<NumericType>::MultiEditChangePropertyValue(const NumericType Value, const bool bInteractive)
	{
		if (!WeakPropertyHandle.IsValid())
		{
			return;
		}
		const TSharedRef<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin().ToSharedRef();

		if (bIsUsingSlider)
		{
			// Multi-edit the value
			const TOptional<NumericType> OldPropertyValue = [&PropertyHandle, this]() -> const TOptional<NumericType>
				{
					NumericType PropertyValue;
					if (PropertyHandle->GetValue(PropertyValue) == FPropertyAccess::Success)
					{
						return PropertyValue;
					}

					TArray<FString> PerObjectValues;
					if (WeakPropertyHandle.Pin()->GetPerObjectValues(PerObjectValues) == FPropertyAccess::Success &&
						!PerObjectValues.IsEmpty())
					{
						TTypeFromString<NumericType>::FromString(PropertyValue, *PerObjectValues[0]);
						return PropertyValue;
					}

					return TOptional<NumericType>();
				}();

			if (OldPropertyValue.IsSet() && WeakProxyManager.IsValid())
			{
				const NumericType Delta = Value - OldPropertyValue.GetValue();
				UE::ControlRigEditor::FAnimDetailsMultiEditUtil::Get().MultiEditChange<NumericType>(*WeakProxyManager.Get(), Delta, PropertyHandle, bInteractive);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
