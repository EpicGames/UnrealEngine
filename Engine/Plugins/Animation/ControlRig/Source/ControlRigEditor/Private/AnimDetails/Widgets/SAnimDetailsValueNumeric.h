// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimDetails/AnimDetailsNavigableWidgetRegistrar.h"
#include "Engine/TimerHandle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyHandle;
class UAnimDetailsProxyManager;
namespace UE::ControlRigEditor 
{ 
	template <typename NumericType> class TAnimDetailsNumericTypeInterface;
}

namespace UE::ControlRigEditor
{	
	/** 
	 * A widget to edit the numeric value of a control proxy, with features specific to the anim details view's needs.
	 * The numeric entry box always only uses the first object value and propagates changes to other objects and other selected properties.
	 */
	template <typename NumericType>
	class SAnimDetailsValueNumeric 
		: public SCompoundWidget
	{
	public:
		/** Defines if where the label of this widget is painted */
		enum class ELabelLocation : uint8
		{
			// Outside the bounds of the editable area of this box. Usually preferred for text based labels
			Outside,
			// Inside the bounds of the editable area of this box. Usually preferred for non-text based labels
			// when a spin box is used the label will appear on top of the spin box in this case
			Inside
		};

		SLATE_BEGIN_ARGS(SAnimDetailsValueNumeric<NumericType>)
			: _Label()
			, _LabelVAlign(VAlign_Fill)
			, _LabelLocation(ELabelLocation::Outside)
			, _LabelPadding(FMargin(3.f, 0.f))
			{}

			/** Slot for this button's content (optional) */
			SLATE_NAMED_SLOT(FArguments, Label)

			/** Vertical alignment of the label content */
			SLATE_ARGUMENT(EVerticalAlignment, LabelVAlign)

			/** If the label should be painted inside or outside of the spinbox */
			SLATE_ARGUMENT(ELabelLocation, LabelLocation)

			/** Padding around the label content */
			SLATE_ARGUMENT(FMargin, LabelPadding)

		SLATE_END_ARGS()

		virtual ~SAnimDetailsValueNumeric();

		/** Constructs this widget. Edits the provided property handle */
		void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);

		/** Sets the currently displayed text */
		void SetDisplayText(const FText& Text);

		/** Builds a text label */
		static TSharedRef<SWidget> BuildLabel(TAttribute<FText> LabelText, const FSlateColor& ForegroundColor, const FSlateColor& BackgroundColor)
		{
			return
				SNew(SBorder)
				.Visibility(EVisibility::HitTestInvisible)
				.BorderImage(FCoreStyle::Get().GetBrush("NumericEntrySpinBox.Decorator"))
				.BorderBackgroundColor(BackgroundColor)
				.ForegroundColor(ForegroundColor)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(FMargin(1.f, 0.f, 6.f, 0.f))
				[
					SNew(STextBlock)
						.Text(LabelText)
				];
		}

		/** Builds a narrow color label */
		static TSharedRef<SWidget> BuildNarrowColorLabel(FLinearColor LabelColor)
		{
			return
				SNew(SBorder)
				.Visibility(EVisibility::HitTestInvisible)
				.BorderImage(FAppStyle::Get().GetBrush("NumericEntrySpinBox.NarrowDecorator"))
				.BorderBackgroundColor(LabelColor)
				.HAlign(HAlign_Left)
				.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f));
		}

	private:
		//~ Begin SWidget Interface
		virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;
		//~ End SWidget Interface

		/** Refreshes the widget on the next tick */
		void RequestRefresh();

		/** Refreshes the widget */
		void ForceRefresh();

		/** Returns the current widget value */
		NumericType GetValue() const;

		/** Returns the display value */
		TOptional<FText> OnGetDisplayValue(NumericType SpinBoxValue);

		/** Called when the value changed */
		void OnValueChanged(NumericType Value);

		/** Called when a value was committed */
		void OnValueCommitted(NumericType Value, ETextCommit::Type CommitType);

		/** Called when slider movement begins */
		void OnBeginSliderMovement();

		/** Called when slider movement ends */
		void OnEndSliderMovement(NumericType Value);

		/** Sets the property value */
		void MultiEditChangePropertyValue(const NumericType Value, const bool bInteractive);

		/** True while the the value is edited using the slider */
		bool bIsUsingSlider = false;

		/** Weak object ptr to the proxy manager that holds displayed values */
		TWeakObjectPtr<UAnimDetailsProxyManager> WeakProxyManager;

		/** The displayed property */
		TWeakPtr<IPropertyHandle> WeakPropertyHandle;

		/** The current display text */
		FText DisplayText;

		/** Timer handle for the refresh methods */
		FTimerHandle RefreshTimerHandle;

		/** The label widget. SNullWidget if there is no label.*/
		TAlwaysValidWidget Label;

		/** Vertical alignment of the label content */
		EVerticalAlignment LabelVAlign;

		/** If the label should be painted inside or outside of the spinbox */
		ELabelLocation LabelLocation;

		/** Padding around the label content */
		FMargin LabelPadding;

		/** The spinbox that is used to display the numeric value*/
		TSharedPtr<SWidget> SpinBox;

		/** Handles registration of the inner widget that can be navigated to */
		FAnimDetailsNavigableWidgetRegistrar NavigableWidgetRegistrar;

		/** Type inteface for the spinbox */
		TSharedPtr<TAnimDetailsNumericTypeInterface<NumericType>> TypeInterface;
	};
}
