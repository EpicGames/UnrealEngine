// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/AnimSequenceFactory.h"
#include "AnimSequenceConverterFactory.generated.h"

UCLASS(MinimalAPI)
class UAnimSequenceConverterFactory : public UAnimSequenceFactory
{
	GENERATED_BODY()

	// Act just like the regular UAnimSequenceFactory, but with a pre-configured skeleton
	virtual bool ConfigureProperties() override { return true; }
};

