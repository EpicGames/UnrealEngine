// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigRuntimeAsset.h"
#include "Misc/AssetFilterData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ControlRig)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UAssetDefinition_ControlRig::UAssetDefinition_ControlRig()
{
	// Set up the Animation category
	Categories.Add(EAssetCategoryPaths::Animation);
}

FText UAssetDefinition_ControlRig::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_ControlRig", "Control Rig");
}

FLinearColor UAssetDefinition_ControlRig::GetAssetColor() const
{
	// Match the color used by FControlRigBlueprintActions and FControlRigAssetActions
	return FLinearColor(FColor(140, 116, 0));
}

TSoftClassPtr<UObject> UAssetDefinition_ControlRig::GetAssetClass() const
{
	// Return UControlRigBlueprint as the "primary" class for this definition.
	// The actual filtering is handled by BuildFilters() which includes both types.
	return UControlRigBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ControlRig::GetAssetCategories() const
{
	return Categories;
}

void UAssetDefinition_ControlRig::BuildFilters(TArray<FAssetFilterData>& OutFilters) const
{
	// Create a single filter that matches BOTH Control Rig asset types
	// Use the primary class path as the filter name for uniqueness (matches default BuildFilters pattern)
	FAssetFilterData ControlRigFilter;
	ControlRigFilter.Name = UControlRigBlueprint::StaticClass()->GetClassPathName().ToString();
	ControlRigFilter.DisplayText = LOCTEXT("AssetTypeActions_ControlRigFilter", "Control Rig");
	ControlRigFilter.FilterCategories = Categories;

	// Add both Control Rig class paths to the filter - this makes the filter match either type
	ControlRigFilter.Filter.ClassPaths.Add(UControlRigBlueprint::StaticClass()->GetClassPathName());
	ControlRigFilter.Filter.ClassPaths.Add(UControlRigRuntimeAsset::StaticClass()->GetClassPathName());
	ControlRigFilter.Filter.bRecursiveClasses = true;

	OutFilters.Add(MoveTemp(ControlRigFilter));
}

#undef LOCTEXT_NAMESPACE
