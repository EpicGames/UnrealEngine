// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ControlRig.generated.h"

/**
 * Asset Definition that provides a unified Content Browser filter for all Control Rig asset types.
 * This creates a single "Control Rig" filter in the Animation category that matches both
 * UControlRigBlueprint and UControlRigRuntimeAsset.
 */
UCLASS()
class UAssetDefinition_ControlRig : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	UAssetDefinition_ControlRig();

	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual bool CanRegisterStatically() const override { return true; }

protected:
	// Build a single filter that matches both Control Rig Blueprint and Control Rig Runtime Asset
	virtual void BuildFilters(TArray<FAssetFilterData>& OutFilters) const override;

private:
	TArray<FAssetCategoryPath> Categories;
};
