// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeToControlRigHelper.h"

#include "ControlRig.h"
#include "Rigs/FKControlRig.h"
#include "ControlRigObjectBinding.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "IControlRigEditorModule.h"
#include "SControlRigAssetReferencePicker.h"
#include "BakeToControlRigSettings.h"
#include "Exporters/AnimSeqExportOption.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Features/IModularFeatures.h"
#include "IMovieSceneAnimSequenceBakeScope.h"
#include "ISequencer.h"
#include "Misc/ScopeExit.h"
#include "MovieScene.h"
#include "ScopedTransaction.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "BakeToControlRigHelper"

// ---------------------------------------------------------------------------
// Options dialog (shared between all callers)
// ---------------------------------------------------------------------------

class SBakeToControlRigOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBakeToControlRigOptionsWindow)
		: _ExportOptions(nullptr), _BakeSettings(nullptr), _WidgetWindow()
	{}
	SLATE_ARGUMENT(UAnimSeqExportOption*, ExportOptions)
	SLATE_ARGUMENT(UBakeToControlRigSettings*, BakeSettings)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ExportOptions = InArgs._ExportOptions;
		BakeSettings = InArgs._BakeSettings;
		WidgetWindow = InArgs._WidgetWindow;
		check(ExportOptions);

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

		TSharedPtr<IDetailsView> ExportDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		ExportDetailsView->SetObject(ExportOptions);
		TSharedPtr<IDetailsView> BakeDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		BakeDetailsView->SetObject(BakeSettings);

		ChildSlot
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot().FillHeight(1.0f).Padding(2) [ ExportDetailsView.ToSharedRef() ]
				+ SVerticalBox::Slot().AutoHeight().Padding(2) [ BakeDetailsView.ToSharedRef() ]
				+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).Padding(2)
				[
					SNew(SUniformGridPanel).SlotPadding(2)
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton).HAlign(HAlign_Center)
						.Text(LOCTEXT("Create", "Create"))
						.OnClicked(this, &SBakeToControlRigOptionsWindow::OnExport)
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton).HAlign(HAlign_Center)
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked(this, &SBakeToControlRigOptionsWindow::OnCancel)
					]
				]
			]
		];
	}

	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape) { return OnCancel(); }
		return FReply::Unhandled();
	}
	bool ShouldExport() const { return bShouldExport; }

private:
	FReply OnExport() { bShouldExport = true; if (WidgetWindow.IsValid()) { WidgetWindow.Pin()->RequestDestroyWindow(); } return FReply::Handled(); }
	FReply OnCancel() { bShouldExport = false; if (WidgetWindow.IsValid()) { WidgetWindow.Pin()->RequestDestroyWindow(); } return FReply::Handled(); }

	UAnimSeqExportOption* ExportOptions = nullptr;
	UBakeToControlRigSettings* BakeSettings = nullptr;
	TWeakPtr<SWindow> WidgetWindow;
	bool bShouldExport = false;
};

// ---------------------------------------------------------------------------
// Sort predicate for control rig asset picker (FK rig first)
// ---------------------------------------------------------------------------

static bool BakeHelperAssetSortPredicate(const TSharedRef<FControlRigAssetSoftReference> A, const TSharedRef<FControlRigAssetSoftReference> B)
{
	const bool bAIsFK = A->Get() == UFKControlRig::StaticClass();
	const bool bBIsFK = B->Get() == UFKControlRig::StaticClass();
	if (!bAIsFK && !bBIsFK)
	{
		return A->GetName().Compare(B->GetName(), ESearchCase::IgnoreCase) < 0;
	}
	return bAIsFK;
}

// ---------------------------------------------------------------------------
// Implementation
// ---------------------------------------------------------------------------

namespace UE::ControlRig
{

FBakeToControlRigResult InitControlRigFromAnimSequence(
	const TSharedPtr<ISequencer>& Sequencer,
	UMovieScene* MovieScene,
	UMovieSceneControlRigParameterTrack* Track,
	const FControlRigAssetStrongReference& AssetReference,
	UObject* BoundActor,
	USkeletalMeshComponent* SkelMeshComp,
	UAnimSequence* AnimSequence,
	bool bReduceKeys,
	const FSmartReduceParams& SmartReduceParams)
{
	FBakeToControlRigResult Result;

	if (!Track || !AssetReference.IsValid() || !MovieScene || !AnimSequence || !SkelMeshComp)
	{
		return Result;
	}

	Track->Modify();

	FString ObjectName = AssetReference.GetName();
	ObjectName.RemoveFromEnd(TEXT("_C"));

	UControlRig* ControlRig = AssetReference.CreateInstance(Track, FName(*ObjectName), RF_Transactional);
	if (!ControlRig)
	{
		return Result;
	}

	if (AssetReference.GetRigClass() != UFKControlRig::StaticClass() && !ControlRig->SupportsEvent(FRigUnit_InverseExecution::EventName))
	{
		return Result;
	}

	ControlRig->Modify();
	ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
	ControlRig->GetObjectBinding()->BindToObject(BoundActor);
	ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, ControlRig->GetObjectBinding()->GetBoundObject());
	ControlRig->Initialize();
	ControlRig->RequestInit();
	ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkelMeshComp, true);
	ControlRig->Evaluate_AnyThread();

	constexpr bool bSequencerOwnsControlRig = true;
	UMovieSceneSection* NewSection = Track->CreateControlRigSection(0, ControlRig, bSequencerOwnsControlRig);
	UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(NewSection);
	if (!ParamSection)
	{
		return Result;
	}

	Track->SetTrackName(FName(*ObjectName));
	Track->SetDisplayName(FText::FromString(ObjectName));

	if (Sequencer)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	}

	TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer ? Sequencer->GetSubSequenceRange() : TOptional<TRange<FFrameNumber>>();
	FFrameNumber StartFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetLowerBoundValue() : MovieScene->GetPlaybackRange().GetLowerBoundValue();

	EMovieSceneKeyInterpolation DefaultInterpolation = Sequencer ? Sequencer->GetKeyInterpolation() : EMovieSceneKeyInterpolation::SmartAuto;

	UMovieSceneControlRigParameterSection::FLoadAnimSequenceData LoadData;
	LoadData.bKeyReduce = false;
	LoadData.Tolerance = 0.0;
	LoadData.bResetControls = true;
	LoadData.StartFrame = StartFrame;

	ParamSection->LoadAnimSequenceIntoThisSection(AnimSequence, FFrameNumber(0), MovieScene, SkelMeshComp, LoadData, DefaultInterpolation);

	if (bReduceKeys)
	{
		FSmartReduceParams Params = SmartReduceParams;
		UControlRigSequencerEditorLibrary::SmartReduce(Params, ParamSection);
	}

	Result.Track = Track;
	Result.Section = ParamSection;
	Result.ControlRig = ControlRig;
	Result.bSuccess = true;
	return Result;
}

FBakeToControlRigResult HandleBakeToControlRig(
	const FControlRigAssetStrongReference& AssetReference,
	const TSharedPtr<ISequencer>& Sequencer,
	FGuid ObjectBinding,
	UObject* BoundActor,
	USkeletalMeshComponent* SkelMeshComp,
	USkeleton* Skeleton,
	const FPopulateAnimSequenceDelegate& PopulateDelegate,
	TFunction<UMovieSceneControlRigParameterTrack*(UMovieScene*)> TrackFactory)
{
	FBakeToControlRigResult Result;

	FSlateApplication::Get().DismissAllMenus();

	if (!AssetReference.IsValid() || !Sequencer || !SkelMeshComp || !Skeleton)
	{
		return Result;
	}

	UMovieSceneSequence* OwnerSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
	if (!ensure(OwnerMovieScene))
	{
		return Result;
	}

	// Create temp anim sequence and export options
	UAnimSequence* TempAnimSequence = NewObject<UAnimSequence>(GetTransientPackage(), NAME_None);
	TempAnimSequence->SetSkeleton(Skeleton);

	UAnimSeqExportOption* AnimSeqExportOption = NewObject<UAnimSeqExportOption>(GetTransientPackage(), NAME_None);
	UBakeToControlRigSettings* BakeSettings = GetMutableDefault<UBakeToControlRigSettings>();
	AnimSeqExportOption->bTransactRecording = false;
	AnimSeqExportOption->CustomDisplayRate = Sequencer->GetFocusedDisplayRate();

	// Show options dialog
	TSharedPtr<SWindow> ParentWindow;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("BakeOptionsTitle", "Options For Baking"))
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.ClientSize(FVector2D(500, 445));

	TSharedPtr<SBakeToControlRigOptionsWindow> OptionWindow;
	Window->SetContent(
		SAssignNew(OptionWindow, SBakeToControlRigOptionsWindow)
		.ExportOptions(AnimSeqExportOption)
		.BakeSettings(BakeSettings)
		.WidgetWindow(Window));

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

	if (!OptionWindow->ShouldExport())
	{
		TempAnimSequence->MarkAsGarbage();
		AnimSeqExportOption->MarkAsGarbage();
		return Result;
	}

	// Let registered plugins bracket the recorder playback (e.g. force mixer root motion onto the root bone).
	TArray<IMovieSceneAnimSequenceBakeScope*> BakeScopes =
		IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneAnimSequenceBakeScope>(
			IMovieSceneAnimSequenceBakeScope::GetModularFeatureName());
	for (IMovieSceneAnimSequenceBakeScope* Scope : BakeScopes) { if (Scope) { Scope->BeginBakeScope(); } }
	ON_SCOPE_EXIT { for (IMovieSceneAnimSequenceBakeScope* Scope : BakeScopes) { if (Scope) { Scope->EndBakeScope(); } } };

	if (!PopulateDelegate.IsBound() || !PopulateDelegate.Execute(TempAnimSequence, AnimSeqExportOption, SkelMeshComp))
	{
		TempAnimSequence->MarkAsGarbage();
		AnimSeqExportOption->MarkAsGarbage();
		return Result;
	}

	// Create the track via caller-provided factory
	OwnerMovieScene->Modify();

	UMovieSceneControlRigParameterTrack* Track = TrackFactory ? TrackFactory(OwnerMovieScene) : nullptr;
	if (!Track)
	{
		TempAnimSequence->MarkAsGarbage();
		AnimSeqExportOption->MarkAsGarbage();
		return Result;
	}

	// Initialize the rig and load the animation
	Result = InitControlRigFromAnimSequence(
		Sequencer, OwnerMovieScene, Track, AssetReference,
		BoundActor, SkelMeshComp, TempAnimSequence,
		BakeSettings->bReduceKeys, BakeSettings->SmartReduce);

	if (!Result.bSuccess)
	{
		OwnerMovieScene->RemoveTrack(*Track);
	}

	TempAnimSequence->MarkAsGarbage();
	AnimSeqExportOption->MarkAsGarbage();
	return Result;
}

void BuildBakeToControlRigMenu(
	FMenuBuilder& MenuBuilder,
	USkeleton* Skeleton,
	bool& bFilterAssetBySkeleton,
	const FPopulateAnimSequenceDelegate& PopulateDelegate,
	const TSharedPtr<ISequencer>& Sequencer,
	FGuid ObjectBinding,
	UObject* BoundActor,
	USkeletalMeshComponent* SkelMeshComp,
	TFunction<UMovieSceneControlRigParameterTrack*(UMovieScene*)> TrackFactory,
	TFunction<void(const FBakeToControlRigResult&)> OnComplete,
	const FText& SubmenuLabelOverride)
{
	if (!Skeleton)
	{
		return;
	}

	// Bake To Control Rig submenu with asset picker.
	// The "Filter Asset By Skeleton" toggle lives inside the submenu alongside
	// the asset picker it controls, not on the parent menu.
	// Submenu label uses the caller's override (e.g. "Bake Layer to Control Rig") if provided.
	const FText SubmenuLabel = SubmenuLabelOverride.IsEmpty()
		? LOCTEXT("BakeToControlRig", "Bake To Control Rig")
		: SubmenuLabelOverride;

	MenuBuilder.AddSubMenu(
		SubmenuLabel,
		LOCTEXT("BakeToControlRigTooltip", "Bake to an invertible Control Rig that matches this skeleton"),
		FNewMenuDelegate::CreateLambda(
			[&bFilterAssetBySkeleton, Skeleton, PopulateDelegate, Sequencer, ObjectBinding, BoundActor, SkelMeshComp, TrackFactory, OnComplete]
			(FMenuBuilder& SubMenuBuilder)
			{
				// Filter Asset By Skeleton toggle (scoped to this submenu so it sits next to the picker it controls)
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("FilterAssetBySkeleton", "Filter Asset By Skeleton"),
					LOCTEXT("FilterAssetBySkeletonTooltip", "Filters Control Rig assets to match current skeleton"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([&bFilterAssetBySkeleton]() { bFilterAssetBySkeleton = !bFilterAssetBySkeleton; }),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda([&bFilterAssetBySkeleton]() { return bFilterAssetBySkeleton; })),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);

				const TSharedPtr<FControlRigClassFilter> ClassFilter = MakeShareable(
					new FControlRigClassFilter(bFilterAssetBySkeleton, false, true, Skeleton));
				TArray<FControlRigAssetSoftReference> ExtraAssets = { UFKControlRig::StaticClass() };

				SubMenuBuilder.AddWidget(
					SNew(SControlRigAssetReferencePicker)
					.OnSelectionChanged_Lambda(
						[PopulateDelegate, Sequencer, ObjectBinding, BoundActor, SkelMeshComp, Skeleton, TrackFactory, OnComplete]
						(FControlRigAssetSoftReference InReference)
						{
							FScopedTransaction Transaction(LOCTEXT("BakeToControlRig_Transaction", "Bake To Control Rig"));
							FBakeToControlRigResult Result = HandleBakeToControlRig(
								InReference.LoadStrongReference(), Sequencer, ObjectBinding,
								BoundActor, SkelMeshComp, Skeleton,
								PopulateDelegate, TrackFactory);

							if (Result.bSuccess && OnComplete)
							{
								OnComplete(Result);
							}
							if (Result.bSuccess == false)
							{
								Transaction.Cancel();
							}
						})
					.Filter(ClassFilter)
					.ExtraAssets(ExtraAssets)
					.SortPredicate(BakeHelperAssetSortPredicate),
					FText::GetEmpty());
			}));
}

} // namespace UE::ControlRig

#undef LOCTEXT_NAMESPACE
