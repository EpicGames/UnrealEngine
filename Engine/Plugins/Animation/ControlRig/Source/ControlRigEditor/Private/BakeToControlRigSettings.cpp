// Copyright Epic Games, Inc. All Rights Reserved.

#include "BakeToControlRigSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BakeToControlRigSettings)

UBakeToControlRigSettings::UBakeToControlRigSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{}

void UBakeToControlRigSettings::Reset()
{
	bReduceKeys = false;
	SmartReduce.Reset();
}

