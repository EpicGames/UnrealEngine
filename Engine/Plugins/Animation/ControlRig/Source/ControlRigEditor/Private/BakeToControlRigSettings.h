// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Filters/CurveEditorSmartReduceFilter.h"
#include "BakeToControlRigSettings.generated.h"

UCLASS(BlueprintType, config = EditorSettings)
class  UBakeToControlRigSettings : public UObject
{
public:
	UBakeToControlRigSettings(const FObjectInitializer& Initializer);

	GENERATED_BODY()
	/** Reduce Keys */
	UPROPERTY(EditAnywhere, Category = "Reduce Keys")
	bool bReduceKeys = false;

	/** Reduce Keys Tolerance*/
	UE_DEPRECATED(5.5, "Use the SmartReduce parameter instead.")
	UPROPERTY()
	float Tolerance = 0.001f;

	UPROPERTY(EditAnywhere, Category = "Reduce Keys", meta = (EditCondition = "bReduceKeys"))
	FSmartReduceParams SmartReduce;

	/** Reset controls to initial value on every frame */
	UE_DEPRECATED(5.8, "When baking onto a new control rig always reset controls")
	UPROPERTY()
	bool bResetControls = true;

	/** Resets the default properties. */
	void Reset();
};


