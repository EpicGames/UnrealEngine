// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigAnimGraphDetails.h"

#include "ControlRig.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "AnimGraphNode_ControlRig.h"
#include "Algo/Transform.h"
#include "Units/Execution/RigUnit_BeginExecution.h"

#define LOCTEXT_NAMESPACE "ControlRigAnimGraphDetails"
static const FText ControlRigAnimDetailsMultipleValues = LOCTEXT("MultipleValues", "Multiple Values");

void FControlRigAnimNodeEventNameDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	AnimNodeBeingCustomized = nullptr;

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		if(UAnimGraphNode_ControlRig* GraphNode = Cast<UAnimGraphNode_ControlRig>(Object))
		{
			AnimNodeBeingCustomized = &GraphNode->Node;
			break;
		}
	}

	if (AnimNodeBeingCustomized == nullptr)
	{
		HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InStructPropertyHandle->CreatePropertyValueWidget()
		];
	}
	else
	{
		NameHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FControlRigAnimNodeEventName, EventName));
		UpdateEntryNameList();

		HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 0.f, 0.f)
			[
				SAssignNew(SearchableComboBox, SSearchableComboBox)
				.OptionsSource(&EntryNameList)
				.OnSelectionChanged(this, &FControlRigAnimNodeEventNameDetails::OnEntryNameChanged)
				.OnGenerateWidget(this, &FControlRigAnimNodeEventNameDetails::OnGetEntryNameWidget)
				.IsEnabled(!NameHandle->IsEditConst())
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &FControlRigAnimNodeEventNameDetails::GetEntryNameAsText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		];
	}
}

void FControlRigAnimNodeEventNameDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		// only fill the children if the blueprint cannot be found
		if (AnimNodeBeingCustomized == nullptr)
		{
			uint32 NumChildren = 0;
			InStructPropertyHandle->GetNumChildren(NumChildren);

			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
			}
		}
	}
}

FString FControlRigAnimNodeEventNameDetails::GetEntryName() const
{
	FString EntryNameStr;
	if (NameHandle.IsValid())
	{
		for(int32 ObjectIndex = 0; ObjectIndex < NameHandle->GetNumPerObjectValues(); ObjectIndex++)
		{
			FString PerObjectValue;
			NameHandle->GetPerObjectValue(ObjectIndex, PerObjectValue);

			if(ObjectIndex == 0)
			{
				EntryNameStr = PerObjectValue;
			}
			else if(EntryNameStr != PerObjectValue)
			{
				return ControlRigAnimDetailsMultipleValues.ToString();
			}
		}
	}
	return EntryNameStr;
}

void FControlRigAnimNodeEventNameDetails::SetEntryName(FString InName)
{
	if (NameHandle.IsValid())
	{
		NameHandle->SetValue(InName);
	}
}

void FControlRigAnimNodeEventNameDetails::UpdateEntryNameList()
{
	EntryNameList.Reset();

	if (AnimNodeBeingCustomized)
	{
		FControlRigAssetStrongReference RigSource = AnimNodeBeingCustomized->GetControlRigAssetReference();
		if(RigSource.IsValid())
		{
			TArray<FName> SupportedEvents = RigSource.GetSupportedEvents();

			// Remove Pre/Post forward solve
			SupportedEvents.Remove(FRigUnit_PreBeginExecution::EventName);
			SupportedEvents.Remove(FRigUnit_PostBeginExecution::EventName);
			
			Algo::Transform(SupportedEvents, EntryNameList,[](const FName& InEntryName)
			{
				return MakeShareable(new FString(InEntryName.ToString()));
			});
			if(SearchableComboBox.IsValid())
			{
				SearchableComboBox->RefreshOptions();
			}
		}
	}
}

void FControlRigAnimNodeEventNameDetails::OnEntryNameChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo)
{
	if (InItem.IsValid())
	{
		SetEntryName(*InItem);
	}
	else
	{
		SetEntryName(FString());
	}
}

TSharedRef<SWidget> FControlRigAnimNodeEventNameDetails::OnGetEntryNameWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(InItem.IsValid() ? *InItem : FString()))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

FText FControlRigAnimNodeEventNameDetails::GetEntryNameAsText() const
{
	return FText::FromString(GetEntryName());
}

#undef LOCTEXT_NAMESPACE
