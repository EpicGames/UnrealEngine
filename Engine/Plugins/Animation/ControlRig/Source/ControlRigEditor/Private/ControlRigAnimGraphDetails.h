// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "AnimNode_ControlRigBase.h"

class IDetailLayoutBuilder;

class IPropertyHandle;

class FControlRigAnimNodeEventNameDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FControlRigAnimNodeEventNameDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	FString GetEntryName() const;
	void SetEntryName(FString InName);
	void UpdateEntryNameList();
	void OnEntryNameChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo);
	TSharedRef<SWidget> OnGetEntryNameWidget(TSharedPtr<FString> InItem);
	FText GetEntryNameAsText() const;

	TSharedPtr<IPropertyHandle> NameHandle;
	TArray<TSharedPtr<FString>> EntryNameList;
	TSharedPtr<SSearchableComboBox> SearchableComboBox;
	FAnimNode_ControlRigBase* AnimNodeBeingCustomized;
};