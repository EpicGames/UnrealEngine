// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions/AssetTypeActions_ClassTypeBase.h"
#include "ControlRigRuntimeAsset.h"

class FMenuBuilder;
class UFactory;
class USkeletalMesh;
class AActor;
class UControlRigBlueprint;

class FControlRigAssetActions : public FAssetTypeActions_ClassTypeBase
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "ControlRigAssetActions", "Control Rig"); }
	virtual FColor GetTypeColor() const override { return FColor(140, 116, 0); }
	virtual UClass* GetSupportedClass() const override { return UControlRigRuntimeAsset::StaticClass(); }
	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual bool CanFilter() override { return false; }
	virtual const TArray<FText>& GetSubMenus() const override;
	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;
	virtual void PerformAssetDiff(UObject* Asset1, UObject* Asset2, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override;
	virtual TWeakPtr<IClassTypeActions> GetClassTypeActions(const FAssetData& AssetData) const override;

	static void ExtendSketalMeshToolMenu();

	static UControlRigRuntimeAsset* CreateNewControlRigAsset(const FString& InDesiredPackagePath, const bool bModularRig);
	static UControlRigRuntimeAsset* CreateNewControlRigAsset(UControlRigBlueprint* InBlueprint);
	static UControlRigRuntimeAsset* CreateControlRigFromSkeletalMeshOrSkeleton(UObject* InSelectedObject, const bool bModularRig);
	
	

	static USkeletalMesh* GetSkeletalMeshFromControlRigAsset(const FAssetData& InAsset);
	static void PostSpawningSkeletalMeshActor(AActor* InSpawnedActor, UObject* InAsset);
	static void OnSpawnedSkeletalMeshActorChanged(UObject* InObject, FPropertyChangedEvent& InEvent, UObject* InAsset);

	static FDelegateHandle OnSpawnedSkeletalMeshActorChangedHandle;
};
