// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigAssetFactory.h"
#include "UObject/Interface.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "ControlRigAssetActions.h"
#include "ControlRigEditorModule.h"
#include "ModularRig.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Graph/ControlRigGraphSchema.h"
#include "Graph/ControlRigGraph.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ModularRigController.h"
#include "Settings/ControlRigSettings.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Layout/SSeparator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigAssetFactory)

#define LOCTEXT_NAMESPACE "ControlRigAssetFactory"

namespace UE::ControlRigEditor::Private
{
	/** 
	 * Dialog to pick the base class from which the asset should be created.
	 * 
	 * As a Rig Module is not a separate class, the implementation works around it 
	 * by showing  core types (Control Rig, Control Rig Module and Modular Rig) in 
	 * a list and user defined types in a class viewer.
	 */
	class SControlRigAssetCreateDialog final :
		public SCompoundWidget 
	{
		/** Holds the result of this dialog */
		struct FControlRigAssetCreateDialogResult
		{
			FControlRigAssetCreateDialogResult(TSubclassOf<UControlRig> InRigClass, const bool bInCreateAsRigModule)
			{
				const bool bValid = !bInCreateAsRigModule || InRigClass != UModularRig::StaticClass();
				if (ensureMsgf(bValid, TEXT("Cannot create a Modular Rig as Rig Module, creating a Modular Rig instead")))
				{
					RigClass = InRigClass;
					bCreateAsRigModule = bInCreateAsRigModule;
				}
				else
				{
					RigClass = InRigClass;
				}
			}

			const TSubclassOf<UControlRig>& GetRigClass() const { return RigClass; }

			bool ShouldCreateAsRigModule() const { return bCreateAsRigModule; }

		private:
			/** The class to create */
			TSubclassOf<UControlRig> RigClass;

			/** True if the Control Rig should be created as a rig module */
			bool bCreateAsRigModule = false;
		};

	public:
		SLATE_BEGIN_ARGS( SControlRigAssetCreateDialog ){}

		SLATE_END_ARGS()

		/** Shows the dialog, returns resulting params or an unset optional if the user canceled in some way */
		[[nodiscard]] bool ConfigureProperties(TWeakObjectPtr<UControlRigAssetFactory> InControlRigAssetFactory)
		{
			ControlRigAssetFactory = InControlRigAssetFactory;

			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(LOCTEXT("CreateControlRigAssetOptions", "Create Control Rig Asset"))
				.ClientSize(FVector2D(400, 400))
				.SupportsMinimize(false)
				.SupportsMaximize(false)
				[
					AsShared()
				];

			PickerWindow = Window;

			GEditor->EditorAddModalWindow(Window);

			if (Result.IsSet())
			{
				ControlRigAssetFactory->ParentClass = Result.GetValue().GetRigClass();
				ControlRigAssetFactory->bCreateAsControlRigModule = Result.GetValue().ShouldCreateAsRigModule();

				return true;
			}

			return false;
		}

		/** Constructs this widget with InArgs */
		void Construct( const FArguments& InArgs )
		{
			ChildSlot
			[
				SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SBox)
					.Visibility(EVisibility::Visible)
					.WidthOverride(500.0f)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.FillHeight(1)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									GenerateCoreTypeList()
								]

								+ SVerticalBox::Slot()
								.FillHeight(1.f)
								[
									SAssignNew(UserClassContent, SVerticalBox)
								]	
							]
						]

						// Ok/Cancel buttons
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						.Padding(8.f)
						[
							SNew(SUniformGridPanel)
							.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
							.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
							.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))

							+ SUniformGridPanel::Slot(0,0)
							[
								SNew(SButton)
								.HAlign(HAlign_Center)
								.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
								.OnClicked(this, &SControlRigAssetCreateDialog::OkClicked)	
								.Text(LOCTEXT("CreateControlRigBlueprintCreate", "Create"))
							]

							+ SUniformGridPanel::Slot(1,0)
							[
								SNew(SButton)
								.HAlign(HAlign_Center)
								.ContentPadding( FAppStyle::GetMargin("StandardDialog.ContentPadding") )
								.OnClicked(this, &SControlRigAssetCreateDialog::CancelClicked)
								.Text(LOCTEXT("CreateControlRigBlueprintCancel", "Cancel"))
							]
						]
					]
				]
			];

			// The engine types are displayed in the core type list,
			// hence only show the user class picker if there are user defined control rig classes avialable
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				const UClass* Class = *ClassIt;
				if (IsSupportedParentClass(Class) &&
					Class != UControlRig::StaticClass() &&
					Class != UModularRig::StaticClass())
				{
					bShouldShowUserClassPicker = true;
					break;
				}
			}

			RefreshUserClassPicker();
		}

	private:
		//~ Begin SWidget interface
		FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
		{
			if (InKeyEvent.GetKey() == EKeys::Escape)
			{
				CloseDialog();
				return FReply::Handled();
			}
			return SWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}
		//~ End SWidget interface

		/** Handler for when ok is clicked */
		FReply OkClicked()
		{
			constexpr bool bCommit = true;
			CloseDialog(bCommit);

			return FReply::Handled();
		}

		/** Handler for when cancel is clicked */
		FReply CancelClicked()
		{
			Result.Reset();

			CloseDialog();
			return FReply::Handled();
		}

		/** Closes the dialog */
		void CloseDialog(const bool bCommit = false)
		{
			if (!bCommit)
			{
				Result.Reset();
			}

			if (PickerWindow.IsValid())
			{
				PickerWindow.Pin()->RequestDestroyWindow();
			}
		}

		/** Generates the core types list that shows Control Rig, Modular Rig and Rig Module */
		TSharedRef<SWidget> GenerateCoreTypeList()
		{
			return
				SNew(SVerticalBox)
				
				+ SVerticalBox::Slot()
				.Padding(2.f, 8.f, 2.f, 2.f)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DefaultTypes", "Default Types"))
						.ShadowOffset(FVector2D(1.0f, 1.0f))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(CoreTypeList, SListView<TSharedPtr<FString>>)
					.ListItemsSource(&CoreTypes)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SControlRigAssetCreateDialog::GenerateCoreTypeRow)
					.OnSelectionChanged(this, &SControlRigAssetCreateDialog::OnCoreTypeRowSelected)
				];
		}

		/** Generates a row in the core type list */
		TSharedRef<ITableRow> GenerateCoreTypeRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
		{
			const FSlateBrush* Icon = [&InItem]() -> const FSlateBrush*
				{
					if (*InItem == "Control Rig" ||
						*InItem == "Control Rig Module")
					{
						return FSlateIconFinder::FindIconBrushForClass(UControlRig::StaticClass());
					}
					else if (*InItem == "Modular Rig")
					{
						return FSlateIconFinder::FindIconBrushForClass(UModularRig::StaticClass());
					}
					else
					{
						ensureMsgf(0, TEXT("Unhandled rig type"));
						return nullptr;
					}
				}();

			return
				SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
				.Padding(2.f)
				.Content()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8.f, 0.f, 4.f, 0.f)
					[
						SNew(SImage)
						.Image(Icon)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::FromString(*InItem))
					]
				];
		}

		/** Called when a core type row was selected */
		void OnCoreTypeRowSelected(TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo)
		{
			if (SelectedItem.IsValid())
			{
				if (*SelectedItem == TEXT("Control Rig"))
				{
					constexpr bool bCreateAsRigModule = false;
					Result = FControlRigAssetCreateDialogResult(UControlRig::StaticClass(), bCreateAsRigModule);
				}
				else if (*SelectedItem == TEXT("Control Rig Module"))
				{
					constexpr bool bCreateAsRigModule = true;
					Result = FControlRigAssetCreateDialogResult(UControlRig::StaticClass(), bCreateAsRigModule);
				}
				else if (*SelectedItem == TEXT("Modular Rig"))
				{
					constexpr bool bCreateAsRigModule = false;
					Result = FControlRigAssetCreateDialogResult(UModularRig::StaticClass(), bCreateAsRigModule);
				}

				// Refresh the user class picker to clear its selection
				// This is a pure workaround, there is no other way to clear its selection
				RefreshUserClassPicker();
			}
		}

		/** 
		 * Creates an additional class picker if there are user defined classes. 
		 * Supports refreshing, useful to clear the selection of the class picker.
		 */
		void RefreshUserClassPicker()
		{
			// The class picker filter implementation
			class FControlRigBlueprintParentFilterImpl
				: public IClassViewerFilter
			{
			public:
				/** All children of these classes will be included unless filtered out by another setting. */
				TSet< const UClass* > AllowedChildrenOfClasses;

				virtual bool IsClassAllowed(
					const FClassViewerInitializationOptions& InInitOptions,
					const UClass* InClass,
					TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
				{
					// If it appears on the allowed child-of classes list (or there is nothing on that list)
					if (InClass)
					{
						if (InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) == EFilterReturn::Failed)
						{
							return false;
						}

						// Don't allow for the base classes self, these are displayed in a separate list top-most to have three entries,
						// So we can display Rig Module there, as it is not a specific class.
						const bool bDisplayedInCoreTypeList = 
							InClass == UControlRig::StaticClass() || 
							InClass == UModularRig::StaticClass();
						if (bDisplayedInCoreTypeList)
						{
							return false;
						}

						// in the future we might allow it to parent to BP classes, but right now, it won't work well because of multi rig graph
						// for now we disable it and only allow native class. 
						if (IsSupportedParentClass(InClass))
						{
							return true;
						}
					}

					return false;
				}

				virtual bool IsUnloadedClassAllowed(
					const FClassViewerInitializationOptions& InInitOptions,
					const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData,
					TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
				{
					// If it appears on the allowed child-of classes list (or there is nothing on that list)
					// in the future we might allow it to parent to BP classes, but right now, it won't work well because of multi rig graph
					// for now we disable it and only allow native class. 
					return false; // InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData) != EFilterReturn::Failed;
				}
			};

			if (!bShouldShowUserClassPicker)
			{
				return;
			}

			if (UserClassViewer.IsValid())
			{
				UserClassContent->RemoveSlot(UserClassViewer.ToSharedRef());
			}

			// Load the classviewer module to display a class picker
			FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

			// Fill in options
			FClassViewerInitializationOptions Options;
			Options.Mode = EClassViewerMode::ClassPicker;
			
			const TSharedRef<FControlRigBlueprintParentFilterImpl> Filter = MakeShared<FControlRigBlueprintParentFilterImpl>();
			Options.ClassFilters.Add(Filter);

			// All child classes of UControlRig are valid.
			Filter->AllowedChildrenOfClasses.Add(UControlRig::StaticClass());
			
			UserClassViewer = 
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.Orientation(EOrientation::Orient_Horizontal)
				]
				
				+ SVerticalBox::Slot()
				.Padding(2.f, 8.f, 2.f, 2.f)
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("UserDefinedTypes", "User Defined Types"))
						.ShadowOffset(FVector2D(1.f, 1.f))
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateLambda(
						[this](UClass* ChosenClass)
						{
							constexpr bool bCreateAsRigModule = false;
							Result = FControlRigAssetCreateDialogResult(ChosenClass, bCreateAsRigModule);

							if (CoreTypeList.IsValid())
							{
								CoreTypeList->ClearSelection();
							}
						}))
				];

			UserClassContent->AddSlot()
				.Padding(2.f)
				.FillHeight(1.f)
				[
					UserClassViewer.ToSharedRef()
				];
		}

		/** Returns true if the factory supports creating assets from the specified parent class */
		static bool IsSupportedParentClass(const UClass* ParentClass)
		{
			// in the future we might allow it to parent to BP classes, but right now, it won't work well because of multi rig graph
			// for now we disable it and only allow native class. 
			return
				ParentClass &&
				!ParentClass->HasAnyClassFlags(CLASS_Deprecated) &&
				ParentClass->HasAnyClassFlags(CLASS_Native) &&
				ParentClass->GetOutermost() != GetTransientPackage() &&
				ParentClass->GetBoolMetaDataHierarchical(FBlueprintMetadata::MD_IsBlueprintBase) &&
				ParentClass->IsChildOf(UControlRig::StaticClass());
		}

		/** The factory for which we are setting up properties */
		TWeakObjectPtr<UControlRigAssetFactory> ControlRigAssetFactory;

		/** The list that displays the core types */
		TSharedPtr<SListView<TSharedPtr<FString>>> CoreTypeList;

		/** The result of this dialog */
		TOptional<FControlRigAssetCreateDialogResult> Result;

		/** The user class viewer widget */
		TSharedPtr<SWidget> UserClassViewer;

		/** The box that holds the user class content */
		TSharedPtr<SVerticalBox> UserClassContent;

		/** A pointer to the window that is asking the user to select a parent class */
		TWeakPtr<SWindow> PickerWindow;

		/** The core types that are always available to the user */
		const TArray<TSharedPtr<FString>> CoreTypes =
		{
			MakeShared<FString>(TEXT("Control Rig")),
			MakeShared<FString>(TEXT("Control Rig Module")),
			MakeShared<FString>(TEXT("Modular Rig"))
		};

		/** If true shows a user class picker along in addition to the default class list */
		bool bShouldShowUserClassPicker = false;
	};
}

UControlRigAssetFactory::UControlRigAssetFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UControlRigRuntimeAsset::StaticClass();
	ParentClass = UControlRig::StaticClass(); // default to control rig
	EditorOnlyClass = UControlRigEditorAsset::StaticClass();
}

bool UControlRigAssetFactory::ConfigureProperties()
{
	using namespace UE::ControlRigEditor::Private;
	if (CVarControlRigHierarchyEnableModules.GetValueOnAnyThread())
	{
		const TSharedRef<SControlRigAssetCreateDialog> Dialog = SNew(SControlRigAssetCreateDialog);
		return Dialog->ConfigureProperties(this);
	}

	return true;
};

UObject* UControlRigAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a Control Rig Blueprint, then create and init one
	check(Class->IsChildOf(UControlRigRuntimeAsset::StaticClass()));

	if ((ParentClass == nullptr) || !ParentClass->IsChildOf(UControlRig::StaticClass()))
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString( ParentClass->GetName() ) : LOCTEXT("Null", "(null)") );
		FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("CannotCreateControlRigAsset", "Cannot create an Control Rig Asset based on the class '{0}'."), Args ) );
		return nullptr;
	}
	
	UControlRigRuntimeAsset* ControlRigAsset = NewObject<UControlRigRuntimeAsset>(InParent, Name, RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted);
	ControlRigAsset->Initialize(EditorOnlyClass);
	FControlRigEditorModule::Get().CreateRootGraphIfRequired(FRigVMEditorAssetInterfacePtr(ControlRigAsset));

	// add the default module
	if(ParentClass->IsChildOf(UModularRig::StaticClass()))
	{
		ControlRigAsset->ControlRigType = EControlRigType::ModularRig;
		FSoftObjectPath DefaultRootModulePath = UControlRigSettings::Get()->DefaultRootModule;
		UObject* DefaultRootModuleObj = DefaultRootModulePath.TryLoad();
		if (!DefaultRootModuleObj)
		{
			DefaultRootModulePath = FString::Printf(TEXT("%s_C"), *UControlRigSettings::Get()->DefaultRootModule.GetAssetPathString());
			DefaultRootModuleObj = DefaultRootModulePath.TryLoad();
		}
		
		if(UControlRigRuntimeAsset* DefaultRootModule = Cast<UControlRigRuntimeAsset>(DefaultRootModuleObj))
		{
			static const FName RootName = TEXT("Root");
			ControlRigAsset->ModularRigModel.GetController()->AddModuleFromAssetReference(RootName, DefaultRootModule->GetControlRigAssetReference(), NAME_None, false);
		}
		else if (const UControlRigBlueprint* DefaultRootModuleBlueprint = Cast<UControlRigBlueprint>(DefaultRootModuleObj))
		{
			FControlRigAssetStrongReference Source = DefaultRootModuleBlueprint->GetControlRigAssetReference();
			if(UClass* DefaultRootModuleClass = Cast<UClass>(Source.GetBlueprintClass()))
			{
				if(DefaultRootModuleClass->IsChildOf(UControlRig::StaticClass()))
				{
					static const FName RootName = TEXT("Root");
					ControlRigAsset->ModularRigModel.GetController()->AddModule(RootName, DefaultRootModuleClass, NAME_None, false);
				}
			}
		}
		else if(UControlRigBlueprintGeneratedClass* DefaultRootModuleGeneratedClass = Cast<UControlRigBlueprintGeneratedClass>(DefaultRootModuleObj))
		{
			static const FName RootName = TEXT("Root");
			ControlRigAsset->ModularRigModel.GetController()->AddModule(RootName, DefaultRootModuleGeneratedClass, NAME_None, false);
		}
	}
	else if (bCreateAsControlRigModule)
	{
		constexpr bool bAutoConvertHierarchy = true;
		FString ErrorMessage;
		FControlRigAssetInterfacePtr(ControlRigAsset->GetEditorOnlyData())->TurnIntoControlRigModule(bAutoConvertHierarchy, &ErrorMessage);

		ensureMsgf(ErrorMessage.IsEmpty(), TEXT("Failed to create Control Rig Module: %s"), *ErrorMessage);
	}

	return ControlRigAsset;
	
}

UObject* UControlRigAssetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
}

bool UControlRigAssetFactory::CanCreateNew() const
{
	return CVarRigVMBlueprintIndependentAssets->GetBool();
}

UControlRigRuntimeAsset* UControlRigAssetFactory::CreateNewControlRigAsset(const FString& InDesiredPackagePath, const bool bModularRig)
{
	return FControlRigAssetActions::CreateNewControlRigAsset(InDesiredPackagePath, bModularRig);
}

UControlRigRuntimeAsset* UControlRigAssetFactory::CreateControlRigFromSkeletalMeshOrSkeleton(UObject* InSelectedObject, const bool bModularRig)
{
	return FControlRigAssetActions::CreateControlRigFromSkeletalMeshOrSkeleton(InSelectedObject, bModularRig);
}

#undef LOCTEXT_NAMESPACE

