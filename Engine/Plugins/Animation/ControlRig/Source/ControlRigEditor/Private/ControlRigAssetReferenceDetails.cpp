// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigAssetReferenceDetails.h"

#include "ContentBrowserModule.h"
#include "ControlRigBlueprintLegacy.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IContentBrowserSingleton.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SControlRigAssetReferencePicker.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ControlRigAssetReferenceDetails"

/** Filter subclass that delegates to a resolved UFUNCTION from GetAssetFilter metadata. */
class FUFunctionAssetFilter : public FControlRigClassFilter
{
public:
	using FFilterDelegate = TDelegate<bool(const FAssetData&)>;

	FUFunctionAssetFilter(FFilterDelegate InDelegate)
		: FControlRigClassFilter(false, false, false, nullptr)
		, FilterDelegate(MoveTemp(InDelegate))
	{}

	virtual bool MatchesFilter(const FAssetData& AssetData) override
	{
		if (FilterDelegate.IsBound())
		{
			return !FilterDelegate.Execute(AssetData);
		}
		return FControlRigClassFilter::MatchesFilter(AssetData);
	}

private:
	FFilterDelegate FilterDelegate;
};

void FControlRigAssetReferenceDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	WeakStructPropertyHandle = InStructPropertyHandle;
	
	FProperty* Property = InStructPropertyHandle->GetProperty();
	if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (StructProperty->Struct == FControlRigAssetStrongReference::StaticStruct())
		{
			bPropertyIsStrongReference = true;
		}
		else if (StructProperty->Struct == FControlRigAssetSoftReference::StaticStruct())
		{
			bPropertyIsStrongReference = false;
		}
		else
		{
			return;
		}
	}
	else
	{
		return;
	}
	
	void* StructData = nullptr;
	const FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructData);
	if(Result == FPropertyAccess::Success)
	{
		check(StructData);
		if (bPropertyIsStrongReference)
		{
			FControlRigAssetStrongReference* StrongReference = static_cast<FControlRigAssetStrongReference*>(StructData);
			SelectedSource = StrongReference->Get();
		}
		else
		{
			FControlRigAssetSoftReference* SoftReference = static_cast<FControlRigAssetSoftReference*>(StructData);
			SelectedSource = *SoftReference;
		}
	}
	
	//RefreshFilteredEntries();

	// Resolve GetAssetFilter metadata to support UPROPERTY-level asset filtering
	static const FName GetAssetFilterMetaName = TEXT("GetAssetFilter");
	if (InStructPropertyHandle->HasMetaData(GetAssetFilterMetaName))
	{
		const FString FilterFunctionName = InStructPropertyHandle->GetMetaData(GetAssetFilterMetaName);
		if (!FilterFunctionName.IsEmpty())
		{
			TArray<UObject*> OuterObjects;
			InStructPropertyHandle->GetOuterObjects(OuterObjects);
			UObject** FoundObject = OuterObjects.FindByPredicate(
				[&FilterFunctionName](const UObject* Obj)
				{
					return Obj && Obj->FindFunction(*FilterFunctionName) != nullptr;
				});
			if (FoundObject)
			{
				AssetFilter = MakeShared<FUFunctionAssetFilter>(
					FUFunctionAssetFilter::FFilterDelegate::CreateUFunction(*FoundObject, FName(*FilterFunctionName))
				);
			}
		}
	}

	InHeaderRow
		.NameContent()
		[
			SNew(STextBlock)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(InStructPropertyHandle, &IPropertyHandle::GetPropertyDisplayName)))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SAssignNew(ComboButton, SComboButton)
					.OnGetMenuContent(this, &FControlRigAssetReferenceDetails::MakeComboMenuContent)
					//.OnComboBoxOpened(this, &FControlRigAssetReferenceDetails::OnComboBoxOpened)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
							{
								return SControlRigAssetReferencePicker::GetDisplayName(SelectedSource);
							})
					]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FControlRigAssetReferenceDetails::OnBrowseToAssetClicked))
			]
		];
}

void FControlRigAssetReferenceDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		// only fill the children if the blueprint cannot be found
		{
			uint32 NumChildren = 0;
			InStructPropertyHandle->GetNumChildren(NumChildren);

			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				InStructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
			}
		}
	}
}

TSharedRef<SWidget> FControlRigAssetReferenceDetails::MakeComboMenuContent()
{
	return SNew(SControlRigAssetReferencePicker)
		.Filter(AssetFilter)
		.OnSelectionChanged_Lambda([this](FControlRigAssetSoftReference InReference)
			{
				if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakStructPropertyHandle.Pin())
				{
					void* StructData = nullptr;
					const FPropertyAccess::Result Result = PropertyHandle->GetValueData(StructData);
					if(Result == FPropertyAccess::Success)
					{
						PropertyHandle->NotifyPreChange();
						
						check(StructData);
						if (bPropertyIsStrongReference)
						{
							FControlRigAssetStrongReference* Data = static_cast<FControlRigAssetStrongReference*>(StructData);
							*Data = InReference.LoadStrongReference();
						}
						else
						{
							FControlRigAssetSoftReference* Data = static_cast<FControlRigAssetSoftReference*>(StructData);
							*Data = InReference;
						}
						
						SelectedSource = InReference;
						PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
						PropertyHandle->NotifyFinishedChangingProperties();
					}
					
					if (ComboButton.IsValid())
					{
						ComboButton->SetIsOpen(false);
					}
				}
			});
}

void FControlRigAssetReferenceDetails::OnBrowseToAssetClicked() const
{
	if (SelectedSource.IsValid())
	{
		const FSoftObjectPath SoftObjectPath = SelectedSource.ToSoftObjectPath();

		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		FAssetData SelectedAsset = AssetRegistry.GetAssetByObjectPath(SoftObjectPath.GetWithoutSubPath());
		if (!SelectedAsset.IsValid())
		{
			SelectedAsset = AssetRegistry.GetAssetByObjectPath(SelectedSource.LoadStrongReference().GetEditorAsset());
		}
		
		IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		ContentBrowserSingleton.SyncBrowserToAssets({SelectedAsset});
	}
}

#undef LOCTEXT_NAMESPACE
