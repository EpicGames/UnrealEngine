// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigAssetReference.h"
#include "IControlRigEditorModule.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

class IPropertyHandle;

class FControlRigAssetReferenceDetails : public IPropertyTypeCustomization
{
	TWeakPtr<IPropertyHandle> WeakStructPropertyHandle;
	bool bPropertyIsStrongReference;
	TArray<TSharedRef<FControlRigAssetSoftReference>> Entries;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SListView<TSharedRef<FControlRigAssetSoftReference>>> AssetList;
	FString FilterText;
	FControlRigAssetSoftReference SelectedSource;
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<FControlRigClassFilter> AssetFilter;
	
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FControlRigAssetReferenceDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;

	TSharedRef<SWidget> MakeComboMenuContent();
	void OnBrowseToAssetClicked() const;
};