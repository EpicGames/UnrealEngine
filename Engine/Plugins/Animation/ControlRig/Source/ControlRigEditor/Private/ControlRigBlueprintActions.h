// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions/AssetTypeActions_Blueprint.h"
#include "ControlRigBlueprintLegacy.h"

class FMenuBuilder;
class UFactory;
class USkeletalMesh;
class AActor;

class FControlRigBlueprintActions : public FAssetTypeActions_Blueprint
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "ControlRigBlueprintActions", "Control Rig"); }
	virtual FColor GetTypeColor() const override { return FColor(140, 116, 0); }
	virtual UClass* GetSupportedClass() const override { return UControlRigBlueprint::StaticClass(); }
	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual bool CanFilter() override { return false; }
	virtual const TArray<FText>& GetSubMenus() const override;
	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& AssetData) const override;
	virtual void PerformAssetDiff(UObject* Asset1, UObject* Asset2, const struct FRevisionInfo& OldRevision, const struct FRevisionInfo& NewRevision) const override;

	// FAssetTypeActions_Blueprint interface
	virtual UFactory* GetFactoryForBlueprintType(UBlueprint* InBlueprint) const override;
	virtual bool CanCreateNewDerivedBlueprint() const override { return false; };

	static void ExtendSketalMeshToolMenu();

	static UControlRigBlueprint* CreateNewControlRigAsset(const FString& InDesiredPackagePath, const bool bModularRig);
	static UControlRigBlueprint* CreateControlRigFromSkeletalMeshOrSkeleton(UObject* InSelectedObject, const bool bModularRig);

	static USkeletalMesh* GetSkeletalMeshFromControlRigBlueprint(const FAssetData& InAsset);
	static void PostSpawningSkeletalMeshActor(AActor* InSpawnedActor, UObject* InAsset);
	static void OnSpawnedSkeletalMeshActorChanged(UObject* InObject, FPropertyChangedEvent& InEvent, UObject* InAsset);

	static FDelegateHandle OnSpawnedSkeletalMeshActorChangedHandle;
};
