// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigBlueprintDetails.h"
#include "ControlRigBlueprintLegacy.h"
#include "DetailLayoutBuilder.h"
#include "Rigs/RigHierarchy.h"

TSharedRef<IDetailCustomization> FControlRigBlueprintDetails::MakeInstance()
{
	return MakeShareable(new FControlRigBlueprintDetails);
}

void FControlRigBlueprintDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	bool bIsValidRigModule = CVarControlRigHierarchyEnableModules.GetValueOnAnyThread();
	if(bIsValidRigModule)
	{
		bIsValidRigModule = false;
		
		TArray<TWeakObjectPtr<UObject>> DetailObjects;
		DetailLayout.GetObjectsBeingCustomized(DetailObjects);
		for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
		{
			if(UControlRigBlueprint* Blueprint = Cast<UControlRigBlueprint>(DetailObject.Get()))
			{
				if(Blueprint->IsControlRigModule())
				{
					Blueprint->UpdateExposedModuleConnectors();
					bIsValidRigModule = true;
				}
			}
		}
	}
	
	if(!bIsValidRigModule)
	{
		DetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(UControlRigBlueprint, RigModuleSettings));
	}
}