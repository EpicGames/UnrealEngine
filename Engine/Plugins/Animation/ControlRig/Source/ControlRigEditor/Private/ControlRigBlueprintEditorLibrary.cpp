// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigBlueprintEditorLibrary.h"

#include "ControlRigAssetActions.h"
#include "Editor/Hierarchy/Views/SRigHierarchy.h"
#include "Editor/Hierarchy/Views/SModularRigHierarchy.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "RigVMModel/RigVMPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigBlueprintEditorLibrary)

FAutoConsoleCommand FCmdControlRigLoadAllAssets
(
	TEXT("ControlRig.LoadAllAssets"),
	TEXT("Loads all control rig assets."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		UControlRigBlueprintEditorLibrary::LoadAssetsByClass(UControlRigBlueprint::StaticClass());
	})
);

void UControlRigBlueprintEditorLibrary::CastToControlRigBlueprint(
	UObject* Object,
	ECastToControlRigBlueprintCases& Branches,
	UControlRigBlueprint*& AsControlRigBlueprint)
{
	AsControlRigBlueprint = Cast<UControlRigBlueprint>(Object);
	Branches = AsControlRigBlueprint == nullptr ? 
		ECastToControlRigBlueprintCases::CastFailed : 
		ECastToControlRigBlueprintCases::CastSucceeded;
}

void UControlRigBlueprintEditorLibrary::SetPreviewMesh(UControlRigBlueprint* InRigBlueprint, USkeletalMesh* PreviewMesh, bool bMarkAsDirty)
{
	if(InRigBlueprint == nullptr)
	{
		return;
	}
	InRigBlueprint->SetPreviewMesh(PreviewMesh, bMarkAsDirty);
}

USkeletalMesh* UControlRigBlueprintEditorLibrary::GetPreviewMesh(UControlRigBlueprint* InRigBlueprint)
{
	if(InRigBlueprint == nullptr)
	{
		return nullptr;
	}
	return InRigBlueprint->GetPreviewMesh();
}

void UControlRigBlueprintEditorLibrary::RequestControlRigInit(UControlRigBlueprint* InRigBlueprint)
{
	if(InRigBlueprint == nullptr)
	{
		return;
	}
	InRigBlueprint->RequestRigVMInit();
}

TArray<UControlRigBlueprint*> UControlRigBlueprintEditorLibrary::GetCurrentlyOpenRigBlueprints()
{
	return UControlRigBlueprint::GetCurrentlyOpenRigBlueprints();
}

TArray<UStruct*> UControlRigBlueprintEditorLibrary::GetAvailableRigUnits()
{
	UControlRigBlueprint* CDO = CastChecked<UControlRigBlueprint>(UControlRigBlueprint::StaticClass()->GetDefaultObject());
	return CDO->GetAvailableRigVMStructs();
}

URigHierarchy* UControlRigBlueprintEditorLibrary::GetHierarchy(UControlRigBlueprint* InRigBlueprint)
{
	if(InRigBlueprint == nullptr)
	{
		return nullptr;
	}
	return InRigBlueprint->Hierarchy;
}

URigHierarchyController* UControlRigBlueprintEditorLibrary::GetHierarchyController(UControlRigBlueprint* InRigBlueprint)
{
	if(InRigBlueprint == nullptr)
	{
		return nullptr;
	}
	return InRigBlueprint->GetHierarchyController();
}

void UControlRigBlueprintEditorLibrary::SetupAllEditorMenus()
{
	SRigHierarchy::CreateContextMenu();
	SRigHierarchy::CreateDragDropMenu();
	SModularRigHierarchy::CreateContextMenu();
}

TArray<FRigModuleDescription> UControlRigBlueprintEditorLibrary::GetAvailableRigModules()
{
	TArray<FRigModuleDescription> ModuleDescriptions;
	
	// Load the asset registry module
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Collect a full list of assets with the specified class
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssetsByClass(UControlRigBlueprint::StaticClass()->GetClassPathName(), AssetDataList, true);

	for(const FAssetData& AssetData : AssetDataList)
	{
		static const FName ModuleSettingsName = GET_MEMBER_NAME_CHECKED(UControlRigBlueprint, RigModuleSettings);
		if(AssetData.FindTag(ModuleSettingsName))
		{
			const FString TagValue = AssetData.GetTagValueRef<FString>(ModuleSettingsName);
			if(TagValue.IsEmpty())
			{
				FRigModuleDescription ModuleDescription;
				ModuleDescription.Path = AssetData.ToSoftObjectPath();
				
				FRigVMPinDefaultValueImportErrorContext ErrorPipe;
				FRigModuleSettings::StaticStruct()->ImportText(*TagValue, &ModuleDescription.Settings, nullptr, PPF_None, &ErrorPipe, FString());
				if(ErrorPipe.NumErrors == 0)
				{
					if(ModuleDescription.Settings.IsValidModule())
					{
						ModuleDescriptions.Add(ModuleDescription);
					}
				}
			}
		}
	}
	
	return ModuleDescriptions;
}

UControlRigRuntimeAsset* UControlRigBlueprintEditorLibrary::CreateControlRigAssetFromBlueprint(UControlRigBlueprint* InRigBlueprint)
{
	return FControlRigAssetActions::CreateNewControlRigAsset(InRigBlueprint);
}
