// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigCompilerDetails.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ControlRigVisualGraphUtils.h"
#include "Units/Execution/RigUnit_BeginExecution.h"

#define LOCTEXT_NAMESPACE "ControlRigCompilerDetails"

#if UE_RIGVM_DEBUG_EXECUTION
//CVar to specify if we should create a float control for each curve in the curve container
//By default we don't but it may be useful to do so for debugging
static TAutoConsoleVariable<int32> CVarControlRigDebugVMExecutionStringEnabled(
	TEXT("ControlRig.DebugVMExecutionStringEnabled"),
	0,
	TEXT("If nonzero we allow to copy the execution of a VM execution."),
	ECVF_Default);
#endif

void FRigVMCompileSettingsDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		InStructPropertyHandle->CreatePropertyValueWidget()
	];

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	ensure(Objects.Num() == 1); // This is in here to ensure we are only showing the modifier details in the blueprint editor

	for (UObject* Object : Objects)
	{
		if (Object->Implements<URigVMEditorAssetInterface>())
		{
			BlueprintBeingCustomized = Object;
		}
	}
}

void FRigVMCompileSettingsDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		uint32 NumChildren = 0;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
		}

		StructBuilder.AddCustomRow(LOCTEXT("MemoryInspection", "Memory Inspection"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Memory Inspection")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked(this, &FRigVMCompileSettingsDetails::OnInspectMemory, ERigVMMemoryType::Literal)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("InspectLiteralMemory", "Inspect Literal Memory"))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked(this, &FRigVMCompileSettingsDetails::OnInspectMemory, ERigVMMemoryType::Work)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("InspectWorkMemory", "Inspect Work Memory"))
					]
				]
			];

		TSharedPtr<SVerticalBox> DebugBox;
		StructBuilder.AddCustomRow(LOCTEXT("DebuggingTools", "Debugging Tools"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Debugging")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SAssignNew(DebugBox, SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked(this, &FRigVMCompileSettingsDetails::OnCopyASTClicked)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("CopyASTToClipboard", "Copy AST Graph"))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked(this, &FRigVMCompileSettingsDetails::OnCopyByteCodeClicked)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("CopyByteCodeToClipboard", "Copy ByteCode"))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked(this, &FRigVMCompileSettingsDetails::OnCopyAnalyticsClicked)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("CopyAnalyticsToClipboard", "Copy Analytics"))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked(this, &FRigVMCompileSettingsDetails::OnCopyHierarchyGraphClicked)
					.ContentPadding(FMargin(2))
					.Content()
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("CopyHierarchyGraphToClipboard", "Copy Hierarchy Graph"))
					]
				]
			];
#if UE_RIGVM_DEBUG_EXECUTION
		if (CVarControlRigDebugVMExecutionStringEnabled->GetBool() == true)
		{
			DebugBox->AddSlot()
			[
				SNew(SButton)
				.OnClicked(this, &FRigVMCompileSettingsDetails::OnCopyVMExecutionClicked)
				.ContentPadding(FMargin(2))
				.Content()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("CopyVMExecution", "Copy VM Execution"))
				]
			];
		}
#endif
	}
}

FReply FRigVMCompileSettingsDetails::OnInspectMemory(ERigVMMemoryType InMemoryType)
{
	if (BlueprintBeingCustomized)
	{
		if(URigVMHost* DebuggedRig = Cast<URigVMHost>(BlueprintBeingCustomized->GetObjectBeingDebugged()))
		{
			if(FRigVMMemoryStorageStruct* MemoryStorage = DebuggedRig->GetMemoryByType(InMemoryType))
			{
				TArray<FRigVMMemoryStorageStruct*> InStructs = { MemoryStorage };
				BlueprintBeingCustomized->RequestInspectMemoryStorage(InStructs);
			}
		}
	}
	return FReply::Handled();
}

FReply FRigVMCompileSettingsDetails::OnCopyASTClicked()
{
	if (BlueprintBeingCustomized)
	{
		if (BlueprintBeingCustomized->GetDefaultModel())
		{
			FString DotContent = BlueprintBeingCustomized->GetDefaultModel()->GetRuntimeAST()->DumpDot();
			FPlatformApplicationMisc::ClipboardCopy(*DotContent);
		}
	}
	return FReply::Handled();
}

FReply FRigVMCompileSettingsDetails::OnCopyByteCodeClicked()
{
	if (BlueprintBeingCustomized)
	{
		if (BlueprintBeingCustomized->GetDefaultModel())
		{
			if(URigVMHost* ControlRig = Cast<URigVMHost>(BlueprintBeingCustomized->GetObjectBeingDebugged()))
			{
				FString ByteCodeContent = ControlRig->GetVM()->DumpByteCodeAsText(ControlRig->GetRigVMExtendedExecuteContext());
				FPlatformApplicationMisc::ClipboardCopy(*ByteCodeContent);
			}
		}
	}
	return FReply::Handled();
}

class FRigVMCompileSettingsDetailsCounterArchive final : public FArchiveUObject
{
public:
	FRigVMCompileSettingsDetailsCounterArchive()
		: Size(0)
	{
		SetIsSaving(true);
		SetIsPersistent(true);
		ArIsCountingMemory = true;
	}

	virtual void Serialize(void*, int64 Length) override { Size += Length; }
	virtual int64 TotalSize() override { return Size; }
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	virtual FArchive& operator<<(UObject*& Obj) override
	{
		Size += sizeof(UObject*);
		return *this;
	}
	virtual FArchive& operator<<(FName& Value) override
	{
		Size += sizeof(FName);
		return *this;
	}

private:
	int64 Size;
};

FReply FRigVMCompileSettingsDetails::OnCopyAnalyticsClicked()
{
	if (BlueprintBeingCustomized)
	{
		if(URigVMHost* Host = Cast<URigVMHost>(BlueprintBeingCustomized->GetObjectBeingDebugged()))
		{
			FString Content;
			URigVM* VM = Host->GetVM();

			int32 ByteCodeFootPrint = 0;
			{
				FRigVMCompileSettingsDetailsCounterArchive Archive;
				Archive << VM->GetByteCode();
				ByteCodeFootPrint = IntCastChecked<int32>(Archive.TotalSize());
			}
			int32 LiteralsFootPrint = 0;
			{
				FRigVMCompileSettingsDetailsCounterArchive Archive;
				Archive << VM->GetDefaultLiteralMemory();
				LiteralsFootPrint = IntCastChecked<int32>(Archive.TotalSize());
			}
			int32 WorkStateFootPrint = 0;
			{
				FRigVMCompileSettingsDetailsCounterArchive Archive;
				Archive << VM->GetDefaultWorkMemory();
				WorkStateFootPrint = IntCastChecked<int32>(Archive.TotalSize());
			}

			Content += FString::Printf(TEXT("ByteCode.Instructions: %04d\n"), VM->GetByteCode().GetNumInstructions());
			Content += FString::Printf(TEXT("ByteCode.Branches: %04d\n"), VM->GetByteCode().NumBranches());
			Content += FString::Printf(TEXT("ByteCode.Callables: %04d\n"), VM->GetByteCode().NumCallables());
			Content += FString::Printf(TEXT("ByteCode.Footprint: %04d bytes\n"), ByteCodeFootPrint);
			Content += FString::Printf(TEXT("Literals.Properties: %04d\n"), VM->GetDefaultLiteralMemory().Num());
			Content += FString::Printf(TEXT("Literals.Footprint: %04d bytes\n"), LiteralsFootPrint);
			Content += FString::Printf(TEXT("WorkState.Properties: %04d\n"), VM->GetDefaultWorkMemory().Num());
			Content += FString::Printf(TEXT("WorkState.Footprint: %04d bytes\n"), WorkStateFootPrint);
			FPlatformApplicationMisc::ClipboardCopy(*Content);
		}
	}
	return FReply::Handled();
}

FReply FRigVMCompileSettingsDetails::OnCopyHierarchyGraphClicked()
{
	if (BlueprintBeingCustomized)
	{
		if(UControlRig* ControlRig = Cast<UControlRig>(BlueprintBeingCustomized->GetObjectBeingDebugged()))
		{
			FName EventName = FRigUnit_BeginExecution::EventName;
			if(!ControlRig->GetEventQueue().IsEmpty())
			{
				EventName = ControlRig->GetEventQueue()[0];
			}
			
			const FString DotGraphContent = FControlRigVisualGraphUtils::DumpRigHierarchyToDotGraph(ControlRig->GetHierarchy(), EventName);
			FPlatformApplicationMisc::ClipboardCopy(*DotGraphContent);
		}
	}
	return FReply::Handled();
}

#if UE_RIGVM_DEBUG_EXECUTION
FReply FRigVMCompileSettingsDetails::OnCopyVMExecutionClicked()
{
	if (BlueprintBeingCustomized)
	{
		if (UControlRig* ControlRig = Cast<UControlRig>(BlueprintBeingCustomized->GetObjectBeingDebugged()))
		{
			FString DebugString = ControlRig->GetDebugExecutionString();
			FPlatformApplicationMisc::ClipboardCopy(*DebugString);
		}
	}
	return FReply::Handled();
}
#endif

#undef LOCTEXT_NAMESPACE
