// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "ControlRigBlueprintLegacy.h"

class IPropertyHandle;

class FRigVMCompileSettingsDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMCompileSettingsDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	FReply OnInspectMemory(ERigVMMemoryType InMemoryType);
	FReply OnCopyASTClicked();
	FReply OnCopyByteCodeClicked();
	FReply OnCopyAnalyticsClicked();
	FReply OnCopyHierarchyGraphClicked();
#if UE_RIGVM_DEBUG_EXECUTION
	FReply OnCopyVMExecutionClicked();
#endif
	
	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized = nullptr;
};
