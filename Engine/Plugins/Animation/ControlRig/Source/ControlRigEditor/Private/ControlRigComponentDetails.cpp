// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigComponentDetails.h"
#include "UObject/Class.h"
#include "ControlRigComponent.h"

#define LOCTEXT_NAMESPACE "ControlRigComponentDetails"

TSharedRef<IDetailCustomization> FControlRigComponentDetails::MakeInstance()
{
	return MakeShareable( new FControlRigComponentDetails );
}

void FControlRigComponentDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
}

#undef LOCTEXT_NAMESPACE
