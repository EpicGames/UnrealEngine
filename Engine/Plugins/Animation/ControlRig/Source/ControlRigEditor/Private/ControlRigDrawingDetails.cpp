// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigDrawingDetails.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "HAL/PlatformApplicationMisc.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "FbxImporter.h"
#include "ScopedTransaction.h"
#include "Dialogs/Dialogs.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigDrawingDetails)

#define LOCTEXT_NAMESPACE "ControlRigDrawingDetails"

void FControlRigDrawContainerDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		InStructPropertyHandle->CreatePropertyValueWidget()
	];

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	ensure(Objects.Num() == 1); // This is in here to ensure we are only showing the modifier details in the blueprint editor

	for (UObject* Object : Objects)
	{
		if (Object->IsA<UControlRigBlueprint>())
		{
			BlueprintBeingCustomized = Cast<UControlRigBlueprint>(Object);
		}
	}
}

void FControlRigDrawContainerDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		uint32 NumChildren = 0;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
		}

		StructBuilder.AddCustomRow(LOCTEXT("Tools", "Tools"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Import")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SButton)
			.OnClicked(this, &FControlRigDrawContainerDetails::OnImportCurvesFromFBXClicked)
			.ContentPadding(FMargin(2))
			.Content()
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text(LOCTEXT("ImportDrawCurvesFromFBX", "Curves from FBX ..."))
			]
		];
	}
}

FReply FControlRigDrawContainerDetails::OnImportCurvesFromFBXClicked()
{
	if (BlueprintBeingCustomized)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		const FText Title = LOCTEXT("ImportCurvesFromFBX", "Import curves from FBX...");
		const FString FileTypes = TEXT("Autodesk FBX (*.fbx)|*.fbx");

		TArray<FString> OutFilenames;
		DesktopPlatform->OpenFileDialog(
			ParentWindowWindowHandle,
			Title.ToString(),
			TEXT(""),
			TEXT("Curves.fbx"),
			FileTypes,
			EFileDialogFlags::None,
			OutFilenames
		);

		if (OutFilenames.Num() == 0)
		{
			return FReply::Unhandled();
		}

		FControlRigDrawContainerImportFbxSettings Settings;
		TSharedPtr<FStructOnScope> StructToDisplay = MakeShareable(new FStructOnScope(FControlRigDrawContainerImportFbxSettings::StaticStruct(), (uint8*)&Settings));
#if WITH_RIGVMLEGACYEDITOR
		TSharedRef<SKismetInspector> KismetInspector = SNew(SKismetInspector);
#else
		TSharedRef<SRigVMDetailsInspector> KismetInspector = SNew(SRigVMDetailsInspector);
#endif
		KismetInspector->ShowSingleStruct(StructToDisplay);

		SGenericDialogWidget::FArguments DialogArguments;
		DialogArguments.OnOkPressed_Lambda([&Settings, this, &OutFilenames] ()
		{
			if(OutFilenames.Num() > 0)
			{
				ImportCurvesFromFBX(OutFilenames[0], BlueprintBeingCustomized, Settings);
			}
		});
	
		SGenericDialogWidget::OpenDialog(LOCTEXT("ControlRigEditorImportFBXCurves", "Import FBX Curves"), KismetInspector, DialogArguments, true);

	}
	return FReply::Handled();
}

void FControlRigDrawContainerDetails::ImportCurvesFromFBX(const FString& InFilePath, FControlRigAssetInterfacePtr InBlueprint, const FControlRigDrawContainerImportFbxSettings& InSettings)
{
	UnFbx::FFbxImporter* Importer = UnFbx::FFbxImporter::GetInstance();

	UnFbx::FBXImportOptions* ImportOptions = Importer->GetImportOptions();
	UnFbx::FBXImportOptions::ResetOptions(ImportOptions);
	ImportOptions->bConvertScene = false;
	ImportOptions->bForceFrontXAxis = false;
	ImportOptions->bConvertSceneUnit = false;
	
	const FString FileExtension = FPaths::GetExtension(InFilePath);
	if (!Importer->ImportFromFile(*InFilePath, FileExtension, true))
	{
		Importer->ReleaseScene();
		return;
	}

	fbxsdk::FbxScene* Scene = Importer->Scene;
	if (Scene == nullptr)
	{
		Importer->ReleaseScene();
		return;
	}

	struct Local
	{
		static void CollectCurveNodes(fbxsdk::FbxNode* FbxNode, TArray<fbxsdk::FbxNode*>& OutFbxCurveNodes)
		{
			if (fbxsdk::FbxNodeAttribute* FbxAttribute = FbxNode->GetNodeAttribute())
			{
				if (FbxAttribute->GetAttributeType() == fbxsdk::FbxNodeAttribute::eNurbsCurve ||
					FbxAttribute->GetAttributeType() == fbxsdk::FbxNodeAttribute::eLine)
				{
					OutFbxCurveNodes.AddUnique(FbxNode);
				}
			}

			for (int32 Index = 0; Index < FbxNode->GetChildCount(); Index++)
			{
				Local::CollectCurveNodes(FbxNode->GetChild(Index), OutFbxCurveNodes);
			}
		}
	};

	TArray<fbxsdk::FbxNode*> FbxCurveNodes;
	for (int32 Index = 0; Index < Scene->GetNodeCount(); Index++)
	{
		Local::CollectCurveNodes(Scene->GetNode(Index), FbxCurveNodes);
	}

	int32 LastInstructionIndex = INDEX_NONE;
	{
		FScopedTransaction Transaction(LOCTEXT("ImportedFbxCurvesToControlRigDrawing", "Import FBX Curves for Drawing"));
		InBlueprint->Modify();

		for (fbxsdk::FbxNode* FbxCurveNode : FbxCurveNodes)
		{
			FRigVMDrawInstruction Instruction;
			Instruction.PrimitiveType = InSettings.bMergeCurves ? ERigVMDrawSettings::Lines : ERigVMDrawSettings::LineStrip;
			Instruction.Transform = UnFbx::FFbxDataConverter::ConvertTransform(FbxCurveNode->EvaluateGlobalTransform());
			Instruction.Transform.SetLocation(Instruction.Transform.GetLocation() * InSettings.Scale);

			fbxsdk::FbxNodeAttribute* FbxNodeAttribute = FbxCurveNode->GetNodeAttribute();
			Instruction.Color = UnFbx::FFbxDataConverter::ConvertColor(FbxNodeAttribute->Color);

			TArray<TArray<FVector>> Lines;
			TArray<bool> LineIsClosed;

			fbxsdk::FbxLine* FbxLine = nullptr;
			fbxsdk::FbxNurbsCurve* FbxNurbsCurve = nullptr;;

			if (FbxNodeAttribute->GetAttributeType() == FbxNodeAttribute::eNurbsCurve)
			{
				FbxNurbsCurve = (fbxsdk::FbxNurbsCurve*)FbxNodeAttribute;

				if (!FbxNurbsCurve->IsPolyline()) // linear curve
				{
					FbxLine = FbxNurbsCurve->TessellateCurve(FMath::Max<int32>(InSettings.Detail, 1));
					FbxNurbsCurve = nullptr;
				}
			}
			else if (FbxNodeAttribute->GetAttributeType() == FbxNodeAttribute::eLine)
			{
				FbxLine = (fbxsdk::FbxLine*)FbxNodeAttribute;
			}

			if (FbxLine)
			{

				TArray<int32> EndPoints;
				EndPoints.Reserve(FbxLine->GetEndPointCount());
				for (int32 Index = 0; Index < FbxLine->GetEndPointCount(); Index++)
				{
					EndPoints.Add(FbxLine->GetEndPointAt(Index));
				}

				TArray<FVector> Line;
				for (int32 Index = 0; Index < FbxLine->GetIndexArraySize(); Index++)
				{
					int32 PointIndex = FbxLine->GetPointIndexAt(Index);
					fbxsdk::FbxVector4 ControlPoint = FbxLine->GetControlPointAt(PointIndex);
					Line.Add(UnFbx::FFbxDataConverter::ConvertPos(ControlPoint) * InSettings.Scale);

					if(EndPoints.Contains(Index))
					{
						Lines.Add(Line);
						LineIsClosed.Add(false);
						Line.Reset();
						continue;
					}
				}
				if (Line.Num() > 0)
				{
					Lines.Add(Line);
					LineIsClosed.Add(true);
				}
			}
			else if (FbxNurbsCurve)
			{
				TArray<FVector> Line;
				for (int32 Index = 0; Index < FbxNurbsCurve->GetControlPointsCount(); Index++)
				{
					fbxsdk::FbxVector4 ControlPoint = FbxNurbsCurve->GetControlPointAt(Index);
					Line.Add(UnFbx::FFbxDataConverter::ConvertPos(ControlPoint) * InSettings.Scale);
				}
				Lines.Add(Line);
				LineIsClosed.Add(FbxNurbsCurve->GetType() == fbxsdk::FbxNurbsCurve::eClosed);
			}

			if (InSettings.bMergeCurves)
			{
				for (TArray<FVector>& Line : Lines)
				{
					for (FVector& Position : Line)
					{
						Position = Instruction.Transform.TransformPosition(Position);
					}
				}
				Instruction.Transform = FTransform::Identity;
			}

			for (int32 LineIndex = 0;LineIndex < Lines.Num(); LineIndex++)
			{
				const TArray<FVector>& Line = Lines[LineIndex];
				if (Line.Num() <= 1)
				{
					continue;
				}

				if (LineIndex == 0)
				{
					Instruction.Name = FbxCurveNode->GetName();
				}
				else
				{
					Instruction.Name = *FString::Printf(TEXT("%hs_Line%d"), FbxCurveNode->GetName(), LineIndex);
				}
				Instruction.Positions.Reset();

				if (Instruction.PrimitiveType == ERigVMDrawSettings::Lines)
				{
					Instruction.Positions.Reserve((Line.Num() - 1) * 2 + (LineIsClosed[LineIndex] ? 2 : 0));
				}
				else
				{
					Instruction.Positions.Reserve(Line.Num() + (LineIsClosed[LineIndex] ? 1 : 0));
				}

				for (int32 Index = 0; Index < Line.Num() - 1; Index++)
				{
					Instruction.Positions.Add(Line[Index]);
					Instruction.Positions.Add(Line[Index + 1]);
				}

				if (LineIsClosed[LineIndex])
				{
					if (Instruction.PrimitiveType == ERigVMDrawSettings::Lines)
					{
						Instruction.Positions.Add(Instruction.Positions.Last());
						Instruction.Positions.Add(Instruction.Positions[0]);
					}
					else
					{
						Instruction.Positions.Add(Instruction.Positions[0]);
					}
				}

				if (InSettings.bMergeCurves && LastInstructionIndex != INDEX_NONE)
				{
					InBlueprint->GetDrawContainer().Instructions[LastInstructionIndex].Positions.Append(Instruction.Positions);
					continue;
				}

				LastInstructionIndex = InBlueprint->GetDrawContainer().Instructions.Add(Instruction);
			}
		}

		FbxCurveNodes.Empty();
	}

	Importer->ReleaseScene();

	if (LastInstructionIndex != INDEX_NONE)
	{
		InBlueprint->PropagateDrawInstructionsFromBPToInstances();
	}
}


#undef LOCTEXT_NAMESPACE

