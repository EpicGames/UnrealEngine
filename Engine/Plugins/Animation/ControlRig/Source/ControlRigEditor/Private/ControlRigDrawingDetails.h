// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigDrawingDetails.generated.h"

class IDetailLayoutBuilder;

USTRUCT()
struct FControlRigDrawContainerImportFbxSettings
{
	GENERATED_BODY()

	FControlRigDrawContainerImportFbxSettings()
		: Scale(1.f)
		, Detail(1)
		, bMergeCurves(false)
	{}

	UPROPERTY(EditAnywhere, Category = "Fbx Import")
	float Scale;

	UPROPERTY(EditAnywhere, Category = "Fbx Import", meta = (UIMin = "1", UIMax = "8"))
	int32 Detail;

	UPROPERTY(EditAnywhere, Category = "Fbx Import")
	bool bMergeCurves;
};

class FControlRigDrawContainerDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FControlRigDrawContainerDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	FReply OnImportCurvesFromFBXClicked();
	static void ImportCurvesFromFBX(const FString& InFilePath, FControlRigAssetInterfacePtr InBlueprint, const FControlRigDrawContainerImportFbxSettings& InSettings);

	FControlRigAssetInterfacePtr BlueprintBeingCustomized;
};
