// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigEditorAssetActions.h"
#include "ControlRigEditorAsset.h"
#include "ControlRigRuntimeAsset.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"

UClass* FControlRigEditorAssetActions::GetSupportedClass() const
{
	return UControlRigEditorAsset::StaticClass();
}

void FControlRigEditorAssetActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	if (IsEngineExitRequested())
	{
		return;
	}

	// Redirect to the runtime asset instead of opening the editor asset directly
	for (UObject* Obj : InObjects)
	{
		if (UControlRigEditorAsset* EditorAsset = Cast<UControlRigEditorAsset>(Obj))
		{
			// The runtime asset is the outer of the editor asset
			if (UControlRigRuntimeAsset* RuntimeAsset = EditorAsset->GetTypedOuter<UControlRigRuntimeAsset>())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RuntimeAsset);
			}
		}
	}
}
