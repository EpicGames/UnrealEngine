// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"

/**
 * Asset type actions for UControlRigEditorAsset that redirects to the runtime asset.
 * When users try to open an EditorAsset directly, we redirect them to open the
 * RuntimeAsset instead, which is the outer object that contains the editor data.
 */
class FControlRigEditorAssetActions : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "ControlRigEditorAssetActions", "Control Rig Editor Data"); }
	virtual FColor GetTypeColor() const override { return FColor(140, 116, 0); }
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual bool CanFilter() override { return false; }
};
