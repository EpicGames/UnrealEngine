// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IControlRigEditorModule.h"
#include "IControlRigModule.h"
#include "IAnimationEditor.h"
#include "IAnimationEditorModule.h"
#include "RigDependencyGraph/RigDependencyGraphPanelNodeFactory.h"

class UControlRigRuntimeAsset;
class IControlRigBaseEditor;
class UBlueprint;
class IAssetTypeActions;
class ISequencer;
class UMaterial;
class UAnimSequence;
class USkeletalMesh;
class USkeleton;
class FToolBarBuilder;
class FExtender;
class FUICommandList;
class UMovieSceneTrack;
class FRigVMEdGraphPanelNodeFactory;
class FControlRigGraphPanelPinFactory;
class FPropertySection;
class FPropertyEditorModule;

class FControlRigEditorModule : public IControlRigEditorModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** FRigVMEditorModule interface */
	virtual UClass* GetRigVMAssetClass() const override;
	virtual UClass* GetRigVMLegacyBlueprintClass() const override;
	virtual UClass* GetRigVMEdGraphSchemaClass() const override;
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const override;
	virtual void GetNodeContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const override;
	virtual void GetPinContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const override;
	virtual bool AssetsPublicFunctionsAllowed(const FAssetData& InAssetData) const override;
	

	/** IControlRigEditorModule interface */
	virtual TSharedRef<IControlRigEditor> CreateControlRigEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UControlRigBlueprint* Blueprint) override;
	virtual TSharedRef<IControlRigEditor> CreateControlRigEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UControlRigRuntimeAsset* Asset) override;
	
	/** Animation Toolbar Extender*/
	void AddControlRigExtenderToToolMenu(FName InToolMenuName);
	TSharedRef< SWidget > GenerateAnimationMenu(TWeakPtr<IAnimationEditor> InAnimationEditor);
	void ToggleIsDrivenByLevelSequence(UAnimSequence* AnimSequence)const;
	bool IsDrivenByLevelSequence(UAnimSequence* AnimSequence)const;
	void EditWithFKControlRig(UAnimSequence* AnimSequence, USkeletalMesh* SkelMesh, USkeleton* InSkeleton);
	CONTROLRIGEDITOR_API void BakeToControlRig(UClass* InClass,UAnimSequence* AnimSequence, USkeletalMesh* SkelMesh,USkeleton* InSkeleton);
	static void OpenLevelSequence(UAnimSequence* AnimSequence);
	static void UnLinkLevelSequence(UAnimSequence* AnimSequence);
	static void ModifyLevelSequenceLinkSettings(UAnimSequence* AnimSequence);
	void ExtendAnimSequenceMenu();

	void GetDirectManipulationMenuActions(IRigVMClientHost* RigVMClientHost, URigVMNode* InNode, URigVMPin* ModelPin, UToolMenu* Menu) const;

private:
	//property sections
	TSharedRef<FPropertySection> RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName);
	void UnregisterPropertySectionMappings();

	TMultiMap<FName, FName> RegisteredPropertySections;

	/** Handle for our sequencer control rig parameter track editor */
	FDelegateHandle ControlRigParameterTrackCreateEditorHandle;

	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

	/** StaticClass is not safe on shutdown, so we cache the name, and use this to unregister on shut down */
	TArray<FName> ClassesToUnregisterOnShutdown;
	TArray<FName> PropertiesToUnregisterOnShutdown;
	
	/** Param to hold Filter Result to pass to Filter*/
	bool bFilterAssetBySkeleton;

	/** Handles for all registered workflows */
	TArray<int32> WorkflowHandles;
	
	TSharedPtr<FRigDependencyGraphPanelNodeFactory> RigDependencyGraphPanelNodeFactory;

	/** Guids for registered functions for creating custom UObjects for space and constraint channels for scripting*/
	FGuid SpaceCustomChannelGuid;
	FGuid ConstraintCustomChannelGuid;

};
