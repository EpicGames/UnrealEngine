// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Brushes/SlateNoResource.h"
#include "Brushes/SlateRoundedBoxBrush.h"

class FControlRigEditorStyle final
	: public FSlateStyleSet
{
	class FContentRootBracket
	{
	public:
		FContentRootBracket(FControlRigEditorStyle* InStyle, const FString& NewContentRoot)
			: Style(InStyle)
			, PreviousContentRoot(InStyle->GetContentRootDir())
		{
			Style->SetContentRoot(NewContentRoot);
		}

		~FContentRootBracket()
		{
			Style->SetContentRoot(PreviousContentRoot);
		}
	private:
		FControlRigEditorStyle* Style;
		FString PreviousContentRoot;
	};
	
public:
	FControlRigEditorStyle()
		: FSlateStyleSet("ControlRigEditorStyle")
		, BoneUserInterfaceColor(0.6f, 0.6f, 0.6f)
		, NullUserInterfaceColor(0.75f, 0.75f, 0.75f)
		, ConnectorUserInterfaceColor(0.0, 112.f/255.f, 224.f/255.f)
		, SocketUserInterfaceColor(0.0, 112.f/255.f, 224.f/255.f)
	{
		const FVector2D Icon10x10(10.0f, 10.0f);
		const FVector2D Icon14x14(14.0f, 14.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon32x32(32.0f, 32.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon128x128(128.0f, 128.0f);
		const FString ControlRigPluginContentDir = FPaths::EnginePluginsDir() / TEXT("Animation/ControlRig/Content");
		const FString EngineSlateDir = FPaths::EngineContentDir() / TEXT("Slate");
		const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
		SetContentRoot(ControlRigPluginContentDir);
		SetCoreContentRoot(EngineEditorSlateDir);

		const FSlateColor ErrorBorderColor = FAppStyle::GetSlateColor("Colors.Error");
		FLinearColor ErrorFillColor = ErrorBorderColor.GetSpecifiedColor();

		const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

		// Class Icons
		{
			Set("ClassIcon.ControlRigSequence", new IMAGE_BRUSH("Slate/ControlRigSequence_16x", Icon16x16));
			Set("ClassIcon.ControlRigBlueprint", new IMAGE_BRUSH("Slate/ControlRig_16", Icon16x16));
			Set("ClassIcon.ControlRigPose", new IMAGE_BRUSH("Slate/ControlRigPose_16", Icon16x16));
		}

		// Editor Icons
		{
			Set("ControlRig.Editor.TabIcon", new IMAGE_BRUSH_SVG("Slate/ControlRigEditorTabIcon_16x", Icon16x16));
		}

		// Sequencer styles
		{
			Set("ControlRig.ExportAnimSequence", new IMAGE_BRUSH("Slate/ExportAnimSequence_24x", Icon24x24));
			Set("ControlRig.ExportAnimSequence.Small", new IMAGE_BRUSH("Slate/ExportAnimSequence_24x", Icon24x24));
			Set("ControlRig.ReExportAnimSequence", new IMAGE_BRUSH("Slate/ExportAnimSequence_24x", Icon24x24));
			Set("ControlRig.ReExportAnimSequence.Small", new IMAGE_BRUSH("Slate/ExportAnimSequence_24x", Icon24x24));
			Set("ControlRig.ImportFromRigSequence", new IMAGE_BRUSH("Slate/ReImportRigSequence_16x", Icon16x16));
			Set("ControlRig.ImportFromRigSequence.Small", new IMAGE_BRUSH("Slate/ReImportRigSequence_16x", Icon16x16));
			Set("ControlRig.ReImportFromRigSequence", new IMAGE_BRUSH("Slate/ReImportRigSequence_16x", Icon16x16));
			Set("ControlRig.ReImportFromRigSequence.Small", new IMAGE_BRUSH("Slate/ReImportRigSequence_16x", Icon16x16));
		}

		//Tool Styles
		{
			Set("ControlRig.OnlySelectControls", new IMAGE_BRUSH_SVG("Slate/AnimationSelectOnlyControlRig", Icon16x16));
			Set("ControlRig.ConstraintTools", new IMAGE_BRUSH_SVG("Slate/AnimationConstraintTools_16", Icon16x16));
			Set("ControlRig.PoseTool", new IMAGE_BRUSH_SVG("Slate/AnimationPoses", Icon16x16));
			Set("ControlRig.CreatePose", new IMAGE_BRUSH_SVG("Slate/AnimationCreatePose", Icon16x16));
			Set("ControlRig.TweenTool", new IMAGE_BRUSH_SVG("Slate/AnimationTweens", Icon16x16));
			Set("ControlRig.EditableMotionTrails", new IMAGE_BRUSH_SVG("Slate/EditableMotionTrails", Icon16x16));
			Set("ControlRig.TemporaryPivot", new IMAGE_BRUSH_SVG("Slate/TemporaryPivot", Icon16x16));
			Set("ControlRig.AnimLayerSelected", new IMAGE_BRUSH_SVG("Slate/AnimLayerSelected", Icon16x16));
			Set("ControlRig.FilterAnimLayerSelected", new IMAGE_BRUSH_SVG("Slate/FilterAnimLayerSelected", Icon16x16));
			Set("ControlRig.AnimLayers", new IMAGE_BRUSH_SVG("Slate/AnimLayers", Icon16x16));
			Set("ControlRig.KeyAdd", new IMAGE_BRUSH_SVG("Slate/KeyAdd_14", Icon14x14));
			Set("ControlRig.KeySpecial", new IMAGE_BRUSH_SVG("Slate/KeySpecial_14", Icon14x14));
			Set("ControlRig.SelectionSet", new IMAGE_BRUSH_SVG("Slate/SectionSelection", Icon16x16));
			Set("ControlRig.FilterSelected", new IMAGE_BRUSH_SVG("Slate/CurveEditorSettingsSelectionFilter", Icon16x16));



		}
		// Control Rig Editor styles
		{
			// tab icons
			Set("RigHierarchy.TabIcon", new IMAGE_BRUSH_SVG("Slate/RigHierarchy", Icon16x16));
			Set("RigValidation.TabIcon", new IMAGE_BRUSH_SVG("Slate/RigValidation", Icon16x16));
			Set("CurveContainer.TabIcon", new IMAGE_BRUSH_SVG("Slate/CurveContainer", Icon16x16));
			Set("HierarchicalProfiler.TabIcon", new IMAGE_BRUSH_SVG("Slate/HierarchicalProfiler", Icon16x16));
			
			{
				FContentRootBracket Bracket(this, EngineEditorSlateDir);
				Set("ControlRig.ConstructionMode", new IMAGE_BRUSH_SVG("Starship/Common/Adjust", Icon40x40));
				Set("ControlRig.ConstructionMode.Small", new IMAGE_BRUSH_SVG("Starship/Common/Adjust", Icon20x20));
				Set("ControlRig.ForwardsSolveEvent", new IMAGE_BRUSH("Icons/diff_next_40x", Icon40x40));
				Set("ControlRig.BackwardsSolveEvent", new IMAGE_BRUSH("Icons/diff_prev_40x", Icon40x40));
				Set("ControlRig.BackwardsAndForwardsSolveEvent", new IMAGE_BRUSH("Icons/Loop_40x", Icon40x40));
			}

			{
				FContentRootBracket Bracket(this, EngineEditorSlateDir);
				// similar style to "LevelViewport.StartingPlayInEditorBorder"
				Set( "ControlRig.Viewport.Border", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(1.0f,1.0f,1.0f,1.0f) ) );
				// similar style to "AnimViewport.Notification.Warning"
				Set( "ControlRig.Viewport.Notification.ChangeShapeTransform", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, FLinearColor(FColor(169, 0, 148))));
				// similar style to "AnimViewport.Notification.Warning"
				Set( "ControlRig.Viewport.Notification.DirectManipulation", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, FLinearColor(FColor(0, 112, 224))));
				// similar style to "AnimViewport.Notification.Warning"
				Set( "ControlRig.Viewport.Notification.ReplayValidation", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, ErrorFillColor));
				// similar style to "AnimViewport.Notification.Warning"
				Set( "ControlRig.Viewport.Notification.PreviewingNode", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, FLinearColor(FColor(0, 0, 255))));
			}

			// Rig hierarchy tree
			{
				Set("ControlRig.Tree.BoneUser", new IMAGE_BRUSH("Slate/BoneNonWeighted_16x", Icon16x16));
				Set("ControlRig.Tree.BoneImported", new IMAGE_BRUSH("Slate/Bone_16x", Icon16x16));
				Set("ControlRig.Tree.Control", new IMAGE_BRUSH("Slate/RigControlCircle_16x", Icon16x16));
				Set("ControlRig.Tree.ProxyControl", new IMAGE_BRUSH("Slate/ProxyControl1_16x", Icon16x16));
				Set("ControlRig.Tree.Null", new IMAGE_BRUSH("Slate/Null_16x", Icon16x16));
				Set("ControlRig.Tree.RigidBody", new IMAGE_BRUSH("Slate/RigidBody_16x", Icon16x16));
				Set("ControlRig.Tree.Socket_Open", new IMAGE_BRUSH_SVG("Slate/Socket_Open", Icon16x16));
				Set("ControlRig.Tree.Socket_Closed", new IMAGE_BRUSH_SVG("Slate/Socket_Closed", Icon16x16));
				{
					FContentRootBracket Bracket(this, EngineEditorSlateDir);
					Set("ControlRig.Tree.Connector", new IMAGE_BRUSH_SVG("Starship/Common/SetShowSockets", Icon16x16));
				}
			}

			// Modular rig hierarchy tree
			{
				Set("ModularRig.Tree.RoundedBox", new FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Transparent, 0.0f));
			}
		}



		// Font?
		{
			Set("ControlRig.Hierarchy.Menu", DEFAULT_FONT("Regular", 12));
		}

		// Space picker
		{
			const FTableRowStyle SpacePickerTransparentTableRowStyle = FTableRowStyle()
				.SetEvenRowBackgroundBrush(FSlateNoResource())
				.SetOddRowBackgroundBrush(FSlateNoResource())
				.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
				.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
				.SetSelectorFocusedBrush(FSlateNoResource())
				.SetActiveBrush(FSlateNoResource())
				.SetActiveHoveredBrush(FSlateNoResource())
				.SetInactiveBrush(FSlateNoResource())
				.SetInactiveHoveredBrush(FSlateNoResource());

			Set("ControlRig.SpacePicker.TableRowStyle", SpacePickerTransparentTableRowStyle);

			Set("ControlRig.SpacePicker.RoundedRect", new FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FStyleColors::Transparent, 0.0f));

			Set("ControlRig.SpacePicker.ActiveSpaceColor", FStyleColors::Select);
			Set("ControlRig.SpacePicker.PartiallyActiveSpaceColor", FStyleColors::SelectInactive);
		}

		// Test Data
		{
			Set("ControlRig.Replay.Record", new IMAGE_BRUSH("Slate/RecordingIndicator", Icon32x32));
		}

		Set("ControlRig.ConnectorPlain.Small", new IMAGE_BRUSH_SVG("Slate/Connector_Plain", Icon16x16));
		Set("ControlRig.ConnectorPrimary", new IMAGE_BRUSH_SVG("Slate/Connector_Primary", Icon128x128));
		Set("ControlRig.ConnectorSecondary", new IMAGE_BRUSH_SVG("Slate/Connector_Secondary", Icon128x128));
		Set("ControlRig.ConnectorOptional", new IMAGE_BRUSH_SVG("Slate/Connector_Optional", Icon128x128));
		Set("ControlRig.ConnectorWarning", new IMAGE_BRUSH_SVG("Slate/Connector_Warning", Icon128x128));

		// Schematic
		{
			Set("ControlRig.Schematic.ConnectorPrimary", new IMAGE_BRUSH_SVG("Slate/Connector_Primary_Schematic", Icon128x128));
			Set("ControlRig.Schematic.ConnectorSecondary", new IMAGE_BRUSH_SVG("Slate/Connector_Secondary_Schematic", Icon128x128));
			Set("ControlRig.Schematic.ConnectorOptional", new IMAGE_BRUSH_SVG("Slate/Connector_Optional_Schematic", Icon128x128));
			Set("ControlRig.Schematic.ConnectorWarning", new IMAGE_BRUSH_SVG("Slate/Connector_Warning_Schematic", Icon128x128));
			Set("ControlRig.Schematic.Bone", new IMAGE_BRUSH_SVG("Slate/Bone_Schematic", Icon128x128));
			Set("ControlRig.Schematic.Control", new IMAGE_BRUSH_SVG("Slate/Control_Schematic", Icon128x128));
			Set("ControlRig.Schematic.Null", new IMAGE_BRUSH_SVG("Slate/Null_Schematic", Icon128x128));
			Set("ControlRig.Schematic.Link", new IMAGE_BRUSH_SVG("Slate/Link_Schematic", Icon128x128));
		}

		// Constraint Manager Icons
		{
			const FButtonStyle ConstraintOptionButton = FButtonStyle()
				.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f))
				.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
				.SetHoveredForeground(FLinearColor::White)
				.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
				.SetPressedForeground(FLinearColor::White)
				.SetPressedPadding(FMargin(0.0, 1.0, 0.0, 0.0));

			FContentRootBracket Bracket(this, EngineSlateDir);		

			FComboButtonStyle ConstraintComboButton = FComboButtonStyle()
				.SetButtonStyle(ConstraintOptionButton)
				.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 15.f)));
			ConstraintComboButton.ButtonStyle = ConstraintOptionButton;
			
			Set("ConstraintManager.ComboButton", ConstraintComboButton);
		}

		// Dependency Graph
		{
 			const FTextBlockStyle NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
			Set("ControlRig.DependencyGraph.TextStyle", FTextBlockStyle(NormalText)
				.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f))
				.SetFont(DEFAULT_FONT("Regular", 8)));

			{
				FContentRootBracket Bracket(this, EngineSlateDir);		
				Set("ControlRig.DependencyGraph.Menu", new IMAGE_BRUSH_SVG("Starship/Common/menu", Icon20x20));
			}

			Set( "ControlRig.DependencyGraph.NodeBody", new BOX_BRUSH( "Slate/DependencyGraphNode_Body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );

			Set("ControlRig.DependencyGraph.Colors.Instruction", FLinearColor(FColor::FromHex(TEXT("#00AAAA"))));
			Set("ControlRig.DependencyGraph.Colors.Bone", FLinearColor(FColor::FromHex(TEXT("#E166B6"))));
			Set("ControlRig.DependencyGraph.Colors.Control", FLinearColor(FColor::FromHex(TEXT("#45C761"))));
			Set("ControlRig.DependencyGraph.Colors.AnimationChannel", FLinearColor(FColor::FromHex(TEXT("#FE8539"))));
			Set("ControlRig.DependencyGraph.Colors.Null", FLinearColor(FColor::FromHex(TEXT("#C7C8FE"))));
			Set("ControlRig.DependencyGraph.Colors.Socket", FLinearColor(FColor::FromHex(TEXT("#F9E264"))));
			Set("ControlRig.DependencyGraph.Colors.Connector", FLinearColor(FColor::FromHex(TEXT("#FE8989"))));
			Set("ControlRig.DependencyGraph.Colors.Metadata", FLinearColor(FColor::FromHex(TEXT("#BB6BF0"))));
			Set("ControlRig.DependencyGraph.Colors.Variable", FLinearColor(FColor::FromHex(TEXT("#CCCC00"))));

			Set( "ControlRig.DependencyGraph.HideUnrelated", new IMAGE_BRUSH_SVG( "Slate/GraphHideUnrelated_20", Icon20x20 ) );
			Set( "ControlRig.DependencyGraph.IsolateSelection", new IMAGE_BRUSH_SVG( "Slate/GraphIsolate_20", Icon20x20 ) );
			Set( "ControlRig.DependencyGraph.ParentChild", new IMAGE_BRUSH_SVG( "Slate/GraphParentChild_20", Icon20x20 ) );
			Set( "ControlRig.DependencyGraph.VMRelationShips", new IMAGE_BRUSH_SVG( "Slate/GraphVMRelationships_20", Icon20x20 ) );
			Set( "ControlRig.DependencyGraph.Cleanup", new IMAGE_BRUSH_SVG( "Slate/GraphCleanup_20", Icon20x20 ) );
			Set( "ControlRig.DependencyGraph.Flashlight", new IMAGE_BRUSH_SVG( "Slate/Flashlight_20", Icon20x20 ) );
		}

		// Tabs
		{
			FCheckBoxStyle BaseCheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
			Set("ControlRig.TabButton", BaseCheckBoxStyle
				.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
				.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
				.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
				.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Input, 1.0f))
				.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
				.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
				.SetPadding(FMargin(16, 6))
			);
			
			Set("ControlRig.Constrain.Spaces", new IMAGE_BRUSH_SVG("Slate/AnimationSpaces_16", Icon16x16));
			Set("ControlRig.Constrain.Constraints", new IMAGE_BRUSH_SVG("Slate/AnimationConstraint_16", Icon16x16));
			Set("ControlRig.Constrain.Snapper", new IMAGE_BRUSH_SVG("Slate/AnimationControlRigSnapper_16", Icon16x16));
		}
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FControlRigEditorStyle& Get()
	{
		static FControlRigEditorStyle Inst;
		return Inst;
	}
	
	~FControlRigEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FLinearColor BoneUserInterfaceColor;
	FLinearColor NullUserInterfaceColor;
	FLinearColor ConnectorUserInterfaceColor;
	FLinearColor SocketUserInterfaceColor;
};
