// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigElementDetails.h"
#include "Widgets/SWidget.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "ModularRig.h"
#include "Graph/ControlRigGraph.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "SEnumCombo.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_DynamicHierarchy.h"
#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Styling/AppStyle.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Widgets/SRigVMGraphPinEnumPicker.h"
#include "IStructureDataProvider.h"
#include "Editor/ControlRigEditor.h"
#include "Editor/SRigConnectorTargetWidget.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyItem.h"
#include "ModularRigRuleManager.h"
#include "Async/TaskGraphInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigElementDetails)

#define LOCTEXT_NAMESPACE "ControlRigElementDetails"

static const FText ControlRigDetailsMultipleValues = LOCTEXT("MultipleValues", "Multiple Values");
static const FText ControlRigDetailsInvalid = LOCTEXT("Invalid", "Invalid");

struct FRigElementTransformWidgetSettings
{
	FRigElementTransformWidgetSettings()
	: RotationRepresentation(MakeShareable(new ESlateRotationRepresentation::Type(ESlateRotationRepresentation::Rotator)))
	, IsComponentRelative(MakeShareable(new UE::Math::TVector<float>(1.f, 1.f, 1.f)))
	, IsScaleLocked(TSharedPtr<bool>(new bool(false)))
	{
	}

	TSharedPtr<ESlateRotationRepresentation::Type> RotationRepresentation;
	TSharedRef<UE::Math::TVector<float>> IsComponentRelative;
	TSharedPtr<bool> IsScaleLocked;

	static FRigElementTransformWidgetSettings& FindOrAdd(
		ERigControlValueType InValueType,
		ERigTransformElementDetailsTransform::Type InTransformType,
		const SAdvancedTransformInputBox<FEulerTransform>::FArguments& WidgetArgs)
	{
		uint32 Hash = GetTypeHash(WidgetArgs._ConstructLocation);
		Hash = HashCombine(Hash, GetTypeHash(WidgetArgs._ConstructRotation));
		Hash = HashCombine(Hash, GetTypeHash(WidgetArgs._ConstructScale));
		Hash = HashCombine(Hash, GetTypeHash(WidgetArgs._AllowEditRotationRepresentation));
		Hash = HashCombine(Hash, GetTypeHash(WidgetArgs._DisplayScaleLock));
		Hash = HashCombine(Hash, GetTypeHash(InValueType));
		Hash = HashCombine(Hash, GetTypeHash(InTransformType));
		return sSettings.FindOrAdd(Hash);
	}

	static TMap<uint32, FRigElementTransformWidgetSettings> sSettings;
};

TMap<uint32, FRigElementTransformWidgetSettings> FRigElementTransformWidgetSettings::sSettings;


void RigElementKeyDetails_GetCustomizedInfo(TSharedRef<IPropertyHandle> InStructPropertyHandle, FControlRigAssetInterfacePtr& OutBlueprint)
{
	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		if(!IsValid(Object))
		{
			continue;
		}
		if (Object->Implements<UControlRigEditorAssetInterface>())
		{
			OutBlueprint = Object;
			break;
		}

		OutBlueprint = IControlRigEditorAssetInterface::GetInterfaceOuter(Object);
		if(OutBlueprint)
		{
			break;
		}

		if(const UControlRig* ControlRig = Object->GetTypedOuter<UControlRig>())
		{
			OutBlueprint = ControlRig->GetAssetReference().GetEditorAsset();
			if(OutBlueprint)
			{
				break;
			}
		}
	}

	if (OutBlueprint == nullptr)
	{
		TArray<UPackage*> Packages;
		InStructPropertyHandle->GetOuterPackages(Packages);
		for (UPackage* Package : Packages)
		{
			if (Package == nullptr)
			{
				continue;
			}

			TArray<UObject*> SubObjects;
			Package->GetDefaultSubobjects(SubObjects);
			for (UObject* SubObject : SubObjects)
			{
				if (UControlRig* Rig = Cast<UControlRig>(SubObject))
				{
					FControlRigAssetInterfacePtr Blueprint = Rig->GetAssetReference().GetEditorAsset();
					if (Blueprint)
					{
						if(Blueprint.GetObject()->GetOutermost() == Package)
						{
							OutBlueprint = Blueprint;
							break;
						}
					}
				}
			}

			if (OutBlueprint)
			{
				break;
			}
		}
	}

	UControlRigGraph* RigGraph = nullptr;
	if(OutBlueprint)
	{
		for (UEdGraph* Graph : OutBlueprint->GetRigVMAssetInterface()->GetUberGraphs())
		{
			RigGraph = Cast<UControlRigGraph>(Graph);
			if (RigGraph)
			{
				break;
			}
		}
	}

	// only allow blueprints with at least one rig graph
	if (RigGraph == nullptr)
	{
		OutBlueprint = nullptr;
	}

}

FControlRigAssetInterfacePtr RigElementDetails_GetBlueprintFromHierarchy(URigHierarchy* InHierarchy)
{
	if(InHierarchy == nullptr)
	{
		return nullptr;
	}

	FControlRigAssetInterfacePtr Blueprint = IControlRigEditorAssetInterface::GetInterfaceOuter(InHierarchy);
	if(Blueprint == nullptr)
	{
		UControlRig* Rig = InHierarchy->GetTypedOuter<UControlRig>();
		if(Rig)
		{
			Blueprint = FControlRigAssetInterfacePtr(Rig->GetClass()->ClassGeneratedBy);
        }
	}
	if (Blueprint == nullptr)
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = IRigVMRuntimeAssetInterface::GetInterfaceOuter(InHierarchy))
		{
			Blueprint = RuntimeAsset->GetEditorOnlyData();
		}
	}
	return Blueprint;
}

void SRigElementKeyWidget::Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InNameHandle, TSharedPtr<IPropertyHandle> InTypeHandle)
{
	NameHandle = InNameHandle;
	TypeHandle = InTypeHandle;
	Construct(InArgs);
}

void SRigElementKeyWidget::Construct(const FArguments& InArgs)
{
	BlueprintBeingCustomized = InArgs._Asset;
	OnGetElementType = InArgs._OnGetElementType;
	OnElementNameChanged = InArgs._OnElementNameChanged;
	OnElementTypeChanged = InArgs._OnElementTypeChanged;
	
	UpdateElementNameList();

	TWeakPtr<SRigElementKeyWidget> WeakThisPtr = StaticCastWeakPtr<SRigElementKeyWidget>(AsWeak());
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			TypeHandle.IsValid() ?
				TypeHandle->CreatePropertyValueWidget()
					:
				SNew(SEnumComboBox, StaticEnum<ERigElementType>())
				.CurrentValue_Lambda([WeakThisPtr]()
				{
					if(const TSharedPtr<SRigElementKeyWidget> StrongThisPtr = WeakThisPtr.Pin())
					{
						if (StrongThisPtr.IsValid() && StrongThisPtr->OnGetElementType.IsBound())
						{
							return (int32)StrongThisPtr->OnGetElementType.Execute();
						}
					}
					return (int32)ERigElementType::None;
				})
				.OnEnumSelectionChanged_Lambda([WeakThisPtr](int32 InEnumValue, ESelectInfo::Type SelectInfo)
				{
					if(const TSharedPtr<SRigElementKeyWidget> StrongThisPtr = WeakThisPtr.Pin())
					{
						if(StrongThisPtr.IsValid())
						{
							ERigElementType EnumValue = static_cast<ERigElementType>(InEnumValue);
							StrongThisPtr->OnElementTypeChanged.ExecuteIfBound(EnumValue);
							StrongThisPtr->UpdateElementNameList();
							StrongThisPtr->SearchableComboBox->ClearSelection();
							StrongThisPtr->OnElementNameChanged.ExecuteIfBound(nullptr, ESelectInfo::Direct);
						}
					}
				})
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SAssignNew(SearchableComboBox, SSearchableComboBox)
			.OptionsSource(&ElementNameList)
			.OnSelectionChanged(InArgs._OnElementNameChanged)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
			{
				return SNew(STextBlock)
				.Text(FText::FromString(InItem.IsValid() ? *InItem : FString()))
				.Font(IDetailLayoutBuilder::GetDetailFont());
			})
			.Content()
			[
				SNew(STextBlock)
				.Text_Lambda([InArgs]()
				{
					if (InArgs._OnGetElementNameAsText.IsBound())
					{
						return InArgs._OnGetElementNameAsText.Execute();
					}
					return FText();
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
		// Use button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0)
		.VAlign(VAlign_Center)
		[
			SAssignNew(UseSelectedButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ButtonColorAndOpacity_Lambda([this, InArgs]() { return UseSelectedButton.IsValid() && UseSelectedButton->IsHovered() ? InArgs._ActiveBackgroundColor : InArgs._InactiveBackgroundColor; })
			.OnClicked(InArgs._OnGetSelectedClicked)
			.ContentPadding(1.f)
			.ToolTipText(NSLOCTEXT("ControlRigElementDetails", "ObjectGraphPin_Use_Tooltip", "Use item selected"))
			[
				SNew(SImage)
				.ColorAndOpacity_Lambda( [this, InArgs]() { return UseSelectedButton.IsValid() && UseSelectedButton->IsHovered() ? InArgs._ActiveForegroundColor : InArgs._InactiveForegroundColor; })
				.Image(FAppStyle::GetBrush("Icons.CircleArrowLeft"))
			]
		]
		// Select in hierarchy button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0)
		.VAlign(VAlign_Center)
		[
			SAssignNew(SelectElementButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.ButtonColorAndOpacity_Lambda([this, InArgs]() { return SelectElementButton.IsValid() && SelectElementButton->IsHovered() ? InArgs._ActiveBackgroundColor : InArgs._InactiveBackgroundColor; })
			.OnClicked(InArgs._OnSelectInHierarchyClicked)
			.ContentPadding(0)
			.ToolTipText(NSLOCTEXT("ControlRigElementDetails", "ObjectGraphPin_Browse_Tooltip", "Select in hierarchy"))
			[
				SNew(SImage)
				.ColorAndOpacity_Lambda( [this, InArgs]() { return SelectElementButton.IsValid() && SelectElementButton->IsHovered() ? InArgs._ActiveForegroundColor : InArgs._InactiveForegroundColor; })
				.Image(FAppStyle::GetBrush("Icons.Search"))
			]
		]
	];

	if (TypeHandle)
	{
		TypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
			[this, InArgs]()
			{
				int32 EnumValue;
				TypeHandle->GetValue(EnumValue);
				OnElementTypeChanged.ExecuteIfBound(static_cast<ERigElementType>(EnumValue));
				UpdateElementNameList();
				SearchableComboBox->ClearSelection();
				OnElementNameChanged.ExecuteIfBound(nullptr, ESelectInfo::Direct);
			}
		));
	}
}

void SRigElementKeyWidget::UpdateElementNameList()
{
	ElementNameList.Reset();

	if (BlueprintBeingCustomized)
	{
		for (UEdGraph* Graph : BlueprintBeingCustomized->GetRigVMAssetInterface()->GetUberGraphs())
		{
			if (UControlRigGraph* RigGraph = Cast<UControlRigGraph>(Graph))
			{
				
				const TArray<TSharedPtr<FRigVMStringWithTag>>* NameList = nullptr;
				if (OnGetElementType.IsBound())
				{
					NameList = RigGraph->GetElementNameList(OnGetElementType.Execute());
				}

				ElementNameList.Reset();
				if (NameList)
				{
					ElementNameList.Reserve(NameList->Num());
					for(const TSharedPtr<FRigVMStringWithTag>& Name : *NameList)
					{
						ElementNameList.Add(MakeShared<FString>(Name->GetString()));
					}
				}
				
				if(SearchableComboBox.IsValid())
				{
					SearchableComboBox->RefreshOptions();
				}
				return;
			}
		}
	}
}

void FRigElementKeyDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	BlueprintBeingCustomized = nullptr;
	RigElementKeyDetails_GetCustomizedInfo(InStructPropertyHandle, BlueprintBeingCustomized);

	TWeakPtr<FRigElementKeyDetails> WeakThisPtr = StaticCastWeakPtr<FRigElementKeyDetails>(AsWeak());

	if (BlueprintBeingCustomized == nullptr)
	{
		HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			InStructPropertyHandle->CreatePropertyValueWidget(false)
		];
	}
	else
	{
		TypeHandle = InStructPropertyHandle->GetChildHandle(TEXT("Type"));
		NameHandle = InStructPropertyHandle->GetChildHandle(TEXT("Name"));

		HeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		[
			SAssignNew(RigElementKeyWidget, SRigElementKeyWidget, NameHandle, TypeHandle)
			.Asset(BlueprintBeingCustomized)
			.IsEnabled_Lambda([WeakThisPtr]()
			{
				if(const TSharedPtr<FRigElementKeyDetails> StrongThisPtr = WeakThisPtr.Pin())
				{
					if(StrongThisPtr.IsValid())
					{
						return !StrongThisPtr->NameHandle->IsEditConst();
					}
				}
				return false;
			})
			.ActiveBackgroundColor(FSlateColor(FLinearColor(1.f, 1.f, 1.f, FRigElementKeyDetailsDefs::ActivePinBackgroundAlpha)))
			.ActiveForegroundColor(FSlateColor(FLinearColor(1.f, 1.f, 1.f, FRigElementKeyDetailsDefs::ActivePinForegroundAlpha)))
			.InactiveBackgroundColor(FSlateColor(FLinearColor(1.f, 1.f, 1.f, FRigElementKeyDetailsDefs::InactivePinBackgroundAlpha)))
			.InactiveForegroundColor(FSlateColor(FLinearColor(1.f, 1.f, 1.f, FRigElementKeyDetailsDefs::InactivePinForegroundAlpha)))
			.OnElementNameChanged(this, &FRigElementKeyDetails::OnElementNameChanged)
			.OnGetSelectedClicked(this, &FRigElementKeyDetails::OnGetSelectedClicked)
			//.OnGetElementNameWidget(this, &FRigElementKeyDetails::OnGetElementNameWidget)
			.OnSelectInHierarchyClicked(this, &FRigElementKeyDetails::OnSelectInHierarchyClicked)
			.OnGetElementNameAsText_Raw(this, &FRigElementKeyDetails::GetElementNameAsText)
			.OnGetElementType(this, &FRigElementKeyDetails::GetElementType)
		];
	}
}

void FRigElementKeyDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		// only fill the children if the blueprint cannot be found
		if (BlueprintBeingCustomized == nullptr)
		{
			uint32 NumChildren = 0;
			InStructPropertyHandle->GetNumChildren(NumChildren);

			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
			}
		}
	}
}

ERigElementType FRigElementKeyDetails::GetElementType() const
{
	ERigElementType ElementType = ERigElementType::None;
	if (TypeHandle.IsValid())
	{
		uint8 Index = 0;
		TypeHandle->GetValue(Index);
		ElementType = (ERigElementType)Index;
	}
	return ElementType;
}

FString FRigElementKeyDetails::GetElementName() const
{
	FString ElementNameStr;
	if (NameHandle.IsValid())
	{
		for(int32 ObjectIndex = 0; ObjectIndex < NameHandle->GetNumPerObjectValues(); ObjectIndex++)
		{
			FString PerObjectValue;
			NameHandle->GetPerObjectValue(ObjectIndex, PerObjectValue);

			if(ObjectIndex == 0)
			{
				ElementNameStr = PerObjectValue;
			}
			else if(ElementNameStr != PerObjectValue)
			{
				return ControlRigDetailsMultipleValues.ToString();
			}
		}
	}
	return ElementNameStr;
}

void FRigElementKeyDetails::SetElementName(FString InName)
{
	if (NameHandle.IsValid())
	{
		NameHandle->SetValue(InName);

		// if this is nested below a connection rule
		const TSharedPtr<IPropertyHandle> KeyHandle = NameHandle->GetParentHandle();
		if(KeyHandle.IsValid())
		{
			const TSharedPtr<IPropertyHandle> ParentHandle = KeyHandle->GetParentHandle();
			if(ParentHandle.IsValid())
			{
				if (const TSharedPtr<IPropertyHandleStruct> StructPropertyHandle = ParentHandle->AsStruct())
				{
					if(const UScriptStruct* RuleStruct = Cast<UScriptStruct>(StructPropertyHandle->GetStructData()->GetStruct()))
					{
						if (RuleStruct->IsChildOf(FRigConnectionRule::StaticStruct()))
						{
							const void* RuleMemory = StructPropertyHandle->GetStructData()->GetStructMemory();
							FString RuleContent;
							RuleStruct->ExportText(RuleContent, RuleMemory, RuleMemory, nullptr, PPF_None, nullptr);

							const TSharedPtr<IPropertyHandle> RuleStashHandle = ParentHandle->GetParentHandle();
							
							FRigConnectionRuleStash Stash;
							Stash.ScriptStructPath = RuleStruct->GetPathName();
							Stash.ExportedText = RuleContent;
							
							FString StashContent;
							FRigConnectionRuleStash::StaticStruct()->ExportText(StashContent, &Stash, &Stash, nullptr, PPF_None, nullptr);

							TArray<UObject*> Objects;
							RuleStashHandle->GetOuterObjects(Objects);
							FString FirstObjectValue;
							for (int32 Index = 0; Index < Objects.Num(); Index++)
							{
								(void)RuleStashHandle->SetPerObjectValue(Index, StashContent, EPropertyValueSetFlags::DefaultFlags);
							}
						}
					}
				}
			}
		}
	}
}

void FRigElementKeyDetails::OnElementNameChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo)
{
	if (InItem.IsValid())
	{
		SetElementName(*InItem);
	}
	else
	{
		SetElementName(FString());
	}
}

FText FRigElementKeyDetails::GetElementNameAsText() const
{
	return FText::FromString(GetElementName());
}

FSlateColor FRigElementKeyDetails::OnGetWidgetForeground(const TSharedPtr<SButton> Button)
{
	float Alpha = (Button.IsValid() && Button->IsHovered()) ? FRigElementKeyDetailsDefs::ActivePinForegroundAlpha : FRigElementKeyDetailsDefs::InactivePinForegroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor FRigElementKeyDetails::OnGetWidgetBackground(const TSharedPtr<SButton> Button)
{
	float Alpha = (Button.IsValid() && Button->IsHovered()) ? FRigElementKeyDetailsDefs::ActivePinBackgroundAlpha : FRigElementKeyDetailsDefs::InactivePinBackgroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FReply FRigElementKeyDetails::OnGetSelectedClicked()
{
	if (BlueprintBeingCustomized)
	{
		const TArray<FRigElementKey>& Selected = BlueprintBeingCustomized->GetHierarchy()->GetSelectedKeys();
		if (Selected.Num() > 0)
		{
			if (TypeHandle.IsValid())
			{
				uint8 Index = (uint8) Selected[0].Type;
				TypeHandle->SetValue(Index);
			}
			SetElementName(Selected[0].Name.ToString());
		}
	}
	return FReply::Handled();
}

FReply FRigElementKeyDetails::OnSelectInHierarchyClicked()
{
	if (BlueprintBeingCustomized)
	{
		FRigElementKey Key;
		if (TypeHandle.IsValid())
		{
			uint8 Type;
			TypeHandle->GetValue(Type);
			Key.Type = (ERigElementType) Type;
		}

		if (NameHandle.IsValid())
		{
			NameHandle->GetValue(Key.Name);
		}
				
		if (Key.IsValid())
		{
			BlueprintBeingCustomized->GetHierarchyController()->SetSelection({Key});
		}	
	}
	return FReply::Handled();
}

void FRigComponentKeyDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	BlueprintBeingCustomized = nullptr;
	RigElementKeyDetails_GetCustomizedInfo(InStructPropertyHandle, BlueprintBeingCustomized);

	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	];

	if (BlueprintBeingCustomized == nullptr)
	{
		HeaderRow
		.ValueContent()
		[
			InStructPropertyHandle->CreatePropertyValueWidget(false)
		];
	}
}

void FRigComponentKeyDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		BlueprintBeingCustomized = nullptr;
		RigElementKeyDetails_GetCustomizedInfo(InStructPropertyHandle, BlueprintBeingCustomized);

		ElementKeyHandle = InStructPropertyHandle->GetChildHandle(TEXT("ElementKey"));
		NameHandle = InStructPropertyHandle->GetChildHandle(TEXT("Name"));

		// only fill the children if the blueprint cannot be found
		if (BlueprintBeingCustomized == nullptr || !ElementKeyHandle.IsValid() || !NameHandle.IsValid())
		{
			uint32 NumChildren = 0;
			InStructPropertyHandle->GetNumChildren(NumChildren);
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
			{
				StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
			}
		}
		else
		{
			TWeakPtr<FRigComponentKeyDetails> WeakThisPtr = StaticCastWeakPtr<FRigComponentKeyDetails>(AsWeak());
			ElementKeyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda([WeakThisPtr]()
			{
				if(const TSharedPtr<FRigComponentKeyDetails> StrongThisPtr = WeakThisPtr.Pin())
				{
					if(StrongThisPtr.IsValid())
					{
						StrongThisPtr->UpdateComponentNameList();
					}
				}
			}));
			
			StructBuilder
			.AddProperty(ElementKeyHandle.ToSharedRef());

			StructBuilder
			.AddProperty(NameHandle.ToSharedRef())
			.CustomWidget()
			.NameContent()
			[
				NameHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SAssignNew(SearchableComboBox, SSearchableComboBox)
					.OptionsSource(&ComponentNameList)
					.OnSelectionChanged(this, &FRigComponentKeyDetails::OnComponentNameChanged)
					.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
					{
						return SNew(STextBlock)
						.Text(FText::FromString(InItem.IsValid() ? *InItem : FString()))
						.Font(IDetailLayoutBuilder::GetDetailFont());
					})
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &FRigComponentKeyDetails::GetComponentNameAsText)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			];
		}
	}

	UpdateComponentNameList();
}

FRigElementKey FRigComponentKeyDetails::GetElementKey() const
{
	FRigElementKey ElementKey;
	if (ElementKeyHandle.IsValid())
	{
		void* ElementKeyData = nullptr;
		if(ElementKeyHandle->GetValueData(ElementKeyData) == FPropertyAccess::Success)
		{
			ElementKey = *static_cast<const FRigElementKey*>(ElementKeyData);
		}
	}
	return ElementKey;
}

FString FRigComponentKeyDetails::GetComponentName() const
{
	FString ElementNameStr;
	if (NameHandle.IsValid())
	{
		for(int32 ObjectIndex = 0; ObjectIndex < NameHandle->GetNumPerObjectValues(); ObjectIndex++)
		{
			FString PerObjectValue;
			NameHandle->GetPerObjectValue(ObjectIndex, PerObjectValue);

			if(ObjectIndex == 0)
			{
				ElementNameStr = PerObjectValue;
			}
			else if(ElementNameStr != PerObjectValue)
			{
				return ControlRigDetailsMultipleValues.ToString();
			}
		}
	}
	return ElementNameStr;
}

void FRigComponentKeyDetails::SetComponentName(FString InName)
{
	if (NameHandle.IsValid())
	{
		NameHandle->SetValue(InName);
	}
}

void FRigComponentKeyDetails::OnComponentNameChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo)
{
	if (InItem.IsValid())
	{
		SetComponentName(*InItem);
	}
	else
	{
		SetComponentName(FString());
	}
}

FText FRigComponentKeyDetails::GetComponentNameAsText() const
{
	return FText::FromString(GetComponentName());
}

void FRigComponentKeyDetails::UpdateComponentNameList()
{
	if(BlueprintBeingCustomized == nullptr)
	{
		return;
	}
	
	const FRigElementKey ElementKey = GetElementKey();
	if(!ElementKey.IsValid())
	{
		return;
	}

	URigHierarchy* Hierarchy = BlueprintBeingCustomized->GetHierarchy();
	if(UControlRig* RigBeingDebugged = Cast<UControlRig>(BlueprintBeingCustomized->GetObjectBeingDebugged()))
	{
		Hierarchy = RigBeingDebugged->GetHierarchy();
	}

	TArray<FRigComponentKey> ComponentKeys = Hierarchy->GetComponentKeys(ElementKey);
	ComponentKeys.Sort();
	
	ComponentNameList.Reset();
	ComponentNameList.Emplace(new FString(FName(NAME_None).ToString()));
	for(const FRigComponentKey& ComponentKey : ComponentKeys)
	{
		ComponentNameList.Emplace(new FString(ComponentKey.Name.ToString()));
	}

	if(SearchableComboBox.IsValid())
	{
		SearchableComboBox->RefreshOptions();
	}
}

void FRigComputedTransformDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	BlueprintBeingCustomized = nullptr;
	RigElementKeyDetails_GetCustomizedInfo(InStructPropertyHandle, BlueprintBeingCustomized);
}

void FRigComputedTransformDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TransformHandle = InStructPropertyHandle->GetChildHandle(TEXT("Transform"));

	StructBuilder
	.AddProperty(TransformHandle.ToSharedRef())
	.DisplayName(InStructPropertyHandle->GetPropertyDisplayName());

    FString PropertyPath = TransformHandle->GeneratePathToProperty();

	if(PropertyPath.StartsWith(TEXT("Struct.")))
	{
		PropertyPath.RightChopInline(7);
	}

	if(PropertyPath.StartsWith(TEXT("Pose.")))
	{
		PropertyPath.RightChopInline(5);
		PropertyChain.AddTail(FRigTransformElement::StaticStruct()->FindPropertyByName(TEXT("Pose")));
	}
	else if(PropertyPath.StartsWith(TEXT("Offset.")))
	{
		PropertyPath.RightChopInline(7);
		PropertyChain.AddTail(FRigControlElement::StaticStruct()->FindPropertyByName(TEXT("Offset")));
	}
	else if(PropertyPath.StartsWith(TEXT("Shape.")))
	{
		PropertyPath.RightChopInline(6);
		PropertyChain.AddTail(FRigControlElement::StaticStruct()->FindPropertyByName(TEXT("Shape")));
	}

	if(PropertyPath.StartsWith(TEXT("Current.")))
	{
		PropertyPath.RightChopInline(8);
		PropertyChain.AddTail(FRigCurrentAndInitialTransform::StaticStruct()->FindPropertyByName(TEXT("Current")));
	}
	else if(PropertyPath.StartsWith(TEXT("Initial.")))
	{
		PropertyPath.RightChopInline(8);
		PropertyChain.AddTail(FRigCurrentAndInitialTransform::StaticStruct()->FindPropertyByName(TEXT("Initial")));
	}

	if(PropertyPath.StartsWith(TEXT("Local.")))
	{
		PropertyPath.RightChopInline(6);
		PropertyChain.AddTail(FRigLocalAndGlobalTransform::StaticStruct()->FindPropertyByName(TEXT("Local")));
	}
	else if(PropertyPath.StartsWith(TEXT("Global.")))
	{
		PropertyPath.RightChopInline(7);
		PropertyChain.AddTail(FRigLocalAndGlobalTransform::StaticStruct()->FindPropertyByName(TEXT("Global")));
	}

	PropertyChain.AddTail(TransformHandle->GetProperty());
	PropertyChain.SetActiveMemberPropertyNode(PropertyChain.GetTail()->GetValue());

	const FSimpleDelegate OnTransformChangedDelegate = FSimpleDelegate::CreateSP(this, &FRigComputedTransformDetails::OnTransformChanged, &PropertyChain);
	TransformHandle->SetOnPropertyValueChanged(OnTransformChangedDelegate);
	TransformHandle->SetOnChildPropertyValueChanged(OnTransformChangedDelegate);
}

void FRigComputedTransformDetails::OnTransformChanged(FEditPropertyChain* InPropertyChain)
{
	if(BlueprintBeingCustomized && InPropertyChain)
	{
		if(InPropertyChain->Num() > 1)
		{
			FPropertyChangedEvent ChangeEvent(InPropertyChain->GetHead()->GetValue(), EPropertyChangeType::ValueSet);
			ChangeEvent.SetActiveMemberProperty(InPropertyChain->GetTail()->GetValue());
			FPropertyChangedChainEvent ChainEvent(*InPropertyChain, ChangeEvent);
			BlueprintBeingCustomized->GetRigVMAssetInterface()->BroadcastPostEditChangeChainProperty(ChainEvent);
		}
	}
}

void FRigControlTransformChannelDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	Handle = InStructPropertyHandle;

	TArray<int32> VisibleEnumValues;
	const TArray<ERigControlTransformChannel>* VisibleChannels = nullptr;

	// loop for controls to figure out the control type
	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		if (const URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(Object))
		{
			if(WrapperObject->GetWrappedStruct() == FRigControlElement::StaticStruct())
			{
				const FRigControlElement ControlElement = WrapperObject->GetContent<FRigControlElement>();
				VisibleChannels = GetVisibleChannelsForControlType(ControlElement.Settings.ControlType);
				break;
			}
			if (const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(WrapperObject->GetOuter()))
			{
				if(UnitNode->GetScriptStruct() && UnitNode->GetScriptStruct()->IsChildOf(FRigUnit_HierarchyAddControlElement::StaticStruct()))
				{
					FStructOnScope StructOnScope(UnitNode->GetScriptStruct());
					WrapperObject->GetContent(StructOnScope.GetStructMemory(), StructOnScope.GetStruct());

					const FRigUnit_HierarchyAddControlElement* RigUnit = (const FRigUnit_HierarchyAddControlElement*)StructOnScope.GetStructMemory();
					VisibleChannels = GetVisibleChannelsForControlType(RigUnit->GetControlTypeToSpawn());
					break;
				}
			}
		}
	}

	if(VisibleChannels)
	{
		VisibleEnumValues.Reserve(VisibleChannels->Num());
		for(int32 Index=0; Index < VisibleChannels->Num(); Index++)
		{
			VisibleEnumValues.Add((int32)(*VisibleChannels)[Index]);
		}
	}
	
	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SEnumComboBox, StaticEnum<ERigControlTransformChannel>())
		.CurrentValue_Raw(this, &FRigControlTransformChannelDetails::GetChannelAsInt32)
		.OnEnumSelectionChanged_Raw(this, &FRigControlTransformChannelDetails::OnChannelChanged)
		.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
		.EnumValueSubset(VisibleEnumValues)
	];
}

void FRigControlTransformChannelDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle,
	IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// nothing to do here
}

ERigControlTransformChannel FRigControlTransformChannelDetails::GetChannel() const
{
	uint8 Value = 0;
	Handle->GetValue(Value);
	return (ERigControlTransformChannel)Value;
}

void FRigControlTransformChannelDetails::OnChannelChanged(int32 NewSelection, ESelectInfo::Type InSelectionInfo)
{
	Handle->SetValue((uint8)NewSelection);
}

const TArray<ERigControlTransformChannel>* FRigControlTransformChannelDetails::GetVisibleChannelsForControlType(ERigControlType InControlType)
{
	switch(InControlType)
	{
		case ERigControlType::Position:
		{
			static const TArray<ERigControlTransformChannel> PositionChannels = {
				ERigControlTransformChannel::TranslationX,
				ERigControlTransformChannel::TranslationY,
				ERigControlTransformChannel::TranslationZ
			};
			return &PositionChannels;
		}
		case ERigControlType::Rotator:
		{
			static const TArray<ERigControlTransformChannel> RotatorChannels = {
				ERigControlTransformChannel::Pitch,
				ERigControlTransformChannel::Yaw,
				ERigControlTransformChannel::Roll
			};
			return &RotatorChannels;
		}
		case ERigControlType::Scale:
		{
			static const TArray<ERigControlTransformChannel> ScaleChannels = {
				ERigControlTransformChannel::ScaleX,
				ERigControlTransformChannel::ScaleY,
				ERigControlTransformChannel::ScaleZ
			};
			return &ScaleChannels;
		}
		case ERigControlType::Vector2D:
		{
			static const TArray<ERigControlTransformChannel> Vector2DChannels = {
				ERigControlTransformChannel::TranslationX,
				ERigControlTransformChannel::TranslationY
			};
			return &Vector2DChannels;
		}
		case ERigControlType::EulerTransform:
		{
			static const TArray<ERigControlTransformChannel> EulerTransformChannels = {
				ERigControlTransformChannel::TranslationX,
				ERigControlTransformChannel::TranslationY,
				ERigControlTransformChannel::TranslationZ,
				ERigControlTransformChannel::Pitch,
				ERigControlTransformChannel::Yaw,
				ERigControlTransformChannel::Roll,
				ERigControlTransformChannel::ScaleX,
				ERigControlTransformChannel::ScaleY,
				ERigControlTransformChannel::ScaleZ
			};
			return &EulerTransformChannels;
		}
		default:
		{
			break;
		}
	}
	return nullptr;
}

void FRigBaseElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PerElementInfos.Reset();

	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);
	for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		URigVMDetailsViewWrapperObject* WrapperObject = CastChecked<URigVMDetailsViewWrapperObject>(DetailObject.Get());
		if (!WrapperObject->IsActive())
		{
			continue;
		}

		const FRigElementKey Key = WrapperObject->GetContent<FRigBaseElement>().GetKey();

		FPerElementInfo Info;
		Info.WrapperObject = WrapperObject;
		if (const URigHierarchy* Hierarchy = Cast<URigHierarchy>(WrapperObject->GetSubject()))
		{
			Info.Element = Hierarchy->GetHandle(Key);
		}

		if(!Info.Element.IsValid())
		{
			return;
		}
		if(const FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
		{
			Info.DefaultElement = Blueprint->GetHierarchy()->GetHandle(Key);
		}

		PerElementInfos.Add(Info);
	}

	IDetailCategoryBuilder& GeneralCategory = DetailBuilder.EditCategory(TEXT("General"), LOCTEXT("General", "General"));

	const bool bIsProcedural = IsAnyElementProcedural();
	if(bIsProcedural)
	{
		GeneralCategory.AddCustomRow(LOCTEXT("ProceduralElement", "ProceduralElement")).WholeRowContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ProceduralElementNote", "This item has been created procedurally."))
			.ToolTipText(LOCTEXT("ProceduralElementTooltip", "You cannot edit the values of the item here.\nPlease change the settings on the node\nthat created the item."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(FLinearColor::Red)
		];
	}

	const bool bAllControls = !IsAnyElementNotOfType(ERigElementType::Control);
	const bool bAllAnimationChannels = bAllControls && !IsAnyControlNotOfAnimationType(ERigControlAnimationType::AnimationChannel);
	if(bAllControls && bAllAnimationChannels)
	{
		GeneralCategory.AddCustomRow(FText::FromString(TEXT("Parent Control")))
		.NameContent()
		[
			SNew(SInlineEditableTextBlock)
			.Text(FText::FromString(TEXT("Parent Control")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(false)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FRigBaseElementDetails::GetParentElementName)
				.IsEnabled(false)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SAssignNew(SelectParentElementButton, SButton)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity_Lambda([this]() { return FRigElementKeyDetails::OnGetWidgetBackground(SelectParentElementButton); })
				.OnClicked(this, &FRigBaseElementDetails::OnSelectParentElementInHierarchyClicked)
				.ContentPadding(0)
				.ToolTipText(NSLOCTEXT("ControlRigElementDetails", "SelectParentInHierarchyToolTip", "Select Parent in hierarchy"))
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda( [this]() { return FRigElementKeyDetails::OnGetWidgetForeground(SelectParentElementButton); })
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
		];
	}

	DetailBuilder.HideCategory(TEXT("RigElement"));

	if(!bAllControls || !bAllAnimationChannels)
	{
		GeneralCategory.AddCustomRow(FText::FromString(TEXT("Name")))
		.IsEnabled(!bIsProcedural)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Name")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SInlineEditableTextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FRigBaseElementDetails::GetName)
			.OnTextCommitted(this, &FRigBaseElementDetails::SetName)
			.OnVerifyTextChanged(this, &FRigBaseElementDetails::OnVerifyNameChanged)
			.IsEnabled(!bIsProcedural && PerElementInfos.Num() == 1 && !bAllAnimationChannels)
		];
	}

	GeneralCategory.AddCustomRow(FText::FromString(TEXT("Spawn Index")), true /* advanced */)
	.IsEnabled(!bIsProcedural)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromString(TEXT("Spawn Index")))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(STextBlock)
		.Text(this, &FRigBaseElementDetails::GetSpawnIndex)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];

	// if we are not a bone, control or null
	if(!IsAnyElementOfType(ERigElementType::Bone) &&
		!IsAnyElementOfType(ERigElementType::Control) &&
		!IsAnyElementOfType(ERigElementType::Null) &&
		!IsAnyElementOfType(ERigElementType::Connector) &&
		!IsAnyElementOfType(ERigElementType::Socket))
	{
		CustomizeComponents(DetailBuilder);
		CustomizeMetadata(DetailBuilder);
	}
}

void FRigBaseElementDetails::PendingDelete()
{
	if (MetadataHandle.IsValid())
	{
		for (const FPerElementInfo& Info : PerElementInfos)
		{
			// We do not check Info.IsValid here, because even if the element
			// doesn't exist anymore in the hierarchy, we still want to detach the
			// metadata handle from the hierarchy

			
			if (URigHierarchy* Hierarchy = Info.GetHierarchy())
			{
				if (Hierarchy->OnMetadataChanged().Remove(MetadataHandle))
				{
					break;
				}
			}
		}
		MetadataHandle.Reset();
	}
	
	IDetailCustomization::PendingDelete();
}

FRigElementKey FRigBaseElementDetails::GetElementKey() const
{
	check(PerElementInfos.Num() == 1);
	if (FRigBaseElement* Element = PerElementInfos[0].GetElement())
	{
		return Element->GetKey();
	}
	return FRigElementKey();
}

FText FRigBaseElementDetails::GetName() const
{
	if (PerElementInfos.IsEmpty())
	{
		return ControlRigDetailsInvalid;
	}
	if(PerElementInfos.Num() > 1)
	{
		return ControlRigDetailsMultipleValues;
	}
	return FText::FromName(GetElementKey().Name);
}

FText FRigBaseElementDetails::GetParentElementName() const
{
	if (PerElementInfos.IsEmpty())
	{
		return ControlRigDetailsInvalid;
	}
	if(PerElementInfos.Num() > 1)
	{
		return ControlRigDetailsMultipleValues;
	}
	return FText::FromName(PerElementInfos[0].GetHierarchy()->GetFirstParent(GetElementKey()).Name);
}

void FRigBaseElementDetails::SetName(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if(InCommitType == ETextCommit::OnCleared)
	{
		return;
	}
	
	if(PerElementInfos.Num() > 1)
	{
		return;
	}
	
	if (PerElementInfos.IsEmpty())
	{
		return;
	}

	if(PerElementInfos[0].IsProcedural())
	{
		return;
	}
	
	if (URigHierarchy* Hierarchy = PerElementInfos[0].GetDefaultHierarchy())
	{
		BeginDestroy();
		
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);
		const FRigElementKey NewKey = Controller->RenameElement(GetElementKey(), *InNewText.ToString(), true, true);
		if(NewKey.IsValid())
		{
			Controller->SelectElement(NewKey, true, true);
		}
	}
}

bool FRigBaseElementDetails::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	if(PerElementInfos.Num() > 1)
	{
		return false;
	}
	
	if (PerElementInfos.IsEmpty())
	{
		return false;
	}

	if(PerElementInfos[0].IsProcedural())
	{
		return false;
	}

	const URigHierarchy* Hierarchy = PerElementInfos[0].GetDefaultHierarchy();
	if (Hierarchy == nullptr)
	{
		return false;
	}

	if (GetElementKey().Name.ToString() == InText.ToString())
	{
		return true;
	}

	FString OutErrorMessageStr;
	if (!Hierarchy->IsNameAvailable(FRigName(InText.ToString()), GetElementKey().Type, &OutErrorMessageStr))
	{
		OutErrorMessage = FText::FromString(OutErrorMessageStr);
		return false;
	}

	return true;
}

FText FRigBaseElementDetails::GetSpawnIndex() const
{
	if (PerElementInfos.IsEmpty())
	{
		return ControlRigDetailsInvalid;
	}
	if(PerElementInfos.Num() > 1)
	{
		return ControlRigDetailsMultipleValues;
	}
	if (PerElementInfos.Num() == 1)
	{
		if (const FRigBaseElement* Element = PerElementInfos[0].GetElement<FRigBaseElement>())
		{
			return FText::FromString(FString::FromInt(Element->GetSpawnIndex()));
		}
	}

	return LOCTEXT("NoElementInfos", "No Element");
}

void FRigBaseElementDetails::OnStructContentsChanged(FProperty* InProperty, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	const FPropertyChangedEvent ChangeEvent(InProperty, EPropertyChangeType::ValueSet);
	PropertyUtilities->NotifyFinishedChangingProperties(ChangeEvent);
}

bool FRigBaseElementDetails::IsConstructionModeEnabled() const
{
	if(PerElementInfos.IsEmpty())
	{
		return false;
	}
	
	if(const FControlRigAssetInterfacePtr Blueprint = PerElementInfos[0].GetAsset())
	{
		if (const UControlRig* DebuggedRig = Cast<UControlRig>(Blueprint ->GetObjectBeingDebugged()))
		{
			return DebuggedRig->IsConstructionModeEnabled();
		}
	}
	return false;
}

TArray<FRigElementKey> FRigBaseElementDetails::GetElementKeys() const
{
	TArray<FRigElementKey> Keys;
	Algo::Transform(PerElementInfos, Keys, [](const FPerElementInfo& Info)
	{
		if (const FRigBaseElement* Element = Info.GetElement())
		{
			return Element->GetKey();
		}
		return FRigElementKey();
	});
	return Keys;
}

const FRigBaseElementDetails::FPerElementInfo& FRigBaseElementDetails::FindElement(const FRigElementKey& InKey) const
{
	const FPerElementInfo* Info = FindElementByPredicate([InKey](const FPerElementInfo& Info)
	{
		if (const FRigBaseElement* Element = Info.GetElement())
		{
			return Element->GetKey() == InKey;
		}
		return false;
	});

	if(Info)
	{
		return *Info;
	}

	static const FPerElementInfo EmptyInfo;
	return EmptyInfo;
}

bool FRigBaseElementDetails::IsAnyElementOfType(ERigElementType InType) const
{
	return ContainsElementByPredicate([InType](const FPerElementInfo& Info)
	{
		if (const FRigBaseElement* Element = Info.GetElement())
		{
			return Element->GetType() == InType;
		}
		return false;
	});
}

bool FRigBaseElementDetails::IsAnyElementNotOfType(ERigElementType InType) const
{
	return ContainsElementByPredicate([InType](const FPerElementInfo& Info)
	{
		if (const FRigBaseElement* Element = Info.GetElement())
		{
			return Element->GetType() != InType;
		}
		return false;
	});
}

bool FRigBaseElementDetails::IsAnyControlOfAnimationType(ERigControlAnimationType InType) const
{
	return ContainsElementByPredicate([InType](const FPerElementInfo& Info)
	{
		if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
		{
			return ControlElement->Settings.AnimationType == InType;
		}
		return false;
	});
}

bool FRigBaseElementDetails::IsAnyControlNotOfAnimationType(ERigControlAnimationType InType) const
{
	return ContainsElementByPredicate([InType](const FPerElementInfo& Info)
	{
		if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
		{
			return ControlElement->Settings.AnimationType != InType;
		}
		return false;
	});
}

bool FRigBaseElementDetails::IsAnyControlOfValueType(ERigControlType InType) const
{
	return ContainsElementByPredicate([InType](const FPerElementInfo& Info)
	{
		if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
		{
			return ControlElement->Settings.ControlType == InType;
		}
		return false;
	});
}

bool FRigBaseElementDetails::IsAnyControlNotOfValueType(ERigControlType InType) const
{
	return ContainsElementByPredicate([InType](const FPerElementInfo& Info)
	{
		if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
		{
			return ControlElement->Settings.ControlType != InType;
		}
		return false;
	});
}

bool FRigBaseElementDetails::IsAnyElementProcedural() const
{
	return ContainsElementByPredicate([](const FPerElementInfo& Info)
	{
		return Info.IsProcedural();
	});
}

bool FRigBaseElementDetails::IsAnyConnectorImported() const
{
	return ContainsElementByPredicate([](const FPerElementInfo& Info)
	{
		return Info.Element.GetKey().Name.ToString().Contains(FRigHierarchyModulePath::ModuleNameSuffix);
	});
}

bool FRigBaseElementDetails::IsAnyConnectorPrimary() const
{
	return ContainsElementByPredicate([](const FPerElementInfo& Info)
	{
		if(const FRigConnectorElement* Connector = Info.Element.Get<FRigConnectorElement>())
		{
			return Connector->IsPrimary();
		}
		return false;
	});
}

bool FRigBaseElementDetails::GetCommonElementType(ERigElementType& OutElementType) const
{
	OutElementType = ERigElementType::None;

	for(const FPerElementInfo& Info : PerElementInfos)
	{
		const FRigElementKey& Key = Info.Element.GetKey();
		if(Key.IsValid())
		{
			if(OutElementType == ERigElementType::None)
			{
				OutElementType = Key.Type;
			}
			else if(OutElementType != Key.Type)
			{
				OutElementType = ERigElementType::None;
				break;
			}
		}
	}

	return OutElementType != ERigElementType::None;
}

bool FRigBaseElementDetails::GetCommonControlType(ERigControlType& OutControlType) const
{
	OutControlType = ERigControlType::Bool;
	
	ERigElementType ElementType = ERigElementType::None;
	if(GetCommonElementType(ElementType))
	{
		if(ElementType == ERigElementType::Control)
		{
			bool bSuccess = false;
			for(const FPerElementInfo& Info : PerElementInfos)
			{
				if(const FRigControlElement* ControlElement = Info.Element.Get<FRigControlElement>())
				{
					if(!bSuccess)
					{
						OutControlType = ControlElement->Settings.ControlType;
						bSuccess = true;
					}
					else if(OutControlType != ControlElement->Settings.ControlType)
					{
						OutControlType = ERigControlType::Bool;
						bSuccess = false;
						break;
					}
				}
			}
			return bSuccess;
		}
	}
	return false;
}

bool FRigBaseElementDetails::GetCommonAnimationType(ERigControlAnimationType& OutAnimationType) const
{
	OutAnimationType = ERigControlAnimationType::AnimationControl;
	
	ERigElementType ElementType = ERigElementType::None;
	if(GetCommonElementType(ElementType))
	{
		if(ElementType == ERigElementType::Control)
		{
			bool bSuccess = false;
			for(const FPerElementInfo& Info : PerElementInfos)
			{
				if(const FRigControlElement* ControlElement = Info.Element.Get<FRigControlElement>())
				{
					if(!bSuccess)
					{
						OutAnimationType = ControlElement->Settings.AnimationType;
						bSuccess = true;
					}
					else if(OutAnimationType != ControlElement->Settings.AnimationType)
					{
						OutAnimationType = ERigControlAnimationType::AnimationControl;
						bSuccess = false;
						break;
					}
				}
			}
			return bSuccess;
		}
	}
	return false;
}

const FRigBaseElementDetails::FPerElementInfo* FRigBaseElementDetails::FindElementByPredicate(const TFunction<bool(const FPerElementInfo&)>& InPredicate) const
{
	return PerElementInfos.FindByPredicate(InPredicate);
}

bool FRigBaseElementDetails::ContainsElementByPredicate(const TFunction<bool(const FPerElementInfo&)>& InPredicate) const
{
	return PerElementInfos.ContainsByPredicate(InPredicate);
}

void FRigBaseElementDetails::RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule)
{
	const URigVMDetailsViewWrapperObject* CDOWrapper = CastChecked<URigVMDetailsViewWrapperObject>(UControlRigWrapperObject::StaticClass()->GetDefaultObject());
	FRigBoneElementDetails().RegisterSectionMappings(PropertyEditorModule, CDOWrapper->GetClassForStruct(FRigBoneElement::StaticStruct()));
	FRigNullElementDetails().RegisterSectionMappings(PropertyEditorModule, CDOWrapper->GetClassForStruct(FRigNullElement::StaticStruct()));
	FRigControlElementDetails().RegisterSectionMappings(PropertyEditorModule, CDOWrapper->GetClassForStruct(FRigControlElement::StaticStruct()));
}

void FRigBaseElementDetails::RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass)
{
	TSharedRef<FPropertySection> MetadataSection = PropertyEditorModule.FindOrCreateSection(InClass->GetFName(), "Metadata", LOCTEXT("Metadata", "Metadata"));
	MetadataSection->AddCategory("Metadata");
}

FReply FRigBaseElementDetails::OnSelectParentElementInHierarchyClicked()
{
	if (PerElementInfos.Num() == 1)
	{
		FRigElementKey Key = GetElementKey();
		if (Key.IsValid())
		{
			const FRigElementKey ParentKey = PerElementInfos[0].GetHierarchy()->GetFirstParent(GetElementKey());
			if(ParentKey.IsValid())
			{
				return OnSelectElementClicked(ParentKey);
			}
		}	
	}
	return FReply::Handled();
}

FReply FRigBaseElementDetails::OnSelectElementClicked(const FRigElementKey& InKey)
{
	if (PerElementInfos.Num() == 1)
	{
		if (InKey.IsValid())
		{
			 PerElementInfos[0].GetHierarchy()->GetController(true)->SetSelection({InKey});
		}	
	}
	return FReply::Handled();
}

class FRigComponentStructProvider : public IStructureDataProvider
{
public:
	FRigComponentStructProvider(URigHierarchy* InHierarchy)
		: HierarchyPtr(InHierarchy)
	{
	}
	
	virtual ~FRigComponentStructProvider() override {}

	int32 Num() const
	{
		return ComponentIndices.Num();
	}

	const FRigBaseComponent* GetComponent(int32 InIndex) const
	{
		if(URigHierarchy* Hierarchy = GetHierarchy())
		{
			return Hierarchy->GetComponent(ComponentIndices[InIndex]);
		}
		return nullptr;
	}

	URigHierarchy* GetHierarchy() const
	{
		if(HierarchyPtr.IsValid())
		{
			return HierarchyPtr.Get();
		}
		return nullptr;
	}

	void Reset()
	{
		HierarchyPtr.Reset();
		ComponentIndices.Reset();
	}

	void AddComponent(const FRigBaseComponent* InComponent)
	{
		ComponentIndices.AddUnique(InComponent->GetIndexInHierarchy());
	}
	
	virtual bool IsValid() const override
	{
		return GetHierarchy() && Num() != 0;
	}
	
	virtual const UStruct* GetBaseStructure() const override
	{
		if(Num() > 0)
		{
			if(const FRigBaseComponent* Component = GetComponent(0))
			{
				return Component->GetScriptStruct();
			}
		}
		return nullptr;
	}

	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override
	{
		for(int32 Index = 0; Index < Num(); Index++)
		{
			if(const FRigBaseComponent* Component = GetComponent(Index))
			{
				check(Component->GetScriptStruct() == ExpectedBaseStructure);
				uint8* Memory = reinterpret_cast<uint8*>(const_cast<FRigBaseComponent*>(Component));
				OutInstances.Add(MakeShareable(new FStructOnScope(ExpectedBaseStructure, Memory)));
			}
		}
	}

	virtual bool IsPropertyIndirection() const override
	{
		return false;
	}

protected:

	mutable TWeakObjectPtr<URigHierarchy> HierarchyPtr;
	TArray<int32> ComponentIndices;
};

void FRigBaseElementDetails::CustomizeComponents(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	if(Hierarchy == nullptr)
	{
		return;
	}

	if(const FRigBaseElement* Element = PerElementInfos[0].GetElement<>())
	{
		for(int32 ComponentIndex = 0; ComponentIndex < Element->NumComponents(); ComponentIndex++)
		{
			if(const FRigBaseComponent* Component = Element->GetComponent(ComponentIndex))
			{
				FSharedComponent SharedComponent;
				SharedComponent.Handle = FRigComponentHandle(Hierarchy, Component);
				SharedComponent.ScriptStruct = Component->GetScriptStruct();
				SharedComponent.bIsProcedural = Component->IsProcedural();
				SharedComponents.Add(SharedComponent);
			}
		}
	}
	
	for(int32 ElementIndex = 1; ElementIndex < PerElementInfos.Num(); ElementIndex++)
	{
		if(const FRigBaseElement* Element = PerElementInfos[ElementIndex].GetElement<>())
		{
			// remove any missing or type-mismatching component from the list to display
			SharedComponents.RemoveAll([Element, Hierarchy](const FSharedComponent& SharedComponent) -> bool {
				const FRigComponentKey ComponentKey(Element->GetKey(), SharedComponent.Handle.GetComponentName());
				if(const FRigBaseComponent* Component = Hierarchy->FindComponent(ComponentKey))
				{
					return Component->GetScriptStruct() != SharedComponent.ScriptStruct;
				}
				return true;
			});

			// update the procedural flag in case any component is procedural
			for(FSharedComponent& SharedComponent : SharedComponents)
			{
				if(SharedComponent.bIsProcedural)
				{
					continue;
				}
				if(const FRigBaseComponent* Component = Element->FindComponent(SharedComponent.Handle.GetComponentName()))
				{
					if(Component->IsProcedural())
					{
						SharedComponent.bIsProcedural = true;
					}
				}
			}
		}
	}

	if(SharedComponents.IsEmpty())
	{
		return;
	}

	for(FSharedComponent& SharedComponent : SharedComponents)
	{
		TSharedPtr<FRigComponentStructProvider> StructProvider = MakeShareable(new FRigComponentStructProvider(Hierarchy));
		for(int32 ElementIndex = 0; ElementIndex < PerElementInfos.Num(); ElementIndex++)
		{
			const FRigComponentKey ComponentKey(PerElementInfos[ElementIndex].Element.GetKey(), SharedComponent.Handle.GetComponentName());
			if(const FRigBaseComponent* Component = Hierarchy->FindComponent(ComponentKey))
			{
				StructProvider->AddComponent(Component);
			}
		}

		if(StructProvider->Num() == 0)
		{
			continue;
		}

		const FName ComponentName = SharedComponent.Handle.GetComponentName();
		const FText DisplayName = FText::Format(LOCTEXT("ComponentCategoryTitleFormat", "{0} Component"), FText::FromName(ComponentName));
		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(ComponentName, DisplayName);

		TArray<IDetailPropertyRow*> DetailPropertyRows;
		Category.AddAllExternalStructureProperties(StructProvider, EPropertyLocation::Default, &DetailPropertyRows);

		for(IDetailPropertyRow* DetailPropertyRow : DetailPropertyRows)
		{
			if(SharedComponent.bIsProcedural)
			{
				DetailPropertyRow->IsEnabled(false);
			}

			FSimpleDelegate OnThisOrChildPropertyChanged = FSimpleDelegate::CreateLambda([this, ComponentName]()
            {
            	for(int32 ElementIndex = 0; ElementIndex < PerElementInfos.Num(); ElementIndex++)
            	{
            		if(URigHierarchy* Hierarchy = PerElementInfos[ElementIndex].GetHierarchy())
            		{
            			const FRigComponentKey ComponentKey(PerElementInfos[ElementIndex].Element.GetKey(), ComponentName);
            			if(const FRigBaseComponent* Component = Hierarchy->FindComponent(ComponentKey))
            			{
            				if(URigHierarchy* DefaultHierarchy = PerElementInfos[ElementIndex].GetDefaultHierarchy())
            				{
            					if(URigHierarchyController* Controller = DefaultHierarchy->GetController())
            					{
            						FRigComponentState State = Component->GetState();
            						Controller->SetComponentState(ComponentKey, State, true);
            					}
            				}
            			}
            		}
            	}
            });
			DetailPropertyRow->GetPropertyHandle()->SetOnPropertyValueChanged(OnThisOrChildPropertyChanged);
			DetailPropertyRow->GetPropertyHandle()->SetOnChildPropertyValueChanged(OnThisOrChildPropertyChanged);
		}
	}
}

void FRigBaseElementDetails::CustomizeMetadata(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.Num() != 1)
	{
		return;
	}
	FPerElementInfo& Info = PerElementInfos[0];

	FRigBaseElement* Element = Info.Element.Get();
	URigHierarchy* Hierarchy = Info.GetHierarchy();

	if (!MetadataHandle.IsValid() &&
		Hierarchy)
	{
		if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
		{
			UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			if (IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(Blueprint.GetObject(), false))
			{
				if (IControlRigBaseEditor* Editor = FControlRigBaseEditor::GetFromAssetEditorInstance(EditorInstance))
				{
					if (Editor->GetReplayPlaybackMode() != EControlRigReplayPlaybackMode::Live)
					{
						Hierarchy->OnMetadataChanged().Remove(MetadataHandle);
						MetadataHandle.Reset();
						return;
					}
				}
			}
		}

		const FRigElementKey ElementKey = Element ? Element->GetKey() : FRigElementKey();
		const TWeakObjectPtr<URigHierarchy> WeakHierarchy = Hierarchy;

		TWeakPtr<IPropertyUtilities> WeakPropertyUtilities =  DetailBuilder.GetPropertyUtilities().ToWeakPtr();
		MetadataHandle = Hierarchy->OnMetadataChanged().AddLambda(
			[WeakPropertyUtilities, ElementKey, WeakHierarchy](const FRigElementKey& InKey, const FName&)
			{
				if (WeakPropertyUtilities.IsValid() &&
					WeakHierarchy.IsValid())
				{
					const FRigBaseElement* Element = WeakHierarchy->Find(ElementKey);
					if (InKey.Type == ERigElementType::All || (Element && Element->GetKey() == InKey))
					{
						if(const UControlRig* ControlRig = WeakHierarchy->GetTypedOuter<UControlRig>())
						{
							if(ControlRig->IsConstructionModeEnabled())
							{
								return;
							}
						}

						// run the refresh of the user interface on the next tick on the game thread 
						ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakPropertyUtilities]()
						{
							const TSharedPtr<IPropertyUtilities> PropertyUtilities = WeakPropertyUtilities.IsValid() ? WeakPropertyUtilities.Pin() : nullptr;
							if(PropertyUtilities.IsValid())
							{
								PropertyUtilities->ForceRefresh();
							}
						});
					}
				}
			});
		}

	if (!Element ||
		!Hierarchy)
	{
		return;
	}

	const TArray<FName> MetadataNames = Hierarchy->GetMetadataNames(Element->GetKey());
	if (MetadataNames.IsEmpty())
	{
		return;
	}

	IDetailCategoryBuilder& MetadataCategory = DetailBuilder.EditCategory(TEXT("Metadata"), LOCTEXT("Metadata", "Metadata"));
	for(FName MetadataName: MetadataNames)
	{
		FRigBaseMetadata* Metadata = Element->GetMetadata(MetadataName);
		
		TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(Metadata->GetMetadataStruct()));
		Metadata->GetMetadataStruct()->CopyScriptStruct(StructOnScope->GetStructMemory(), Metadata, 1);

		FAddPropertyParams Params;
		Params.CreateCategoryNodes(false);
		Params.ForceShowProperty();
		
		IDetailPropertyRow* Row = MetadataCategory.AddExternalStructureProperty(StructOnScope, TEXT("Value"), EPropertyLocation::Default, Params);
		if(Row)
		{
			(*Row)
			.DisplayName(FText::FromName(Metadata->GetName()))
			.IsEnabled(false);
		}
	}
}

TSharedPtr<TArray<ERigTransformElementDetailsTransform::Type>> FRigTransformElementDetails::PickedTransforms;

void FRigTransformElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FRigBaseElementDetails::CustomizeDetails(DetailBuilder);
}

void FRigTransformElementDetails::RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass)
{
	FRigBaseElementDetails::RegisterSectionMappings(PropertyEditorModule, InClass);
	
	TSharedRef<FPropertySection> TransformSection = PropertyEditorModule.FindOrCreateSection(InClass->GetFName(), "Transform", LOCTEXT("Transform", "Transform"));
	TransformSection->AddCategory("General");
	TransformSection->AddCategory("Value");
	TransformSection->AddCategory("Transform");
}

void FRigTransformElementDetails::CustomizeTransform(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}
	
	TArray<FRigElementKey> Keys = GetElementKeys();
	Keys = PerElementInfos[0].GetHierarchy()->SortKeys(Keys);

	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bAllControls = !IsAnyElementNotOfType(ERigElementType::Control) && !IsAnyControlOfValueType(ERigControlType::Bool);;
	const bool bAllAnimationChannels = !IsAnyControlNotOfAnimationType(ERigControlAnimationType::AnimationChannel);
	if(bAllControls && bAllAnimationChannels)
	{
		return;
	}

	bool bShowLimits = false;
	TArray<ERigTransformElementDetailsTransform::Type> TransformTypes;
	TArray<FText> ButtonLabels;
	TArray<FText> ButtonTooltips;

	if(bAllControls)
	{
		TransformTypes = {
			ERigTransformElementDetailsTransform::Initial,
			ERigTransformElementDetailsTransform::Current,
			ERigTransformElementDetailsTransform::Offset
		};
		ButtonLabels = {
			LOCTEXT("Initial", "Initial"),
			LOCTEXT("Current", "Current"),
			LOCTEXT("Offset", "Offset")
		};
		ButtonTooltips = {
			LOCTEXT("InitialTooltip", "Initial transform in the reference pose"),
			LOCTEXT("CurrentTooltip", "Current animation transform"),
			LOCTEXT("OffsetTooltip", "Offset transform under the control")
		};

		bShowLimits = !IsAnyControlNotOfValueType(ERigControlType::EulerTransform);

		if(bShowLimits)
		{
			TransformTypes.Append({
				ERigTransformElementDetailsTransform::Minimum,
				ERigTransformElementDetailsTransform::Maximum
			});
			ButtonLabels.Append({
				LOCTEXT("Min", "Min"),
				LOCTEXT("Max", "Max")
			});
			ButtonTooltips.Append({
				LOCTEXT("ValueMinimumTooltip", "The minimum limit(s) for the control"),
				LOCTEXT("ValueMaximumTooltip", "The maximum limit(s) for the control")
			});
		}
	}
	else
	{
		TransformTypes = {
			ERigTransformElementDetailsTransform::Initial,
			ERigTransformElementDetailsTransform::Current
		};
		ButtonLabels = {
			LOCTEXT("Initial", "Initial"),
			LOCTEXT("Current", "Current")
		};
		ButtonTooltips = {
			LOCTEXT("InitialTooltip", "Initial transform in the reference pose"),
			LOCTEXT("CurrentTooltip", "Current animation transform")
		};
	}

	TArray<bool> bTransformsEnabled;

	// determine if the transforms are enabled
	for(int32 Index = 0; Index < TransformTypes.Num(); Index++)
	{
		const ERigTransformElementDetailsTransform::Type CurrentTransformType = TransformTypes[Index];

		bool bIsTransformEnabled = true;

		if(bIsProcedural)
		{
			// procedural items only allow editing of the current transform
			bIsTransformEnabled = CurrentTransformType == ERigTransformElementDetailsTransform::Current; 
		}

		if(bIsTransformEnabled)
		{
			if (IsAnyElementOfType(ERigElementType::Control))
			{
				bIsTransformEnabled = IsAnyControlOfValueType(ERigControlType::EulerTransform) ||
					IsAnyControlOfValueType(ERigControlType::Transform) ||
					CurrentTransformType == ERigTransformElementDetailsTransform::Offset;

				if(!bIsTransformEnabled)
				{
					ButtonTooltips[Index] = FText::FromString(
						FString::Printf(TEXT("%s\n%s"),
							*ButtonTooltips[Index].ToString(), 
							TEXT("Only transform controls can be edited here. Refer to the 'Value' section instead.")));
				}
			}
			else if (IsAnyElementOfType(ERigElementType::Bone) && CurrentTransformType == ERigTransformElementDetailsTransform::Initial)
			{
				for(const FPerElementInfo& Info : PerElementInfos)
				{
					if(const FRigBoneElement* BoneElement = Info.GetElement<FRigBoneElement>())
					{
						bIsTransformEnabled = BoneElement->BoneType == ERigBoneType::User;

						if(!bIsTransformEnabled)
						{
							ButtonTooltips[Index] = FText::FromString(
								FString::Printf(TEXT("%s\n%s"),
									*ButtonTooltips[Index].ToString(), 
									TEXT("Imported Bones' initial transform cannot be edited.")));
						}
					}
				}			
			}
		}
		bTransformsEnabled.Add(bIsTransformEnabled);
	}

	if(!PickedTransforms.IsValid())
	{
		PickedTransforms = MakeShareable(new TArray<ERigTransformElementDetailsTransform::Type>({ERigTransformElementDetailsTransform::Current}));
	}

	TSharedPtr<SSegmentedControl<ERigTransformElementDetailsTransform::Type>> TransformChoiceWidget =
		SSegmentedControl<ERigTransformElementDetailsTransform::Type>::Create(
			TransformTypes,
			ButtonLabels,
			ButtonTooltips,
			*PickedTransforms.Get(),
			true,
			SSegmentedControl<ERigTransformElementDetailsTransform::Type>::FOnValuesChanged::CreateLambda(
				[](TArray<ERigTransformElementDetailsTransform::Type> NewSelection)
				{
					(*FRigTransformElementDetails::PickedTransforms.Get()) = NewSelection;
				}
			)
		);

	IDetailCategoryBuilder& TransformCategory = DetailBuilder.EditCategory(TEXT("Transform"), LOCTEXT("Transform", "Transform"));
	AddChoiceWidgetRow(TransformCategory, FText::FromString(TEXT("TransformType")), TransformChoiceWidget.ToSharedRef());

	SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FEulerTransform>::FArguments()
	.DisplayToggle(false)
	.DisplayRelativeWorld(true)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.PreventThrottling(true);

	for(int32 Index = 0; Index < ButtonLabels.Num(); Index++)
	{
		const ERigTransformElementDetailsTransform::Type CurrentTransformType = TransformTypes[Index];
		ERigControlValueType CurrentValueType = ERigControlValueType::Current;
		switch(CurrentTransformType)
		{
			case ERigTransformElementDetailsTransform::Initial:
			{
				CurrentValueType = ERigControlValueType::Initial;
				break;
			}
			case ERigTransformElementDetailsTransform::Minimum:
			{
				CurrentValueType = ERigControlValueType::Minimum;
				break;
			}
			case ERigTransformElementDetailsTransform::Maximum:
			{
				CurrentValueType = ERigControlValueType::Maximum;
				break;
			}
		}

		TransformWidgetArgs.Visibility_Lambda([TransformChoiceWidget, Index]() -> EVisibility
		{
			return TransformChoiceWidget->HasValue((ERigTransformElementDetailsTransform::Type)Index) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		TransformWidgetArgs.IsEnabled(bTransformsEnabled[Index]);

		CreateEulerTransformValueWidgetRow(
			Keys,
			TransformWidgetArgs,
			TransformCategory,
			ButtonLabels[Index],
			ButtonTooltips[Index],
			CurrentTransformType,
			CurrentValueType);
	}
}

bool FRigTransformElementDetails::IsCurrentLocalEnabled() const
{
	return IsAnyElementOfType(ERigElementType::Control);
}

void FRigTransformElementDetails::AddChoiceWidgetRow(IDetailCategoryBuilder& InCategory, const FText& InSearchText, TSharedRef<SWidget> InWidget)
{
	InCategory.AddCustomRow(FText::FromString(TEXT("TransformType")))
	.ValueContent()
	.MinDesiredWidth(375.f)
	.MaxDesiredWidth(375.f)
	.HAlign(HAlign_Left)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			InWidget
		]
	];
}

FDetailWidgetRow& FRigTransformElementDetails::CreateTransformComponentValueWidgetRow(
	ERigControlType InControlType,
	const TArray<FRigElementKey>& Keys,
	SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs,
	IDetailCategoryBuilder& CategoryBuilder,
	const FText& Label,
	const FText& Tooltip,
	ERigTransformElementDetailsTransform::Type CurrentTransformType,
	ERigControlValueType ValueType,
	TSharedPtr<SWidget> NameContent)
{
	TransformWidgetArgs
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.AllowEditRotationRepresentation(false);

	if(TransformWidgetArgs._DisplayRelativeWorld &&
		!TransformWidgetArgs._OnGetIsComponentRelative.IsBound() &&
		!TransformWidgetArgs._OnIsComponentRelativeChanged.IsBound())
	{
		TSharedRef<UE::Math::TVector<float>> IsComponentRelative = MakeShareable(new UE::Math::TVector<float>(1.f, 1.f, 1.f));
		
		TransformWidgetArgs
		.OnGetIsComponentRelative_Lambda(
			[IsComponentRelative](ESlateTransformComponent::Type InComponent)
			{
				return IsComponentRelative->operator[]((int32)InComponent) > 0.f;
			})
		.OnIsComponentRelativeChanged_Lambda(
			[IsComponentRelative](ESlateTransformComponent::Type InComponent, bool bIsRelative)
			{
				IsComponentRelative->operator[]((int32)InComponent) = bIsRelative ? 1.f : 0.f;
			});
	}

	TransformWidgetArgs.ConstructLocation(InControlType == ERigControlType::Position);
	TransformWidgetArgs.ConstructRotation(InControlType == ERigControlType::Rotator);
	TransformWidgetArgs.ConstructScale(InControlType == ERigControlType::Scale);

	return CreateEulerTransformValueWidgetRow(
		Keys,
		TransformWidgetArgs,
		CategoryBuilder,
		Label,
		Tooltip,
		CurrentTransformType,
		ValueType,
		NameContent);
}

TSharedPtr<TArray<ERigControlValueType>> FRigControlElementDetails::PickedValueTypes;

FDetailWidgetRow& FRigTransformElementDetails::CreateEulerTransformValueWidgetRow(
	const TArray<FRigElementKey>& Keys,
	SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs,
	IDetailCategoryBuilder& CategoryBuilder,
	const FText& Label,
	const FText& Tooltip,
	ERigTransformElementDetailsTransform::Type CurrentTransformType,
	ERigControlValueType ValueType,
	TSharedPtr<SWidget> NameContent)
{
	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
	if(ValueType == ERigControlValueType::Current)
	{
		HierarchyToChange = Hierarchy;
	}
	
	const FRigElementTransformWidgetSettings& Settings = FRigElementTransformWidgetSettings::FindOrAdd(ValueType, CurrentTransformType, TransformWidgetArgs);

	const bool bDisplayRelativeWorldOnCurrent = TransformWidgetArgs._DisplayRelativeWorld; 
	if(bDisplayRelativeWorldOnCurrent &&
		!TransformWidgetArgs._OnGetIsComponentRelative.IsBound() &&
		!TransformWidgetArgs._OnIsComponentRelativeChanged.IsBound())
	{
		TSharedRef<UE::Math::TVector<float>> IsComponentRelativeStorage = Settings.IsComponentRelative;
		
		TransformWidgetArgs.OnGetIsComponentRelative_Lambda(
			[IsComponentRelativeStorage](ESlateTransformComponent::Type InComponent)
			{
				return IsComponentRelativeStorage->operator[]((int32)InComponent) > 0.f;
			})
		.OnIsComponentRelativeChanged_Lambda(
			[IsComponentRelativeStorage](ESlateTransformComponent::Type InComponent, bool bIsRelative)
			{
				IsComponentRelativeStorage->operator[]((int32)InComponent) = bIsRelative ? 1.f : 0.f;
			});
	}

	const TSharedPtr<ESlateRotationRepresentation::Type> RotationRepresentationStorage = Settings.RotationRepresentation;
	TransformWidgetArgs.RotationRepresentation(RotationRepresentationStorage);

	auto IsComponentRelative = [TransformWidgetArgs](int32 Component) -> bool
	{
		if(TransformWidgetArgs._OnGetIsComponentRelative.IsBound())
		{
			return TransformWidgetArgs._OnGetIsComponentRelative.Execute((ESlateTransformComponent::Type)Component);
		}
		return true;
	};

	auto ConformComponentRelative = [TransformWidgetArgs, IsComponentRelative](int32 Component)
	{
		if(TransformWidgetArgs._OnIsComponentRelativeChanged.IsBound())
		{
			bool bRelative = IsComponentRelative(Component);
			TransformWidgetArgs._OnIsComponentRelativeChanged.Execute(ESlateTransformComponent::Location, bRelative);
			TransformWidgetArgs._OnIsComponentRelativeChanged.Execute(ESlateTransformComponent::Rotation, bRelative);
			TransformWidgetArgs._OnIsComponentRelativeChanged.Execute(ESlateTransformComponent::Scale, bRelative);
		}
	};

	TransformWidgetArgs.IsScaleLocked(Settings.IsScaleLocked);

	switch(CurrentTransformType)
	{
		case ERigTransformElementDetailsTransform::Minimum:
		case ERigTransformElementDetailsTransform::Maximum:
		{
			TransformWidgetArgs.AllowEditRotationRepresentation(false);
			TransformWidgetArgs.DisplayRelativeWorld(false);
			TransformWidgetArgs.DisplayToggle(true);
			TransformWidgetArgs.OnGetToggleChecked_Lambda([Keys, Hierarchy, ValueType]
				(
					ESlateTransformComponent::Type Component,
					ESlateRotationRepresentation::Type RotationRepresentation,
					ESlateTransformSubComponent::Type SubComponent
				) -> ECheckBoxState
				{
					TOptional<bool> FirstValue;

					for(const FRigElementKey& Key : Keys)
					{
						if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
						{
							TOptional<bool> Value;

							switch(ControlElement->Settings.ControlType)
							{
								case ERigControlType::Position:
								case ERigControlType::Rotator:
								case ERigControlType::Scale:
								{
									if(ControlElement->Settings.LimitEnabled.Num() == 3)
									{
										int32 Index = INDEX_NONE;
										if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
										{
											// TRotator is ordered Roll,Pitch,Yaw, while SNumericRotatorInputBox is ordered Pitch,Yaw,Roll
											switch (SubComponent)
											{
												case ESlateTransformSubComponent::Pitch: Index = 1; break;
												case ESlateTransformSubComponent::Yaw: Index = 2; break;
												case ESlateTransformSubComponent::Roll: Index = 0; break;
											}
										}
										else
										{
											Index = int32(SubComponent) - int32(ESlateTransformSubComponent::X);
										}

										if (Index != INDEX_NONE)
										{
											Value = ControlElement->Settings.LimitEnabled[Index].GetForValueType(ValueType);
										}
									}
									break;
								}
								case ERigControlType::EulerTransform:
								{
									if(ControlElement->Settings.LimitEnabled.Num() == 9)
									{
										switch(Component)
										{
											case ESlateTransformComponent::Location:
											{
												switch(SubComponent)
												{
													case ESlateTransformSubComponent::X:
													{
														Value = ControlElement->Settings.LimitEnabled[0].GetForValueType(ValueType);
														break;
													}
													case ESlateTransformSubComponent::Y:
													{
														Value = ControlElement->Settings.LimitEnabled[1].GetForValueType(ValueType);
														break;
													}
													case ESlateTransformSubComponent::Z:
													{
														Value = ControlElement->Settings.LimitEnabled[2].GetForValueType(ValueType);
														break;
													}
													default:
													{
														break;
													}
												}
												break;
											}
											case ESlateTransformComponent::Rotation:
											{
												switch(SubComponent)
												{
													case ESlateTransformSubComponent::Pitch:
													{
														Value = ControlElement->Settings.LimitEnabled[3].GetForValueType(ValueType);
														break;
													}
													case ESlateTransformSubComponent::Yaw:
													{
														Value = ControlElement->Settings.LimitEnabled[4].GetForValueType(ValueType);
														break;
													}
													case ESlateTransformSubComponent::Roll:
													{
														Value = ControlElement->Settings.LimitEnabled[5].GetForValueType(ValueType);
														break;
													}
													default:
													{
														break;
													}
												}
												break;
											}
											case ESlateTransformComponent::Scale:
											{
												switch(SubComponent)
												{
													case ESlateTransformSubComponent::X:
													{
														Value = ControlElement->Settings.LimitEnabled[6].GetForValueType(ValueType);
														break;
													}
													case ESlateTransformSubComponent::Y:
													{
														Value = ControlElement->Settings.LimitEnabled[7].GetForValueType(ValueType);
														break;
													}
													case ESlateTransformSubComponent::Z:
													{
														Value = ControlElement->Settings.LimitEnabled[8].GetForValueType(ValueType);
														break;
													}
													default:
													{
														break;
													}
												}
												break;
											}
										}
									}
									break;
								}
							}

							if(Value.IsSet())
							{
								if(FirstValue.IsSet())
								{
									if(FirstValue.GetValue() != Value.GetValue())
									{
										return ECheckBoxState::Undetermined;
									}
								}
								else
								{
									FirstValue = Value;
								}
							}
						}
					}

					if(!ensure(FirstValue.IsSet()))
					{
						return ECheckBoxState::Undetermined;
					}
					return FirstValue.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});
				
			TransformWidgetArgs.OnToggleChanged_Lambda([ValueType, Keys, this, Hierarchy]
			(
				ESlateTransformComponent::Type Component,
				ESlateRotationRepresentation::Type RotationRepresentation,
				ESlateTransformSubComponent::Type SubComponent,
				ECheckBoxState CheckState
			)
			{
				if(CheckState == ECheckBoxState::Undetermined)
				{
					return;
				}

				const bool Value = CheckState == ECheckBoxState::Checked;

				FScopedTransaction Transaction(LOCTEXT("ChangeLimitToggle", "Change Limit Toggle"));
				Hierarchy->Modify();

				for(const FRigElementKey& Key : Keys)
				{
					if(FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
					{
						switch(ControlElement->Settings.ControlType)
						{
							case ERigControlType::Position:
							case ERigControlType::Rotator:
							case ERigControlType::Scale:
							{
								if(ControlElement->Settings.LimitEnabled.Num() == 3)
								{
									int32 Index = INDEX_NONE;
									if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
									{
										// TRotator is ordered Roll,Pitch,Yaw, while SNumericRotatorInputBox is ordered Pitch,Yaw,Roll
										switch (SubComponent)
										{
											case ESlateTransformSubComponent::Pitch: Index = 1; break;
											case ESlateTransformSubComponent::Yaw: Index = 2; break;
											case ESlateTransformSubComponent::Roll: Index = 0; break;
										}
									}
									else
									{
										Index = int32(SubComponent) - int32(ESlateTransformSubComponent::X);
									}

									if (Index != INDEX_NONE)
									{
										ControlElement->Settings.LimitEnabled[Index].SetForValueType(ValueType, Value);
									}
								}
								break;
							}
							case ERigControlType::EulerTransform:
							{
								if(ControlElement->Settings.LimitEnabled.Num() == 9)
								{
									switch(Component)
									{
										case ESlateTransformComponent::Location:
										{
											switch(SubComponent)
											{
												case ESlateTransformSubComponent::X:
												{
													ControlElement->Settings.LimitEnabled[0].SetForValueType(ValueType, Value);
													break;
												}
												case ESlateTransformSubComponent::Y:
												{
													ControlElement->Settings.LimitEnabled[1].SetForValueType(ValueType, Value);
													break;
												}
												case ESlateTransformSubComponent::Z:
												{
													ControlElement->Settings.LimitEnabled[2].SetForValueType(ValueType, Value);
													break;
												}
												default:
												{
													break;
												}
											}
											break;
										}
										case ESlateTransformComponent::Rotation:
										{
											switch(SubComponent)
											{
												case ESlateTransformSubComponent::Pitch:
												{
													ControlElement->Settings.LimitEnabled[3].SetForValueType(ValueType, Value);
													break;
												}
												case ESlateTransformSubComponent::Yaw:
												{
													ControlElement->Settings.LimitEnabled[4].SetForValueType(ValueType, Value);
													break;
												}
												case ESlateTransformSubComponent::Roll:
												{
													ControlElement->Settings.LimitEnabled[5].SetForValueType(ValueType, Value);
													break;
												}
												default:
												{
													break;
												}
											}
											break;
										}
										case ESlateTransformComponent::Scale:
										{
											switch(SubComponent)
											{
												case ESlateTransformSubComponent::X:
												{
													ControlElement->Settings.LimitEnabled[6].SetForValueType(ValueType, Value);
													break;
												}
												case ESlateTransformSubComponent::Y:
												{
													ControlElement->Settings.LimitEnabled[7].SetForValueType(ValueType, Value);
													break;
												}
												case ESlateTransformSubComponent::Z:
												{
													ControlElement->Settings.LimitEnabled[8].SetForValueType(ValueType, Value);
													break;
												}
												default:
												{
													break;
												}
											}
											break;
										}
									}
								}
								break;
							}
						}
						
						Hierarchy->SetControlSettings(ControlElement, ControlElement->Settings, true, true, true);
					}
				}
			});
			break;
		}
		default:
		{
			TransformWidgetArgs.AllowEditRotationRepresentation(true);
			TransformWidgetArgs.DisplayRelativeWorld(bDisplayRelativeWorldOnCurrent);
			TransformWidgetArgs.DisplayToggle(false);
			TransformWidgetArgs._OnGetToggleChecked.Unbind();
			TransformWidgetArgs._OnToggleChanged.Unbind();
			break;
		}
	}

	auto GetRelativeAbsoluteTransforms = [CurrentTransformType, Keys, Hierarchy](
		const FRigElementKey& Key,
		ERigTransformElementDetailsTransform::Type InTransformType = ERigTransformElementDetailsTransform::Max
		) -> TPair<FEulerTransform, FEulerTransform>
	{
		if(InTransformType == ERigTransformElementDetailsTransform::Max)
		{
			InTransformType = CurrentTransformType;
		}

		FEulerTransform RelativeTransform = FEulerTransform::Identity;
		FEulerTransform AbsoluteTransform = FEulerTransform::Identity;

		const bool bInitial = InTransformType == ERigTransformElementDetailsTransform::Initial; 
		if(bInitial || InTransformType == ERigTransformElementDetailsTransform::Current)
		{
			RelativeTransform.FromFTransform(Hierarchy->GetLocalTransform(Key, bInitial));
			AbsoluteTransform.FromFTransform(Hierarchy->GetGlobalTransform(Key, bInitial));

			if(FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
			{
				switch(ControlElement->Settings.ControlType)
				{
					case ERigControlType::Rotator:
					case ERigControlType::EulerTransform:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					{
						FVector Vector;
						if(const UControlRig* ControlRig = Hierarchy->GetTypedOuter<UControlRig>())
						{
							Vector = ControlRig->GetControlSpecifiedEulerAngle(ControlElement, bInitial);
						}
						else
						{
							Vector = Hierarchy->GetControlSpecifiedEulerAngle(ControlElement, bInitial);
						}
						RelativeTransform.Rotation =  FRotator(Vector.Y, Vector.Z, Vector.X);
						break;
					}
					default:
					{
						break;
					}
				}
			}
		}
		else
		{
			if(FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
			{
				const ERigControlType ControlType = ControlElement->Settings.ControlType;

				if(InTransformType == ERigTransformElementDetailsTransform::Offset)
				{
					RelativeTransform.FromFTransform(Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialLocal));
					AbsoluteTransform.FromFTransform(Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialGlobal));
				}
				else if(InTransformType == ERigTransformElementDetailsTransform::Minimum)
				{
					switch(ControlType)
					{
						case ERigControlType::Position:
						{
							const FVector Data = 
								(FVector)Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Minimum)
								.Get<FVector3f>();
							AbsoluteTransform = RelativeTransform = FEulerTransform(Data, FRotator::ZeroRotator, FVector::OneVector);
							break;
						}
						case ERigControlType::Rotator:
						{
							const FVector Data = 
								(FVector)Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Minimum)
								.Get<FVector3f>();
							FRotator Rotator = FRotator::MakeFromEuler(Data);
							AbsoluteTransform = RelativeTransform = FEulerTransform(FVector::ZeroVector, Rotator, FVector::OneVector);
							break;
						}
						case ERigControlType::Scale:
						{
							const FVector Data = 
								(FVector)Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Minimum)
								.Get<FVector3f>();
							AbsoluteTransform = RelativeTransform = FEulerTransform(FVector::ZeroVector, FRotator::ZeroRotator, Data);
							break;
						}
						case ERigControlType::EulerTransform:
						{
							const FRigControlValue::FEulerTransform_Float EulerTransform = 
								Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Minimum)
								.Get<FRigControlValue::FEulerTransform_Float>();
							AbsoluteTransform = RelativeTransform = EulerTransform.ToTransform();
							break;
						}
					}
				}
				else if(InTransformType == ERigTransformElementDetailsTransform::Maximum)
				{
					switch(ControlType)
					{
						case ERigControlType::Position:
						{
							const FVector Data = 
								(FVector)Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Maximum)
								.Get<FVector3f>();
							AbsoluteTransform = RelativeTransform = FEulerTransform(Data, FRotator::ZeroRotator, FVector::OneVector);
							break;
						}
						case ERigControlType::Rotator:
						{
							const FVector Data = 
								(FVector)Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Maximum)
								.Get<FVector3f>();
							FRotator Rotator = FRotator::MakeFromEuler(Data);
							AbsoluteTransform = RelativeTransform = FEulerTransform(FVector::ZeroVector, Rotator, FVector::OneVector);
							break;
						}
						case ERigControlType::Scale:
						{
							const FVector Data = 
								(FVector)Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Maximum)
								.Get<FVector3f>();
							AbsoluteTransform = RelativeTransform = FEulerTransform(FVector::ZeroVector, FRotator::ZeroRotator, Data);
							break;
						}
						case ERigControlType::EulerTransform:
						{
							const FRigControlValue::FEulerTransform_Float EulerTransform = 
								Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Maximum)
								.Get<FRigControlValue::FEulerTransform_Float>();
							AbsoluteTransform = RelativeTransform = EulerTransform.ToTransform();
							break;
						}
					}
				}
			}
		}

		return TPair<FEulerTransform, FEulerTransform>(RelativeTransform, AbsoluteTransform);
	};

	
	auto GetCombinedTransform = [IsComponentRelative, GetRelativeAbsoluteTransforms](
		const FRigElementKey& Key,
		ERigTransformElementDetailsTransform::Type InTransformType = ERigTransformElementDetailsTransform::Max
		) -> FEulerTransform
	{
		const TPair<FEulerTransform, FEulerTransform> TransformPair = GetRelativeAbsoluteTransforms(Key, InTransformType);
		const FEulerTransform RelativeTransform = TransformPair.Key;
		const FEulerTransform AbsoluteTransform = TransformPair.Value;

		FEulerTransform Xfo;
		Xfo.SetLocation((IsComponentRelative(0)) ? RelativeTransform.GetLocation() : AbsoluteTransform.GetLocation());
		Xfo.SetRotator((IsComponentRelative(1)) ? RelativeTransform.Rotator() : AbsoluteTransform.Rotator());
		Xfo.SetScale3D((IsComponentRelative(2)) ? RelativeTransform.GetScale3D() : AbsoluteTransform.GetScale3D());

		return Xfo;
	};

	auto GetSingleTransform = [GetRelativeAbsoluteTransforms](
		const FRigElementKey& Key,
		bool bIsRelative,
		ERigTransformElementDetailsTransform::Type InTransformType = ERigTransformElementDetailsTransform::Max
		) -> FEulerTransform
	{
		const TPair<FEulerTransform, FEulerTransform> TransformPair = GetRelativeAbsoluteTransforms(Key, InTransformType);
		const FEulerTransform RelativeTransform = TransformPair.Key;
		const FEulerTransform AbsoluteTransform = TransformPair.Value;
		return bIsRelative ? RelativeTransform : AbsoluteTransform;
	};

	const TWeakPtr<FRigTransformElementDetails> WeakThisPtr = StaticCastWeakPtr<FRigTransformElementDetails>(AsWeak());
	auto SetSingleTransform = [CurrentTransformType, GetRelativeAbsoluteTransforms, WeakThisPtr, Hierarchy](
		const FRigElementKey& Key,
		FEulerTransform InTransform,
		bool bIsRelative,
		bool bSetupUndoRedo)
	{
		const TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin();
		if(!StrongThisPtr)
		{
			return;
		}
		
		const bool bCurrent = CurrentTransformType == ERigTransformElementDetailsTransform::Current; 
		const bool bInitial = CurrentTransformType == ERigTransformElementDetailsTransform::Initial; 

		bool bConstructionModeEnabled = false;
		if (UControlRig* DebuggedRig = Cast<UControlRig>(StrongThisPtr->PerElementInfos[0].GetAsset()->GetObjectBeingDebugged()))
		{
			bConstructionModeEnabled = DebuggedRig->IsConstructionModeEnabled();
		}

		TArray<URigHierarchy*> HierarchiesToUpdate;
		HierarchiesToUpdate.Add(Hierarchy);
		if(!bCurrent || bConstructionModeEnabled)
		{
			HierarchiesToUpdate.Add(StrongThisPtr->PerElementInfos[0].GetDefaultHierarchy());
		}

		for(URigHierarchy* HierarchyToUpdate : HierarchiesToUpdate)
		{
			if (UControlRig* RigToUpdate = HierarchyToUpdate->GetTypedOuter<UControlRig>())
			{
				// make sure we can run this rig as we are interacting with it.
				RigToUpdate->ResetEvaluationsLeft();
			}
			
			if(bInitial || CurrentTransformType == ERigTransformElementDetailsTransform::Current)
			{
				if(bIsRelative)
				{
					HierarchyToUpdate->SetLocalTransform(Key, InTransform.ToFTransform(), bInitial, true, bSetupUndoRedo);

					if(FRigControlElement* ControlElement = HierarchyToUpdate->Find<FRigControlElement>(Key))
					{
						switch(ControlElement->Settings.ControlType)
						{
							case ERigControlType::Rotator:
							{
								const FVector EulerAngle(InTransform.Rotator().Roll, InTransform.Rotator().Pitch, InTransform.Rotator().Yaw);
								HierarchyToUpdate->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle, bInitial);

								const ERigControlValueType ValueType = bInitial ? ERigControlValueType::Initial : ERigControlValueType::Current;
								const FRotator Rotator(Hierarchy->GetControlQuaternion(ControlElement, EulerAngle));
								HierarchyToUpdate->SetControlValue(ControlElement, FRigControlValue::Make<FRotator>(Rotator), ValueType, bSetupUndoRedo);
								break;
							}
							case ERigControlType::EulerTransform:
							case ERigControlType::Transform:
							case ERigControlType::TransformNoScale:
							{
								FVector EulerAngle(InTransform.Rotator().Roll, InTransform.Rotator().Pitch, InTransform.Rotator().Yaw);
								HierarchyToUpdate->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle, bInitial);
								break;
							}
							default:
							{
								break;
							}
						}
					}
				}
				else
				{
					HierarchyToUpdate->SetGlobalTransform(Key, InTransform.ToFTransform(), bInitial, true, bSetupUndoRedo);
				}
			}
			else
			{
				if(FRigControlElement* ControlElement = HierarchyToUpdate->Find<FRigControlElement>(Key))
				{
					const ERigControlType ControlType = ControlElement->Settings.ControlType;

					if(CurrentTransformType == ERigTransformElementDetailsTransform::Offset)
					{
						if(!bIsRelative)
						{
							const FTransform ParentTransform = HierarchyToUpdate->GetParentTransform(Key, bInitial);
							InTransform.FromFTransform(InTransform.ToFTransform().GetRelativeTransform(ParentTransform));
						}
						HierarchyToUpdate->SetControlOffsetTransform(Key, InTransform.ToFTransform(), true, true, bSetupUndoRedo);
					}
					else if(CurrentTransformType == ERigTransformElementDetailsTransform::Minimum)
					{
						switch(ControlType)
						{
							case ERigControlType::Position:
							{
								const FRigControlValue Value = FRigControlValue::Make<FVector3f>((FVector3f)InTransform.GetLocation());
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Minimum, bSetupUndoRedo, true);
								break;
							}
							case ERigControlType::Rotator:
							{
								const FVector3f Euler = (FVector3f)InTransform.Rotator().Euler();
								const FRigControlValue Value = FRigControlValue::Make<FVector3f>(Euler);
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Minimum, bSetupUndoRedo, true);
								break;
							}
							case ERigControlType::Scale:
							{
								const FRigControlValue Value = FRigControlValue::Make<FVector3f>((FVector3f)InTransform.GetScale3D());
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Minimum, bSetupUndoRedo, true);
								break;
							}
							case ERigControlType::EulerTransform:
							{
								const FRigControlValue Value = FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(InTransform);
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Minimum, bSetupUndoRedo, true);
								break;
							}
						}
					}
					else if(CurrentTransformType == ERigTransformElementDetailsTransform::Maximum)
					{
						switch(ControlType)
						{
							case ERigControlType::Position:
							{
								const FRigControlValue Value = FRigControlValue::Make<FVector3f>((FVector3f)InTransform.GetLocation());
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Maximum, bSetupUndoRedo, true);
								break;
							}
							case ERigControlType::Rotator:
							{
								const FVector3f Euler = (FVector3f)InTransform.Rotator().Euler();
								const FRigControlValue Value = FRigControlValue::Make<FVector3f>(Euler);
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Maximum, bSetupUndoRedo, true);
								break;
							}
							case ERigControlType::Scale:
							{
								const FRigControlValue Value = FRigControlValue::Make<FVector3f>((FVector3f)InTransform.GetScale3D());
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Maximum, bSetupUndoRedo, true);
								break;
							}
							case ERigControlType::EulerTransform:
							{
								const FRigControlValue Value = FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(InTransform);
								HierarchyToUpdate->SetControlValue(ControlElement, Value, ERigControlValueType::Maximum, bSetupUndoRedo, true);
								break;
							}
						}
					}
				}
			}
		}
	};

	TransformWidgetArgs.OnGetNumericValue_Lambda([Keys, GetCombinedTransform](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent) -> TOptional<FVector::FReal>
	{
		TOptional<FVector::FReal> FirstValue;

		for(int32 Index = 0; Index < Keys.Num(); Index++)
		{
			const FRigElementKey& Key = Keys[Index];
			FEulerTransform Xfo = GetCombinedTransform(Key);

			TOptional<FVector::FReal> CurrentValue = SAdvancedTransformInputBox<FEulerTransform>::GetNumericValueFromTransform(Xfo, Component, Representation, SubComponent);
			if(!CurrentValue.IsSet())
			{
				return CurrentValue;
			}

			if(Index == 0)
			{
				FirstValue = CurrentValue;
			}
			else
			{
				if(!FMath::IsNearlyEqual(FirstValue.GetValue(), CurrentValue.GetValue()))
				{
					return TOptional<FVector::FReal>();
				}
			}
		}
		
		return FirstValue;
	});

	TransformWidgetArgs.OnNumericValueChanged_Lambda(
	[
		Keys,
		WeakThisPtr,
		IsComponentRelative,
		GetSingleTransform,
		SetSingleTransform,
		HierarchyToChange
	](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal InNumericValue)
	{
		const TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin();
		if (!StrongThisPtr)
		{
			return;
		}
			
		const bool bIsRelative = IsComponentRelative((int32)Component);

		for(const FRigElementKey& Key : Keys)
		{
			FEulerTransform Transform = GetSingleTransform(Key, bIsRelative);
			FEulerTransform PreviousTransform = Transform;
			SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(Transform, InNumericValue, Component, Representation, SubComponent);

			if(!FRigControlElementDetails::Equals(Transform, PreviousTransform))
			{
				if(!StrongThisPtr->SliderTransaction.IsValid())
				{
					StrongThisPtr->SliderTransaction = MakeShareable(new FScopedTransaction(NSLOCTEXT("ControlRigElementDetails", "ChangeNumericValue", "Change Numeric Value")));
					HierarchyToChange->Modify();
				}
							
				SetSingleTransform(Key, Transform, bIsRelative, false);
			}
		}
	});

	TransformWidgetArgs.OnNumericValueCommitted_Lambda(
	[
		Keys,
		WeakThisPtr,
		IsComponentRelative,
		GetSingleTransform,
		SetSingleTransform,
		HierarchyToChange
	](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal InNumericValue,
		ETextCommit::Type InCommitType)
	{
		const TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin();
		if (!StrongThisPtr)
		{
			return;
		}
		
		const bool bIsRelative = IsComponentRelative((int32)Component);

		{
			FScopedTransaction Transaction(LOCTEXT("ChangeNumericValue", "Change Numeric Value"));
			if(!StrongThisPtr->SliderTransaction.IsValid())
			{
				HierarchyToChange->Modify();
			}
			
			for(const FRigElementKey& Key : Keys)
			{
				FEulerTransform Transform = GetSingleTransform(Key, bIsRelative);
				SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(Transform, InNumericValue, Component, Representation, SubComponent);
				SetSingleTransform(Key, Transform, bIsRelative, true);
			}
		}

		StrongThisPtr->SliderTransaction.Reset();
	});

	TransformWidgetArgs.OnBeginSliderMovement_Lambda(
		[
			WeakThisPtr
		](
			ESlateTransformComponent::Type Component,
			ESlateRotationRepresentation::Type Representation,
			ESlateTransformSubComponent::Type SubComponent)
		{
			TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin();
			if (!StrongThisPtr)
			{
				return;
			}
			
			if (UControlRig* DebuggedRig = Cast<UControlRig>(StrongThisPtr->PerElementInfos[0].GetAsset()->GetObjectBeingDebugged()))
			{
				EControlRigInteractionType Type = EControlRigInteractionType::None;
				switch (Component)
				{
					case ESlateTransformComponent::Location: Type = EControlRigInteractionType::Translate; break;
					case ESlateTransformComponent::Rotation: Type = EControlRigInteractionType::Rotate; break;
					case ESlateTransformComponent::Scale: Type = EControlRigInteractionType::Scale; break;
					default: Type = EControlRigInteractionType::All;
				}
				DebuggedRig->InteractionType = (uint8)Type;
				DebuggedRig->ElementsBeingInteracted.Reset();
				for (const FPerElementInfo& ElementInfo : StrongThisPtr->PerElementInfos)
				{
					DebuggedRig->ElementsBeingInteracted.AddUnique(ElementInfo.Element.GetKey());
				}
				
				StrongThisPtr->InteractionScopes.Emplace(MakeUnique<FControlRigInteractionScope>(DebuggedRig));
			}
		});
	TransformWidgetArgs.OnEndSliderMovement_Lambda(
		[
			WeakThisPtr
		](
			ESlateTransformComponent::Type Component,
			ESlateRotationRepresentation::Type Representation,
			ESlateTransformSubComponent::Type SubComponent,
			FVector::FReal InNumericValue)
		{
			const TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin();
			if (!StrongThisPtr)
			{
				return;
			}
			
			if (UControlRig* DebuggedRig = Cast<UControlRig>(StrongThisPtr->PerElementInfos[0].GetAsset()->GetObjectBeingDebugged()))
			{
				DebuggedRig->InteractionType = (uint8)EControlRigInteractionType::None;
				DebuggedRig->ElementsBeingInteracted.Reset();
			}
			StrongThisPtr->InteractionScopes.Reset();
		});

	TransformWidgetArgs.OnCopyToClipboard_Lambda([Keys, IsComponentRelative, ConformComponentRelative, GetSingleTransform](
		ESlateTransformComponent::Type InComponent
		)
	{
		if(Keys.Num() == 0)
		{
			return;
		}

		// make sure that we use the same relative setting on all components when copying
		ConformComponentRelative(0);
		const bool bIsRelative = IsComponentRelative(0); 

		const FRigElementKey& FirstKey = Keys[0];
		FEulerTransform Xfo = GetSingleTransform(FirstKey, bIsRelative);

		FString Content;
		switch(InComponent)
		{
			case ESlateTransformComponent::Location:
			{
				const FVector Data = Xfo.GetLocation();
				TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				const FRotator Data = Xfo.Rotator();
				TBaseStructure<FRotator>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				const FVector Data = Xfo.GetScale3D();
				TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Max:
			default:
			{
				TBaseStructure<FEulerTransform>::Get()->ExportText(Content, &Xfo, &Xfo, nullptr, PPF_None, nullptr);
				break;
			}
		}

		if(!Content.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*Content);
		}
	});

	TransformWidgetArgs.OnPasteFromClipboard_Lambda([this, Keys, IsComponentRelative, ConformComponentRelative, GetSingleTransform, SetSingleTransform, HierarchyToChange](
		ESlateTransformComponent::Type InComponent
		)
	{
		if(Keys.Num() == 0)
		{
			return;
		}
		
		
		// make sure that we use the same relative setting on all components when pasting
		ConformComponentRelative(0);
		const bool bIsRelative = IsComponentRelative(0); 

		FString Content;
		FPlatformApplicationMisc::ClipboardPaste(Content);

		if(Content.IsEmpty())
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PasteTransform", "Paste Transform"));
		HierarchyToChange->Modify();

		for(const FRigElementKey& Key : Keys)
		{
			FEulerTransform Xfo = GetSingleTransform(Key, bIsRelative);
			{
				class FRigPasteTransformWidgetErrorPipe : public FOutputDevice
				{
				public:

					int32 NumErrors;

					FRigPasteTransformWidgetErrorPipe()
						: FOutputDevice()
						, NumErrors(0)
					{
					}

					virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
					{
						UE_LOGF(LogControlRig, Error, "Error Pasting to Widget: %ls", V);
						NumErrors++;
					}
				};

				FRigPasteTransformWidgetErrorPipe ErrorPipe;
				
				switch(InComponent)
				{
					case ESlateTransformComponent::Location:
					{
						FVector Data = Xfo.GetLocation();
						TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
						Xfo.SetLocation(Data);
						break;
					}
					case ESlateTransformComponent::Rotation:
					{
						FRotator Data = Xfo.Rotator();
						TBaseStructure<FRotator>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FRotator>::Get()->GetName(), true);
						Xfo.SetRotator(Data);
						break;
					}
					case ESlateTransformComponent::Scale:
					{
						FVector Data = Xfo.GetScale3D();
						TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
						Xfo.SetScale3D(Data);
						break;
					}
					case ESlateTransformComponent::Max:
					default:
					{
						TBaseStructure<FEulerTransform>::Get()->ImportText(*Content, &Xfo, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FEulerTransform>::Get()->GetName(), true);
						break;
					}
				}

				if(ErrorPipe.NumErrors == 0)
				{
					SetSingleTransform(Key, Xfo, bIsRelative, true);
				}
			}
		}
	});

	TransformWidgetArgs.DiffersFromDefault_Lambda([
		CurrentTransformType,
		Keys,
		GetSingleTransform
		
	](
		ESlateTransformComponent::Type InComponent) -> bool
	{
		for(const FRigElementKey& Key : Keys)
		{
			const FEulerTransform CurrentTransform = GetSingleTransform(Key, true);
			FEulerTransform DefaultTransform;

			switch(CurrentTransformType)
			{
				case ERigTransformElementDetailsTransform::Current:
				{
					DefaultTransform = GetSingleTransform(Key, true, ERigTransformElementDetailsTransform::Initial);
					break;
				}
				default:
				{
					DefaultTransform = FEulerTransform::Identity; 
					break;
				}
			}

			switch(InComponent)
			{
				case ESlateTransformComponent::Location:
				{
					if(!DefaultTransform.GetLocation().Equals(CurrentTransform.GetLocation()))
					{
						return true;
					}
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					if(!DefaultTransform.Rotator().Equals(CurrentTransform.Rotator()))
					{
						return true;
					}
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					if(!DefaultTransform.GetScale3D().Equals(CurrentTransform.GetScale3D()))
					{
						return true;
					}
					break;
				}
				default: // also no component whole transform
				{
					if(!DefaultTransform.GetLocation().Equals(CurrentTransform.GetLocation()) ||
						!DefaultTransform.Rotator().Equals(CurrentTransform.Rotator()) ||
						!DefaultTransform.GetScale3D().Equals(CurrentTransform.GetScale3D()))
					{
						return true;
					}
					break;
				}
			}
		}
		return false;
	});

	TransformWidgetArgs.OnResetToDefault_Lambda([this, CurrentTransformType, Keys, GetSingleTransform, SetSingleTransform, HierarchyToChange](
		ESlateTransformComponent::Type InComponent)
	{
		FScopedTransaction Transaction(LOCTEXT("ResetTransformToDefault", "Reset Transform to Default"));
		HierarchyToChange->Modify();

		for(const FRigElementKey& Key : Keys)
		{
			FEulerTransform CurrentTransform = GetSingleTransform(Key, true);
			FEulerTransform DefaultTransform;

			switch(CurrentTransformType)
			{
				case ERigTransformElementDetailsTransform::Current:
				{
					DefaultTransform = GetSingleTransform(Key, true, ERigTransformElementDetailsTransform::Initial);
					break;
				}
				default:
				{
					DefaultTransform = FEulerTransform::Identity; 
					break;
				}
			}

			switch(InComponent)
			{
				case ESlateTransformComponent::Location:
				{
					CurrentTransform.SetLocation(DefaultTransform.GetLocation());
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					CurrentTransform.SetRotator(DefaultTransform.Rotator());
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					CurrentTransform.SetScale3D(DefaultTransform.GetScale3D());
					break;
				}
				default: // whole transform / max component
				{
					CurrentTransform = DefaultTransform;
					break;
				}
			}

			SetSingleTransform(Key, CurrentTransform, true, true);
		}
	});

	return SAdvancedTransformInputBox<FEulerTransform>::ConstructGroupedTransformRows(
		CategoryBuilder, 
		Label, 
		Tooltip, 
		TransformWidgetArgs,
		NameContent);
}

ERigTransformElementDetailsTransform::Type FRigTransformElementDetails::GetTransformTypeFromValueType(
	ERigControlValueType InValueType)
{
	ERigTransformElementDetailsTransform::Type TransformType = ERigTransformElementDetailsTransform::Current;
	switch(InValueType)
	{
		case ERigControlValueType::Initial:
		{
			TransformType = ERigTransformElementDetailsTransform::Initial;
			break;
		}
		case ERigControlValueType::Minimum:
		{
			TransformType = ERigTransformElementDetailsTransform::Minimum;
			break;
		}
		case ERigControlValueType::Maximum:
		{
			TransformType = ERigTransformElementDetailsTransform::Maximum;
			break;
		}
		default:
		{
			break;
		}
	}
	return TransformType;
}

void FRigBoneElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FRigTransformElementDetails::CustomizeDetails(DetailBuilder);
	CustomizeTransform(DetailBuilder);
	CustomizeComponents(DetailBuilder);
	CustomizeMetadata(DetailBuilder);
}

void FRigControlElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FRigTransformElementDetails::CustomizeDetails(DetailBuilder);

	CustomizeControl(DetailBuilder);
	CustomizeValue(DetailBuilder);
	CustomizeTransform(DetailBuilder);
	CustomizeShape(DetailBuilder);
	CustomizeAvailableSpaces(DetailBuilder);
	CustomizeAnimationChannels(DetailBuilder);
	CustomizeComponents(DetailBuilder);
	CustomizeMetadata(DetailBuilder);
}

void FRigControlElementDetails::CustomizeValue(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	if(IsAnyElementNotOfType(ERigElementType::Control))
	{
		return;
	}

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();

	// only show this section if all controls are the same type
	const FRigControlElement* FirstControlElement = PerElementInfos[0].GetElement<FRigControlElement>();
	const ERigControlType ControlType = FirstControlElement->Settings.ControlType;
	bool bAllAnimationChannels = true;
	
	for(const FPerElementInfo& Info : PerElementInfos)
	{
		const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>();
		if(ControlElement->Settings.ControlType != ControlType)
		{
			return;
		}
		if(ControlElement->Settings.AnimationType != ERigControlAnimationType::AnimationChannel)
		{
			bAllAnimationChannels = false;
		}
	}

	// transforms don't show their value here - instead they are shown in the transform section
	if((ControlType == ERigControlType::EulerTransform ||
		ControlType == ERigControlType::Transform ||
		ControlType == ERigControlType::TransformNoScale) &&
		!bAllAnimationChannels)
	{
		return;
	}
	
	TArray<FText> Labels = {
		LOCTEXT("Initial", "Initial"),
		LOCTEXT("Current", "Current")
	};
	TArray<FText> Tooltips = {
		LOCTEXT("ValueInitialTooltip", "The initial animation value of the control"),
		LOCTEXT("ValueCurrentTooltip", "The current animation value of the control")
	};
	TArray<ERigControlValueType> ValueTypes = {
		ERigControlValueType::Initial,
		ERigControlValueType::Current
	};

	// bool doesn't have limits,
	// transform types already got filtered out earlier.
	// integers with enums don't have limits either
	if(ControlType != ERigControlType::Bool &&
		(ControlType != ERigControlType::Integer || !FirstControlElement->Settings.ControlEnum))
	{
		Labels.Append({
			LOCTEXT("Min", "Min"),
			LOCTEXT("Max", "Max")
		});
		Tooltips.Append({
			LOCTEXT("ValueMinimumTooltip", "The minimum limit(s) for the control"),
			LOCTEXT("ValueMaximumTooltip", "The maximum limit(s) for the control")
		});
		ValueTypes.Append({
			ERigControlValueType::Minimum,
			ERigControlValueType::Maximum
		});
	}
	
	IDetailCategoryBuilder& ValueCategory = DetailBuilder.EditCategory(TEXT("Value"), LOCTEXT("Value", "Value"));

	if(!PickedValueTypes.IsValid())
	{
		PickedValueTypes = MakeShareable(new TArray<ERigControlValueType>({ERigControlValueType::Current}));
	}

	TSharedPtr<SSegmentedControl<ERigControlValueType>> ValueTypeChoiceWidget =
		SSegmentedControl<ERigControlValueType>::Create(
			ValueTypes,
			Labels,
			Tooltips,
			*PickedValueTypes.Get(),
			true,
			SSegmentedControl<ERigControlValueType>::FOnValuesChanged::CreateLambda(
				[](TArray<ERigControlValueType> NewSelection)
				{
					(*FRigControlElementDetails::PickedValueTypes.Get()) = NewSelection;
				}
			)
		);

	AddChoiceWidgetRow(ValueCategory, FText::FromString(TEXT("ValueType")), ValueTypeChoiceWidget.ToSharedRef());

	TArray<FRigElementKey> Keys = GetElementKeys();
	Keys = Hierarchy->SortKeys(Keys);

	for(int32 Index=0; Index < ValueTypes.Num(); Index++)
	{
		const ERigControlValueType ValueType = ValueTypes[Index];

		const TAttribute<EVisibility> VisibilityAttribute =
			TAttribute<EVisibility>::CreateLambda([ValueType, ValueTypeChoiceWidget]()-> EVisibility
			{
				return ValueTypeChoiceWidget->HasValue(ValueType) ? EVisibility::Visible : EVisibility::Collapsed; 
			});
		
		switch(ControlType)
		{
			case ERigControlType::Bool:
			{
				CreateBoolValueWidgetRow(Keys, ValueCategory, Labels[Index], Tooltips[Index], ValueType, VisibilityAttribute);
				break;
			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				CreateFloatValueWidgetRow(Keys, ValueCategory, Labels[Index], Tooltips[Index], ValueType, VisibilityAttribute);
				break;
			}
			case ERigControlType::Integer:
			{
				bool bIsEnum = false;
				for(const FRigElementKey& Key : Keys)
				{
					if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
					{
						if(ControlElement->Settings.ControlEnum)
						{
							bIsEnum = true;
							break;
						}
					}
				}

				if(bIsEnum)
				{
					CreateEnumValueWidgetRow(Keys, ValueCategory, Labels[Index], Tooltips[Index], ValueType, VisibilityAttribute);
				}
				else
				{
					CreateIntegerValueWidgetRow(Keys, ValueCategory, Labels[Index], Tooltips[Index], ValueType, VisibilityAttribute);
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				CreateVector2DValueWidgetRow(Keys, ValueCategory, Labels[Index], Tooltips[Index], ValueType, VisibilityAttribute);
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Rotator:
			case ERigControlType::Scale:
			{
				SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FEulerTransform>::FArguments()
				.DisplayToggle(false)
				.DisplayRelativeWorld(true)
				.Visibility(VisibilityAttribute)
				.PreventThrottling(true);

				CreateTransformComponentValueWidgetRow(
					ControlType,
					GetElementKeys(),
					TransformWidgetArgs,
					ValueCategory,
					Labels[Index],
					Tooltips[Index],
					GetTransformTypeFromValueType(ValueType),
					ValueType);
				break;
			}
			default:
			{
				break;
			}
		}
	}
}

void FRigControlElementDetails::CustomizeControl(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	if(IsAnyElementNotOfType(ERigElementType::Control))
	{
		return;
	}

	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural;
	
	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();

	const TSharedPtr<IPropertyHandle> SettingsHandle = DetailBuilder.GetProperty(TEXT("Settings"));
	DetailBuilder.HideProperty(SettingsHandle);

	IDetailCategoryBuilder& ControlCategory = DetailBuilder.EditCategory(TEXT("Control"), LOCTEXT("Control", "Control"));

	const bool bAllAnimationChannels = !IsAnyControlNotOfAnimationType(ERigControlAnimationType::AnimationChannel);
	static const FText DisplayNameText = LOCTEXT("DisplayName", "Display Name");
	static const FText ChannelNameText = LOCTEXT("ChannelName", "Channel Name");
	const FText DisplayNameLabelText = bAllAnimationChannels ? ChannelNameText : DisplayNameText;

	const TSharedPtr<IPropertyHandle> DisplayNameHandle = SettingsHandle->GetChildHandle(TEXT("DisplayName"));
	ControlCategory.AddCustomRow(DisplayNameLabelText)
	.IsEnabled(bIsEnabled)
	.NameContent()
	[
		DisplayNameHandle->CreatePropertyNameWidget(DisplayNameLabelText)
	]
	.ValueContent()
	[
		SNew(SInlineEditableTextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(this, &FRigControlElementDetails::GetDisplayName)
		.OnTextCommitted(this, &FRigControlElementDetails::SetDisplayName)
		.OnVerifyTextChanged_Lambda([this](const FText& InText, FText& OutErrorMessage) -> bool
		{
			return OnVerifyDisplayNameChanged(InText, OutErrorMessage, GetElementKey());
		})
		.IsEnabled(bIsEnabled && (PerElementInfos.Num() == 1))
	];

	if(bAllAnimationChannels)
	{
		ControlCategory.AddCustomRow(FText::FromString(TEXT("Script Name")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Script Name")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(!bIsProcedural)
		]
		.ValueContent()
		[
			SNew(SInlineEditableTextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FRigBaseElementDetails::GetName)
			.OnTextCommitted(this, &FRigBaseElementDetails::SetName)
			.OnVerifyTextChanged(this, &FRigBaseElementDetails::OnVerifyNameChanged)
			.IsEnabled(!bIsProcedural && PerElementInfos.Num() == 1)
		];
	}

	const TSharedRef<IPropertyUtilities> PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	// when control type changes, we have to refresh detail panel
	const TSharedPtr<IPropertyHandle> AnimationTypeHandle = SettingsHandle->GetChildHandle(TEXT("AnimationType"));
	AnimationTypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
		[this, PropertyUtilities, HierarchyToChange, Hierarchy]()
		{
			TArray<FRigControlElement> ControlElementsInView = GetElementsInDetailsView<FRigControlElement>();

			if (HierarchyToChange && ControlElementsInView.Num() == PerElementInfos.Num())
			{
				HierarchyToChange->Modify();
				
				for(int32 ControlIndex = 0; ControlIndex< ControlElementsInView.Num(); ControlIndex++)
				{
					const FRigControlElement& ViewElement = ControlElementsInView[ControlIndex];
					FRigControlElement* ControlElement = PerElementInfos[ControlIndex].GetDefaultElement<FRigControlElement>();
					check(ControlElement);
					
					ControlElement->Settings.AnimationType = ViewElement.Settings.AnimationType;

					ControlElement->Settings.bGroupWithParentControl =
						ControlElement->Settings.ControlType == ERigControlType::Bool ||
						ControlElement->Settings.ControlType == ERigControlType::Float ||
						ControlElement->Settings.ControlType == ERigControlType::ScaleFloat ||
						ControlElement->Settings.ControlType == ERigControlType::Integer ||
						ControlElement->Settings.ControlType == ERigControlType::Vector2D;

					switch(ControlElement->Settings.AnimationType)
					{
						case ERigControlAnimationType::AnimationControl:
						{
							ControlElement->Settings.ShapeVisibility = ERigControlVisibility::UserDefined;
							ControlElement->Settings.bShapeVisible = true;
							break;
						}
						case ERigControlAnimationType::AnimationChannel:
						{
							ControlElement->Settings.ShapeVisibility = ERigControlVisibility::UserDefined;
							ControlElement->Settings.bShapeVisible = false;
							break;
						}
						case ERigControlAnimationType::ProxyControl:
						{
							ControlElement->Settings.ShapeVisibility = ERigControlVisibility::BasedOnSelection;
							ControlElement->Settings.bShapeVisible = true;
							ControlElement->Settings.bGroupWithParentControl = false;
							break;
						}
						default:
						{
							ControlElement->Settings.ShapeVisibility = ERigControlVisibility::UserDefined;
							ControlElement->Settings.bShapeVisible = true;
							ControlElement->Settings.bGroupWithParentControl = false;
							break;
						}
					}

					HierarchyToChange->SetControlSettings(ControlElement, ControlElement->Settings, true, true, true);
					PerElementInfos[ControlIndex].WrapperObject->SetContent<FRigControlElement>(*ControlElement);

					if (HierarchyToChange != Hierarchy)
					{
						if(FRigControlElement* OtherControlElement = PerElementInfos[0].GetElement<FRigControlElement>())
						{
							OtherControlElement->Settings = ControlElement->Settings;
							Hierarchy->SetControlSettings(OtherControlElement, OtherControlElement->Settings, true, true, true);
						}
					}
				}
				
				PropertyUtilities->ForceRefresh();
			}
		}
	));

	ControlCategory.AddProperty(AnimationTypeHandle.ToSharedRef())
	.IsEnabled(bIsEnabled);

	// when control type changes, we have to refresh detail panel
	const TSharedPtr<IPropertyHandle> ControlTypeHandle = SettingsHandle->GetChildHandle(TEXT("ControlType"));
	ControlTypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(
		[this, PropertyUtilities]()
		{
			TArray<FRigControlElement> ControlElementsInView = GetElementsInDetailsView<FRigControlElement>();
			HandleControlTypeChanged(ControlElementsInView[0].Settings.ControlType, TArray<FRigElementKey>(), PropertyUtilities);
		}
	));

	ControlCategory.AddProperty(ControlTypeHandle.ToSharedRef())
	.IsEnabled(bIsEnabled);

	const bool bSupportsShape = !IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationChannel) &&
		!IsAnyControlOfAnimationType(ERigControlAnimationType::VisualCue);

	if (HierarchyToChange != nullptr)
	{
		bool bEnableGroupWithParentControl = true;
		for(const FPerElementInfo& Info : PerElementInfos)
		{
			if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
			{
				bool bSingleEnableGroupWithParentControl = false;
				if(const FRigControlElement* ParentElement =
					Cast<FRigControlElement>(Info.GetHierarchy()->GetFirstParent(ControlElement)))
				{
					if(ControlElement->Settings.IsAnimatable() &&
						Info.GetHierarchy()->GetChildren(ControlElement).IsEmpty())
					{
						bSingleEnableGroupWithParentControl = true;
					}
				}

				if(!bSingleEnableGroupWithParentControl)
				{
					bEnableGroupWithParentControl = false;
					break;
				}
			}
		}
		if(bEnableGroupWithParentControl)
		{
			const TSharedPtr<IPropertyHandle> GroupWithParentControlHandle = SettingsHandle->GetChildHandle(TEXT("bGroupWithParentControl"));
			ControlCategory.AddProperty(GroupWithParentControlHandle.ToSharedRef()).DisplayName(FText::FromString(TEXT("Group Channels")))
			.IsEnabled(bIsEnabled);
		}
	}
	
	if(bSupportsShape &&
		!(IsAnyControlNotOfValueType(ERigControlType::Integer) &&
		IsAnyControlNotOfValueType(ERigControlType::Float) &&
		IsAnyControlNotOfValueType(ERigControlType::ScaleFloat) &&
		IsAnyControlNotOfValueType(ERigControlType::Vector2D)))
	{
		const TSharedPtr<IPropertyHandle> PrimaryAxisHandle = SettingsHandle->GetChildHandle(TEXT("PrimaryAxis"));
		ControlCategory.AddProperty(PrimaryAxisHandle.ToSharedRef()).DisplayName(FText::FromString(TEXT("Primary Axis")))
		.IsEnabled(bIsEnabled);
	}

	if(CVarControlRigHierarchyEnableRotationOrder.GetValueOnAnyThread())
	{
		if (IsAnyControlOfValueType(ERigControlType::EulerTransform) || IsAnyControlOfValueType(ERigControlType::Rotator))
		{
			const TSharedPtr<IPropertyHandle> UsePreferredRotationOrderHandle = SettingsHandle->GetChildHandle(TEXT("bUsePreferredRotationOrder"));
			ControlCategory.AddProperty(UsePreferredRotationOrderHandle.ToSharedRef())
				.DisplayName(FText::FromString(TEXT("Use Preferred Rotation Order")))
				.IsEnabled(bIsEnabled);

			const TSharedPtr<IPropertyHandle> PreferredRotationOrderHandle = SettingsHandle->GetChildHandle(TEXT("PreferredRotationOrder"));
			ControlCategory.AddProperty(PreferredRotationOrderHandle.ToSharedRef())
				.DisplayName(FText::FromString(TEXT("Preferred Rotation Order")))
				.IsEnabled(bIsEnabled);
		}
	}

	if(IsAnyControlOfValueType(ERigControlType::Integer))
	{		
		FDetailWidgetRow* EnumWidgetRow = &ControlCategory.AddCustomRow(FText::FromString(TEXT("ControlEnum")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Control Enum")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(bIsEnabled)
		]
		.ValueContent()
		[
			SNew(SRigVMEnumPicker)
			.OnEnumChanged(this, &FRigControlElementDetails::HandleControlEnumChanged, PropertyUtilities)
			.IsEnabled(bIsEnabled)
			.GetCurrentEnum_Lambda([this]()
			{
				UEnum* CommonControlEnum = nullptr;
				for (int32 ControlIndex=0; ControlIndex < PerElementInfos.Num(); ++ControlIndex)
				{
					FPerElementInfo& Info = PerElementInfos[ControlIndex];
					const FRigControlElement ControlInView = Info.WrapperObject->GetContent<FRigControlElement>();
					FRigControlElement* ControlBeingCustomized = Info.GetDefaultElement<FRigControlElement>();
					check(ControlBeingCustomized);
						
					UEnum* ControlEnum = ControlBeingCustomized->Settings.ControlEnum;
					if (ControlIndex == 0)
					{
						CommonControlEnum = ControlEnum;
					}
					else if(ControlEnum != CommonControlEnum)
					{
						CommonControlEnum = nullptr;
						break;
					}
				}
				return CommonControlEnum;
			})
		];
	}

	if(bSupportsShape)
	{
		const TSharedPtr<IPropertyHandle> RestrictSpaceSwitchingHandle = SettingsHandle->GetChildHandle(TEXT("bRestrictSpaceSwitching"));
		ControlCategory
		.AddProperty(RestrictSpaceSwitchingHandle.ToSharedRef())
		.DisplayName(FText::FromString(TEXT("Restrict Switching")))
		.IsEnabled(bIsEnabled);

		// Available Spaces is now handled by its own category (CustomizeAvailableSpaces)
		//const TSharedPtr<IPropertyHandle> CustomizationHandle = SettingsHandle->GetChildHandle(TEXT("Customization"));
		//const TSharedPtr<IPropertyHandle> AvailableSpacesHandle = CustomizationHandle->GetChildHandle(TEXT("AvailableSpaces"));
		//ControlCategory.AddProperty(AvailableSpacesHandle.ToSharedRef())
		//.IsEnabled(bIsEnabled);
	}

	TArray<FRigElementKey> Keys = GetElementKeys();

	if(bSupportsShape)
	{
		const TSharedPtr<IPropertyHandle> DrawLimitsHandle = SettingsHandle->GetChildHandle(TEXT("bDrawLimits"));
		
		ControlCategory
		.AddProperty(DrawLimitsHandle.ToSharedRef()).DisplayName(FText::FromString(TEXT("Draw Limits")))
		.IsEnabled(TAttribute<bool>::CreateLambda([Keys, Hierarchy, bIsEnabled]() -> bool
		{
			if(!bIsEnabled)
			{
				return false;
			}
			
			for(const FRigElementKey& Key : Keys)
			{
				if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
				{
					if(ControlElement->Settings.LimitEnabled.Contains(FRigControlLimitEnabled(true, true)))
					{
						return true;
					}
				}
			}
			return false;
		}));
	}

	ERigControlType CommonControlType = ERigControlType::Bool;
	if(GetCommonControlType(CommonControlType))
	{
		if(FRigControlTransformChannelDetails::GetVisibleChannelsForControlType(CommonControlType) != nullptr)
		{
			const TSharedPtr<IPropertyHandle> FilteredChannelsHandle = SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigControlSettings, FilteredChannels));
			ControlCategory.AddProperty(FilteredChannelsHandle.ToSharedRef())
			.IsEnabled(bIsEnabled);
		}
	}
	
	if(IsAnyControlOfAnimationType(ERigControlAnimationType::ProxyControl) || IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationControl))
	{
		ControlCategory.AddProperty(SettingsHandle->GetChildHandle(TEXT("DrivenControls")).ToSharedRef())
		.IsEnabled(bIsEnabled);
	}
}

void FRigControlElementDetails::HandleControlEnumChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	PropertyUtilities->ForceRefresh();
	UEnum* ControlEnum = FindObject<UEnum>(nullptr, **InItem.Get(), EFindObjectFlags::None);

	for(int32 ControlIndex = 0; ControlIndex < PerElementInfos.Num(); ControlIndex++)
	{
		FPerElementInfo& Info = PerElementInfos[ControlIndex];
		const FRigControlElement ControlInView = Info.WrapperObject->GetContent<FRigControlElement>();
		FRigControlElement* ControlBeingCustomized = Info.GetDefaultElement<FRigControlElement>();
		check(ControlBeingCustomized);
		
		ControlBeingCustomized->Settings.ControlEnum = ControlEnum;
		if (ControlEnum != nullptr)
		{
			int32 Maximum = (int32)ControlEnum->GetMaxEnumValue() - 1;
			ControlBeingCustomized->Settings.MinimumValue.Set<int32>(0);
			ControlBeingCustomized->Settings.MaximumValue.Set<int32>(Maximum);
			ControlBeingCustomized->Settings.LimitEnabled.Reset();
			ControlBeingCustomized->Settings.LimitEnabled.Add(true);
			Info.GetDefaultHierarchy()->SetControlSettings(ControlBeingCustomized, ControlBeingCustomized->Settings, true, true, true);

			FRigControlValue InitialValue = Info.GetDefaultHierarchy()->GetControlValue(ControlBeingCustomized, ERigControlValueType::Initial);
			FRigControlValue CurrentValue = Info.GetDefaultHierarchy()->GetControlValue(ControlBeingCustomized, ERigControlValueType::Current);

			ControlBeingCustomized->Settings.ApplyLimits(InitialValue);
			ControlBeingCustomized->Settings.ApplyLimits(CurrentValue);
			Info.GetDefaultHierarchy()->SetControlValue(ControlBeingCustomized, InitialValue, ERigControlValueType::Initial, false, false, true);
			Info.GetDefaultHierarchy()->SetControlValue(ControlBeingCustomized, CurrentValue, ERigControlValueType::Current, false, false, true);

			if (UControlRig* DebuggedRig = Cast<UControlRig>(Info.GetAsset()->GetObjectBeingDebugged()))
			{
				URigHierarchy* DebuggedHierarchy = DebuggedRig->GetHierarchy();
				if(FRigControlElement* DebuggedControlElement = DebuggedHierarchy->Find<FRigControlElement>(ControlBeingCustomized->GetKey()))
				{
					DebuggedControlElement->Settings.MinimumValue.Set<int32>(0);
					DebuggedControlElement->Settings.MaximumValue.Set<int32>(Maximum);
					DebuggedHierarchy->SetControlSettings(DebuggedControlElement, DebuggedControlElement->Settings, true, true, true);

					DebuggedHierarchy->SetControlValue(DebuggedControlElement, InitialValue, ERigControlValueType::Initial);
					DebuggedHierarchy->SetControlValue(DebuggedControlElement, CurrentValue, ERigControlValueType::Current);
				}
			}
		}

		Info.WrapperObject->SetContent<FRigControlElement>(*ControlBeingCustomized);
	}
}

void FRigControlElementDetails::CustomizeAnimationChannels(IDetailLayoutBuilder& DetailBuilder)
{
	// We only show this section for parents of animation channels
	if(!IsAnyControlNotOfAnimationType(ERigControlAnimationType::AnimationChannel))
	{
		// If all controls are animation channels, just return 
		return;
	}

	// only show this if only one control is selected
	if(PerElementInfos.Num() != 1)
	{
		return;
	}

	const FRigControlElement* ControlElement = PerElementInfos[0].GetElement<FRigControlElement>();
	if(ControlElement == nullptr)
	{
		return;
	}

	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural;

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("AnimationChannels"), LOCTEXT("AnimationChannels", "Animation Channels"));
	
	const TSharedRef<IPropertyUtilities> PropertyUtilities = DetailBuilder.GetPropertyUtilities();
	
	const TSharedRef<SHorizontalBox> HeaderContentWidget = SNew(SHorizontalBox);
	HeaderContentWidget->AddSlot()
	.HAlign(HAlign_Right)
	[
		SNew(SButton)
		.IsEnabled(bIsEnabled)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(FMargin(1, 0))
		.OnClicked(this, &FRigControlElementDetails::OnAddAnimationChannelClicked)
		.HAlign(HAlign_Right)
		.ToolTipText(LOCTEXT("AddAnimationChannelToolTip", "Add a new animation channel"))
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
	Category.HeaderContent(HeaderContentWidget);

	const TArray<FRigControlElement*> AnimationChannels = Hierarchy->GetAnimationChannels(ControlElement, false);
	const bool bHasAnimationChannels = !AnimationChannels.IsEmpty();
	const FRigElementKey ControlElementKey = ControlElement->GetKey();

	for(const FRigControlElement* AssignedAnimationChannel : AnimationChannels)
	{
		const FRigElementKey ChildElementKey = AssignedAnimationChannel->GetKey();
		const bool bIsDirectlyParentedAnimationChannel = HierarchyToChange->GetDefaultParent(ChildElementKey) == ControlElementKey;
		
		const TPair<const FSlateBrush*, FSlateColor> BrushAndColor = SRigHierarchyItem::GetBrushForElementType(Hierarchy, ChildElementKey);

		static TArray<TSharedPtr<ERigControlType>> ControlValueTypes;
		if(ControlValueTypes.IsEmpty())
		{
			const UEnum* ValueTypeEnum = StaticEnum<ERigControlType>();
			for(int64 EnumValue = 0; EnumValue < ValueTypeEnum->GetMaxEnumValue(); EnumValue++)
			{
				if(ValueTypeEnum->HasMetaData(TEXT("Hidden"), (int32)EnumValue))
				{
					continue;
				}
				ControlValueTypes.Add(MakeShareable(new ERigControlType((ERigControlType)EnumValue)));
			}
		}

		TSharedPtr<SButton> SelectAnimationChannelButton;
		TSharedPtr<SImage> SelectAnimationChannelImage;
		TSharedPtr<SWidget> NameContent;

		SAssignNew(NameContent, SHorizontalBox)
		.IsEnabled(bIsEnabled)

		+ SHorizontalBox::Slot()
		.MaxWidth(32)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
		[
			SNew(SComboButton)
			.ContentPadding(0)
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(BrushAndColor.Key)
				.ColorAndOpacity(BrushAndColor.Value)
			]
			.MenuContent()
			[
				SNew(SListView<TSharedPtr<ERigControlType>>)
				.ListItemsSource( &ControlValueTypes )
				.OnGenerateRow(this, &FRigControlElementDetails::HandleGenerateAnimationChannelTypeRow, ChildElementKey)
				.OnSelectionChanged(this, &FRigControlElementDetails::HandleControlTypeChanged, ChildElementKey, PropertyUtilities)
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
		[
			SNew(SInlineEditableTextBlock)
			.Font(bIsDirectlyParentedAnimationChannel ? IDetailLayoutBuilder::GetDetailFont() : IDetailLayoutBuilder::GetDetailFontItalic())
			.Text_Lambda([this, ChildElementKey]() -> FText
			{
				return GetDisplayNameForElement(ChildElementKey);
			})
			.OnTextCommitted_Lambda([this, ChildElementKey](const FText& InNewText, ETextCommit::Type InCommitType)
			{
				SetDisplayNameForElement(InNewText, InCommitType, ChildElementKey);
			})
			.OnVerifyTextChanged_Lambda([this, ChildElementKey](const FText& InText, FText& OutErrorMessage) -> bool
			{
				return OnVerifyDisplayNameChanged(InText, OutErrorMessage, ChildElementKey);
			})
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
		[
			SAssignNew(SelectAnimationChannelButton, SButton)
			.ButtonStyle( FAppStyle::Get(), "NoBorder" )
			.OnClicked_Lambda([this, ChildElementKey]() -> FReply
			{
				return OnSelectElementClicked(ChildElementKey);
			})
			.ContentPadding(0)
			.ToolTipText(NSLOCTEXT("ControlRigElementDetails", "SelectAnimationChannelInHierarchyToolTip", "Select Animation Channel"))
			[
				SAssignNew(SelectAnimationChannelImage, SImage)
				.Image(FAppStyle::GetBrush("Icons.Search"))
			]
		];

		SelectAnimationChannelImage->SetColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([this, SelectAnimationChannelButton]() { return FRigElementKeyDetails::OnGetWidgetForeground(SelectAnimationChannelButton); }));

		const FText Label = FText::FromString(FString::Printf(TEXT("Channel%s"), *AssignedAnimationChannel->GetDisplayName().ToString()));
		const TArray<FRigElementKey> ChildElementKeys = {ChildElementKey};
		TAttribute<EVisibility> Visibility = EVisibility::Visible;

		FDetailWidgetRow* WidgetRow = nullptr; 
		switch(AssignedAnimationChannel->Settings.ControlType)
		{
			case ERigControlType::Bool:
			{
				WidgetRow = &CreateBoolValueWidgetRow(ChildElementKeys, Category, Label, FText(), ERigControlValueType::Current, Visibility, NameContent);
				break;
			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				WidgetRow = &CreateFloatValueWidgetRow(ChildElementKeys, Category, Label, FText(), ERigControlValueType::Current, Visibility, NameContent);
				break;
			}
			case ERigControlType::Integer:
			{
				if(AssignedAnimationChannel->Settings.ControlEnum)
				{
					WidgetRow = &CreateEnumValueWidgetRow(ChildElementKeys, Category, Label, FText(), ERigControlValueType::Current, Visibility, NameContent);
				}
				else
				{
					WidgetRow = &CreateIntegerValueWidgetRow(ChildElementKeys, Category, Label, FText(), ERigControlValueType::Current, Visibility, NameContent);
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				WidgetRow = &CreateVector2DValueWidgetRow(ChildElementKeys, Category, Label, FText(), ERigControlValueType::Current, Visibility, NameContent);
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Rotator:
			case ERigControlType::Scale:
			{
				SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs =
					SAdvancedTransformInputBox<FEulerTransform>::FArguments()
				.DisplayToggle(false)
				.DisplayRelativeWorld(false)
				.Visibility(EVisibility::Visible)
				.PreventThrottling(true);

				WidgetRow = &CreateTransformComponentValueWidgetRow(
					AssignedAnimationChannel->Settings.ControlType,
					ChildElementKeys,
					TransformWidgetArgs,
					Category,
					Label,
					FText(),
					GetTransformTypeFromValueType(ERigControlValueType::Current),
					ERigControlValueType::Current,
					NameContent);
				break;
			}
			case ERigControlType::Transform:
			case ERigControlType::EulerTransform:
			{
				SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs =
					SAdvancedTransformInputBox<FEulerTransform>::FArguments()
				.DisplayToggle(false)
				.DisplayRelativeWorld(false)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Visibility(EVisibility::Visible)
				.PreventThrottling(true);

				WidgetRow = &CreateEulerTransformValueWidgetRow(
					ChildElementKeys,
					TransformWidgetArgs,
					Category,
					Label,
					FText(),
					ERigTransformElementDetailsTransform::Current,
					ERigControlValueType::Current,
					NameContent);
				break;
			}
			default:
			{
				WidgetRow = &Category.AddCustomRow(Label)
				.NameContent()
				[
					NameContent.ToSharedRef()
				];
				break;
			}
		}

		if(WidgetRow)
		{
			if(bIsDirectlyParentedAnimationChannel)
			{
				WidgetRow->AddCustomContextMenuAction(FUIAction(
					FExecuteAction::CreateLambda([this, ChildElementKeys, HierarchyToChange]()
					{
						if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
						{
							FScopedTransaction Transaction(LOCTEXT("DeleteAnimationChannels", "Delete Animation Channels"));
							HierarchyToChange->Modify();
							
							for(const FRigElementKey& KeyToRemove : ChildElementKeys)
							{
								Controller->RemoveElement(KeyToRemove, true, true);
							}
						}
					})),
					LOCTEXT("DeleteAnimationChannel", "Delete"),
					LOCTEXT("DeleteAnimationChannelTooltip", "Deletes this animation channel"),
					FSlateIcon());
			}
			else
			{
				WidgetRow->AddCustomContextMenuAction(FUIAction(
					FExecuteAction::CreateLambda([this, ControlElementKey, ChildElementKeys, HierarchyToChange, PropertyUtilities]()
					{
						if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
						{
							FScopedTransaction Transaction(LOCTEXT("RemoveAnimationChannelHosts", "Remove Animation Channel Hosts"));
							HierarchyToChange->Modify();
											
							for(const FRigElementKey& KeyToRemove : ChildElementKeys)
							{
								Controller->RemoveChannelHost(KeyToRemove, ControlElementKey, true, true);
							}
							PropertyUtilities->ForceRefresh();
						}
					})),
					LOCTEXT("RemoveAnimationChannelHost", "Remove from this host"),
					LOCTEXT("RemoveAnimationChannelHostTooltip", "Remove the animation channel from this host"),
					FSlateIcon());
			}
			
			// move up or down
			WidgetRow->AddCustomContextMenuAction(FUIAction(
				FExecuteAction::CreateLambda([this, ControlElementKey, ChildElementKeys, HierarchyToChange]()
				{
					if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
					{
						FScopedTransaction Transaction(LOCTEXT("MoveAnimationChannelUpTransaction", "Move Animation Channel Up"));
						HierarchyToChange->Modify();
						
						for(const FRigElementKey& KeyToMove : ChildElementKeys)
						{
							const int32 LocalIndex = HierarchyToChange->GetLocalIndex(KeyToMove);
							Controller->ReorderElement(KeyToMove, LocalIndex - 1, true);
						}
						Controller->SelectElement(ControlElementKey, true, true);
					}
				})),
				LOCTEXT("MoveAnimationChannelUp", "Move Up"),
				LOCTEXT("MoveAnimationChannelUpTooltip", "Reorders this animation channel to show up one higher"),
				FSlateIcon());
			WidgetRow->AddCustomContextMenuAction(FUIAction(
				FExecuteAction::CreateLambda([this, ControlElementKey, ChildElementKeys, HierarchyToChange]()
				{
					if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
					{
						FScopedTransaction Transaction(LOCTEXT("MoveAnimationChannelDownTransaction", "Move Animation Channel Down"));
						HierarchyToChange->Modify();
						
						for(const FRigElementKey& KeyToMove : ChildElementKeys)
						{
							const int32 LocalIndex = HierarchyToChange->GetLocalIndex(KeyToMove);
							Controller->ReorderElement(KeyToMove, LocalIndex + 1, true);
						}
						Controller->SelectElement(ControlElementKey, true, true);
					}
				})),
				LOCTEXT("MoveAnimationChannelDown", "Move Down"),
				LOCTEXT("MoveAnimationChannelDownTooltip", "Reorders this animation channel to show up one lower"),
				FSlateIcon());
		}
	}

	Category.InitiallyCollapsed(!bHasAnimationChannels);
	if(!bHasAnimationChannels)
	{
		Category.AddCustomRow(FText()).WholeRowContent()
		[
			SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
			[
				SNew(STextBlock)
				.IsEnabled(bIsEnabled)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("NoAnimationChannels", "No animation channels"))
			]
		];
	}
}

void FRigControlElementDetails::CustomizeAvailableSpaces(IDetailLayoutBuilder& DetailBuilder)
{
	// only show this if only one control / animation channel is selected
	if(PerElementInfos.Num() != 1)
	{
		return;
	}

	const FRigControlElement* ControlElement = PerElementInfos[0].GetElement<FRigControlElement>();
	if(ControlElement == nullptr)
	{
		return;
	}

	const bool bIsAnimationChannel = IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationChannel);
	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural;

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();

	static const FText ControlSpaces = LOCTEXT("AvailableSpaces", "Available Spaces"); 
	static const FText ChannelHosts = LOCTEXT("ChannelHosts", "Channel Hosts"); 
	static const FText ControlSpacesToolTip = LOCTEXT("AvailableSpacesToolTip", "Spaces available for this Control"); 
	static const FText ChannelHostsToolTip = LOCTEXT("ChannelHostsToolTip", "A list of controls this channel is listed under"); 
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("MultiParents"), bIsAnimationChannel ? ChannelHosts : ControlSpaces);
	Category.SetToolTip(bIsAnimationChannel ? ChannelHostsToolTip : ControlSpaces);
	
	const TSharedRef<IPropertyUtilities> PropertyUtilities = DetailBuilder.GetPropertyUtilities();

	DisplaySettings.bShowBones = true;
	DisplaySettings.bShowControls = true;
	DisplaySettings.bShowNulls = true;
	DisplaySettings.bShowReferences = false;
	DisplaySettings.bShowSockets = false;
	DisplaySettings.bShowComponents = false;
	DisplaySettings.bHideParentsOnFilter = true;
	DisplaySettings.bFlattenHierarchyOnFilter = true;
	DisplaySettings.bShowIconColors = true;
	DisplaySettings.bArrangeByModules = false;
	DisplaySettings.bFlattenModules = false;
	DisplaySettings.NameDisplayMode = EElementNameDisplayMode::AssetDefault;
	DisplaySettings.OutlinerDisplayMode = EMultiRigTreeDisplayMode::All;

	const TSharedRef<SHorizontalBox> HeaderContentWidget = SNew(SHorizontalBox);
	HeaderContentWidget->AddSlot()
	.HAlign(HAlign_Right)
	[
		SAssignNew(AddSpaceMenuAnchor, SMenuAnchor)
		.Placement( MenuPlacement_BelowAnchor )
		.OnGetMenuContent( this, &FRigControlElementDetails::GetAddSpaceContent, PropertyUtilities)
		.Content()
		[
			SNew(SImage)
			.OnMouseButtonDown(this, &FRigControlElementDetails::OnAddSpaceMouseDown, PropertyUtilities)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]	
	];
	Category.HeaderContent(HeaderContentWidget);

	TArray<FRigElementKeyWithLabel> AvailableSpaces;
	const FRigElementKey DefaultParent = Hierarchy->GetDefaultParent(ControlElement->GetKey());
	if(DefaultParent.IsValid())
	{
		const FName SpaceLabel = Hierarchy->GetDisplayLabelForParent(ControlElement->GetKey(), DefaultParent);
		AvailableSpaces.Emplace(DefaultParent, SpaceLabel);
	}
	for(const FRigElementKeyWithLabel& AvailableSpace : ControlElement->Settings.Customization.AvailableSpaces)
	{
		AvailableSpaces.AddUnique(AvailableSpace);
	}

	static const FText RemoveSpaceText = LOCTEXT("RemoveSpace", "Remove Space");
	static const FText RemoveChannelHostText = LOCTEXT("RemoveChannelHost", "Remove Channel Host");
	static const FText RemoveSpaceToolTipText = LOCTEXT("RemoveSpaceToolTip", "Removes this space from the list of available spaces");
	static const FText RemoveChannelHostToolTipText = LOCTEXT("RemoveChannelHostToolTip", "Remove the channel from this hosting control");

	for(int32 Index = 0; Index < AvailableSpaces.Num(); Index++)
	{
		const FRigElementKey ControlKey = ControlElement->GetKey();
		const FRigElementKeyWithLabel AvailableSpace = AvailableSpaces[Index];
		const bool bIsParentSpace = Index == 0;
		const TPair<const FSlateBrush*, FSlateColor> BrushAndColor = SRigHierarchyItem::GetBrushForElementType(Hierarchy, AvailableSpace.Key);

		TSharedPtr<SButton> SelectSpaceButton, RemoveSpaceButton, MoveSpaceUpButton, MoveSpaceDownButton;
		TSharedPtr<SImage> SelectSpaceImage, RemoveSpaceImage, MoveSpaceUpImage, MoveSpaceDownImage;

		FDetailWidgetRow& WidgetRow = Category.AddCustomRow(FText::FromString(AvailableSpace.Key.ToString()))
		.NameContent()
		.MinDesiredWidth(200.f)
		.MaxDesiredWidth(800.f)
		[
			SNew(SHorizontalBox)
			.IsEnabled(bIsEnabled)

			+ SHorizontalBox::Slot()
			.MaxWidth(32)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
			[
				SNew(SImage)
				.Image(BrushAndColor.Key)
				.ColorAndOpacity(BrushAndColor.Value)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SEditableText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsReadOnly((Index == 0) && bIsEnabled)
				.Text_Lambda([this, AvailableSpace]() -> FText
				{
					const FName Label = PerElementInfos[0].GetHierarchy()->GetDisplayLabelForParent(
						PerElementInfos[0].Element.GetKey(),
						AvailableSpace.Key);
					if(Label.IsNone())
					{
						return GetDisplayNameForElement(AvailableSpace.Key);
					}
					return FText::FromName(Label);
				})
				.OnTextCommitted_Lambda([this, AvailableSpace](const FText& InText, ETextCommit::Type InCommitType)
				{
					if(InCommitType == ETextCommit::OnCleared)
					{
						return;
					}
					FControlRigAssetInterfacePtr Blueprint = PerElementInfos[0].GetAsset();
					URigHierarchyController* Controller = Blueprint->GetHierarchyController();
					(void)Controller->SetAvailableSpaceLabel(PerElementInfos[0].Element.GetKey(), AvailableSpace.Key, *InText.ToString(), true);
				})
				.ToolTipText_Lambda([this, Index, AvailableSpace]()
				{
					if(Index == 0)
					{
						return FText::Format(
							LOCTEXT("AvailableSpaceToolTipDefaultParentFormat", "{0}\n\nThis is the default parent - the label cannot be edited."),
							GetDisplayNameForElement(AvailableSpace.Key));
					}
					return FText::Format(
						LOCTEXT("AvailableSpaceToolTipFormat", "{0}\n\nDouble-click here to edit the label of this space."),
						GetDisplayNameForElement(AvailableSpace.Key));
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
			[
				SAssignNew(SelectSpaceButton, SButton)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.OnClicked_Lambda([this, AvailableSpace]() -> FReply
				{
					return OnSelectElementClicked(AvailableSpace.Key);
				})
				.ContentPadding(0)
				.ToolTipText(LOCTEXT("SelectElementInHierarchy", "Select Element in hierarchy"))
				[
					SAssignNew(SelectSpaceImage, SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(bIsEnabled)
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(MoveSpaceUpButton, SButton)
				.Visibility((Index > 0 && !bIsAnimationChannel) ? EVisibility::Visible : EVisibility::Collapsed)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ContentPadding(0)
				.IsEnabled(Index > 1)
				.OnClicked_Lambda([this, ControlKey, AvailableSpace, HierarchyToChange, Index, PropertyUtilities]
				{
					if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
					{
						FScopedTransaction Transaction(LOCTEXT("MoveAvailableSpaceUp", "Move Available Space Up"));
						HierarchyToChange->Modify();
						Controller->SetAvailableSpaceIndex(ControlKey, AvailableSpace.Key, Index - 2);
						PropertyUtilities->ForceRefresh();
					}
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("MoveUp", "Move Up"))
				[
					SAssignNew(MoveSpaceUpImage, SImage)
					.Image(FAppStyle::GetBrush("Icons.ChevronUp"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(MoveSpaceDownButton, SButton)
				.Visibility((Index > 0 && !bIsAnimationChannel) ? EVisibility::Visible : EVisibility::Collapsed)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ContentPadding(0)
				.IsEnabled(Index > 0 && Index < AvailableSpaces.Num() - 1)
				.OnClicked_Lambda([this, ControlKey, AvailableSpace, HierarchyToChange, Index, PropertyUtilities]
				{
					if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
					{
						FScopedTransaction Transaction(LOCTEXT("MoveAvailableSpaceDown", "Move Available Space Down"));
						HierarchyToChange->Modify();
						Controller->SetAvailableSpaceIndex(ControlKey, AvailableSpace.Key, Index);
						PropertyUtilities->ForceRefresh();
					}
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("MoveDown", "Move Down"))
				[
					SAssignNew(MoveSpaceDownImage, SImage)
					.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(RemoveSpaceButton, SButton)
				.Visibility(Index > 0 ? EVisibility::Visible : EVisibility::Collapsed)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ContentPadding(0)
				.IsEnabled(Index > 0)
				.OnClicked_Lambda([this, ControlKey, AvailableSpace, HierarchyToChange, Index, PropertyUtilities]
				{
					if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
					{
						const bool bIsAnimationChannel = IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationChannel);
						FScopedTransaction Transaction(bIsAnimationChannel ? RemoveChannelHostText : RemoveSpaceText);
						HierarchyToChange->Modify();
						if(bIsAnimationChannel)
						{
							Controller->RemoveChannelHost(ControlKey, AvailableSpace.Key);
						}
						else
						{
							Controller->RemoveAvailableSpace(ControlKey, AvailableSpace.Key);
						}
						PropertyUtilities->ForceRefresh();
					}
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("Remove", "Remove"))
				[
					SAssignNew(RemoveSpaceImage, SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
		];

		SelectSpaceImage->SetColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([this, SelectSpaceButton]() { return FRigElementKeyDetails::OnGetWidgetForeground(SelectSpaceButton); }));
		MoveSpaceUpImage->SetColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([this, MoveSpaceUpButton]() { return FRigElementKeyDetails::OnGetWidgetForeground(MoveSpaceUpButton); }));
		MoveSpaceDownImage->SetColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([this, MoveSpaceDownButton]() { return FRigElementKeyDetails::OnGetWidgetForeground(MoveSpaceDownButton); }));
		RemoveSpaceImage->SetColorAndOpacity(TAttribute<FSlateColor>::CreateLambda([this, RemoveSpaceButton]() { return FRigElementKeyDetails::OnGetWidgetForeground(RemoveSpaceButton); }));

		if(!bIsProcedural)
		{
			if(!bIsAnimationChannel)
			{
				WidgetRow.AddCustomContextMenuAction(FUIAction(
					FExecuteAction::CreateLambda([this, ControlKey, AvailableSpace, HierarchyToChange, Index, PropertyUtilities]()
					{
						if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
						{
							FScopedTransaction Transaction(LOCTEXT("MoveAvailableSpaceUp", "Move Available Space Up"));
							HierarchyToChange->Modify();
							Controller->SetAvailableSpaceIndex(ControlKey, AvailableSpace.Key, Index - 2);
							PropertyUtilities->ForceRefresh();
						}
					}),
					FCanExecuteAction::CreateLambda([Index](){ return Index > 1; })),
					LOCTEXT("MoveUp", "Move Up"),
					LOCTEXT("MoveAvailableSpaceUpToolTip", "Moves this available space up in the list of spaces"),
					FSlateIcon());

				const int32 NumSpaces = AvailableSpaces.Num();
				WidgetRow.AddCustomContextMenuAction(FUIAction(
					FExecuteAction::CreateLambda([this, ControlKey, AvailableSpace, HierarchyToChange, Index, PropertyUtilities]()
					{
						if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
						{
							FScopedTransaction Transaction(LOCTEXT("MoveAvailableSpaceDown", "Move Available Space Down"));
							HierarchyToChange->Modify();
							Controller->SetAvailableSpaceIndex(ControlKey, AvailableSpace.Key, Index);
							PropertyUtilities->ForceRefresh();
						}
					}),
					FCanExecuteAction::CreateLambda([Index, NumSpaces](){ return Index > 0 && Index < NumSpaces - 1; })),
					LOCTEXT("MoveDown", "Move Down"),
					LOCTEXT("MoveAvailableSpaceDownToolTip", "Moves this available space down in the list of spaces"),
					FSlateIcon());
			}

			WidgetRow.AddCustomContextMenuAction(FUIAction(
				FExecuteAction::CreateLambda([this, ControlKey, AvailableSpace, HierarchyToChange, PropertyUtilities]()
				{
					if(URigHierarchyController* Controller = HierarchyToChange->GetController(true))
					{
						const bool bIsAnimationChannel = IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationChannel);
						FScopedTransaction Transaction(bIsAnimationChannel ? RemoveChannelHostText : RemoveSpaceText);
						HierarchyToChange->Modify();
						if(bIsAnimationChannel)
						{
							Controller->RemoveChannelHost(ControlKey, AvailableSpace.Key);
						}
						else
						{
							Controller->RemoveAvailableSpace(ControlKey, AvailableSpace.Key);
						}
						PropertyUtilities->ForceRefresh();
					}
				}),
				FCanExecuteAction::CreateLambda([bIsParentSpace](){ return !bIsParentSpace; })),
				bIsAnimationChannel ? RemoveChannelHostText : RemoveSpaceText,
				bIsAnimationChannel ? RemoveChannelHostToolTipText : RemoveSpaceToolTipText,
				FSlateIcon());
		}
	}

	Category.InitiallyCollapsed(AvailableSpaces.Num() < 2);
	if(AvailableSpaces.IsEmpty())
	{
		static const FText NoSpacesText = LOCTEXT("NoSpacesSet", "No Available Spaces set");
		static const FText NoChannelHostsText = LOCTEXT("NoChannelHostsSet", "No Channel Hosts set");

		Category.AddCustomRow(FText()).WholeRowContent()
		[
			SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
			[
				SNew(STextBlock)
				.IsEnabled(bIsEnabled)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(bIsAnimationChannel ? NoChannelHostsText : NoSpacesText)
			]
		];
	}
}

FReply FRigControlElementDetails::OnAddAnimationChannelClicked()
{
	if(IsAnyElementNotOfType(ERigElementType::Control) || IsAnyElementProcedural())
	{
		return FReply::Handled();
	}

	if (!PerElementInfos[0].GetElement())
	{
		return FReply::Unhandled();
	}

	const FRigElementKey Key = PerElementInfos[0].GetElement()->GetKey();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();

	static const FName ChannelName = TEXT("Channel");
	FRigControlSettings Settings;
	Settings.AnimationType = ERigControlAnimationType::AnimationChannel;
	Settings.ControlType = ERigControlType::Float;
	Settings.MinimumValue = FRigControlValue::Make<float>(0.f);
	Settings.MaximumValue = FRigControlValue::Make<float>(1.f);
	Settings.DisplayName = HierarchyToChange->GetSafeNewDisplayName(Key, ChannelName);
	HierarchyToChange->GetController(true)->AddAnimationChannel(ChannelName, Key, Settings, true, true);
	HierarchyToChange->GetController(true)->SelectElement(Key);
	return FReply::Handled();
}

TSharedRef<ITableRow> FRigControlElementDetails::HandleGenerateAnimationChannelTypeRow(TSharedPtr<ERigControlType> ControlType, const TSharedRef<STableViewBase>& OwnerTable, FRigElementKey ControlKey)
{
	const URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();

	TPair<const FSlateBrush*, FSlateColor> BrushAndColor = SRigHierarchyItem::GetBrushForElementType(HierarchyToChange, ControlKey);
	BrushAndColor.Value = SRigHierarchyItem::GetColorForControlType(*ControlType.Get(), nullptr);

	return SNew(STableRow<TSharedPtr<ERigControlType>>, OwnerTable)
	.Content()
	[
		SNew(SHorizontalBox)
	
		+ SHorizontalBox::Slot()
		.MaxWidth(18)
		.FillWidth(1.0)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
		[
			SNew(SImage)
			.Image(BrushAndColor.Key)
			.ColorAndOpacity(BrushAndColor.Value)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 0.f, 0.f))
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(StaticEnum<ERigControlType>()->GetDisplayNameTextByValue((int64)*ControlType.Get()))
		]
	];
}

TSharedRef<SWidget> FRigControlElementDetails::GetAddSpaceContent(const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	if(PerElementInfos.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	FRigHierarchyTreeDelegates RigTreeDelegates;
	RigTreeDelegates.OnGetHierarchy = FOnGetRigTreeHierarchy::CreateLambda([this]()
	{
		return PerElementInfos[0].GetHierarchy();
	});
	RigTreeDelegates.OnGetDisplaySettings = FOnGetRigTreeDisplaySettings::CreateSP(this, &FRigControlElementDetails::GetDisplaySettings);
	RigTreeDelegates.OnGetSelection = FOnRigTreeGetSelection::CreateLambda([]() -> TArray<FRigHierarchyKey> { return {}; });
	RigTreeDelegates.OnSelectionChanged = FOnRigTreeSelectionChanged::CreateSP(this, &FRigControlElementDetails::OnAddSpaceSelection, PropertyUtilities);

	constexpr const TCHAR* AddSpaceRigHierarchyViewName = TEXT("AddSpaceRigHierarchyView");

	return SNew(SBox)
		.Padding(2.5)
		.MinDesiredWidth(200)
		.MinDesiredHeight(300)
		[
			SNew(SRigHierarchyTreeView, AddSpaceRigHierarchyViewName)
			.RigTreeDelegates(RigTreeDelegates)
			.PopulateOnConstruct(true)
		];
}

FReply FRigControlElementDetails::OnAddSpaceMouseDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	AddSpaceMenuAnchor->SetIsOpen(true);
	return FReply::Handled();
}

void FRigControlElementDetails::OnAddSpaceSelection(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	if(Selection)
	{
		if (!PerElementInfos[0].GetElement())
		{
			return;
		}
		
		const FRigElementKey ChildKey = PerElementInfos[0].GetElement()->GetKey();
		const FRigElementKey NewParentKey = Selection->Key.GetElement();
		URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
		AddSpaceMenuAnchor->SetIsOpen(false);

		const bool bIsAnimationChannel = IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationChannel);
		static const FText AddSpaceText = LOCTEXT("AddSpace", "Add Space");
		static const FText AddChannelHostText = LOCTEXT("AddChannelHost", "Add Channel Host");
		FScopedTransaction Transaction(bIsAnimationChannel ? AddChannelHostText : AddSpaceText);
		HierarchyToChange->Modify();
		if(bIsAnimationChannel)
		{
			HierarchyToChange->GetController(true)->AddChannelHost(ChildKey, NewParentKey);
		}
		else
		{
			HierarchyToChange->GetController(true)->AddAvailableSpace(ChildKey, NewParentKey);
		}
		PropertyUtilities->ForceRefresh();
	}
}

void FRigControlElementDetails::HandleControlTypeChanged(TSharedPtr<ERigControlType> ControlType, ESelectInfo::Type SelectInfo, FRigElementKey ControlKey, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	HandleControlTypeChanged(*ControlType.Get(), {ControlKey}, PropertyUtilities);
}

void FRigControlElementDetails::HandleControlTypeChanged(ERigControlType ControlType, TArray<FRigElementKey> ControlKeys, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}
	
	if(ControlKeys.IsEmpty())
	{
		for(const FPerElementInfo& Info : PerElementInfos)
		{
			if (const FRigControlElement* ControlElement = Info.GetDefaultElement<FRigControlElement>())
			{
				ControlKeys.Add(ControlElement->GetKey());
			}
		}
	}

	for(const FRigElementKey& ControlKey : ControlKeys)
	{
		URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
		URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
		HierarchyToChange->Modify();
		
		FRigControlElement* ControlElement = HierarchyToChange->FindChecked<FRigControlElement>(ControlKey);
		
		FRigControlValue ValueToSet;

		ControlElement->Settings.ControlType = ControlType;
		ControlElement->Settings.LimitEnabled.Reset();
		ControlElement->Settings.bGroupWithParentControl = false;
		ControlElement->Settings.FilteredChannels.Reset();

		switch (ControlElement->Settings.ControlType)
		{
			case ERigControlType::Bool:
			{
				ControlElement->Settings.AnimationType = ERigControlAnimationType::AnimationChannel;
				ValueToSet = FRigControlValue::Make<bool>(false);
				ControlElement->Settings.bGroupWithParentControl = ControlElement->Settings.IsAnimatable();
				break;
			}
			case ERigControlType::Float:
			{
				ValueToSet = FRigControlValue::Make<float>(0.f);
				ControlElement->Settings.SetupLimitArrayForType(true);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<float>(0.f);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<float>(100.f);
				ControlElement->Settings.bGroupWithParentControl = ControlElement->Settings.IsAnimatable();
				break;
			}
			case ERigControlType::ScaleFloat:
			{
				ValueToSet = FRigControlValue::Make<float>(1.f);
				ControlElement->Settings.SetupLimitArrayForType(false);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<float>(0.f);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<float>(10.f);
				ControlElement->Settings.bGroupWithParentControl = ControlElement->Settings.IsAnimatable();
				break;
			}
			case ERigControlType::Integer:
			{
				ValueToSet = FRigControlValue::Make<int32>(0);
				ControlElement->Settings.SetupLimitArrayForType(true);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<int32>(0);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<int32>(100);
				ControlElement->Settings.bGroupWithParentControl = ControlElement->Settings.IsAnimatable();
				break;
			}
			case ERigControlType::Vector2D:
			{
				ValueToSet = FRigControlValue::Make<FVector2D>(FVector2D::ZeroVector);
				ControlElement->Settings.SetupLimitArrayForType(true);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<FVector2D>(FVector2D::ZeroVector);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<FVector2D>(FVector2D(100.f, 100.f));
				ControlElement->Settings.bGroupWithParentControl = ControlElement->Settings.IsAnimatable();
				break;
			}
			case ERigControlType::Position:
			{
				ValueToSet = FRigControlValue::Make<FVector>(FVector::ZeroVector);
				ControlElement->Settings.SetupLimitArrayForType(false);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<FVector>(-FVector::OneVector);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<FVector>(FVector::OneVector);
				break;
			}
			case ERigControlType::Scale:
			{
				ValueToSet = FRigControlValue::Make<FVector>(FVector::OneVector);
				ControlElement->Settings.SetupLimitArrayForType(false);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<FVector>(FVector::ZeroVector);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<FVector>(FVector::OneVector);
				break;
			}
			case ERigControlType::Rotator:
			{
				ValueToSet = FRigControlValue::Make<FRotator>(FRotator::ZeroRotator);
				ControlElement->Settings.SetupLimitArrayForType(false, false);
				ControlElement->Settings.MinimumValue = FRigControlValue::Make<FRotator>(FRotator::ZeroRotator);
				ControlElement->Settings.MaximumValue = FRigControlValue::Make<FRotator>(FRotator(180.f, 180.f, 180.f));
				break;
			}
			case ERigControlType::Transform:
			{
				ValueToSet = FRigControlValue::Make<FTransform>(FTransform::Identity);
				ControlElement->Settings.SetupLimitArrayForType(false, false, false);
				ControlElement->Settings.MinimumValue = ValueToSet;
				ControlElement->Settings.MaximumValue = ValueToSet;
				break;
			}
			case ERigControlType::TransformNoScale:
			{
				FTransformNoScale Identity = FTransform::Identity;
				ValueToSet = FRigControlValue::Make<FTransformNoScale>(Identity);
				ControlElement->Settings.SetupLimitArrayForType(false, false, false);
				ControlElement->Settings.MinimumValue = ValueToSet;
				ControlElement->Settings.MaximumValue = ValueToSet;
				break;
			}
			case ERigControlType::EulerTransform:
			{
				FEulerTransform Identity = FEulerTransform::Identity;
				ValueToSet = FRigControlValue::Make<FEulerTransform>(Identity);
				ControlElement->Settings.SetupLimitArrayForType(false, false, false);
				ControlElement->Settings.MinimumValue = ValueToSet;
				ControlElement->Settings.MaximumValue = ValueToSet;
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		HierarchyToChange->SetControlSettings(ControlElement, ControlElement->Settings, true, true, true);
		HierarchyToChange->SetControlValue(ControlElement, ValueToSet, ERigControlValueType::Initial, true, false, true);
		HierarchyToChange->SetControlValue(ControlElement, ValueToSet, ERigControlValueType::Current, true, false, true);

		for(const FPerElementInfo& Info : PerElementInfos)
		{
			if(Info.Element.Get()->GetKey() == ControlKey)
			{
				Info.WrapperObject->SetContent<FRigControlElement>(*ControlElement);
			}
		}

		if (HierarchyToChange != Hierarchy)
		{
			if(FRigControlElement* OtherControlElement = Hierarchy->Find<FRigControlElement>(ControlKey))
			{
				OtherControlElement->Settings = ControlElement->Settings;
				Hierarchy->SetControlSettings(OtherControlElement, OtherControlElement->Settings, true, true, true);
				Hierarchy->SetControlValue(OtherControlElement, ValueToSet, ERigControlValueType::Initial, true);
				Hierarchy->SetControlValue(OtherControlElement, ValueToSet, ERigControlValueType::Current, true);
			}
		}
		else
		{
			PerElementInfos[0].GetAsset()->PropagateHierarchyFromBPToInstances();
		}
	}
	
	PropertyUtilities->ForceRefresh();
}

void FRigControlElementDetails::CustomizeShape(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	if(ContainsElementByPredicate([](const FPerElementInfo& Info)
	{
		if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
		{
			return !ControlElement->Settings.SupportsShape();
		}
		return true;
	}))
	{
		return;
	}

	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural;

	ShapeNameList.Reset();
	
	if (FControlRigAssetInterfacePtr Blueprint = PerElementInfos[0].GetAsset())
	{
		if(UEdGraph* RootEdGraph = Blueprint->GetRigVMAssetInterface()->GetEdGraph(Blueprint->GetRigVMAssetInterface()->GetModel()))
		{
			if(UControlRigGraph* RigGraph = Cast<UControlRigGraph>(RootEdGraph))
			{
				URigHierarchy* Hierarchy = Blueprint->GetHierarchy();
				if(UControlRig* RigBeingDebugged = Cast<UControlRig>(Blueprint->GetObjectBeingDebugged()))
				{
					Hierarchy = RigBeingDebugged->GetHierarchy();
				}

				const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>* ShapeLibraries = &Blueprint->GetShapeLibraries();
				if(const UControlRig* DebuggedControlRig = Hierarchy->GetTypedOuter<UControlRig>())
				{
					ShapeLibraries = &DebuggedControlRig->GetShapeLibraries();
				}
				RigGraph->CacheNameLists(Hierarchy, &Blueprint->GetDrawContainer(), *ShapeLibraries);
				
				if(const TArray<TSharedPtr<FRigVMStringWithTag>>* GraphShapeNameListPtr = RigGraph->GetShapeNameList())
				{
					ShapeNameList = *GraphShapeNameListPtr;
				}
			}
		}

		if(ShapeNameList.IsEmpty())
		{
			const bool bUseNameSpace = Blueprint->GetShapeLibraries().Num() > 1;
			for(TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary : Blueprint->GetShapeLibraries())
			{
				if (!ShapeLibrary.IsValid())
				{
					(void)ShapeLibrary.LoadSynchronous();
				}
				if (ShapeLibrary.IsValid())
				{
					const FString NameSpace = bUseNameSpace ? ShapeLibrary->GetName() + TEXT(".") : FString();
					auto AddShapeDefinition = [&ShapeNameListRef = ShapeNameList, NameSpace](const FControlRigShapeDefinition& ShapeDefinition)
					{
						if (!ShapeDefinition.ShapeProxy.IsNull())
						{
							ShapeNameListRef.Add(MakeShared<FRigVMStringWithTag>(NameSpace + ShapeDefinition.ShapeName.ToString()));
						}
					};
					
					AddShapeDefinition(ShapeLibrary->DefaultShape);
					for (const FControlRigShapeDefinition& Shape : ShapeLibrary->Shapes)
					{
						AddShapeDefinition(Shape);
					}
				}
			}
		}
	}

	IDetailCategoryBuilder& ShapeCategory = DetailBuilder.EditCategory(TEXT("Shape"), LOCTEXT("Shape", "Shape"));

	const TSharedPtr<IPropertyHandle> SettingsHandle = DetailBuilder.GetProperty(TEXT("Settings"));

	if(!IsAnyControlNotOfAnimationType(ERigControlAnimationType::ProxyControl))
	{
		ShapeCategory.AddProperty(SettingsHandle->GetChildHandle(TEXT("ShapeVisibility")).ToSharedRef())
		.IsEnabled(bIsEnabled)
		.DisplayName(FText::FromString(TEXT("Visibility Mode")));
	}

	ShapeCategory.AddProperty(SettingsHandle->GetChildHandle(TEXT("bShapeVisible")).ToSharedRef())
	.IsEnabled(bIsEnabled)
	.DisplayName(FText::FromString(TEXT("Visible")));

	IDetailGroup& ShapePropertiesGroup = ShapeCategory.AddGroup(TEXT("Shape Properties"), LOCTEXT("ShapeProperties", "Shape Properties"));
	ShapePropertiesGroup.HeaderRow()
	.IsEnabled(bIsEnabled)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("ShapeProperties", "Shape Properties"))
		.ToolTipText(LOCTEXT("ShapePropertiesTooltip", "Customize the properties of the shape"))
	]
	.CopyAction(FUIAction(
		FExecuteAction::CreateSP(this, &FRigControlElementDetails::OnCopyShapeProperties)))
	.PasteAction(FUIAction(
		FExecuteAction::CreateSP(this, &FRigControlElementDetails::OnPasteShapeProperties),
		FCanExecuteAction::CreateLambda([bIsEnabled]() { return bIsEnabled; })));
	
	// setup shape transform
	SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FEulerTransform>::FArguments()
	.IsEnabled(bIsEnabled)
	.DisplayToggle(false)
	.DisplayRelativeWorld(false)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.PreventThrottling(true);

	TArray<FRigElementKey> Keys = GetElementKeys();
	Keys = PerElementInfos[0].GetHierarchy()->SortKeys(Keys);

	TWeakPtr<FRigControlElementDetails> WeakThisPtr = StaticCastWeakPtr<FRigControlElementDetails>(AsWeak());

	auto GetShapeTransform = [WeakThisPtr](
		const FRigElementKey& Key
		) -> FEulerTransform
	{
		if(const TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin())
		{
			if(const FPerElementInfo& Info = StrongThisPtr->FindElement(Key))
			{
				if(FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
				{
					return FEulerTransform(Info.GetHierarchy()->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal));
				}
			}
		}
		return FEulerTransform::Identity;
	};

	auto SetShapeTransform = [WeakThisPtr](
		const FRigElementKey& Key,
		const FEulerTransform& InTransform,
		bool bSetupUndo
		)
	{
		if(const TSharedPtr<FRigTransformElementDetails> StrongThisPtr = WeakThisPtr.Pin())
		{
			if(const FPerElementInfo& Info = StrongThisPtr->FindElement(Key))
			{
				if(const FRigControlElement* ControlElement = Info.GetDefaultElement<FRigControlElement>())
				{
					Info.GetDefaultHierarchy()->SetControlShapeTransform((FRigControlElement*)ControlElement, InTransform.ToFTransform(), ERigTransformType::InitialLocal, bSetupUndo, true, bSetupUndo);
				}
			}
		}
	};

	TransformWidgetArgs.OnGetNumericValue_Lambda([Keys, GetShapeTransform](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent) -> TOptional<FVector::FReal>
	{
		TOptional<FVector::FReal> FirstValue;

		for(int32 Index = 0; Index < Keys.Num(); Index++)
		{
			const FRigElementKey& Key = Keys[Index];
			FEulerTransform Xfo = GetShapeTransform(Key);

			TOptional<FVector::FReal> CurrentValue = SAdvancedTransformInputBox<FEulerTransform>::GetNumericValueFromTransform(Xfo, Component, Representation, SubComponent);
			if(!CurrentValue.IsSet())
			{
				return CurrentValue;
			}

			if(Index == 0)
			{
				FirstValue = CurrentValue;
			}
			else
			{
				if(!FMath::IsNearlyEqual(FirstValue.GetValue(), CurrentValue.GetValue()))
				{
					return TOptional<FVector::FReal>();
				}
			}
		}
		
		return FirstValue;
	});

	TransformWidgetArgs.OnNumericValueChanged_Lambda(
	[
		Keys,
		this,
		GetShapeTransform,
		SetShapeTransform
	](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal InNumericValue)
	{
		for(const FRigElementKey& Key : Keys)
		{
			FEulerTransform Transform = GetShapeTransform(Key);
			FEulerTransform PreviousTransform = Transform;
			SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(Transform, InNumericValue, Component, Representation, SubComponent);

			if(!FRigControlElementDetails::Equals(Transform, PreviousTransform))
			{
				if(!SliderTransaction.IsValid())
				{
					SliderTransaction = MakeShareable(new FScopedTransaction(NSLOCTEXT("ControlRigElementDetails", "ChangeNumericValue", "Change Numeric Value")));
					PerElementInfos[0].GetDefaultHierarchy()->Modify();
				}
				SetShapeTransform(Key, Transform, false);
			}
		}
	});

	TransformWidgetArgs.OnNumericValueCommitted_Lambda(
	[
		Keys,
		this,
		GetShapeTransform,
		SetShapeTransform
	](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal InNumericValue,
		ETextCommit::Type InCommitType)
	{
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeNumericValue", "Change Numeric Value"));
			PerElementInfos[0].GetDefaultHierarchy()->Modify();

			for(const FRigElementKey& Key : Keys)
			{
				FEulerTransform Transform = GetShapeTransform(Key);
				FEulerTransform PreviousTransform = Transform;
				SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(Transform, InNumericValue, Component, Representation, SubComponent);
				if(!FRigControlElementDetails::Equals(Transform, PreviousTransform))
				{
					SetShapeTransform(Key, Transform, true);
				}
			}
		}
		SliderTransaction.Reset();
	});

	TransformWidgetArgs.OnCopyToClipboard_Lambda([Keys, GetShapeTransform](
		ESlateTransformComponent::Type InComponent
		)
	{
		if(Keys.Num() == 0)
		{
			return;
		}

		const FRigElementKey& FirstKey = Keys[0];
		FEulerTransform Xfo = GetShapeTransform(FirstKey);

		FString Content;
		switch(InComponent)
		{
			case ESlateTransformComponent::Location:
			{
				const FVector Data = Xfo.GetLocation();
				TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				const FRotator Data = Xfo.Rotator();
				TBaseStructure<FRotator>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				const FVector Data = Xfo.GetScale3D();
				TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Max:
			default:
			{
				TBaseStructure<FEulerTransform>::Get()->ExportText(Content, &Xfo, &Xfo, nullptr, PPF_None, nullptr);
				break;
			}
		}

		if(!Content.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*Content);
		}
	});

	TransformWidgetArgs.OnPasteFromClipboard_Lambda([Keys, GetShapeTransform, SetShapeTransform, this](
		ESlateTransformComponent::Type InComponent
		)
	{
		if(Keys.Num() == 0)
		{
			return;
		}

		FString Content;
		FPlatformApplicationMisc::ClipboardPaste(Content);

		if(Content.IsEmpty())
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("PasteTransform", "Paste Transform"));
		PerElementInfos[0].GetDefaultHierarchy()->Modify();

		for(const FRigElementKey& Key : Keys)
		{
			FEulerTransform Xfo = GetShapeTransform(Key);
			{
				class FRigPasteTransformWidgetErrorPipe : public FOutputDevice
				{
				public:

					int32 NumErrors;

					FRigPasteTransformWidgetErrorPipe()
						: FOutputDevice()
						, NumErrors(0)
					{
					}

					virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
					{
						UE_LOGF(LogControlRig, Error, "Error Pasting to Widget: %ls", V);
						NumErrors++;
					}
				};

				FRigPasteTransformWidgetErrorPipe ErrorPipe;
				
				switch(InComponent)
				{
					case ESlateTransformComponent::Location:
					{
						FVector Data = Xfo.GetLocation();
						TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
						Xfo.SetLocation(Data);
						break;
					}
					case ESlateTransformComponent::Rotation:
					{
						FRotator Data = Xfo.Rotator();
						TBaseStructure<FRotator>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FRotator>::Get()->GetName(), true);
						Xfo.SetRotator(Data);
						break;
					}
					case ESlateTransformComponent::Scale:
					{
						FVector Data = Xfo.GetScale3D();
						TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
						Xfo.SetScale3D(Data);
						break;
					}
					case ESlateTransformComponent::Max:
					default:
					{
						TBaseStructure<FEulerTransform>::Get()->ImportText(*Content, &Xfo, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FEulerTransform>::Get()->GetName(), true);
						break;
					}
				}

				if(ErrorPipe.NumErrors == 0)
				{
					SetShapeTransform(Key, Xfo, true);
				}
			}
		}
	});

	TransformWidgetArgs.DiffersFromDefault_Lambda([
		Keys,
		GetShapeTransform
	](
		ESlateTransformComponent::Type InComponent) -> bool
	{
		for(const FRigElementKey& Key : Keys)
		{
			const FEulerTransform CurrentTransform = GetShapeTransform(Key);
			static const FEulerTransform DefaultTransform = FEulerTransform::Identity;

			switch(InComponent)
			{
				case ESlateTransformComponent::Location:
				{
					if(!DefaultTransform.GetLocation().Equals(CurrentTransform.GetLocation()))
					{
						return true;
					}
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					if(!DefaultTransform.Rotator().Equals(CurrentTransform.Rotator()))
					{
						return true;
					}
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					if(!DefaultTransform.GetScale3D().Equals(CurrentTransform.GetScale3D()))
					{
						return true;
					}
					break;
				}
				default: // also no component whole transform
				{
					if(!DefaultTransform.GetLocation().Equals(CurrentTransform.GetLocation()) ||
						!DefaultTransform.Rotator().Equals(CurrentTransform.Rotator()) ||
						!DefaultTransform.GetScale3D().Equals(CurrentTransform.GetScale3D()))
					{
						return true;
					}
					break;
				}
			}
		}
		return false;
	});

	TransformWidgetArgs.OnResetToDefault_Lambda([Keys, GetShapeTransform, SetShapeTransform, this](
		ESlateTransformComponent::Type InComponent)
	{
		FScopedTransaction Transaction(LOCTEXT("ResetTransformToDefault", "Reset Transform to Default"));
		PerElementInfos[0].GetDefaultHierarchy()->Modify();

		for(const FRigElementKey& Key : Keys)
		{
			FEulerTransform CurrentTransform = GetShapeTransform(Key);
			static const FEulerTransform DefaultTransform = FEulerTransform::Identity; 

			switch(InComponent)
			{
				case ESlateTransformComponent::Location:
				{
					CurrentTransform.SetLocation(DefaultTransform.GetLocation());
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					CurrentTransform.SetRotator(DefaultTransform.Rotator());
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					CurrentTransform.SetScale3D(DefaultTransform.GetScale3D());
					break;
				}
				default: // whole transform / max component
				{
					CurrentTransform = DefaultTransform;
					break;
				}
			}

			SetShapeTransform(Key, CurrentTransform, true);
		}
	});

	TArray<FRigControlElement*> ControlElements;
	Algo::Transform(PerElementInfos, ControlElements, [](const FPerElementInfo& Info)
	{
		return Info.GetElement<FRigControlElement>();
	});

	SAdvancedTransformInputBox<FEulerTransform>::ConstructGroupedTransformRows(
		ShapeCategory, 
		LOCTEXT("ShapeTransform", "Shape Transform"), 
		LOCTEXT("ShapeTransformTooltip", "The relative transform of the shape under the control"),
		TransformWidgetArgs);

	ShapeNameHandle = SettingsHandle->GetChildHandle(TEXT("ShapeName"));
	ShapePropertiesGroup.AddPropertyRow(ShapeNameHandle.ToSharedRef()).CustomWidget()
	.NameContent()
	[
		SNew(STextBlock)
		.IsEnabled(bIsEnabled)
		.Text(FText::FromString(TEXT("Shape")))
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled(this, &FRigControlElementDetails::IsShapeEnabled)
	]
	.ValueContent()
	[
		SAssignNew(ShapeNameListWidget, SControlRigShapeNameList, ControlElements, PerElementInfos[0].GetAsset())
		.OnGetNameListContent(this, &FRigControlElementDetails::GetShapeNameList)
		.IsEnabled(this, &FRigControlElementDetails::IsShapeEnabled)
	];

	ShapeColorHandle = SettingsHandle->GetChildHandle(TEXT("ShapeColor"));
	ShapePropertiesGroup.AddPropertyRow(ShapeColorHandle.ToSharedRef())
	.IsEnabled(bIsEnabled)
	.DisplayName(FText::FromString(TEXT("Color")));
}

void FRigControlElementDetails::BeginDestroy()
{
	FRigTransformElementDetails::BeginDestroy();

	if(ShapeNameListWidget.IsValid())
	{
		ShapeNameListWidget->BeginDestroy();
	}
}

void FRigControlElementDetails::RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass)
{
	FRigTransformElementDetails::RegisterSectionMappings(PropertyEditorModule, InClass);

	TSharedRef<FPropertySection> ControlSection = PropertyEditorModule.FindOrCreateSection(InClass->GetFName(), "Control", LOCTEXT("Control", "Control"));
	ControlSection->AddCategory("General");
	ControlSection->AddCategory("Control");
	ControlSection->AddCategory("Value");
	ControlSection->AddCategory("AnimationChannels");

	TSharedRef<FPropertySection> ShapeSection = PropertyEditorModule.FindOrCreateSection(InClass->GetFName(), "Shape", LOCTEXT("Shape", "Shape"));
	ShapeSection->AddCategory("General");
	ShapeSection->AddCategory("Shape");

	TSharedRef<FPropertySection> ChannelsSection = PropertyEditorModule.FindOrCreateSection(InClass->GetFName(), "Channels", LOCTEXT("Channels", "Channels"));
	ChannelsSection->AddCategory("AnimationChannels");
}

bool FRigControlElementDetails::IsShapeEnabled() const
{
	if(IsAnyElementProcedural())
	{
		 return false;
	}
	
	return ContainsElementByPredicate([](const FPerElementInfo& Info)
	{
		if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
		{
			return ControlElement->Settings.SupportsShape();
		}
		return false;
	});
}

const TArray<TSharedPtr<FRigVMStringWithTag>>& FRigControlElementDetails::GetShapeNameList() const
{
	return ShapeNameList;
}

FText FRigControlElementDetails::GetDisplayName() const
{
	FName DisplayName(NAME_None);

	for(int32 ObjectIndex = 0; ObjectIndex < PerElementInfos.Num(); ObjectIndex++)
	{
		const FPerElementInfo& Info = PerElementInfos[ObjectIndex];
		if(const FRigControlElement* ControlElement = Info.GetDefaultElement<FRigControlElement>())
		{
 			const FName ThisDisplayName =
 				(ControlElement->IsAnimationChannel()) ?
 				ControlElement->GetDisplayName() :
				ControlElement->Settings.DisplayName;

			if(ObjectIndex == 0)
			{
				DisplayName = ThisDisplayName;
			}
			else if(DisplayName != ThisDisplayName)
			{
				return ControlRigDetailsMultipleValues;
			}
		}
	}

	if(!DisplayName.IsNone())
	{
		return FText::FromName(DisplayName);
	}
	return FText();
}

void FRigControlElementDetails::SetDisplayName(const FText& InNewText, ETextCommit::Type InCommitType)
{
	for(int32 ObjectIndex = 0; ObjectIndex < PerElementInfos.Num(); ObjectIndex++)
	{
		const FPerElementInfo& Info = PerElementInfos[ObjectIndex];
		if(const FRigControlElement* ControlElement = Info.GetDefaultElement<FRigControlElement>())
		{
			SetDisplayNameForElement(InNewText, InCommitType, ControlElement->GetKey());
		}
	}
}

FText FRigControlElementDetails::GetDisplayNameForElement(const FRigElementKey& InKey) const
{
	if(PerElementInfos.IsEmpty())
	{
		return FText();
	}

	URigHierarchy* Hierarchy = PerElementInfos[0].GetDefaultHierarchy();
	const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InKey);
	if(ControlElement == nullptr)
	{
		return FText::FromName(InKey.Name);
	}

	return FText::FromName(ControlElement->GetDisplayName());
}

void FRigControlElementDetails::SetDisplayNameForElement(const FText& InNewText, ETextCommit::Type InCommitType, const FRigElementKey& InKeyToRename)
{
	if(InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	URigHierarchy* Hierarchy = PerElementInfos[0].GetDefaultHierarchy();
	const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InKeyToRename);
	if(ControlElement == nullptr)
	{
		return;
	}
	if(ControlElement->IsProcedural())
	{
		return;
	}

	const FName DisplayName = InNewText.IsEmpty() ? FName(NAME_None) : FName(*InNewText.ToString());
	const bool bRename = IsAnyControlOfAnimationType(ERigControlAnimationType::AnimationChannel);
	Hierarchy->GetController(true)->SetDisplayName(InKeyToRename, DisplayName, bRename, true, true);
}

bool FRigControlElementDetails::OnVerifyDisplayNameChanged(const FText& InText, FText& OutErrorMessage, const FRigElementKey& InKeyToRename)
{
	const FString NewName = InText.ToString();
	if (NewName.IsEmpty())
	{
		OutErrorMessage = FText::FromString(TEXT("Name is empty."));
		return false;
	}

	if(PerElementInfos.IsEmpty())
	{
		return false;
	}

	const URigHierarchy* Hierarchy = PerElementInfos[0].GetDefaultHierarchy();
	const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InKeyToRename);
	if(ControlElement == nullptr)
	{
		return false;
	}
	if(ControlElement->IsProcedural())
	{
		return false;
	}

	// make sure there is no duplicate
	if(const FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(ControlElement))
	{
		FString OutErrorString;
		if (!Hierarchy->IsDisplayNameAvailable(ParentElement->GetKey(), FRigName(NewName), &OutErrorString))
		{
			OutErrorMessage = FText::FromString(OutErrorString);
			return false;
		}
	}
	return true;
}

void FRigControlElementDetails::OnCopyShapeProperties()
{
	FString Value;

	if (!PerElementInfos.IsEmpty())
	{
		if(const FRigControlElement* ControlElement = PerElementInfos[0].GetElement<FRigControlElement>())
		{
			Value = FString::Printf(TEXT("(ShapeName=\"%s\",ShapeColor=%s)"),
				*ControlElement->Settings.ShapeName.ToString(),
				*ControlElement->Settings.ShapeColor.ToString());
		}
	}
		
	if (!Value.IsEmpty())
	{
		// Copy.
		FPlatformApplicationMisc::ClipboardCopy(*Value);
	}
}

void FRigControlElementDetails::OnPasteShapeProperties()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	
	FString TrimmedText = PastedText.LeftChop(1).RightChop(1);
	FString ShapeName;
	FString ShapeColorStr;
	bool bSuccessful = FParse::Value(*TrimmedText, TEXT("ShapeName="), ShapeName) &&
					   FParse::Value(*TrimmedText, TEXT("ShapeColor="), ShapeColorStr, false);

	if (bSuccessful)
	{
		FScopedTransaction Transaction(LOCTEXT("PasteShape", "Paste Shape"));
		
		// Name
		{
			ShapeNameHandle->NotifyPreChange();
			ShapeNameHandle->SetValue(ShapeName);
			ShapeNameHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
		
		// Color
		{
			ShapeColorHandle->NotifyPreChange();
			TArray<void*> RawDataPtrs;
			ShapeColorHandle->AccessRawData(RawDataPtrs);
			for (void* RawPtr: RawDataPtrs)
			{
				bSuccessful &= static_cast<FLinearColor*>(RawPtr)->InitFromString(ShapeColorStr);
				if (!bSuccessful)
				{
					Transaction.Cancel();
					return;
				}
			}		
			ShapeColorHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}
	}
}

FDetailWidgetRow& FRigControlElementDetails::CreateBoolValueWidgetRow(
	const TArray<FRigElementKey>& Keys,
	IDetailCategoryBuilder& CategoryBuilder,
	const FText& Label,
	const FText& Tooltip,
	ERigControlValueType ValueType,
	TAttribute<EVisibility> Visibility,
	TSharedPtr<SWidget> NameContent)
{
	const static TCHAR* TrueText = TEXT("True");
	const static TCHAR* FalseText = TEXT("False");
	
	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural || ValueType == ERigControlValueType::Current;

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
	if(ValueType == ERigControlValueType::Current)
	{
		HierarchyToChange = Hierarchy;
	}

	if(!NameContent.IsValid())
	{
		SAssignNew(NameContent, STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled(bIsEnabled);
	}

	FDetailWidgetRow& WidgetRow = CategoryBuilder.AddCustomRow(Label)
	.Visibility(Visibility)
	.NameContent()
	.MinDesiredWidth(200.f)
	.MaxDesiredWidth(800.f)
	[
		NameContent.ToSharedRef()
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked_Lambda([ValueType, Keys, Hierarchy]() -> ECheckBoxState
		{
			const bool FirstValue = Hierarchy->GetControlValue<bool>(Keys[0], ValueType);
			for(int32 Index = 1; Index < Keys.Num(); Index++)
			{
				const bool SecondValue = Hierarchy->GetControlValue<bool>(Keys[Index], ValueType);
				if(FirstValue != SecondValue)
				{
					return ECheckBoxState::Undetermined;
				}
			}
			return FirstValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnCheckStateChanged_Lambda([ValueType, Keys, HierarchyToChange](ECheckBoxState NewState)
		{
			if(NewState == ECheckBoxState::Undetermined)
			{
				return;
			}

			const bool Value = NewState == ECheckBoxState::Checked;
			FScopedTransaction Transaction(LOCTEXT("ChangeValue", "Change Value"));
			HierarchyToChange->Modify();
			for(const FRigElementKey& Key : Keys)
			{
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<bool>(Value), ValueType, true, true); 
			}
		})
		.IsEnabled(bIsEnabled)
	]
	.CopyAction(FUIAction(
	FExecuteAction::CreateLambda([ValueType, Keys, Hierarchy]()
		{
			const bool FirstValue = Hierarchy->GetControlValue<bool>(Keys[0], ValueType);
			FPlatformApplicationMisc::ClipboardCopy(FirstValue ? TrueText : FalseText);
		}),
		FCanExecuteAction())
	)
	.PasteAction(FUIAction(
		FExecuteAction::CreateLambda([ValueType, Keys, HierarchyToChange]()
		{
			FString Content;
			FPlatformApplicationMisc::ClipboardPaste(Content);

			const bool Value = FToBoolHelper::FromCStringWide(*Content);
			FScopedTransaction Transaction(LOCTEXT("ChangeValue", "Change Value"));
			HierarchyToChange->Modify();
			for(const FRigElementKey& Key : Keys)
			{
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<bool>(Value), ValueType, true, true); 
			}
		}),
		FCanExecuteAction::CreateLambda([bIsEnabled]() { return bIsEnabled; }))
	)
	.OverrideResetToDefault(FResetToDefaultOverride::Create(
		TAttribute<bool>::CreateLambda([ValueType, Keys, Hierarchy, bIsEnabled]() -> bool
		{
			if(!bIsEnabled)
			{
				return false;
			}
			
			const bool FirstValue = Hierarchy->GetControlValue<bool>(Keys[0], ValueType);
			const bool ReferenceValue = ValueType == ERigControlValueType::Initial ? false :
				Hierarchy->GetControlValue<bool>(Keys[0], ERigControlValueType::Initial);

			return FirstValue != ReferenceValue;
		}),
		FSimpleDelegate::CreateLambda([ValueType, Keys, HierarchyToChange]()
		{
			FScopedTransaction Transaction(LOCTEXT("ResetValueToDefault", "Reset Value To Default"));
			HierarchyToChange->Modify();
			for(const FRigElementKey& Key : Keys)
			{
				const bool ReferenceValue = ValueType == ERigControlValueType::Initial ? false :
					HierarchyToChange->GetControlValue<bool>(Keys[0], ERigControlValueType::Initial);
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<bool>(ReferenceValue), ValueType, true, true); 
			}
		})
	));

	return WidgetRow;
}

FDetailWidgetRow& FRigControlElementDetails::CreateFloatValueWidgetRow(
	const TArray<FRigElementKey>& Keys,
	IDetailCategoryBuilder& CategoryBuilder,
	const FText& Label,
	const FText& Tooltip,
	ERigControlValueType ValueType,
	TAttribute<EVisibility> Visibility,
	TSharedPtr<SWidget> NameContent)
{
	return CreateNumericValueWidgetRow<float>(Keys, CategoryBuilder, Label, Tooltip, ValueType, Visibility, NameContent);
}

FDetailWidgetRow& FRigControlElementDetails::CreateIntegerValueWidgetRow(
	const TArray<FRigElementKey>& Keys,
	IDetailCategoryBuilder& CategoryBuilder,
	const FText& Label,
	const FText& Tooltip,
	ERigControlValueType ValueType,
	TAttribute<EVisibility> Visibility,
	TSharedPtr<SWidget> NameContent)
{
	return CreateNumericValueWidgetRow<int32>(Keys, CategoryBuilder, Label, Tooltip, ValueType, Visibility, NameContent);
}

FDetailWidgetRow& FRigControlElementDetails::CreateEnumValueWidgetRow(
	const TArray<FRigElementKey>& Keys,
	IDetailCategoryBuilder& CategoryBuilder,
	const FText& Label,
	const FText& Tooltip,
	ERigControlValueType ValueType,
	TAttribute<EVisibility> Visibility,
	TSharedPtr<SWidget> NameContent)
{
	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural || ValueType == ERigControlValueType::Current;

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
	if(ValueType == ERigControlValueType::Current)
	{
		HierarchyToChange = Hierarchy;
	}

	UEnum* Enum = nullptr;
	for(const FRigElementKey& Key : Keys)
	{
		if(const FPerElementInfo& Info = FindElement(Key))
		{
			if(const FRigControlElement* ControlElement = Info.GetElement<FRigControlElement>())
			{
				Enum = ControlElement->Settings.ControlEnum.Get();
				if(Enum)
				{
					break;
				}
			}
		}
		else
		{
			// If the key was not found for selected elements, it might be a child channel of one of the elements
			for (const FPerElementInfo& ElementInfo : PerElementInfos)
			{
				if (const FRigControlElement* ControlElement = ElementInfo.GetElement<FRigControlElement>())
				{
					const FRigBaseElementChildrenArray Children = Hierarchy->GetChildren(ControlElement, false);
					if (FRigBaseElement* const* Child = Children.FindByPredicate([Key](const FRigBaseElement* Info)
					{
						return Info->GetKey() == Key;
					}))
					{
						if (const FRigControlElement* ChildElement = Cast<FRigControlElement>(*Child))
						{
							Enum = ChildElement->Settings.ControlEnum.Get();
							if(Enum)
							{
								break;
							}
						}
					}
				}
			}
			if(Enum)
			{
				break;
			}
		}
	}

	check(Enum != nullptr);

	if(!NameContent.IsValid())
	{
		SAssignNew(NameContent, STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled(bIsEnabled);
	}

	FDetailWidgetRow& WidgetRow = CategoryBuilder.AddCustomRow(Label)
	.Visibility(Visibility)
	.NameContent()
	.MinDesiredWidth(200.f)
	.MaxDesiredWidth(800.f)
	[
		NameContent.ToSharedRef()
	]
	.ValueContent()
	[
		SNew(SEnumComboBox, Enum)
		.CurrentValue_Lambda([ValueType, Keys, Hierarchy]() -> int32
		{
			const int32 FirstValue = Hierarchy->GetControlValue<int32>(Keys[0], ValueType);
			for(int32 Index = 1; Index < Keys.Num(); Index++)
			{
				const int32 SecondValue = Hierarchy->GetControlValue<int32>(Keys[Index], ValueType);
				if(FirstValue != SecondValue)
				{
					return INDEX_NONE;
				}
			}
			return FirstValue;
		})
		.OnEnumSelectionChanged_Lambda([ValueType, Keys, HierarchyToChange](int32 NewSelection, ESelectInfo::Type)
		{
			if(NewSelection == INDEX_NONE)
			{
				return;
			}

			FScopedTransaction Transaction(LOCTEXT("ChangeValue", "Change Value"));
			HierarchyToChange->Modify();
			for(const FRigElementKey& Key : Keys)
			{
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<int32>(NewSelection), ValueType, true, true); 
			}
		})
		.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
		.IsEnabled(bIsEnabled)
	]
	.CopyAction(FUIAction(
	FExecuteAction::CreateLambda([ValueType, Keys, Hierarchy]()
		{
			const int32 FirstValue = Hierarchy->GetControlValue<int32>(Keys[0], ValueType);
			FPlatformApplicationMisc::ClipboardCopy(*FString::FromInt(FirstValue));
		}),
		FCanExecuteAction())
	)
	.PasteAction(FUIAction(
		FExecuteAction::CreateLambda([ValueType, Keys, HierarchyToChange]()
		{
			FString Content;
			FPlatformApplicationMisc::ClipboardPaste(Content);
			if(!Content.IsNumeric())
			{
				return;
			}

			const int32 Value = FCString::Atoi(*Content);
			FScopedTransaction Transaction(LOCTEXT("ChangeValue", "Change Value"));
			HierarchyToChange->Modify();

			for(const FRigElementKey& Key : Keys)
			{
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<int32>(Value), ValueType, true, true); 
			}
		}),
		FCanExecuteAction::CreateLambda([bIsEnabled]() { return bIsEnabled; }))
	)
	.OverrideResetToDefault(FResetToDefaultOverride::Create(
		TAttribute<bool>::CreateLambda([ValueType, Keys, Hierarchy, bIsEnabled]() -> bool
		{
			if(!bIsEnabled)
			{
				return false;
			}
			
			const int32 FirstValue = Hierarchy->GetControlValue<int32>(Keys[0], ValueType);
			const int32 ReferenceValue = ValueType == ERigControlValueType::Initial ? 0 :
				Hierarchy->GetControlValue<int32>(Keys[0], ERigControlValueType::Initial);

			return FirstValue != ReferenceValue;
		}),
		FSimpleDelegate::CreateLambda([ValueType, Keys, HierarchyToChange]()
		{
			FScopedTransaction Transaction(LOCTEXT("ResetValueToDefault", "Reset Value To Default"));
			HierarchyToChange->Modify();
			for(const FRigElementKey& Key : Keys)
			{
				const int32 ReferenceValue = ValueType == ERigControlValueType::Initial ? 0 :
					HierarchyToChange->GetControlValue<int32>(Keys[0], ERigControlValueType::Initial);
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<int32>(ReferenceValue), ValueType, true, true); 
			}
		})
	));

	return WidgetRow;
}

FDetailWidgetRow& FRigControlElementDetails::CreateVector2DValueWidgetRow(
	const TArray<FRigElementKey>& Keys,
	IDetailCategoryBuilder& CategoryBuilder, 
	const FText& Label, 
	const FText& Tooltip, 
	ERigControlValueType ValueType,
	TAttribute<EVisibility> Visibility,
	TSharedPtr<SWidget> NameContent)
{
	const bool bIsProcedural = IsAnyElementProcedural();
	const bool bIsEnabled = !bIsProcedural || ValueType == ERigControlValueType::Current;
	const bool bShowToggle = (ValueType == ERigControlValueType::Minimum) || (ValueType == ERigControlValueType::Maximum);
	
	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
	if(ValueType == ERigControlValueType::Current)
	{
		HierarchyToChange = Hierarchy;
	}

	using SNumericVector2DInputBox = SNumericVectorInputBox<float, FVector2f, 2>;
	TSharedPtr<SNumericVector2DInputBox> VectorInputBox;
	
	FDetailWidgetRow& WidgetRow = CategoryBuilder.AddCustomRow(Label);
	TAttribute<ECheckBoxState> ToggleXChecked, ToggleYChecked;
	FOnCheckStateChanged OnToggleXChanged, OnToggleYChanged;

	if(bShowToggle)
	{
		auto ToggleChecked = [ValueType, Keys, Hierarchy](int32 Index) -> ECheckBoxState
		{
			TOptional<bool> FirstValue;

			for(const FRigElementKey& Key : Keys)
			{
				if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
				{
					if(ControlElement->Settings.LimitEnabled.Num() == 2)
					{
						const bool Value = ControlElement->Settings.LimitEnabled[Index].GetForValueType(ValueType);
						if(FirstValue.IsSet())
						{
							if(FirstValue.GetValue() != Value)
							{
								return ECheckBoxState::Undetermined;
							}
						}
						else
						{
							FirstValue = Value;
						}
					}
				}
			}

			if(!ensure(FirstValue.IsSet()))
			{
				return ECheckBoxState::Undetermined;
			}

			return FirstValue.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		};
		
		ToggleXChecked = TAttribute<ECheckBoxState>::CreateLambda([ToggleChecked]() -> ECheckBoxState
		{
			return ToggleChecked(0);
		});

		ToggleYChecked = TAttribute<ECheckBoxState>::CreateLambda([ToggleChecked]() -> ECheckBoxState
		{
			return ToggleChecked(1);
		});

		auto OnToggleChanged = [ValueType, Keys, HierarchyToChange](ECheckBoxState InValue, int32 Index)
		{
			if(InValue == ECheckBoxState::Undetermined)
			{
				return;
			}
					
			FScopedTransaction Transaction(LOCTEXT("ChangeLimitToggle", "Change Limit Toggle"));
			HierarchyToChange->Modify();

			for(const FRigElementKey& Key : Keys)
			{
				if(FRigControlElement* ControlElement = HierarchyToChange->Find<FRigControlElement>(Key))
				{
					if(ControlElement->Settings.LimitEnabled.Num() == 2)
					{
						ControlElement->Settings.LimitEnabled[Index].SetForValueType(ValueType, InValue == ECheckBoxState::Checked);
						HierarchyToChange->SetControlSettings(ControlElement, ControlElement->Settings, true, true, true);
					}
				}
			}
		};

		OnToggleXChanged = FOnCheckStateChanged::CreateLambda([OnToggleChanged](ECheckBoxState InValue)
		{
			OnToggleChanged(InValue, 0);
		});

		OnToggleYChanged = FOnCheckStateChanged::CreateLambda([OnToggleChanged](ECheckBoxState InValue)
		{
			OnToggleChanged(InValue, 1);
		});
	}

	auto GetValue = [ValueType, Keys, Hierarchy](int32 Component) -> TOptional<float>
	{
		const float FirstValue = Hierarchy->GetControlValue<FVector3f>(Keys[0], ValueType).Component(Component);
		for(int32 Index = 1; Index < Keys.Num(); Index++)
		{
			const float SecondValue = Hierarchy->GetControlValue<FVector3f>(Keys[Index], ValueType).Component(Component);
			if(FirstValue != SecondValue)
			{
				return TOptional<float>();
			}
		}
		return FirstValue;
	};

	auto OnValueChanged = [ValueType, Keys, Hierarchy, HierarchyToChange, this]
		(TOptional<float> InValue, ETextCommit::Type InCommitType, bool bSetupUndo, int32 Component)
		{
			if(!InValue.IsSet())
			{
				return;
			}

			const float Value = InValue.GetValue();
		
			for(const FRigElementKey& Key : Keys)
			{
				FVector3f Vector = Hierarchy->GetControlValue<FVector3f>(Key, ValueType);
				if(!FMath::IsNearlyEqual(Vector.Component(Component), Value))
				{
					if(!SliderTransaction.IsValid())
					{
						SliderTransaction = MakeShareable(new FScopedTransaction(NSLOCTEXT("ControlRigElementDetails", "ChangeValue", "Change Value")));
						HierarchyToChange->Modify();
					}
					Vector.Component(Component) = Value;
					HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<FVector3f>(Vector), ValueType, bSetupUndo, bSetupUndo);
				};
			}

			if(bSetupUndo)
			{
				SliderTransaction.Reset();
			}
	};

	if(!NameContent.IsValid())
	{
		SAssignNew(NameContent, STextBlock)
		.Text(Label)
		.ToolTipText(Tooltip)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.IsEnabled(bIsEnabled);
	}

	WidgetRow
	.Visibility(Visibility)
	.NameContent()
	.MinDesiredWidth(200.f)
	.MaxDesiredWidth(800.f)
	[
		NameContent.ToSharedRef()
	]
	.ValueContent()
	[
		SAssignNew(VectorInputBox, SNumericVector2DInputBox)
        .Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
        .AllowSpin(ValueType == ERigControlValueType::Current || ValueType == ERigControlValueType::Initial)
		.SpinDelta(0.01f)
		.X_Lambda([GetValue]() -> TOptional<float>
        {
			return GetValue(0);
        })
        .Y_Lambda([GetValue]() -> TOptional<float>
		{
			return GetValue(1);
		})
		.OnXChanged_Lambda([OnValueChanged](TOptional<float> InValue)
		{
			OnValueChanged(InValue, ETextCommit::Default, false, 0);
		})
		.OnYChanged_Lambda([OnValueChanged](TOptional<float> InValue)
		{
			OnValueChanged(InValue, ETextCommit::Default, false, 1);
		})
		.OnXCommitted_Lambda([OnValueChanged](TOptional<float> InValue, ETextCommit::Type InCommitType)
		{
			OnValueChanged(InValue, InCommitType, true, 0);
		})
		.OnYCommitted_Lambda([OnValueChanged](TOptional<float> InValue, ETextCommit::Type InCommitType)
		{
			OnValueChanged(InValue, InCommitType, true, 1);
		})
		 .DisplayToggle(bShowToggle)
		 .ToggleXChecked(ToggleXChecked)
		 .ToggleYChecked(ToggleYChecked)
		 .OnToggleXChanged(OnToggleXChanged)
		 .OnToggleYChanged(OnToggleYChanged)
		 .IsEnabled(bIsEnabled)
		 .PreventThrottling(true)
	]
	.CopyAction(FUIAction(
	FExecuteAction::CreateLambda([ValueType, Keys, Hierarchy]()
		{
			const FVector3f Data3 = Hierarchy->GetControlValue<FVector3f>(Keys[0], ValueType);
			const FVector2f Data(Data3.X, Data3.Y);
			FString Content = Data.ToString();
			FPlatformApplicationMisc::ClipboardCopy(*Content);
		}),
		FCanExecuteAction())
	)
	.PasteAction(FUIAction(
		FExecuteAction::CreateLambda([ValueType, Keys, HierarchyToChange]()
		{
			FString Content;
			FPlatformApplicationMisc::ClipboardPaste(Content);
			if(Content.IsEmpty())
			{
				return;
			}

			FVector2f Data = FVector2f::ZeroVector;
			Data.InitFromString(Content);

			FVector3f Data3(Data.X, Data.Y, 0);

			FScopedTransaction Transaction(NSLOCTEXT("ControlRigElementDetails", "ChangeValue", "Change Value"));
			HierarchyToChange->Modify();
			
			for(const FRigElementKey& Key : Keys)
			{
				HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<FVector3f>(Data3), ValueType, true, true); 
			}
		}),
		FCanExecuteAction::CreateLambda([bIsEnabled]() { return bIsEnabled; }))
	);

	if((ValueType == ERigControlValueType::Current || ValueType == ERigControlValueType::Initial) && bIsEnabled)
	{
		WidgetRow.OverrideResetToDefault(FResetToDefaultOverride::Create(
			TAttribute<bool>::CreateLambda([ValueType, Keys, Hierarchy]() -> bool
			{
				const FVector3f FirstValue = Hierarchy->GetControlValue<FVector3f>(Keys[0], ValueType);
				const FVector3f ReferenceValue = ValueType == ERigControlValueType::Initial ? FVector3f::ZeroVector :
					Hierarchy->GetControlValue<FVector3f>(Keys[0], ERigControlValueType::Initial);

				return !(FirstValue - ReferenceValue).IsNearlyZero();
			}),
			FSimpleDelegate::CreateLambda([ValueType, Keys, HierarchyToChange]()
			{
				FScopedTransaction Transaction(LOCTEXT("ResetValueToDefault", "Reset Value To Default"));
				HierarchyToChange->Modify();
				
				for(const FRigElementKey& Key : Keys)
				{
					const FVector3f ReferenceValue = ValueType == ERigControlValueType::Initial ? FVector3f::ZeroVector :
						HierarchyToChange->GetControlValue<FVector3f>(Keys[0], ERigControlValueType::Initial);
					HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<FVector3f>(ReferenceValue), ValueType, true, true); 
				}
			})
		));
	}

	return WidgetRow;
}

void FRigNullElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FRigTransformElementDetails::CustomizeDetails(DetailBuilder);
	CustomizeTransform(DetailBuilder);
	CustomizeComponents(DetailBuilder);
	CustomizeMetadata(DetailBuilder);
}

void FRigConnectorElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FRigTransformElementDetails::CustomizeDetails(DetailBuilder);

	PropertyUtils = DetailBuilder.GetPropertyUtilities();

	CustomizeSettings(DetailBuilder);
	CustomizeComponents(DetailBuilder);
	CustomizeConnectorTargets(DetailBuilder);
	CustomizeMetadata(DetailBuilder);
}

void FRigConnectorElementDetails::CustomizeSettings(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	if(IsAnyElementNotOfType(ERigElementType::Connector))
	{
		return;
	}

	const TSharedPtr<IPropertyHandle> SettingsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FRigConnectorElement, Settings));
	DetailBuilder.HideProperty(SettingsHandle);

	IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"), LOCTEXT("Settings", "Settings"));

	ConnectorTypeHandle = SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigConnectorSettings, Type));
	SettingsCategory
		.AddProperty(ConnectorTypeHandle)
		.IsEnabled(false);

	SettingsCategory
		.AddProperty(SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigConnectorSettings, bOptional)))
		.Visibility(IsAnyConnectorPrimary() ? EVisibility::Collapsed : EVisibility::Visible)
		.IsEnabled(!IsAnyConnectorImported());

	bool bHideRules = false;
	uint32 FirstHash = UINT32_MAX;
	for (const FPerElementInfo& Info : PerElementInfos)
	{
		if (URigHierarchy* Hierarchy = Info.IsValid() ? Info.GetHierarchy() : nullptr)
		{
			if(const FRigConnectorElement* Connector = Info.GetElement<FRigConnectorElement>())
			{
				const uint32 Hash = Connector->Settings.GetRulesHash();
				if(FirstHash == UINT32_MAX)
				{
					FirstHash = Hash;
				}
				else if(FirstHash != Hash)
				{
					bHideRules = true;
					break;
				}
			}
			else
			{
				bHideRules = true;
			}
		}
	}

	IsArrayHandle = SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigConnectorSettings, bIsArray));
	IDetailPropertyRow& IsArrayPropertyRow = SettingsCategory
		.AddProperty(IsArrayHandle)
		.IsEnabled(CanBeArray());

	if(!CanBeArray())
	{
		IsArrayPropertyRow.ToolTip(LOCTEXT("PrimaryConnectorsDontAllowArrayToolTip", "Primary Connectors don't support arrays. Add a secondary connector for that."));
	}

	IsPostConstructionHandle = SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigConnectorSettings, bPostConstruction));
	IDetailPropertyRow& IsPostConstructionPropertyRow = SettingsCategory
		.AddProperty(IsPostConstructionHandle)
		.IsEnabled(CanBeArray());

	if(!CanBeArray())
	{
		IsPostConstructionPropertyRow.ToolTip(LOCTEXT("PrimaryConnectorsDontAllowPostConstructionToolTip", "Primary Connectors cannot be used during post construction."));
	}
	else
	{
		IsPostConstructionPropertyRow.ToolTip(LOCTEXT("PostConstructionConnectorToolTip", "This connector will be available only during the Post Construction Event.\nYou can connect any element spawned during Construction from any module to it."));
	}
	
	if(!bHideRules)
	{
		SettingsCategory
			.AddProperty(SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigConnectorSettings, Rules)))
			.IsEnabled(!IsAnyConnectorImported());
	}

	SettingsCategory
		.AddProperty(SettingsHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigConnectorSettings, Description)))
		.IsEnabled(!IsAnyConnectorImported());
}

void FRigConnectorElementDetails::CustomizeConnectorTargets(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.Num() != 1)
	{
		return;
	}

	URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
	check(Hierarchy);

	FRigHierarchyTreeDelegates RigTreeDelegates;
	RigTreeDelegates.OnGetHierarchy.BindLambda([Hierarchy]()
	{
		return Hierarchy;
	});

	TArray<FRigElementKey> CurrentTargets;
	if(UControlRig* ControlRig = Hierarchy->GetTypedOuter<UControlRig>())
	{
		if(const FRigConnectorElement* Connector = PerElementInfos[0].Element.Get<FRigConnectorElement>())
		{
			if (const UModularRig* ModularRig = Cast<UModularRig>(ControlRig))
			{
				CurrentTargets = ModularRig->GetModularRigModel().Connections.FindTargetsFromConnector(Connector->GetKey());
			}
			else
			{
				const FRigElementKeyRedirector& Redirector = ControlRig->GetElementKeyRedirector();
				const FRigElementKeyRedirector::FCachedKeyArray* Cache = Redirector.Find(Connector->GetKey());
				if(Cache && !Cache->IsEmpty())
				{
					for(int32 Index = 0; Index < Cache->Num(); Index++)
					{
						CurrentTargets.Add((*Cache)[Index].GetKey());
					}
				}
			}
		}
	}

	bool bArrayEnabled = false;
	if (ensureMsgf(IsArrayHandle.IsValid(), TEXT("Unexpected invalid IsArrayHandle, call FRigConnectorElementDetails::CustomizeSettings first")) &&
		ensureMsgf(IsArrayHandle->GetValue(bArrayEnabled) == FPropertyAccess::Success, TEXT("Unexpected invalid property handle when trying to access the IsArrayHandle property. Assuming not an array")))
	{
		// Show an invalid key if there is no key. This way the user can add a first target and UE-357788 can be worked around temporarily
		if (CurrentTargets.IsEmpty())
		{
			CurrentTargets.Add(FRigElementKey());
		}

		IsArrayHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRigConnectorElementDetails::RequestRefresh));
	}

	IDetailCategoryBuilder& TargetsCategory = DetailBuilder.EditCategory(TEXT("Targets"), LOCTEXT("Targets", "Targets"));
	FDetailWidgetRow& TargetsRow = TargetsCategory.AddCustomRow(LOCTEXT("Targets", "Targets"));
	
	TSharedRef<SRigConnectorTargetWidget> ConnectorTargetWidget = SNew(SRigConnectorTargetWidget)
	.Outer(PerElementInfos[0].WrapperObject.Get())
	.ConnectorKey(PerElementInfos[0].Element.GetKey())
	.IsArray(bArrayEnabled)
	.ExpandArrayByDefault(true)
	.Targets(CurrentTargets)
	.OnSetTargetArray(FRigConnectorTargetWidget_SetTargetArray::CreateSP(this, &FRigConnectorElementDetails::OnTargetsChanged))
	.RigTreeDelegates(RigTreeDelegates);

	if(bArrayEnabled)
	{
		TargetsRow.WholeRowContent()
		[
			ConnectorTargetWidget
		];
	}
	else
	{
		TargetsRow
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Target", "Target"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			ConnectorTargetWidget
		];
	}
}

TOptional<EConnectorType> FRigConnectorElementDetails::GetConnectorType() const
{
	TOptional<EConnectorType> Result;
	for (const FPerElementInfo& Info : PerElementInfos)
	{
		if(const FRigConnectorElement* Connector = Info.GetElement<FRigConnectorElement>())
		{
			if(!Result.IsSet())
			{
				Result = Connector->Settings.Type;
			}
			else if(Result.GetValue() != Connector->Settings.Type)
			{
				return {};
			}
		}
	}
	return Result;
}

TOptional<bool> FRigConnectorElementDetails::GetIsConnectorArray() const
{
	TOptional<bool> Result;
	for (const FPerElementInfo& Info : PerElementInfos)
	{
		if(const FRigConnectorElement* Connector = Info.GetElement<FRigConnectorElement>())
		{
			if(!Result.IsSet())
			{
				Result = Connector->IsArrayConnector();
			}
			else if(Result.GetValue() != Connector->IsArrayConnector())
			{
				return {};
			}
		}
	}
	return Result;
}

bool FRigConnectorElementDetails::CanBeArray() const
{
	const TOptional<EConnectorType> ConnectorType = GetConnectorType();
	if(ConnectorType.Get(EConnectorType::Primary) != EConnectorType::Primary)
	{
		return !IsAnyConnectorImported();
	}
	return false;
}

bool FRigConnectorElementDetails::OnTargetsChanged(TArray<FRigElementKey> InTargets)
{
	if(PerElementInfos.Num() == 1)
	{
		if(FControlRigAssetInterfacePtr ControlRigBlueprint = PerElementInfos[0].GetAsset())
		{
			return ControlRigBlueprint->ResolveConnectorToArray(PerElementInfos[0].Element.GetKey(), InTargets, true);
		}
	}
	return false;
}

void FRigConnectorElementDetails::RequestRefresh()
{
	if (PropertyUtils.IsValid())
	{
		PropertyUtils->RequestRefresh();
	}
}

void FRigSocketElementDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FRigTransformElementDetails::CustomizeDetails(DetailBuilder);
	CustomizeSettings(DetailBuilder);
	CustomizeTransform(DetailBuilder);
	CustomizeComponents(DetailBuilder);
	CustomizeMetadata(DetailBuilder);
}

void FRigSocketElementDetails::CustomizeSettings(IDetailLayoutBuilder& DetailBuilder)
{
	if(PerElementInfos.IsEmpty())
	{
		return;
	}

	if(IsAnyElementNotOfType(ERigElementType::Socket))
	{
		return;
	}

	const bool bIsProcedural = IsAnyElementProcedural();
	
	IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"), LOCTEXT("Settings", "Settings"));

	SettingsCategory.AddCustomRow(FText::FromString(TEXT("Color")))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Color", "Color"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SColorBlock)
		.IsEnabled(!bIsProcedural)
		//.Size(FVector2D(6.0, 38.0))
		.Color(this, &FRigSocketElementDetails::GetSocketColor) 
		.OnMouseButtonDown(this, &FRigSocketElementDetails::SetSocketColor)
	];

	SettingsCategory.AddCustomRow(FText::FromString(TEXT("Description")))
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("Description", "Description"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SEditableText)
		.IsEnabled(!bIsProcedural)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(this, &FRigSocketElementDetails::GetSocketDescription)
		.OnTextCommitted(this, &FRigSocketElementDetails::SetSocketDescription)
	];
}

FReply FRigSocketElementDetails::SetSocketColor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.InitialColor = GetSocketColor();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FRigSocketElementDetails::OnSocketColorPicked);
	OpenColorPicker(PickerArgs);
	return FReply::Handled();
}

FLinearColor FRigSocketElementDetails::GetSocketColor() const 
{
	if(PerElementInfos.Num() > 1)
	{
		return FRigSocketElement::SocketDefaultColor;
	}
	URigHierarchy* Hierarchy = PerElementInfos[0].GetDefaultHierarchy();
	const FRigSocketElement* Socket = PerElementInfos[0].GetDefaultElement<FRigSocketElement>();
	return Socket->GetColor(Hierarchy);
}

void FRigSocketElementDetails::OnSocketColorPicked(FLinearColor NewColor)
{
	FScopedTransaction Transaction(LOCTEXT("SocketColorChanged", "Socket Color Changed"));
	for(FPerElementInfo& Info : PerElementInfos)
	{
		URigHierarchy* Hierarchy = Info.GetDefaultHierarchy();
		Hierarchy->Modify();
		FRigSocketElement* Socket = Info.GetDefaultElement<FRigSocketElement>();
		Socket->SetColor(NewColor, Hierarchy);
	}
}

void FRigSocketElementDetails::SetSocketDescription(const FText& InDescription, ETextCommit::Type InCommitType)
{
	const FString Description = InDescription.ToString();
	for(FPerElementInfo& Info : PerElementInfos)
	{
		URigHierarchy* Hierarchy = Info.GetDefaultHierarchy();
		Hierarchy->Modify();
		FRigSocketElement* Socket = Info.GetDefaultElement<FRigSocketElement>();
		Socket->SetDescription(Description, Hierarchy);
	}
}

FText FRigSocketElementDetails::GetSocketDescription() const
{
	FString FirstValue;
	for(int32 Index = 0; Index < PerElementInfos.Num(); Index++)
	{
		URigHierarchy* Hierarchy = PerElementInfos[Index].GetDefaultHierarchy();
		const FRigSocketElement* Socket = PerElementInfos[Index].GetDefaultElement<FRigSocketElement>();
		const FString Description = Socket->GetDescription(Hierarchy);
		if(Index == 0)
		{
			FirstValue = Description;
		}
		else if(!FirstValue.Equals(Description, ESearchCase::CaseSensitive))
		{
			return ControlRigDetailsMultipleValues;
		}
	}
	return FText::FromString(FirstValue);
}

void FRigConnectionRuleDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle.ToSharedPtr();
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();
	BlueprintBeingCustomized = nullptr;
	EnabledAttribute = false;
	RigElementKeyDetails_GetCustomizedInfo(InStructPropertyHandle, BlueprintBeingCustomized);

	TArray<UObject*> Objects;
	StructPropertyHandle->GetOuterObjects(Objects);
	FString FirstObjectValue;
	for (int32 Index = 0; Index < Objects.Num(); Index++)
	{
		const UObject* Object = Objects[Index];
		if (!Object)
		{
			// Keep, for some reason these can go invalid in certain cases
			continue;
		}

		FString ObjectValue;
		if(InStructPropertyHandle->GetPerObjectValue(Index, ObjectValue) == FPropertyAccess::Result::Success)
		{
			if(FirstObjectValue.IsEmpty())
			{
				FirstObjectValue = ObjectValue;
			}
			else
			{
				if(!FirstObjectValue.Equals(ObjectValue, ESearchCase::CaseSensitive))
				{
					FirstObjectValue.Reset();
					break;
				}
			}
		}

		// only enable editing of the rule if the widget is nested under a wrapper object (a rig element)
		if(Object->IsA<URigVMDetailsViewWrapperObject>())
		{
			EnabledAttribute = true;
		}
	}

	if(!FirstObjectValue.IsEmpty())
	{
		FRigConnectionRuleStash::StaticStruct()->ImportText(*FirstObjectValue, &RuleStash, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigConnectionRuleStash::StaticStruct()->GetName(), true);
	}

	if (BlueprintBeingCustomized == nullptr || FirstObjectValue.IsEmpty())
	{
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			StructPropertyHandle->CreatePropertyValueWidget()
		];
	}
	else
	{
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.ContentPadding(FMargin(2,2,2,1))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FRigConnectionRuleDetails::OnGetStructTextValue)
			]
			.OnGetMenuContent(this, &FRigConnectionRuleDetails::GenerateStructPicker)
			.IsEnabled(EnabledAttribute)
		];
	}
}

void FRigConnectionRuleDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const UScriptStruct* ScriptStruct = RuleStash.GetScriptStruct();
	if(ScriptStruct == nullptr)
	{
		return;
	}

	(void)RuleStash.Get(Storage);
	const TSharedRef<FStructOnScope> StorageRef = Storage.ToSharedRef();
	const FSimpleDelegate OnPropertyChanged = FSimpleDelegate::CreateSP(this, &FRigConnectionRuleDetails::OnRuleContentChanged);

	TArray<TSharedPtr<IPropertyHandle>> ChildProperties = InStructPropertyHandle->AddChildStructure(StorageRef);
	for (TSharedPtr<IPropertyHandle> ChildHandle : ChildProperties)
	{
		ChildHandle->SetOnPropertyValueChanged(OnPropertyChanged);
		IDetailPropertyRow& ChildRow = StructBuilder.AddProperty(ChildHandle.ToSharedRef());
		ChildRow.IsEnabled(EnabledAttribute);
	}
}

TSharedRef<SWidget> FRigConnectionRuleDetails::GenerateStructPicker()
{
	FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

	class FRigConnectionRuleFilter : public IStructViewerFilter
	{
	public:
		FRigConnectionRuleFilter(const TSharedRef<IPropertyHandle>& InStructPropertyHandle)
		{
			const FName PropertyName = InStructPropertyHandle->GetProperty()->GetFName();

			if (PropertyName == GET_MEMBER_NAME_CHECKED(FRigAndConnectionRule, ChildRules) ||
				PropertyName == GET_MEMBER_NAME_CHECKED(FRigOrConnectionRule, ChildRules))
			{
				// The array size rule is not forcing, hence doesn't work with AND / OR logic
				ExcludedStructs.Add(FRigArraySizeConnectionRule::StaticStruct());
			}
		}

		virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			static const UScriptStruct* BaseStruct = FRigConnectionRule::StaticStruct();
			return 
				InStruct != BaseStruct && 
				InStruct->IsChildOf(BaseStruct) &&
				!ExcludedStructs.Contains(InStruct);
		}

		virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}

	private:
		TSet<UScriptStruct*> ExcludedStructs;
	};
		
	const TSharedRef<FRigConnectionRuleFilter> Filter = MakeShared<FRigConnectionRuleFilter>(StructPropertyHandle.ToSharedRef());
	FStructViewerInitializationOptions Options;
	{
		Options.StructFilter = Filter;
		Options.Mode = EStructViewerMode::StructPicker;
		Options.DisplayMode = EStructViewerDisplayMode::ListView;
		Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
		Options.bShowNoneOption = false;
		Options.bShowUnloadedStructs = false;
		Options.bAllowViewOptions = false;
	}

	return
		SNew(SBox)
		.WidthOverride(330.0f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.MaxHeight(500)
			[
				SNew(SBorder)
				.Padding(4)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateSP(this, &FRigConnectionRuleDetails::OnPickedStruct))
				]
			]
		];
}

void FRigConnectionRuleDetails::OnPickedStruct(const UScriptStruct* ChosenStruct)
{
	if(ChosenStruct == nullptr)
	{
		RuleStash = FRigConnectionRuleStash();
	}
	else
	{
		RuleStash.ScriptStructPath = ChosenStruct->GetPathName();
		RuleStash.ExportedText = TEXT("()");
		Storage.Reset();
		RuleStash.Get(Storage);
	}
	OnRuleContentChanged();
}

FText FRigConnectionRuleDetails::OnGetStructTextValue() const
{
	const UScriptStruct* ScriptStruct = RuleStash.GetScriptStruct();
	return ScriptStruct
		? FText::AsCultureInvariant(ScriptStruct->GetDisplayNameText())
		: LOCTEXT("None", "None");
}

void FRigConnectionRuleDetails::OnRuleContentChanged()
{
	const UScriptStruct* ScriptStruct = RuleStash.GetScriptStruct();
	if(Storage && Storage->GetStruct() == ScriptStruct)
	{
		RuleStash.ExportedText.Reset();
		const uint8* StructMemory = Storage->GetStructMemory();
		ScriptStruct->ExportText(RuleStash.ExportedText, StructMemory, StructMemory, nullptr, PPF_None, nullptr);
	}
	
	FString Content;
	FRigConnectionRuleStash::StaticStruct()->ExportText(Content, &RuleStash, &RuleStash, nullptr, PPF_None, nullptr);

	TArray<UObject*> Objects;
	StructPropertyHandle->GetOuterObjects(Objects);
	FString FirstObjectValue;
	for (int32 Index = 0; Index < Objects.Num(); Index++)
	{
		(void)StructPropertyHandle->SetPerObjectValue(Index, Content, EPropertyValueSetFlags::DefaultFlags);
	}
	StructPropertyHandle->GetParentHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
}

void FRigBaseComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);

	bool bIsReadOnly = false;
	TArray<URigVMDetailsViewWrapperObject*> WrapperObjects;
	for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		URigVMDetailsViewWrapperObject* WrapperObject = CastChecked<URigVMDetailsViewWrapperObject>(DetailObject.Get());
		if (!WrapperObject->GetSubject())
		{
			continue;
		}
		
		if(WrapperObject->GetWrappedStruct()->IsChildOf(FRigBaseComponent::StaticStruct()))
		{
			WrapperObjects.Add(WrapperObject);
			
			if (const URigHierarchy* Hierarchy = Cast<URigHierarchy>(WrapperObject->GetSubject()))
			{
				const FRigBaseComponent WrappedComponent = WrapperObject->GetContent<FRigBaseComponent>();
				if(const FRigBaseComponent* Component = Hierarchy->FindComponent(WrappedComponent.GetKey()))
				{
					if(Component->IsProcedural())
					{
						bIsReadOnly = true;
						break;
					}
				}
			}
		}
	}

	if(bIsReadOnly)
	{
		TArray<FName> CategoryNames;
		DetailBuilder.GetCategoryNames(CategoryNames);
		for(const FName& CategoryName : CategoryNames)
		{
			TArray<TSharedRef<IPropertyHandle>> Properties;
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(CategoryName);
			Category.GetDefaultProperties(Properties);
			
			for(TSharedRef<IPropertyHandle>& Property : Properties)
			{
				if(IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(Property))
				{
					Row->IsEnabled(false);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
