// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Rigs/RigHierarchy.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "Editor/ControlRigWrapperObject.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Editor/SControlRigGizmoNameList.h"
#include "Styling/SlateTypes.h"
#include "IPropertyUtilities.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "SAdvancedTransformInputBox.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Internationalization/FastDecimalFormat.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Algo/Transform.h"
#include "Editor/ControlRigEditor.h"
#include "ControlRigEditorAsset.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"

#include "ControlRigElementDetails.generated.h"

class IPropertyHandle;

namespace FRigElementKeyDetailsDefs
{
	// Active foreground pin alpha
	static const float ActivePinForegroundAlpha = 1.f;
	// InActive foreground pin alpha
	static const float InactivePinForegroundAlpha = 0.15f;
	// Active background pin alpha
	static const float ActivePinBackgroundAlpha = 0.8f;
	// InActive background pin alpha
	static const float InactivePinBackgroundAlpha = 0.4f;
};

class SRigElementKeyWidget : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal(FText, FGetElementNameAsText);
	DECLARE_DELEGATE_RetVal(ERigElementType, FGetElementType);
	DECLARE_DELEGATE_RetVal(bool, FIsEnabled);
	DECLARE_DELEGATE_OneParam(FOnElementTypeChanged, ERigElementType);
	
	SLATE_BEGIN_ARGS(SRigElementKeyWidget)
	{
	}
	SLATE_ARGUMENT_DEPRECATED(UControlRigBlueprint*, Blueprint, 5.7, "Please use Asset instead")
	SLATE_ARGUMENT(FControlRigAssetInterfacePtr, Asset)
	SLATE_ARGUMENT(FSlateColor, ActiveBackgroundColor)
	SLATE_ARGUMENT(FSlateColor, InactiveBackgroundColor)
	SLATE_ARGUMENT(FSlateColor, ActiveForegroundColor)
	SLATE_ARGUMENT(FSlateColor, InactiveForegroundColor)
	SLATE_EVENT(SSearchableComboBox::FOnSelectionChanged, OnElementNameChanged)
	SLATE_EVENT(FOnClicked, OnGetSelectedClicked)
	SLATE_EVENT(FOnClicked, OnSelectInHierarchyClicked)
	SLATE_EVENT(FGetElementNameAsText, OnGetElementNameAsText)
	SLATE_EVENT(FGetElementType, OnGetElementType)
	SLATE_EVENT(FOnElementTypeChanged, OnElementTypeChanged)
	SLATE_EVENT(FIsEnabled, IsEnabled);
	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InNameHandle, TSharedPtr<IPropertyHandle> InTypeHandle);
	void Construct(const FArguments& InArgs);

	void UpdateElementNameList();

private:
	TSharedPtr<IPropertyHandle> NameHandle;
	TSharedPtr<IPropertyHandle> TypeHandle;

	/** Helper buttons. */
	TSharedPtr<SButton> UseSelectedButton;
	TSharedPtr<SButton> SelectElementButton;

	FGetElementType OnGetElementType;
	FOnElementTypeChanged OnElementTypeChanged;
	SSearchableComboBox::FOnSelectionChanged OnElementNameChanged;

	FControlRigAssetInterfacePtr BlueprintBeingCustomized;
	TArray<TSharedPtr<FString>> ElementNameList;
	TSharedPtr<SSearchableComboBox> SearchableComboBox;
};

class FRigElementKeyDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigElementKeyDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	ERigElementType GetElementType() const;
	FString GetElementName() const;
	void SetElementName(FString InName);
	void OnElementNameChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo);
	FText GetElementNameAsText() const;

	/** Helper buttons. */
	TSharedPtr<SButton> UseSelectedButton;
	TSharedPtr<SButton> SelectElementButton;

public:
	
	static FSlateColor OnGetWidgetForeground(const TSharedPtr<SButton> Button);
	static FSlateColor OnGetWidgetBackground(const TSharedPtr<SButton> Button);
	
protected:
	
	FReply OnGetSelectedClicked();
	FReply OnSelectInHierarchyClicked();
	
	TSharedPtr<IPropertyHandle> TypeHandle;
	TSharedPtr<IPropertyHandle> NameHandle;
	FControlRigAssetInterfacePtr BlueprintBeingCustomized;
	TSharedPtr<SRigElementKeyWidget> RigElementKeyWidget;
};

class FRigComponentKeyDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigComponentKeyDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	FRigElementKey GetElementKey() const;
	FString GetComponentName() const;
	void SetComponentName(FString InName);
	void OnComponentNameChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo);
	FText GetComponentNameAsText() const;
	void UpdateComponentNameList();
	
	TSharedPtr<IPropertyHandle> ElementKeyHandle;
	TSharedPtr<IPropertyHandle> NameHandle;
	FControlRigAssetInterfacePtr BlueprintBeingCustomized;
	
	TArray<TSharedPtr<FString>> ComponentNameList;
	TSharedPtr<SSearchableComboBox> SearchableComboBox;

};

UENUM()
enum class ERigElementDetailsTransformComponent : uint8
{
	TranslationX,
	TranslationY,
	TranslationZ,
	RotationRoll,
	RotationPitch,
	RotationYaw,
	ScaleX,
	ScaleY,
	ScaleZ
};

class FRigComputedTransformDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigComputedTransformDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	TSharedPtr<IPropertyHandle> TransformHandle;
	FEditPropertyChain PropertyChain;
	FControlRigAssetInterfacePtr BlueprintBeingCustomized;

	void OnTransformChanged(FEditPropertyChain* InPropertyChain);
};

class FRigControlTransformChannelDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigControlTransformChannelDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	ERigControlTransformChannel GetChannel() const;
	int32 GetChannelAsInt32() const { return (int32)GetChannel(); }
	void OnChannelChanged(int32 NewSelection, ESelectInfo::Type);

public:
	static const TArray<ERigControlTransformChannel>* GetVisibleChannelsForControlType(ERigControlType InControlType);

private:
	TSharedPtr<IPropertyHandle> Handle;
};

class FRigBaseElementDetails : public IDetailCustomization
{
public:

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void BeginDestroy() {};
	virtual void PendingDelete() override;

	FRigElementKey GetElementKey() const;
	FText GetName() const;
	void SetName(const FText& InNewText, ETextCommit::Type InCommitType);
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);
	FText GetSpawnIndex() const;

	void OnStructContentsChanged(FProperty* InProperty, const TSharedRef<IPropertyUtilities> PropertyUtilities);
	bool IsConstructionModeEnabled() const;

	FText GetParentElementName() const;

	TArray<FRigElementKey> GetElementKeys() const;

	template<typename T>
	TArray<T> GetElementsInDetailsView(const TArray<FRigElementKey>& InFilter = TArray<FRigElementKey>()) const
	{
		TArray<T> Elements;
		for(const FPerElementInfo& Info : PerElementInfos)
		{
			T Content = Info.WrapperObject->GetContent<T>();
			if(!InFilter.IsEmpty() && !InFilter.Contains(Content.GetKey()))
			{
				continue;
			}
			Elements.Add(Content);
		}
		return Elements;
	}

	struct FPerElementInfo
	{
		FPerElementInfo()
			: WrapperObject()
			, Element()
			, DefaultElement()
		{}

		bool IsValid() const { return Element.IsValid(); }
		bool IsProcedural() const { return Element.IsValid() && Element.Get()->IsProcedural(); }
		operator bool() const { return IsValid(); }

		URigHierarchy* GetHierarchy() const { return (URigHierarchy*)Element.GetHierarchy(); }
		URigHierarchy* GetDefaultHierarchy() const
		{
			if(DefaultElement.IsValid())
			{
				return (URigHierarchy*)DefaultElement.GetHierarchy();
			}
			return GetHierarchy();
		}

		UE_DEPRECATED(5.7, "Use GetAsset instead")
		UControlRigBlueprint* GetBlueprint() const
		{
			return nullptr;
		}

		FControlRigAssetInterfacePtr GetAsset() const
		{
			FControlRigAssetInterfacePtr Asset;
			if(const UControlRig* ControlRig = GetHierarchy()->GetTypedOuter<UControlRig>())
			{
				Asset = ControlRig->GetClass()->ClassGeneratedBy;
				if (!Asset)
				{
					if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = ControlRig->GetGeneratedByAsset())
					{
						Asset = RuntimeAsset->GetEditorOnlyData();
					}
				}
			}
			if (!Asset)
			{
				if (FControlRigAssetInterfacePtr EditorAsset = IControlRigEditorAssetInterface::GetInterfaceOuter(GetDefaultHierarchy()))
				{
					Asset = EditorAsset;
				}
			}
			if (!Asset)
			{
				if (TScriptInterface<IControlRigRuntimeAssetInterface> CRRuntimeAsset = IControlRigRuntimeAssetInterface::GetInterfaceOuter(GetDefaultHierarchy()))
				{
					if (TScriptInterface<IRigVMRuntimeAssetInterface> RigVMRuntimeAsset = CRRuntimeAsset.GetObject())
					{
						Asset = RigVMRuntimeAsset->GetEditorOnlyData();
					}
				}
			}
			ensure(Asset);
			return Asset;
		}

		template<typename T = FRigBaseElement>
		T* GetElement() const
		{
			return (T*)Element.Get<T>();
		}

		template<typename T = FRigBaseElement>
		T* GetDefaultElement() const
		{
			if(DefaultElement)
			{
				return (T*)DefaultElement.Get<T>();
			}
			return GetElement<T>();
		}

		TWeakObjectPtr<URigVMDetailsViewWrapperObject> WrapperObject;
		FRigElementHandle Element;
		FRigElementHandle DefaultElement;
	};

	const FPerElementInfo& FindElement(const FRigElementKey& InKey) const;
	bool IsAnyElementOfType(ERigElementType InType) const;
	bool IsAnyElementNotOfType(ERigElementType InType) const;
	bool IsAnyControlOfAnimationType(ERigControlAnimationType InType) const;
	bool IsAnyControlNotOfAnimationType(ERigControlAnimationType InType) const;
	bool IsAnyControlOfValueType(ERigControlType InType) const;
	bool IsAnyControlNotOfValueType(ERigControlType InType) const;
	bool IsAnyElementProcedural() const;
	bool IsAnyConnectorImported() const;
	bool IsAnyConnectorPrimary() const;
	bool GetCommonElementType(ERigElementType& OutElementType) const;
	bool GetCommonControlType(ERigControlType& OutControlType) const;
	bool GetCommonAnimationType(ERigControlAnimationType& OutAnimationType) const;
	const FPerElementInfo* FindElementByPredicate(const TFunction<bool(const FPerElementInfo&)>& InPredicate) const;
	bool ContainsElementByPredicate(const TFunction<bool(const FPerElementInfo&)>& InPredicate) const;

	static void RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule);
	virtual void RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass);

	FReply OnSelectParentElementInHierarchyClicked();
	FReply OnSelectElementClicked(const FRigElementKey& InKey);

protected:

	void CustomizeComponents(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeMetadata(IDetailLayoutBuilder& DetailBuilder);

	TArray<FPerElementInfo> PerElementInfos;
	FDelegateHandle MetadataHandle;
	TArray<TUniquePtr<FControlRigInteractionScope>> InteractionScopes;

	struct FSharedComponent
	{
		FRigComponentHandle Handle;
		const UScriptStruct* ScriptStruct = nullptr;
		bool bIsProcedural = false;
	};
	TArray<FSharedComponent> SharedComponents;

	TSharedPtr<SButton> SelectParentElementButton;
};

namespace ERigTransformElementDetailsTransform
{
	enum Type
	{
		Initial,
		Current,
		Offset,
		Minimum,
		Maximum,
		Max
	};
}

class FRigTransformElementDetails : public FRigBaseElementDetails
{
public:

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/** FRigBaseElementDetails interface */
	virtual void RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass) override;

	virtual void CustomizeTransform(IDetailLayoutBuilder& DetailBuilder);

protected:

	bool IsCurrentLocalEnabled() const;

	void AddChoiceWidgetRow(IDetailCategoryBuilder& InCategory, const FText& InSearchText, TSharedRef<SWidget> InWidget);

protected:

	FDetailWidgetRow& CreateTransformComponentValueWidgetRow(
		ERigControlType InControlType,
		const TArray<FRigElementKey>& Keys,
		SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip,
		ERigTransformElementDetailsTransform::Type CurrentTransformType,
		ERigControlValueType ValueType,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());

	FDetailWidgetRow& CreateEulerTransformValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		SAdvancedTransformInputBox<FEulerTransform>::FArguments TransformWidgetArgs,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip,
		ERigTransformElementDetailsTransform::Type CurrentTransformType,
		ERigControlValueType ValueType,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());

	static ERigTransformElementDetailsTransform::Type GetTransformTypeFromValueType(ERigControlValueType InValueType);

private:

	static TSharedPtr<TArray<ERigTransformElementDetailsTransform::Type>> PickedTransforms;

protected:

	TSharedPtr<FScopedTransaction> SliderTransaction;
};

class FRigBoneElementDetails : public FRigTransformElementDetails
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigBoneElementDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

class FRigControlElementDetails : public FRigTransformElementDetails
{
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigControlElementDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	void CustomizeValue(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeControl(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeAnimationChannels(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeAvailableSpaces(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeShape(IDetailLayoutBuilder& DetailBuilder);
	virtual void BeginDestroy() override;

	/** FRigBaseElementDetails interface */
	virtual void RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass) override;

	bool IsShapeEnabled() const;

	const TArray<TSharedPtr<FRigVMStringWithTag>>& GetShapeNameList() const;

	FText GetDisplayName() const;
	void SetDisplayName(const FText& InNewText, ETextCommit::Type InCommitType);
	FText GetDisplayNameForElement(const FRigElementKey& InKey) const;
	void SetDisplayNameForElement(const FText& InNewText, ETextCommit::Type InCommitType, const FRigElementKey& InKeyToRename);
	bool OnVerifyDisplayNameChanged(const FText& InText, FText& OutErrorMessage, const FRigElementKey& InKeyToRename);

	void OnCopyShapeProperties();
	void OnPasteShapeProperties();

	FDetailWidgetRow& CreateBoolValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip, 
		ERigControlValueType ValueType,
		TAttribute<EVisibility> Visibility = EVisibility::Visible,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());

	FDetailWidgetRow& CreateFloatValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip, 
		ERigControlValueType ValueType,
		TAttribute<EVisibility> Visibility = EVisibility::Visible,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());

	FDetailWidgetRow& CreateIntegerValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip, 
		ERigControlValueType ValueType,
		TAttribute<EVisibility> Visibility = EVisibility::Visible,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());

	FDetailWidgetRow& CreateEnumValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip, 
		ERigControlValueType ValueType,
		TAttribute<EVisibility> Visibility = EVisibility::Visible,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());

	FDetailWidgetRow& CreateVector2DValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip, 
		ERigControlValueType ValueType,
		TAttribute<EVisibility> Visibility = EVisibility::Visible,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>());


	// this is a template since we specialize it further down for
	// a 'nearly equals' implementation for floats and math types.
	template<typename T>
	static bool Equals(const T& A, const T& B)
	{
		return A == B;
	}

private:

	template<typename T>
	FDetailWidgetRow& CreateNumericValueWidgetRow(
		const TArray<FRigElementKey>& Keys,
		IDetailCategoryBuilder& CategoryBuilder, 
		const FText& Label, 
		const FText& Tooltip, 
		ERigControlValueType ValueType,
		TAttribute<EVisibility> Visibility = EVisibility::Visible,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>())
	{
		const bool bShowToggle = (ValueType == ERigControlValueType::Minimum) || (ValueType == ERigControlValueType::Maximum);
		const bool bIsProcedural = IsAnyElementProcedural();
		const bool bIsEnabled = !bIsProcedural || ValueType == ERigControlValueType::Current;

		URigHierarchy* Hierarchy = PerElementInfos[0].GetHierarchy();
		URigHierarchy* HierarchyToChange = PerElementInfos[0].GetDefaultHierarchy();
		if(ValueType == ERigControlValueType::Current)
		{
			HierarchyToChange = Hierarchy;
		}

		FControlRigEditorEditMode* EditMode = nullptr;
		if (FControlRigAssetInterfacePtr Blueprint = PerElementInfos[0].GetAsset())
		{
			UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			if (IAssetEditorInstance* EditorInstance = AssetEditorSubsystem->FindEditorForAsset(Blueprint.GetObject(), false))
			{
				if (IControlRigBaseEditor* Editor = FControlRigBaseEditor::GetFromAssetEditorInstance(EditorInstance))
				{
					EditMode = Editor->GetEditMode();
				}
			}
		}

		TSharedPtr<SNumericEntryBox<T>> NumericEntryBox;
		
		FDetailWidgetRow& WidgetRow = CategoryBuilder.AddCustomRow(Label);
		TAttribute<ECheckBoxState> ToggleChecked;
		FOnCheckStateChanged OnToggleChanged;

		if(bShowToggle)
		{
			ToggleChecked = TAttribute<ECheckBoxState>::CreateLambda(
				[ValueType, Keys, Hierarchy]() -> ECheckBoxState
				{
					TOptional<bool> FirstValue;

					for(const FRigElementKey& Key : Keys)
					{
						if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Key))
						{
							if(ControlElement->Settings.LimitEnabled.Num() == 1)
							{
								const bool Value = ControlElement->Settings.LimitEnabled[0].GetForValueType(ValueType);
								if(FirstValue.IsSet())
								{
									if(FirstValue.GetValue() != Value)
									{
										return ECheckBoxState::Undetermined;
									}
								}
								else
								{
									FirstValue = Value;
								}
							}
						}
					}

					if(!ensure(FirstValue.IsSet()))
					{
						return ECheckBoxState::Undetermined;
					}

					return FirstValue.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				});

			OnToggleChanged = FOnCheckStateChanged::CreateLambda(
				[ValueType, Keys, HierarchyToChange](ECheckBoxState InValue)
				{
					if(InValue == ECheckBoxState::Undetermined)
					{
						return;
					}

					FScopedTransaction Transaction(NSLOCTEXT("ControlRigElementDetails", "ChangeLimitToggle", "Change Limit Toggle"));
					for(const FRigElementKey& Key : Keys)
					{
						if(FRigControlElement* ControlElement = HierarchyToChange->Find<FRigControlElement>(Key))
						{
							if(ControlElement->Settings.LimitEnabled.Num() == 1)
							{
								HierarchyToChange->Modify();
								ControlElement->Settings.LimitEnabled[0].SetForValueType(ValueType, InValue == ECheckBoxState::Checked);
								HierarchyToChange->SetControlSettings(ControlElement, ControlElement->Settings, true, true, true);
							}
						}
					}
				});
		}

		auto OnValueChanged = [ValueType, Keys, HierarchyToChange, EditMode, this]
			(TOptional<T> InValue, ETextCommit::Type InCommitType, bool bSetupUndo)
			{
				if(!InValue.IsSet())
				{
					return;
				}

				const T Value = InValue.GetValue();
				
				for(const FRigElementKey& Key : Keys)
				{
					const T PreviousValue = HierarchyToChange->GetControlValue(Key, ValueType).Get<T>();
					if(!Equals(PreviousValue, Value))
					{
						if(!SliderTransaction.IsValid())
						{
							SliderTransaction = MakeShareable(new FScopedTransaction(NSLOCTEXT("ControlRigElementDetails", "ChangeValue", "Change Value")));
							HierarchyToChange->Modify();
						}
						HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<T>(Value), ValueType, bSetupUndo, bSetupUndo);
						if (EditMode)
						{
							EditMode->ModifiedRigElements.AddUnique(Key);
						}
					}
				}

				if(bSetupUndo)
				{
					SliderTransaction.Reset();
				}
			};

		if(!NameContent.IsValid())
		{
			SAssignNew(NameContent, STextBlock)
			.Text(Label)
			.ToolTipText(Tooltip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(bIsEnabled);
		}

		WidgetRow
		.Visibility(Visibility)
		.NameContent()
		.MinDesiredWidth(200.f)
		.MaxDesiredWidth(800.f)
		[
			NameContent.ToSharedRef()
		]
		.ValueContent()
		[
			SAssignNew(NumericEntryBox, SNumericEntryBox<T>)
	        .Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
	        .AllowSpin(ValueType == ERigControlValueType::Current || ValueType == ERigControlValueType::Initial)
	        .LinearDeltaSensitivity(1)
			.Delta(0.01f)
	        .Value_Lambda([ValueType, Keys, Hierarchy]() -> TOptional<T>
	        {
		        const T FirstValue = Hierarchy->GetControlValue<T>(Keys[0], ValueType);
				for(int32 Index = 1; Index < Keys.Num(); Index++)
				{
					const T SecondValue = Hierarchy->GetControlValue<T>(Keys[Index], ValueType);
					if(FirstValue != SecondValue)
					{
						return TOptional<T>();
					}
				}
				return FirstValue;
	        })
	        .OnValueChanged_Lambda([ValueType, Keys, HierarchyToChange, OnValueChanged](TOptional<T> InValue)
	        {
        		OnValueChanged(InValue, ETextCommit::Default, false);
	        })
	        .OnValueCommitted_Lambda([ValueType, Keys, HierarchyToChange, OnValueChanged](TOptional<T> InValue, ETextCommit::Type InCommitType)
	        {
        		OnValueChanged(InValue, InCommitType, true);
	        })
	        .MinSliderValue_Lambda([ValueType, Keys, Hierarchy]() -> TOptional<T>
			 {
				 if(ValueType == ERigControlValueType::Current || ValueType == ERigControlValueType::Initial)
				 {
			 		return Hierarchy->GetControlValue<T>(Keys[0], ERigControlValueType::Minimum);
				 }
				 return TOptional<T>();
			 })
			 .MaxSliderValue_Lambda([ValueType, Keys, Hierarchy]() -> TOptional<T>
			 {
		 		if(ValueType == ERigControlValueType::Current || ValueType == ERigControlValueType::Initial)
		 		{
					 return Hierarchy->GetControlValue<T>(Keys[0], ERigControlValueType::Maximum);
				 }
				 return TOptional<T>();
			 })
			 .DisplayToggle(bShowToggle)
			 .ToggleChecked(ToggleChecked)
			 .OnToggleChanged(OnToggleChanged)
			 .UndeterminedString(NSLOCTEXT("FRigControlElementDetails", "MultipleValues", "Multiple Values"))
			 .IsEnabled(bIsEnabled)
			 .PreventThrottling(true)
		]
		.CopyAction(FUIAction(
		FExecuteAction::CreateLambda([ValueType, Keys, Hierarchy]()
			{
				const T FirstValue = Hierarchy->GetControlValue<T>(Keys[0], ValueType);
				const FString Content = FastDecimalFormat::NumberToString(
					FirstValue,
					FastDecimalFormat::GetCultureAgnosticFormattingRules(),
					FNumberFormattingOptions());
				FPlatformApplicationMisc::ClipboardCopy(*Content);
			}),
			FCanExecuteAction())
		)
		.PasteAction(FUIAction(
			FExecuteAction::CreateLambda([ValueType, Keys, HierarchyToChange]()
			{
				FString Content;
				FPlatformApplicationMisc::ClipboardPaste(Content);
				if(Content.IsEmpty())
				{
					return;
				}

				T Value = T(0);
				if(FastDecimalFormat::StringToNumber(
					*Content,
					Content.Len(),
					FastDecimalFormat::GetCultureAgnosticFormattingRules(),
					FNumberParsingOptions(),
					Value))
				{
					FScopedTransaction Transaction(NSLOCTEXT("ControlRigElementDetails", "ChangeValue", "Change Value"));
					for(const FRigElementKey& Key : Keys)
					{
						HierarchyToChange->Modify();
						HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<T>(Value), ValueType, true, true); 
					}
				}
			}),
			FCanExecuteAction::CreateLambda([bIsEnabled]() { return bIsEnabled; }))
		);

		if((ValueType == ERigControlValueType::Current || ValueType == ERigControlValueType::Initial) && bIsEnabled)
		{
			WidgetRow.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>::CreateLambda([ValueType, Keys, Hierarchy]() -> bool
				{
					const T FirstValue = Hierarchy->GetControlValue<T>(Keys[0], ValueType);
					const T ReferenceValue = ValueType == ERigControlValueType::Initial ? T(0) :
						Hierarchy->GetControlValue<T>(Keys[0], ERigControlValueType::Initial);

					return !FRigControlElementDetails::Equals(FirstValue, ReferenceValue);
				}),
				FSimpleDelegate::CreateLambda([ValueType, Keys, HierarchyToChange]()
				{
					FScopedTransaction Transaction(NSLOCTEXT("ControlRigElementDetails", "ResetValueToDefault", "Reset Value To Default"));
					for(const FRigElementKey& Key : Keys)
					{
						const T ReferenceValue = ValueType == ERigControlValueType::Initial ? T(0) :
							HierarchyToChange->GetControlValue<T>(Keys[0], ERigControlValueType::Initial);
						HierarchyToChange->Modify();
						HierarchyToChange->SetControlValue(Key, FRigControlValue::Make<T>(ReferenceValue), ValueType, true, true); 
					}
				})
			));
		}

		return WidgetRow;
	}

	// animation channel related callbacks
	FReply OnAddAnimationChannelClicked();
	TSharedRef<ITableRow> HandleGenerateAnimationChannelTypeRow(TSharedPtr<ERigControlType> ControlType, const TSharedRef<STableViewBase>& OwnerTable, FRigElementKey ControlKey);

	// multi parent related callbacks
	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings() const { return DisplaySettings; }
	TSharedRef<SWidget> GetAddSpaceContent(const TSharedRef<IPropertyUtilities> PropertyUtilities);
	FReply OnAddSpaceMouseDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, const TSharedRef<IPropertyUtilities> PropertyUtilities);
	void OnAddSpaceSelection(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo, const TSharedRef<IPropertyUtilities> PropertyUtilities);

	void HandleControlTypeChanged(TSharedPtr<ERigControlType> ControlType, ESelectInfo::Type SelectInfo, FRigElementKey ControlKey, const TSharedRef<IPropertyUtilities> PropertyUtilities);
	void HandleControlTypeChanged(ERigControlType ControlType, TArray<FRigElementKey> ControlKeys, const TSharedRef<IPropertyUtilities> PropertyUtilities);
	void HandleControlEnumChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo, const TSharedRef<IPropertyUtilities> PropertyUtilities);

	TArray<TSharedPtr<FRigVMStringWithTag>> ShapeNameList;
	TSharedPtr<FRigInfluenceEntryModifier> InfluenceModifier;
	TSharedPtr<FStructOnScope> InfluenceModifierStruct;

	TSharedPtr<IPropertyHandle> ShapeNameHandle;
	TSharedPtr<IPropertyHandle> ShapeColorHandle;

	TSharedPtr<SControlRigShapeNameList> ShapeNameListWidget; 
	static TSharedPtr<TArray<ERigControlValueType>> PickedValueTypes;

	FRigHierarchyTreeDisplaySettings DisplaySettings;
	TSharedPtr<SMenuAnchor> AddSpaceMenuAnchor;
};

template<>
inline bool FRigControlElementDetails::Equals<float>(const float& A, const float& B)
{
	return FMath::IsNearlyEqual(A, B);
}

template<>
inline bool FRigControlElementDetails::Equals<double>(const double& A, const double& B)
{
	return FMath::IsNearlyEqual(A, B);
}

template<>
inline bool FRigControlElementDetails::Equals<FVector>(const FVector& A, const FVector& B)
{
	return (A - B).IsNearlyZero();
}

template<>
inline bool FRigControlElementDetails::Equals<FRotator>(const FRotator& A, const FRotator& B)
{
	return (A - B).IsNearlyZero();
}

template<>
inline bool FRigControlElementDetails::Equals<FEulerTransform>(const FEulerTransform& A, const FEulerTransform& B)
{
	return Equals(A.Location, B.Location) && Equals(A.Rotation, B.Rotation) && Equals(A.Scale, B.Scale);
}

class FRigNullElementDetails : public FRigTransformElementDetails
{
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigNullElementDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

class FRigConnectorElementDetails : public FRigTransformElementDetails
{
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigConnectorElementDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	void CustomizeSettings(IDetailLayoutBuilder& DetailBuilder);
	void CustomizeConnectorTargets(IDetailLayoutBuilder& DetailBuilder);

private:
	TOptional<EConnectorType> GetConnectorType() const;
	TOptional<bool> GetIsConnectorArray() const;
	bool CanBeArray() const;
	bool OnTargetsChanged(TArray<FRigElementKey> InTargets);

	/** Requests to refresh these details */
	void RequestRefresh();

	TSharedPtr<IPropertyHandle> ConnectorTypeHandle;
	TSharedPtr<IPropertyHandle> IsArrayHandle;
	TSharedPtr<IPropertyHandle> IsPostConstructionHandle;

	/** Property utils for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtils;
};

class FRigSocketElementDetails : public FRigNullElementDetails
{
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigSocketElementDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	void CustomizeSettings(IDetailLayoutBuilder& DetailBuilder);

private:

	FReply SetSocketColor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	FLinearColor GetSocketColor() const; 
	void OnSocketColorPicked(FLinearColor NewColor);
	void SetSocketDescription(const FText& InDescription, ETextCommit::Type InCommitType);
	FText GetSocketDescription() const; 
};

class FRigConnectionRuleDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigConnectionRuleDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	
	TSharedRef<SWidget> GenerateStructPicker();
	void OnPickedStruct(const UScriptStruct* ChosenStruct);
	FText OnGetStructTextValue() const;
	void OnRuleContentChanged();
	
	FRigConnectionRuleStash RuleStash;
	TSharedPtr<FStructOnScope> Storage;
	FControlRigAssetInterfacePtr BlueprintBeingCustomized;
	TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TAttribute<bool> EnabledAttribute;
};

class FRigBaseComponentDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigBaseComponentDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};
