// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigGizmoLibraryActions.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

void FControlRigShapeLibraryActions::GetActions( const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder )
{
	//FAssetTypeActions_AnimationAsset::GetActions(InObjects, MenuBuilder);
}

const TArray<FText>& FControlRigShapeLibraryActions::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("AnimControlRigSubMenu", "Control Rig")
	};
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
