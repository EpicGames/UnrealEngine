// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"
#include "ControlRigGizmoLibrary.h"

class FMenuBuilder;

class FControlRigShapeLibraryActions : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual void GetActions(const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder) override;
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ControlRigShapeLibrary", "Control Rig Shape Library"); }
	virtual FColor GetTypeColor() const override { return FColor(100,100,255); }
	virtual UClass* GetSupportedClass() const override { return UControlRigShapeLibrary::StaticClass(); }
	virtual bool CanFilter() override { return true; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual const TArray<FText>& GetSubMenus() const override;
};
