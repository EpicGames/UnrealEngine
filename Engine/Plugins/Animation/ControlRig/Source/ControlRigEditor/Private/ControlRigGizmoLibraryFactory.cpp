// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigGizmoLibraryFactory.h"
#include "AssetTypeCategories.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigGizmoLibraryFactory)

#define LOCTEXT_NAMESPACE "ControlRigGizmoLibraryFactory"

UControlRigShapeLibraryFactory::UControlRigShapeLibraryFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UControlRigShapeLibrary::StaticClass();
}

UObject* UControlRigShapeLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UControlRigShapeLibrary* ShapeLibrary = NewObject<UControlRigShapeLibrary>(InParent, Name, Flags);

	ShapeLibrary->DefaultShape.ShapeProxy = LoadObject<UStaticMesh>(nullptr, TEXT("/ControlRig/Controls/ControlRig_Sphere_solid.ControlRig_Sphere_solid"));
	ShapeLibrary->DefaultMaterial = LoadObject<UMaterial>(nullptr, TEXT("/ControlRig/Controls/ControlRigGizmoMaterial.ControlRigGizmoMaterial"));
	ShapeLibrary->XRayMaterial = LoadObject<UMaterial>(nullptr, TEXT("/ControlRig/Controls/ControlRigXRayMaterial.ControlRigXRayMaterial"));
	ShapeLibrary->MaterialColorParameter = TEXT("Color");

	return ShapeLibrary;
}

FText UControlRigShapeLibraryFactory::GetDisplayName() const
{
	return LOCTEXT("ControlRigShapeLibraryFactoryName", "Control Rig Shape Library");
}

uint32 UControlRigShapeLibraryFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

#undef LOCTEXT_NAMESPACE

