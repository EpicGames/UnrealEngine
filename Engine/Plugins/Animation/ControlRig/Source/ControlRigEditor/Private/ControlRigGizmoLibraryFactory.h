// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "ControlRigGizmoLibrary.h"
#include "ControlRigGizmoLibraryFactory.generated.h"

UCLASS(MinimalAPI, HideCategories=Object)
class UControlRigShapeLibraryFactory : public UFactory
{
	GENERATED_BODY()

public:
	UControlRigShapeLibraryFactory();

	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual uint32 GetMenuCategories() const override;
};

