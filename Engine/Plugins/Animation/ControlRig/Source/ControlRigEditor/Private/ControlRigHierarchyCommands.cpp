// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigHierarchyCommands.h"

#define LOCTEXT_NAMESPACE "ControlRigHierarchyCommands"

void FControlRigHierarchyCommands::RegisterCommands()
{
	UI_COMMAND(AddBoneItem, "New Bone", "Add new bone at the origin (0, 0, 0) to the hierarchy.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddControlItem, "New Control", "Add new control at the origin (0, 0, 0) to the hierarchy.", EUserInterfaceActionType::Button, FInputChord(EKeys::N, EModifierKey::Control));
	UI_COMMAND(AddAnimationChannelItem, "New Animation Channel", "Add new animation channel.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNullItem, "New Null", "Add new null at the origin (0, 0, 0) to the hierarchy.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddConnectorItem, "New Connector", "Add new connector at the origin (0, 0, 0) to the hierarchy.\nNote: This is only available for Rig Modules.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddSocketItem, "New Socket", "Add new socket at the origin (0, 0, 0) to the hierarchy.\nNote: This is only available for Rig Modules and Modular Rigs.", EUserInterfaceActionType::Button, FInputChord(EKeys::K, EModifierKey::Control));
	UI_COMMAND(FindReferencesOfItem, "Find References", "Find all references of the selected item.", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
	UI_COMMAND(DuplicateItem, "Duplicate", "Duplicate the selected items in the hierarchy.", EUserInterfaceActionType::Button, FInputChord(EKeys::D, EModifierKey::Control));
	UI_COMMAND(MirrorItem, "Mirror", "Mirror the selected items in the hierarchy.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteItem, "Delete", "Deletes the selected items from the hierarchy.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
	UI_COMMAND(RenameItem, "Rename", "Rename the selected item.", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(CopyItems, "Copy", "Copy the selected items.", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control));
	UI_COMMAND(PasteItems, "Paste", "Paste the selected items.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control));
	UI_COMMAND(PasteLocalTransforms, "Paste Local Transform", "Paste the local transforms.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteGlobalTransforms, "Paste Global Transform", "Paste the global transforms.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control | EModifierKey::Shift));
	UI_COMMAND(ZeroControls, "Zero Controls", "Sets the transform of the control as offset, zeroes the initial and current transform.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZeroControlsFromClosestBone, "Zero Controls from Closest Bone", "Sets the offset to the transform of the closest bone, zeroes the initial and current transform.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZeroControlShape, "Zero Control Shape", "Sets the current shape relative to the control's offset, zeroes the control's initial and current transform.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetInitialTransformFromCurrent, "Set Initial Transform from Current", "Set the current transform of selected bone, null or connector as its initial and current transform.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetTransform, "Reset Transform", "Reset the Transform", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Control));
	UI_COMMAND(ResetAllTransforms, "Reset All Transforms", "Resets all Transforms", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Control | EModifierKey::Shift));
	UI_COMMAND(ResetNull, "Reset Null", "Resets or injects a Null below the Control", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FrameSelection, "Frame Selection", "Expands and frames the selection in the tree", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(ControlBoneTransform, "Control Bone Transform", "Sets the bone transform using a shape", EUserInterfaceActionType::Button, FInputChord(EKeys::B, EModifierKey::Alt));
	UI_COMMAND(ControlSpaceTransform, "Control Space Transform", "Sets the space transform using a shape", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Unparent, "Unparent", "Unparents the selected elements from the hierarchy", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Shift));
	UI_COMMAND(FilteringFlattensHierarchy, "Filtering Flattens Hierarchy", "Whether to keep the hierarchy or flatten it when searching for tree items", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(HideParentsWhenFiltering, "Hide Parents When Filtering", "Whether to show parent items grayed out, or hide them entirely when filtering", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ArrangeByModules, "Arrange by Modules", "Arrange the hierarchy by the modules that spawn each control", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FlattenModules, "Flatten Modules", "Flatten the modular rig into a list of modules", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SetMultiRigMode_All, "All", "Change outliner multi rig display mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetMultiRigMode_SelectedRigs, "Selected Rigs", "Change outliner multi rig display mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SetMultiRigMode_SelectedModules, "Selected Modules", "Change outliner multi rig display mode", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FocusOnSelection, "Focus On Selection", "Focus on the selection when it changes", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowImportedBones, "Show Imported Bones", "Whether to show or hide imported bones", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowBones, "Show Bones", "Whether to show or hide bones", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowControls, "Show Controls", "Whether to show or hide controls", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowNulls, "Show Nulls", "Whether to show or hide nulls", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowReferences, "Show References", "Whether to show or hide references", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowSockets, "Show Sockets", "Whether to show or hide sockets", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowComponents, "Show Components", "Whether to show or hide components", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleControlShapeTransformEdit, "Toggle Shape Transform Edit", "Toggle Editing Selected Control's Shape Transform", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Period, EModifierKey::Control)); 
	UI_COMMAND(SpaceSwitching, "Space Switching", "Space switching on the control", EUserInterfaceActionType::Button, FInputChord(EKeys::Tab)); 
	UI_COMMAND(ShowIconColors, "Show Icon Colors", "Whether to tint the icons with the element color", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
