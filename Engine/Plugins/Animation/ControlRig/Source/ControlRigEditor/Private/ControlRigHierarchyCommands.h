// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "ControlRigEditorStyle.h"

class FControlRigHierarchyCommands : public TCommands<FControlRigHierarchyCommands>
{
public:
	FControlRigHierarchyCommands() : TCommands<FControlRigHierarchyCommands>
	(
		"ControlRigHierarchy",
		NSLOCTEXT("Contexts", "RigHierarchy", "Rig Hierarchy"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FControlRigEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	/** Add Item at origin */
	TSharedPtr< FUICommandInfo > AddBoneItem;

	/** Add Item at origin */
	TSharedPtr< FUICommandInfo > AddControlItem;

	/** Add an animation channel */
	TSharedPtr< FUICommandInfo > AddAnimationChannelItem;

	/** Add Item at origin */
	TSharedPtr< FUICommandInfo > AddNullItem;

	/** Add Item at origin */
	TSharedPtr< FUICommandInfo > AddConnectorItem;

	/** Add Item at origin */
	TSharedPtr< FUICommandInfo > AddSocketItem;

	/** Find all references of an item in the graphs */
	TSharedPtr< FUICommandInfo > FindReferencesOfItem;
	
	/** Duplicate currently selected items */
	TSharedPtr< FUICommandInfo > DuplicateItem;

	/** Mirror currently selected items */
	TSharedPtr< FUICommandInfo > MirrorItem;

	/** Delete currently selected items */
	TSharedPtr< FUICommandInfo > DeleteItem;

	/** Rename selected item */
	TSharedPtr< FUICommandInfo > RenameItem;

	/** Copy the selected items. */
	TSharedPtr< FUICommandInfo > CopyItems;

	/** Paste the selected items. */
	TSharedPtr< FUICommandInfo > PasteItems;

	/** Paste Local Transform", "Paste the local transforms. */
	TSharedPtr< FUICommandInfo > PasteLocalTransforms;

	/** Paste Global Transform", "Paste the global transforms. */
	TSharedPtr< FUICommandInfo > PasteGlobalTransforms;

	/* Reset transform */
	TSharedPtr<FUICommandInfo> ResetTransform;

	/* Reset all transforms */
	TSharedPtr<FUICommandInfo> ResetAllTransforms;

	/* Reset space */
	TSharedPtr<FUICommandInfo> ResetNull;

	/* Sets the transform of the control as offset, zeroes the initial and current transform */
	TSharedPtr<FUICommandInfo> ZeroControls;

	/* Sets the offset to the transform of the closest bone, zeroes the initial and current transform */
	TSharedPtr<FUICommandInfo> ZeroControlsFromClosestBone;

	/* Sets the current shape relative to the control's offset, zeroes the control's initial and current transform */
	TSharedPtr<FUICommandInfo> ZeroControlShape;

	/* Set the current transform of selected bone, null or connector as its initial and current transform */
	TSharedPtr<FUICommandInfo> SetInitialTransformFromCurrent;

	/* frames the selection in the tree */
	TSharedPtr<FUICommandInfo> FrameSelection;

	/* sets the bone transform using a shape */
	TSharedPtr<FUICommandInfo> ControlBoneTransform;

	/* sets the space transform using a shape */
	TSharedPtr<FUICommandInfo> ControlSpaceTransform;

	/* Unparents the selected elements from the hierarchy */
	TSharedPtr<FUICommandInfo> Unparent;

	/* Flatten hierarchy on filter */
	TSharedPtr< FUICommandInfo > FilteringFlattensHierarchy;

	/* Hide parents on filter */
	TSharedPtr< FUICommandInfo > HideParentsWhenFiltering;

	/* Arrange the controls by modules in a modular rig */
	TSharedPtr< FUICommandInfo > ArrangeByModules;

	/* Flatten the modules in a modular rig */
	TSharedPtr< FUICommandInfo > FlattenModules;

	/* Options of outliner reflection mode for multi rigs */
	TSharedPtr< FUICommandInfo > SetMultiRigMode_All; // show all rigs
	TSharedPtr< FUICommandInfo > SetMultiRigMode_SelectedRigs; // show only rigs with selected controls
	TSharedPtr< FUICommandInfo > SetMultiRigMode_SelectedModules; // show only modules with selected controls (for modular rigs)

	/* Show imported bones */
	TSharedPtr< FUICommandInfo > ShowImportedBones;

	/* Show bones */
	TSharedPtr< FUICommandInfo > ShowBones;

	/* Show controls */
	TSharedPtr< FUICommandInfo > ShowControls;

	/* Show spaces */
	TSharedPtr< FUICommandInfo > ShowNulls;

	/* Show references */
	TSharedPtr< FUICommandInfo > ShowReferences;

	/* Show sockets */
	TSharedPtr< FUICommandInfo > ShowSockets;

	/* Show components */
	TSharedPtr< FUICommandInfo > ShowComponents;

	/** Toggle Shape Transform Edit*/
	TSharedPtr< FUICommandInfo > ToggleControlShapeTransformEdit;

	/** Space switch as it would look like for animator */
	TSharedPtr< FUICommandInfo > SpaceSwitching;

	/** Whether to tint the icons with the element color */
	TSharedPtr< FUICommandInfo > ShowIconColors;

	/* Scroll to selection when it changes */
	TSharedPtr< FUICommandInfo > FocusOnSelection;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
