// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigInfluenceMapDetails.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "ControlRig.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#define LOCTEXT_NAMESPACE "RigInfluenceMapDetails"

void FRigInfluenceMapPerEventDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	BlueprintBeingCustomized = GetBlueprintFromDetailBuilder(DetailBuilder);
	if (BlueprintBeingCustomized == nullptr)
	{
		return;
	}

	FRigInfluenceMapPerEvent& MapPerEvent = BlueprintBeingCustomized->Influences;

	// add default events
	MapPerEvent.FindOrAdd(FRigUnit_BeginExecution::EventName);
	MapPerEvent.FindOrAdd(FRigUnit_PrepareForExecution::EventName);
}

UControlRigBlueprint* FRigInfluenceMapPerEventDetails::GetBlueprintFromDetailBuilder(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TSharedPtr<FStructOnScope>> StructsBeingCustomized;
	DetailBuilder.GetStructsBeingCustomized(StructsBeingCustomized);
	if (StructsBeingCustomized.Num() > 0)
	{
		TSharedPtr<FStructOnScope> StructBeingCustomized = StructsBeingCustomized[0];
		if (UPackage* Package = StructBeingCustomized->GetPackage())
		{
			TArray<UObject*> SubObjects;
			Package->GetDefaultSubobjects(SubObjects);

			for (UObject* SubObject : SubObjects)
			{
				if (UControlRig* Rig = Cast<UControlRig>(SubObject))
				{
					if(UControlRigBlueprint* Blueprint = Cast<UControlRigBlueprint>(Rig->GetClass()->ClassGeneratedBy))
					{
						return Blueprint;
					}
				}
			}
		}
	}
	return nullptr;
}


#undef LOCTEXT_NAMESPACE
