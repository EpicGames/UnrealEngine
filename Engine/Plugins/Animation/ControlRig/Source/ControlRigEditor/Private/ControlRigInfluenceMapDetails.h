// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ControlRigBlueprintLegacy.h"
#include "Styling/SlateTypes.h"
#include "IPropertyUtilities.h"

class IPropertyHandle;

class FRigInfluenceMapPerEventDetails : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigInfluenceMapPerEventDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	static UControlRigBlueprint* GetBlueprintFromDetailBuilder(IDetailLayoutBuilder& DetailBuilder);

protected:

	UControlRigBlueprint* BlueprintBeingCustomized;
};
