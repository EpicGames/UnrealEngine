// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigModularRigCommands.h"

#define LOCTEXT_NAMESPACE "ControlRigModularRigCommands"

void FControlRigModularRigCommands::RegisterCommands()
{
	UI_COMMAND(AddModuleItem, "New Module", "Add new module to the rig.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RenameModuleItem, "Rename", "Rename module to the rig.", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(DeleteModuleItem, "Delete", "Delete module from the rig.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
	UI_COMMAND(DuplicateModuleItem, "Duplicate", "Duplicate module from the rig.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D));
	UI_COMMAND(MirrorModuleItem, "Mirror", "Mirror module from the rig.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReresolveModuleItem, "Auto Resolve", "Auto Resolve secondary connectors.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SwapModuleClassItem, "Swap Module", "Swap module of common class.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopyModuleSettings, "Copy Settings", "Copies the module(s) settings to the clipboard.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::C));
	UI_COMMAND(PasteModuleSettings, "Paste Settings", "Pastes the module(s) settings from the clipboard.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::V));
	UI_COMMAND(ToggleShowSecondaryContectors, "Secondary Connectors", "Toggles the visibility of secondary connectors in the module hierarchy.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::One));
	UI_COMMAND(ToggleShowOptionalContectors, "Optional Connectors", "Toggles the visibility of optional connectors in the module hierarchy.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Two));
	UI_COMMAND(ToggleShowUnresolvedContectors, "Unresolved Connectors", "Toggles the visibility of unresolved connectors in the module hierarchy.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::Three));
}

#undef LOCTEXT_NAMESPACE
