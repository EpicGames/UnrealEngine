// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "ControlRigEditorStyle.h"

class FControlRigModularRigCommands : public TCommands<FControlRigModularRigCommands>
{
public:
	FControlRigModularRigCommands() : TCommands<FControlRigModularRigCommands>
	(
		"ControlRigModularRigModel",
		NSLOCTEXT("Contexts", "ModularRigModel", "Modular Rig Modules"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FControlRigEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	/** Add Module at root */
	TSharedPtr< FUICommandInfo > AddModuleItem;

	/** Rename Module */
	TSharedPtr< FUICommandInfo > RenameModuleItem;

	/** Delete Module */
	TSharedPtr< FUICommandInfo > DeleteModuleItem;

	/** Duplicate Module */
	TSharedPtr< FUICommandInfo > DuplicateModuleItem;

	/** Mirror Module */
	TSharedPtr< FUICommandInfo > MirrorModuleItem;

	/** Reresolve Module */
	TSharedPtr< FUICommandInfo > ReresolveModuleItem;

	/** Swap Module Class */
	TSharedPtr< FUICommandInfo > SwapModuleClassItem;

	/** Copy module settings */
	TSharedPtr< FUICommandInfo > CopyModuleSettings;

	/** Paste module settings */
	TSharedPtr< FUICommandInfo > PasteModuleSettings;

	/** Toggle visibility of secondary connectors */
	TSharedPtr< FUICommandInfo > ToggleShowSecondaryContectors;

	/** Toggle visibility of optional connectors */
	TSharedPtr< FUICommandInfo > ToggleShowOptionalContectors;

	/** Toggle visibility of unresolved connectors */
	TSharedPtr< FUICommandInfo > ToggleShowUnresolvedContectors;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
