// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigModuleDetails.h"
#include "Widgets/SWidget.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "ModularRigController.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigEditorStyle.h"
#include "ControlRigElementDetails.h"
#include "Graph/ControlRigGraph.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "SEnumCombo.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Styling/AppStyle.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Features/IModularFeatures.h"
#include "IPropertyAccessEditor.h"
#include "ModularRigRuleManager.h"
#include "PropertyPath.h"
#include "ScopedTransaction.h"
#include "Widgets/SRigVMVariantTagWidget.h"
#include "Algo/Sort.h"
#include "Editor/SRigConnectorTargetWidget.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SControlRigDragDropBox.h"
#include "Editor/ControlRigShowSchematicViewportOverride.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyElementDragDropOp.h"

#define LOCTEXT_NAMESPACE "ControlRigModuleDetails"

static const FText ControlRigModuleDetailsMultipleValues = LOCTEXT("MultipleValues", "Multiple Values");

void FRigModuleInstanceDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	using namespace UE::ControlRigEditor;

	PerModuleInfos.Reset();
	PerModuleWrappers.Reset();

	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);
	for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		if(UControlRig* ModuleInstance = Cast<UControlRig>(DetailObject))
		{
			if(const UModularRig* ModularRig = Cast<UModularRig>(ModuleInstance->GetOuter()))
			{
				if(const FRigModuleInstance* Module = ModularRig->FindModule(ModuleInstance))
				{
					const FName ModuleName = Module->Name;

					FPerModuleInfo Info;
					Info.ModuleName = ModuleName;
					Info.Module = ModularRig->GetHandle(ModuleName);
					if(!Info.Module.IsValid())
					{
						return;
					}

					PerModuleInfos.Add(Info);
				}
			}
		}
	}

	// don't customize if the 
	if(PerModuleInfos.IsEmpty())
	{
		return;
	}

	TWeakPtr<FRigModuleInstanceDetails> WeakThisPtr = SharedThis(this);

	TArray<FName> OriginalCategoryNames;
	DetailBuilder.GetCategoryNames(OriginalCategoryNames);

	IDetailCategoryBuilder& GeneralCategory = DetailBuilder.EditCategory(TEXT("General"), LOCTEXT("General", "General"));
	{
		static const FText NameTooltip = LOCTEXT("NameTooltip", "The name is used to determine the long name (the full path) and to provide a unique address within the rig.");
		GeneralCategory.AddCustomRow(FText::FromString(TEXT("Name")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Name")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(NameTooltip)
		]
		.ValueContent()
		[
			SNew(SInlineEditableTextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FRigModuleInstanceDetails::GetName)
			.OnTextCommitted(this, &FRigModuleInstanceDetails::SetName, DetailBuilder.GetPropertyUtilities())
			.ToolTipText(NameTooltip)
			.OnVerifyTextChanged(this, &FRigModuleInstanceDetails::OnVerifyNameChanged)
		];

		const TSharedRef<SWidget> BrowseToRigModuleAssetButton = 
			PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FRigModuleInstanceDetails::OnBrowseToRigModuleAssetClicked));
		
		const FText RigClassLabel = LOCTEXT("RigClassLabel", "Rig Class");

		GeneralCategory.AddCustomRow(RigClassLabel)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(RigClassLabel)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 6.f, 0.f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &FRigModuleInstanceDetails::GetAssetName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[	
				BrowseToRigModuleAssetButton
			]
		];

		const FText RigClassPathLabel = LOCTEXT("RigClassPathLabel", "Rig Class Path");

		GeneralCategory.AddCustomRow(RigClassPathLabel)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(RigClassPathLabel)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(this, &FRigModuleInstanceDetails::GetAssetPath)
			.ToolTipText(this, &FRigModuleInstanceDetails::GetAssetPath)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		GeneralCategory.AddCustomRow(FText::FromString(TEXT("Construction Start Spawn Index")), true /* advanced */)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Construction Start Index")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(LOCTEXT("ConstructionStartSpawnIndexToolTip", "The construction spawn index up to which the module can read from,\nconnectors can connect to. This represents any elements that spawned\nduring construction prior to this module."))
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(this, &FRigModuleInstanceDetails::GetConstructionStartSpawnIndex)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		GeneralCategory.AddCustomRow(FText::FromString(TEXT("Post-Construction Start Spawn Index")), true /* advanced */)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Post-Construction Start Index")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(LOCTEXT("PostConstructionStartSpawnIndexToolTip", "The post construction spawn index up to which the module can read from,\nconnectors can connect to. This represents any elements that spawned\nduring construction of any module - as well as during\npost-construction prior to this module."))
		]
		.ValueContent()
		[
			SNew(STextBlock)
			.Text(this, &FRigModuleInstanceDetails::GetPostConstructionStartSpawnIndex)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		GeneralCategory.AddCustomRow(FText::FromString(TEXT("Variant Tags")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Variant Tags")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(true)
		]
		.ValueContent()
		[
			SNew(SRigVMVariantTagWidget)
			.Orientation(EOrientation::Orient_Horizontal)
			.CanAddTags(false)
			.EnableContextMenu(false)
			.OnGetTags_Lambda([WeakThisPtr]() -> TArray<FRigVMTag>
			{
				TArray<FRigVMTag> Tags;
				if (TSharedPtr<FRigModuleInstanceDetails> StrongThis = WeakThisPtr.Pin())
				{
					const TArray<FPerModuleInfo>& Infos = StrongThis->PerModuleInfos;
					for (int32 InfoIndex=0; InfoIndex<Infos.Num(); ++InfoIndex)
					{
						const FPerModuleInfo& ModuleInfo = Infos[InfoIndex]; 
						if(ModuleInfo.Module.IsValid())
						{
							if (const FRigModuleInstance* Module = ModuleInfo.GetModule())
							{
								FRigVMVariant Variant = Module->GetRig()->GetAssetReference().GetVariant();	
								if (Variant.IsValid())
								{
									if (InfoIndex == 0)
									{
										Tags = Variant.Tags;
									}
									else
									{
										const TArray<FRigVMTag>& OtherTags = Variant.Tags;
										bool bSameArray = Tags.Num() == OtherTags.Num();
										if (bSameArray)
										{
											for (const FRigVMTag& OtherTag : OtherTags)
											{
												if (!Tags.ContainsByPredicate([OtherTag](const FRigVMTag& Tag) { return OtherTag.Name == Tag.Name; }))
												{
													return {};
												}
											}
										}
										else
										{
											return {};
										}
									}
								}
							}
						}
					}
				}
				return Tags;
			})
		];
	}

	IDetailCategoryBuilder& ConnectionsCategory = DetailBuilder.EditCategory(TEXT("Connections"), LOCTEXT("Connections", "Connections"));
	{
		bool bDisplayConnectors = PerModuleInfos.Num() >= 1;
		if (PerModuleInfos.Num() > 1)
		{
			UModularRig* ModularRig = PerModuleInfos[0].GetModularRig();
			for (FPerModuleInfo& Info : PerModuleInfos)
			{
				if (Info.GetModularRig() != ModularRig)
				{
					bDisplayConnectors = false;
					break;
				}
			}
		}

		TArray<FRigModuleConnector> Connectors;
		TArray<TOptional<bool>> IsArrayConnector;
		if (bDisplayConnectors)
		{
			Connectors = GetConnectors();
			
			// sort connectors primary first, then secondary, then optional
			Algo::SortBy(Connectors, [](const FRigModuleConnector& Connector) -> int32
			{
				return Connector.IsPrimary() ? 0 : (Connector.IsOptional() ? 2 : 1);
			});

			IsArrayConnector.Reserve(Connectors.Num());
			
			for(const FRigModuleConnector& Connector : Connectors)
			{
				const FText Label = FText::FromString(Connector.Name);
				IsArrayConnector.Emplace();
				TOptional<bool>& IsArray = IsArrayConnector.Last();
				
				TArray<FRigElementResolveResult> Matches;
				for (int32 ModuleIndex=0; ModuleIndex<PerModuleInfos.Num(); ++ModuleIndex)
				{
					const FPerModuleInfo& Info = PerModuleInfos[ModuleIndex];
					if (const FRigModuleInstance* Module = Info.GetModule())
					{
						if (UModularRig* ModularRig = Info.GetModularRig())
						{
							if (URigHierarchy* Hierarchy = ModularRig->GetHierarchy())
							{
								const FRigHierarchyModulePath ConnectorModulePath(Info.ModuleName.ToString(), Connector.Name);
								FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
								if (FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Hierarchy->Find(ConnectorKey)))
								{
									if(ModuleIndex == 0)
									{
										IsArray = ConnectorElement->IsArrayConnector();
									}
									else if(IsArray.Get(false) != ConnectorElement->IsArrayConnector())
									{
										IsArray.Reset();
									}
								}
							}
						}
					}
				}
			}
		}

		if(bDisplayConnectors)
		{
			for(int32 ConnectorIndex = 0; ConnectorIndex < Connectors.Num(); ConnectorIndex++)
			{
				const FRigModuleConnector& Connector = Connectors[ConnectorIndex];
				const FString ConnectorName = Connector.Name;
				const FText Label = FText::FromString(Connector.Name);
				bool bIsArrayConnector = IsArrayConnector[ConnectorIndex].Get(false);

				const FRigHierarchyModulePath ConnectorModulePath(PerModuleInfos[0].ModuleName.ToString(), Connector.Name);
				const FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);

				TOptional<TArray<FRigElementKey>> TargetKeys;
				for (int32 ModuleIndex=0; ModuleIndex<PerModuleInfos.Num(); ++ModuleIndex)
				{
					if(const FControlRigAssetInterfacePtr Blueprint = PerModuleInfos[ModuleIndex].GetAsset())
					{
						const TArray<FRigElementKey> CurrentTargets = Blueprint->GetModularRigModel().Connections.FindTargetsFromConnector(ConnectorKey);
						if (!TargetKeys.IsSet())
						{
							TargetKeys = CurrentTargets;
						}
						else
						{
							const TArray<FRigElementKey>& FlatKeys = TargetKeys.GetValue();
							if(FlatKeys.Num() != CurrentTargets.Num())
							{
								TargetKeys.Reset();
								break;
							}

							for(int32 TargetKeyIndex = 0; TargetKeyIndex < CurrentTargets.Num(); TargetKeyIndex++)
							{
								if(FlatKeys[TargetKeyIndex] != CurrentTargets[TargetKeyIndex])
								{
									TargetKeys.Reset();
									break;
								}
							}

							if(!TargetKeys.IsSet())
							{
								break;
							}
						}
					}
				}

				static const FSlateBrush* PrimaryBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorPrimary");
				static const FSlateBrush* SecondaryBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorSecondary");
				static const FSlateBrush* OptionalBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorOptional");

				const FSlateBrush* IconBrush = Connector.IsPrimary() ? PrimaryBrush : (Connector.IsOptional() ? OptionalBrush : SecondaryBrush);

				const FText TooltipText = GetTooltipForConnector(ConnectorKey);
				const EVerticalAlignment ContentVerticalAlignment = bIsArrayConnector ? VAlign_Top : VAlign_Center;
				const float MarginTop = bIsArrayConnector ? 6.f : 2.f;

				FDetailWidgetRow& ConnectorRow = ConnectionsCategory.AddCustomRow(Label)
				.NameContent()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SControlRigDragDropBox)
					.OnDragDetected(this, &FRigModuleInstanceDetails::OnConnectorDragDetected, ConnectorKey)
					[
						SNew(SHorizontalBox)
						.ToolTipText(TooltipText)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.f, MarginTop, 2.f, 2.f)
						.HAlign(HAlign_Left)
						.VAlign(ContentVerticalAlignment)
						[
							SNew(SImage)
							.Image(IconBrush)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.DesiredSizeOverride(FVector2D(16, 16))
						]
									
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.f, MarginTop, 2.f, 2.f)
						.HAlign(HAlign_Left)
						.VAlign(ContentVerticalAlignment)
						[
							SNew(STextBlock)
							.Text(Label)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.IsEnabled(true)
						]			
					]
				];

				if(TargetKeys.IsSet())
				{
					FRigHierarchyTreeDelegates RigTreeDelegates;
					RigTreeDelegates.OnGetHierarchy = FOnGetRigTreeHierarchy::CreateLambda([WeakThisPtr]() -> const URigHierarchy*
						{
							if (TSharedPtr<FRigModuleInstanceDetails> StrongThis = WeakThisPtr.Pin())
							{
								if (UModularRig* ModularRig = StrongThis->PerModuleInfos[0].GetModularRig())
								{
									return ModularRig->GetHierarchy();
								}
							}
							return nullptr;
						});

					const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
						.ToolTipText(TooltipText)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.f, 0.f, 0.f, 0.f)
						[
							SNew(SRigConnectorTargetWidget)
							.Outer(PerModuleInfos[0].GetAsset().GetObject())
							.ConnectorKey(ConnectorKey)
							.IsArray(bIsArrayConnector)
							.ExpandArrayByDefault(true)
							.Targets(TargetKeys.GetValue())
							.OnSetTargetArray(FRigConnectorTargetWidget_SetTargetArray::CreateSP(this, &FRigModuleInstanceDetails::OnConnectorTargetChanged, Connector))
							.RigTreeDelegates(RigTreeDelegates)
						];
					
					ConnectorRow
					.ValueContent()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						HorizontalBox
					];

					if(!bIsArrayConnector)
					{
						// Reset button
						HorizontalBox->AddSlot()
						.AutoWidth()
						.Padding(4.f, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SAssignNew(ResetConnectorButton.FindOrAdd(Connector.Name), SButton)
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.ButtonColorAndOpacity_Lambda([this, Connector]()
							{
								const TSharedPtr<SButton>& Button = ResetConnectorButton.FindRef(Connector.Name);
								return Button.IsValid() && Button->IsHovered()
									? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
									: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
							})
							.OnClicked_Lambda([this, Connector]()
							{
								for (FPerModuleInfo& Info : PerModuleInfos)
								{
									const FRigHierarchyModulePath ConnectorModulePath(Info.ModuleName.ToString(), Connector.Name);
									FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
									Info.GetAsset()->GetModularRigController()->DisconnectConnector(ConnectorKey);
								}
								return FReply::Handled();
							})
							.ContentPadding(1.f)
							.ToolTipText(NSLOCTEXT("ControlRigModuleDetails", "Reset_Connector", "Reset Connector"))
							[
								SNew(SImage)
								//.DesiredSizeOverride(FVector2D(16,16))
								.ColorAndOpacity_Lambda( [this, Connector]()
								{
									const TSharedPtr<SButton>& Button = ResetConnectorButton.FindRef(Connector.Name);
									return Button.IsValid() && Button->IsHovered()
									? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
									: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
								})
								.Image(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault").GetIcon())
							]
						];

						// Use button
						HorizontalBox->AddSlot()
						.AutoWidth()
						.Padding(0.f, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SAssignNew(UseSelectedButton.FindOrAdd(Connector.Name), SButton)
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.ButtonColorAndOpacity_Lambda([this, Connector]()
							{
								const TSharedPtr<SButton>& Button = UseSelectedButton.FindRef(Connector.Name);
								return Button.IsValid() && Button->IsHovered()
									? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
									: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
							})
							.OnClicked_Lambda([this, Connector]()
							{
								if (UModularRig* ModularRig = PerModuleInfos[0].GetModularRig())
								{
									const TArray<FRigElementKey>& Selected = ModularRig->GetHierarchy()->GetSelectedKeys();
									if (Selected.Num() > 0)
									{
										for (FPerModuleInfo& Info : PerModuleInfos)
										{
											const FRigHierarchyModulePath ConnectorModulePath(Info.ModuleName.ToString(), Connector.Name);
											FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
											Info.GetAsset()->GetModularRigController()->ConnectConnectorToElements(ConnectorKey, Selected, true, ModularRig->GetModularRigSettings().bAutoResolve);
										}
									}
								}
								return FReply::Handled();
							})
							.ContentPadding(1.f)
							.ToolTipText(NSLOCTEXT("ControlRigModuleDetails", "Use_Selected", "Use Selected"))
							[
								SNew(SImage)
								//.DesiredSizeOverride(FVector2D(16,16))
								.ColorAndOpacity_Lambda( [this, Connector]()
								{
									const TSharedPtr<SButton>& Button = UseSelectedButton.FindRef(Connector.Name);
									return Button.IsValid() && Button->IsHovered()
									? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
									: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
								})
								.Image(FAppStyle::GetBrush("Icons.CircleArrowLeft"))
							]
						];

						// Select in hierarchy button
						HorizontalBox->AddSlot()
						.AutoWidth()
						.Padding(0.f, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SAssignNew(SelectElementButton.FindOrAdd(Connector.Name), SButton)
							.ButtonStyle( FAppStyle::Get(), "NoBorder" )
							.ButtonColorAndOpacity_Lambda([this, Connector]()
							{
								const TSharedPtr<SButton>& Button = SelectElementButton.FindRef(Connector.Name);
								return Button.IsValid() && Button->IsHovered()
									? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
									: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
							})
							.OnClicked_Lambda([this, Connector]()
							{
								if (UModularRig* ModularRig = PerModuleInfos[0].GetModularRig())
								{
									const FRigHierarchyModulePath ConnectorModulePath(PerModuleInfos[0].ModuleName.ToString(), Connector.Name);
									FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
									if (const FRigElementKeyRedirector::FKeyArray* TargetKeys = ModularRig->GetElementKeyRedirector().FindExternalKey(ConnectorKey))
									{
										TArray<FRigElementKey> KeysToSelect;
										KeysToSelect.Append(*TargetKeys);
										ModularRig->GetHierarchy()->GetController()->SetSelection(KeysToSelect, true);
									}
								}
								return FReply::Handled();
							})
							.ContentPadding(1.f)
							.ToolTipText(NSLOCTEXT("ControlRigModuleDetails", "Select_Element", "Select Element"))
							[
								SNew(SImage)
								//.DesiredSizeOverride(FVector2D(16,16))
								.ColorAndOpacity_Lambda( [this, Connector]()
								{
									const TSharedPtr<SButton>& Button = SelectElementButton.FindRef(Connector.Name);
									return Button.IsValid() && Button->IsHovered()
									? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
									: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
								})
								.Image(FAppStyle::GetBrush("Icons.Search"))
							]
						];
					}
				}
				else
				{
					ConnectorRow
					.ValueContent()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ControlRigModuleDetailsMultipleValues)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					];
				}
			}
		}
	}

	for(const FName& OriginalCategoryName : OriginalCategoryNames)
	{
		IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(OriginalCategoryName);
		{
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

			TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
			Category.GetDefaultProperties(DefaultProperties, true, true);

			// Property handles backed by per-module UControlRigWrapperObjects (BP-independent rigs).
			// Edits on these are dispatched through each wrapper's WrappedPropertyChangedChainEvent
			// (bound to OnWrapperConfigValueChanged), so the per-property
			// SetOnPropertyValueChangedWithData(OnConfigValueChanged) hook is skipped for them to
			// avoid double-firing. GetDefaultProperties is expected to return only top-level
			// handles, so a Contains() check against this set is sufficient (no parent walk).
			TSet<TSharedRef<IPropertyHandle>> WrapperBackedProperties;

			if (const FRigModuleReference* ModuleReference = PerModuleInfos[0].GetReference())
			{
				FControlRigAssetStrongReference AssetReference = ModuleReference->ControlRigAssetReference.LoadStrongReference();
				if (AssetReference.GetControlRigAsset())
				{
					if (URigVMRuntimeAsset* RuntimeAsset = Cast<URigVMRuntimeAsset>(AssetReference.Get()))
					{
						// Build one UControlRigWrapperObject per module. Each wrapper:
						//   * mirrors the rig's variables struct via a dynamically-generated UClass
						//   * carries the owning UControlRig as its Subject (so customizations
						//     looking up the rig via GetOuterObjects() can discover it)
						//   * broadcasts WrappedPropertyChangedChainEvent on edits, which we route
						//     through OnWrapperConfigValueChanged -> OnConfigValueChanged.
						TArray<UObject*> WrapperObjects;
						WrapperObjects.Reserve(PerModuleInfos.Num());

						for (FPerModuleInfo& Info : PerModuleInfos)
						{
							FRigModuleInstance* ModuleInstance = Info.GetModule();
							UControlRig* ModuleRig = ModuleInstance ? ModuleInstance->GetRig() : nullptr;
							UScriptStruct* ModuleVariablesStruct = ModuleRig ? Cast<UScriptStruct>(ModuleRig->GetVariablesStruct()) : nullptr;
							uint8* ModuleVariablesMemory = ModuleRig ? ModuleRig->GetVariablesMemory() : nullptr;
							if (!ModuleVariablesStruct || !ModuleVariablesMemory)
							{
								UE_LOG(LogControlRig, Verbose, TEXT("FRigModuleInstanceDetails: skipping module '%s' — no variables struct/memory (likely a BP-based rig falling through the BPI branch)."),
									*Info.ModuleName.ToString());
								continue;
							}

							UControlRigWrapperObject* Wrapper = Cast<UControlRigWrapperObject>(
								URigVMDetailsViewWrapperObject::MakeInstance(
									UControlRigWrapperObject::StaticClass(),
									GetTransientPackage(),
									ModuleVariablesStruct,
									ModuleVariablesMemory,
									/*InSubject=*/ ModuleRig));
							if (!Wrapper)
							{
								continue;
							}

							// Seed wrapper UPROPERTY storage from live rig memory.
							Wrapper->SetContent(ModuleVariablesMemory, ModuleVariablesStruct);

							Wrapper->GetWrappedPropertyChangedChainEvent().AddSP(
								SharedThis(this),
								&FRigModuleInstanceDetails::OnWrapperConfigValueChanged);

							PerModuleWrappers.Add(TStrongObjectPtr<UControlRigWrapperObject>(Wrapper));
							WrapperObjects.Add(Wrapper);
						}

						if (!WrapperObjects.IsEmpty())
						{
							TArray<FRigVMExternalVariable> Variables = AssetReference.GetExternalVariables();
							// Cheap insurance against rehashes during the loop.
							WrapperBackedProperties.Reserve(WrapperBackedProperties.Num() + Variables.Num());
							DefaultProperties.Reserve(DefaultProperties.Num() + Variables.Num());

							for (FRigVMExternalVariable& Variable : Variables)
							{
								const FName Name = FRigVMPropertyBag::SanitizePropertyName(Variable.GetName());
								TSharedPtr<IPropertyHandle> WrapperProperty = DetailBuilder.AddObjectPropertyData(WrapperObjects, Name);
								if (WrapperProperty.IsValid())
								{
									TSharedRef<IPropertyHandle> WrapperPropertyRef = WrapperProperty.ToSharedRef();
									DefaultProperties.Add(WrapperPropertyRef);
									WrapperBackedProperties.Add(WrapperPropertyRef);
								}
								else
								{
									// Asset variable list and the modules' actual VariablesStruct have diverged
									// (e.g., hot-reload, asset re-cook, stale instance). Edit-row missing rather
									// than a crash — log so it's traceable from the panel.
									UE_LOG(LogControlRig, Verbose, TEXT("FRigModuleInstanceDetails: variable '%s' from asset external-variable list not present on every module's wrapper class — row omitted."),
										*Name.ToString());
								}
							}
						}
					}
				}
			}

			for(const TSharedRef<IPropertyHandle>& DefaultProperty : DefaultProperties)
			{
				const FProperty* Property = DefaultProperty->GetProperty();
				if(Property == nullptr)
				{
					DetailBuilder.HideProperty(DefaultProperty);
					continue;
				}

				// skip advanced properties for now
				const bool bAdvancedDisplay = Property->HasAnyPropertyFlags(CPF_AdvancedDisplay);
				if(bAdvancedDisplay)
				{
					DetailBuilder.HideProperty(DefaultProperty);
					continue;
				}

				// skip non-public properties for now
				const bool bIsPublic = Property->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst);
				const bool bIsInstanceEditable = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
				if(!bIsPublic || !bIsInstanceEditable)
				{
					DetailBuilder.HideProperty(DefaultProperty);
					continue;
				}

				// Wrapper-backed handles drive override writes through the wrapper's chain event
				// (see PerModuleWrappers / OnWrapperConfigValueChanged); skip the per-handle hook
				// here so OnConfigValueChanged doesn't fire twice for the same edit.
				if (!WrapperBackedProperties.Contains(DefaultProperty))
				{
					TDelegate<void(const FPropertyChangedEvent&)> OnValueChangedDelegate = TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FRigModuleInstanceDetails::OnConfigValueChanged);
					DefaultProperty->SetOnPropertyValueChangedWithData(OnValueChangedDelegate);
					DefaultProperty->SetOnChildPropertyValueChangedWithData(OnValueChangedDelegate);
				}

				FPropertyBindingWidgetArgs BindingArgs;
				BindingArgs.Property = (FProperty*)Property;
				BindingArgs.CurrentBindingText = TAttribute<FText>::CreateLambda([this, Property]()
				{
					return GetBindingText(Property);
				});
				BindingArgs.CurrentBindingImage = TAttribute<const FSlateBrush*>::CreateLambda([this, Property]()
				{
					return GetBindingImage(Property);
				});
				BindingArgs.CurrentBindingColor = TAttribute<FLinearColor>::CreateLambda([this, Property]()
				{
					return GetBindingColor(Property);
				});

				BindingArgs.OnCanBindPropertyWithBindingChain.BindLambda([](const FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain) -> bool { return true; });
				BindingArgs.OnCanBindToClass.BindLambda([](UClass* InClass) -> bool { return false; });
				BindingArgs.OnCanRemoveBinding.BindRaw(this, &FRigModuleInstanceDetails::CanRemoveBinding);
				BindingArgs.OnRemoveBinding.BindSP(this, &FRigModuleInstanceDetails::HandleRemoveBinding);

				BindingArgs.bGeneratePureBindings = true;
				BindingArgs.bAllowNewBindings = true;
				BindingArgs.bAllowArrayElementBindings = false;
				BindingArgs.bAllowStructMemberBindings = false;
				BindingArgs.bAllowUObjectFunctions = false;

				BindingArgs.MenuExtender = MakeShareable(new FExtender);
				BindingArgs.MenuExtender->AddMenuExtension(
					"Properties",
					EExtensionHook::After,
					nullptr,
					FMenuExtensionDelegate::CreateSPLambda(this, [this, Property](FMenuBuilder& MenuBuilder)
					{
						FillBindingMenu(MenuBuilder, Property);
					})
				);

				TSharedPtr<SWidget> ValueWidget = DefaultProperty->CreatePropertyValueWidgetWithCustomization(DetailBuilder.GetDetailsViewSharedPtr().Get());

				const bool bShowChildren = true;
				Category.AddProperty(DefaultProperty).CustomWidget(bShowChildren)
				.NameContent()
				[
					DefaultProperty->CreatePropertyNameWidget()
				]

				.ValueContent()
				[
					ValueWidget ? ValueWidget.ToSharedRef() : SNullWidget::NullWidget
					// todo: if the property is bound / or partially bound
					// mark the property value widget as disabled / read only.
				]

				.ExtensionContent()
				[
					PropertyAccessEditor.MakePropertyBindingWidget(nullptr, BindingArgs)
				];
			}
		}
	}
}

FText FRigModuleInstanceDetails::GetName() const
{
	const FRigModuleInstance* FirstModule = PerModuleInfos[0].GetModule();
	if(FirstModule == nullptr)
	{
		return FText();
	}
	
	const FName FirstValue = FirstModule->Name;
	if(PerModuleInfos.Num() > 1)
	{
		bool bSame = true;
		for (int32 i=1; i<PerModuleInfos.Num(); ++i)
		{
			if(const FRigModuleInstance* Module = PerModuleInfos[i].GetModule())
			{
				if (!Module->Name.IsEqual(FirstValue, ENameCase::IgnoreCase))
				{
					bSame = false;
					break;
				}
			}
		}
		if (!bSame)
		{
			return ControlRigModuleDetailsMultipleValues;
		}
	}
	return FText::FromName(FirstValue);
}

void FRigModuleInstanceDetails::SetName(const FText& InValue, ETextCommit::Type InCommitType, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	if(InValue.IsEmpty())
	{
		return;
	}
	
	for (FPerModuleInfo& Info : PerModuleInfos)
	{
		if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
		{
			if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
			{
				UModularRigController* Controller = Blueprint->GetModularRigController();
				const FName OldModuleName = ModuleInstance->Name;
				(void)Controller->RenameModule(OldModuleName, *InValue.ToString(), true);
			}
		}
	}
}

bool FRigModuleInstanceDetails::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	if(InText.IsEmpty())
	{
		static const FText EmptyNameIsNotAllowed = LOCTEXT("EmptyNameIsNotAllowed", "Empty name is not allowed.");
		OutErrorMessage = EmptyNameIsNotAllowed;
		return false;
	}

	for (FPerModuleInfo& Info : PerModuleInfos)
	{
		if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
		{
			if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
			{
				UModularRigController* Controller = Blueprint->GetModularRigController();
				if (!Controller->CanRenameModule(ModuleInstance->Name, *InText.ToString(), OutErrorMessage))
				{
					return false;
				}
			}
		}
	}

	return true;
}

TArray<FRigModuleConnector> FRigModuleInstanceDetails::GetConnectors() const
{
	if(PerModuleInfos.Num() > 1)
	{
		TArray<FRigModuleConnector> CommonConnectors;
		if (const FRigModuleInstance* Module = PerModuleInfos[0].GetModule())
		{
			if (const UControlRig* ModuleRig = Module->GetRig())
			{
				CommonConnectors = ModuleRig->GetRigModuleSettings().ExposedConnectors;
			}
		}
		for (int32 ModuleIndex=1; ModuleIndex<PerModuleInfos.Num(); ++ModuleIndex)
		{
			if (const FRigModuleInstance* Module = PerModuleInfos[ModuleIndex].GetModule())
			{
				if (const UControlRig* ModuleRig = Module->GetRig())
				{
					const TArray<FRigModuleConnector>& ModuleConnectors = ModuleRig->GetRigModuleSettings().ExposedConnectors;
					CommonConnectors = CommonConnectors.FilterByPredicate([ModuleConnectors](const FRigModuleConnector& Connector)
					{
						return ModuleConnectors.Contains(Connector);
					});
				}
			}
		}
		return CommonConnectors;
	}

	if (const FRigModuleInstance* Module = PerModuleInfos[0].GetModule())
	{
		if (const UControlRig* ModuleRig = Module->GetRig())
		{
			return ModuleRig->GetRigModuleSettings().ExposedConnectors;
		}
	}

	return TArray<FRigModuleConnector>();
}

FRigElementKeyRedirector FRigModuleInstanceDetails::GetConnections() const
{
	if(PerModuleInfos.Num() > 1)
	{
		return FRigElementKeyRedirector();
	}

	if (const FRigModuleInstance* Module = PerModuleInfos[0].GetModule())
	{
		if (UControlRig* ModuleRig = Module->GetRig())
		{
			return ModuleRig->GetElementKeyRedirector();
		}
	}

	return FRigElementKeyRedirector();
}

void FRigModuleInstanceDetails::OnConfigValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	// Re-entrant entry via OnWrapperConfigValueChanged or undo-during-interactive can race the
	// panel being torn down; guard PerModuleInfos[0] dereference accordingly.
	if (PerModuleInfos.IsEmpty())
	{
		return;
	}

	FControlRigAssetInterfacePtr Blueprint = PerModuleInfos[0].GetAsset();
	if(Blueprint == nullptr)
	{
		return;
	}

	TStringBuilder<256> PathBuilder;

	auto AppendProperty = [&PathBuilder](const FProperty* Property, int32 InArrayIndex)
	{
		if (PathBuilder.Len() > 0)
		{
			PathBuilder.Append(TEXT("->"));
		}
		PathBuilder.Append(Property->GetFName().ToString());
		if (InArrayIndex >= 0)
		{
			PathBuilder.Appendf(TEXT("[%d]"), InArrayIndex);
		}
	};

	{
		FProperty* Property = InPropertyChangedEvent.Property;
		if(!ensure(Property))
		{
			return;
		}

		TMap<FString, int32> PropertyNameStack;
		InPropertyChangedEvent.GetArrayIndicesPerObject(0, PropertyNameStack);

		if (PropertyNameStack.Num() > 0)
		{
			TArray<FString> PropertyNames;
			PropertyNameStack.GetKeys(PropertyNames);

			// the property names in the array are provided in reverse order
			for(int32 Index = PropertyNames.Num() - 1; Index >= 0; Index--)
			{
				if(PathBuilder.Len() == 0)
				{
					// Prefer the module rig's variables struct as the lookup root for the ROOT-most
					// segment of the chain. The override system addresses values against the
					// variables struct (UPropertyBag for BP-independent rigs,
					// UControlRigBlueprintGeneratedClass for BP rigs), so the path-string must be
					// reconstructed against that same struct.
					//
					// Note we cannot just use InPropertyChangedEvent.Property->GetOwnerStruct()
					// for this: Property is the LEAF (e.g. an FName "Name" inside
					// FRigUnit_HierarchyAddControl_ShapeSettings), so its OwnerStruct is the
					// leaf-containing struct, not the root variable's owning struct. The legacy
					// fall-throughs (object class / leaf-owner-struct) only happened to work for
					// the BP path where the leaf was a top-level UControlRig variable.
					//
					// PerModuleInfos[0] is fine for homogeneous multi-select (all modules share a
					// variables struct). Heterogeneous BPI multi-select is not a supported
					// scenario — AddObjectPropertyData would not even merge handles cleanly
					// across modules with disjoint wrapper classes.
					const UStruct* LookupStruct = nullptr;
					if (const FRigModuleInstance* ModuleInstance = PerModuleInfos[0].GetModule())
					{
						if (const UControlRig* ModuleRig = ModuleInstance->GetRig())
						{
							LookupStruct = ModuleRig->GetVariablesStruct();
						}
					}
					if (LookupStruct == nullptr)
					{
						if (InPropertyChangedEvent.GetNumObjectsBeingEdited() > 0)
						{
							LookupStruct = InPropertyChangedEvent.GetObjectBeingEdited(0)->GetClass();
						}
						else
						{
							LookupStruct = InPropertyChangedEvent.Property->GetOwnerStruct();
						}
					}
					Property = FControlRigOverrideValue::FindProperty(LookupStruct, PropertyNames[Index]);
					if(!ensure(Property != nullptr))
					{
						return;
					}
				}
				else
				{
					if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						Property = ArrayProperty->Inner;
					}
				
					const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
					if(!ensure(StructProperty != nullptr))
					{
						return;
					}

					Property = FControlRigOverrideValue::FindProperty(StructProperty->Struct, PropertyNames[Index]);
					if(!ensure(Property))
					{
						return;
					}
				}

				AppendProperty(Property, PropertyNameStack.FindChecked(PropertyNames[Index]));
			}
		}
		else
		{
			AppendProperty(InPropertyChangedEvent.MemberProperty, InPropertyChangedEvent.GetArrayIndex(InPropertyChangedEvent.GetMemberPropertyName().ToString()));
		}
	}

	FString PropertyPathString(PathBuilder);
	
	// we need to shorten the path if we already have data on something above this.
	bool bFoundValueWithShorterPath = true;
	while(bFoundValueWithShorterPath)
	{
		bFoundValueWithShorterPath = false;

		for(const FPerModuleInfo& Info : PerModuleInfos)
		{
			if (const FRigModuleReference* ModuleReference = Blueprint->GetModularRigModel().FindModule(Info.ModuleName))
			{
				for(const FControlRigOverrideValue& Override : ModuleReference->ConfigOverrides)
				{
					const FString& ParentPath = Override.GetPath();
					if(FControlRigOverrideContainer::IsChildPathOf(PropertyPathString, ParentPath))
					{
						bFoundValueWithShorterPath = true;
						PropertyPathString = ParentPath;
						break;
					}
				}
			}
			
			if(bFoundValueWithShorterPath)
			{
				break;
			}
		}
	}

	TMap<FName, FControlRigOverrideValue> ModuleValues;
	ModuleValues.Reserve(PerModuleInfos.Num());
	
	for(const FPerModuleInfo& Info : PerModuleInfos)
	{
		if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
		{
			if (UControlRig* ModuleRig = ModuleInstance->GetRig())
			{
				FControlRigOverrideValue ConfigValue = FControlRigOverrideValue(PropertyPathString, ModuleRig->GetVariablesStruct(), ModuleRig->GetVariablesMemory());
				ModuleValues.Add(ModuleInstance->Name, ConfigValue);
			}
		}
	}

	FScopedTransaction Transaction(LOCTEXT("ModuleHierarchySetConfigValue", "Set Module Config Value"));
	for (const TPair<FName, FControlRigOverrideValue>& Value : ModuleValues)
	{
		UModularRigController* Controller = Blueprint->GetModularRigController();
		Controller->SetConfigValueInModule(Value.Key, Value.Value);
	}
}

void FRigModuleInstanceDetails::OnWrapperConfigValueChanged(URigVMDetailsViewWrapperObject* InWrapper, const FString& InPropertyPath, FPropertyChangedChainEvent& InEvent)
{
	if (InWrapper == nullptr)
	{
		return;
	}

	// Re-entrancy guard: AddObjectPropertyData fans the merged handle's PostEditChange across
	// all N wrappers, so this delegate fires N times for a single user edit. The first call
	// already writes the value to every module via OnConfigValueChanged's PerModuleInfos loop,
	// so collapse the rest.
	if (bDispatchingWrapperEvent)
	{
		return;
	}
	TGuardValue<bool> DispatchGuard(bDispatchingWrapperEvent, true);

	// Load-bearing: copy wrapper UPROPERTY state into the rig's variables memory BEFORE calling
	// OnConfigValueChanged. OnConfigValueChanged constructs FControlRigOverrideValue from
	// (path, variables-struct, variables-memory) — without this copy it would read the stale
	// pre-edit value from rig memory and SetConfigValueInModule would record a no-op override.
	// (The downstream SetConfigValueInModule -> ConfigOverrides.CopyToSubject sync that runs
	// later happens to land on the same value, but it is not what propagates the edit here.)
	if (UControlRig* ModuleRig = Cast<UControlRig>(InWrapper->GetSubject()))
	{
		if (UStruct* VariablesStruct = ModuleRig->GetVariablesStruct())
		{
			InWrapper->GetContent(ModuleRig->GetVariablesMemory(), VariablesStruct);
		}
	}

	OnConfigValueChanged(InEvent);
}

bool FRigModuleInstanceDetails::OnConnectorTargetChanged(TArray<FRigElementKey> InTargets, FRigModuleConnector InConnector)
{
	bool bResult = true;
	FScopedTransaction Transaction(LOCTEXT("ModuleHierarchyResolveConnector", "Resolve Connector"));
	for (FPerModuleInfo& Info : PerModuleInfos)
	{
		if (UModularRigController* Controller = Info.GetAsset()->GetModularRigController())
		{
			const FRigHierarchyModulePath ConnectorModulePath(Info.ModuleName.ToString(), InConnector.Name);
			FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
			
			if (!InTargets.IsEmpty())
			{
				const FModularRigSettings& Settings = Info.GetModularRig()->GetModularRigSettings();
				if(!Controller->ConnectConnectorToElements(ConnectorKey, InTargets, true, Settings.bAutoResolve))
				{
					bResult = false;
				}
			}
			else
			{
				if(!Controller->DisconnectConnector(ConnectorKey))
				{
					bResult = false;
				}
			}
		}
	}
	return bResult;
}

FText FRigModuleInstanceDetails::GetConstructionStartSpawnIndex() const
{
	if(PerModuleInfos.Num() > 1)
	{
		return ControlRigModuleDetailsMultipleValues;
	}
	if(PerModuleInfos[0].GetModule()->ConstructionSpawnStartIndex == INDEX_NONE)
	{
		static const FText ConstructionWasNeverRun = LOCTEXT("ConstructionNeverRun", "Construction was never run.");
		return ConstructionWasNeverRun;
	}
	return FText::FromString(FString::FromInt(PerModuleInfos[0].GetModule()->ConstructionSpawnStartIndex));
}

FText FRigModuleInstanceDetails::GetPostConstructionStartSpawnIndex() const
{
	if(PerModuleInfos.Num() > 1)
	{
		return ControlRigModuleDetailsMultipleValues;
	}
	if(PerModuleInfos[0].GetModule()->PostConstructionSpawnStartIndex == INDEX_NONE)
	{
		static const FText PostConstructionWasNeverRun = LOCTEXT("PostConstructionNeverRun", "Post-Construction was never run.");
		return PostConstructionWasNeverRun;
	}
	return FText::FromString(FString::FromInt(PerModuleInfos[0].GetModule()->PostConstructionSpawnStartIndex));
}

const FRigModuleInstanceDetails::FPerModuleInfo& FRigModuleInstanceDetails::FindModule(const FName& InModuleName) const
{
	const FPerModuleInfo* Info = FindModuleByPredicate([InModuleName](const FPerModuleInfo& Info)
	{
		if(const FRigModuleInstance* Module = Info.GetModule())
		{
			return Module->Name == InModuleName;
		}
		return false;
	});

	if(Info)
	{
		return *Info;
	}

	static const FPerModuleInfo EmptyInfo;
	return EmptyInfo;
}

const FRigModuleInstanceDetails::FPerModuleInfo* FRigModuleInstanceDetails::FindModuleByPredicate(const TFunction<bool(const FPerModuleInfo&)>& InPredicate) const
{
	return PerModuleInfos.FindByPredicate(InPredicate);
}

bool FRigModuleInstanceDetails::ContainsModuleByPredicate(const TFunction<bool(const FPerModuleInfo&)>& InPredicate) const
{
	return PerModuleInfos.ContainsByPredicate(InPredicate);
}

void FRigModuleInstanceDetails::RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass)
{
	TSharedRef<FPropertySection> MetadataSection = PropertyEditorModule.FindOrCreateSection(InClass->GetFName(), "Metadata", LOCTEXT("Metadata", "Metadata"));
	MetadataSection->AddCategory("Metadata");
}

FText FRigModuleInstanceDetails::GetBindingText(const FProperty* InProperty) const
{
	const FName VariableName = InProperty->GetFName();
	FText FirstValue;
	for (int32 ModuleIndex=0; ModuleIndex<PerModuleInfos.Num(); ++ModuleIndex)
	{
		if (const FRigModuleReference* ModuleReference = PerModuleInfos[ModuleIndex].GetReference())
		{
			if(ModuleReference->Bindings.Contains(VariableName))
			{
				const FText BindingText = FText::FromString(ModuleReference->Bindings.FindChecked(VariableName));
				if(ModuleIndex == 0)
				{
					FirstValue = BindingText;
				}
				else if(!FirstValue.EqualTo(BindingText))
				{
					return ControlRigModuleDetailsMultipleValues;
				}
			}
		}
	}
	return FirstValue;
}

const FSlateBrush* FRigModuleInstanceDetails::GetBindingImage(const FProperty* InProperty) const
{
	static const FLazyName TypeIcon(TEXT("Kismet.VariableList.TypeIcon"));
	static const FLazyName ArrayTypeIcon(TEXT("Kismet.VariableList.ArrayTypeIcon"));

	if(CastField<FArrayProperty>(InProperty))
	{
		return FAppStyle::GetBrush(ArrayTypeIcon);
	}
	return FAppStyle::GetBrush(TypeIcon);
}

FLinearColor FRigModuleInstanceDetails::GetBindingColor(const FProperty* InProperty) const
{
	if(InProperty)
	{
		FEdGraphPinType PinType;
		const UEdGraphSchema_K2* Schema_K2 = GetDefault<UEdGraphSchema_K2>();
		if (Schema_K2->ConvertPropertyToPinType(InProperty, PinType))
		{
			const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
			return Schema->GetPinTypeColor(PinType);
		}
	}
	return FLinearColor::White;
}

void FRigModuleInstanceDetails::FillBindingMenu(FMenuBuilder& MenuBuilder, const FProperty* InProperty) const
{
	if(PerModuleInfos.IsEmpty())
	{
		return;
	}

	FControlRigAssetInterfacePtr Blueprint = PerModuleInfos[0].GetAsset();
	UModularRigController* Controller = Blueprint->GetModularRigController();

	TArray<FString> CombinedBindings;
	for(int32 Index = 0; Index < PerModuleInfos.Num(); Index++)
	{
		const FPerModuleInfo& Info  = PerModuleInfos[Index];
		const TArray<FString> Bindings = Controller->GetPossibleBindings(Info.ModuleName, InProperty->GetFName());
		if(Index == 0)
		{
			CombinedBindings = Bindings;
		}
		else
		{
			// reduce the set of bindings to the overall possible bindings
			CombinedBindings.RemoveAll([Bindings](const FString& Binding)
			{
				return !Bindings.Contains(Binding);
			});
		}
	}

	if(CombinedBindings.IsEmpty())
	{
		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction()),
			SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoBindingAvailable", "No bindings available for this property."))
					.ColorAndOpacity(FLinearColor::White)
				]
			);
		return;
	}

	// sort lexically
	CombinedBindings.Sort();

	// create a map of all of the variables per menu prefix (the module path the variables belong to)
	struct FPerMenuData
	{
		FString Name;
		FString ParentMenuPath;
		TArray<FString> SubMenuPaths;
		TArray<FString> Variables;

		static void SetupMenu(
			TSharedRef<FRigModuleInstanceDetails const> ThisDetails,
			const FProperty* InProperty,
			FMenuBuilder& InMenuBuilder,
			const FString& InMenuPath,
			TSharedRef<TMap<FString, FPerMenuData>> PerMenuData)
		{
			FPerMenuData& Data = PerMenuData->FindChecked((InMenuPath));

			Data.SubMenuPaths.Sort();
			Data.Variables.Sort();

			for(const FString& VariablePath : Data.Variables)
			{
				FString VariableName = VariablePath;
				(void)FRigHierarchyModulePath(VariablePath).Split(nullptr, &VariableName);
				
				InMenuBuilder.AddMenuEntry(
					FUIAction(FExecuteAction::CreateLambda([ThisDetails, InProperty, VariablePath]()
					{
						ThisDetails->HandleChangeBinding(InProperty, VariablePath);
					})),
					SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(1.0f, 0.0f)
						[
							SNew(SImage)
							.Image(ThisDetails->GetBindingImage(InProperty))
							.ColorAndOpacity(ThisDetails->GetBindingColor(InProperty))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(FText::FromString(VariableName))
							.ColorAndOpacity(FLinearColor::White)
						]
					);
			}

			for(const FString& SubMenuPath : Data.SubMenuPaths)
			{
				const FPerMenuData& SubMenuData = PerMenuData->FindChecked(SubMenuPath);

				const FText Label = FText::FromString(SubMenuData.Name);
				static const FText TooltipFormat = LOCTEXT("BindingMenuTooltipFormat", "Access to all variables of the {0} module");
				const FText Tooltip = FText::Format(TooltipFormat, Label);  
				InMenuBuilder.AddSubMenu(Label, Tooltip, FNewMenuDelegate::CreateLambda([ThisDetails, InProperty, SubMenuPath, PerMenuData](FMenuBuilder& SubMenuBuilder)
				{
					SetupMenu(ThisDetails, InProperty, SubMenuBuilder, SubMenuPath, PerMenuData);
				}));
			}
		}
	};
	
	// define the root menu
	const TSharedRef<TMap<FString, FPerMenuData>> PerMenuData = MakeShared<TMap<FString, FPerMenuData>>();
	PerMenuData->FindOrAdd(FString());

	// make sure all levels of the menu are known and we have the variables available
	for(const FString& BindingPath : CombinedBindings)
	{
		FString MenuPath;
		(void)FRigHierarchyModulePath(BindingPath).Split(&MenuPath, nullptr);

		FString PreviousMenuPath = MenuPath;
		FString ParentMenuPath = MenuPath, RemainingPath;
		while(FRigHierarchyModulePath(ParentMenuPath).Split(&ParentMenuPath, &RemainingPath))
		{
			// scope since the map may change at the end of this block
			{
				FPerMenuData& Data = PerMenuData->FindOrAdd(MenuPath);
				if(Data.Name.IsEmpty())
				{
					Data.Name = RemainingPath;
				}
			}
			
			PerMenuData->FindOrAdd(ParentMenuPath).SubMenuPaths.AddUnique(PreviousMenuPath);
			PerMenuData->FindOrAdd(PreviousMenuPath).ParentMenuPath = ParentMenuPath;
			PerMenuData->FindOrAdd(PreviousMenuPath).Name = RemainingPath;
			if(!ParentMenuPath.Contains(FRigHierarchyModulePath::ModuleNameSuffix))
			{
				PerMenuData->FindOrAdd(FString()).SubMenuPaths.AddUnique(ParentMenuPath);
				PerMenuData->FindOrAdd(ParentMenuPath).Name = ParentMenuPath;
			}
			PreviousMenuPath = ParentMenuPath;
		}

		FPerMenuData& Data = PerMenuData->FindOrAdd(MenuPath);
		if(Data.Name.IsEmpty())
		{
			Data.Name = MenuPath;
		}

		Data.Variables.Add(BindingPath);
		if(!MenuPath.IsEmpty())
		{
			PerMenuData->FindChecked(Data.ParentMenuPath).SubMenuPaths.AddUnique(MenuPath);
		}
	}

	// build the menu
	FPerMenuData::SetupMenu(SharedThis(this), InProperty, MenuBuilder, FString(), PerMenuData);
}

bool FRigModuleInstanceDetails::CanRemoveBinding(FName InPropertyName) const
{
	// offer the "removing binding" button if any of the selected module instances
	// has a binding for the given variable
	for(const FPerModuleInfo& Info : PerModuleInfos)
	{
		if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
		{
			if(ModuleInstance->VariableBindings.Contains(InPropertyName))
			{
				return true;
			}
		}
	}
	return false; 
}

void FRigModuleInstanceDetails::HandleRemoveBinding(FName InPropertyName) const
{
	FScopedTransaction Transaction(LOCTEXT("RemoveModuleVariableTransaction", "Remove Binding"));
	for(const FPerModuleInfo& Info : PerModuleInfos)
	{
		if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
		{
			if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
			{
				UModularRigController* Controller = Blueprint->GetModularRigController();
				Controller->UnBindModuleVariable(ModuleInstance->Name, InPropertyName);
			}
		}
	}
}

void FRigModuleInstanceDetails::HandleChangeBinding(const FProperty* InProperty, const FString& InNewVariablePath) const
{
	FScopedTransaction Transaction(LOCTEXT("BindModuleVariableTransaction", "Bind Module Variable"));
	for(const FPerModuleInfo& Info : PerModuleInfos)
	{
		if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
		{
			if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
			{
				UModularRigController* Controller = Blueprint->GetModularRigController();
				Controller->BindModuleVariable(ModuleInstance->Name, InProperty->GetFName(), InNewVariablePath);
			}
		}
	}
}

FReply FRigModuleInstanceDetails::OnAddTargetToArrayConnector(const FString InConnectorName, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	FScopedTransaction Transaction(LOCTEXT("AddTargetToArrayConnector", "Add Target To Array Connector"));
	for(const FPerModuleInfo& Info : PerModuleInfos)
	{
		if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
		{
			if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
			{
				UModularRigController* Controller = Blueprint->GetModularRigController();
				const FRigHierarchyModulePath ConnectorModulePath(Info.ModuleName.ToString(), InConnectorName);
				FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
				Controller->AddTargetToArrayConnector(ConnectorKey, FRigElementKey(), true, false, false);
				Blueprint->RecompileModularRig();
			}
		}
	}
	PropertyUtilities->RequestForceRefresh();
	return FReply::Handled();
}

FReply FRigModuleInstanceDetails::OnClearTargetsForArrayConnector(const FString InConnectorName, const TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	FScopedTransaction Transaction(LOCTEXT("TransactionClearTargetsForArrayConnector", "Clear Targets For Array Connector"));
	for(const FPerModuleInfo& Info : PerModuleInfos)
	{
		if (FControlRigAssetInterfacePtr Blueprint = Info.GetAsset())
		{
			if (const FRigModuleInstance* ModuleInstance = Info.GetModule())
			{
				UModularRigController* Controller = Blueprint->GetModularRigController();
				const FRigHierarchyModulePath ConnectorModulePath(Info.ModuleName.ToString(), InConnectorName);
				FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
				Controller->DisconnectConnector(ConnectorKey);
			}
		}
	}
	PropertyUtilities->RequestForceRefresh();
	return FReply::Handled();
}

FReply FRigModuleInstanceDetails::OnConnectorDragDetected(
	const FGeometry& MyGeometry, 
	const FPointerEvent& MouseEvent, 
	const FRigElementKey ElementKey)
{
	const FRigHierarchyKey HierarchyKey = FRigHierarchyKey(ElementKey);
	if (HierarchyKey.IsValid())
	{
		GEditor->GetTimerManager()->SetTimerForNextTick([WeakThis = AsWeak(), this]
			{
				if (WeakThis.IsValid())
				{
					constexpr bool bShowSchemanticViewport = true;
					ShowSchematicViewportOverride.OverrideDuringDragDrop(bShowSchemanticViewport);
				}
			});

		const TSharedRef<FRigHierarchyElementDragDropOp> DragDropOp = FRigHierarchyElementDragDropOp::New({ HierarchyKey });
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

FText FRigModuleInstanceDetails::GetTooltipForConnector(const FRigElementKey& ConnectorKey)
{
	const UControlRig* ControlRig = PerModuleInfos.IsEmpty() ? nullptr : PerModuleInfos[0].GetModularRig();
	const URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
	if (!ControlRig || !Hierarchy)
	{
		return FText::GetEmpty();
	}
						
	const FText ShortName = Hierarchy->GetDisplayNameForUI(ConnectorKey, EElementNameDisplayMode::ForceShort);

	const FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Hierarchy->Find(ConnectorKey));
	if (ensureMsgf(ConnectorElement, TEXT("Unexpected trying to get tooltip for connector, but the provided key is not a connector element")) &&
		!ConnectorElement->Settings.Description.IsEmpty())
	{
		const FText& ConnectorDescription = FText::FromString(ConnectorElement->Settings.Description);
		return FText::Format(FText::FromString("{0} - {1}"), ShortName, ConnectorDescription);
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText FRigModuleInstanceDetails::GetAssetName() const
{
	const UObject* AssetObject = nullptr;
	if (GetSingleAsset(AssetObject) &&
		AssetObject)
	{
		return FText::FromString(AssetObject->GetName());
	}
	else
	{
		return ControlRigModuleDetailsMultipleValues;
	}
}

FText FRigModuleInstanceDetails::GetAssetPath() const
{
	const UObject* AssetObject = nullptr;
	if (GetSingleAsset(AssetObject) &&
		AssetObject)
	{
		return FText::FromString(AssetObject->GetPathName());
	}
	else
	{
		return ControlRigModuleDetailsMultipleValues;
	}
}

void FRigModuleInstanceDetails::OnBrowseToRigModuleAssetClicked() const
{
	if (PerModuleInfos.Num() == 1)
	{
		const FRigModuleInstance* Module = PerModuleInfos[0].GetModule();
		const UControlRig* ModuleRig = Module ? Module->GetRig() : nullptr;
		
		const UObject* Blueprint = nullptr;
		if (GetSingleAsset(Blueprint))
		{
			if (Module && ModuleRig && Blueprint)
			{
				const FSoftObjectPath SoftObjectPath(Blueprint);

				IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			
				TArray<FAssetData> AssetData;
				AssetData.Add(AssetRegistry.GetAssetByObjectPath(SoftObjectPath.GetWithoutSubPath()));
			
				IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
				ContentBrowserSingleton.SyncBrowserToAssets(AssetData);
			}
		}
	}
}

bool FRigModuleInstanceDetails::GetSingleAsset(const UObject*& OutAssetObject) const
{
	if (PerModuleInfos.IsEmpty())
	{
		return false;
	}
	else if (PerModuleInfos.Num() > 1)
	{
		if (const FRigModuleInstance* FirstModule = PerModuleInfos[0].GetModule())
		{
			const UControlRig* FirstControlRig = FirstModule->GetRig();
			const FControlRigAssetStrongReference FirstModuleClass = FirstControlRig ? FirstControlRig->GetAssetReference() : nullptr;

			for (int32 i = 1; i < PerModuleInfos.Num(); ++i)
			{
				const FRigModuleInstance* OtherModule = PerModuleInfos[i].GetModule();
				const UControlRig* OtherControlRig = OtherModule ? OtherModule->GetRig() : nullptr;
				const FControlRigAssetStrongReference OtherModuleClass = OtherControlRig ? OtherControlRig->GetAssetReference() : nullptr;
				if (OtherModule && OtherControlRig && OtherModuleClass.IsValid())
				{
					if (OtherModuleClass != FirstModuleClass)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}
		}
	}

	if (const FRigModuleInstance* Module = PerModuleInfos[0].GetModule())
	{
		const UControlRig* ModuleRig = Module->GetRig();
		const FControlRigAssetStrongReference ModuleRigClass = ModuleRig ? ModuleRig->GetAssetReference() : FControlRigAssetStrongReference();
		TObjectPtr<UObject> ModuleRigClassGeneratedBy = nullptr;
		
		if (ModuleRigClass.IsValid() && ModuleRigClass.GetBlueprintClass())
		{
			ModuleRigClassGeneratedBy = ModuleRigClass.GetBlueprintClass()->ClassGeneratedBy;
		}

		OutAssetObject = (ModuleRigClassGeneratedBy != nullptr) ? ModuleRigClassGeneratedBy.Get() : ModuleRigClass.Get();
		if (OutAssetObject)
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
