// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "ControlRig.h"
#include "ModularRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigElementDetails.h"
#include "Editor/ControlRigWrapperObject.h"
#include "Styling/SlateTypes.h"
#include "IPropertyUtilities.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Internationalization/FastDecimalFormat.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Algo/Transform.h"
#include "IPropertyUtilities.h"
#include "Editor/ControlRigShowSchematicViewportOverride.h"

class IPropertyHandle;

class FRigModuleInstanceDetails : public IDetailCustomization
{
	using FControlRigShowSchematicViewportOverride = UE::ControlRigEditor::FControlRigShowSchematicViewportOverride;

public:

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigModuleInstanceDetails);
	}

	FText GetName() const;
	void SetName(const FText& InValue, ETextCommit::Type InCommitType, const TSharedRef<IPropertyUtilities> PropertyUtilities);
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);

	TArray<FRigModuleConnector> GetConnectors() const;
	FRigElementKeyRedirector GetConnections() const;

	void OnConfigValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	// Routed from each module's UControlRigWrapperObject. Mirrors the wrapper's UPROPERTY
	// state back into the rig's variables memory, then defers to OnConfigValueChanged for
	// override-path reconstruction + SetConfigValueInModule. The wrapper-driven path is used
	// for BP-independent rigs where variables live in a UPropertyBag, so that downstream
	// customizations (e.g. shape-name combo) discover the UControlRig via GetOuterObjects()
	// instead of seeing a bare FStructOnScope.
	void OnWrapperConfigValueChanged(URigVMDetailsViewWrapperObject* InWrapper, const FString& InPropertyPath, FPropertyChangedChainEvent& InEvent);

	bool OnConnectorTargetChanged(TArray<FRigElementKey> InTargets, FRigModuleConnector InConnector);

	FText GetConstructionStartSpawnIndex() const;
	FText GetPostConstructionStartSpawnIndex() const;

	struct FPerModuleInfo
	{
		FPerModuleInfo()
			: ModuleName(NAME_None)
			, Module()
		{}

		bool IsValid() const { return Module.IsValid(); }
		operator bool() const { return IsValid(); }

		const FName& GetModuleName() const
		{
			return ModuleName;
		}

		UModularRig* GetModularRig() const
		{
			return (UModularRig*)Module.GetModularRig();
		}

		FControlRigAssetInterfacePtr GetAsset() const
		{
			if(const UModularRig* ControlRig = GetModularRig())
			{
				FControlRigAssetStrongReference AssetReference = ControlRig->GetAssetReference();
				if (AssetReference.IsValid())
				{
					return AssetReference.GetEditorAsset();
				}
			}
			return nullptr;
		}

		FRigModuleInstance* GetModule() const
		{
			return (FRigModuleInstance*)Module.Get();
		}

		const FRigModuleReference* GetReference() const
		{
			if(const FControlRigAssetInterfacePtr Blueprint = GetAsset())
			{
				return Blueprint->GetModularRigModel().FindModule(ModuleName);
			}
			return nullptr;
		}

		FName ModuleName;
		FModuleInstanceHandle Module;
	};

	const FPerModuleInfo& FindModule(const FName& InModuleName) const;
	const FPerModuleInfo* FindModuleByPredicate(const TFunction<bool(const FPerModuleInfo&)>& InPredicate) const;
	bool ContainsModuleByPredicate(const TFunction<bool(const FPerModuleInfo&)>& InPredicate) const;

	virtual void RegisterSectionMappings(FPropertyEditorModule& PropertyEditorModule, UClass* InClass);

protected:

	FText GetBindingText(const FProperty* InProperty) const;
	const FSlateBrush* GetBindingImage(const FProperty* InProperty) const;
	FLinearColor GetBindingColor(const FProperty* InProperty) const;
	void FillBindingMenu(FMenuBuilder& MenuBuilder, const FProperty* InProperty) const;
	bool CanRemoveBinding(FName InPropertyName) const;
	void HandleRemoveBinding(FName InPropertyName) const;
	void HandleChangeBinding(const FProperty* InProperty, const FString& InNewVariablePath) const;
	FReply OnAddTargetToArrayConnector(const FString InConnectorName, const TSharedRef<IPropertyUtilities> PropertyUtilities);
	FReply OnClearTargetsForArrayConnector(const FString InConnectorName, const TSharedRef<IPropertyUtilities> PropertyUtilities);

	TArray<FPerModuleInfo> PerModuleInfos;

	/** Strong refs that keep the per-module variable wrappers alive while the details panel is open.
	 *  Each wrapper carries the owning UControlRig via SetSubject(), so property handles backed by it
	 *  expose the rig via GetOuterObjects() (needed by FControlRigShapeSettingsDetails and similar). */
	TArray<TStrongObjectPtr<UControlRigWrapperObject>> PerModuleWrappers;

	/** Re-entrancy guard for OnWrapperConfigValueChanged. In multi-module selection
	 *  AddObjectPropertyData merges N wrappers behind one IPropertyHandle; an edit on
	 *  the merged handle dispatches PostEditChangeChainProperty to each wrapper, so the
	 *  chain event fires N times. The first dispatch already iterates all modules in
	 *  OnConfigValueChanged, so subsequent dispatches for the same edit are redundant. */
	bool bDispatchingWrapperEvent = false;

	/** Helper buttons. */
	TMap<FString, TSharedPtr<SButton>> UseSelectedButton;
	TMap<FString, TSharedPtr<SButton>> SelectElementButton;
	TMap<FString, TSharedPtr<SButton>> ResetConnectorButton;

private:
	/** Called when a connector was dragged, starts a drag drop op only if ElementKeys is set */
	FReply OnConnectorDragDetected(
		const FGeometry& MyGeometry, 
		const FPointerEvent& MouseEvent, 
		const FRigElementKey ElementKey);

	/** Returns the tooltip for a connector element, or empty text if the element is not a connector */
	FText GetTooltipForConnector(const FRigElementKey& Key);

	/** Returns path name of the rig asset in use, assumes multiple assets if there is no single outer asset */
	FText GetAssetPath() const;

	/** Returns name of the rig asset in use, assumes multiple assets if there is no single outer asset */
	FText GetAssetName() const;

	/** Called when the Browse to Rig Module Asset button was clicked */
	void OnBrowseToRigModuleAssetClicked() const;

	/** Gets a single asset object the asset was generated by, returns false if there is none or multiple */
	[[nodiscard]] bool GetSingleAsset(const UObject*& OutAssetObject) const;

	/** Overrides the per project per user schematic viewport visibility */
	FControlRigShowSchematicViewportOverride ShowSchematicViewportOverride;
};
