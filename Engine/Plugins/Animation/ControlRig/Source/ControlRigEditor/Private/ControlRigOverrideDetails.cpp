// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigOverrideDetails.h"
#include "ControlRigOverride.h"
#include "PropertyCustomizationHelpers.h"
#include "IPropertyUtilities.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Colors/SColorBlock.h"

#define LOCTEXT_NAMESPACE "ControlRigOverrideDetails"

void FControlRigOverrideDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// nothing to do here
}

void FControlRigOverrideDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = StructCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
	if(SelectedObjects.Num() != 1)
	{
		return;
	}
	if(!SelectedObjects[0].IsValid() || SelectedObjects[0].Get() == nullptr)
	{
		return;
	}

	UControlRigOverrideAsset* OverrideAsset = Cast<UControlRigOverrideAsset>(SelectedObjects[0].Get());
	if(OverrideAsset == nullptr)
	{
		return;
	}

	struct FSortNamesAlphabetically
	{
		bool operator()(const FName& A, const FName& B) const
		{
			return A.Compare(B) < 0;
		}
	};

	TArray<FName> SubjectKeys = OverrideAsset->Overrides.GenerateSubjectArray();
	SubjectKeys.Sort(FSortNamesAlphabetically());

	for(const FName& SubjectKey : SubjectKeys)
	{
		const TArray<int32>* IndicesPtr = OverrideAsset->Overrides.GetIndicesForSubject(SubjectKey);
		if(IndicesPtr == nullptr)
		{
			continue;
		}

		const TArray<int32>& Indices = *IndicesPtr;
		if(Indices.IsEmpty())
		{
			continue;
		}

		StructBuilder.AddCustomBuilder(MakeShared<FControlRigOverrideDetailsBuilder>(OverrideAsset, SubjectKey));
	}
}

FControlRigOverrideDetailsBuilder::FControlRigOverrideDetailsBuilder(UControlRigOverrideAsset* InOverrideAsset, const FName& InSubjectKey)
	: WeakOverrideAsset(InOverrideAsset)
	, SubjectKey(InSubjectKey)
{
}

void FControlRigOverrideDetailsBuilder::Tick(float DeltaTime)
{
	if(!WeakOverrideAsset.IsValid())
	{
		return;
	}
	const UControlRigOverrideAsset* OverrideAsset = WeakOverrideAsset.Get();
	if(OverrideAsset == nullptr)
	{
		return;
	}
	const uint32 Hash = GetTypeHash(OverrideAsset->Overrides);
	if(LastHash.Get(UINT32_MAX) != Hash)
	{
		(void)OnRebuildChildren.ExecuteIfBound();
	}
}

void FControlRigOverrideDetailsBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromName(SubjectKey))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
}

void FControlRigOverrideDetailsBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if(!WeakOverrideAsset.IsValid())
	{
		return;
	}

	UControlRigOverrideAsset* OverrideAsset = WeakOverrideAsset.Get();
	if(OverrideAsset == nullptr)
	{
		return;
	}

	const TArray<int32>* IndicesPtr = OverrideAsset->Overrides.GetIndicesForSubject(SubjectKey);
	if(IndicesPtr == nullptr)
	{
		return;
	}

	const TArray<int32>& Indices = *IndicesPtr;
	const FString SubjectKeyPrefix = SubjectKey.ToString() + TEXT("|");
	
	for(int32 Index : Indices)
	{
		if(!OverrideAsset->Overrides.IsValidIndex(Index))
		{
			continue;
		}
		FControlRigOverrideValue& Override = OverrideAsset->Overrides[Index];
		if(Override.GetSubjectKey() != SubjectKey)
		{
			continue;
		}

		const FProperty* LeafProperty = Override.GetLeafProperty();
		if(LeafProperty == nullptr)
		{
			continue;
		}
		const FStructProperty* StructProperty = CastField<FStructProperty>(LeafProperty);
		
		FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(FText::FromString(SubjectKeyPrefix + Override.GetPath()));

		// add the default name content for anything but a transform property
		if(StructProperty == nullptr || StructProperty->Struct != TBaseStructure<FTransform>::Get())
		{
			Row.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(Override.GetPath()))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			];
		}

		// initially overrides will only be offered one a limited set of types
		//
		if(CastField<FBoolProperty>(LeafProperty))
		{
			TSharedPtr<TControlRigOverrideHandle<bool>> Handle = MakeShared<TControlRigOverrideHandle<bool>>(OverrideAsset, Index);
			Row.ValueContent()
			[
				SNew(SCheckBox)
				.IsEnabled(false)
				.IsChecked_Lambda([Handle]() -> ECheckBoxState
				{
					if(const bool* Data = Handle->GetData())
					{
						return (*Data) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					return ECheckBoxState::Undetermined;
				})
			];
		}
		else if(CastField<FFloatProperty>(LeafProperty))
		{
			TSharedPtr<TControlRigOverrideHandle<float>> Handle = MakeShared<TControlRigOverrideHandle<float>>(OverrideAsset, Index);
			Row.ValueContent()
			[
				SNew(SNumericEntryBox<float>)
				.IsEnabled(false)
				.Value_Lambda([Handle]() -> TOptional<float>
				{
					if(const float* Data = Handle->GetData())
					{
						return *Data;
					}
					return TOptional<float>();
				})
			];
		}
		else if(CastField<FDoubleProperty>(LeafProperty))
		{
			TSharedPtr<TControlRigOverrideHandle<double>> Handle = MakeShared<TControlRigOverrideHandle<double>>(OverrideAsset, Index);
			Row.ValueContent()
			[
				SNew(SNumericEntryBox<double>)
				.IsEnabled(false)
				.Value_Lambda([Handle]() -> TOptional<double>
				{
					if(const double* Data = Handle->GetData())
					{
						return *Data;
					}
					return TOptional<double>();
				})
			];
		}
		else if(CastField<FNameProperty>(LeafProperty))
		{
			TSharedPtr<TControlRigOverrideHandle<FName>> Handle = MakeShared<TControlRigOverrideHandle<FName>>(OverrideAsset, Index);
			Row.ValueContent()
			[
				SNew(SEditableText)
				.IsEnabled(false)
				.Text_Lambda([Handle]() -> FText
				{
					if(const FName* Data = Handle->GetData())
					{
						return FText::FromName(*Data);
					}
					return FText();
				})
			];
		}
		else if(CastField<FStrProperty>(LeafProperty))
		{
			TSharedPtr<TControlRigOverrideHandle<FString>> Handle = MakeShared<TControlRigOverrideHandle<FString>>(OverrideAsset, Index);
			Row.ValueContent()
			[
				SNew(SEditableText)
				.IsEnabled(false)
				.Text_Lambda([Handle]() -> FText
				{
					if(const FString* Data = Handle->GetData())
					{
						return FText::FromString(*Data);
					}
					return FText();
				})
			];
		}
		else if(StructProperty)
		{
			const UStruct* Struct = StructProperty->Struct;

			if(Struct == TBaseStructure<FVector>::Get())
			{
				TSharedPtr<TControlRigOverrideHandle<FVector>> Handle = MakeShared<TControlRigOverrideHandle<FVector>>(OverrideAsset, Index);
				Row.ValueContent()
				[
					SNew(SNumericVectorInputBox<double>)
					.IsEnabled(false)
					.Vector_Lambda([Handle]() -> TOptional<FVector>
					{
						if(const FVector* Data = Handle->GetData())
						{
							return *Data;
						}
						return {};
					})
				];
			}
			else if(Struct == TBaseStructure<FRotator>::Get())
			{
				TSharedPtr<TControlRigOverrideHandle<FRotator>> Handle = MakeShared<TControlRigOverrideHandle<FRotator>>(OverrideAsset, Index);
				Row.ValueContent()
				[
					SNew(SNumericRotatorInputBox<double>)
					.IsEnabled(false)
					.Pitch_Lambda([Handle]() -> TOptional<double>
					{
						if(const FRotator* Data = Handle->GetData())
						{
							return Data->Pitch;
						}
						return {};
					})
					.Yaw_Lambda([Handle]() -> TOptional<double>
					{
						if(const FRotator* Data = Handle->GetData())
						{
							return Data->Yaw;
						}
						return {};
					})
					.Roll_Lambda([Handle]() -> TOptional<double>
					{
						if(const FRotator* Data = Handle->GetData())
						{
							return Data->Roll;
						}
						return {};
					})
				];
			}
			else if(Struct == TBaseStructure<FQuat>::Get())
			{
				TSharedPtr<TControlRigOverrideHandle<FQuat>> Handle = MakeShared<TControlRigOverrideHandle<FQuat>>(OverrideAsset, Index);
				Row.ValueContent()
				[
					SNew(SNumericRotatorInputBox<double>)
					.IsEnabled(false)
					.Pitch_Lambda([Handle]() -> TOptional<double>
					{
						if(const FQuat* Data = Handle->GetData())
						{
							return Data->Rotator().Pitch;
						}
						return {};
					})
					.Yaw_Lambda([Handle]() -> TOptional<double>
					{
						if(const FQuat* Data = Handle->GetData())
						{
							return Data->Rotator().Yaw;
						}
						return {};
					})
					.Roll_Lambda([Handle]() -> TOptional<double>
					{
						if(const FQuat* Data = Handle->GetData())
						{
							return Data->Rotator().Roll;
						}
						return {};
					})
				];
			}
			else if(Struct == TBaseStructure<FTransform>::Get())
			{
				TSharedPtr<TControlRigOverrideHandle<FTransform>> Handle = MakeShared<TControlRigOverrideHandle<FTransform>>(OverrideAsset, Index);
				Row.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(Override.GetPath()+TEXT("->Location")))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SNumericVectorInputBox<double>)
					.IsEnabled(false)
					.Vector_Lambda([Handle]() -> TOptional<FVector>
					{
					if(const FTransform* Data = Handle->GetData())
					{
						return Data->GetLocation();
					}
					return {};
					})
				];

				ChildrenBuilder.AddCustomRow(FText::FromString(SubjectKeyPrefix + Override.GetPath() + TEXT("->Rotation")))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(Override.GetPath()+TEXT("->Rotation")))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SNumericRotatorInputBox<double>)
					.IsEnabled(false)
					.Pitch_Lambda([Handle]() -> TOptional<double>
					{
						if(const FTransform* Data = Handle->GetData())
						{
							return Data->Rotator().Pitch;
						}
						return {};
					})
					.Yaw_Lambda([Handle]() -> TOptional<double>
					{
						if(const FTransform* Data = Handle->GetData())
						{
							return Data->Rotator().Yaw;
						}
						return {};
					})
					.Roll_Lambda([Handle]() -> TOptional<double>
					{
						if(const FTransform* Data = Handle->GetData())
						{
							return Data->Rotator().Roll;
						}
						return {};
					})
				];

				ChildrenBuilder.AddCustomRow(FText::FromString(SubjectKeyPrefix + Override.GetPath() + TEXT("->Rotation")))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(Override.GetPath()+TEXT("->Scale3D")))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SNumericVectorInputBox<double>)
					.IsEnabled(false)
					.Vector_Lambda([Handle]() -> TOptional<FVector>
					{
					if(const FTransform* Data = Handle->GetData())
					{
						return Data->GetScale3D();
					}
					return {};
					})
				];
			}
			else if(Struct == TBaseStructure<FLinearColor>::Get())
			{
				TSharedPtr<TControlRigOverrideHandle<FLinearColor>> Handle = MakeShared<TControlRigOverrideHandle<FLinearColor>>(OverrideAsset, Index);
				Row.ValueContent()
				[
					SNew(SColorBlock)
					.IsEnabled(false)
					.Color_Lambda([Handle]() -> FLinearColor
					{
						if(const FLinearColor* Data = Handle->GetData())
						{
							return *Data;
						}
						return FLinearColor::Black;
					})
				];
			}
		}
	}

	LastHash = GetTypeHash(OverrideAsset->Overrides);
}

#undef LOCTEXT_NAMESPACE
