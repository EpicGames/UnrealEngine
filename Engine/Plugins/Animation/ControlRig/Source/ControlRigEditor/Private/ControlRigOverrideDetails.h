// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigOverride.h"
#include "IPropertyTypeCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "IStructureDataProvider.h"

class IPropertyHandle;
class UControlRigOverrideAsset;

class FControlRigOverrideDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FControlRigOverrideDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};

class FControlRigOverrideDetailsBuilder
	: public IDetailCustomNodeBuilder
	, public TSharedFromThis<FControlRigOverrideDetailsBuilder>
{
public:
	FControlRigOverrideDetailsBuilder(UControlRigOverrideAsset* InOverrideAsset, const FName& InSubjectKey);
	
	/** IDetailCustomNodeBuilder interface */
	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRebuildChildren  ) override { OnRebuildChildren = InOnRebuildChildren; } 
	virtual bool RequiresTick() const override { return true; }
	virtual void Tick( float DeltaTime ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual FName GetName() const override { return SubjectKey; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	TWeakObjectPtr<UControlRigOverrideAsset> WeakOverrideAsset;
	FName SubjectKey;
	FSimpleDelegate OnRebuildChildren;
	TOptional<uint32> LastHash;
};