// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSequencerEditorLibrary.h"
#include "Animation/AnimSequence.h"
#include "LevelSequence.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "UnrealEdGlobals.h"
#include "ISequencer.h"
#include "ControlRigEditorModule.h"
#include "Channels/FloatChannelCurveModel.h"
#include "TransformNoScale.h"
#include "ControlRigComponent.h"
#include "MovieSceneToolHelpers.h"
#include "Rigs/FKControlRig.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "ControlRigObjectBinding.h"
#include "Engine/SCS_Node.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceActor.h"
#include "Logging/LogMacros.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Exporters/AnimSeqExportOption.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "ScopedTransaction.h"
#include "Misc/ScopedSlowTask.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "ControlRigSpaceChannelEditors.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "Tools/ConstraintBaker.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"
#include "Rigs/FKControlRig.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "EditorScriptingHelpers.h"
#include "ConstraintsScripting.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "Sections/MovieSceneConstrainedSection.h"
#include "BakingAnimationKeySettings.h"
#include "EditMode/ControlRigEditMode.h"
#include "SequencerTools.h"
#include "EditMode/Tween/ControlRigTweenModel.h"
#include "Math/Abstraction/KeyBlendingAbstraction.h"
#include "Tools/EvaluateSequencerTools.h"
#include "Sequencer/AnimLayers/AnimLayers.h"
#include "Sequencer/SelectionSets/SelectionSets.h"
#include "Sequencer/Scripting/MovieSceneScriptingSpace.h"
#include "Sequencer/Scripting/MovieSceneScriptingConstraint.h"
#include "ExtensionLibraries/MovieSceneSectionExtensions.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "IMovieSceneLinkedAnimTrackProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSequencerEditorLibrary)

#define LOCTEXT_NAMESPACE "ControlrigSequencerEditorLibrary"

TArray<UControlRig*> UControlRigSequencerEditorLibrary::GetVisibleControlRigs()
{
	TArray<UControlRig*> ControlRigs;
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigs = ControlRigEditMode->GetControlRigsArray(true /*bIsVisible*/);
	}
	return ControlRigs;
}

TArray<FControlRigSequencerBindingProxy> UControlRigSequencerEditorLibrary::GetControlRigs(ULevelSequence* LevelSequence)
{
	TArray<FControlRigSequencerBindingProxy> ControlRigBindingProxies;
	if (LevelSequence)
	{
		const UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (MovieScene)
		{
			const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
			for (const FMovieSceneBinding& Binding : Bindings)
			{
				TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
				for (UMovieSceneTrack* AnyOleTrack : Tracks)
				{
					UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack);
					if (Track && Track->GetControlRig())
					{
						FControlRigSequencerBindingProxy BindingProxy;
						BindingProxy.Track = Track;
						
						BindingProxy.ControlRig = Cast<UControlRig>(Track->GetControlRig());
						BindingProxy.Proxy.BindingID = Binding.GetObjectGuid();
						BindingProxy.Proxy.Sequence = LevelSequence;
						ControlRigBindingProxies.Add(BindingProxy);
					}
				}
			}
		}
	}
	return ControlRigBindingProxies;
}

static TArray<UObject*> GetBoundObjects(UWorld* World, ULevelSequence* LevelSequence, const FMovieSceneBindingProxy& Binding, ULevelSequencePlayer** OutPlayer, ALevelSequenceActor** OutActor)
{
	FMovieSceneSequencePlaybackSettings Settings;

	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, *OutActor);
	*OutPlayer = Player;

	// Evaluation needs to occur in order to obtain spawnables
	Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(LevelSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue().Value, EUpdatePositionMethod::Play));

	FMovieSceneSequenceID SequenceId = Player->GetEvaluationState()->FindSequenceId(LevelSequence);

	FMovieSceneObjectBindingID ObjectBinding = UE::MovieScene::FFixedObjectBindingID(Binding.BindingID, SequenceId);
	TArray<UObject*> BoundObjects = Player->GetBoundObjects(ObjectBinding);
	
	return BoundObjects;
}

static void AcquireSkeletonAndSkelMeshCompFromObject(UObject* BoundObject, USkeleton** OutSkeleton, USkeletalMeshComponent** OutSkeletalMeshComponent)
{
	*OutSkeletalMeshComponent = nullptr;
	*OutSkeleton = nullptr;
	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(Component);
			if (SkeletalMeshComp)
			{
				*OutSkeletalMeshComponent = SkeletalMeshComp;
				if (SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
				{
					*OutSkeleton = SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
				}
				return;
			}
		}

		AActor* ActorCDO = Cast<AActor>(Actor->GetClass()->GetDefaultObject());
		if (ActorCDO)
		{
			for (UActorComponent* Component : ActorCDO->GetComponents())
			{
				USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(Component);
				if (SkeletalMeshComp)
				{
					*OutSkeletalMeshComponent = SkeletalMeshComp;
					if (SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
					{
						*OutSkeleton = SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
					}
					return;
				}
			}
		}

		UBlueprintGeneratedClass* ActorBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(Actor->GetClass());
		if (ActorBlueprintGeneratedClass)
		{
			const TArray<USCS_Node*>& ActorBlueprintNodes = ActorBlueprintGeneratedClass->SimpleConstructionScript->GetAllNodes();

			for (USCS_Node* Node : ActorBlueprintNodes)
			{
				if (Node->ComponentClass && Node->ComponentClass->IsChildOf(USkeletalMeshComponent::StaticClass()))
				{
					USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(Node->GetActualComponentTemplate(ActorBlueprintGeneratedClass));
					if (SkeletalMeshComp)
					{
						*OutSkeletalMeshComponent = SkeletalMeshComp;
						if (SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
						{
							*OutSkeleton = SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
						}
					}
				}
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		*OutSkeletalMeshComponent = SkeletalMeshComponent;
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() && SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton())
		{
			*OutSkeleton = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton();
		}
	}
}

static TSharedPtr<ISequencer> GetSequencerFromAsset()
{
	//if getting sequencer from level sequence need to use the current(leader), not the focused
	ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, false);
	ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
	TSharedPtr<ISequencer> Sequencer = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	if (Sequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "Can not open Sequencer for the LevelSequence %ls", *(LevelSequence->GetPathName()));
	}
	return Sequencer;
}

static UMovieSceneControlRigParameterTrack* AddControlRig(ULevelSequence* LevelSequence,const UClass* InClass, FGuid ObjectBinding, UControlRig* InExistingControlRig, bool bIsAdditiveControlRig)
{
	FSlateApplication::Get().DismissAllMenus();
	if (!InClass || !InClass->IsChildOf(UControlRig::StaticClass()) ||
		!LevelSequence || !LevelSequence->GetMovieScene())
	{
		return nullptr;
	}
	
	UMovieScene* OwnerMovieScene = LevelSequence->GetMovieScene();
	TSharedPtr<ISequencer> SharedSequencer = GetSequencerFromAsset();
	ISequencer* Sequencer = nullptr; // will be valid  if we have a ISequencer AND it's focused.
	if (SharedSequencer.IsValid() && SharedSequencer->GetFocusedMovieSceneSequence() == LevelSequence)
	{
		Sequencer = SharedSequencer.Get();
	}
	LevelSequence->Modify();
	OwnerMovieScene->Modify();
	
	if (bIsAdditiveControlRig && InClass != UFKControlRig::StaticClass() && !InClass->GetDefaultObject<UControlRig>()->SupportsEvent(FRigUnit_InverseExecution::EventName))
	{
		UE_LOGF(LogControlRigEditor, Error, "Cannot add an additive control rig which does not contain a backwards solve event.");
		return nullptr;
	}
	
	FScopedTransaction AddControlRigTrackTransaction(LOCTEXT("AddControlRigTrack", "Add Control Rig Track"));

	UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(OwnerMovieScene->AddTrack(UMovieSceneControlRigParameterTrack::StaticClass(), ObjectBinding));
	if (Track)
	{
		FString ObjectName = InClass->GetName(); //GetDisplayNameText().ToString();
		ObjectName.RemoveFromEnd(TEXT("_C"));

		bool bSequencerOwnsControlRig = false;
		UControlRig* ControlRig = InExistingControlRig;
		if (ControlRig == nullptr)
		{
			ControlRig = NewObject<UControlRig>(Track, InClass, FName(*ObjectName), RF_Transactional);
			bSequencerOwnsControlRig = true;
		}

		ControlRig->Modify();
		if (UFKControlRig* FKControlRig = Cast<UFKControlRig>(Cast<UControlRig>(ControlRig)))
		{
			if (bIsAdditiveControlRig)
			{
				FKControlRig->SetApplyMode(EControlRigFKRigExecuteMode::Additive);
			}
		}
		else
		{
			ControlRig->SetIsAdditive(bIsAdditiveControlRig);
		}
		ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
		// Do not re-initialize existing control rig
		if (!InExistingControlRig)
		{
			ControlRig->Initialize();
		}
		ControlRig->Evaluate_AnyThread();

		if (SharedSequencer.IsValid())
		{
			SharedSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		}

		Track->Modify();
		UMovieSceneSection* NewSection = Track->CreateControlRigSection(0, ControlRig, bSequencerOwnsControlRig);
		NewSection->Modify();

		Track->SetTrackName(FName(*ObjectName));
		if (bIsAdditiveControlRig)
		{
			const FString AdditiveObjectName = ObjectName + TEXT(" (Layered)");
			Track->SetDisplayName(FText::FromString(AdditiveObjectName));
		}
		else
		{
			Track->SetDisplayName(FText::FromString(ObjectName));
		}
		UControlRigSequencerEditorLibrary::MarkLayeredModeOnTrackDisplay(Track);

		if (SharedSequencer.IsValid())
		{
			SharedSequencer->EmptySelection();
			SharedSequencer->SelectSection(NewSection);
			SharedSequencer->ThrobSectionSelection();
			SharedSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			SharedSequencer->ObjectImplicitlyAdded(ControlRig);
		}

		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!ControlRigEditMode)
		{
			GLevelEditorModeTools().ActivateMode(FControlRigEditMode::ModeName);
			ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));

		}
		if (ControlRigEditMode)
		{
			ControlRigEditMode->AddControlRigObject(ControlRig, SharedSequencer);
		}
		return Track;
	}
	return nullptr;
}

UMovieSceneTrack* UControlRigSequencerEditorLibrary::FindOrCreateControlRigTrack(UWorld* World, ULevelSequence* LevelSequence, const UClass* ControlRigClass, const FMovieSceneBindingProxy& InBinding, bool bIsLayeredControlRig)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	UMovieSceneTrack* BaseTrack = nullptr;
	if (LevelSequence && MovieScene && InBinding.BindingID.IsValid())
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
		{
			TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding->GetObjectGuid(), NAME_None);
			for (UMovieSceneTrack* AnyOleTrack : Tracks)
			{
				UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack);
				if (Track && Track->GetControlRig() && Track->GetControlRig()->GetClass() == ControlRigClass)
				{
					return Track;
				}
			}

			UMovieSceneControlRigParameterTrack* Track = AddControlRig(LevelSequence, ControlRigClass,  InBinding.BindingID, nullptr, bIsLayeredControlRig);

			if (Track)
			{
				BaseTrack = Track;
			}
		}
	}
	return BaseTrack;
}


TArray<UMovieSceneTrack*> UControlRigSequencerEditorLibrary::FindOrCreateControlRigComponentTrack(UWorld* World, ULevelSequence* LevelSequence, const FMovieSceneBindingProxy& InBinding)
{
	TArray< UMovieSceneTrack*> Tracks;
	TArray<UObject*, TInlineAllocator<1>> Result;
	if (LevelSequence == nullptr || LevelSequence->GetMovieScene() == nullptr)
	{
		return Tracks;
	}
	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	UObject* Context = nullptr;
	ALevelSequenceActor* OutActor = nullptr;
	ULevelSequencePlayer* OutPlayer = nullptr;
	Result = GetBoundObjects(World, LevelSequence, InBinding, &OutPlayer, &OutActor);
	if (Result.Num() > 0 && Result[0])
	{
		UObject* BoundObject = Result[0];
		if (AActor* BoundActor = Cast<AActor>(BoundObject))
		{
			TArray<UControlRigComponent*> ControlRigComponents;
			BoundActor->GetComponents<UControlRigComponent>(ControlRigComponents);
			for (UControlRigComponent* ControlRigComponent : ControlRigComponents)
			{
				if (UControlRig* CR = Cast<UControlRig>(ControlRigComponent->GetControlRig()))
				{
					UMovieSceneControlRigParameterTrack* GoodTrack = nullptr;
					if (FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
					{
						for (UMovieSceneTrack* Track : Binding->GetTracks())
						{
							if (UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
							{
								if (ControlRigParameterTrack->GetControlRig() == CR)
								{
									GoodTrack = ControlRigParameterTrack;
									break;
								}
							}
						}
					}

					if (GoodTrack == nullptr)
					{
						GoodTrack = AddControlRig(LevelSequence, CR->GetClass(), InBinding.BindingID, CR, false);
					}
					Tracks.Add(GoodTrack);
				}
			}
		}
	}

	if (OutActor)
	{
		World->DestroyActor(OutActor);
	}

	return Tracks;
}

bool UControlRigSequencerEditorLibrary::TweenControlRig(ULevelSequence* LevelSequence, UControlRig* ControlRig, float TweenValue)
{
	using namespace UE::TweeningUtilsEditor;
	using namespace UE::ControlRigEditor;
	
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() && WeakSequencer.Pin()->GetFocusedMovieSceneSequence() == LevelSequence
		&& ControlRig && LevelSequence->GetMovieScene())
	{
		LevelSequence->GetMovieScene()->Modify();
		
		TControlRigTweenModel<EBlendFunction::ControlsToTween> BlendTool(WeakSequencer); 
		BlendTool.StartBlendOperationWithControlRigs({ ControlRig });
		BlendTool.BlendValues(TweenValue);
		BlendTool.StopBlendOperation();
		return true;
	}
	return false;
}

namespace UE::ControlRigEditor::SequencerLibraryDetail
{
static bool BlendSingle(
	const TWeakPtr<ISequencer>& InWeakSequencer,
	const TWeakPtr<FControlRigEditMode> InWeakMode,
	EAnimToolBlendOperation InOperation,
	double InBlendValue
	)
{
	using namespace TweeningUtilsEditor;
	
	switch (InOperation)
	{
	case EAnimToolBlendOperation::Tween:
		return TControlRigTweenModel<EBlendFunction::ControlsToTween>(InWeakSequencer, InWeakMode).BlendSingle(InBlendValue);
	case EAnimToolBlendOperation::BlendToNeighbor:
		return TControlRigTweenModel<EBlendFunction::BlendNeighbor>(InWeakSequencer, InWeakMode).BlendSingle(InBlendValue);
	case EAnimToolBlendOperation::PushPull:
		return TControlRigTweenModel<EBlendFunction::PushPull>(InWeakSequencer, InWeakMode).BlendSingle(InBlendValue);
	case EAnimToolBlendOperation::BlendRelative:
		return TControlRigTweenModel<EBlendFunction::BlendRelative>(InWeakSequencer, InWeakMode).BlendSingle(InBlendValue);
	case EAnimToolBlendOperation::BlendToEase:
		return TControlRigTweenModel<EBlendFunction::BlendEase>(InWeakSequencer, InWeakMode).BlendSingle(InBlendValue);
	case EAnimToolBlendOperation::SmoothRough:
		return TControlRigTweenModel<EBlendFunction::SmoothRough>(InWeakSequencer, InWeakMode).BlendSingle(InBlendValue);
	default: ensure(false); return false;
	}
}
}

bool UControlRigSequencerEditorLibrary::BlendValuesOnSelected(ULevelSequence* LevelSequence, EAnimToolBlendOperation BlendOperation, float BlendValue)
{
	using namespace UE::TweeningUtilsEditor;
	using namespace UE::ControlRigEditor;
	
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() && WeakSequencer.Pin()->GetFocusedMovieSceneSequence() == LevelSequence
		&& LevelSequence->GetMovieScene())
	{
		if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName)))
		{
			TWeakPtr<FControlRigEditMode> WeakMode = StaticCastSharedRef<FControlRigEditMode, FEdMode>(EditMode->AsShared()).ToWeakPtr();
			LevelSequence->GetMovieScene()->Modify();
			
			return SequencerLibraryDetail::BlendSingle(WeakSequencer, WeakMode, BlendOperation, BlendValue);
		}
	}
	return false;
}

UTickableConstraint* UControlRigSequencerEditorLibrary::AddConstraint(UWorld* World, ETransformConstraintType InType, UTransformableHandle* InChildHandle, UTransformableHandle* InParentHandle, const bool bMaintainOffset)
{
	if (!World)
	{
		UE_LOGF(LogControlRig, Error, "AddConstraint: Need Valid World");
		return nullptr;
	}

	if (!InChildHandle || !InChildHandle->IsValid())
    {
    	UE_LOGF(LogControlRig, Error, "AddConstraint: Need Valid Child Handle");
    	return nullptr;
    }

	if (!InParentHandle || !InParentHandle->IsValid())
	{
		UE_LOGF(LogControlRig, Error, "AddConstraint: Need Valid Parent Handle");
		return nullptr;
	}

	UTickableTransformConstraint* Constraint = UE::TransformConstraintUtil::CreateFromType(World, InType);
	if (!Constraint)
	{
		UE_LOGF(LogControlRig, Error, "AddConstraint: Can not create constraint from type");
		return nullptr;
	}

	if(UE::TransformConstraintUtil::AddConstraint(World, InParentHandle, InChildHandle, Constraint, bMaintainOffset) == false)
	{
		UE_LOGF(LogControlRig, Error, "AddConstraint: Constraint not added");
		Constraint->MarkAsGarbage();
		return nullptr;
	}

	if (ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence())
	{
		//add key
		const TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
		if (WeakSequencer.IsValid())
		{
			FMovieSceneConstraintChannelHelper::SmartConstraintKey(WeakSequencer.Pin(), Constraint, TOptional<bool>(), TOptional<FFrameNumber>());
		}
	}
	else
	{
		FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.StaticConstraintCreated(Constraint);
	}
	return Constraint;
}

TArray <UTickableConstraint*> UControlRigSequencerEditorLibrary::GetConstraintsForHandle(UWorld* InWorld, const UTransformableHandle* InChild)
{
	TArray <UTickableConstraint*> Constraints;
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
	const TArray<TWeakObjectPtr<UTickableConstraint>>& AllConstraints = Controller.GetAllConstraints(false);
	for (const TWeakObjectPtr<UTickableConstraint>& TickConstraint : AllConstraints)
	{
		if (TObjectPtr<UTickableTransformConstraint> Constraint = Cast<UTickableTransformConstraint>(TickConstraint.Get()))
		{
			if (Constraint->ChildTRSHandle == InChild)
			{
				Constraints.Add(Constraint.Get());
			}
		}
	}
	return Constraints;
}

bool UControlRigSequencerEditorLibrary::Compensate(UTickableConstraint* InConstraint, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "Compensate: Need open Sequencer");
		return false;
	}
	if (UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InConstraint))
	{
		TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		TOptional<FFrameNumber> OptTime(InTime);
		FMovieSceneConstraintChannelHelper::Compensate(Sequencer, Constraint, OptTime, true /*bCompPreviousTick*/);
		return true;
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Compensate: Not Transform Constraint");
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::CompensateAll(UTickableConstraint* InConstraint)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "CompensateAll: Need open Sequencer");
		return false;
	}
	if (UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InConstraint))
	{
		FMovieSceneConstraintChannelHelper::Compensate(WeakSequencer.Pin(), Constraint, TOptional<FFrameNumber>(), true /*bCompPreviousTick*/);
		return true;
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "CompensateAll: Not Transform Constraint");
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::SetConstraintActiveKey(UTickableConstraint* InConstraint, bool bActive, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "SetConstraintActiveKey: Need open Sequencer");
		return false;
	}

	if (UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InConstraint))
	{
		TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		return FMovieSceneConstraintChannelHelper::SmartConstraintKey(Sequencer, Constraint, TOptional<bool>(bActive), TOptional<FFrameNumber>(InTime));
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "SetConstraintActiveKey: Not Transform Constraint");
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::GetConstraintKeys(UTickableConstraint* InConstraint, UMovieSceneSection* ConstraintSection, TArray<bool>& OutBools, TArray<FFrameNumber>& OutFrames, EMovieSceneTimeUnit TimeUnit)
{
	if (InConstraint == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "GetConstraintKeys: Constraint not valid");
		return false;
	}
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "GetConstraintKeys: Need open Sequencer");
		return false;
	}

	IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(ConstraintSection);
	if (ConstrainedSection == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "GetConstraintKeys: Section doesn't support constraints");
		return false;
	}
	FConstraintAndActiveChannel* ConstraintAndChannel = ConstrainedSection->GetConstraintChannel(InConstraint->ConstraintID);
	if (ConstraintAndChannel == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "GetConstraintKeys: Constraint not found in section");
		return false;
	}

	if (UTickableTransformConstraint* Constraint = Cast<UTickableTransformConstraint>(InConstraint))
	{
		TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();


		TArray<FFrameNumber> OurKeyTimes;
		TArray<FKeyHandle> OurKeyHandles;
		TRange<FFrameNumber> CurrentFrameRange;
		CurrentFrameRange.SetLowerBound(TRangeBound<FFrameNumber>());
		CurrentFrameRange.SetUpperBound(TRangeBound<FFrameNumber>());

		TMovieSceneChannelData<bool> ChannelInterface = ConstraintAndChannel->ActiveChannel.GetData();
		ChannelInterface.GetKeys(CurrentFrameRange, &OurKeyTimes, &OurKeyHandles);
		
		if (OurKeyTimes.Num() > 0)
		{
			OutBools.SetNum(OurKeyTimes.Num());
			OutFrames.SetNum(OurKeyTimes.Num());
			int32 Index = 0;
			for (FFrameNumber Frame : OurKeyTimes)
			{
				ConstraintAndChannel->ActiveChannel.Evaluate(FFrameTime(Frame, 0.0f), OutBools[Index]);
				if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
				{
					Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), TickResolution,DisplayRate).RoundToFrame();
				}
				OutFrames[Index] = Frame;
				++Index;
			}
		}
		else
		{
			OutBools.SetNum(0);
			OutFrames.SetNum(0);
		}
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "GetConstraintKeys: Not Transform Constraint");
		return false;
	}
	return true;
}


bool UControlRigSequencerEditorLibrary::MoveConstraintKey(UTickableConstraint* Constraint, UMovieSceneSection* ConstraintSection, FFrameNumber InTime, FFrameNumber InNewTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "MoveConstraintKey: Need open Sequencer");
		return false;
	}
	IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(ConstraintSection);
	if (ConstrainedSection == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "MoveConstraintKey: Section doesn't support constraints");
		return false;
	}
	if (Constraint == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "MoveConstraintKey: Constraint not valid");
		return false;
	}
	FConstraintAndActiveChannel* ConstraintAndChannel = ConstrainedSection->GetConstraintChannel(Constraint->ConstraintID);
	if (ConstraintAndChannel == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "MoveConstraintKey: Constraint not found in section");
		return false;
	}
	TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

	if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
	{
		InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), DisplayRate, TickResolution).RoundToFrame();
		InNewTime = FFrameRate::TransformTime(FFrameTime(InNewTime, 0), DisplayRate, TickResolution).RoundToFrame();
	}

	TArray<FFrameNumber> OurKeyTimes;
	TArray<FKeyHandle> OurKeyHandles;
	TRange<FFrameNumber> CurrentFrameRange;
	CurrentFrameRange.SetLowerBound(TRangeBound<FFrameNumber>(InTime));
	CurrentFrameRange.SetUpperBound(TRangeBound<FFrameNumber>(InTime));
	TMovieSceneChannelData<bool> ChannelInterface = ConstraintAndChannel->ActiveChannel.GetData();
	ChannelInterface.GetKeys(CurrentFrameRange, &OurKeyTimes, &OurKeyHandles);
	if (OurKeyHandles.Num() > 0)
	{
		ConstraintSection->TryModify();
		TArray<FFrameNumber> NewKeyTimes;
		NewKeyTimes.Add(InNewTime);
		ConstraintAndChannel->ActiveChannel.SetKeyTimes(OurKeyHandles, NewKeyTimes);
		WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "MoveConstraintKey: No Key To Move At that Time");
		return false;
	}
	return true;
}

bool UControlRigSequencerEditorLibrary::DeleteConstraintKey(UTickableConstraint* Constraint, UMovieSceneSection* ConstraintSection, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "DeleteConstraintKey: Need open Sequencer");
		return false;
	}
	IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(ConstraintSection);
	if (ConstrainedSection == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "DeleteConstraintKey: Section doesn't support constraints");
		return false;
	}
	if (Constraint == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "DeleteConstraintKey: Constraint not valid");
		return false;
	}
	FConstraintAndActiveChannel* ConstraintAndChannel = ConstrainedSection->GetConstraintChannel(Constraint->ConstraintID);
	if (ConstraintAndChannel == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "DeleteConstraintKey: Constraint not found in section");
		return false;
	}
	TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

	if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
	{
		InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), DisplayRate, TickResolution).RoundToFrame();
	}

	TArray<FFrameNumber> OurKeyTimes;
	TArray<FKeyHandle> OurKeyHandles;
	TRange<FFrameNumber> CurrentFrameRange;
	CurrentFrameRange.SetLowerBound(TRangeBound<FFrameNumber>(InTime));
	CurrentFrameRange.SetUpperBound(TRangeBound<FFrameNumber>(InTime));
	TMovieSceneChannelData<bool> ChannelInterface = ConstraintAndChannel->ActiveChannel.GetData();
	ChannelInterface.GetKeys(CurrentFrameRange, &OurKeyTimes, &OurKeyHandles);
	if (OurKeyHandles.Num() > 0)
	{
		ConstraintSection->TryModify();
		ConstraintAndChannel->ActiveChannel.DeleteKeys(OurKeyHandles);
		WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "DeleteConstraintKey: No Key To Delete At that Time");
		return false;
	}
	return true;
}

bool UControlRigSequencerEditorLibrary::BakeConstraint(UWorld* World, UTickableConstraint* Constraint, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	if (!World)
	{
		UE_LOGF(LogControlRig, Error, "BakeConstraint: Need Valid World");
		return false;
	}
	TOptional< FBakingAnimationKeySettings> Settings;

	if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint))
	{
		TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
		if (!Sequencer || !Sequencer->GetFocusedMovieSceneSequence())
		{
			UE_LOGF(LogControlRig, Error, "BakeConstraint: Need loaded level Sequence");
			return false;
		}
		const UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		if (!MovieScene)
		{
			UE_LOGF(LogControlRig, Error, "BakeConstraint: Need valid Movie Scene");
			return false;
		}
		
		TOptional<TArray<FFrameNumber>> FramesToBake;
		TArray<FFrameNumber> RealFramesToBake;
		RealFramesToBake.SetNum(Frames.Num());

		int32 Index = 0;
		for (FFrameNumber Frame : Frames) 
		{
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();
			}
			RealFramesToBake[Index++] = Frame;
		} 
		if (RealFramesToBake.Num() > 0)
		{
			FramesToBake = RealFramesToBake;
		}
		FConstraintBaker::Bake(World, TransformConstraint, Sequencer, Settings, FramesToBake);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "BakeConstraint: Need Valid Constraint");
		return false;
	}
	return true;
}

bool UControlRigSequencerEditorLibrary::BakeConstraints(UWorld* World, TArray<UTickableConstraint*>& InConstraints, const FBakingAnimationKeySettings& InSettings)
{
	if (!World)
	{
		UE_LOGF(LogControlRig, Error, "BakeConstraint: Need Valid World");
		return false;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer || !Sequencer->GetFocusedMovieSceneSequence())
	{
		UE_LOGF(LogControlRig, Error, "BakeConstraint: Need loaded level Sequence");
		return false;
	}
	const UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!MovieScene)
	{
		UE_LOGF(LogControlRig, Error, "BakeConstraint: Need valid Movie Scene");
		return false;
	}
	TArray<UTickableTransformConstraint*> TransformConstraints;
	for (UTickableConstraint* Constraint : InConstraints)
	{
		if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint))
		{
			TransformConstraints.Add(TransformConstraint);
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "BakeConstraint: Need Valid Constraint");
			return false;
		}
	}

	if(FConstraintBaker::BakeMultiple(World, TransformConstraints, Sequencer, InSettings) == false)
	{
		UE_LOGF(LogControlRig, Error, "BakeMultiple: Failed");
		return false;
	}
	return true;
}


bool UControlRigSequencerEditorLibrary::SnapControlRig(ULevelSequence* LevelSequence, FFrameNumber StartFrame, FFrameNumber EndFrame, const FControlRigSnapperSelection& ChildrenToSnap,
	const FControlRigSnapperSelection& ParentToSnap, const UControlRigSnapSettings* SnapSettings, EMovieSceneTimeUnit TimeUnit)
{
	if (LevelSequence == nullptr || LevelSequence->GetMovieScene() == nullptr)
	{
		return false;
	}
	FControlRigSnapper Snapper;
	if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
	{
		StartFrame = FFrameRate::TransformTime(FFrameTime(StartFrame, 0), LevelSequence->GetMovieScene()->GetDisplayRate(), LevelSequence->GetMovieScene()->GetTickResolution()).RoundToFrame();
		EndFrame = FFrameRate::TransformTime(FFrameTime(EndFrame, 0), LevelSequence->GetMovieScene()->GetDisplayRate(), LevelSequence->GetMovieScene()->GetTickResolution()).RoundToFrame();

	}
	return Snapper.SnapIt(StartFrame, EndFrame, ChildrenToSnap, ParentToSnap,SnapSettings);
}

FTransform UControlRigSequencerEditorLibrary::GetActorWorldTransform(ULevelSequence* LevelSequence, AActor* Actor, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FFrameNumber> Frames;
	Frames.Add(Frame);
	TArray<FTransform> Transforms = GetActorWorldTransforms(LevelSequence, Actor, Frames, TimeUnit);
	if (Transforms.Num() == 1)
	{
		return Transforms[0];
	}
	return FTransform::Identity;
}
static void ConvertFramesToTickResolution(UMovieScene* MovieScene, const TArray<FFrameNumber>& InFrames, EMovieSceneTimeUnit TimeUnit, TArray<FFrameNumber>& OutFrames)
{
	OutFrames.SetNum(InFrames.Num());
	int32 Index = 0;
	for (const FFrameNumber& Frame : InFrames)
	{
		FFrameTime FrameTime(Frame);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			FrameTime = FFrameRate::TransformTime(FrameTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
		}
		OutFrames[Index++] = FrameTime.RoundToFrame();
	}
}

TArray<FTransform> UControlRigSequencerEditorLibrary::GetActorWorldTransforms(ULevelSequence* LevelSequence,AActor* Actor, const TArray<FFrameNumber>& InFrames, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	TArray<FTransform> OutWorldTransforms;
	if (WeakSequencer.IsValid() && Actor)
	{
		FActorForWorldTransforms Actors;
		Actors.Actor = Actor;
		TArray<FFrameNumber> Frames;
		ConvertFramesToTickResolution(LevelSequence->GetMovieScene(), InFrames, TimeUnit, Frames);
		MovieSceneToolHelpers::GetActorWorldTransforms(WeakSequencer.Pin().Get(), Actors, Frames, OutWorldTransforms); 
	}
	return OutWorldTransforms;

}

FTransform UControlRigSequencerEditorLibrary::GetSkeletalMeshComponentWorldTransform(ULevelSequence* LevelSequence,USkeletalMeshComponent* SkeletalMeshComponent, FFrameNumber Frame, 
	EMovieSceneTimeUnit TimeUnit, FName SocketName)
{
	TArray<FFrameNumber> Frames;
	Frames.Add(Frame);
	TArray<FTransform> Transforms = GetSkeletalMeshComponentWorldTransforms(LevelSequence, SkeletalMeshComponent, Frames, TimeUnit,SocketName);
	if (Transforms.Num() == 1)
	{
		return Transforms[0];
	}
	return FTransform::Identity;
}

TArray<FTransform> UControlRigSequencerEditorLibrary::GetSkeletalMeshComponentWorldTransforms(ULevelSequence* LevelSequence,USkeletalMeshComponent* SkeletalMeshComponent, const TArray<FFrameNumber>& InFrames,
	EMovieSceneTimeUnit TimeUnit, FName SocketName)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	TArray<FTransform> OutWorldTransforms;
	if (WeakSequencer.IsValid() && SkeletalMeshComponent)
	{
		FActorForWorldTransforms Actors;
		AActor* Actor = SkeletalMeshComponent->GetTypedOuter<AActor>();
		if (Actor)
		{
			Actors.Actor = Actor;
			Actors.Component = SkeletalMeshComponent;
			Actors.SocketName = SocketName;
			TArray<FFrameNumber> Frames;
			ConvertFramesToTickResolution(LevelSequence->GetMovieScene(), InFrames, TimeUnit, Frames);
			MovieSceneToolHelpers::GetActorWorldTransforms(WeakSequencer.Pin().Get(), Actors, Frames, OutWorldTransforms);
		}
	}
	return OutWorldTransforms;
}

FTransform UControlRigSequencerEditorLibrary::GetControlRigWorldTransform(
	ULevelSequence* LevelSequence,
	UControlRig* ControlRig,
	FName ControlName,
	FFrameNumber Frame,
	EMovieSceneTimeUnit TimeUnit)
{
	TArray<FFrameNumber> Frames;
	Frames.Add(Frame);
	TArray<FTransform> Transforms = GetControlRigWorldTransforms(LevelSequence, ControlRig, ControlName, Frames, TimeUnit);
	if (Transforms.Num() == 1)
	{
		return Transforms[0];
	}
	return FTransform::Identity;
}

TArray<FTransform> UControlRigSequencerEditorLibrary::GetControlRigWorldTransforms(
	ULevelSequence* LevelSequence,
	UControlRig* ControlRig,
	FName ControlName,
	const TArray<FFrameNumber>& InFrames,
	EMovieSceneTimeUnit TimeUnit)
{
	TArray<FTransform> OutWorldTransforms;

	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	const bool bCanCompute = Sequencer && ControlRig && LevelSequence && LevelSequence->GetMovieScene();
	if (!bCanCompute)
	{
		return OutWorldTransforms;
	}

	const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding();
	USceneComponent* Component = ObjectBinding ? Cast<USceneComponent>(ObjectBinding->GetBoundObject()) : nullptr;
	AActor* Actor = Component ? Component->GetTypedOuter< AActor >() : nullptr;
	if (!Actor)
	{
		return OutWorldTransforms;
	}

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
				
	TArray<FFrameNumber> Frames;
	ConvertFramesToTickResolution(MovieScene, InFrames, TimeUnit, Frames);
	OutWorldTransforms.Init(FTransform::Identity, Frames.Num());

	const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
	const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
	const UE::AIE::FControlRigWorldTransformProvider Provider(Actor, Component, ControlRig, DeltaTime);
	
	MovieSceneToolHelpers::RunSequencerTask(Sequencer.Get(), Actor->GetWorld(), Frames, 
	[&OutWorldTransforms, &Provider, ControlName](int32 Index)
	{
		if (OutWorldTransforms.IsValidIndex(Index))
		{
			constexpr bool bUpdateAnimation = true;
			OutWorldTransforms[Index] = Provider.GetTransform(ControlName, bUpdateAnimation);
		}
	});
		
	return OutWorldTransforms;
}


static void LocalSetControlRigWorldTransforms(ULevelSequence* LevelSequence,UControlRig* ControlRig, FName ControlName, EControlRigSetKey SetKey, const TArray<FFrameNumber>& InFrames, 
	const TArray<FTransform>& WorldTransforms, EMovieSceneTimeUnit TimeUnit)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset();
	if (SequencerPtr && ControlRig)
	{
		if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
		{
			USceneComponent* Component = Cast<USceneComponent>(ObjectBinding->GetBoundObject());
			if (Component)
			{
				AActor* Actor = Component->GetTypedOuter< AActor >();
				if (Actor)
				{
					if (FRigControlElement* ControlElement = ControlRig->GetHierarchy()->Find<FRigControlElement>((FRigElementKey(ControlName, ERigElementType::Control))))
					{

						TArray<FFrameNumber> Frames;
						ConvertFramesToTickResolution(LevelSequence->GetMovieScene(), InFrames, TimeUnit, Frames);

						UMovieScene* MovieScene = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene();
						FFrameRate TickResolution = MovieScene->GetTickResolution();
						FRigControlModifiedContext Context;
						Context.SetKey = SetKey;

						ControlRig->Modify();
						TArray<FTransform> ControlRigParentWorldTransforms;
						FActorForWorldTransforms ControlRigActorSelection;
						ControlRigActorSelection.Actor = Actor;
						MovieSceneToolHelpers::GetActorWorldTransforms(SequencerPtr.Get(), ControlRigActorSelection, Frames, ControlRigParentWorldTransforms);
						FControlRigSnapper Snapper;

						TArray<FFrameNumber> OneFrame;
						OneFrame.SetNum(1);
						TArray<FTransform> CurrentControlRigTransform, CurrentParentWorldTransform;
						CurrentControlRigTransform.SetNum(1);
						CurrentParentWorldTransform.SetNum(1);

						for (int32 Index = 0; Index < WorldTransforms.Num(); ++Index)
						{
							OneFrame[0] = Frames[Index];
							CurrentParentWorldTransform[0] = ControlRigParentWorldTransforms[Index];
							//this will evaluate at the current frame which we want
							Snapper.GetControlRigControlTransforms(SequencerPtr.Get(), ControlRig, ControlName, OneFrame, CurrentParentWorldTransform, CurrentControlRigTransform);

							const FFrameNumber& FrameNumber = Frames[Index];
							Context.LocalTime = TickResolution.AsSeconds(FFrameTime(FrameNumber));
							FTransform GlobalTransform = WorldTransforms[Index].GetRelativeTransform(ControlRigParentWorldTransforms[Index]);

							constexpr ERigTransformType::Type TransformType = ERigTransformType::CurrentGlobal;
							FRigControlValue Value = ControlRig->GetControlValueFromGlobalTransform(ControlName, GlobalTransform, TransformType);
							GlobalTransform = Value.GetAsTransform(ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
							FEulerTransform EulerTransform(GlobalTransform);
							UE::AIE::FSetTransformHelpers::SetControlTransform(EulerTransform, ControlRig, ControlElement, Context);
						}
					}
				}
			}
		}
	}
}

void UControlRigSequencerEditorLibrary::SetControlRigWorldTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, const FTransform& WorldTransform,
	EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	EControlRigSetKey SetKey = bSetKey ? EControlRigSetKey::Always : EControlRigSetKey::DoNotCare;
	TArray<FFrameNumber> Frames;
	Frames.Add(Frame);
	TArray<FTransform> WorldTransforms;
	WorldTransforms.Add(WorldTransform);

	LocalSetControlRigWorldTransforms(LevelSequence, ControlRig, ControlName, SetKey, Frames, WorldTransforms, TimeUnit);

}

void UControlRigSequencerEditorLibrary::SetControlRigWorldTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FTransform>& WorldTransforms,
	EMovieSceneTimeUnit TimeUnit)
{
	LocalSetControlRigWorldTransforms(LevelSequence, ControlRig, ControlName, EControlRigSetKey::Always, Frames, WorldTransforms,TimeUnit);
}

bool UControlRigSequencerEditorLibrary::SmartReduce(FSmartReduceParams& ReduceParams, UMovieSceneSection* MovieSceneSection)
{
	//get level sequence if one exists...
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "Smart Reduce: No open level sequence");
		return false;
	}
	if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(MovieSceneSection))
	{
		TSharedPtr<ISequencer>  SequencerPtr = WeakSequencer.Pin();
		FControlRigParameterTrackEditor::SmartReduce(SequencerPtr, ReduceParams, Section);
		return true;
	}
	UE_LOGF(LogControlRig, Error, "Smart Reduce: Section is not Control Rig Section");
	return false;
}

bool UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer(UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FMovieSceneBindingProxy& Binding, bool bCreateLink)
{
	//get level sequence if one exists...
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer: No open level sequence");
		return false;
	}
	const TSharedPtr<ISequencer> ParentSequencer = WeakSequencer.Pin();

	UMovieSceneSequence* MovieSceneSequence = ParentSequencer->GetFocusedMovieSceneSequence();
	if (MovieSceneSequence == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer: No valid UMovieSceneSequence");
		return false;
	}
	ULevelSequence* Sequence = Cast<ULevelSequence>(MovieSceneSequence);
	if (Sequence == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer: No valid ULevelSequence");
		return false;
	}
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (Binding.Sequence != Sequence || !AnimSequence)
	{
		return false;
	}

	bool bResult = false;

	IMovieScenePlayer* Player = ParentSequencer.Get();
	FMovieSceneSequenceIDRef Template = ParentSequencer->GetFocusedTemplateID();
	TArrayView<TWeakObjectPtr<>>  Result = Player->FindBoundObjects(Binding.BindingID, Template);

	if (Result.Num() > 0 && Result[0].IsValid())
	{
		UObject* BoundObject = Result[0].Get();
		USkeleton* Skeleton = nullptr;
		USkeletalMeshComponent* SkeletalMeshComp = nullptr;
		AcquireSkeletonAndSkelMeshCompFromObject(BoundObject, &Skeleton, &SkeletalMeshComp);

		if (SkeletalMeshComp && SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
		{
			AnimSequence->SetSkeleton(SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton());

			UMovieSceneSequence* RootMovieSceneSequence = ParentSequencer->GetRootMovieSceneSequence();
			FMovieSceneSequenceTransform RootToLocalTransform = ParentSequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
			FAnimExportSequenceParameters AESQ;
			AESQ.Player = Player;
			AESQ.RootToLocalTransform = RootToLocalTransform;
			AESQ.MovieSceneSequence = Sequence;
			AESQ.RootMovieSceneSequence = RootMovieSceneSequence;
			AESQ.bForceUseOfMovieScenePlaybackRange = ParentSequencer.Get()->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();
			bResult = MovieSceneToolHelpers::ExportToAnimSequence(AnimSequence, ExportOptions, AESQ, SkeletalMeshComp);
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer: No skeletal mesh on skel mesh found in Sequencer");
		}
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer: No skel mesh found in Sequencer");
	}

	//the recording will have sequencer evaluating the last frame, force an evaluation so it re-evaluates at the current frame.
	WeakSequencer.Pin()->ForceEvaluate();

	//create the link to the anim sequence
	if (bResult && bCreateLink)
	{
		return USequencerToolsFunctionLibrary::LinkAnimSequence(Sequence, AnimSequence, ExportOptions, Binding);
	}
	return bResult;
}

bool UControlRigSequencerEditorLibrary::DeleteLinkedAnimSequence(const FMovieSceneBindingProxy& Binding)
{
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(Binding.Sequence);
	if (!LevelSequence)
	{
		return false;
	}
	if (IInterface_AssetUserData* AssetUserDataInterface = Cast<IInterface_AssetUserData>(LevelSequence))
	{
		ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData<ULevelSequenceAnimSequenceLink>();
		if (LevelAnimLink)
		{
			for (FLevelSequenceAnimSequenceLinkItem& Item : LevelAnimLink->AnimSequenceLinks)
			{
				if (Item.SkelTrackGuid == Binding.BindingID)
				{
					if (UAnimSequence* AnimSequence = Item.ResolveAnimSequence())
					{
						UE::Sequencer::FCommonAnimationTrackEditor::UnlinkLinkedAnimSequence(AnimSequence);
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::DeleteLinkedAnimTrack(const FMovieSceneBindingProxy& Binding)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer.IsValid())
	{
		return false;
	}
	return UE::Sequencer::FCommonAnimationTrackEditor::DeleteLinkedAnimTrackForBinding(
		Sequencer, Binding.BindingID, /*bOnlyAutoBaked=*/false);
}

UMovieSceneTrack* UControlRigSequencerEditorLibrary::GetLinkedAnimTrack(const FMovieSceneBindingProxy& InBinding)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}
	if (IMovieSceneLinkedAnimTrackProvider* Provider =
		UE::Sequencer::FCommonAnimationTrackEditor::FindProviderForBinding(Sequencer, InBinding.BindingID))
	{
		return Provider->GetLinkedAnimTrack(Sequencer, InBinding.BindingID);
	}
	return nullptr;
}

UMovieSceneTrack* UControlRigSequencerEditorLibrary::GetOrCreateLinkedAnimTrack(const FMovieSceneBindingProxy& InBinding)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}
	return UE::Sequencer::FCommonAnimationTrackEditor::GetOrCreateLinkedAnimTrack(Sequencer, InBinding.BindingID);
}

void UControlRigSequencerEditorLibrary::AddAutoBake(const TArray<FMovieSceneBindingProxy>& InBindings)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer.IsValid())
	{
		return;
	}
	TArray<FGuid> Guids;
	Guids.Reserve(InBindings.Num());
	for (const FMovieSceneBindingProxy& Binding : InBindings)
	{
		Guids.Add(Binding.BindingID);
	}
	UE::Sequencer::FCommonAnimationTrackEditor::ConfigureAutoBake(Sequencer, Guids);
}

bool UControlRigSequencerEditorLibrary::IsolateLinkedAnimTrack(const FMovieSceneBindingProxy& InBinding, bool bIsolate)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer.IsValid())
	{
		return false;
	}
	return UE::Sequencer::FCommonAnimationTrackEditor::IsolateLinkedAnimTrack(Sequencer, InBinding.BindingID, bIsolate);
}

bool UControlRigSequencerEditorLibrary::IsolateLinkedAnimTracksForAllBindings(bool bIsolate)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (!Sequencer.IsValid())
	{
		return false;
	}
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return false;
	}
	const UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}
	const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
	TArray<FGuid> Guids;
	Guids.SetNum(Bindings.Num());
	for (int32 Index = 0; Index < Bindings.Num(); ++Index)
	{
		Guids[Index] = Bindings[Index].GetObjectGuid();
	}
	return UE::Sequencer::FCommonAnimationTrackEditor::IsolateLinkedAnimTracks(Sequencer, Guids, bIsolate);
}

bool UControlRigSequencerEditorLibrary::BakeToControlRig(UWorld* World, ULevelSequence* LevelSequence, UClass* InClass, UAnimSeqExportOption* ExportOptions, bool bReduceKeys, float Tolerance,
	const FMovieSceneBindingProxy& Binding, bool bResetControls)
{
	if (World == nullptr || LevelSequence == nullptr || InClass == nullptr || ExportOptions == nullptr || LevelSequence->GetMovieScene() == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "BakeToControlRig: Need valid parameters");
		return false;
	}
	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	if (Binding.Sequence != LevelSequence)
	{
		UE_LOGF(LogControlRig, Error, "BakeToControlRig: Binding.Sequence different");
		return false;
	}
	//get level sequence if one exists...
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	ALevelSequenceActor* OutActor = nullptr;
	FMovieSceneSequencePlaybackSettings Settings;
	FMovieSceneSequenceID Template = MovieSceneSequenceID::Root;
	if (const TSharedPtr<ISequencer> ParentSequencer = WeakSequencer.Pin())
	{
		Template = ParentSequencer->GetFocusedTemplateID();
	}
	FMovieSceneSequenceTransform RootToLocalTransform;
	IMovieScenePlayer* Player = nullptr;
	ULevelSequencePlayer* LevelPlayer = nullptr;
	UMovieScene* MovieSceneForSpawnableRestore = nullptr;
	if (WeakSequencer.IsValid())
	{
		Player = WeakSequencer.Pin().Get();
	}
	else
	{
		Player = LevelPlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
		MovieSceneForSpawnableRestore = MovieScene;
	}
	if (Player == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Baking: Problem Setting up Player");
		return false;
	}
	bool bResult = false;
	const FScopedTransaction Transaction(LOCTEXT("BakeToControlRig_Transaction", "Bake To Control Rig"));
	{
		FAllSpawnableRestoreState SpawnableRestoreState(Player->GetSharedPlaybackState().ToSharedPtr(), MovieSceneForSpawnableRestore);

		if (LevelPlayer && SpawnableRestoreState.bWasChanged)
		{
			// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
			FFrameTime StartTime = FFrameRate::TransformTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value, MovieScene->GetTickResolution(), MovieScene->GetDisplayRate());
			LevelPlayer->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTime, EUpdatePositionMethod::Play));
		}
		TArrayView<TWeakObjectPtr<>>  Result = Player->FindBoundObjects(Binding.BindingID, static_cast<FMovieSceneSequenceIDRef>(Template));
		
		if (Result.Num() > 0 && Result[0].IsValid())
		{
			UObject* BoundObject = Result[0].Get();
			USkeleton* Skeleton = nullptr;
			USkeletalMeshComponent* SkeletalMeshComp = nullptr;
			AcquireSkeletonAndSkelMeshCompFromObject(BoundObject, &Skeleton, &SkeletalMeshComp);
			if (SkeletalMeshComp && SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
			{
				UAnimSequence* TempAnimSequence = NewObject<UAnimSequence>(GetTransientPackage(), NAME_None);
				TempAnimSequence->SetSkeleton(Skeleton);
				FAnimExportSequenceParameters AESP;
				AESP.Player = Player;
				AESP.RootToLocalTransform = RootToLocalTransform;
				AESP.MovieSceneSequence = LevelSequence;
				AESP.RootMovieSceneSequence = LevelSequence;
				AESP.bForceUseOfMovieScenePlaybackRange = false;
				bResult = MovieSceneToolHelpers::ExportToAnimSequence(TempAnimSequence, ExportOptions, AESP, SkeletalMeshComp);
				if (bResult == false)
				{
					TempAnimSequence->MarkAsGarbage();
					if (OutActor)
					{
						World->DestroyActor(OutActor);
					}
					return false;
				}

				MovieScene->Modify();

				TArray<UMovieSceneTrack*> ExistingTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.BindingID, NAME_None);
				for (UMovieSceneTrack* Existing : ExistingTracks)
				{
					MovieScene->RemoveTrack(*Existing);
				}

				UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(MovieScene->AddTrack(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.BindingID));
				if (Track)
				{
					Track->Modify();
					FString ObjectName = InClass->GetName();
					ObjectName.RemoveFromEnd(TEXT("_C"));
					UControlRig* ControlRig = NewObject<UControlRig>(Track, InClass, FName(*ObjectName), RF_Transactional);
					FName OldEventString = FName(FString(TEXT("Inverse")));

					if (InClass != UFKControlRig::StaticClass() && !(ControlRig->SupportsEvent(FRigUnit_InverseExecution::EventName) || ControlRig->SupportsEvent(OldEventString)))
					{
						TempAnimSequence->MarkAsGarbage();
						MovieScene->RemoveTrack(*Track);
						if (OutActor)
						{
							World->DestroyActor(OutActor);
						}
						return false;
					}
					FControlRigEditMode* ControlRigEditMode = nullptr;
					if (WeakSequencer.IsValid())
					{
						ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
						if (!ControlRigEditMode)
						{
							GLevelEditorModeTools().ActivateMode(FControlRigEditMode::ModeName);
							ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));

						}
						else
						{
							/* mz todo we don't unbind  will test more
							UControlRig* OldControlRig = ControlRigEditMode->GetControlRig(false);
							if (OldControlRig)
							{
								WeakSequencer.Pin()->ObjectImplicitlyRemoved(OldControlRig);
							}
							*/
						}
					}

					ControlRig->Modify();
					ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
					ControlRig->GetObjectBinding()->BindToObject(SkeletalMeshComp);
					ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, ControlRig->GetObjectBinding()->GetBoundObject());
					ControlRig->Initialize();
					ControlRig->RequestInit();
					ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkeletalMeshComp, true);
					ControlRig->Evaluate_AnyThread();

					bool bSequencerOwnsControlRig = true;
					UMovieSceneSection* NewSection = Track->CreateControlRigSection(0, ControlRig, bSequencerOwnsControlRig);
					UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(NewSection);

					//mz todo need to have multiple rigs with same class
					Track->SetTrackName(FName(*ObjectName));
					Track->SetDisplayName(FText::FromString(ObjectName));

					EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::SmartAuto;
					if (WeakSequencer.IsValid())
					{
						WeakSequencer.Pin()->EmptySelection();
						WeakSequencer.Pin()->SelectSection(NewSection);
						WeakSequencer.Pin()->ThrobSectionSelection();
						WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
						DefaultInterpolation = WeakSequencer.Pin()->GetKeyInterpolation();
					}
					FFrameNumber StartFrame = MovieScene->GetPlaybackRange().GetLowerBoundValue();
					UMovieSceneControlRigParameterSection::FLoadAnimSequenceData Data;
					Data.bKeyReduce = bReduceKeys;
					Data.Tolerance = Tolerance;
					Data.bResetControls = bResetControls;
					Data.StartFrame = FFrameNumber(0);
					ParamSection->LoadAnimSequenceIntoThisSection(TempAnimSequence, StartFrame, MovieScene, SkeletalMeshComp,
						Data, DefaultInterpolation);

					//Turn Off Any Skeletal Animation Tracks
					UMovieSceneSkeletalAnimationTrack* SkelTrack = Cast<UMovieSceneSkeletalAnimationTrack>(MovieScene->FindTrack(UMovieSceneSkeletalAnimationTrack::StaticClass(), Binding.BindingID, NAME_None));
					if (SkelTrack)
					{
						SkelTrack->Modify();

						MovieSceneHelpers::DisableTrack(SkelTrack);
					}
					//Finish Setup
					if (ControlRigEditMode)
					{
						ControlRigEditMode->AddControlRigObject(ControlRig, WeakSequencer.Pin());
					}

					TempAnimSequence->MarkAsGarbage();
					if (WeakSequencer.IsValid())
					{
						WeakSequencer.Pin()->ObjectImplicitlyAdded(ControlRig);
						WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
					}
					bResult = true;
				}
			}
		}
	}

	if (LevelPlayer)
	{
		LevelPlayer->Stop();
	}
	if (OutActor)
	{
		World->DestroyActor(OutActor);
	}
	return bResult;
}

//deprecreated 5.6
bool UControlRigSequencerEditorLibrary::LoadAnimSequenceIntoControlRigSection(UMovieSceneSection* MovieSceneSection, UAnimSequence* AnimSequence, USkeletalMeshComponent* SkelMeshComp,
	FFrameNumber InStartFrame, EMovieSceneTimeUnit TimeUnit, bool bKeyReduce, float Tolerance, EMovieSceneKeyInterpolation Interpolation, bool bResetControls)
{
	return  UControlRigSequencerEditorLibrary::LoadAnimSequenceIntoControlRigSectionWithRange(MovieSceneSection, AnimSequence, SkelMeshComp, InStartFrame,
		false, FFrameNumber(0), FFrameNumber(0),
		TimeUnit, bKeyReduce, Tolerance, Interpolation, bResetControls, false);
}

bool UControlRigSequencerEditorLibrary::LoadAnimSequenceIntoControlRigSectionWithRange(UMovieSceneSection* MovieSceneSection, UAnimSequence* AnimSequence, USkeletalMeshComponent* SkelMeshComp,
	FFrameNumber InStartFrame, bool bUseCustomAnimRange, FFrameNumber AnimStartRange, FFrameNumber AnimEndRange,
	EMovieSceneTimeUnit TimeUnit,bool bKeyReduce, float Tolerance, EMovieSceneKeyInterpolation Interpolation, bool bResetControls, bool bOntoSelectedControls)
{
	if (MovieSceneSection == nullptr || AnimSequence == nullptr || SkelMeshComp == nullptr)
	{
		return false;
	}
	UMovieScene* MovieScene = MovieSceneSection->GetTypedOuter<UMovieScene>();
	if (MovieScene == nullptr)
	{
		return false;
	}
	if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(MovieSceneSection))
	{
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InStartFrame = FFrameRate::TransformTime(FFrameTime(InStartFrame, 0), MovieScene->GetDisplayRate(),MovieScene->GetTickResolution()).RoundToFrame();
		}
		FFrameNumber StartFrame  = MovieScene->GetPlaybackRange().GetLowerBoundValue();
		UMovieSceneControlRigParameterSection::FLoadAnimSequenceData Data;
		Data.bKeyReduce = bKeyReduce;
		Data.Tolerance = Tolerance;
		Data.bResetControls = bResetControls;
		Data.StartFrame = InStartFrame;
		Data.bOntoSelectedControls = bOntoSelectedControls;
		if (bUseCustomAnimRange)
		{
			TRange<FFrameNumber> Range{ 0 };
			Range.SetLowerBoundValue(AnimStartRange);
			Range.SetUpperBoundValue(AnimEndRange);
			Data.AnimFrameRange = Range;
		}
		return Section->LoadAnimSequenceIntoThisSection(AnimSequence, StartFrame,  MovieScene, SkelMeshComp, Data, Interpolation);
	}
	return false;
}

static bool SetUpArrayOfControlRigValues(UControlRig* ControlRig, const TArray<FName>& ControlNames, int32 NumFrames, TArray<FArrayOfRigControlValues>& OutValues)
{
	if (ControlRig && NumFrames > 0)
	{
		for (const FName& ControlName : ControlNames)
		{
			if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
			{
				FArrayOfRigControlValues ControlValues;
				ControlValues.ControlName = ControlName;
				ControlValues.CachedElement = Element;
				ControlValues.Values.SetNum(NumFrames);
				OutValues.Add(ControlValues);
			}
		}
		return true;
	}
	return false;
}

static void StoreControlRigValues(UControlRig* ControlRig, int32 CurrentFrame, TArray<FArrayOfRigControlValues>& Values)
{
	if (ControlRig)
	{
		for (FArrayOfRigControlValues& Value : Values)
		{
			if (Value.CachedElement && CurrentFrame >= 0 && CurrentFrame < Value.Values.Num())
			{
				FRigControlValue RigValue = ControlRig->GetControlValue(Value.CachedElement, ERigControlValueType::Current);
				Value.Values[CurrentFrame] = RigValue;
			}
		}
	}
}

static bool LocalGetControlRigControlArrayOfValues(IMovieScenePlayer* Player, UMovieSceneSequence* MovieSceneSequence, FMovieSceneSequenceIDRef Template, FMovieSceneSequenceTransform& RootToLocalTransform,
	UControlRig* ControlRig, const TArray<FName>& ControlNames, EMovieSceneTimeUnit TimeUnit,
	const TArray<FFrameNumber>& InFrames, TArray<FArrayOfRigControlValues>& OutValues)

{
	if (Player == nullptr || MovieSceneSequence == nullptr || ControlRig == nullptr)
	{
		return false;
	}
	
	if (UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene())
	{

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		for (int32 Index = 0; Index < InFrames.Num(); ++Index)
		{
			const FFrameNumber& FrameNumber = InFrames[Index];
			FFrameTime GlobalTime(FrameNumber);
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				GlobalTime = FFrameRate::TransformTime(GlobalTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
			}
			GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);
			FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);

			Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
			ControlRig->Evaluate_AnyThread();
			StoreControlRigValues(ControlRig, Index, OutValues);
		}
	}
	else
	{
		return false;
	}
	return true;
}
TArray<FArrayOfRigControlValues> UControlRigSequencerEditorLibrary::GetControlValues(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FName>& ControlNames,
	const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FArrayOfRigControlValues> ControlValues;

	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	//if no level sequence get from sequencer, if there is a level sequence and it's not the same as the sequencer one, then make sequencer invalid
	if (!LevelSequence)
	{
		if (Sequencer.IsValid())
		{
			LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence());
		}
	}
	else if (Sequencer.IsValid())
	{
		if (LevelSequence != Sequencer->GetFocusedMovieSceneSequence())
		{
			Sequencer.Reset();
		}
	}

	if (!LevelSequence || !LevelSequence->GetMovieScene() || !ControlRig || !ControlRig->GetWorld())
	{
		UE_LOGF(LogControlRig, Error, "Missing level sequence or control rig");
		return ControlValues;
	}

	UWorld* World = ControlRig->GetWorld();

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();

	ALevelSequenceActor* OutActor = nullptr;
	FMovieSceneSequencePlaybackSettings Settings;
	FMovieSceneSequenceID Template = MovieSceneSequenceID::Root;
	if (Sequencer.IsValid())
	{
		Template = Sequencer->GetFocusedTemplateID();
	}
	FMovieSceneSequenceTransform RootToLocalTransform;
	IMovieScenePlayer* Player = nullptr;
	ULevelSequencePlayer* LevelPlayer = nullptr;
	UMovieScene* MovieSceneForSpawnableRestore = nullptr;
	if (Sequencer.IsValid())
	{
		Player = Sequencer.Get();
		Template = Sequencer->GetFocusedTemplateID();
		RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
	}
	else
	{
		Player = LevelPlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
	}
	if (Player == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "GetControlValues: Problem Setting up Player");
		return ControlValues;
	}

	if (SetUpArrayOfControlRigValues(ControlRig, ControlNames,Frames.Num(), ControlValues) == false)
	{
		return ControlValues;
	}

	bool bJustGetCurrent = false; // if we are at current sequencer time and just getting one frame don't evaluate
	if (Frames.Num() == 1 && Sequencer.IsValid())
	{
		FFrameTime GlobalTime(Frames[0]);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			GlobalTime = FFrameRate::TransformTime(GlobalTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
		}
		GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);
		if (Sequencer->GetLocalTime().Time == GlobalTime.GetFrame())
		{
			bJustGetCurrent = true;
		}
	}
	if (bJustGetCurrent)
	{
		constexpr int32 StartIndex = 0;
		StoreControlRigValues(ControlRig, StartIndex, ControlValues);
	}
	else
	{
		LocalGetControlRigControlArrayOfValues(Player, LevelSequence, Template, RootToLocalTransform,
			ControlRig, ControlNames, TimeUnit, Frames, ControlValues);
	}

	if (LevelPlayer)
	{
		LevelPlayer->Stop();
	}
	if (OutActor)
	{
		World->DestroyActor(OutActor);
	}
	if (Sequencer.IsValid())
	{
		Sequencer->ForceEvaluate();
	}
	return ControlValues;
}


static bool SetUpArrayOfControlRigTransforms(UControlRig* ControlRig, const TArray<FName>& ControlNames, int32 NumFrames, TArray<FArrayOfRigControlTransforms>& OutValues)
{
	if (ControlRig && NumFrames > 0)
	{
		for (const FName& ControlName : ControlNames)
		{
			if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
			{
				FArrayOfRigControlTransforms ControlValues;
				ControlValues.ControlName = ControlName;
				ControlValues.CachedElement = Element;
				ControlValues.Transforms.SetNum(NumFrames);
				OutValues.Add(ControlValues);
			}
		}
		return true;
	}
	return false;
}

static void StoreControlRigTransforms(UControlRig* ControlRig, int32 CurrentFrame, EControlRigTransformSpace Space, TArray<FArrayOfRigControlTransforms>& Values)
{
	constexpr bool bInitial = false;
	if (ControlRig && ControlRig->GetHierarchy())
	{
		for (FArrayOfRigControlTransforms& Value : Values)
		{
			if (Value.CachedElement && CurrentFrame >= 0 && CurrentFrame < Value.Transforms.Num())
			{
				if (Space == EControlRigTransformSpace::Local)
				{
					if (ControlRig->IsAdditive())
					{
						FTransform Transform = ControlRig->GetControlValue(Value.CachedElement, ERigControlValueType::Current).GetAsTransform(Value.CachedElement->Settings.ControlType, Value.CachedElement->Settings.PrimaryAxis);
						Value.Transforms[CurrentFrame] = Transform;
					}
					else
					{
						FTransform Transform  = ControlRig->GetHierarchy()->GetLocalTransformByIndex(Value.CachedElement->GetIndex(), bInitial);
						Value.Transforms[CurrentFrame] = Transform;
					}
				} 
				else //global
				{
					FTransform Transform = ControlRig->GetHierarchy()->GetGlobalTransformByIndex(Value.CachedElement->GetIndex(), bInitial);
					Value.Transforms[CurrentFrame] = Transform;

				}
			}
		}
	}
}

static bool LocalGetControlRigControlTransformOfValues(IMovieScenePlayer* Player, UMovieSceneSequence* MovieSceneSequence, FMovieSceneSequenceIDRef Template, FMovieSceneSequenceTransform& RootToLocalTransform,
	UControlRig* ControlRig, const TArray<FName>& ControlNames, EControlRigTransformSpace Space, EMovieSceneTimeUnit TimeUnit,
	const TArray<FFrameNumber>& InFrames, TArray<FArrayOfRigControlTransforms>& OutValues)

{
	if (Player == nullptr || MovieSceneSequence == nullptr || ControlRig == nullptr)
	{
		return false;
	}
	FAllSpawnableRestoreState SpawnableRestoreState(Player->GetSharedPlaybackState().ToSharedPtr());
	UWorld* World = ControlRig->GetWorld();

	if (UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene())
	{

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		for (int32 Index = 0; Index < InFrames.Num(); ++Index)
		{
			const FFrameNumber& FrameNumber = InFrames[Index];
			FFrameTime GlobalTime(FrameNumber);
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				GlobalTime = FFrameRate::TransformTime(GlobalTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
			}
			GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);
			FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);

			Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			Controller.EvaluateAllConstraints();
			ControlRig->Evaluate_AnyThread();
			StoreControlRigTransforms(ControlRig, Index, Space, OutValues);
		}
	}
	else
	{
		return false;
	}
	return true;
}

TArray<FArrayOfRigControlTransforms> UControlRigSequencerEditorLibrary::BatchGetControlTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FName>& ControlNames, const TArray<FFrameNumber>& Frames,
	EControlRigTransformSpace Space, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FArrayOfRigControlTransforms> ControlValues;

	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	//if no level sequence get from sequencer, if there is a level sequence and it's not the same as the sequencer one, then make sequencer invalid
	if (!LevelSequence)
	{
		if (Sequencer.IsValid())
		{
			LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence());
		}
	}
	else if (Sequencer.IsValid())
	{
		if (LevelSequence != Sequencer->GetFocusedMovieSceneSequence())
		{
			Sequencer.Reset();
		}
	}

	if (!LevelSequence || !LevelSequence->GetMovieScene() || !ControlRig || !ControlRig->GetWorld())
	{
		UE_LOGF(LogControlRig, Error, "Missing level sequence or control rig");
		return ControlValues;
	}

	UWorld* World = ControlRig->GetWorld();

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();

	ALevelSequenceActor* OutActor = nullptr;
	FMovieSceneSequencePlaybackSettings Settings;
	FMovieSceneSequenceID Template = MovieSceneSequenceID::Root;
	if (Sequencer.IsValid())
	{
		Template = Sequencer->GetFocusedTemplateID();
	}
	FMovieSceneSequenceTransform RootToLocalTransform;
	IMovieScenePlayer* Player = nullptr;
	ULevelSequencePlayer* LevelPlayer = nullptr;
	UMovieScene* MovieSceneForSpawnableRestore = nullptr;
	if (Sequencer.IsValid())
	{
		Player = Sequencer.Get();
		Template = Sequencer->GetFocusedTemplateID();
		RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
	}
	else
	{
		Player = LevelPlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
	}
	if (Player == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "BatchGetControlTransforms: Problem Setting up Player");
		return ControlValues;
	}

	if (SetUpArrayOfControlRigTransforms(ControlRig, ControlNames, Frames.Num(), ControlValues) == false)
	{
		return ControlValues;
	}

	bool bJustGetCurrent = false; // if we are at current sequencer time and just getting one frame don't evaluate
	if (Frames.Num() == 1 && Sequencer.IsValid())
	{
		FFrameTime GlobalTime(Frames[0]);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			GlobalTime = FFrameRate::TransformTime(GlobalTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
		}
		GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);
		if (Sequencer->GetLocalTime().Time == GlobalTime.GetFrame())
		{
			bJustGetCurrent = true;
		}
	}
	if (bJustGetCurrent)
	{
		constexpr int32 StartIndex = 0;
		StoreControlRigTransforms(ControlRig, StartIndex, Space, ControlValues);
	}
	else
	{
		LocalGetControlRigControlTransformOfValues(Player, LevelSequence, Template, RootToLocalTransform,
			ControlRig, ControlNames, Space, TimeUnit, Frames, ControlValues);
	}

	if (LevelPlayer)
	{
		LevelPlayer->Stop();
	}
	if (OutActor)
	{
		World->DestroyActor(OutActor);
	}
	if (Sequencer.IsValid())
	{
		Sequencer->ForceEvaluate();
	}
	return ControlValues;
}

bool UControlRigSequencerEditorLibrary::BatchSetControlTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FArrayOfRigControlTransforms>& InControls, const TArray<FFrameNumber>& Frames,
	EControlRigTransformSpace Space, UMovieSceneControlRigParameterSection* OptionalSectionToSet, EMovieSceneTimeUnit TimeUnit)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	//if no level sequence get from sequencer, if there is a level sequence and it's not the same as the sequencer one, then make sequencer invalid
	if (!LevelSequence)
	{
		if (Sequencer.IsValid())
		{
			LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence());
		}
	}
	else if (Sequencer.IsValid())
	{
		if (LevelSequence != Sequencer->GetFocusedMovieSceneSequence())
		{
			Sequencer.Reset();
		}
	}

	if (!LevelSequence || !LevelSequence->GetMovieScene() || !ControlRig || !ControlRig->GetWorld() || !ControlRig->GetHierarchy())
	{
		UE_LOGF(LogControlRig, Error, "Missing level sequence or control rig");
		return false;
	}

	UWorld* World = ControlRig->GetWorld();

	ALevelSequenceActor* OutActor = nullptr;
	FMovieSceneSequencePlaybackSettings Settings;
	FMovieSceneSequenceID Template = MovieSceneSequenceID::Root;
	if (Sequencer.IsValid())
	{
		Template = Sequencer->GetFocusedTemplateID();
	}
	FMovieSceneSequenceTransform RootToLocalTransform;
	IMovieScenePlayer* Player = nullptr;
	ULevelSequencePlayer* LevelPlayer = nullptr;
	UMovieScene* MovieSceneForSpawnableRestore = nullptr;
	if (Sequencer.IsValid())
	{
		Player = Sequencer.Get();
		Template = Sequencer->GetFocusedTemplateID();
		RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
	}
	else
	{
		Player = LevelPlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
	}
	if (Player == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "BatchSetControlTransforms: Problem Setting up Player");
		return false;
	}

	//remove controls that don't have transforms set up or have wrong name/type
	TArray<FArrayOfRigControlTransforms> Controls = InControls;
	for (int32 ControlIndex = Controls.Num() - 1; ControlIndex >= 0; --ControlIndex)
	{
		FArrayOfRigControlTransforms& ArrayOfCRTransforms = Controls[ControlIndex];
		if (ArrayOfCRTransforms.Transforms.Num() != Frames.Num())
		{
			Controls.RemoveAt(ControlIndex);
			continue;
		}
		ArrayOfCRTransforms.CachedElement = ControlRig->FindControl(ArrayOfCRTransforms.ControlName);
		if(ArrayOfCRTransforms.CachedElement == nullptr)
		{
			Controls.RemoveAt(ControlIndex);
			continue;
		}
	}
	UMovieSceneControlRigParameterTrack* Track = OptionalSectionToSet ? OptionalSectionToSet->GetTypedOuter<UMovieSceneControlRigParameterTrack>()
		: nullptr;
	TMap<FName, TWeakObjectPtr<UMovieSceneSection>> Empty;
	//if Track,Section null is fine, will just do nothing, otherwise if set it will make sure we set the keys on that section
	FControlRigParameterTrackSectionToKeyRestore Restore(Track, OptionalSectionToSet, Empty);
	FScopedLegacyEvaluationTrack RetoreEval(Track);
	FRigControlModifiedContext CRContext;
	CRContext.SetKey = EControlRigSetKey::Always;
	constexpr ERigTransformType::Type TransformType = ERigTransformType::CurrentGlobal;
	FScopedTransaction AddControlRigTrackTransaction(LOCTEXT("BatchSetControlRigTransforms", "Batch Set Control Rig Transforms"));
	FScopedSlowTask Progress(Frames.Num(), LOCTEXT("BatchSetControlTransforms_SlowTask", "Batch Set Control Rig Transforms..."));
	Progress.MakeDialog(true);


	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		Progress.EnterProgressFrame(1);
		const FFrameNumber& FrameNumber = Frames[Index];
		FFrameTime GlobalTime(FrameNumber);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			GlobalTime = FFrameRate::TransformTime(GlobalTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
		}
		GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);
		FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);

		Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.EvaluateAllConstraints();
		ControlRig->Evaluate_AnyThread();
		CRContext.LocalTime = TickResolution.AsSeconds(GlobalTime);
		constexpr bool bFixEulerFlips = true;
		constexpr bool bNotify = true;
		constexpr bool bSetupUndo = false;
		for (FArrayOfRigControlTransforms& ArrayOfCRTransforms : Controls)
		{
			FTransform Transform = ArrayOfCRTransforms.Transforms[Index];
			
			if (ArrayOfCRTransforms.CachedElement)
			{
				if (Space == EControlRigTransformSpace::Global)
				{
					//todo create function that uses index or element directly

					FRigControlValue Value = ControlRig->GetControlValueFromGlobalTransform(ArrayOfCRTransforms.ControlName, Transform, TransformType);
					Transform = Value.GetAsTransform(ArrayOfCRTransforms.CachedElement->Settings.ControlType, ArrayOfCRTransforms.CachedElement->Settings.PrimaryAxis);
					FEulerTransform EulerTransform(Transform);
					UE::AIE::FSetTransformHelpers::SetControlTransform(EulerTransform, ControlRig, ArrayOfCRTransforms.CachedElement, CRContext);

				}
				else
				{
					FRigControlValue Value;
					Value.SetFromTransform(Transform, ArrayOfCRTransforms.CachedElement->Settings.ControlType, ArrayOfCRTransforms.CachedElement->Settings.PrimaryAxis);

					ControlRig->SetControlValue(ArrayOfCRTransforms.ControlName, Value, bNotify, CRContext, bSetupUndo, false /*bPrintPython not defined!*/, bFixEulerFlips);
				}
			}
		}
	}

	if (LevelPlayer)
	{
		LevelPlayer->Stop();
	}
	if (OutActor)
	{
		World->DestroyActor(OutActor);
	}
	if (Sequencer.IsValid())
	{
		Sequencer->ForceEvaluate();
	}

	return true;
}

static bool LocalGetControlRigControlValues(IMovieScenePlayer* Player, UMovieSceneSequence* MovieSceneSequence, FMovieSceneSequenceIDRef Template, FMovieSceneSequenceTransform& RootToLocalTransform,
	UControlRig* ControlRig, const FName& ControlName, EMovieSceneTimeUnit TimeUnit,
	const TArray<FFrameNumber>& InFrames, TArray<FRigControlValue>& OutValues)
{
	if (Player == nullptr || MovieSceneSequence == nullptr || ControlRig == nullptr)
	{
		return false;
	}
	if (ControlRig->FindControl(ControlName) == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Can not find Control %ls", *(ControlName.ToString()));
		return false;
	}
	if (UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene())
	{

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		OutValues.SetNum(InFrames.Num());
		for (int32 Index = 0; Index < InFrames.Num(); ++Index)
		{
			const FFrameNumber& FrameNumber = InFrames[Index];
			FFrameTime GlobalTime(FrameNumber);
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				GlobalTime = FFrameRate::TransformTime(GlobalTime, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution());
			}
			GlobalTime = RootToLocalTransform.Inverse().TryTransformTime(GlobalTime).Get(GlobalTime);
			FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);

			Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
			ControlRig->Evaluate_AnyThread();
			OutValues[Index] = ControlRig->GetControlValue(ControlName);
		}
	}
	return true;
}

static bool GetControlRigValues(ISequencer* Sequencer, UControlRig* ControlRig, const FName& ControlName, EMovieSceneTimeUnit TimeUnit,
	const TArray<FFrameNumber>& Frames,  TArray<FRigControlValue>& OutValues)
{
	if (Sequencer->GetFocusedMovieSceneSequence())
	{
		FMovieSceneSequenceIDRef Template = Sequencer->GetFocusedTemplateID();
		FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
		bool bGood = LocalGetControlRigControlValues(Sequencer, Sequencer->GetFocusedMovieSceneSequence(), Template, RootToLocalTransform,
			ControlRig, ControlName,TimeUnit, Frames, OutValues);
		Sequencer->RequestEvaluate();
		return bGood;
	}
	return false;
}

static bool GetControlRigValue(ISequencer* Sequencer, UControlRig* ControlRig, const FName& ControlName, EMovieSceneTimeUnit TimeUnit,
	const FFrameNumber Frame, FRigControlValue& OutValue)
{
	if (Sequencer->GetFocusedMovieSceneSequence())
	{
		TArray<FFrameNumber> Frames;
		Frames.Add(Frame);
		TArray<FRigControlValue> OutValues;
		FMovieSceneSequenceIDRef Template = Sequencer->GetFocusedTemplateID();
		FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
		bool bVal = LocalGetControlRigControlValues(Sequencer, Sequencer->GetFocusedMovieSceneSequence(), Template, RootToLocalTransform,
			ControlRig, ControlName,TimeUnit, Frames, OutValues);
		if (bVal)
		{
			OutValue = OutValues[0];
		}
		return bVal;
	}
	return false;
}

static bool GetControlRigValues(UWorld* World, ULevelSequence* LevelSequence, UControlRig* ControlRig, const FName& ControlName, EMovieSceneTimeUnit TimeUnit,
	const TArray<FFrameNumber>& Frames, TArray<FRigControlValue>& OutValues)
{
	if (LevelSequence)
	{
		ALevelSequenceActor* OutActor = nullptr;
		FMovieSceneSequencePlaybackSettings Settings;
		FMovieSceneSequenceIDRef Template = MovieSceneSequenceID::Root;
		FMovieSceneSequenceTransform RootToLocalTransform;
		ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
		return LocalGetControlRigControlValues(Player, LevelSequence, Template, RootToLocalTransform,
			ControlRig, ControlName,TimeUnit, Frames, OutValues);

	}
	return false;
}

float UControlRigSequencerEditorLibrary::GetLocalControlRigFloat(ULevelSequence* LevelSequence,UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	float Value = 0.0f;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Float && Element->Settings.ControlType != ERigControlType::ScaleFloat)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frame, RigValue))
		{
			Value = RigValue.Get<float>();
		}
	}
	return Value;
}

TArray<float> UControlRigSequencerEditorLibrary::GetLocalControlRigFloats(ULevelSequence* LevelSequence,UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<float> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Float && Element->Settings.ControlType != ERigControlType::ScaleFloat)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}

		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				float Value = RigValue.Get<float>();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

namespace UE::Private
{
	FRigControlModifiedContext GetContext(FFrameNumber InFrame, const UMovieScene* InMovieScene, const EMovieSceneTimeUnit InTimeUnit, const bool bInSetKey)
	{
		const FFrameRate TickResolution = InMovieScene->GetTickResolution();
		if (InTimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InFrame = FFrameRate::TransformTime(FFrameTime(InFrame, 0), InMovieScene->GetDisplayRate(), TickResolution).RoundToFrame();
		}
		
		FRigControlModifiedContext Context;
		Context.SetKey = bInSetKey ? EControlRigSetKey::Always : EControlRigSetKey::DoNotCare;
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(InFrame));

		return MoveTemp(Context);
	}
	
	struct FRigSetterContext
	{
	public:
		FRigSetterContext(UControlRig* InControlRig, const FName InControlName)
			: ControlRig(InControlRig)
			, ControlName(InControlName)
		{}

		FRigControlElement* GetControl(const TFunctionRef<bool(const ERigControlType ControlType)>& InTypeFilter) const
		{
			if (!ControlRig)
			{
				UE_LOGF(LogControlRig, Error, "ControlRig is null.");
				return nullptr;
			}
			
			FRigControlElement* Control = ControlRig->FindControl(ControlName);
			if (!Control)
			{
				UE_LOGF(LogControlRig, Error, "Control '%ls' not found.", *ControlName.ToString());
				return nullptr;
			}

			if (!InTypeFilter(Control->Settings.ControlType))
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return nullptr;
			}

			return Control;
		}

		FRigControlElement* GetControl(const ERigControlType InControlType) const
		{
			return GetControl([InControlType](const ERigControlType ControlType)
			{
				return ControlType == InControlType;
			});
		}
		
	private:
		UControlRig* ControlRig = nullptr;
		FName ControlName = NAME_None;
	};

	static bool bIsInteracting = false;
}

void UControlRigSequencerEditorLibrary::SetInteraction(bool bIsInteracting)
{
	UE::Private::bIsInteracting = bIsInteracting;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigFloat(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, float Value, EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl([](const ERigControlType ControlType)
	{
		return ControlType == ERigControlType::Float || ControlType == ERigControlType::ScaleFloat;
	});

	if (!Control)
	{
		return;
	}
	
	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<float>(ControlName, Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<float>(ControlName, Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigFloats(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<float> Values,
	EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl([](const ERigControlType ControlType)
	{
		return ControlType == ERigControlType::Float || ControlType == ERigControlType::ScaleFloat;
	});

	if (!Control)
	{
		return;
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		float  Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<float>(ControlName, Value, true, Context);
	}
}


bool UControlRigSequencerEditorLibrary::GetLocalControlRigBool(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	bool Value = true;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Bool)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frame, RigValue))
		{
			Value = RigValue.Get<bool>();
		}
	}
	return Value;
}

TArray<bool> UControlRigSequencerEditorLibrary::GetLocalControlRigBools(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<bool> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Bool)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				bool Value = RigValue.Get<bool>();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigBool(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, bool Value, EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (!MovieScene || !ControlRig)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Bool);
	if (!Control)
	{
		return;
	}

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<bool>(ControlName, Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<bool>(ControlName, Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigBools(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
	const TArray<bool> Values, EMovieSceneTimeUnit TimeUnit)
{	
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Bool);
	if (!Control)
	{
		return;
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;

	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		
		bool Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<bool>(ControlName, Value, true, Context);
	}
}

int32 UControlRigSequencerEditorLibrary::GetLocalControlRigInt(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	int32 Value = 0;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Integer)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frame, RigValue))
		{
			Value = RigValue.Get<int32>();
		}
	}
	return Value;
}

TArray<int32> UControlRigSequencerEditorLibrary::GetLocalControlRigInts(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<int32> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Integer)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit,Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				int32 Value = RigValue.Get<int32>();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigInt(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, int32 Value, EMovieSceneTimeUnit TimeUnit,bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (!MovieScene || !ControlRig)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Integer);
	if (!Control)
	{
		return;
	}

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);

	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<int32>(ControlName, Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<int32>(ControlName, Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigInts(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
	const TArray<int32> Values , EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}
	
	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Integer);
	if (!Control)
	{
		return;
	}
	
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		int32 Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<int32>(ControlName, Value, true, Context);
	}
}


FVector2D UControlRigSequencerEditorLibrary::GetLocalControlRigVector2D(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	FVector2D Value = FVector2D::ZeroVector;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Vector2D)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frame, RigValue))
		{
			const FVector3f TempValue = RigValue.Get<FVector3f>(); 
			Value = FVector2D(TempValue.X, TempValue.Y);
		}
	}
	return Value;
}

TArray<FVector2D> UControlRigSequencerEditorLibrary::GetLocalControlRigVector2Ds(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FVector2D> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Vector2D)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				const FVector3f TempValue = RigValue.Get<FVector3f>(); 
				Values.Add(FVector2D(TempValue.X, TempValue.Y));
			}
		}
	}
	return Values;

}

void UControlRigSequencerEditorLibrary::SetLocalControlRigVector2D(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, FVector2D Value, EMovieSceneTimeUnit TimeUnit,bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Vector2D);
	if (!Control)
	{
		return;
	}

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FVector3f>(ControlName, FVector3f(Value.X, Value.Y, 0.f), true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FVector3f>(ControlName, FVector3f(Value.X, Value.Y, 0.f), true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigVector2Ds(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, 
	const TArray<FVector2D> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Vector2D);
	if (!Control)
	{
		return;
	}
	
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		FVector2D Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<FVector3f>(ControlName, FVector3f(Value.X, Value.Y, 0.f), true, Context);
	}
}


FVector UControlRigSequencerEditorLibrary::GetLocalControlRigPosition(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	FVector Value = FVector::ZeroVector;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Position)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frame, RigValue))
		{
			Value = (FVector)RigValue.Get<FVector3f>();
		}
	}
	return Value;
}

TArray<FVector> UControlRigSequencerEditorLibrary::GetLocalControlRigPositions(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FVector> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Position)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				FVector Value = (FVector)RigValue.Get<FVector3f>();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigPosition(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, FVector Value, EMovieSceneTimeUnit TimeUnit,bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Position);
	if (!Control)
	{
		return;
	}
	
	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);

	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigPositions(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, 
	const TArray<FVector> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}
	
	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Position);
	if (!Control)
	{
		return;
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		FVector Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value, true, Context);
	}
}


FRotator UControlRigSequencerEditorLibrary::GetLocalControlRigRotator(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	FRotator Value = FRotator::ZeroRotator;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Rotator)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frame, RigValue))
		{
			Value = FRotator::MakeFromEuler((FVector)RigValue.Get<FVector3f>());
		}
	}
	return Value;
}

TArray<FRotator> UControlRigSequencerEditorLibrary::GetLocalControlRigRotators(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FRotator> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Rotator)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				FRotator Value = FRotator::MakeFromEuler((FVector)RigValue.Get<FVector3f>());
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigRotator(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, FRotator Value, EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	FRigControlElement* Control = Setter.GetControl(ERigControlType::Rotator);
	if (!Control)
	{
		return;
	}

	const FVector EulerAngle(Value.Roll, Value.Pitch, Value.Yaw);
	ControlRig->GetHierarchy()->SetControlSpecifiedEulerAngle(Control, EulerAngle);

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);

	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value.Euler(), true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value.Euler(), true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigRotators(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
	const TArray<FRotator> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	FRigControlElement* Control = Setter.GetControl(ERigControlType::Rotator);
	if (!Control)
	{
		return;
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));

		const FRotator Value = Values[Index];
		const FVector EulerAngle(Value.Roll, Value.Pitch, Value.Yaw);
		ControlRig->GetHierarchy()->SetControlSpecifiedEulerAngle(Control, EulerAngle);
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value.Euler(), true, Context);
	}
}


FVector UControlRigSequencerEditorLibrary::GetLocalControlRigScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,  EMovieSceneTimeUnit TimeUnit)
{
	FVector Value = FVector::ZeroVector;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Scale)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frame, RigValue))
		{
			Value = (FVector)RigValue.Get<FVector3f>();
		}
	}
	return Value;
}

TArray<FVector>UControlRigSequencerEditorLibrary::GetLocalControlRigScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FVector> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Scale)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				FVector Value = (FVector)RigValue.Get<FVector3f>();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, FVector Value, EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Scale);
	if (!Control)
	{
		return;
	}
	
	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, 
	const TArray<FVector> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}
	
	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Scale);
	if (!Control)
	{
		return;
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		FVector Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<FVector3f>(ControlName, (FVector3f)Value, true, Context);
	}
}


FEulerTransform UControlRigSequencerEditorLibrary::GetLocalControlRigEulerTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	FEulerTransform Value = FEulerTransform::Identity;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::EulerTransform)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frame, RigValue))
		{
			Value = RigValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
		}
	}
	return Value;
}

TArray<FEulerTransform> UControlRigSequencerEditorLibrary::GetLocalControlRigEulerTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FEulerTransform> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::EulerTransform)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				FEulerTransform Value = RigValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigEulerTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FEulerTransform Value,
	EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}
	
	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	FRigControlElement* Control = Setter.GetControl(ERigControlType::EulerTransform);
	if (!Control)
	{
		return;
	}

	const FVector EulerAngle(Value.Rotation.Roll, Value.Rotation.Pitch, Value.Rotation.Yaw);
	ControlRig->GetHierarchy()->SetControlSpecifiedEulerAngle(Control, EulerAngle);

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlName, Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlName, Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigEulerTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
	const TArray<FEulerTransform> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	FRigControlElement* Control = Setter.GetControl(ERigControlType::EulerTransform);
	if (!Control)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		FEulerTransform Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));

		const FVector EulerAngle(Value.Rotation.Roll, Value.Rotation.Pitch, Value.Rotation.Yaw);
		Hierarchy->SetControlSpecifiedEulerAngle(Control, EulerAngle);
		ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlName, Value, true, Context);
	}
}


FTransformNoScale UControlRigSequencerEditorLibrary::GetLocalControlRigTransformNoScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	FTransformNoScale Value = FTransformNoScale::Identity;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::TransformNoScale)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frame, RigValue))
		{
			Value = RigValue.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
		}
	}
	return Value;
}

TArray<FTransformNoScale> UControlRigSequencerEditorLibrary::GetLocalControlRigTransformNoScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FTransformNoScale> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::TransformNoScale)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName,TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				FTransformNoScale Value = RigValue.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigTransformNoScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, FTransformNoScale Value, EMovieSceneTimeUnit TimeUnit,bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}
	
	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::TransformNoScale);
	if (!Control)
	{
		return;
	}

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(ControlName, Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(ControlName, Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigTransformNoScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, 
	const TArray<FTransformNoScale> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}

	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::TransformNoScale);
	if (!Control)
	{
		return;
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		FTransformNoScale Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(ControlName, Value, true, Context);
	}
}


FTransform UControlRigSequencerEditorLibrary::GetLocalControlRigTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit)
{
	FTransform Value = FTransform::Identity;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Transform)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Value;
			}
		}
		
		FRigControlValue RigValue;
		if (GetControlRigValue(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frame, RigValue))
		{
			Value = RigValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
		}
	}
	return Value;
}

TArray<FTransform> UControlRigSequencerEditorLibrary::GetLocalControlRigTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit)
{
	TArray<FTransform> Values;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();

	if (WeakSequencer.IsValid() && ControlRig)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			if (Element->Settings.ControlType != ERigControlType::Transform)
			{
				UE_LOGF(LogControlRig, Error, "Control Rig Wrong Type");
				return Values;
			}
		}
		TArray<FRigControlValue> RigValues;
		if (GetControlRigValues(WeakSequencer.Pin().Get(), ControlRig, ControlName, TimeUnit, Frames, RigValues))
		{
			Values.Reserve(RigValues.Num());
			for (const FRigControlValue& RigValue : RigValues)
			{
				FTransform Value = RigValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
				Values.Add(Value);
			}
		}
	}
	return Values;
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName,
	FFrameNumber Frame, FTransform Value, EMovieSceneTimeUnit TimeUnit, bool bSetKey)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr)
	{
		return;
	}


	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Transform);
	if (!Control)
	{
		return;
	}

	const FRigControlModifiedContext Context = UE::Private::GetContext(Frame, MovieScene, TimeUnit, bSetKey);
	if (UE::Private::bIsInteracting)
	{
		FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());
		ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(ControlName, Value, true, Context);
		ControlRig->Evaluate_AnyThread();
	}
	else
	{
		ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(ControlName, Value, true, Context);
	}
}

void UControlRigSequencerEditorLibrary::SetLocalControlRigTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, 
	const TArray<FTransform> Values, EMovieSceneTimeUnit TimeUnit)
{
	const UMovieScene* MovieScene = LevelSequence ? LevelSequence->GetMovieScene() : nullptr;
	if (MovieScene == nullptr || ControlRig == nullptr || (Frames.Num() != Values.Num()))
	{
		return;
	}
	
	const UE::Private::FRigSetterContext Setter(ControlRig, ControlName);
	const FRigControlElement* Control = Setter.GetControl(ERigControlType::Transform);
	if (!Control)
	{
		return;
	}
	
	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
	
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		FFrameNumber Frame = Frames[Index];
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Frame = FFrameRate::TransformTime(FFrameTime(Frame, 0), DisplayRate, TickResolution).RoundToFrame();
		}
		FTransform Value = Values[Index];
		Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
		ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(ControlName, Value, true, Context);
	}
}

bool UControlRigSequencerEditorLibrary::ImportFBXToControlRigTrack(UWorld* World, ULevelSequence* Sequence, UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneControlRigParameterSection* InSection,
	const TArray<FString>& ControlRigNames,
	UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
	const FString& ImportFilename)
{

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene || MovieScene->IsReadOnly() || !InTrack)
	{
		return false;
	}

	bool bValid = false;
	ALevelSequenceActor* OutActor;
	FMovieSceneSequencePlaybackSettings Settings;
	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, Sequence, Settings, OutActor);

	INodeAndChannelMappings* ChannelMapping = Cast<INodeAndChannelMappings>(InTrack);
	if (ChannelMapping)
	{
		TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels = ChannelMapping->GetNodeAndChannelMappings(InSection);
		TArray<FName> SelectedControls;
		for (const FString& StringName : ControlRigNames)
		{
			FName Name(*StringName);
			SelectedControls.Add(Name);
		}

		bValid = MovieSceneToolHelpers::ImportFBXIntoControlRigChannels(MovieScene, ImportFilename, ImportFBXControlRigSettings,
			NodeAndChannels, SelectedControls, MovieScene->GetTickResolution());
		if (NodeAndChannels)
		{
			delete NodeAndChannels;
		}
	}
	return bValid;
}

bool UControlRigSequencerEditorLibrary::ControlRigCopyVectorParameterCurvesToTransform(UMovieSceneControlRigParameterTrack* InTrack,
	const FName& InControlName, ERigControlType InType)
{
	if (InTrack == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "UControlRigSequencerEditorLibrary::ControlRigCopyVectorParameterCurvesToTransform:  Track is null");
		return false;
	}
	bool bPassed = false;
	for (UMovieSceneSection* Section : InTrack->GetAllSections())
	{
		if (Section)
		{
			UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section);
			if (CRSection)
			{
				if (CRSection->CopyVectorParameterCurvesToTransform(InControlName, InType))
				{
					bPassed = true;
				}
			}
		}
	}
	if (bPassed == false)
	{
		UE_LOGF(LogControlRig, Error, "UControlRigSequencerEditorLibrary::ControlRigCopyVectorParameterCurvesToTransform:  No Sections Converted");
	}
	return bPassed;
}

bool UControlRigSequencerEditorLibrary::ExportFBXFromControlRigSection(ULevelSequence* Sequence, const UMovieSceneControlRigParameterSection* Section, const UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings)
{
	if (!Sequence || !ExportFBXControlRigSettings ||
		!Section || !Section->GetControlRig() ||
		!Sequence->GetMovieScene() || Sequence->GetMovieScene()->IsReadOnly())
	{
		return false;
	}

	TArray<FName> SelectedControls;
	if (UMovieSceneControlRigParameterTrack* Track = Section->GetTypedOuter<UMovieSceneControlRigParameterTrack>())
	{
		Track->GetSelectedNodes(SelectedControls);
	}

	FMovieSceneSequenceTransform RootToLocalTransform;
	if (IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Sequence, false))
	{
		if (const ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor))
		{
			if (const TSharedPtr<ISequencer> Sequencer = LevelSequenceEditor->GetSequencer())
			{
				RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
			}
		}
	}
	
	return  MovieSceneToolHelpers::ExportFBXFromControlRigChannels(Section, ExportFBXControlRigSettings, SelectedControls, RootToLocalTransform);
}

bool UControlRigSequencerEditorLibrary::CollapseControlRigAnimLayersWithSettings(ULevelSequence* InSequence, UMovieSceneControlRigParameterTrack* InTrack, const FBakingAnimationKeySettings& InSettings)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;
	if (WeakSequencer.IsValid() && InTrack)
	{
		TSharedPtr<ISequencer>  SequencerPtr = WeakSequencer.Pin();
		bValid = FControlRigParameterTrackEditor::CollapseAllLayers(SequencerPtr, InTrack, InSettings);
		
	}
	return bValid;
}

bool UControlRigSequencerEditorLibrary::CollapseControlRigAnimLayers(ULevelSequence* LevelSequence, UMovieSceneControlRigParameterTrack* InTrack, bool bKeyReduce, float Tolerance)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;

	if (WeakSequencer.IsValid() && InTrack)
	{
		TSharedPtr<ISequencer>  SequencerPtr = WeakSequencer.Pin();
		FBakingAnimationKeySettings CollapseControlsSettings;
		const FFrameRate TickResolution = SequencerPtr->GetFocusedTickResolution();
		const FFrameTime FrameTime = SequencerPtr->GetLocalTime().ConvertTo(TickResolution);
		FFrameNumber CurrentTime = FrameTime.GetFrame();

		TRange<FFrameNumber> Range = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();

		CollapseControlsSettings.StartFrame = Range.GetLowerBoundValue();
		CollapseControlsSettings.EndFrame = Range.GetUpperBoundValue();
		bValid = FControlRigParameterTrackEditor::CollapseAllLayers(SequencerPtr,InTrack, CollapseControlsSettings);	
	}
	return bValid;
}

bool UControlRigSequencerEditorLibrary::DeleteTimeWarp(UMovieSceneControlRigParameterSection* InSection)
{
	if (InSection)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteTimeWarp", "Delete Time Warp"));
		InSection->DeleteTimeWarp();
		return true;
		
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::BakeTimeWarp(UMovieSceneControlRigParameterSection* InSection)
{
	if(TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		if (InSection)
		{
			FScopedTransaction Transaction(LOCTEXT("BakeTimeWarp", "Bake Time Warp"));
			return FControlRigParameterTrackEditor::BakeTimeWarp(SequencerPtr, InSection);
		}
	}
	return false;
}

UMovieSceneScriptingChannel* UControlRigSequencerEditorLibrary::GetSpaceChannel(UMovieSceneSequence* InSequence, UControlRig* ControlRig, FName ControlName, bool bCreateIfNeeded)
{
	UMovieSceneScriptingChannel* ScriptingChannel = nullptr;
	if (InSequence == nullptr ||  ControlRig == nullptr)
	{
		return ScriptingChannel;
	}
	UMovieScene* MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return ScriptingChannel;

	}

	bool bFoundTrack = false;

	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
	bool bRecreateCurves = false;
	TArray<TPair<UControlRig*, FName>> ControlRigPairsToReselect;
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
		for (UMovieSceneTrack* Track : Tracks)
		{
			UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
			if (ControlRigParameterTrack && ControlRigParameterTrack->GetControlRig() == ControlRig)
			{
				TArray<UMovieSceneSection*> AllSections = ControlRigParameterTrack->GetAllSections();
				if (UMovieSceneControlRigParameterSection* ActiveSection = Cast<UMovieSceneControlRigParameterSection>(AllSections.Num() > 0 ? AllSections[0] : nullptr))
				{
					FSpaceControlNameAndChannel* NameAndChannel = ActiveSection->GetSpaceChannel(ControlName);
					if (NameAndChannel == nullptr && bCreateIfNeeded)
					{
						ControlRig->Modify();
						ActiveSection->AddSpaceChannel(ControlName, true /*ReconstructChannelProxy*/);
						NameAndChannel = ActiveSection->GetSpaceChannel(ControlName);
						//important we don't want the adding of the space channel to be in the undo since we want the channel to stick around if we get undone
						ActiveSection->Modify();
					}

					if (NameAndChannel)
					{
						FMovieSceneChannelProxy& ChannelProxy = ActiveSection->GetChannelProxy();
						TArray<UMovieSceneScriptingChannel*> Channels;
						GetScriptingChannelsForChannel<FMovieSceneControlRigSpaceChannel, UMovieSceneScriptingSpaceChannel>(ChannelProxy, InSequence, ActiveSection, Channels);
						if (Channels.Num() > 0)
						{
							for (UMovieSceneScriptingChannel* Channel : Channels)
							{
								if (Channel)
								{
									FString ChannelName = Channel->ChannelName.ToString();
									FString CName, SpaceName;
									ChannelName.Split(".", &CName, &SpaceName);
									if (NameAndChannel->ControlName == FName(*CName))
									{
										ScriptingChannel = Channel;
										break;
									}
								}
							}
						}
					}
				}
				break;
			}
		}
	}
	return ScriptingChannel;
}

bool UControlRigSequencerEditorLibrary::SetControlRigSpace(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const FRigElementKey& InSpaceKey, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;

	if (WeakSequencer.IsValid() && ControlRig && ControlName != NAME_None)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
			
			constexpr bool bDoNotCreate = false;
			FSpaceChannelAndSection SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, ControlName, Sequencer.Get(), bDoNotCreate);
			if (!SpaceChannelAndSection.SpaceChannel)
			{
				// no need to create a space channel if InSpaceKey is the current parent
				const FRigElementKey ControlKey(ControlName, ERigElementType::Control);
				if (ControlRig->GetHierarchy()->GetActiveParent(ControlKey) == InSpaceKey)
				{
					return true;
				}
			}
			
			FScopedTransaction Transaction(LOCTEXT("KeyControlRigSpace", "Key Control Rig Space"));
			SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, ControlName, Sequencer.Get(), true /*bCreateIfNeeded*/);
			if (SpaceChannelAndSection.SpaceChannel)
			{
				if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
				{
					InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), LevelSequence->GetMovieScene()->GetDisplayRate(), LevelSequence->GetMovieScene()->GetTickResolution()).RoundToFrame();
				}
				FKeyHandle Handle = FControlRigSpaceChannelHelpers::SequencerKeyControlRigSpaceChannel(ControlRig, Sequencer.Get(), SpaceChannelAndSection.SpaceChannel, SpaceChannelAndSection.SectionToKey, InTime, ControlRig->GetHierarchy(), Element->GetKey(), InSpaceKey);
				bValid = Handle != FKeyHandle::Invalid();
			}
			else
			{
				UE_LOGF(LogControlRig, Error, "Can not find Space Channel");
				return false;
			}
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Can not find Control with that Name");
			return false;
		}
	}
	return bValid;
}


bool UControlRigSequencerEditorLibrary::SpaceCompensate(UControlRig* InControlRig, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() && InControlRig)
	{
		TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
		TOptional<FFrameNumber> OptionalTime;
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
			const FFrameRate& FrameRate = Sequencer->GetFocusedDisplayRate();
			OptionalTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), FrameRate, TickResolution).RoundToFrame();
		}
		else
		{
			OptionalTime = InTime;
		}
		FScopedTransaction Transaction(LOCTEXT("SpacecCompensate", "Space Compensate"));
		// compensate spaces
		if (UMovieSceneControlRigParameterSection* CRSection = FControlRigSpaceChannelHelpers::GetControlRigSection(Sequencer.Get(), InControlRig))
		{
			// compensate spaces
			FControlRigSpaceChannelHelpers::CompensateIfNeeded(
				InControlRig, Sequencer.Get(), CRSection,
				OptionalTime, true /*comp previous*/);
		}
		return true;
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::SpaceCompensateAll(UControlRig* InControlRig)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;
	TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
	if (WeakSequencer.IsValid() && InControlRig)
	{
		TOptional<FFrameNumber> OptionalTime;
		FScopedTransaction Transaction(LOCTEXT("SpacecCompensateAll", "Space Compensate All"));
		// compensate spaces
		if (UMovieSceneControlRigParameterSection* CRSection = FControlRigSpaceChannelHelpers::GetControlRigSection(Sequencer.Get(), InControlRig))
		{
			// compensate spaces
			FControlRigSpaceChannelHelpers::CompensateIfNeeded(
				InControlRig, Sequencer.Get(), CRSection,
				OptionalTime, true /*comp previous*/);
		}
		return true;
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::BakeControlRigSpace(ULevelSequence* InSequence, UControlRig* InControlRig, const TArray<FName>& InControlNames, FRigSpacePickerBakeSettings InSettings, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;

	if (WeakSequencer.IsValid() && InControlRig && InControlNames.Num() > 0)
	{
		TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
		const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		const FFrameRate& FrameRate = Sequencer->GetFocusedDisplayRate();
		FFrameNumber FrameRateInFrameNumber = TickResolution.AsFrameNumber(FrameRate.AsInterval());
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InSettings.Settings.StartFrame = FFrameRate::TransformTime(FFrameTime(InSettings.Settings.StartFrame, 0), FrameRate, TickResolution).RoundToFrame();
			InSettings.Settings.EndFrame = FFrameRate::TransformTime(FFrameTime(InSettings.Settings.EndFrame, 0), FrameRate, TickResolution).RoundToFrame();
		}

		FScopedTransaction Transaction(LOCTEXT("BakeControlToSpace", "Bake Control In Space"));
		for (const FName& ControlName : InControlNames)
		{
			if (FRigControlElement* Element = InControlRig->FindControl(ControlName))
			{
				FSpaceChannelAndSection SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(InControlRig, ControlName, Sequencer.Get(), false /*bCreateIfNeeded*/);
				if (SpaceChannelAndSection.SpaceChannel)
				{
					FControlRigSpaceChannelHelpers::SequencerBakeControlInSpace(InControlRig, Sequencer.Get(), SpaceChannelAndSection.SpaceChannel, SpaceChannelAndSection.SectionToKey,
						InControlRig->GetHierarchy(), Element->GetKey(), InSettings);
				}
			}
		}
		bValid = true;
	}
	return bValid;
}

bool UControlRigSequencerEditorLibrary::DeleteControlRigSpace(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;

	if (WeakSequencer.IsValid() && ControlRig && ControlName != NAME_None)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
			FSpaceChannelAndSection SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, ControlName, Sequencer.Get(), false /*bCreateIfNeeded*/);
			if (SpaceChannelAndSection.SpaceChannel)
			{

				if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
				{
					InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), LevelSequence->GetMovieScene()->GetDisplayRate(), LevelSequence->GetMovieScene()->GetTickResolution()).RoundToFrame();
				}
				UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(SpaceChannelAndSection.SectionToKey);


				TArray<FFrameNumber> OurKeyTimes;
				TArray<FKeyHandle> OurKeyHandles;
				TRange<FFrameNumber> CurrentFrameRange;
				CurrentFrameRange.SetLowerBound(TRangeBound<FFrameNumber>(InTime));
				CurrentFrameRange.SetUpperBound(TRangeBound<FFrameNumber>(InTime));
				TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelInterface = SpaceChannelAndSection.SpaceChannel->GetData();
				ChannelInterface.GetKeys(CurrentFrameRange, &OurKeyTimes, &OurKeyHandles);
				if (OurKeyHandles.Num() > 0)
				{
					FScopedTransaction DeleteKeysTransaction(LOCTEXT("DeleteSpaceChannelKeys_Transaction", "Delete Space Channel Keys"));
					ParamSection->TryModify();
					SpaceChannelAndSection.SpaceChannel->DeleteKeys(OurKeyHandles);
					WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					bValid = true;
				}
				else
				{
					UE_LOGF(LogControlRig, Error, "No Keys To Delete At that Time");
					return false;
				}
			}
			else
			{
				UE_LOGF(LogControlRig, Error, "Can not find Space Channel");
				return false;
			}
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Can not find Control with that Name");
			return false;
		}
	}
	return bValid;
}

bool UControlRigSequencerEditorLibrary::MoveControlRigSpace(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber InTime, FFrameNumber InNewTime, EMovieSceneTimeUnit TimeUnit)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	bool bValid = false;

	if (WeakSequencer.IsValid() && ControlRig && ControlName != NAME_None)
	{
		if (FRigControlElement* Element = ControlRig->FindControl(ControlName))
		{
			TSharedPtr<ISequencer>  Sequencer = WeakSequencer.Pin();
			FSpaceChannelAndSection SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, ControlName, Sequencer.Get(), false /*bCreateIfNeeded*/);
			if (SpaceChannelAndSection.SpaceChannel)
			{

				if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
				{
					InTime = FFrameRate::TransformTime(FFrameTime(InTime, 0), LevelSequence->GetMovieScene()->GetDisplayRate(), LevelSequence->GetMovieScene()->GetTickResolution()).RoundToFrame();
					InNewTime = FFrameRate::TransformTime(FFrameTime(InNewTime, 0), LevelSequence->GetMovieScene()->GetDisplayRate(), LevelSequence->GetMovieScene()->GetTickResolution()).RoundToFrame();
				}
				UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(SpaceChannelAndSection.SectionToKey);

				TArray<FFrameNumber> OurKeyTimes;
				TArray<FKeyHandle> OurKeyHandles;
				TRange<FFrameNumber> CurrentFrameRange;
				CurrentFrameRange.SetLowerBound(TRangeBound<FFrameNumber>(InTime));
				CurrentFrameRange.SetUpperBound(TRangeBound<FFrameNumber>(InTime));
				TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelInterface = SpaceChannelAndSection.SpaceChannel->GetData();
				ChannelInterface.GetKeys(CurrentFrameRange, &OurKeyTimes, &OurKeyHandles);
				if (OurKeyHandles.Num() > 0)
				{
					FScopedTransaction MoveKeys(LOCTEXT("MoveSpaceChannelKeys_Transaction", "Move Space Channel Keys"));
					ParamSection->TryModify();
					TArray<FFrameNumber> NewKeyTimes;
					NewKeyTimes.Add(InNewTime);
					SpaceChannelAndSection.SpaceChannel->SetKeyTimes(OurKeyHandles, NewKeyTimes);
					WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
					bValid = true;
				}
				else
				{
					UE_LOGF(LogControlRig, Error, "No Key To Move At that Time");
					return false;
				}
			}
			else
			{
				UE_LOGF(LogControlRig, Error, "Can not find Space Channel");
				return false;
			}
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Can not find Control with that Name");
			return false;
		}
	}
	return bValid;
}
bool UControlRigSequencerEditorLibrary::ReplaceControlRig(UMovieSceneTrack* InTrack, UControlRig* InControlRig)
{
	if (InControlRig == nullptr || InTrack == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "ControlRig and Control Rig Track must be valid");
		return false;
	}
	if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(InTrack))
	{
		if (CRTrack->GetControlRig() != InControlRig)
		{
			CRTrack->Modify();
			CRTrack->ReplaceControlRig(InControlRig, true /*RecreateChannels*/);
		}
		else
		{
			UE_LOGF(LogControlRig, Error, "Control Rig already exists on the track");
			return false;
		}
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Track must be a Control Rig Track");
		return false;
	}
	return true;
}

bool UControlRigSequencerEditorLibrary::RenameControlRigControlChannels(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FName>& InOldControlNames, const TArray<FName>& InNewControlNames)
{
	if (LevelSequence == nullptr || ControlRig == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "LevelSequence and Control Rig must be valid");
		return false;
	}
	if (InOldControlNames.Num() != InNewControlNames.Num() || InOldControlNames.Num() < 1)
	{
		UE_LOGF(LogControlRig, Error, "Old and New Control Name arrays don't match in length");
		return false;
	}
	for (const FName& NewName : InNewControlNames)
	{
		if (ControlRig->FindControl(NewName) == nullptr)
		{
			const FString StringName = NewName.ToString();
			UE_LOGF(LogControlRig, Error, "Missing Control Name %ls", *(StringName));
			return false;
		}
	}
	bool bValid = false;


	const UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	if (MovieScene)
	{
		const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
		for (const FMovieSceneBinding& Binding : Bindings)
		{
			TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
			for (UMovieSceneTrack* AnyOleTrack : Tracks)
			{
				UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack);
				if (Track && Track->GetControlRig() == ControlRig)
				{
					Track->Modify();
					bValid = true;
					for (int32 Index = 0; Index < InOldControlNames.Num(); ++Index)
					{
						Track->RenameParameterName(InOldControlNames[Index], InNewControlNames[Index]);
					}
				}
			}
		}
	}
	
	return bValid;
}

FRigElementKey UControlRigSequencerEditorLibrary::GetDefaultParentKey()
{
	return URigHierarchy::GetDefaultParentKey();
}

FRigElementKey UControlRigSequencerEditorLibrary::GetWorldSpaceReferenceKey()
{
	return URigHierarchy::GetWorldSpaceReferenceKey();
}

void UControlRigSequencerEditorLibrary::SetControlRigPriorityOrder(UMovieSceneTrack* InTrack, int32 PriorityOrder)
{
	UMovieSceneControlRigParameterTrack* ParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(InTrack);
	if (!ParameterTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetControlRigPriorityOrder without a UMovieSceneControlRigParameterTrack"), ELogVerbosity::Error);
		return;
	}

	ParameterTrack->SetPriorityOrder(PriorityOrder);
}

int32 UControlRigSequencerEditorLibrary::GetControlRigPriorityOrder(UMovieSceneTrack* InTrack)
{
	UMovieSceneControlRigParameterTrack* ParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(InTrack);
	if (!ParameterTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetControlRigPriorityOrder without a UMovieSceneControlRigParameterTrack"), ELogVerbosity::Error);
		return INDEX_NONE;
	}
	return 	ParameterTrack->GetPriorityOrder();

}

bool UControlRigSequencerEditorLibrary::GetControlsMask(UMovieSceneSection* InSection, FName ControlName)
{
	UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!ParameterSection)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetControlsMask without a UMovieSceneControlRigParameterSection"), ELogVerbosity::Error);
		return false;
	}
	
	UControlRig* ControlRig = Cast<UControlRig>(ParameterSection->GetControlRig());
	if (!ControlRig)
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have a control rig"), ELogVerbosity::Error);
		return false;
	}
	return ParameterSection->GetControlNameMask(ControlName);
}

void UControlRigSequencerEditorLibrary::SetControlsMask(UMovieSceneSection* InSection, const TArray<FName>& ControlNames, bool bVisible)
{
	UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!ParameterSection)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetControlsMask without a UMovieSceneControlRigParameterSection"), ELogVerbosity::Error);
		return;
	}

	UControlRig* ControlRig = Cast<UControlRig>(ParameterSection->GetControlRig());
	if (!ControlRig)
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have a control rig"), ELogVerbosity::Error);
		return;
	}

	ParameterSection->Modify();
	constexpr bool bReconstructChannelProxy = false;
	for (const FName& ControlName : ControlNames)
	{
		ParameterSection->SetControlNameMask(ControlName, bVisible, bReconstructChannelProxy);
	}
	ParameterSection->ReconstructChannelProxy();

	
}

void UControlRigSequencerEditorLibrary::ClearSelection()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->ClearSelection();
	}
}

void UControlRigSequencerEditorLibrary::FrameSelection()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->FrameSelection();
	}
}

void UControlRigSequencerEditorLibrary::SelectMirroredControls()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->SelectMirroredControls();
	}
}

void UControlRigSequencerEditorLibrary::AddMirroredControlsToSelection()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->AddMirroredControlsToSelection();
	}
}

void UControlRigSequencerEditorLibrary::MirrorSelectedControls()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->MirrorSelectedControls();
	}
}

void UControlRigSequencerEditorLibrary::ZeroTransforms(bool bSelectionOnly, bool bIncludeChannels)
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->ZeroTransforms(bSelectionOnly, bIncludeChannels);
	}
}

void UControlRigSequencerEditorLibrary::InvertInputPose(bool bSelectionOnly, bool bIncludeChannels)
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->InvertInputPose(bSelectionOnly, bIncludeChannels);
	}
}

void UControlRigSequencerEditorLibrary::MirrorUnselectedControls()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->MirrorUnselectedControls();
	}
}

void UControlRigSequencerEditorLibrary::OpenPoseLibraryTab()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->OpenPoseLibraryTab();
	}
}

void UControlRigSequencerEditorLibrary::ClosePoseLibraryTab()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->ClosePoseLibraryTab();
	}
}

void UControlRigSequencerEditorLibrary::KeyControls(UMovieSceneSection* InSection, const TArray<FName>& ControlNames)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (Sequencer.IsValid() == false)
	{
		FFrame::KismetExecutionMessage(TEXT("Need open and valid Sequencer"), ELogVerbosity::Error);
		return;
	}
	FFrameNumber FrameNumber = Sequencer->GetLocalTime().Time.GetFrame();
	TArray<FFrameNumber> Frames;
	Frames.Add(FrameNumber);
	UControlRigSequencerEditorLibrary::KeyControlsAtFrames(InSection, ControlNames, Frames, EMovieSceneTimeUnit::TickResolution);
}

void UControlRigSequencerEditorLibrary::KeyControlsAtFrames(UMovieSceneSection* InSection, const TArray<FName>& ControlNames, const TArray<FFrameNumber>& InTimes, EMovieSceneTimeUnit TimeUnit)
{
	using namespace UE::AIE;
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!ParameterSection)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call KeyControls without a UMovieSceneControlRigParameterSection"), ELogVerbosity::Error);
		return;
	}

	UControlRig* ControlRig = Cast<UControlRig>(ParameterSection->GetControlRig());
	if (!ControlRig)
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have a control rig"), ELogVerbosity::Error);
		return;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencerFromAsset();
	if (Sequencer.IsValid() == false)
	{
		FFrame::KismetExecutionMessage(TEXT("Need open and valid Sequencer"), ELogVerbosity::Error);
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("KeyControlFrames_Transaction", "Key Controls"));

	ParameterSection->Modify();
	EMovieSceneKeyInterpolation DefaultInterpolation = Sequencer->GetKeyInterpolation();
	FMovieSceneChannelProxy& ChannelProxy = InSection->GetChannelProxy();

	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	FFrameRate DisplayRate = Sequencer->GetFocusedDisplayRate();

	for (FFrameNumber FrameNumber : InTimes)
	{
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			FrameNumber = FFrameRate::TransformTime(FFrameTime(FrameNumber, 0), DisplayRate, TickResolution).RoundToFrame();
		}

		for (const FName& ControlName : ControlNames)
		{
			if (FRigControlElement* Control = ControlRig->FindControl(ControlName))
			{
				int32 StartIndex = 0, EndIndex = 0;
				if (FControlRigKeys::GetStartEndIndicesForControl(ParameterSection, Control, StartIndex, EndIndex))
				{
					FControlRigInteractionScope InteractionScope(ControlRig, Control->GetKey());

					switch (Control->Settings.ControlType)
					{
					case ERigControlType::Bool:
					{
						if (const FMovieSceneChannelEntry* ChannelEntry = ChannelProxy.FindEntry(FMovieSceneBoolChannel::StaticStruct()->GetFName()))
						{
							TArrayView<FMovieSceneBoolChannel*> Channels = ParameterSection->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
							SetCurrentKeys<FMovieSceneBoolChannel, bool>(Channels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
						}
						break;
					}
					case ERigControlType::Integer:
					{
						if (Control->Settings.ControlEnum)
						{
							if (const FMovieSceneChannelEntry* ChannelEntry = ChannelProxy.FindEntry(FMovieSceneByteChannel::StaticStruct()->GetFName()))
							{
								TArrayView<FMovieSceneByteChannel*> Channels = ParameterSection->GetChannelProxy().GetChannels<FMovieSceneByteChannel>();
								SetCurrentKeys<FMovieSceneByteChannel, uint8>(Channels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
							}
						}
						else
						{
							if (const FMovieSceneChannelEntry* ChannelEntry = ChannelProxy.FindEntry(FMovieSceneIntegerChannel::StaticStruct()->GetFName()))
							{
								TArrayView<FMovieSceneIntegerChannel*> Channels = ParameterSection->GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
								SetCurrentKeys<FMovieSceneIntegerChannel, int32>(Channels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
							}

						}
						break;
					}
					case ERigControlType::Float:
					case ERigControlType::ScaleFloat:
					case ERigControlType::Vector2D:
					case ERigControlType::Position:
					case ERigControlType::Scale:
					case ERigControlType::Rotator:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					default:
					{
						if (const FMovieSceneChannelEntry* ChannelEntry = ChannelProxy.FindEntry(FMovieSceneFloatChannel::StaticStruct()->GetFName()))
						{
							TArrayView<FMovieSceneFloatChannel*> Channels = ParameterSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
							SetCurrentKeys<FMovieSceneFloatChannel, float>(Channels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
						}
						break;
					}
					}
				}
			}
		}
	}
	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

void UControlRigSequencerEditorLibrary::ShowAllControls(UMovieSceneSection* InSection)
{
	UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!ParameterSection)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call ShowAllControls without a UMovieSceneControlRigParameterSection"), ELogVerbosity::Error);
		return;
	}

	ParameterSection->Modify();
	ParameterSection->FillControlNameMask(true);
}

void UControlRigSequencerEditorLibrary::HideAllControls(UMovieSceneSection* InSection)
{
	UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!ParameterSection)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call HideAllControls without a UMovieSceneControlRigParameterSection"), ELogVerbosity::Error);
		return;
	}

	ParameterSection->Modify();
	ParameterSection->FillControlNameMask(false);
}

bool UControlRigSequencerEditorLibrary::IsFKControlRig(UControlRig* InControlRig)
{
	return (InControlRig && InControlRig->IsA<UFKControlRig>());
}

bool UControlRigSequencerEditorLibrary::IsLayeredControlRig(UControlRig* InControlRig)
{
	return (InControlRig && InControlRig->IsAdditive());
}

bool UControlRigSequencerEditorLibrary::MarkLayeredModeOnTrackDisplay(UMovieSceneControlRigParameterTrack* InTrack)
{
	if (!InTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid track"), ELogVerbosity::Error);
		return false;
	}

	UControlRig* ControlRig = InTrack->GetControlRig();
	if (!ControlRig)
	{
		FFrame::KismetExecutionMessage(TEXT("Track does not have a control rig"), ELogVerbosity::Error);
		return false;
	}

	if (ControlRig->IsAdditive())
	{
		InTrack->SetColorTint(UMovieSceneControlRigParameterTrack::LayeredRigTrackColor);
	}
	else
	{
		InTrack->SetColorTint(UMovieSceneControlRigParameterTrack::AbsoluteRigTrackColor);
	}

	return true;
}

bool UControlRigSequencerEditorLibrary::SetControlRigLayeredMode(UMovieSceneControlRigParameterTrack* InTrack, bool bSetIsLayered)
{
	if (!InTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid track"), ELogVerbosity::Error);
		return false;
	}

	UControlRig* ControlRig = InTrack->GetControlRig();
	if (!ControlRig)
	{
		FFrame::KismetExecutionMessage(TEXT("Track does not have a control rig"), ELogVerbosity::Error);
		return false;
	}

	if (ControlRig->IsAdditive() == bSetIsLayered)
	{
		if(bSetIsLayered)
		{
			FFrame::KismetExecutionMessage(TEXT("Control rig is already in layered mode"), ELogVerbosity::Error);
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Control rig is already in absolute mode"), ELogVerbosity::Error);
		}
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("ConvertToLayeredControlRig_Transaction", "Convert to Layered Control Rig"));
	InTrack->Modify();
	ControlRig->Modify();

	if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
	{
		FKRig->SetApplyMode(bSetIsLayered ? EControlRigFKRigExecuteMode::Additive : EControlRigFKRigExecuteMode::Replace);
	}
	else
	{
		ControlRig->ClearPoseBeforeBackwardsSolve();
		ControlRig->ResetControlValues();
		ControlRig->SetIsAdditive(bSetIsLayered);

		ControlRig->Evaluate_AnyThread();
	}

	FString ObjectName = ControlRig->GetClass()->GetName(); //GetDisplayNameText().ToString();
	ObjectName.RemoveFromEnd(TEXT("_C"));
	InTrack->SetTrackName(FName(*ObjectName));
	
	MarkLayeredModeOnTrackDisplay(InTrack);

	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		ControlRigEditMode->ZeroTransforms(false, true, ControlRig);
	}

	for (UMovieSceneSection* Section : InTrack->GetAllSections())
	{
		if (Section)
		{
			UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section);
			if (CRSection)
			{
				Section->Modify();
				CRSection->ClearAllParameters();
				CRSection->RecreateWithThisControlRig(CRSection->GetControlRig(), true);
			}
		}
	}

	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid())
	{
		WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}

	return true;
}

EControlRigFKRigExecuteMode UControlRigSequencerEditorLibrary::GetFKControlRigApplyMode(UControlRig* InControlRig)
{
	EControlRigFKRigExecuteMode ApplyMode = EControlRigFKRigExecuteMode::Direct;
	if (UFKControlRig* FKRig = Cast<UFKControlRig>(Cast<UControlRig>(InControlRig)))
	{
		ApplyMode = FKRig->GetApplyMode();
	}
	return ApplyMode;
}

bool UControlRigSequencerEditorLibrary::SetControlRigApplyMode(UControlRig* InControlRig, EControlRigFKRigExecuteMode InApplyMode)
{
	if (UFKControlRig* FKRig = Cast<UFKControlRig>(Cast<UControlRig>(InControlRig)))
	{
		FKRig->SetApplyMode(InApplyMode);
		return true;
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::DeleteAnimLayer(int32 Index)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "DeleteAnimLayer: Need open Sequencer");
		return false;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		if (Index != INDEX_NONE)
		{
			return AnimLayers->DeleteAnimLayer(SequencerPtr.Get(), Index);
		}
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "DeleteAnimLayer: No Anim Layers on Level Sequence");
	}
	return false;
}

int32 UControlRigSequencerEditorLibrary::DuplicateAnimLayer(int32 Index)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "DuplicateAnimLayer: Need open Sequencer");
		return INDEX_NONE;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		if (Index != INDEX_NONE)
		{
			return AnimLayers->DuplicateAnimLayer(SequencerPtr.Get(), Index);
		}
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "DuplicateAnimLayer: No Anim Layers on Level Sequence");
	}
	return INDEX_NONE;
}

int32 UControlRigSequencerEditorLibrary::AddAnimLayerFromSelection()
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "AddAnimLayerFromSelection: Need open Sequencer");
		return INDEX_NONE;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	constexpr bool bAddIfDoesNotExist = true;
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get(), bAddIfDoesNotExist))
	{
		return AnimLayers->AddAnimLayerFromSelection(SequencerPtr.Get());
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "AddAnimLayerFromSelection: No Anim Layers on Level Sequence");
	}
	return INDEX_NONE;

}

bool UControlRigSequencerEditorLibrary::ReorderAnimLayers(int32 OldIndex, int32 NewIndex)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "ReorderAnimLayers: Need open Sequencer");
		return false;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		return AnimLayers->ReorderAnimLayers(SequencerPtr.Get(), OldIndex, NewIndex);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "ReorderAnimLayers: No Anim Layers on Level Sequence");
	}
	return false;
}

//remove after 5.7
bool UControlRigSequencerEditorLibrary::MergeAnimLayers(const TArray<int32>& Indices)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "MergeAnimLayers: Need open Sequencer");
		return false;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		return AnimLayers->MergeAnimLayers(SequencerPtr, Indices, nullptr);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "MergeAnimLayers: No Anim Layers on Level Sequence");
	}
	return false;
}

bool UControlRigSequencerEditorLibrary::MergeAnimLayersWithSettings(const TArray<int32>& Indices, const FMergeAnimLayerSettings& MergeAnimLayerSettings)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "MergeAnimLayers: Need open Sequencer");
		return false;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		return AnimLayers->MergeAnimLayers(SequencerPtr, Indices, &MergeAnimLayerSettings);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "MergeAnimLayers: No Anim Layers on Level Sequence");
	}
	return false;
}

TArray<UAnimLayer*> UControlRigSequencerEditorLibrary::GetAnimLayers(bool bAddIfDoesNotExist)
{
	TArray<UAnimLayer*> AnimLayersArray;
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "GetAnimLayers: Need open Sequencer");
		return AnimLayersArray;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get(), bAddIfDoesNotExist))
	{
		return AnimLayers->AnimLayers;
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "GetAnimLayers: No Anim Layers on Level Sequence");
	}
	return AnimLayersArray;
}

int32 UControlRigSequencerEditorLibrary::GetAnimLayerIndex(UAnimLayer* AnimLayer)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Error, "GetAnimLayerIndex: Need open Sequencer");
		return INDEX_NONE;
	}
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		return AnimLayers->GetAnimLayerIndex(AnimLayer);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "GetAnimLayerIndex: No Anim Layers on Level Sequence");
	}
	return INDEX_NONE;
}


UAIESelectionSets* UControlRigSequencerEditorLibrary::GetSelectionSets(bool bAddIfDoesNotExist)
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, bAddIfDoesNotExist))
		{
			return SelectionSets;
		}
	}
	return nullptr;
}

bool UControlRigSequencerEditorLibrary::RecreateControlRigWithNewSettings(UMovieSceneControlRigParameterTrack* InTrack, const TMap<FName, FString>& InSettingsToOverride)
{
	if (!InTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid track"), ELogVerbosity::Error);
		return false;
	}

	UControlRig* ControlRig = InTrack->GetControlRig();
	if (!ControlRig)
	{
		FFrame::KismetExecutionMessage(TEXT("Track does not have a control rig"), ELogVerbosity::Error);
		return false;
	}

	FInstancedPropertyBag NewSettings = InTrack->BuildNewRigSettings(InSettingsToOverride);
	if (!NewSettings.IsValid())
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid Settings"), ELogVerbosity::Error);
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("RecreateControlRigWithNewSettings_Transaction", "Recreated Control Rig with New Settings"));
	
	InTrack->UpdateControlRigSettingsOverrides(NewSettings);
	InTrack->RecreateControlRig();

	TWeakPtr<ISequencer> WeakSequencer = GetSequencerFromAsset();
	if (WeakSequencer.IsValid())
	{
		WeakSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
	
	return true;
}

#undef LOCTEXT_NAMESPACE

