// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigShapeNameList.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "ControlRig.h"
#include "ModularRig.h"
#include "ControlRigEditorAsset.h"

#define LOCTEXT_NAMESPACE "ControlRigShapeNameList"

namespace UE::ControlRigEditor
{

void FControlRigShapeNameList::CreateShapeLibraryListWidget(IDetailChildrenBuilder& InStructBuilder, TSharedPtr<IPropertyHandle>& InShapeSettingsNameProperty)
{
	if (!InShapeSettingsNameProperty.IsValid())
	{
		return;
	}

	ShapeSettingsNameProperty = InShapeSettingsNameProperty;

	TSharedPtr<FRigVMStringWithTag> InitialSelected;
	const FString CurrentShapeName = GetShapeNameListText().ToString();
	for (TSharedPtr<FRigVMStringWithTag> Item : ShapeNameList)
	{
		if (Item->Equals(CurrentShapeName))
		{
			InitialSelected = Item;
		}
	}

	IDetailPropertyRow& Row = InStructBuilder.AddProperty(ShapeSettingsNameProperty.ToSharedRef());

	constexpr bool bShowChildren = true;
	Row.CustomWidget(bShowChildren)
		.NameContent()
		[
			ShapeSettingsNameProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SAssignNew(ShapeNameListWidget, SRigVMGraphPinNameListValueWidget)
			.OptionsSource(&ShapeNameList)
			.OnGenerateWidget(this, &FControlRigShapeNameList::MakeShapeNameListItemWidget)
			.OnSelectionChanged(this, &FControlRigShapeNameList::OnShapeNameListChanged)
			.OnComboBoxOpening(this, &FControlRigShapeNameList::OnShapeNameListComboBox)
			.InitiallySelectedItem(InitialSelected)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FControlRigShapeNameList::GetShapeNameListText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void FControlRigShapeNameList::GenerateShapeLibraryList(const UControlRig* ControlRig)
{
	ShapeNameList.Reset();

	if (ControlRig == nullptr)
	{
		return;
	}

	const TMap<FString, FString>& LibraryNameMap = ControlRig->GetShapeLibraryNameMap();
	TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries = ControlRig->GetShapeLibraries();

	// If this is within a modular rig, also add the modular rig's shape libraries
	if (const UModularRig* ModularRig = ControlRig->GetTypedOuter<UModularRig>())
	{
		for (const TSoftObjectPtr<UControlRigShapeLibrary>& ModularRigShapeLibrary : ModularRig->GetShapeLibraries())
		{
			ShapeLibraries.AddUnique(ModularRigShapeLibrary);
		}
	}

	const bool bUseNameSpace = ShapeLibraries.Num() > 1;
	for (const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrarySoftPtr : ShapeLibraries)
	{
		if (ShapeLibrarySoftPtr.IsNull() || !ShapeLibrarySoftPtr.IsValid())
		{
			(void)ShapeLibrarySoftPtr.LoadSynchronous();
		}
		
		UControlRigShapeLibrary* ShapeLibrary = ShapeLibrarySoftPtr.Get();
		if (!ShapeLibrary)
		{
			continue;
		}
		
		auto AddShapeDefinition = [&ShapeNameListRef = ShapeNameList, ShapeLibrary, bUseNameSpace, &LibraryNameMap](const FControlRigShapeDefinition& ShapeDefinition)
		{
			if (!ShapeDefinition.ShapeProxy.IsNull())
			{
				const FString ShapeName = UControlRigShapeLibrary::GetShapeName(ShapeLibrary, bUseNameSpace, LibraryNameMap, ShapeDefinition);
				ShapeNameListRef.Add(MakeShared<FRigVMStringWithTag>(ShapeName));
			}
		};
		
		AddShapeDefinition(ShapeLibrary->DefaultShape);
		for (const FControlRigShapeDefinition& Shape : ShapeLibrary->Shapes)
		{
			AddShapeDefinition(Shape);
		}
	}
}

TSharedRef<SWidget> FControlRigShapeNameList::MakeShapeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem)
{
	const FText ItemText = InItem.IsValid() ? FText::FromString(InItem->GetStringWithTag()) : FText();

	return 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(2.f, 1.f)
		[
			SNew(STextBlock)
			.Text(ItemText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		];
}
	
void FControlRigShapeNameList::OnShapeNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	const FString& NewShapeNameString = NewSelection.IsValid() ? NewSelection->GetString() : FString();
	if(ShapeSettingsNameProperty)
	{
		const FName ShapeName = NewShapeNameString.IsEmpty() ? FName(NAME_None) : FName(*NewShapeNameString);
		ShapeSettingsNameProperty->SetValue(ShapeName);
	}
}
	
void FControlRigShapeNameList::OnShapeNameListComboBox()
{
	const FString ShapeNameListText = GetShapeNameListText().ToString();
	const TSharedPtr<FRigVMStringWithTag>* CurrentlySelectedItem =
		ShapeNameList.FindByPredicate([ShapeNameListText](const TSharedPtr<FRigVMStringWithTag>& InItem)
		{
			return ShapeNameListText == InItem->GetString();
		});
		
	if(CurrentlySelectedItem)
	{
		ShapeNameListWidget->SetSelectedItem(*CurrentlySelectedItem);
	}
}
	
FText FControlRigShapeNameList::GetShapeNameListText() const
{
	if(!ShapeSettingsNameProperty)
	{
		return FText();
	}
		
	TOptional<FString> SharedValue;
	for(int32 Index = 0; Index < ShapeSettingsNameProperty->GetNumPerObjectValues(); Index++)
	{
		FString SingleValue;
		if(!ShapeSettingsNameProperty->GetPerObjectValue(Index, SingleValue))
		{
			SharedValue.Reset();
			break;
		}
		if(!SharedValue.IsSet())
		{
			SharedValue = SingleValue;
		}
		else if(SharedValue.GetValue() != SingleValue)
		{
			SharedValue.Reset();
			break;
		}
	}
		
	if(SharedValue.IsSet())
	{
		return FText::FromString(SharedValue.GetValue());
	}
	return LOCTEXT("MultipleValues", "Multiple Values");
}

} // end namespace UE::ControlRigEditor

#undef LOCTEXT_NAMESPACE
