// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SRigVMGraphPinNameList.h"

class IPropertyHandle;
class UControlRig;

namespace UE::ControlRigEditor
{

/*
* Helper to create a ShapeLibraryListWidget for a ControlRig details
*/
struct FControlRigShapeNameList : public TSharedFromThis<FControlRigShapeNameList>
{
	void GenerateShapeLibraryList(const UControlRig* ControlRig);
	TSharedRef<SWidget> MakeShapeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem);

	void CreateShapeLibraryListWidget(IDetailChildrenBuilder& InStructBuilder, TSharedPtr<IPropertyHandle>& ShapeSettingsNameProperty);
	void OnShapeNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo);
	void OnShapeNameListComboBox();
	FText GetShapeNameListText() const;

	TSharedPtr<IPropertyHandle> ShapeSettingsNameProperty;
	TArray<TSharedPtr<FRigVMStringWithTag>> ShapeNameList;
	TSharedPtr<SRigVMGraphPinNameListValueWidget> ShapeNameListWidget;
};

} // end namespace UE::ControlRigEditor
