// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigShapeSettingsDetails.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "IPythonScriptPlugin.h"
#include "RigVMPythonUtils.h"
#include "Editor/ControlRigWrapperObject.h"
#include "Units/Execution/RigUnit_DynamicHierarchy.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "ControlRig.h"
#include "ControlRigEditorAsset.h"
#include "ControlRigShapeNameList.h"

#define LOCTEXT_NAMESPACE "ControlRigCompilerDetails"

namespace UE::ControlRigEditor
{

FControlRigShapeSettingsDetails::FControlRigShapeSettingsDetails()
	: ControlRigShapeNameList(MakeShared<FControlRigShapeNameList>())
{
}

void FControlRigShapeSettingsDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	InHeaderRow.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		];
}

void FControlRigShapeSettingsDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);

	TArray<TWeakObjectPtr<UControlRig>> ControlRigObjects;

	// If one of the objects is either a Control Rig or a Control Rig Wrapper, we can generate the combo box for selection
	// Else, a FName Text Box will be shown (the default property details)
	for (UObject* Object : Objects)
	{
		if (UControlRig* ControlRig = Cast<UControlRig>(Object))	// Needed by Modular Rig module details
		{
			ControlRigObjects.AddUnique(ControlRig);
		}
		else if (UControlRigWrapperObject* ControlRigWrapperObject = Cast<UControlRigWrapperObject>(Object)) // Needed by Control Rig editor node editor details
		{
			if (UControlRig* WrappedControlRig = GetControlRig(ControlRigWrapperObject))
			{
				// This customization works for multiselection of nodes with the same rig
				ControlRigObjects.AddUnique(WrappedControlRig);
			}
		}
	}

	if (InStructPropertyHandle->IsValidHandle())
	{
		uint32 NumChildren = 0;
		if (InStructPropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
		{
			TSharedPtr<IPropertyHandle> ShapeSettingsNameProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRigUnit_HierarchyAddControl_ShapeSettings, Name));
			if (!ShapeSettingsNameProperty.IsValid())
			{
				return;
			}

			for (uint32 Index = 0; Index < NumChildren; ++Index)
			{
				bool bDefaultContent = true;

				TSharedRef<IPropertyHandle> ChildPropertyHandle = InStructPropertyHandle->GetChildHandle(Index).ToSharedRef();
				if (ChildPropertyHandle->IsSamePropertyNode(ShapeSettingsNameProperty))
				{
					const int32 NumUniqueControlRigs = ControlRigObjects.Num();
					if (NumUniqueControlRigs > 0)
					{
						if (NumUniqueControlRigs == 1)
						{
							if (UControlRig* ControlRig = ControlRigObjects[0].Get())
							{
								ControlRigShapeNameList->GenerateShapeLibraryList(ControlRig);
								ControlRigShapeNameList->CreateShapeLibraryListWidget(InStructBuilder, ShapeSettingsNameProperty);
								bDefaultContent = false;
							}
						}
						else
						{
							IDetailPropertyRow& Row = InStructBuilder.AddProperty(ShapeSettingsNameProperty.ToSharedRef());

							constexpr bool bShowChildren = true;
							Row.CustomWidget(bShowChildren)
								.NameContent()
								[
									ShapeSettingsNameProperty->CreatePropertyNameWidget()
								]
								.ValueContent()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("ShapeSettings_MultipleControlRigs", "Multiple Control Rigs in selection"))
										.Font(IDetailLayoutBuilder::GetDetailFont())
								];

							bDefaultContent = false;
						}
					}
				}

				if (bDefaultContent)
				{
					InStructBuilder.AddProperty(ChildPropertyHandle);
				}
			}
		}
	}
}

UControlRig* FControlRigShapeSettingsDetails::GetControlRig(UControlRigWrapperObject* InControlRigWrapperObject)
{
	if (InControlRigWrapperObject)
	{
		// Modular Rig module-instance details path: the wrapper's subject is the live UControlRig
		// directly. Used for BP-independent rigs where variables are stored in a UPropertyBag and
		// wrapped by FRigModuleInstanceDetails::PerModuleWrappers.
		if (UControlRig* ModuleSubjectRig = Cast<UControlRig>(InControlRigWrapperObject->GetSubject()))
		{
			return ModuleSubjectRig;
		}

		// RigVM node-editor path: subject is a URigVMUnitNode, resolve to the debugged rig
		// through the asset interface.
		if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InControlRigWrapperObject->GetSubject()))
		{
			FControlRigAssetInterfacePtr AssetInterface = IControlRigEditorAssetInterface::GetInterfaceOuter(UnitNode);
			if (AssetInterface)
			{
				if (UControlRig* ControlRig = AssetInterface->GetDebuggedControlRig())
				{
					return ControlRig;
				}
			}
		}
	}

	return nullptr;
}

} // end namespace UE::ControlRigEditor

#undef LOCTEXT_NAMESPACE
