// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SRigVMGraphPinNameList.h"

class IPropertyHandle;
class UControlRigWrapperObject;
class UControlRig;

namespace UE::ControlRigEditor
{

struct FControlRigShapeNameList;

class FControlRigShapeSettingsDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FControlRigShapeSettingsDetails);
	}

	FControlRigShapeSettingsDetails();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override;

protected:
	TSharedPtr<FControlRigShapeNameList> ControlRigShapeNameList;

	static UControlRig* GetControlRig(UControlRigWrapperObject* ControlRigWrapperObject);
};

} // end namespace UE::ControlRigEditor
