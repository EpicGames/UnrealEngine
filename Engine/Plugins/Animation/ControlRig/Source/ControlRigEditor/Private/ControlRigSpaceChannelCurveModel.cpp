// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSpaceChannelCurveModel.h"
#include "HAL/PlatformMath.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneSection.h"
#include "MovieScene.h"
#include "CurveDrawInfo.h"
#include "CurveDataAbstraction.h"
#include "CurveEditor.h"
#include "CurveEditorScreenSpace.h"
#include "CurveEditorSnapMetrics.h"
#include "Styling/AppStyle.h"
#include "SequencerChannelTraits.h"
#include "SequencerSectionPainter.h"
#include "ISequencer.h"
#include "ControlRigSpaceChannelEditors.h"
#include "MVVM/Views/STrackAreaView.h"

ECurveEditorViewID FControlRigSpaceChannelCurveModel::ViewID = ECurveEditorViewID::Invalid;

FControlRigSpaceChannelCurveModel::FControlRigSpaceChannelCurveModel(TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel> InChannel, UMovieSceneSection* OwningSection, TWeakPtr<ISequencer> InWeakSequencer)
{
	using namespace UE::Sequencer;
	
	FMovieSceneChannelProxy* NewChannelProxy = &OwningSection->GetChannelProxy();
	ChannelHandle = NewChannelProxy->MakeHandle<FMovieSceneControlRigSpaceChannel>(InChannel.GetChannelIndex());
	if (FMovieSceneChannelProxy* ChannelProxy = InChannel.GetChannelProxy())
	{
		OnDestroyHandle = NewChannelProxy->OnDestroy.AddRaw(this, &FControlRigSpaceChannelCurveModel::FixupCurve);
	}

	WeakSection = OwningSection;
	LastSignature = OwningSection->GetSignature();
	WeakSequencer = InWeakSequencer;
	SupportedViews = ViewID;

	Color = STrackAreaView::BlendDefaultTrackColor(OwningSection->GetTypedOuter<UMovieSceneTrack>()->GetColorTint());
}

bool FControlRigSpaceChannelCurveModel::HasChangedAndResetTest()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		if (Section->GetSignature() != LastSignature)
		{
			LastSignature = Section->GetSignature();
			return true;
		}
		return false;
	}
	return true;
}

UObject* FControlRigSpaceChannelCurveModel::GetOwningObject() const 
{
	return WeakSection.Get();
}

FControlRigSpaceChannelCurveModel::~FControlRigSpaceChannelCurveModel()
{
	if (FMovieSceneChannelProxy* ChannelProxy = ChannelHandle.GetChannelProxy())
	{
		ChannelProxy->OnDestroy.Remove(OnDestroyHandle);
	}
}

void FControlRigSpaceChannelCurveModel::FixupCurve()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		FMovieSceneChannelProxy* NewChannelProxy = &Section->GetChannelProxy();
		ChannelHandle = NewChannelProxy->MakeHandle<FMovieSceneControlRigSpaceChannel>(ChannelHandle.GetChannelIndex());
		OnDestroyHandle = NewChannelProxy->OnDestroy.AddRaw(this, &FControlRigSpaceChannelCurveModel::FixupCurve);
	}
}
TArray<FKeyBarCurveModel::FBarRange> FControlRigSpaceChannelCurveModel::FindRanges()
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection* Section = WeakSection.Get();
	TArray<FKeyBarCurveModel::FBarRange> Range =  FControlRigSpaceChannelHelpers::FindRanges(Channel, Section);
	return Range;
}

const void* FControlRigSpaceChannelCurveModel::GetCurve() const
{
	return ChannelHandle.Get();
}

void FControlRigSpaceChannelCurveModel::Modify()
{
	if (UMovieSceneSection* Section = WeakSection.Get())
	{
		Section->Modify();
	}
}

void FControlRigSpaceChannelCurveModel::AddKeys(TArrayView<const FKeyPosition> InKeyPositions, TArrayView<const FKeyAttributes> InKeyAttributes, TArrayView<TOptional<FKeyHandle>>* OutKeyHandles)
{
	check(InKeyPositions.Num() == InKeyAttributes.Num() && (!OutKeyHandles || OutKeyHandles->Num() == InKeyPositions.Num()));

	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();
	if (Channel && Section)
	{
		Section->Modify();
		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		for (int32 Index = 0; Index < InKeyPositions.Num(); ++Index)
		{
			FKeyPosition   Position = InKeyPositions[Index];
			FKeyAttributes Attributes = InKeyAttributes[Index];

			FFrameNumber Time = (Position.InputValue * TickResolution).RoundToFrame();
			Section->ExpandToFrame(Time);

			FMovieSceneControlRigSpaceBaseKey Value;
			FKeyHandle NewHandle = ChannelData.UpdateOrAddKey(Time, Value);
			if (NewHandle != FKeyHandle::Invalid())
			{
				if (OutKeyHandles)
				{
					(*OutKeyHandles)[Index] = NewHandle;
				}
			}
		}
	}
}

bool FControlRigSpaceChannelCurveModel::Evaluate(double Time, double& OutValue) const
{
	// Spaces don't evaluate into a valid value
	OutValue = 0.0;
	return false;
}

void FControlRigSpaceChannelCurveModel::RemoveKeys(TArrayView<const FKeyHandle> InKeys, double InCurrentTime)
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();
	if (Channel && Section)
	{
		Section->Modify();

		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();

		for (FKeyHandle Handle : InKeys)
		{
			int32 KeyIndex = ChannelData.GetIndex(Handle);
			if (KeyIndex != INDEX_NONE)
			{
				ChannelData.RemoveKey(KeyIndex);
			}
		}
	}
}

void FControlRigSpaceChannelCurveModel::DrawCurve(const FCurveEditor& CurveEditor, const FCurveEditorScreenSpace& ScreenSpace, TArray<TTuple<double, double>>& InterpolatingPoints) const
{
	// Space Channels don't draw any lines so there's no need to fill out the Interpolating Points array.
}

void FControlRigSpaceChannelCurveModel::GetKeys(double MinTime, double MaxTime, double MinValue, double MaxValue, TArray<FKeyHandle>& OutKeyHandles) const
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();
	
	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber>          Times = ChannelData.GetTimes();

		const FFrameNumber StartFrame = MinTime <= MIN_int32 ? MIN_int32 : (MinTime * TickResolution).CeilToFrame();
		const FFrameNumber EndFrame = MaxTime >= MAX_int32 ? MAX_int32 : (MaxTime * TickResolution).FloorToFrame();

		const int32 StartingIndex = Algo::LowerBound(Times, StartFrame);
		const int32 EndingIndex = Algo::UpperBound(Times, EndFrame);

		for (int32 KeyIndex = StartingIndex; KeyIndex < EndingIndex; ++KeyIndex)
		{
			// Space Channels don't have Values associated with them so we ignore the Min/Max value and always return the key.
			OutKeyHandles.Add(ChannelData.GetHandle(KeyIndex));
		}
	}
}

//we actually still draw the 'key' as a the border which you can move and right click, context menu with.
void FControlRigSpaceChannelCurveModel::GetKeyDrawInfo(ECurvePointType PointType, const FKeyHandle InKeyHandle, FKeyDrawInfo& OutDrawInfo) const
{
	OutDrawInfo.Brush = FAppStyle::Get().GetBrush("FilledBorder");
	// NOTE Y is set to SCurveEditorKeyBarView::TrackHeight
	OutDrawInfo.ScreenSize = FVector2D(10, 24);
}

void FControlRigSpaceChannelCurveModel::GetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyPosition> OutKeyPositions) const
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();
		TArrayView<const FFrameNumber>          Times = ChannelData.GetTimes();
		TArrayView<const FMovieSceneControlRigSpaceBaseKey> Values = ChannelData.GetValues();

		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				OutKeyPositions[Index].InputValue = Times[KeyIndex] / TickResolution;
				// Spaces have no values so we just output zero.
				OutKeyPositions[Index].OutputValue = 0.0;
			}
		}
	}
}

void FControlRigSpaceChannelCurveModel::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		Section->Modify();

		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();

		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			int32 KeyIndex = ChannelData.GetIndex(InKeys[Index]);
			if (KeyIndex != INDEX_NONE)
			{
				FFrameNumber NewTime = (InKeyPositions[Index].InputValue * TickResolution).FloorToFrame();

				const bool bRemoveDuplicates = ChangeType == EPropertyChangeType::ValueSet;
				KeyIndex = ChannelData.MoveKey(KeyIndex, NewTime, bRemoveDuplicates);
				Section->ExpandToFrame(NewTime);
			}
		}
	}
}

void FControlRigSpaceChannelCurveModel::GetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<FKeyAttributes> OutAttributes) const
{
}

void FControlRigSpaceChannelCurveModel::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
}

void FControlRigSpaceChannelCurveModel::GetCurveAttributes(FCurveAttributes& OutCurveAttributes) const
{
	// Space Channels have no Pre/Post Extrapolation
	OutCurveAttributes.SetPreExtrapolation(RCCE_None);
	OutCurveAttributes.SetPostExtrapolation(RCCE_None);
}

void FControlRigSpaceChannelCurveModel::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{
}

void FControlRigSpaceChannelCurveModel::CreateKeyProxies(TWeakPtr<FCurveEditor> InWeakCurveEditor, FCurveModelID InCurveModelID, TArrayView<const FKeyHandle> InKeyHandles, TArrayView<UObject*> OutObjects)
{
}

void FControlRigSpaceChannelCurveModel::GetTimeRange(double& MinTime, double& MaxTime) const
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
	UMovieSceneSection*      Section = WeakSection.Get();

	if (Channel && Section)
	{
		TArrayView<const FFrameNumber> Times = Channel->GetData().GetTimes();
		if (Times.Num() == 0)
		{
			MinTime = 0.f;
			MaxTime = 0.f;
		}
		else
		{
			FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
			double ToTime = TickResolution.AsInterval();
			MinTime = static_cast<double> (Times[0].Value) * ToTime;
			MaxTime = static_cast<double>(Times[Times.Num() - 1].Value) * ToTime;
		}
	}
}

void FControlRigSpaceChannelCurveModel::GetValueRange(double& MinValue, double& MaxValue) const
{
	// Space channels have no value, thus their value range is just zero.
	MinValue = MaxValue = 0.0;
}

int32 FControlRigSpaceChannelCurveModel::GetNumKeys() const
{
	FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();

	if (Channel)
	{
		return Channel->GetData().GetTimes().Num();
	}

	return 0;
}

void FControlRigSpaceChannelCurveModel::BuildContextMenu(const FCurveEditor& CurveEditor,FMenuBuilder& MenuBuilder, TOptional<FCurvePointHandle> ClickedPoint)
{
	if (IsReadOnly())
	{
		return;
	}
	if (ClickedPoint.IsSet())
	{
		MenuBuilder.BeginSection("SpaceContextMenuSection", NSLOCTEXT("CurveEditor", "Space", "Space"));
		{
			FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get();
			const FKeyHandle KeyHandle = ClickedPoint.GetValue().KeyHandle;
			if (Channel && KeyHandle != FKeyHandle::Invalid())
			{
				TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();
				int32 KeyIndex = ChannelData.GetIndex(KeyHandle);
				TArrayView<const FMovieSceneControlRigSpaceBaseKey> Values = ChannelData.GetValues();
				if (KeyIndex >= 0 && KeyIndex < Values.Num())
				{
					FControlRigSpaceChannelHelpers::OpenBakeDialog(WeakSequencer.Pin().Get(), Channel, KeyIndex, WeakSection.Get());
				}

			}
		}
		MenuBuilder.EndSection();
	}
}
