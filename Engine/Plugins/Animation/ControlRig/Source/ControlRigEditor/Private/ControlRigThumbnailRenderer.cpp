// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigThumbnailRenderer.h"
#include "Materials/Material.h"
#include "ThumbnailHelpers.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "Animation/SkeletalMeshActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ControlRig.h"
#include "CanvasTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigThumbnailRenderer)

UControlRigThumbnailRenderer::UControlRigThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	RigBlueprint = nullptr;
}

bool UControlRigThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	FControlRigAssetInterfacePtr InRigBlueprint = Object;
	if (!InRigBlueprint)
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = Object)
		{
			InRigBlueprint = RuntimeAsset->GetEditorOnlyData();
		}
	}
	
	if (InRigBlueprint)
	{
		if(InRigBlueprint->GetRigModuleIcon())
		{
			return true;
		}
		
		int32 MissingMeshCount = 0;

		for(const TSoftObjectPtr<UControlRigShapeLibrary>& ShapeLibrary : InRigBlueprint->GetShapeLibraries())
		{
			if (ShapeLibrary.IsValid())
			{
				InRigBlueprint->GetHierarchy()->ForEach<FRigControlElement>([&](FRigControlElement* ControlElement) -> bool
				{
					if (const FControlRigShapeDefinition* ShapeDef = ShapeLibrary->GetShapeByName(ControlElement->Settings.ShapeName))
					{
						constexpr bool bDoNotLoad = false;
						if (!ShapeDef->GetShapeProxy(bDoNotLoad)) // not yet loaded
						{
							MissingMeshCount++;
						}
					}

					return true; // continue the iteration
				});
			}
		}
		return MissingMeshCount == 0;
	}
	return false;
}

void UControlRigThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	RigBlueprint = nullptr;
	
	FControlRigAssetInterfacePtr InRigBlueprint = Object;
	if (!InRigBlueprint)
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = Object)
		{
			InRigBlueprint = RuntimeAsset->GetEditorOnlyData();
		}
	}

	if (InRigBlueprint)
	{
		if(UTexture2D* ModuleIcon = InRigBlueprint->GetRigModuleIcon())
		{
			Canvas->DrawTile(X, Y, Width, Height, 0, 0, 1, 1, FLinearColor::White, ModuleIcon->GetResource(), 1.f);
			return;
		}
		
		UObject* ObjectToDraw = InRigBlueprint->GetPreviewSkeletalMesh().Get();
		if(ObjectToDraw == nullptr)
		{
			ObjectToDraw = InRigBlueprint.GetObject();
		}

		RigBlueprint = InRigBlueprint;
		Super::Draw(ObjectToDraw, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);

		for (auto Pair : ShapeActors)
		{
			if (Pair.Value && Pair.Value->GetOuter())
			{
				Pair.Value->Rename(nullptr, GetTransientPackage());
				Pair.Value->MarkAsGarbage();
			}
		}
		ShapeActors.Reset();
	}
}

void UControlRigThumbnailRenderer::AddAdditionalPreviewSceneContent(UObject* Object, UWorld* PreviewWorld)
{
	TSharedRef<FSkeletalMeshThumbnailScene> ThumbnailScene = ThumbnailSceneCache.EnsureThumbnailScene(Object);
	if (ThumbnailScene->GetPreviewActor() && RigBlueprint && !RigBlueprint->GetShapeLibraries().IsEmpty() && RigBlueprint->GetRuntimeAssetInterface())
	{
		if(RigBlueprint->GetRigModuleIcon())
		{
			return;
		}

		UControlRig* ControlRig = nullptr;

		// reuse the current control rig if possible
		TArray<UObject*> ArchetypeInstances = RigBlueprint->GetRigVMAssetInterface()->GetArchetypeInstances(false, true);
		if (ArchetypeInstances.Num() > 0)
		{
			ControlRig = Cast<UControlRig>(ArchetypeInstances[0]);
		}

		URigHierarchy* Hierarchy = nullptr;
		if (ControlRig == nullptr)
		{
			// fall back to the CDO. we only need to pull out
			// the pose of the default hierarchy so the CDO is fine.
			// this case only happens if the editor had been closed
			// and there are no archetype instances left.
			Hierarchy = RigBlueprint->GetHierarchy();
		}
		else
		{
			Hierarchy = ControlRig->GetHierarchy();
		}

		FTransform ComponentToWorld = ThumbnailScene->GetPreviewActor()->GetSkeletalMeshComponent()->GetComponentToWorld();

		if (Hierarchy)
		{
			Hierarchy->ForEach<FRigControlElement>([&](FRigControlElement* ControlElement) -> bool
			{
				switch (ControlElement->Settings.ControlType)
				{
					case ERigControlType::Float:
					case ERigControlType::ScaleFloat:
					case ERigControlType::Integer:
					case ERigControlType::Vector2D:
					case ERigControlType::Position:
					case ERigControlType::Scale:
					case ERigControlType::Rotator:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						if (const FControlRigShapeDefinition* ShapeDef = RigBlueprint->GetControlShapeByName(ControlElement->Settings.ShapeName))
						{
							constexpr bool bDoNotLoad = false;
							UObject* Proxy = ShapeDef->GetShapeProxy(bDoNotLoad);
							if (Proxy == nullptr) // not yet loaded
							{
								return true;
							}

							AActor* ShapeActor = [Proxy, PreviewWorld]() -> AActor*
							{
								FActorSpawnParameters SpawnInfo;
								SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
								SpawnInfo.bNoFail = true;
								SpawnInfo.ObjectFlags = RF_Transient;

								if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Proxy))
								{
									AStaticMeshActor* StaticMeshActor = PreviewWorld->SpawnActor<AStaticMeshActor>(SpawnInfo);
									StaticMeshActor->GetStaticMeshComponent()->SetStaticMesh(StaticMesh);
									return StaticMeshActor;
								}

								if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Proxy))
								{
									ASkeletalMeshActor* SkeletalMeshActor = PreviewWorld->SpawnActor<ASkeletalMeshActor>(SpawnInfo);
									SkeletalMeshActor->GetSkeletalMeshComponent()->SetSkeletalMesh(SkeletalMesh);
									return SkeletalMeshActor;
								}

								ensure(false);
								return nullptr;
							}();

							if (ShapeActor)
							{
								ShapeActor->SetActorEnableCollision(false);
								
								UControlRigShapeLibrary* ShapeLibrary = ShapeDef->Library.Get();
								if (!ShapeLibrary) // not yet loaded
								{
									return true;
								}

								UMaterial* DefaultMaterial = ShapeLibrary->DefaultMaterial.Get();
								if (!DefaultMaterial) // not yet loaded
								{
									return true;
								}

								UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(DefaultMaterial, ShapeActor);
								MaterialInstance->SetVectorParameterValue(ShapeLibrary->MaterialColorParameter, FVector(ControlElement->Settings.ShapeColor));

								UPrimitiveComponent* Component = [ShapeActor]() -> UPrimitiveComponent*
								{
									if (AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(ShapeActor))
									{
										return StaticMeshActor->GetStaticMeshComponent();
									}

									if (ASkeletalMeshActor* SkeletalMeshActor = Cast<ASkeletalMeshActor>(ShapeActor))
									{
										return SkeletalMeshActor->GetSkeletalMeshComponent();
									}

									ensure(false);
									return nullptr;
								}();

								if (Component)
								{
									for (int32 Index = 0; Index < Component->GetNumMaterials(); ++Index)
									{
										Component->SetMaterial(Index, MaterialInstance);
									}
								}

								const FTransform ShapeGlobalTransform = Hierarchy->GetGlobalControlShapeTransform(ControlElement->GetKey());
								ShapeActor->SetActorTransform(ShapeDef->Transform * ShapeGlobalTransform);
								
								ShapeActors.Add(ControlElement->GetFName(), ShapeActor);
							}
						}
						break;
					}
					default:
					{
						break;
					}
				}
				return true;
			});
		}
	}
}
