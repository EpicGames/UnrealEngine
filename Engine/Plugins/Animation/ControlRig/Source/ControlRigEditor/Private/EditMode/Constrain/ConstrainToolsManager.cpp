// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConstrainToolsManager.h"

#include "EditMode/ControlRigEditModeToolkit.h"
#include "Framework/Docking/TabManager.h"
#include "SConstrainToolsRoot.h"
#include "EditMode/ControlRigEditModeCommands.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FConstrainToolsManager"

namespace UE::ControlRigEditor
{
FConstrainToolsManager::FConstrainToolsManager(
	const TSharedRef<FTabManager>& InToolkitTabManager, 
	const TSharedRef<FWorkspaceItem>& InWorkspaceMenuGroup,
	const TSharedRef<FUICommandList>& InToolkitCommandList,
	FControlRigEditMode& InOwningEditMode,
	const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel
	)
	: TabManager(InToolkitTabManager)
	, WeakCommandList(InToolkitCommandList)
	, OwningEditMode(InOwningEditMode)
	, SelectionViewModel(InSelectionViewModel)
{
	RegisterTabSpawner(TabManager, InWorkspaceMenuGroup);
	BindCommands();
}

FConstrainToolsManager::~FConstrainToolsManager()
{
	HideWidget();
	UnregisterTabSpawner(TabManager);
	UnbindCommands();
}

void FConstrainToolsManager::ToggleVisibility() const
{
	if (IsShowingWidget())
	{
		HideWidget();
	}
	else
	{
		ShowWidget();
	}
}

void FConstrainToolsManager::ShowWidget() const
{
	TabManager->TryInvokeTab(FControlRigEditModeToolkit::ConstrainingTabName);
	
	if (const TSharedPtr<SConstrainToolsRoot> WidgetContentPin = ConstrainWindowContent.Pin())
	{
		EControlRigConstrainTab TabState = UControlRigEditModeSettings::Get()->LastUIStates.ConstraintsTabState.LastOpenInlineTab;
		WidgetContentPin->OpenTab(TabState);
	}
}

void FConstrainToolsManager::HideWidget() const
{
	if (const TSharedPtr<SDockTab> Tab = SpawnedTab.Pin())
	{
		Tab->RequestCloseTab();
	}
}

bool FConstrainToolsManager::IsShowingWidget() const
{
	return SpawnedTab.IsValid();
}

void FConstrainToolsManager::RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FWorkspaceItem>& InWorkspaceMenuGroup)
{
	InTabManager->RegisterTabSpawner(
		FControlRigEditModeToolkit::ConstrainingTabName, FOnSpawnTab::CreateRaw(this, &FConstrainToolsManager::SpawnSnapperTab)
		)
		.SetDisplayName(LOCTEXT("Tab.Title", "Constrain"))
		.SetTooltipText(LOCTEXT("Tab.ToolTip", "Snap child objects to a parent object over a set of frames."))
		.SetIcon(FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.ConstraintTools")))
		.SetGroup(InWorkspaceMenuGroup);
	InTabManager->RegisterDefaultTabWindowSize(FControlRigEditModeToolkit::ConstrainingTabName, FVector2D(380, 500));
}

void FConstrainToolsManager::UnregisterTabSpawner(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(FControlRigEditModeToolkit::ConstrainingTabName);
}

TSharedRef<SDockTab> FConstrainToolsManager::SpawnSnapperTab(const FSpawnTabArgs& Args)
{
	return SAssignNew(SpawnedTab, SDockTab)
	[
		SAssignNew(ConstrainWindowContent, SConstrainToolsRoot, OwningEditMode, SelectionViewModel)
		.OnTabSelected_Lambda([](EControlRigConstrainTab NewTab)
		{
			UControlRigEditModeSettings* Settings = UControlRigEditModeSettings::Get();
			Settings->LastUIStates.ConstraintsTabState.LastOpenInlineTab = NewTab;
			Settings->SaveConfig();
		})
	];
}

void FConstrainToolsManager::BindCommands()
{
	const TSharedPtr<FUICommandList> CommandListPin = WeakCommandList.Pin();
	if (!CommandListPin)
	{
		return;
	}
	
	FControlRigEditModeCommands& Commands = FControlRigEditModeCommands::Get();
	CommandListPin->MapAction(Commands.ToggleConstrainTab, FExecuteAction::CreateRaw(this, &FConstrainToolsManager::ToggleVisibility));
	CommandListPin->MapAction(Commands.OpenSpacesTab, FExecuteAction::CreateRaw(this, &FConstrainToolsManager::HandleOpenTabCommand, EControlRigConstrainTab::Spaces));
	CommandListPin->MapAction(Commands.OpenConstraintsTab, FExecuteAction::CreateRaw(this, &FConstrainToolsManager::HandleOpenTabCommand, EControlRigConstrainTab::Constraints));
	CommandListPin->MapAction(Commands.OpenSnapperTab, FExecuteAction::CreateRaw(this, &FConstrainToolsManager::HandleOpenTabCommand, EControlRigConstrainTab::Snapper));
}

void FConstrainToolsManager::UnbindCommands()
{
	const TSharedPtr<FUICommandList> CommandListPin = WeakCommandList.Pin();
	if (!CommandListPin || !FControlRigEditModeCommands::IsRegistered())
	{
		return;
	}
	
	FControlRigEditModeCommands& Commands = FControlRigEditModeCommands::Get();
	CommandListPin->UnmapAction(Commands.ToggleConstrainTab);
	CommandListPin->UnmapAction(Commands.OpenSpacesTab);
	CommandListPin->UnmapAction(Commands.OpenConstraintsTab);
	CommandListPin->UnmapAction(Commands.OpenSnapperTab);
}

void FConstrainToolsManager::HandleOpenTabCommand(EControlRigConstrainTab InTab)
{
	// This will flash the tab if already visible, which is desirable.
	ShowWidget();

	if (const TSharedPtr<SConstrainToolsRoot> WidgetContentPin = ConstrainWindowContent.Pin())
	{
		WidgetContentPin->OpenTab(InTab);
	}
}
}

#undef LOCTEXT_NAMESPACE