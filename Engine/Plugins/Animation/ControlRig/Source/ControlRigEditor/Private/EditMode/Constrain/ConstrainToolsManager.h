// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FControlRigEditMode;
class FSpawnTabArgs;
class FTabManager;
class FToolBarBuilder;
class FUICommandList;
class FWorkspaceItem;
class SDockTab;
enum class EControlRigConstrainTab : uint8;

namespace UE::ControlRigEditor
{
class FRigSelectionViewModel;
class SConstrainToolsRoot;
	
/**
 * Manages UI that is used for constraining, namely:
 * - SControlRigSnapper
 * - SRigSpacePickerWidget
 * - SConstraintsEditionWidget
 *
 * This class responsibility is spawning & managing a window with tab-like buttons to switch between these widgets. 
 */
class FConstrainToolsManager : public FNoncopyable
{
public:

	explicit FConstrainToolsManager(
		const TSharedRef<FTabManager>& InToolkitTabManager,
		const TSharedRef<FWorkspaceItem>& InWorkspaceMenuGroup,
		const TSharedRef<FUICommandList>& InToolkitCommandList,
		FControlRigEditMode& InOwningEditMode UE_LIFETIMEBOUND,
		const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel
		);
	~FConstrainToolsManager();
	
	void ToggleVisibility() const;
	void ShowWidget() const;
	void HideWidget() const;
	bool IsShowingWidget() const;

private:

	/** Used to show / hide the tab containing our widgets. */
	const TSharedRef<FTabManager> TabManager;
	/** The command list that we are bound to for switching between inline tabs. */
	const TWeakPtr<FUICommandList> WeakCommandList;
	
	/** Needed to construct widget hierarchy.  */
	FControlRigEditMode& OwningEditMode;
	/** Handles selection changing. Needed to construct widget hierarchy. */
	const TSharedRef<FRigSelectionViewModel> SelectionViewModel;

	/** The tab we spawned. Used to detect whether the tab is currently being shown. */
	TWeakPtr<SDockTab> SpawnedTab;
	/** The widget that contains the tabs. Only valid when SpawnedTab is valid. Weak as we don't keep a strong reference to the SDockTab either. */
	TWeakPtr<SConstrainToolsRoot> ConstrainWindowContent;

	// Tab spawning
	void RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedRef<FWorkspaceItem>& InWorkspaceMenuGroup);
	void UnregisterTabSpawner(const TSharedRef<FTabManager>& InTabManager);
	TSharedRef<SDockTab> SpawnSnapperTab(const FSpawnTabArgs& Args);

	// Commands
	void BindCommands();
	void UnbindCommands();
	void HandleOpenTabCommand(EControlRigConstrainTab InTab);
};
}

