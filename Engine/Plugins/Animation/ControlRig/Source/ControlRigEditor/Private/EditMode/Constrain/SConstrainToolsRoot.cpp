// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConstrainToolsRoot.h"

#include "ControlRigEditorStyle.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditMode/SControlRigSnapper.h"
#include "SpacePicker/Views/SRigLevelEditorSpacePicker.h"
#include "EditMode/Settings/ConstraintsTabRestoreState.h"
#include "Editor/Constraints/SConstraintsWidget.h"
#include "Misc/RigEditModeUtils.h"
#include "Misc/Tab/SInlineTabPanel.h"
#include "SConstraintsEditionWidget_ConstrainPanel.h"

#define LOCTEXT_NAMESPACE "SConstrainToolsRoot"

namespace UE::ControlRigEditor
{
void SConstrainToolsRoot::Construct(
	const FArguments& InArgs, FControlRigEditMode& InOwningEditMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel
	)
{
	static_assert(
		static_cast<int32>(EControlRigConstrainTab::Spaces) == 0
		&& static_cast<int32>(EControlRigConstrainTab::Constraints) == 1
		&& static_cast<int32>(EControlRigConstrainTab::Snapper) == 2,
		"Open tab relies on this order. Update the code accordingly."
		);

	OnTabSelectedDelegate = InArgs._OnTabSelected;
	
	ChildSlot
	[
		SAssignNew(InlineTabs, SInlineTabPanel)
		.Tabs({
			FInlineTabArgs()
				.SetLabel(LOCTEXT("Spaces", "Spaces"))
				.SetContent(CreateSpacesContent(InOwningEditMode, InSelectionViewModel))
				.SetOnTabSelected(FSimpleDelegate::CreateSP(this, &SConstrainToolsRoot::BroadcastOnTabSelected, EControlRigConstrainTab::Spaces))
				.SetIcon(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.Constrain.Spaces")),
			
			FInlineTabArgs()
				.SetLabel(LOCTEXT("Constraints", "Constraints"))
				.SetContent(CreateConstraintsContent(InOwningEditMode, InSelectionViewModel))
				.SetOnTabSelected(FSimpleDelegate::CreateSP(this, &SConstrainToolsRoot::OnTabSelected_ConstraintsEditionWidget))
				.SetIcon(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.Constrain.Constraints")),
			
			FInlineTabArgs()
				.SetLabel(LOCTEXT("Snapper", "Snapper"))
				.SetContent(CreateSnapperContent())
				.SetOnTabSelected(FSimpleDelegate::CreateSP(this, &SConstrainToolsRoot::BroadcastOnTabSelected, EControlRigConstrainTab::Snapper))
				.SetIcon(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.Constrain.Snapper")),
		})
	];
}

void SConstrainToolsRoot::OpenTab(EControlRigConstrainTab InTab) const
{
	// The enumeration is declared in display order.
	InlineTabs->SwitchToTab(static_cast<int32>(InTab));
}

TSharedRef<SWidget> SConstrainToolsRoot::CreateSpacesContent(FControlRigEditMode& InOwningEditMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel)
{
	using namespace UE::ControlRigEditor;
	const FInitialSpacePickerSelection InitialSelection = DetermineInitialSpacePickerSelection(InOwningEditMode);

	return SNew(SRigLevelEditorSpacePicker, InOwningEditMode, InSelectionViewModel)
		.InitialSelection(InitialSelection.WeakHierarchyToControlsMap);
}

TSharedRef<SWidget> SConstrainToolsRoot::CreateConstraintsContent(
	FControlRigEditMode& InOwningMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel
	)
{
	return SAssignNew(ConstrainsEditionWidget, SConstraintsEditionWidget_ConstrainPanel, InOwningMode, InSelectionViewModel)
		.InitiallyCollapsed(false);
}

TSharedRef<SWidget> SConstrainToolsRoot::CreateSnapperContent()
{
	return SNew(SControlRigSnapper);
}

void SConstrainToolsRoot::OnTabSelected_ConstraintsEditionWidget() const
{
	ConstrainsEditionWidget->RefreshConstraintList();
	BroadcastOnTabSelected(EControlRigConstrainTab::Constraints);
}
}

#undef LOCTEXT_NAMESPACE