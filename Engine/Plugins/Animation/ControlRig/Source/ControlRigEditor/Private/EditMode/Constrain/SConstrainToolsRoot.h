// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FControlRigEditMode;
enum class EControlRigConstrainTab : uint8;

namespace UE::ControlRigEditor
{
class FRigSelectionViewModel;
class SConstraintsEditionWidget_ConstrainPanel;
class SInlineTabPanel;

DECLARE_DELEGATE_OneParam(FOnTabSelected, EControlRigConstrainTab);
	
/**
 * Coordinator widget that contains
 * - SControlRigSnapper
 * - SRigSpacePickerWidget
 * - SConstraintsEditionWidget
 * and ranges them under a tab-like structure.
 */
class SConstrainToolsRoot : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SConstrainToolsRoot){}
		/** Invoked when one of the named tabs is selected. */
		SLATE_EVENT(FOnTabSelected, OnTabSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FControlRigEditMode& InOwningEditMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel);

	/** Opens the specified tab. */
	void OpenTab(EControlRigConstrainTab InTab) const;
	
private:

	/** Manages the tabs. */
	TSharedPtr<SInlineTabPanel> InlineTabs;
	
	/** The widget for editing constraints. */
	TSharedPtr<SConstraintsEditionWidget_ConstrainPanel> ConstrainsEditionWidget;
	
	/** Invoked when one of the named tabs is selected. */
	FOnTabSelected OnTabSelectedDelegate;

	TSharedRef<SWidget> CreateSpacesContent(FControlRigEditMode& InOwningEditMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel);
	TSharedRef<SWidget> CreateConstraintsContent(FControlRigEditMode& InOwningMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel);
	TSharedRef<SWidget> CreateSnapperContent();

	/** Refreshes the list when we switch to the tab. */
	void OnTabSelected_ConstraintsEditionWidget() const;
	/** Invoked when one of the named tabs is selected. */
	void BroadcastOnTabSelected(EControlRigConstrainTab ControlRigConstrainTab) const { OnTabSelectedDelegate.ExecuteIfBound(ControlRigConstrainTab); }
};
}
