// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConstraintsEditionWidget_ConstrainPanel.h"

#include "EditMode/ControlRigEditMode.h"
#include "EditMode/Models/RigSelectionViewModel.h"
#include "EditMode/SComponentPickerPopup.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SExpandableConstraintsEditionWidget"

namespace UE::ControlRigEditor
{
void SConstraintsEditionWidget_ConstrainPanel::Construct(
	const FArguments& InArgs, FControlRigEditMode& InEditMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel
	)
{
	OwningEditMode = &InEditMode;
	SelectionViewModel = InSelectionViewModel;

	InSelectionViewModel->OnControlSelected().AddSP(this, &SConstraintsEditionWidget_ConstrainPanel::OnControlSelected);

	constexpr float TopAreaHeight = 22;
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Padding(2.f)
			[
				SAssignNew(ConstraintsEditionWidget, SConstraintsEditionWidget)

				// We need to make sure the splitter "row" we're adding is the same height. 
				.LeftSplitterContent()
				[
					SNew(SBox).HeightOverride(TopAreaHeight) [ SNullWidget::NullWidget ]
				]
				.MiddleSplitterContent()
				[
					SNew(SBox).HAlign(HAlign_Left).HeightOverride(TopAreaHeight) [ CreateSelectionComboButton() ]
				]
				.RightSplitterContent()
				[
					SNew(SBox).HeightOverride(TopAreaHeight) [ SNullWidget::NullWidget ]
				]
				
				.LeftButtonArea()
				[
					CreateAddConstraintButton()
				]
			]
		]
	];

	// LevelSequence is the most sensible default. It should be set back to this even if the user had the panel open earlier, closed it, and reopend it.
	ConstraintsEditionWidget->SetShowConstraints(FBaseConstraintListWidget::EShowConstraints::ShowLevelSequence);
}

SConstraintsEditionWidget_ConstrainPanel::~SConstraintsEditionWidget_ConstrainPanel()
{
	SelectionViewModel->OnControlSelected().RemoveAll(this);
}

TSharedRef<SWidget> SConstraintsEditionWidget_ConstrainPanel::CreateSelectionComboButton()
{
	return SNew(SComboButton)
		.OnGetMenuContent_Lambda([this]()
		{
			FMenuBuilder MenuBuilder(true, nullptr);
				
			MenuBuilder.BeginSection("Constraints");
			for (int32 Index = 0; Index < 4; ++Index)
			{
				const FBaseConstraintListWidget::EShowConstraints Constraints = static_cast<FBaseConstraintListWidget::EShowConstraints>(Index);
				FUIAction ItemAction(FExecuteAction::CreateSP(this, &SConstraintsEditionWidget_ConstrainPanel::OnSelectShowConstraints, Index));
				const TAttribute<FText> Text = ConstraintsEditionWidget->GetShowConstraintsText(Constraints);
				const TAttribute<FText> Tooltip = ConstraintsEditionWidget->GetShowConstraintsTooltip(Constraints);
				MenuBuilder.AddMenuEntry(Text, Tooltip, FSlateIcon(), ItemAction);
			}
			MenuBuilder.EndSection();

			return MenuBuilder.MakeWidget();
		})
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			[
				SNew(STextBlock)
				.Text_Lambda([this](){ return GetShowConstraintsName(); })
				.ToolTipText_Lambda([this]() { return GetShowConstraintsTooltip(); })
			]
		];
}

TSharedRef<SWidget> SConstraintsEditionWidget_ConstrainPanel::CreateAddConstraintButton()
{
	const auto IsEnabled = [this]
	{
		const ULevel* CurrentLevel = OwningEditMode->GetWorld()->GetCurrentLevel();
		const TArray<AActor*> SelectedActors = ObjectPtrDecay(CurrentLevel->Actors).FilterByPredicate([](const AActor* Actor)
		{
			return Actor && Actor->IsSelected();
		});
		return !SelectedActors.IsEmpty();
	};
	
	return SNew(SPositiveActionButton)
		.IsEnabled_Lambda(IsEnabled)
		.OnClicked(this, &SConstraintsEditionWidget_ConstrainPanel::HandleAddConstraintClicked)
		.Cursor(EMouseCursor::Default)
		.Text(LOCTEXT("AddConstraint.Label", "Add"))
		.ToolTipText_Lambda([IsEnabled]
		{
			return IsEnabled()
				? LOCTEXT("AddConstraint.ToolTip.Enabled", "Add constraint")
				: LOCTEXT("AddConstraint.ToolTip.Disabled", "Select a control rig to add constraint");
		});
}

void SConstraintsEditionWidget_ConstrainPanel::OnSelectShowConstraints(int32 Index)
{
	if (ConstraintsEditionWidget.IsValid())
	{
		const SConstraintsEditionWidget::EShowConstraints ShowConstraint = static_cast<SConstraintsEditionWidget::EShowConstraints>(Index);
		ConstraintsEditionWidget->SetShowConstraints(ShowConstraint);
	}
}

FReply SConstraintsEditionWidget_ConstrainPanel::HandleAddConstraintClicked()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	const auto AddConstraintWidget = [&](ETransformConstraintType InConstraintType)
	{
		const TSharedRef<SConstraintMenuEntry> Entry =
			SNew(SConstraintMenuEntry, InConstraintType)
			.OnConstraintCreated_Lambda([this]()
			{
				ConstraintsEditionWidget->RefreshConstraintList();
				ConstraintsEditionWidget->ResetParent();
			})
			.OnGetParent_Lambda([this]()
			{
				const FConstrainable InvalidParent;
				return ConstraintsEditionWidget ? ConstraintsEditionWidget->GetParent() : InvalidParent;
			});
		MenuBuilder.AddWidget(Entry, FText::GetEmpty(), true);
	};
	
	MenuBuilder.BeginSection("CreateConstraint", LOCTEXT("CreateConstraintHeader", "Create New..."));
	{
		AddConstraintWidget(ETransformConstraintType::Translation);
		AddConstraintWidget(ETransformConstraintType::Rotation);
		AddConstraintWidget(ETransformConstraintType::Scale);
		AddConstraintWidget(ETransformConstraintType::Parent);
		AddConstraintWidget(ETransformConstraintType::LookAt);
	}
	MenuBuilder.EndSection();
	
	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	
	return FReply::Handled();
}

FText SConstraintsEditionWidget_ConstrainPanel::GetShowConstraintsName() const
{
	return ConstraintsEditionWidget.IsValid()
		? ConstraintsEditionWidget->GetShowConstraintsText(FBaseConstraintListWidget::ShowConstraints)
		: FText::GetEmpty();
}

FText SConstraintsEditionWidget_ConstrainPanel::GetShowConstraintsTooltip() const
{
	return ConstraintsEditionWidget.IsValid()
		? ConstraintsEditionWidget->GetShowConstraintsTooltip(FBaseConstraintListWidget::ShowConstraints)
		: FText::GetEmpty();
}

void SConstraintsEditionWidget_ConstrainPanel::OnControlSelected(UControlRig* Subject, FRigControlElement* RigControlElement, bool bIsSelected) const
{
	if (Subject && Subject->GetHierarchy())
	{
		ConstraintsEditionWidget->InvalidateConstraintList();
	}
}
}

#undef LOCTEXT_NAMESPACE