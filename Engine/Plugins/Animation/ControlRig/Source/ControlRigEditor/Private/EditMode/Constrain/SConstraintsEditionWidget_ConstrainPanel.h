// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRig.h"
#include "Editor/Constraints/SConstraintsWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FControlRigEditMode;

namespace UE::ControlRigEditor
{
class FRigSelectionViewModel;
	
/**
 * Constraints edition widget that is shown in the constraints window.
 *
 * Places a SConstraintsEditionWidget in an SExpandableArea.
 * The SExpandableArea's header contains a combo button and + button in the header.
 */
class SConstraintsEditionWidget_ConstrainPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SConstraintsEditionWidget_ConstrainPanel)
		: _InitiallyCollapsed(true)
	{}
		/** Whether or not the area is initially collapsed */
		SLATE_ARGUMENT(bool, InitiallyCollapsed)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode, const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel);
	virtual ~SConstraintsEditionWidget_ConstrainPanel() override;

	void RefreshConstraintList() const { ConstraintsEditionWidget->RefreshConstraintList(); }

private:

	/** Control rig edit mode we were constructed with */
	FControlRigEditMode* OwningEditMode = nullptr;

	/** View model for when control rig selection changes. */
	TSharedPtr<FRigSelectionViewModel> SelectionViewModel;
	
	/** The constraints edition widget we wrap. */
	TSharedPtr<SConstraintsEditionWidget> ConstraintsEditionWidget;

	// Header area (similar to expandable area border we used to have)
	TSharedRef<SWidget> CreateSelectionComboButton();
	TSharedRef<SWidget> CreateAddConstraintButton();

	/**  */
	void OnSelectShowConstraints(int32 Index);
	/** Spawns the constraints menu when the plus button is pressed. */
	FReply HandleAddConstraintClicked();
	
	FText GetShowConstraintsName() const;
	FText GetShowConstraintsTooltip() const;

	/** Refresh the constrain widget when the selected control rig changes. */
	void OnControlSelected(UControlRig* Subject, FRigControlElement* RigControlElement, bool bIsSelected) const;
	
};
}

