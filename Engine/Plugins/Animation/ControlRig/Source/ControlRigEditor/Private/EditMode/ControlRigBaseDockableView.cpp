// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/ControlRigBaseDockableView.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Styling/CoreStyle.h"
#include "ScopedTransaction.h"
#include "ControlRig.h"
#include "UnrealEdGlobals.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ISequencer.h"


FControlRigBaseDockableView::FControlRigBaseDockableView()
{
}

FControlRigBaseDockableView::~FControlRigBaseDockableView()
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->OnControlRigAddedOrRemoved().RemoveAll(this);
		EditMode->OnControlRigSelected().RemoveAll(this);
		EditMode->OnControlRigVisibilityChanged().RemoveAll(this);
	}
}

TArray<UControlRig*> FControlRigBaseDockableView::GetControlRigs()  const
{
	TArray<UControlRig*> ControlRigs;
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		ControlRigs = EditMode->GetControlRigsArray(false);
	}
	return ControlRigs;
}

void FControlRigBaseDockableView::SetEditMode(FControlRigEditMode& InEditMode)
{
	ModeTools = InEditMode.GetModeManager();
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->OnControlRigAddedOrRemoved().RemoveAll(this);
		EditMode->OnControlRigAddedOrRemoved().AddSP(AsSharedWidget(), &FControlRigBaseDockableView::HandleControlAdded);
		EditMode->OnControlRigVisibilityChanged().RemoveAll(this);
		EditMode->OnControlRigVisibilityChanged().AddSP(AsSharedWidget(), &FControlRigBaseDockableView::HandleRigVisibilityChanged);
		EditMode->OnControlRigSelected().RemoveAll(this);
		EditMode->OnControlRigSelected().AddSP(AsSharedWidget(), &FControlRigBaseDockableView::HandlElementSelected);
	}
}

void FControlRigBaseDockableView::HandleControlAdded(UControlRig* ControlRig, bool bIsAdded)
{
}

void FControlRigBaseDockableView::HandleRigVisibilityChanged(TArray<UControlRig*> InControlRigs)
{
}

void FControlRigBaseDockableView::HandleControlSelected(UControlRig* ControlRig, FRigControlElement* InControl, bool bSelected)
{
}

void FControlRigBaseDockableView::HandlElementSelected(UControlRig* ControlRig, const FRigElementKey& Key, bool bSelected)
{
	if (ControlRig)
	{
		if (Key.Type == ERigElementType::Control)
		{
			if (FRigBaseElement* BaseElement = ControlRig->GetHierarchy()->Find(Key))
			{
				if (FRigControlElement* ControlElement = Cast<FRigControlElement>(BaseElement))
				{
					HandleControlSelected(ControlRig, ControlElement, bSelected);
				}
			}
		}
	}
}


FControlRigEditMode* FControlRigBaseDockableView::GetEditMode() const
{
	FControlRigEditMode* EditMode = ModeTools ? static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)) : nullptr;
	return EditMode;
}

ISequencer* FControlRigBaseDockableView::GetSequencer() const
{
	if (!ModeTools)
	{
		return nullptr;
	}
	
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TWeakPtr<ISequencer> Sequencer = EditMode->GetWeakSequencer();
		return Sequencer.Pin().Get();
	}
	return nullptr;
}
