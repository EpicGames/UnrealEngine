// Copyright Epic Games, Inc. All Rights Reserved.
/**
* Base View for Dockable Control Rig Animation widgets Details/Outliner
*/
#pragma once

#include "CoreMinimal.h"
#include "EditorModeManager.h"


class UControlRig;
class ISequencer;
class FControlRigEditMode;
struct FRigControlElement;
struct FRigElementKey;

class FControlRigBaseDockableView 
{
public:
	FControlRigBaseDockableView();
	virtual ~FControlRigBaseDockableView();
	TArray<UControlRig*> GetControlRigs() const;

	virtual TSharedRef<FControlRigBaseDockableView> AsSharedWidget() = 0;
	virtual void SetEditMode(FControlRigEditMode& InEditMode);
	FControlRigEditMode* GetEditMode() const;

protected:
	virtual void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected);
	virtual void HandleControlAdded(UControlRig* ControlRig, bool bIsAdded);
	virtual void HandleRigVisibilityChanged(TArray<UControlRig*> InControlRigs);

	void HandlElementSelected(UControlRig* Subject, const FRigElementKey& Key, bool bSelected);

	ISequencer* GetSequencer() const;

	FEditorModeTools* ModeTools = nullptr;

};

