// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/ControlRigEditMode.h"

#include "AnimationCoreLibrary.h"
#include "AnimationEditorPreviewActor.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "AnimDetails/Proxies/AnimDetailsProxyBool.h"
#include "AnimDetails/Proxies/AnimDetailsProxyEnum.h"
#include "AnimDetails/Proxies/AnimDetailsProxyFloat.h"
#include "AnimDetails/Proxies/AnimDetailsProxyInteger.h"
#include "AnimDetails/Proxies/AnimDetailsProxyLocation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyRotation.h"
#include "AnimDetails/Proxies/AnimDetailsProxyScale.h"
#include "AnimDetails/Proxies/AnimDetailsProxyTransform.h"
#include "AnimDetails/Proxies/AnimDetailsProxyVector2D.h"
#include "BlutilityMenuExtensions.h"
#include "Components/ActorComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Delegates/IDelegateInstance.h"
#include "EditMode/ControlRigEditModeToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "EditMode/SControlRigEditModeTools.h"
#include "ControlRig.h"
#include "HitProxies.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "EditMode/SControlRigOutliner.h"
#include "ISequencer.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "MovieScene.h"
#include "Editor.h"
#include "EditorViewportClient.h"
#include "EditorViewportSelectability.h"
#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "LevelEditorViewport.h"
#include "Components/SkeletalMeshComponent.h"
#include "EditMode/ControlRigEditModeCommands.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ControlRigEditorModule.h"
#include "Constraint.h"
#include "EngineUtils.h"
#include "IControlRigObjectBinding.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigGizmoActor.h"
#include "SEditorViewport.h"
#include "ScopedTransaction.h"
#include "Rigs/AdditiveControlRig.h"
#include "Rigs/FKControlRig.h"
#include "ControlRigComponent.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_InteractionExecution.h"
#include "PersonaSelectionProxies.h"
#include "PropertyHandle.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Settings/ControlRigSettings.h"
#include "ToolMenus.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "ControlRigSpaceChannelEditors.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "LevelSequence.h"
#include "LevelEditor.h"
#include "InteractiveToolManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Transform/TransformConstraint.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "TrackEditors/CommonAnimationTrackEditor.h"
#include "Tools/ControlRigPose.h"
#include "Materials/Material.h"
#include "ControlRigEditorStyle.h"
#include "DragTool_BoxSelect.h"
#include "DragTool_FrustumSelect.h"
#include "AnimationEditorViewportClient.h"
#include "ControlRigUtilityActions.h"
#include "ControlShapeActorHelper.h"
#include "EditorInteractiveGizmoManager.h"
#include "ISequencerWidget.h"
#include "ModularRig.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Editor/ControlRigViewportToolbarExtensions.h"
#include "Slate/SceneViewport.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "Misc/RigEditModeUtils.h"
#include "SpacePicker/RigFloatingSpacePickerManager.h"
#include "SpacePicker/Views/SRigSpacePickerMenuEntry.h"
#include "Sequencer/AnimLayers/AnimLayers.h"
#include "Tools/ControlRigFrustumSelectInteraction.h"
#include "Tools/MotionTrailOptions.h"
#include "Transform/TransformableHandleUtils.h"
#include "Features/IModularFeatures.h"
#include "IMovieSceneRootMotionOffsetProvider.h"
#include "UnrealWidget.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportLegacyClickInteraction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigEditMode)

TAutoConsoleVariable<bool> CVarClickSelectThroughGizmo(TEXT("ControlRig.Sequencer.ClickSelectThroughGizmo"), false, TEXT("When false you can't click through a gizmo and change selection if you will select the gizmo when in Animation Mode, default to false."));

TAutoConsoleVariable<bool> CVarMarqueePivotSelect(
	TEXT("ControlRig.Sequencer.MarqueePivotSelect"),
	false,
	TEXT("If true, marquee selection (BoxSelect/FrustumSelect) in Control Rig edit mode ")
	TEXT("additionally picks any control whose pivot lies inside the marquee region, ")
	TEXT("in addition to the existing bounds/hit-proxy tests. Catches controls whose ")
	TEXT("gizmo mesh is asymmetric, sparse, or occluded. Set to false to restore the ")
	TEXT("prior bounds-only behavior."));

namespace UE::ControlRigEditMode::Private
{
	const TCHAR* FocusModeName = TEXT("AnimMode.PendingFocusMode");

	static bool bFocusMode = false;
	static FAutoConsoleVariableRef CVarSetFocusOnHover(
	   FocusModeName,
	   bFocusMode,
	   TEXT("Force setting focus on the hovered viewport when entering a key.")
	   );

	IConsoleVariable* GetFocusModeVariable()
	{
		return IConsoleManager::Get().FindConsoleVariable(FocusModeName);
	}

	static ILevelEditor* GetLevelEditor()
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
		{
			if (TSharedPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance().Pin())
			{
				return LevelEditorPtr.Get();
			}
		}

		return nullptr;
	}
}

using namespace UE::ControlRigEditMode;
using namespace UE::AnimationEditMode;

void UControlRigEditModeDelegateHelper::OnPoseInitialized()
{
	if (EditMode)
	{
		EditMode->OnPoseInitialized();
	}
}
void UControlRigEditModeDelegateHelper::PostPoseUpdate()
{
	if (EditMode)
	{
		EditMode->PostPoseUpdate();
	}
}

void UControlRigEditModeDelegateHelper::AddDelegates(USkeletalMeshComponent* InSkeletalMeshComponent)
{
	if (BoundComponent.IsValid())
	{
		if (BoundComponent.Get() == InSkeletalMeshComponent)
		{
			return;
		}
	}

	RemoveDelegates();

	BoundComponent = InSkeletalMeshComponent;

	if (BoundComponent.IsValid())
	{
		BoundComponent->OnAnimInitialized.AddDynamic(this, &UControlRigEditModeDelegateHelper::OnPoseInitialized);
		OnBoneTransformsFinalizedHandle = BoundComponent->RegisterOnBoneTransformsFinalizedDelegate(
			FOnBoneTransformsFinalizedMultiCast::FDelegate::CreateUObject(this, &UControlRigEditModeDelegateHelper::PostPoseUpdate));
	}
}

void UControlRigEditModeDelegateHelper::RemoveDelegates()
{
	if(BoundComponent.IsValid())
	{
		BoundComponent->OnAnimInitialized.RemoveAll(this);
		BoundComponent->UnregisterOnBoneTransformsFinalizedDelegate(OnBoneTransformsFinalizedHandle);
		OnBoneTransformsFinalizedHandle.Reset();
		BoundComponent = nullptr;
	}
}

#define LOCTEXT_NAMESPACE "ControlRigEditMode"

/** The different parts of a transform that manipulators can support */
enum class ETransformComponent
{
	None,

	Rotation,

	Translation,

	Scale
};

namespace ControlRigSelectionConstants
{
	/** Distance to trace for physics bodies */
	static const float BodyTraceDistance = 100000.0f;
}

bool FControlRigEditMode::bDoPostPoseUpdate = true;

FControlRigEditMode::FControlRigEditMode()
	: PendingFocus(FPendingWidgetFocus::MakeNoTextEdit())
	, bIsChangingControlShapeTransform(false)
	, bIsTracking(false)
	, bManipulatorMadeChange(false)
	, bSelecting(false)
	, bSelectionChanged(false)
	, RecreateControlShapesRequired(ERecreateControlRigShape::RecreateNone)
	, bSuspendHierarchyNotifs(false)
	, CurrentViewportClient(nullptr)
	, bIsChangingCoordSystem(false)
	, bShowControlsAsOverlay(false)
	, InteractionType((uint8)EControlRigInteractionType::None)
	, bIsConstructionEventRunning(false)
{
	StoredPose = NewObject<UControlRigPoseAsset>(GetTransientPackage(), NAME_None);
	DetailKeyFrameCache = MakeShared<FDetailKeyFrameCacheAndHandler>();

	bShowControlsAsOverlay = IsControlsAsOverlayEnabled();

	CommandBindings = MakeShareable(new FUICommandList);
	BindCommands();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FControlRigEditMode::OnObjectsReplaced);
#endif
}

FControlRigEditMode::~FControlRigEditMode()
{	
	CommandBindings = nullptr;

	DestroyShapesActors(nullptr);
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
	OnWorldCleanupHandle.Reset();
	OnControlRigAddedOrRemovedDelegate.Clear();
	OnControlRigSelectedDelegate.Clear();
	OnControlRigVisibilityChangedDelegate.Clear();

	TArray<TWeakObjectPtr<UControlRig>> PreviousRuntimeRigs = RuntimeControlRigs;
	for (int32 PreviousRuntimeRigIndex = 0; PreviousRuntimeRigIndex < PreviousRuntimeRigs.Num(); PreviousRuntimeRigIndex++)
	{
		if (PreviousRuntimeRigs[PreviousRuntimeRigIndex].IsValid())
		{
			RemoveControlRig(PreviousRuntimeRigs[PreviousRuntimeRigIndex].Get());
		}
	}
	RuntimeControlRigs.Reset();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif

}

bool FControlRigEditMode::SetSequencer(TWeakPtr<ISequencer> InSequencer)
{
	if (InSequencer != WeakSequencer)
	{
		if (WeakSequencer.IsValid())
		{
			constexpr bool bDisable = false;
			const TSharedPtr<ISequencer> PreviousSequencer = WeakSequencer.Pin();
			PreviousSequencer->GetSequencerWidgetInterface().EnablePendingFocusOnHovering(bDisable);
		}
		
		WeakSequencer = InSequencer;

		UnsetSequencerDelegates();

		DestroyShapesActors(nullptr);
		TArray<TWeakObjectPtr<UControlRig>> PreviousRuntimeRigs = RuntimeControlRigs;
		for (int32 PreviousRuntimeRigIndex = 0; PreviousRuntimeRigIndex < PreviousRuntimeRigs.Num(); PreviousRuntimeRigIndex++)
		{
			if (PreviousRuntimeRigs[PreviousRuntimeRigIndex].IsValid())
			{
				RemoveControlRig(PreviousRuntimeRigs[PreviousRuntimeRigIndex].Get());
			}
		}
		
		RuntimeControlRigs.Reset();

		if (InSequencer.IsValid())
		{
			TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
			if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence()))
			{
				TArray<FControlRigSequencerBindingProxy> Proxies = UControlRigSequencerEditorLibrary::GetControlRigs(LevelSequence);
				for (FControlRigSequencerBindingProxy& Proxy : Proxies)
				{
					if (UControlRig* ControlRig = Proxy.ControlRig.Get())
					{
						AddControlRigInternal(ControlRig);
					}
				}
			}

			LastMovieSceneSig = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
			
			SetSequencerDelegates(WeakSequencer);
			
			if (AnimDetailsProxyManager)
			{
				AnimDetailsProxyManager->RequestUpdateProxies();
			}
			
			Sequencer->GetSequencerWidgetInterface().EnablePendingFocusOnHovering(Private::bFocusMode);
		}
		
		SetObjects_Internal();
		if (FControlRigEditModeToolkit::Outliner.IsValid())
		{
			FControlRigEditModeToolkit::Outliner->SetEditMode(*this);
		}

		if (WeakSequencer.IsValid() && !RuntimeControlRigs.IsEmpty())
		{
			RequestToRecreateControlShapeActors();
		}
	}
	return false;
}

bool FControlRigEditMode::AddControlRigObject(UControlRig* InControlRig, const TWeakPtr<ISequencer>& InSequencer)
{
	if (InControlRig)
	{
		if (RuntimeControlRigs.Contains(InControlRig) == false)
		{
			if (InSequencer.IsValid())
			{
				if (SetSequencer(InSequencer) == false) //was already there so just add it,otherwise this function will add everything in the active 
				{
					AddControlRigInternal(InControlRig);
					SetObjects_Internal();
				}
				return true;
			}		
		}
	}
	return false;
}

void FControlRigEditMode::SetObjects(UControlRig* ControlRig,  UObject* BindingObject, const TWeakPtr<ISequencer>& InSequencer)
{
	TArray<TWeakObjectPtr<UControlRig>> PreviousRuntimeRigs = RuntimeControlRigs;
	for (int32 PreviousRuntimeRigIndex = 0; PreviousRuntimeRigIndex < PreviousRuntimeRigs.Num(); PreviousRuntimeRigIndex++)
	{
		if (PreviousRuntimeRigs[PreviousRuntimeRigIndex].IsValid())
		{
			RemoveControlRig(PreviousRuntimeRigs[PreviousRuntimeRigIndex].Get());
		}
	}
	RuntimeControlRigs.Reset();

	if (InSequencer.IsValid())
	{
		WeakSequencer = InSequencer;
	
	}
	// if we get binding object, set it to control rig binding object
	if (BindingObject && ControlRig)
	{
		if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
		{
			if (ObjectBinding->GetBoundObject() == nullptr)
			{
				ObjectBinding->BindToObject(BindingObject);
			}
		}

		AddControlRigInternal(ControlRig);
	}
	else if (ControlRig)
	{
		AddControlRigInternal(ControlRig);
	}

	SetObjects_Internal();
}

bool FControlRigEditMode::IsInLevelEditor() const
{
	return !AreEditingControlRigDirectly() && GetModeManager() == &GLevelEditorModeTools();
}

void FControlRigEditMode::SetUpDetailPanel() const
{
	if (!AreEditingControlRigDirectly() && Toolkit)
	{
		TSharedPtr<SControlRigEditModeTools> ModeTools = StaticCastSharedPtr<SControlRigEditModeTools>(Toolkit->GetInlineContent());
		if (ModeTools.IsValid())
		{
			ModeTools->SetSequencer(WeakSequencer.Pin());
		}	
	}
}

void FControlRigEditMode::SetObjects_Internal()
{
	bool bHasValidRuntimeControlRig = false;
	for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* RuntimeControlRig = RuntimeRigPtr.Get())
		{
			RuntimeControlRig->ControlModified().RemoveAll(this);
			RuntimeControlRig->GetHierarchy()->OnModified().RemoveAll(this);

			RuntimeControlRig->ControlModified().AddSP(this, &FControlRigEditMode::OnControlModified);
			RuntimeControlRig->GetHierarchy()->OnModified().AddSP(this, &FControlRigEditMode::OnHierarchyModified_AnyThread);
			if (USkeletalMeshComponent* MeshComponent = Cast<USkeletalMeshComponent>(GetHostingSceneComponent(RuntimeControlRig)))
			{
				TStrongObjectPtr<UControlRigEditModeDelegateHelper>* DelegateHelper = DelegateHelpers.Find(RuntimeControlRig);
				if (!DelegateHelper)
				{
					DelegateHelpers.Add(RuntimeControlRig, TStrongObjectPtr<UControlRigEditModeDelegateHelper>(NewObject<UControlRigEditModeDelegateHelper>()));
					DelegateHelper = DelegateHelpers.Find(RuntimeControlRig);
				}
				else if (DelegateHelper->IsValid() == false)
				{
					DelegateHelper->Get()->RemoveDelegates();
					DelegateHelpers.Remove(RuntimeControlRig);
					*DelegateHelper = TStrongObjectPtr<UControlRigEditModeDelegateHelper>(NewObject<UControlRigEditModeDelegateHelper>());
					DelegateHelper->Get()->EditMode = this;
					DelegateHelper->Get()->AddDelegates(MeshComponent);
					DelegateHelpers.Add(RuntimeControlRig, *DelegateHelper);
				}
				
				if (DelegateHelper && DelegateHelper->IsValid())
				{
					bHasValidRuntimeControlRig = true;
				}
			}
		}
	}

	if (UsesToolkits() && Toolkit.IsValid())
	{
		StaticCastSharedPtr<FControlRigEditModeToolkit>(Toolkit)->OnControlsChanged(RuntimeControlRigs);
	}

	if (!bHasValidRuntimeControlRig)
	{
		DestroyShapesActors(nullptr);
		SetUpDetailPanel();
	}
	else
	{
		// create default manipulation layer
		RequestToRecreateControlShapeActors();
	}
}

bool FControlRigEditMode::UsesToolkits() const
{
	return true;
}

void FControlRigEditMode::Enter()
{
	// Call parent implementation
	FEdMode::Enter();
	LastMovieSceneSig = FGuid();
	if (UsesToolkits())
	{
		if (!AreEditingControlRigDirectly())
		{
			if (WeakSequencer.IsValid() == false)
			{
				SetSequencer(UE::AnimationEditMode::GetSequencer());
			}
		}
		if (!Toolkit.IsValid())
		{
			Toolkit = MakeShareable(new FControlRigEditModeToolkit(*this));
			Toolkit->Init(Owner->GetToolkitHost());
			
			Toolkit->GetToolkitCommands()->Append(CommandBindings.ToSharedRef());
		}

		FEditorModeTools* ModeManager = GetModeManager();

		bIsChangingCoordSystem = false;
		if (CoordSystemPerWidgetMode.Num() < (UE::Widget::WM_Max))
		{
			CoordSystemPerWidgetMode.SetNum(UE::Widget::WM_Max);
			ECoordSystem CoordSystem = ModeManager->GetCoordSystem();
			for (int32 i = 0; i < UE::Widget::WM_Max; ++i)
			{
				CoordSystemPerWidgetMode[i] = CoordSystem;
			}
		}

		ModeManager->OnWidgetModeChanged().AddSP(this, &FControlRigEditMode::OnWidgetModeChanged);
		ModeManager->OnCoordSystemChanged().AddSP(this, &FControlRigEditMode::OnCoordSystemChanged);
	}
	
	WorldPtr = GetWorld();
	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddSP(this, &FControlRigEditMode::OnWorldCleanup);
	SetObjects_Internal();

	if (UControlRigEditModeSettings* Settings = GetMutableSettings())
	{
		if (!Settings->OnSettingsChange.IsBoundToObject(this))
		{
			Settings->OnSettingsChange.AddSP(this, &FControlRigEditMode::OnSettingsChanged);
		}
		if (!Settings->OnSettingChanged().IsBoundToObject(this))
		{
			Settings->OnSettingChanged().AddSP(this, &FControlRigEditMode::OnSettingsPropertyChanged);
		}
	}

	if (IsInLevelEditor() && !ViewportSelectabilityDelegateHandle.IsValid())
	{
		ViewportSelectabilityDelegateHandle = FEditorViewportSelectability::RegisterObjectSelectablePredicate(
			FOnIsObjectSelectableInViewport::CreateRaw(this, &FControlRigEditMode::IsViewportObjectSelectable)
		);
	}

	if (IsInLevelEditor())
	{
		UE::ControlRig::PopulateControlRigViewportToolbarTransformSubmenu("LevelEditor.ViewportToolbar.Transform");
		UE::ControlRig::PopulateControlRigViewportToolbarShowSubmenu("LevelEditor.ViewportToolbar.Show");
		if (GEngine)
		{
			GEngine->OnActorMoving().AddRaw(this, &FControlRigEditMode::HandleActorMoving);
		}
	}

	if (UModeManagerInteractiveToolsContext* const InteractiveToolsContext = GetModeManager()->GetInteractiveToolsContext())
	{
		if (!BuildViewportInteractionsDelegate.IsValid())
		{
			BuildViewportInteractionsDelegate = InteractiveToolsContext->OnBuildViewportInteractions().AddSPLambda(this, [](UViewportInteractionsBehaviorSource* Source)
			{
				// Add interactions specific to Control Rig
				Source->AddInteraction<UControlRigFrustumSelectInteraction>();
			});
		}
	}

	RegisterPendingFocusMode();
	BindDirectControl();

	// initialize the gizmo context
	WeakGizmoContext = GetModeManager()->GetGizmoContext();

	if (IsInLevelEditor())
	{
		ConstraintsCache.RegisterNotifications();
		Keyframer.Initialize();

		AnimDetailsProxyManager = NewObject<UAnimDetailsProxyManager>(GetTransientPackage(), NAME_None);
	}
}

//todo get working with Persona
static void ClearOutAnyActiveTools()
{
	if (ILevelEditor* LevelEditor = Private::GetLevelEditor())
	{
		FString ActiveToolName =
			LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveToolName(EToolSide::Left);
		if (ActiveToolName == TEXT("SequencerPivotTool"))
		{
			LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->DeactivateTool(
				EToolSide::Left, EToolShutdownType::Completed
			);
		}
	}
}

void FControlRigEditMode::Exit()
{
	AnimDetailsProxyManager = nullptr;

	ConstraintsCache.UnregisterNotifications();
	ConstraintsCache.Reset();
	
	if (WeakGizmoContext.IsValid())
	{
		WeakGizmoContext->RotationContext = FRotationContext();
		WeakGizmoContext.Reset();
	}

	if (IsInLevelEditor())
	{
		UE::ControlRig::RemoveControlRigViewportToolbarExtensions();
		
		if (const ILevelEditor* const LevelEditor = Private::GetLevelEditor())
		{
			if (UModeManagerInteractiveToolsContext* const InteractiveToolsContext = LevelEditor->GetEditorModeManager().GetInteractiveToolsContext())
			{
				InteractiveToolsContext->OnBuildViewportInteractions().Remove(BuildViewportInteractionsDelegate);
				BuildViewportInteractionsDelegate.Reset();
			}
		}
	}

	if (GEngine)
	{
		GEngine->OnActorMoving().RemoveAll(this);
	}

	if (ViewportSelectabilityDelegateHandle.IsValid())
	{
		FEditorViewportSelectability::UnregisterObjectSelectablePredicate(ViewportSelectabilityDelegateHandle);
		ViewportSelectabilityDelegateHandle.Reset();
	}

	UnregisterPendingFocusMode();
	DirectControl.UnBind();
	
	ClearOutAnyActiveTools();
	OnControlRigAddedOrRemovedDelegate.Clear();
	OnControlRigSelectedDelegate.Clear();
	OnControlRigVisibilityChangedDelegate.Clear();
	for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			ControlRig->ClearControlSelection();
		}
	}

	if (!InteractionScopes.IsEmpty())
	{
		if (GEditor)
		{
			GEditor->EndTransaction();
		}

		InteractionScopes.Reset();
		bManipulatorMadeChange = false;
	}

	if (FControlRigEditModeToolkit::Details.IsValid())
	{
		FControlRigEditModeToolkit::Details.Reset();
	}
	if (FControlRigEditModeToolkit::Outliner.IsValid())
	{
		FControlRigEditModeToolkit::Outliner.Reset();
	}
	if (Toolkit.IsValid())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
		Toolkit.Reset();
	}

	DestroyShapesActors(nullptr);

	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
	OnWorldCleanupHandle.Reset();

	TArray<TWeakObjectPtr<UControlRig>> PreviousRuntimeRigs = RuntimeControlRigs;
	for (int32 PreviousRuntimeRigIndex = 0; PreviousRuntimeRigIndex < PreviousRuntimeRigs.Num(); PreviousRuntimeRigIndex++)
	{
		if (PreviousRuntimeRigs[PreviousRuntimeRigIndex].IsValid())
		{
			RemoveControlRig(PreviousRuntimeRigs[PreviousRuntimeRigIndex].Get());
		}
	}
	RuntimeControlRigs.Reset();

	//clear delegates
	FEditorModeTools* ModeManager = GetModeManager();
	ModeManager->OnWidgetModeChanged().RemoveAll(this);
	ModeManager->OnCoordSystemChanged().RemoveAll(this);

	if (UControlRigEditModeSettings* Settings = GetMutableSettings())
	{
		Settings->OnSettingsChange.RemoveAll(this);
		Settings->OnSettingChanged().RemoveAll(this);
	}
	
	// Call parent implementation
	FEdMode::Exit();
}

void FControlRigEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);
	
	//if we have don't have a viewport client or viewport, bail we can be in UMG for example
	if (ViewportClient == nullptr || ViewportClient->Viewport == nullptr)
	{
		return;
	}
	
	CheckMovieSceneSig();

	if(DeferredItemsToFrame.Num() > 0)
	{
		TGuardValue<FEditorViewportClient*> ViewportGuard(CurrentViewportClient, ViewportClient);
		FrameItems(DeferredItemsToFrame);
		DeferredItemsToFrame.Reset();
	}

	if (bSelectionChanged)
	{
		SetUpDetailPanel();
		HandleSelectionChanged();
		bSelectionChanged = false;
	}
	else
	{
		// HandleSelectionChanged() will already update the pivots 
		UpdatePivotTransforms();
	}

	if (!AreEditingControlRigDirectly() == false)
	{
		ViewportClient->Invalidate();
	}

	// Defer creation of shapes if manipulating the viewport
	if (RecreateControlShapesRequired != ERecreateControlRigShape::RecreateNone && !(FSlateApplication::Get().HasAnyMouseCaptor() || GUnrealEd->IsUserInteracting()))
	{
		RecreateControlShapeActors();
		const bool bAreEditingControlRigDirectly = AreEditingControlRigDirectly();
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
				for (const FRigElementKey& SelectedKey : SelectedRigElements)
				{
					if (SelectedKey.Type == ERigElementType::Control)
					{
						AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig,SelectedKey.Name);
						if (ShapeActor)
						{
							ShapeActor->SetSelected(true);
						}
					}
				}
			}
		}
		SetUpDetailPanel();
		HandleSelectionChanged();
		RecreateControlShapesRequired = ERecreateControlRigShape::RecreateNone;
		ControlRigsToRecreate.SetNum(0);
		if (DetailKeyFrameCache)
        {
        	DetailKeyFrameCache->ResetCachedData();
        }

		OnControlRigShapeActorsRecreatedDelegate.Broadcast();
	}

	// Ensure transient control shape actors are selected while DM is active.
	// RecreateControlShapeActors() may be called from paths outside this Tick
	// (e.g. OnPostConstruction_AnyThread), which resets RecreateControlShapesRequired
	// before the selection loop above can run, leaving the shape actor unselected.
	// ComputePivotFromShapeActors gates entirely on IsSelected(), so an unselected
	// shape actor causes the gizmo to fall back to the world origin.
	for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
			for (const FRigElementKey& SelectedKey : SelectedRigElements)
			{
				if (SelectedKey.Type == ERigElementType::Control)
				{
					AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig, SelectedKey.Name);
					if (ShapeActor && !ShapeActor->IsSelected())
					{
						ShapeActor->SetSelected(true);
					}
				}
			}
		}
	}

	{
		// We need to tick here since changing a bone for example
		// might have changed the transform of the Control
		PostPoseUpdate(ViewportClient);

		if (!AreEditingControlRigDirectly() == false) //only do this check if not in level editor
		{
			for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
			{
				if (UControlRig* ControlRig = RuntimeRigPtr.Get())
				{
					TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
					UE::Widget::EWidgetMode CurrentWidgetMode = ViewportClient->GetWidgetMode();
					if(!RequestedWidgetModes.IsEmpty())
					{
						if(RequestedWidgetModes.Last() != CurrentWidgetMode)
						{
							CurrentWidgetMode = RequestedWidgetModes.Last();
							ViewportClient->SetWidgetMode(CurrentWidgetMode);
						}
						RequestedWidgetModes.Reset();
					}
					for (FRigElementKey SelectedRigElement : SelectedRigElements)
					{
						//need to loop through the shape actors and set widget based upon the first one
						if (AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig, SelectedRigElement.Name))
						{
							if (!ModeSupportedByShapeActor(ShapeActor, CurrentWidgetMode))
							{
								if (FRigControlElement* ControlElement = ControlRig->FindControl(SelectedRigElement.Name))
								{
									switch (ControlElement->Settings.ControlType)
									{
									case ERigControlType::Float:
									case ERigControlType::Integer:
									case ERigControlType::Vector2D:
									case ERigControlType::Position:
									case ERigControlType::Transform:
									case ERigControlType::TransformNoScale:
									case ERigControlType::EulerTransform:
									{
										ViewportClient->SetWidgetMode(UE::Widget::WM_Translate);
										break;
									}
									case ERigControlType::Rotator:
									{
										ViewportClient->SetWidgetMode(UE::Widget::WM_Rotate);
										break;
									}
									case ERigControlType::Scale:
									case ERigControlType::ScaleFloat:
									{
										ViewportClient->SetWidgetMode(UE::Widget::WM_Scale);
										break;
									}
									}
									return; //exit if we switchted
								}
							}
						}
					}
				}
			}
		}
	}
	DetailKeyFrameCache->UpdateIfDirty();
	
	UpdateRotationContext();

	if (bUpdateNonInteractingRigs && InteractionScopes.IsEmpty())
	{
		// mark non-interacting constraints for evaluation
		for (const TWeakObjectPtr<UControlRig>& NonInteractingRig: NonInteractingRigs)
		{
			UControlRig* Rig = NonInteractingRig.Get();
			if (USceneComponent* BoundComponent = Rig ? GetHostingSceneComponent(Rig) : nullptr)
			{
				TransformableHandleUtils::MarkComponentForEvaluation(BoundComponent, false);
				
				for (const TWeakObjectPtr<UTickableConstraint>& WeakConstraint: ConstraintsCache.Get(BoundComponent, WorldPtr))
				{
					if (UTickableConstraint* Constraint = WeakConstraint.Get(); Constraint->Active)
					{
						if (UTickableParentConstraint* ParentConst = Cast<UTickableParentConstraint>(Constraint))
						{
							if (USceneComponent* ParentComponent = TransformableHandleUtils::GetTarget<USceneComponent>(ParentConst->ParentTRSHandle))
							{
								constexpr bool bRecursive = true;
								TransformableHandleUtils::MarkComponentForEvaluation(ParentComponent, bRecursive);								
							}
						}
						
						FConstraintsManagerController::Get(WorldPtr).MarkConstraintForEvaluation(Constraint);
					}
				}
			}
		}
		FConstraintsManagerController::Get(WorldPtr).FlushEvaluationGraph();
		TickManipulatableObjects(NonInteractingRigs);
		bUpdateNonInteractingRigs = false;
	}
}

//Hit proxy for FK Rigs and bones.
struct  HFKRigBoneProxy : public HHitProxy
{
	DECLARE_HIT_PROXY()

	FName BoneName;
	UControlRig* ControlRig;

	HFKRigBoneProxy()
		: HHitProxy(HPP_Foreground)
		, BoneName(NAME_None)
		, ControlRig(nullptr)
	{}

	HFKRigBoneProxy(FName InBoneName, UControlRig *InControlRig)
		: HHitProxy(HPP_Foreground)
		, BoneName(InBoneName)
		, ControlRig(InControlRig)
	{
	}

	// HHitProxy interface
	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }
	// End of HHitProxy interface
};

IMPLEMENT_HIT_PROXY(HFKRigBoneProxy, HHitProxy)


TSet<FName> FControlRigEditMode::GetActiveControlsFromSequencer(UControlRig* ControlRig)
{
	TSet<FName> ActiveControls;
	if (WeakSequencer.IsValid() == false)
	{
		return ActiveControls;
	}
	if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
	{
		USceneComponent* Component = Cast<USceneComponent>(ObjectBinding->GetBoundObject());
		if (!Component)
		{
			return ActiveControls;
		}
		const bool bCreateHandleIfMissing = false;
		FName CreatedFolderName = NAME_None;
		FGuid ObjectHandle = WeakSequencer.Pin()->GetHandleToObject(Component, bCreateHandleIfMissing);
		if (!ObjectHandle.IsValid())
		{
			UObject* ActorObject = Component->GetOwner();
			ObjectHandle = WeakSequencer.Pin()->GetHandleToObject(ActorObject, bCreateHandleIfMissing);
			if (!ObjectHandle.IsValid())
			{
				return ActiveControls;
			}
		}
		bool bCreateTrack = false;
		UMovieScene* MovieScene = WeakSequencer.Pin()->GetFocusedMovieSceneSequence()->GetMovieScene();
		if (!MovieScene)
		{
			return ActiveControls;
		}
		if (FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectHandle))
		{
			for (UMovieSceneTrack* Track : Binding->GetTracks())
			{
				if (UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
				{
					if (ControlRigParameterTrack->GetControlRig() == ControlRig)
					{
						TArray<FRigControlElement*> Controls;
						ControlRig->GetControlsInOrder(Controls);
						int Index = 0;
						for (FRigControlElement* ControlElement : Controls)
						{
							UMovieSceneControlRigParameterSection* ActiveSection = Cast<UMovieSceneControlRigParameterSection>(ControlRigParameterTrack->GetSectionToKey(ControlElement->GetFName()));
							if (ActiveSection)
							{
								if (ActiveSection->GetControlNameMask(ControlElement->GetFName()))
								{
									ActiveControls.Add(ControlElement->GetFName());
								}
								++Index;
							}
						}
					}
				}
			}
		}
	}
	return ActiveControls;
}

void FControlRigEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	DragToolHandler.Render3DDragTool(View, PDI);

	const UControlRigEditModeSettings* EditModeSettings = GetSettings();
	if (!IsDisplayControlsEnabled())
	{
		return;
	}

	FEditorViewportClient* EditorViewportClient = (FEditorViewportClient*)Viewport->GetClient();
	const bool bIsInGameView = !AreEditingControlRigDirectly() ? EditorViewportClient && EditorViewportClient->IsInGameView() : false;
	if (bIsInGameView)
	{
		//only draw stuff if not in game view
		return;
	}

	const EWorldType::Type WorldType = Viewport->GetClient()->GetWorld()->WorldType;
	const bool bIsEditorWorld = WorldType == EWorldType::Editor || WorldType == EWorldType::EditorPreview;
	
	const bool bIsAssetEditor =
		(WorldType == EWorldType::Editor || WorldType == EWorldType::EditorPreview) &&
		EditorViewportClient &&
		!EditorViewportClient->IsLevelEditorClient();
					
	for (TWeakObjectPtr<UControlRig>& ControlRigPtr : RuntimeControlRigs)
	{
		UControlRig* ControlRig = ControlRigPtr.Get();
		//actor game view drawing is handled by not drawing in game via SetActorHiddenInGame().
		if (ControlRig && ControlRig->GetControlsVisible())
		{
			FTransform ComponentTransform = FTransform::Identity;
			if (!AreEditingControlRigDirectly())
			{
				ComponentTransform = GetHostingSceneComponentTransform(ControlRig);
			}
		
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			const bool bHasFKRig = (ControlRig->IsA<UAdditiveControlRig>() || ControlRig->IsA<UFKControlRig>());

			if (EditModeSettings->bDisplayHierarchy || bHasFKRig)
			{
				const bool bBoolSetHitProxies = PDI && PDI->IsHitTesting() && bHasFKRig;
				TSet<FName> ActiveControlName;
				if (bHasFKRig)
				{
					ActiveControlName = GetActiveControlsFromSequencer(ControlRig);
				}
				Hierarchy->ForEach<FRigTransformElement>([PDI, Hierarchy, ComponentTransform, ControlRig, bHasFKRig, bBoolSetHitProxies, ActiveControlName](FRigTransformElement* TransformElement) -> bool
					{
						if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement))
						{
							if(ControlElement->Settings.AnimationType != ERigControlAnimationType::AnimationControl)
							{
								return true;
							}

							if (UModularRig* ModularRig = Cast<UModularRig>(ControlRig))
							{
								const FString ModuleName = Hierarchy->GetModuleName(ControlElement->GetKey());
								if (FRigModuleInstance* Module = ModularRig->FindModule(*ModuleName))
								{
									if (UControlRig* ModuleRig = Module->GetRig())
									{
										if (!ModuleRig->GetControlsVisible())
										{
											return true;
										}
									}
								}
							}
						}
				
						const FTransform Transform = Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentGlobal);

						FRigBaseElementParentArray Parents = Hierarchy->GetParents(TransformElement);
						for (FRigBaseElement* ParentElement : Parents)
						{
							if (FRigTransformElement* ParentTransformElement = Cast<FRigTransformElement>(ParentElement))
							{
								FLinearColor Color = FLinearColor::White;
								if (bHasFKRig)
								{
									FName ControlName = UFKControlRig::GetControlName(ParentTransformElement->GetFName(), ParentTransformElement->GetType());
									if (ActiveControlName.Num() > 0 && ActiveControlName.Contains(ControlName) == false)
									{
										continue;
									}
									if (ControlRig->IsControlSelected(ControlName))
									{
										Color = FLinearColor::Yellow;
									}
								}
								const FTransform ParentTransform = Hierarchy->GetTransform(ParentTransformElement, ERigTransformType::CurrentGlobal);
								const bool bHitTesting = bBoolSetHitProxies && (ParentTransformElement->GetType() == ERigElementType::Bone);
								if (PDI)
								{
									if (bHitTesting)
									{
										PDI->SetHitProxy(new HFKRigBoneProxy(ParentTransformElement->GetFName(), ControlRig));
									}
									PDI->DrawLine(ComponentTransform.TransformPosition(Transform.GetLocation()), ComponentTransform.TransformPosition(ParentTransform.GetLocation()), Color, SDPG_Foreground);
									if (bHitTesting)
									{
										PDI->SetHitProxy(nullptr);
									}
								}
							}
						}

						FLinearColor Color = FLinearColor::White;
						if (bHasFKRig)
						{
							FName ControlName = UFKControlRig::GetControlName(TransformElement->GetFName(), TransformElement->GetType());
							if (ActiveControlName.Num() > 0 && ActiveControlName.Contains(ControlName) == false)
							{
								return true;
							}
							if (ControlRig->IsControlSelected(ControlName))
							{
								Color = FLinearColor::Yellow;
							}
						}
						if (PDI)
						{
							const bool bHitTesting = PDI->IsHitTesting() && bBoolSetHitProxies && (TransformElement->GetType() == ERigElementType::Bone);
							if (bHitTesting)
							{
								PDI->SetHitProxy(new HFKRigBoneProxy(TransformElement->GetFName(), ControlRig));
							}
							PDI->DrawPoint(ComponentTransform.TransformPosition(Transform.GetLocation()), Color, 5.0f, SDPG_Foreground);

							if (bHitTesting)
							{
								PDI->SetHitProxy(nullptr);
							}
						}

						return true;
					});
			}

			const float AxisScale = GetAxesOnSelectionScale();

			if (bIsEditorWorld && (IsDisplayNullsEnabled() || ControlRig->IsConstructionModeEnabled()))
			{
				TArray<FTransform> SpaceTransforms;
				TArray<FTransform> SelectedSpaceTransforms;
				Hierarchy->ForEach<FRigNullElement>([&SpaceTransforms, &SelectedSpaceTransforms, Hierarchy](FRigNullElement* NullElement) -> bool
				{
					if (Hierarchy->IsSelected(NullElement->GetIndex()))
					{
						SelectedSpaceTransforms.Add(Hierarchy->GetTransform(NullElement, ERigTransformType::CurrentGlobal));
					}
					else
					{
						SpaceTransforms.Add(Hierarchy->GetTransform(NullElement, ERigTransformType::CurrentGlobal));
					}
					return true;
				});

				ControlRig->DrawInterface.DrawAxes(FTransform::Identity, SpaceTransforms, AxisScale);
				ControlRig->DrawInterface.DrawAxes(FTransform::Identity, SelectedSpaceTransforms, FLinearColor(1.0f, 0.34f, 0.0f, 1.0f), AxisScale);
			}

			const bool bDisplayAxesOnSelection = IsDisplayAxesOnSelectionEnabled();

			if (PDI &&
				bDisplayAxesOnSelection &&
				AxisScale > SMALL_NUMBER)
			{
				TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
				PDI->AddReserveLines(SDPG_Foreground, SelectedRigElements.Num() * 3);

				for (const FRigElementKey& SelectedElement : SelectedRigElements)
				{
					FTransform ElementTransform = Hierarchy->GetGlobalTransform(SelectedElement);
					ElementTransform = ElementTransform * ComponentTransform;

					PDI->DrawLine(ElementTransform.GetTranslation(), ElementTransform.TransformPosition(FVector(AxisScale, 0.f, 0.f)), FLinearColor::Red, SDPG_Foreground);
					PDI->DrawLine(ElementTransform.GetTranslation(), ElementTransform.TransformPosition(FVector(0.f, AxisScale, 0.f)), FLinearColor::Green, SDPG_Foreground);
					PDI->DrawLine(ElementTransform.GetTranslation(), ElementTransform.TransformPosition(FVector(0.f, 0.f, AxisScale)), FLinearColor::Blue, SDPG_Foreground);
				}
			}

			// temporary implementation to draw sockets in 3D
			if (PDI &&
				bIsEditorWorld &&
				(IsDisplaySocketsEnabled() || ControlRig->IsConstructionModeEnabled()) &&
				AxisScale > SMALL_NUMBER)
			{
				const float Scale = AxisScale;
				PDI->AddReserveLines(SDPG_Foreground, Hierarchy->Num(ERigElementType::Socket) * 3);
				static const FLinearColor SocketColor = FControlRigEditorStyle::Get().SocketUserInterfaceColor;

				Hierarchy->ForEach<FRigSocketElement>([this, Hierarchy, PDI, ComponentTransform, Scale](FRigSocketElement* Socket)
					{
						FTransform ElementTransform = Hierarchy->GetGlobalTransform(Socket->GetIndex());
						ElementTransform = ElementTransform * ComponentTransform;

						PDI->DrawLine(ElementTransform.GetTranslation(), ElementTransform.TransformPosition(FVector(Scale, 0.f, 0.f)), SocketColor, SDPG_Foreground);
						PDI->DrawLine(ElementTransform.GetTranslation(), ElementTransform.TransformPosition(FVector(0.f, Scale, 0.f)), SocketColor, SDPG_Foreground);
						PDI->DrawLine(ElementTransform.GetTranslation(), ElementTransform.TransformPosition(FVector(0.f, 0.f, Scale)), SocketColor, SDPG_Foreground);

						return true;
					});
			}
			
			ControlRig->DrawIntoPDI(PDI, ComponentTransform);
		}
	}
}

void FControlRigEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	IPersonaEditMode::DrawHUD(ViewportClient, Viewport, View, Canvas);
	DragToolHandler.RenderDragTool(View, Canvas);

	// Render any text instructions that were queued into the draw interfaces of the active rigs.
	// These get pushed by rigs via FRigVMDrawInterface::DrawText and can't be drawn via PDI.
	for (TWeakObjectPtr<UControlRig>& ControlRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = ControlRigPtr.Get())
		{
			FTransform ComponentTransform = FTransform::Identity;
			if (!AreEditingControlRigDirectly())
			{
				ComponentTransform = GetHostingSceneComponentTransform(ControlRig);
			}
			ControlRig->DrawIntoCanvas(Canvas, View, ComponentTransform);
		}
	}
}

bool FControlRigEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if(RuntimeControlRigs.IsEmpty())
	{
		return false;
	}

	if (IsMovingCamera(InViewport))
	{
		InViewportClient->SetCurrentWidgetAxis(EAxisList::None);
		return true;
	}
	if (IsDoingDrag(InViewport))
	{
		DragToolHandler.MakeDragTool(InViewportClient);
		return DragToolHandler.StartTracking(InViewportClient, InViewport);
	}

	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	if (CurrentAxis == EAxisList::None)
	{
		// not manipulating a required axis
		return false;
	}
	
	return HandleBeginTransform(InViewportClient);
}

bool FControlRigEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (RuntimeControlRigs.IsEmpty())
	{
		return false;
	}

	if (IsMovingCamera(InViewport))
	{
		return true;
	}
	if (DragToolHandler.EndTracking(InViewportClient, InViewport))
	{
		return true;
	}

	return HandleEndTransform(InViewportClient);
}

bool FControlRigEditMode::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform(Owner->GetFocusedViewportClient());
}

bool FControlRigEditMode::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform(Owner->GetFocusedViewportClient());
}

bool FControlRigEditMode::HandleBeginTransform(const FEditorViewportClient* InViewportClient)
{
	bOtherSelectedObjects = false;
	
	if (!InViewportClient)
	{
		return false;
	}

	// If moving in a control rig editor, but the rig is a module instance, moving controls is forbidden
	if (AreEditingControlRigDirectly())
	{
		for (TWeakObjectPtr<UControlRig> ControlRig : RuntimeControlRigs)
		{
			if (ControlRig->IsRigModuleInstance())
			{
				return false;
			}
		}
	}
	
	InteractionType = GetInteractionType(InViewportClient);
	bIsTracking = true;
	InteractionDependencies.Reset();

	const bool bDeferAutokeyOnMouseRelease = !bSequencerPlaying && IsInLevelEditor();
	Keyframer.Enable(bDeferAutokeyOnMouseRelease);
	
	if (InteractionScopes.Num() == 0)
	{
		const bool bShouldModify = [this]() -> bool
		{
			if (AreEditingControlRigDirectly())
			{
				for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
				{
					if (UControlRig* ControlRig = RuntimeRigPtr.Get())
					{
						TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
						for (const FRigElementKey& Key : SelectedRigElements)
						{
							if (Key.Type != ERigElementType::Control)
							{
								return true;
							}
						}
					}
				}
			}
			
			return !AreEditingControlRigDirectly();
		}();

		if (AreEditingControlRigDirectly())
		{
			for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
			{
				if (UControlRig* ControlRig = RuntimeRigPtr.Get())
				{
					UObject* Blueprint = ControlRig->GetClass()->ClassGeneratedBy;
					if (!Blueprint)
					{
						Blueprint = ControlRig->GetGeneratedByAsset().GetObject();
					}
					if (Blueprint)
					{
						Blueprint->SetFlags(RF_Transactional);
						if (bShouldModify)
						{
							Blueprint->Modify();
						}
					}
					ControlRig->SetFlags(RF_Transactional);
					if (bShouldModify)
					{
						ControlRig->Modify();
					}
				}
			}
		}

	}

	//in level editor only transact if we have at least one control selected, in editor we only select CR stuff so always transact

	if (!AreEditingControlRigDirectly())
	{
		if (OnGizmoInteractionStartedDelegate.IsBound())
		{
			FMultiControlRigElementSelection Selection;
			Selection.Rigs = RuntimeControlRigs;
			for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
			{
				if (UControlRig* ControlRig = RuntimeRigPtr.Get())
				{
					if (AreRigElementSelectedAndMovable(ControlRig))
					{
						Selection.KeysPerRig.Add(GetSelectedRigElements(ControlRig));
					}
				}
			}
			OnGizmoInteractionStartedDelegate.Broadcast(Selection, (EControlRigInteractionType)InteractionType);
		}
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				if (AreRigElementSelectedAndMovable(ControlRig))
				{
					//todo need to add multiple
					TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
					TUniquePtr<FControlRigInteractionScope> InteractionScope = MakeUnique<FControlRigInteractionScope>(
						ControlRig, SelectedRigElements, static_cast<EControlRigInteractionType>(InteractionType)
						);
					InteractionScopes.Emplace(ControlRig, MoveTemp(InteractionScope));
					ControlRig->bInteractionJustBegan = true;
				}
				else
				{
					bManipulatorMadeChange = false;
				}
			}
		}
	}
	else if(UControlRigEditorSettings::Get()->bEnableUndoForPoseInteraction)
	{
		UControlRig* ControlRig = RuntimeControlRigs[0].Get();
		if (OnGizmoInteractionStartedDelegate.IsBound())
		{
			FMultiControlRigElementSelection Selection;
			Selection.Rigs = {ControlRig};
			Selection.KeysPerRig.Add(GetSelectedRigElements(ControlRig));
			
			OnGizmoInteractionStartedDelegate.Broadcast(Selection, (EControlRigInteractionType)InteractionType);
		}
		
		TUniquePtr<FControlRigInteractionScope> InteractionScope = MakeUnique<FControlRigInteractionScope>(
			ControlRig, GetSelectedRigElements(ControlRig), static_cast<EControlRigInteractionType>(InteractionType)
			);
		InteractionScopes.Emplace(ControlRig, MoveTemp(InteractionScope));
	}
	else
	{
		bManipulatorMadeChange = false;
	}

	const bool bInteractingWithControls = !InteractionScopes.IsEmpty();
	
	// do not stop if there are other selected objects that are not related to the control rig mode
	// note that we are only interested in this value if controls are also being manipulated
	CacheAnyOtherSelectedObjects();
	CacheNonInteractingRigsToUpdate();
	
	bOtherSelectedObjects = bInteractingWithControls ? !SelectedObjects.IsEmpty() : false;
	
	return bOtherSelectedObjects ? false : bInteractingWithControls;
}

bool FControlRigEditMode::HandleEndTransform(FEditorViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return false;
	}

	const FControlRigInteractionTransformContext TransformContext(InViewportClient->GetWidgetMode());
	const bool bWasInteracting = bManipulatorMadeChange && InteractionType != (uint8)EControlRigInteractionType::None;
	const bool bHadOtherSelectedObjects = bOtherSelectedObjects;
	
	InteractionType = (uint8)EControlRigInteractionType::None;
	bIsTracking = false;
	InteractionDependencies.Reset();
	bOtherSelectedObjects = false;
	SelectedObjects.Reset();
	bUpdateNonInteractingRigs = false;
	NonInteractingRigs.Reset();
	
	if (InteractionScopes.Num() > 0)
	{		
		TArray<TWeakObjectPtr<UControlRig>> RigsToTick;
		RigsToTick.Reserve(InteractionScopes.Num());

		if (OnGizmoInteractionEndedDelegate.IsBound())
		{
			FMultiControlRigElementSelection Selection;
			TArray<UControlRig*> Rigs;
			InteractionScopes.GetKeys(Rigs);
			Selection.Rigs.Append(Rigs);
			for (UControlRig* ControlRig : Rigs)
			{
				if (AreRigElementSelectedAndMovable(ControlRig))
				{
					Selection.KeysPerRig.Add(GetSelectedRigElements(ControlRig));
				}
			}
			OnGizmoInteractionEndedDelegate.Broadcast(Selection);
		}
		
		for (const TPair<UControlRig*, TUniquePtr<FControlRigInteractionScope>>& RigInteractionScope : InteractionScopes)
		{
			if (UControlRig* ControlRig = RigInteractionScope.Key)
			{
				RigsToTick.Add(ControlRig);
			}

			if (const TUniquePtr<FControlRigInteractionScope>& InteractionScope = RigInteractionScope.Value)
			{
				Keyframer.Apply(*InteractionScope, TransformContext);
			}
		}

		Keyframer.Finalize(InViewportClient->GetWorld());
		Keyframer.Reset();

		// Keyframer may cause additional changes, such as auto-keying in Sequencer, etc. Those changes shoudl be part of the current transaction.
		if (bManipulatorMadeChange)
		{
			bManipulatorMadeChange = false;
			GEditor->EndTransaction();
		}
		
		InteractionScopes.Reset();

		if (bWasInteracting && !AreEditingControlRigDirectly())
		{
			// We invalidate the hit proxies when in level editor to ensure that the gizmo's hit proxy is up to date.
			// The invalidation is called here to avoid useless viewport update in the FControlRigEditMode::Tick
			// function (that does an update when not in level editor)
			TickManipulatableObjects(RigsToTick);
			
			static constexpr bool bInvalidateChildViews = false;
			static constexpr bool bInvalidateHitProxies = true;
			InViewportClient->Invalidate(bInvalidateChildViews, bInvalidateHitProxies);
		}

		// do not stop if there were other selected objects that were not related to the control rig mode
		return !bHadOtherSelectedObjects;
	}

	bManipulatorMadeChange = false;
	
	return false;
}

bool FControlRigEditMode::UsesTransformWidget() const
{
	for (const auto& Pairs : ControlRigShapeActors)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = Pairs.Key.Pin())
		{
			for (const AControlRigShapeActor* ShapeActor : Pairs.Value)
			{
				if (ShapeActor->IsSelected())
				{
					return true;
				}
			}
			if (AreRigElementSelectedAndMovable(ControlRig.Get()))
			{
				return true;
			}
		}
	}
	return FEdMode::UsesTransformWidget();
}

bool FControlRigEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	for (const auto& Pairs : ControlRigShapeActors)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = Pairs.Key.Pin())
		{
			for (const AControlRigShapeActor* ShapeActor : Pairs.Value)
			{
				if (ShapeActor->IsSelected())
				{
					return ModeSupportedByShapeActor(ShapeActor, CheckMode);
				}
			}
			if (AreRigElementSelectedAndMovable(ControlRig.Get()))
			{
				return true;
			}
		}
	}
	return FEdMode::UsesTransformWidget(CheckMode);
}

FVector FControlRigEditMode::GetWidgetLocation() const
{
	FVector PivotLocation(0.0, 0.0, 0.0);
	int NumSelected = 0;
	for (const auto& Pairs : ControlRigShapeActors)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = Pairs.Key.Pin())
		{
			if (AreRigElementSelectedAndMovable(ControlRig.Get()))
			{
				if (const FTransform* PivotTransform = PivotTransforms.Find(ControlRig.Get()))
				{
					// check that the cached pivot is up-to-date and update it if needed
					FTransform Transform = *PivotTransform;
					UpdatePivotTransformsIfNeeded(ControlRig.Get(), Transform);
					const FTransform ComponentTransform = GetHostingSceneComponentTransform(ControlRig.Get());
					PivotLocation += ComponentTransform.TransformPosition(Transform.GetLocation());
					++NumSelected;
				}
			}
		}
	}	
	if (NumSelected > 0)
	{
		PivotLocation /= (NumSelected);
		return PivotLocation;
	}

	return FEdMode::GetWidgetLocation();
}

bool FControlRigEditMode::GetPivotForOrbit(FVector& OutPivot) const
{
	constexpr bool bUseShape = true;
	if (IsControlSelected(bUseShape))
	{
		OutPivot = GetWidgetLocation();
		return true;
	}

	return FEdMode::GetPivotForOrbit(OutPivot);
}

bool FControlRigEditMode::GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	//since we strip translation just want the first one
	for (const auto& Pairs : ControlRigShapeActors)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = Pairs.Key.Pin())
		{
			if (AreRigElementSelectedAndMovable(ControlRig.Get()))
			{
				if (const FTransform* PivotTransform = PivotTransforms.Find(ControlRig.Get()))
				{
					// check that the cached pivot is up-to-date and update it if needed
					FTransform Transform = *PivotTransform;
					UpdatePivotTransformsIfNeeded(ControlRig.Get(), Transform);
					OutMatrix = Transform.ToMatrixNoScale().RemoveTranslation();
					return true;
				}
			}
		}
	}
	return false;
}

bool FControlRigEditMode::GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(OutMatrix, InData);
}

bool FControlRigEditMode::GetCursor(EMouseCursor::Type& OutCursor) const
{
	if (!GetOnlySelectRigControls())
	{
		return false;
	}

	const FEditorModeTools* const ModeManager = GetModeManager();
	const FEditorViewportClient* const HoveredViewportClient = ModeManager ? ModeManager->GetHoveredViewportClient() : nullptr;
	if (!HoveredViewportClient || !HoveredViewportClient->Viewport)
	{
		return false;
	}

	HHitProxy* const HitProxy = HoveredViewportClient->Viewport->GetHitProxy(
		HoveredViewportClient->Viewport->GetMouseX(),
		HoveredViewportClient->Viewport->GetMouseY()
	);
	if (!HitProxy)
	{
		OutCursor = EMouseCursor::Default;
		return true;
	}

	const bool bSelectable = IsViewportHitProxySelectable(HitProxy);
	if (!bSelectable)
	{
		OutCursor = EMouseCursor::Default;
		return true;
	}

	return false;
}

bool FControlRigEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click)
{
	const bool bClickSelectThroughGizmo = CVarClickSelectThroughGizmo.GetValueOnGameThread();
	//if Control is down we act like we are selecting an axis so don't do this check
	//if doing control else we can't do control selection anymore, see FMouseDeltaTracker::DetermineCurrentAxis(
	if (Click.IsControlDown() == false && bClickSelectThroughGizmo == false)
	{
		const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
		//if we are hitting a widget, besides arcball then bail saying we are handling it
		if (CurrentAxis != EAxisList::None)
		{
			return true;
		}
	}

	InteractionType = GetInteractionType(InViewportClient);

	if(HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy))
	{
		if(ActorHitProxy->Actor)
		{
			if (ActorHitProxy->Actor->IsA<AControlRigShapeActor>())
			{
				AControlRigShapeActor* ShapeActor = CastChecked<AControlRigShapeActor>(ActorHitProxy->Actor);
				if (ShapeActor->IsSelectable() && ShapeActor->ControlRig.IsValid())
				{
					FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);

					// temporarily disable the interaction scope
					const bool bInteractionScopePresent = InteractionScopes.Remove(ShapeActor->ControlRig.Get()) > 0;
					
					const FName& ControlName = ShapeActor->ControlName;
					if (Click.IsShiftDown()) //guess we just select
					{
						SetRigElementSelection(ShapeActor->ControlRig.Get(),ERigElementType::Control, ControlName, true);
					}
					else if(Click.IsControlDown()) //if ctrl we toggle selection
					{
						if (UControlRig* ControlRig = ShapeActor->ControlRig.Get())
						{
							bool bIsSelected = ControlRig->IsControlSelected(ControlName);
							SetRigElementSelection(ControlRig, ERigElementType::Control, ControlName, !bIsSelected);
						}
					}
					else
					{
						//also need to clear actor selection. Sequencer will handle this automatically if done in Sequencder UI but not if done by clicking
						if (!AreEditingControlRigDirectly())
						{
							if (GEditor && GEditor->GetSelectedActorCount())
							{
								GEditor->SelectNone(false, true);
								GEditor->NoteSelectionChange();
							}
							//also need to clear explicitly in sequencer
							if (WeakSequencer.IsValid())
							{
								if (ISequencer* SequencerPtr = WeakSequencer.Pin().Get())
								{
									SequencerPtr->GetViewModel()->GetSelection()->Empty();
								}
							}
						}
						ClearRigElementSelection(ValidControlTypeMask());
						SetRigElementSelection(ShapeActor->ControlRig.Get(),ERigElementType::Control, ControlName, true);
					}

					if (bInteractionScopePresent)
					{
						UControlRig* Rig = ShapeActor->ControlRig.Get();
						TUniquePtr<FControlRigInteractionScope> InteractionScope = MakeUnique<FControlRigInteractionScope>(
							Rig, GetSelectedRigElements(ShapeActor->ControlRig.Get()), static_cast<EControlRigInteractionType>(InteractionType)
							);
						InteractionScopes.Add(Rig, MoveTemp(InteractionScope));

					}

					// for now we show this menu all the time if body is selected
					// if we want some global menu, we'll have to move this
					if (Click.GetKey() == EKeys::RightMouseButton)
					{
						OpenContextMenu(InViewportClient);
					}
	
					return true;
				}

				return true;
			}
			else 
			{ 
				for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
				{
					if (UControlRig* ControlRig = RuntimeRigPtr.Get())
					{

						//if we have an additive or fk control rig active select the control based upon the selected bone.
						UAdditiveControlRig* AdditiveControlRig = Cast<UAdditiveControlRig>(ControlRig);
						UFKControlRig* FKControlRig = Cast<UFKControlRig>(ControlRig);

						if ((AdditiveControlRig || FKControlRig) && ControlRig->GetObjectBinding().IsValid())
						{
							if (USkeletalMeshComponent* RigMeshComp = Cast<USkeletalMeshComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
							{
								const USkeletalMeshComponent* SkelComp = Cast<USkeletalMeshComponent>(ActorHitProxy->PrimComponent);

								if (SkelComp == RigMeshComp)
								{
									FHitResult Result(1.0f);
									bool bHit = RigMeshComp->LineTraceComponent(Result, Click.GetOrigin(), Click.GetOrigin() + Click.GetDirection() * ControlRigSelectionConstants::BodyTraceDistance, FCollisionQueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(), true));

									if (bHit)
									{
										FName ControlName(*(Result.BoneName.ToString() + TEXT("_CONTROL")));
										if (ControlRig->FindControl(ControlName))
										{
											FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);

											if (Click.IsShiftDown()) //guess we just select
											{
												SetRigElementSelection(ControlRig,ERigElementType::Control, ControlName, true);
											}
											else if (Click.IsControlDown()) //if ctrl we toggle selection
											{
												bool bIsSelected = ControlRig->IsControlSelected(ControlName);
												SetRigElementSelection(ControlRig, ERigElementType::Control, ControlName, !bIsSelected);
											}
											else
											{
												ClearRigElementSelection(ValidControlTypeMask());
												SetRigElementSelection(ControlRig,ERigElementType::Control, ControlName, true);
											}
											return true;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
	else if (HFKRigBoneProxy* FKBoneProxy = HitProxyCast<HFKRigBoneProxy>(HitProxy))
	{
		FName ControlName(*(FKBoneProxy->BoneName.ToString() + TEXT("_CONTROL")));
		if (FKBoneProxy->ControlRig->FindControl(ControlName))
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);

			if (Click.IsShiftDown()) //guess we just select
			{
				SetRigElementSelection(FKBoneProxy->ControlRig,ERigElementType::Control, ControlName, true);
			}
			else if (Click.IsControlDown()) //if ctrl we toggle selection
			{
				for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
				{
					if (UControlRig* ControlRig = RuntimeRigPtr.Get())
					{
						bool bIsSelected = ControlRig->IsControlSelected(ControlName);
						SetRigElementSelection(FKBoneProxy->ControlRig, ERigElementType::Control, ControlName, !bIsSelected);
					}
				}
			}
			else
			{
				ClearRigElementSelection(ValidControlTypeMask());
				SetRigElementSelection(FKBoneProxy->ControlRig,ERigElementType::Control, ControlName, true);
			}
			return true;
		}
	}
	else if (HPersonaBoneHitProxy* BoneHitProxy = HitProxyCast<HPersonaBoneHitProxy>(HitProxy))
	{
		if (RuntimeControlRigs.Num() > 0)
		{
			if (UControlRig* DebuggedControlRig = RuntimeControlRigs[0].Get())
			{
				URigHierarchy* Hierarchy = DebuggedControlRig->GetHierarchy();

				// Cache mapping?
				for (int32 Index = 0; Index < Hierarchy->Num(); Index++)
				{
					const FRigElementKey ElementToSelect = Hierarchy->GetKey(Index);
					if (ElementToSelect.Type == ERigElementType::Bone && ElementToSelect.Name == BoneHitProxy->BoneName)
					{
						if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
						{
							Hierarchy->GetController()->SelectElement(ElementToSelect, true);
						}
						else if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
						{
							const bool bSelect = !Hierarchy->IsSelected(ElementToSelect);
							Hierarchy->GetController()->SelectElement(ElementToSelect, bSelect);
						}
						else
						{
							TArray<FRigElementKey> NewSelection;
							NewSelection.Add(ElementToSelect);
							Hierarchy->GetController()->SetSelection(NewSelection);
						}
						return true;
					}
				}
			}
		}
	}
	else
	{
		InteractionType = (uint8)EControlRigInteractionType::None;
	}

	// for now we show this menu all the time if body is selected
	// if we want some global menu, we'll have to move this
	if (Click.GetKey() == EKeys::RightMouseButton)
	{
		OpenContextMenu(InViewportClient);
		return true;
	}

	
	// clear selected controls
	if (Click.IsShiftDown() ==false && Click.IsControlDown() == false)
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);
		ClearRigElementSelection(FRigElementTypeHelper::ToMask(ERigElementType::All));
	}

	const UControlRigEditModeSettings* Settings = GetSettings();
	if (Settings && Settings->bOnlySelectRigControls)
	{
		return true;
	}
	
	return FEdMode::HandleClick(InViewportClient, HitProxy, Click);
}

void FControlRigEditMode::OpenContextMenu(FEditorViewportClient* InViewportClient)
{
	TSharedPtr<FUICommandList> Commands = CommandBindings;
	if (OnContextMenuCommandsDelegate.IsBound())
	{
		Commands = OnContextMenuCommandsDelegate.Execute();
	}

	if (OnGetContextMenuDelegate.IsBound())
	{
		TSharedPtr<SWidget> MenuWidget = SNullWidget::NullWidget;
		
		if (UToolMenu* ContextMenu = OnGetContextMenuDelegate.Execute())
		{
			UToolMenus* ToolMenus = UToolMenus::Get();
			MenuWidget = ToolMenus->GenerateWidget(ContextMenu);
		}

		TSharedPtr<SWidget> ParentWidget = InViewportClient->GetEditorViewportWidget();

		if (MenuWidget.IsValid() && ParentWidget.IsValid())
		{
			const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

			FSlateApplication::Get().PushMenu(
				ParentWidget.ToSharedRef(),
				FWidgetPath(),
				MenuWidget.ToSharedRef(),
				MouseCursorLocation,
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
		}
	}
}


bool IntersectsBox( AActor& InActor, const FBox& InBox, FLevelEditorViewportClient* LevelViewportClient, bool bUseStrictSelection )
{
	bool bActorHitByBox = false;
	if (InActor.IsHiddenEd())
	{
		return false;
	}

	const TArray<FName>& HiddenLayers = LevelViewportClient->ViewHiddenLayers;
	bool bActorIsVisible = true;
	for ( auto Layer : InActor.Layers )
	{
		// Check the actor isn't in one of the layers hidden from this viewport.
		if( HiddenLayers.Contains( Layer ) )
		{
			return false;
		}
	}

	// Iterate over all actor components, selecting out primitive components
	for (UActorComponent* Component : InActor.GetComponents())
	{
		UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
		if (PrimitiveComponent && PrimitiveComponent->IsRegistered() && PrimitiveComponent->IsVisibleInEditor())
		{
			if (PrimitiveComponent->IsShown(LevelViewportClient->EngineShowFlags) && PrimitiveComponent->ComponentIsTouchingSelectionBox(InBox, false, bUseStrictSelection))
			{
				return true;
			}
		}
	}
	
	return false;
}

bool FControlRigEditMode::BoxSelect(FBox& InBox, bool InSelect)
{
	const UControlRigEditModeSettings* Settings = GetSettings();
	FLevelEditorViewportClient* LevelViewportClient = GCurrentLevelEditingViewportClient;
	if (LevelViewportClient->IsInGameView() == true || !IsDisplayControlsEnabled())
	{
		return  FEdMode::BoxSelect(InBox, InSelect);
	}
	const bool bStrictDragSelection = GetDefault<ULevelEditorViewportSettings>()->bStrictBoxSelection;

	FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);
	const bool bShiftDown = LevelViewportClient->Viewport->KeyState(EKeys::LeftShift) || LevelViewportClient->Viewport->KeyState(EKeys::RightShift);
	if (!bShiftDown)
	{
		ClearRigElementSelection(ValidControlTypeMask());
	}

	// Select all actors that are within the selection box area.  Be aware that certain modes do special processing below.
	TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>> ControlsToSelect;
	for (const auto& [WeakControlRig, ShapeActors]: ControlRigShapeActors)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
		{
			if (ControlRig && ControlRig->GetControlsVisible())
			{
				for (AControlRigShapeActor* ShapeActor : ShapeActors)
				{
					const bool bTreatShape = ShapeActor && ShapeActor->IsSelectable() && !ShapeActor->IsTemporarilyHiddenInEditor();
					if (!bTreatShape)
					{
						continue;
					}

					const bool bPivotInBox = CVarMarqueePivotSelect.GetValueOnGameThread()
						&& InBox.IsInsideOrOn(ShapeActor->GetActorLocation());

					if (bPivotInBox || IntersectsBox(*ShapeActor, InBox, LevelViewportClient, bStrictDragSelection))
					{
						TArray<FRigElementKey>& Controls = ControlsToSelect.FindOrAdd(ControlRig.Get());
						Controls.Add(ShapeActor->GetElementKey());
					}
				}
			}
		}
	}

	const bool bSomethingSelected = !ControlsToSelect.IsEmpty();
	if (bSomethingSelected)
	{
		static constexpr bool bSelected = true;
		SetRigElementsSelectionInternal(ControlsToSelect, bSelected);
		return true;
	}
	
	ScopedTransaction.Cancel();
	//if only selecting controls return true to stop any more selections
	if (Settings && Settings->bOnlySelectRigControls)
	{
		return true;
	}
	return FEdMode::BoxSelect(InBox, InSelect);
}

bool FControlRigEditMode::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect)
{
	const UControlRigEditModeSettings* Settings = GetSettings();
	if (!Settings)
	{
		return false;
	}
	
	//need to check for a zero frustum since ComponentIsTouchingSelectionFrustum will return true, selecting everything, when this is the case
	// cf. FDragTool_ActorFrustumSelect::CalculateFrustum 
	const bool bAreTopBottomMalformed = InFrustum.Planes[0].IsNearlyZero() && InFrustum.Planes[2].IsNearlyZero();
	const bool bAreRightLeftMalformed = InFrustum.Planes[1].IsNearlyZero() && InFrustum.Planes[3].IsNearlyZero();
	const bool bMalformedFrustum = bAreTopBottomMalformed || bAreRightLeftMalformed;
	if (bMalformedFrustum || InViewportClient->IsInGameView() == true || !IsDisplayControlsEnabled())
	{
		return Settings->bOnlySelectRigControls;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);
	const bool bShiftDown = InViewportClient->Viewport->KeyState(EKeys::LeftShift) || InViewportClient->Viewport->KeyState(EKeys::RightShift);
	if (!bShiftDown)
	{
		ClearRigElementSelection(ValidControlTypeMask());
	}

	TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>> ControlsToSelect;

	if (CVarMarqueePivotSelect.GetValueOnGameThread())
	{
		for (const auto& [WeakControlRig, ShapeActors] : ControlRigShapeActors)
		{
			if (TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
			{
				if (ControlRig && ControlRig->GetControlsVisible())
				{
					for (AControlRigShapeActor* ShapeActor : ShapeActors)
					{
						if (!ShapeActor || !ShapeActor->IsSelectable() || ShapeActor->IsTemporarilyHiddenInEditor())
						{
							continue;
						}
						if (InFrustum.IntersectPoint(ShapeActor->GetActorLocation()))
						{
							ControlsToSelect.FindOrAdd(ControlRig.Get()).AddUnique(ShapeActor->GetElementKey());
						}
					}
				}
			}
		}
	}

	FSelectionHelper SelectionHelper(InViewportClient, ControlRigShapeActors, ControlsToSelect);
	SelectionHelper.GetFromFrustum(InFrustum);

	bool bSomethingSelected = !ControlsToSelect.IsEmpty();
	if (bSomethingSelected)
	{
		static constexpr bool bSelected = true;
		SetRigElementsSelectionInternal(ControlsToSelect, bSelected);
	}

	EWorldType::Type WorldType = InViewportClient->GetWorld()->WorldType;
	const bool bIsAssetEditor =
		(WorldType == EWorldType::Editor || WorldType == EWorldType::EditorPreview) &&
			!InViewportClient->IsLevelEditorClient();

	if (bIsAssetEditor)
	{
		float BoneRadius = 1;
		EBoneDrawMode::Type BoneDrawMode = EBoneDrawMode::None;
		if (const FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(InViewportClient))
		{
			BoneDrawMode = AnimViewportClient->GetBoneDrawMode();
			BoneRadius = AnimViewportClient->GetBoneDrawSize();
		}

		if(BoneDrawMode != EBoneDrawMode::None)
		{
			for(TWeakObjectPtr<UControlRig> WeakControlRig : RuntimeControlRigs)
			{
				if(UControlRig* ControlRig = WeakControlRig.Get())
				{
					if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
					{
						TArray<FRigBoneElement*> Bones = Hierarchy->GetBones();
						for(int32 Index = 0; Index < Bones.Num(); Index++)
						{
							const int32 BoneIndex = Bones[Index]->GetIndex();
							const TArray<int32> Children = Hierarchy->GetChildren(BoneIndex);

							const FVector Start = Hierarchy->GetGlobalTransform(BoneIndex).GetLocation();

							if(InFrustum.IntersectSphere(Start, 0.1f * BoneRadius))
							{
								bSomethingSelected = true;
								SetRigElementSelection(ControlRig, ERigElementType::Bone, Bones[Index]->GetFName(), true);
								continue;
							}

							bool bSelectedBone = false;
							for(int32 ChildIndex : Children)
							{
								if(Hierarchy->Get(ChildIndex)->GetType() != ERigElementType::Bone)
								{
									continue;
								}
								
								const FVector End = Hierarchy->GetGlobalTransform(ChildIndex).GetLocation();

								const float BoneLength = (End - Start).Size();
								const float Radius = FMath::Max(BoneLength * 0.05f, 0.1f) * BoneRadius;
								const int32 Steps = FMath::CeilToInt(BoneLength / (Radius * 1.5f) + 0.5);
								const FVector Step = (End - Start) / FVector::FReal(Steps - 1);

								// intersect segment-wise along the bone
								FVector Position = Start;
								for(int32 StepIndex = 0; StepIndex < Steps; StepIndex++)
								{
									if(InFrustum.IntersectSphere(Position, Radius))
									{
										bSomethingSelected = true;
										bSelectedBone = true;
										SetRigElementSelection(ControlRig, ERigElementType::Bone, Bones[Index]->GetFName(), true);
										break;
									}
									Position += Step;
								}

								if(bSelectedBone)
								{
									break;
								}
							}
						}
					}
				}
			}
		}

		for(TWeakObjectPtr<UControlRig> WeakControlRig : RuntimeControlRigs)
		{
			if(UControlRig* ControlRig = WeakControlRig.Get())
			{
				if (IsDisplayNullsEnabled() || ControlRig->IsConstructionModeEnabled())
				{
					if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
					{
						TArray<FRigNullElement*> Nulls = Hierarchy->GetNulls();
						for(int32 Index = 0; Index < Nulls.Num(); Index++)
						{
							const int32 NullIndex = Nulls[Index]->GetIndex();

							const FTransform Transform = Hierarchy->GetGlobalTransform(NullIndex);
							const FVector Origin = Transform.GetLocation();
							const float MaxScale = Transform.GetMaximumAxisScale();

							const float AxisScale = GetAxesOnSelectionScale();
							if(InFrustum.IntersectSphere(Origin, MaxScale * AxisScale))
							{
								bSomethingSelected = true;
								SetRigElementSelection(ControlRig, ERigElementType::Null, Nulls[Index]->GetFName(), true);
							}
						}
					}
				}
			}
		}
	}

	if (bSomethingSelected == true)
	{
		return true;
	}
	
	ScopedTransaction.Cancel();
	//if only selecting controls return true to stop any more selections
	if (Settings->bOnlySelectRigControls)
	{
		return true;
	}
	return FEdMode::FrustumSelect(InFrustum, InViewportClient, InSelect);
}

void FControlRigEditMode::SelectNone()
{
	ClearRigElementSelection(FRigElementTypeHelper::ToMask(ERigElementType::All));

	FEdMode::SelectNone();
}

bool FControlRigEditMode::IsMovingCamera(const FViewport* InViewport) const
{
	const bool LeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
	const bool bIsAltKeyDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	return LeftMouseButtonDown && bIsAltKeyDown;
}

bool FControlRigEditMode::IsDoingDrag(const FViewport* InViewport) const
{
	if(!UControlRigEditorSettings::Get()->bLeftMouseDragDoesMarquee)
	{
		return false;
	}

	if (Owner && Owner->GetInteractiveToolsContext()->InputRouter->HasActiveMouseCapture())
	{
		// don't start dragging if the ITF handled tracking event first   
		return false;
	}
	
	const bool LeftMouseButtonDown = InViewport->KeyState(EKeys::LeftMouseButton);
	const bool bIsCtrlKeyDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bIsAltKeyDown = InViewport->KeyState(EKeys::LeftAlt) || InViewport->KeyState(EKeys::RightAlt);
	const EAxisList::Type CurrentAxis = GetCurrentWidgetAxis();
	
	//if shift is down we still want to drag
	return LeftMouseButtonDown && (CurrentAxis == EAxisList::None) && !bIsCtrlKeyDown && !bIsAltKeyDown;
}

bool FControlRigEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (IsDoingDrag(InViewport))
	{
		return DragToolHandler.InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
	}

	const bool bCtrlDown = InViewport->KeyState(EKeys::LeftControl) || InViewport->KeyState(EKeys::RightControl);
	const bool bShiftDown = InViewport->KeyState(EKeys::LeftShift) || InViewport->KeyState(EKeys::RightShift);

	//button down if left and ctrl and right is down, needed for indirect posting

	// enable MMB with the new TRS gizmos
	const bool bEnableMMB = UEditorInteractiveGizmoManager::AreEditorGizmosAllowed(InViewportClient->GetModeTools());
	
	const bool bMouseButtonDown =
		InViewport->KeyState(EKeys::LeftMouseButton) ||
		(bCtrlDown && InViewport->KeyState(EKeys::RightMouseButton)) ||
		bEnableMMB;

	const UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	const EControlRigInteractionTransformSpace TransformSpace = GetTransformSpace();
	
	FControlRigInteractionTransformContext TransformContext;
	TransformContext.bTranslation = !InDrag.IsZero() && (WidgetMode == UE::Widget::WM_Translate || WidgetMode == UE::Widget::WM_TranslateRotateZ);
	TransformContext.Drag = InDrag;
	TransformContext.bRotation = !InRot.IsZero() && (WidgetMode == UE::Widget::WM_Rotate || WidgetMode == UE::Widget::WM_TranslateRotateZ);
	TransformContext.Rot = InRot;
	TransformContext.bScale = !InScale.IsZero() && WidgetMode == UE::Widget::WM_Scale;
	TransformContext.Scale = InScale;
	TransformContext.Space = TransformSpace;

	const bool EnableLocalTransform = [Settings = GetSettings(), TransformSpace, TransformContext]()
	{
		if (Settings && Settings->bLocalTransformsInEachLocalSpace)
		{
			switch (TransformSpace)
			{
			case EControlRigInteractionTransformSpace::World:
				return false;
			case EControlRigInteractionTransformSpace::Local:
			case EControlRigInteractionTransformSpace::Parent:
				return true;
			case EControlRigInteractionTransformSpace::Explicit:
				return TransformContext.bTranslation;
			default:
				break;
			}
		}
		return false;
	}();

	auto GatherSelectedKeys = [&]() -> FMultiControlRigElementSelection
	{
		FMultiControlRigElementSelection MultiRigSelection;
		for (TPair<TWeakObjectPtr<UControlRig>, TArray<TObjectPtr<AControlRigShapeActor>>>& Pairs : ControlRigShapeActors)
		{
			if (TStrongObjectPtr<UControlRig> ControlRig = Pairs.Key.Pin())
			{
				MultiRigSelection.Rigs.Add(Pairs.Key);
				FRigElementKeyCollection& Keys = MultiRigSelection.KeysPerRig.Add_GetRef({});
				if (AreRigElementsSelected(ValidControlTypeMask(), ControlRig.Get()))
				{
					for (TObjectPtr<AControlRigShapeActor> ShapeActor : Pairs.Value)
					{
						if (ShapeActor->IsEnabled() && ShapeActor->IsSelected())
						{
							Keys.Add(ShapeActor->GetElementKey());
						}
					}
				}
			}
		}
		return MultiRigSelection;
	};

	if (InteractionScopes.Num() > 0 && bMouseButtonDown && CurrentAxis != EAxisList::None
		&& TransformContext.CanTransform())
	{
		if (bSequencerPlaying)
		{
			// reset the dependency cache as the hierarchy might have changed since the previous frame
			InteractionDependencies.Reset();
		}

		// The interaction update event must be broadcasted before setting any control value or executing the rig
		if (OnGizmoInteractionPreUpdatedDelegate.IsBound())
		{
			FMultiControlRigElementSelection MultiRigSelection = GatherSelectedKeys();
			OnGizmoInteractionPreUpdatedDelegate.Broadcast(MultiRigSelection, TransformContext);
		}
		
		for (auto& Pairs : ControlRigShapeActors)
		{
			if (TStrongObjectPtr<UControlRig> ControlRig = Pairs.Key.Pin())
			{
				if (AreRigElementsSelected(ValidControlTypeMask(), ControlRig.Get()))
				{
					FTransform ComponentTransform = GetHostingSceneComponentTransform(ControlRig.Get());

					if (bIsChangingControlShapeTransform)
					{
						for (AControlRigShapeActor* ShapeActor : Pairs.Value)
						{
							if (ShapeActor->IsSelected())
							{
								if (bManipulatorMadeChange == false)
								{
									GEditor->BeginTransaction(LOCTEXT("ChangeControlShapeTransaction", "Change Control Shape Transform"));
								}

								ChangeControlShapeTransform(ShapeActor, TransformContext, ComponentTransform);
								bManipulatorMadeChange = true;

								// break here since we only support changing shape transform of a single control at a time
								break;
							}
						}
					}
					else
					{
						bool bDoLocal = EnableLocalTransform;
						bool bUseLocal = false;
						bool bCalcLocal = bDoLocal;
						bool bFirstTime = true;
						FTransform InOutLocal = FTransform::Identity;
					
						bool const bJustStartedManipulation = !bManipulatorMadeChange;
						const bool bAnyAdditiveRig = ControlRig->IsAdditive();

						TMap<AControlRigShapeActor*, TArray<TFunction<void()>>> TasksPerActor;
						for (AControlRigShapeActor* ShapeActor : Pairs.Value)
						{
							if (ShapeActor->IsEnabled() && ShapeActor->IsSelected())
							{
								// test local vs global
								if (bManipulatorMadeChange == false)
								{
									GEditor->BeginTransaction(LOCTEXT("MoveControlTransaction", "Move Control"));
								}

								// cache interaction dependencies + evaluate the rig at least once before manipulating anything
								if (ControlRig->ElementsBeingInteracted.IsEmpty() && InteractionScopes.Contains(ControlRig.Get()))
								{
									GetInteractionDependencies(ControlRig.Get());
									EvaluateRig(ControlRig.Get());
								}

								// Cannot benefit of same local transform when applying to additive rigs
								if (!bAnyAdditiveRig)
								{
									if (bFirstTime)
									{
										bFirstTime = false;
									}
									else
									{
										if (bDoLocal)
										{
											bUseLocal = true;
											bDoLocal = false;
										}
									}
								}

								if(bJustStartedManipulation)
								{
									if(const FRigControlElement* ControlElement = ControlRig->FindControl(ShapeActor->ControlName))
									{
										ShapeActor->OffsetTransform = ControlRig->GetHierarchy()->GetGlobalControlOffsetTransform(ControlElement->GetKey(), false);
									}
								}

								TArray<TFunction<void()>> Tasks;
								MoveControlShape(ShapeActor, TransformContext, ComponentTransform, bUseLocal, bDoLocal, &InOutLocal, Tasks);
								TasksPerActor.Add(ShapeActor, Tasks);
								bManipulatorMadeChange = true;
							}
						}

						{
							// evaluate the rig(s) before the tasks
							const FPendingControlRigEvaluator _(this);
						}

						// process remaining tasks
						while(!TasksPerActor.IsEmpty())
						{
							// place another evaluator here which is going to run after the tasks for this phase
							const FPendingControlRigEvaluator _(this);

							// run one tasks for each control - until there are no tasks lefts
							TArray<AControlRigShapeActor*> KeysToRemove;
							for(TPair<AControlRigShapeActor*, TArray<TFunction<void()>>>& Pair : TasksPerActor)
							{
								if(!Pair.Value.IsEmpty())
								{
									TFunction<void()> Task = Pair.Value[0];
									Pair.Value.RemoveAt(0);
									Task();
								}
								else
								{
									KeysToRemove.Add(Pair.Key);
								}
							}

							for(AControlRigShapeActor* KeyToRemove : KeysToRemove)
							{
								TasksPerActor.Remove(KeyToRemove);
							}
						}
					}
				}
				else if (AreRigElementSelectedAndMovable(ControlRig.Get()))
				{
					FTransform ComponentTransform = GetHostingSceneComponentTransform(ControlRig.Get());

					// set Bone transform
					// that will set initial Bone transform
					TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig.Get());

					for (int32 Index = 0; Index < SelectedRigElements.Num(); ++Index)
					{
						const ERigElementType SelectedRigElementType = SelectedRigElements[Index].Type;

						if (SelectedRigElementType == ERigElementType::Control)
						{
							FTransform NewWorldTransform = OnGetRigElementTransformDelegate.Execute(SelectedRigElements[Index], false, true) * ComponentTransform;
							bool bTransformChanged = false;
							if (TransformContext.bRotation)
							{
								FQuat CurrentRotation = NewWorldTransform.GetRotation();
								CurrentRotation = (TransformContext.Rot.Quaternion() * CurrentRotation);
								NewWorldTransform.SetRotation(CurrentRotation);
								bTransformChanged = true;
							}

							if (TransformContext.bTranslation)
							{
								FVector CurrentLocation = NewWorldTransform.GetLocation();
								CurrentLocation = CurrentLocation + TransformContext.Drag;
								NewWorldTransform.SetLocation(CurrentLocation);
								bTransformChanged = true;
							}

							if (TransformContext.bScale)
							{
								FVector CurrentScale = NewWorldTransform.GetScale3D();
								CurrentScale = CurrentScale + TransformContext.Scale;
								NewWorldTransform.SetScale3D(CurrentScale);
								bTransformChanged = true;
							}

							if (bTransformChanged)
							{
								if (bManipulatorMadeChange == false)
								{
									GEditor->BeginTransaction(LOCTEXT("MoveControlTransaction", "Move Control"));
								}
								FTransform NewComponentTransform = NewWorldTransform.GetRelativeTransform(ComponentTransform);
								OnSetRigElementTransformDelegate.Execute(SelectedRigElements[Index], NewComponentTransform, false);
								bManipulatorMadeChange = true;
							}
						}
					}
				}
			}
		}
	}

	UpdatePivotTransforms();

	if (OnGizmoInteractionPostUpdatedDelegate.IsBound())
		{
			FMultiControlRigElementSelection MultiRigSelection = GatherSelectedKeys();
			OnGizmoInteractionPostUpdatedDelegate.Broadcast(MultiRigSelection, TransformContext);
		}

	if (bManipulatorMadeChange)
	{
		TArray<TWeakObjectPtr<UControlRig>> RigsToTick;
		RigsToTick.Reserve(InteractionScopes.Num());
		Algo::Transform(InteractionScopes, RigsToTick, [](const TPair<UControlRig*, TUniquePtr<FControlRigInteractionScope>>& InteractionData)
		{
			return InteractionData.Key;
		});
		
		RigsToTick.Append(NonInteractingRigs);

		// mark non-interacting constraints for evaluation
		for (const TWeakObjectPtr<UControlRig>& NonInteractingRig: NonInteractingRigs)
		{
			UControlRig* Rig = NonInteractingRig.Get();
			if (USceneComponent* BoundComponent = Rig ? GetHostingSceneComponent(Rig) : nullptr)
			{
				for (const TWeakObjectPtr<UTickableConstraint>& WeakConstraint: ConstraintsCache.Get(BoundComponent, WorldPtr))
				{
					if (UTickableConstraint* Constraint = WeakConstraint.Get(); Constraint->Active)
					{
						FConstraintsManagerController::Get(WorldPtr).MarkConstraintForEvaluation(Constraint);
					}
				}
			}
		}
		FConstraintsManagerController::Get(WorldPtr).FlushEvaluationGraph();
		
		TickManipulatableObjects(RigsToTick);
	}

	DirectControl.bHasTransformChanged = bOtherSelectedObjects || bManipulatorMadeChange; 
	
	// do not stop if there are other selected objects that are not related to the control rig mode
	return bOtherSelectedObjects ? false : bManipulatorMadeChange;
}

bool FControlRigEditMode::ShouldDrawWidget() const
{
	for (const TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			const TSharedPtr<IControlRigObjectBinding>& Binding = ControlRig->GetObjectBinding();
			if (Binding && Binding->GetBoundObject())
			{
				if (AreRigElementSelectedAndMovable(ControlRig))
				{
					return true;
				}
			}
		}
	}

	return FEdMode::ShouldDrawWidget();
}

bool FControlRigEditMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return OtherModeID == FName(TEXT("EM_SequencerMode"), FNAME_Find) || OtherModeID == FName(TEXT("MotionTrailEditorMode"), FNAME_Find); /*|| OtherModeID == FName(TEXT("EditMode.ControlRigEditor"), FNAME_Find);*/
}

void FControlRigEditMode::AddReferencedObjects( FReferenceCollector& Collector )
{
	for (auto& ShapeActors : ControlRigShapeActors)
	{
		for (auto& ShapeActor : ShapeActors.Value)
		{		
			Collector.AddReferencedObject(ShapeActor);
		}
	}
	
	Collector.AddReferencedObject(AnimDetailsProxyManager);
	
	if (StoredPose)
	{
		Collector.AddReferencedObject(StoredPose);
	}
}

void FControlRigEditMode::ClearRigElementSelection(uint32 InTypes)
{
	auto GetController = [bControlRigEditor = AreEditingControlRigDirectly()](UControlRig* InControlRig) -> URigHierarchyController*
	{
		if (InControlRig)
		{
			if (!bControlRigEditor)
			{
				if (URigHierarchy* Hierarchy = InControlRig->GetHierarchy())
				{
					return Hierarchy->GetController();
				}
			}
			else if (FControlRigAssetInterfacePtr Blueprint = InControlRig->GetClass()->ClassGeneratedBy)
			{
				return Blueprint->GetHierarchyController();
			}
			else if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = InControlRig->GetGeneratedByAsset().GetObject())
			{
				return RuntimeAsset->GetHierarchy()->GetController(true);
			}
		}

		return nullptr;
	};

	// put sequencer's selection changed listener on hold during selection to avoid a notifications storm
	// and only send it once the full selection has been done
	using namespace UE::Sequencer;
	TUniquePtr<FSelectionEventSuppressor> SequencerSelectionGuard;
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr)
	{
		TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
		if (TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
		{
			SequencerSelectionGuard.Reset(new FSelectionEventSuppressor(SequencerSelection.Get()));
		}
	}

	for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (URigHierarchyController* Controller = GetController(RuntimeRigPtr.Get()))
		{
			constexpr bool bSetupUndo = true;
			Controller->ClearSelection(bSetupUndo);
		}
	}
}

// internal private function that doesn't use guarding.
void FControlRigEditMode::SetRigElementSelectionInternal(UControlRig* ControlRig, ERigElementType Type, const FName& InRigElementName, bool bSelected)
{
	if(URigHierarchyController* Controller = ControlRig->GetHierarchy()->GetController())
	{
		constexpr bool bSetupUndo = true;
		Controller->SelectElement(FRigElementKey(InRigElementName, Type), bSelected, false, bSetupUndo);
	}
}

void FControlRigEditMode::SetRigElementSelection(UControlRig* ControlRig, ERigElementType Type, const FName& InRigElementName, bool bSelected)
{
	if (!bSelecting)
	{
		TGuardValue<bool> ReentrantGuard(bSelecting, true);

		SetRigElementSelectionInternal(ControlRig,Type, InRigElementName, bSelected);

		HandleSelectionChanged();
	}
}

void FControlRigEditMode::SetRigElementSelection(UControlRig* ControlRig, ERigElementType Type, const TArray<FName>& InRigElementNames, bool bSelected)
{
	if (!bSelecting && ControlRig)
	{
		TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>> RigElementsToSelect;
		TArray<FRigElementKey>& ElementsToSelect = RigElementsToSelect.FindOrAdd(ControlRig);
		ElementsToSelect.Reserve(InRigElementNames.Num());
		Algo::Transform(InRigElementNames, ElementsToSelect, [Type](const FName& ElementName)
		{
			return FRigElementKey(ElementName, Type);
		});

		return SetRigElementsSelectionInternal(RigElementsToSelect, bSelected);
	}
}

void FControlRigEditMode::SetRigElementsSelectionInternal(const TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>>& InRigElementsToSelect, const bool bSelected)
{
	if (bSelecting || InRigElementsToSelect.IsEmpty())
	{
		return;
	}

	// put sequencer's selection changed listener on hold during selection to avoid a notifications storm
	// and only send it once the full selection has been done
	using namespace UE::Sequencer;
	TUniquePtr<FSelectionEventSuppressor> SequencerSelectionGuard;
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr)
	{
		TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
		if (TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
		{
			SequencerSelectionGuard.Reset(new FSelectionEventSuppressor(SequencerSelection.Get()));
		}
	}

	TGuardValue<bool> ReentrantGuard(bSelecting, true);
	
	for (const auto& [WeakControlRig, Elements]: InRigElementsToSelect)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
		{
			URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
			if (URigHierarchyController* Controller = Hierarchy ? Hierarchy->GetController() : nullptr)
			{
				for (const FRigElementKey& Element : Elements)
				{
					constexpr bool bSetupUndo = true;
					Controller->SelectElement(Element, bSelected, false, bSetupUndo);
				}
			}
		}
	}
	
	HandleSelectionChanged();
}

TArray<FRigElementKey> FControlRigEditMode::GetSelectedRigElements() const
{
	if (GetControlRigs().Num() > 0)
	{
		UControlRig* ControlRig = GetControlRigs()[0].Get();
		return GetSelectedRigElements(ControlRig);
	}
	return TArray<FRigElementKey>();

}

TArray<FRigElementKey> FControlRigEditMode::GetSelectedRigElements(UControlRig* ControlRig) 
{
	TArray<FRigElementKey> SelectedKeys;
	if (ControlRig == nullptr)
	{
		return SelectedKeys;
	}
	
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (Hierarchy == nullptr)
	{
		return SelectedKeys;
	}

	SelectedKeys = Hierarchy->GetSelectedKeys();

	// currently only 1 transient control is allowed at a time
	// Transient Control's bSelected flag is never set to true, probably to avoid confusing other parts of the system
	// But since Edit Mode directly deals with transient controls, its selection status is given special treatment here.
	// So basically, whenever a bone is selected, and there is a transient control present, we consider both selected.
	if (SelectedKeys.Num() == 1)
	{
		if (SelectedKeys[0].Type == ERigElementType::Bone || SelectedKeys[0].Type == ERigElementType::Null)
		{
			const FName ControlName = UControlRig::GetNameForTransientControl(SelectedKeys[0]);
			const FRigElementKey TransientControlKey = FRigElementKey(ControlName, ERigElementType::Control);
			if(Hierarchy->Contains(TransientControlKey))
			{
				SelectedKeys.Add(TransientControlKey);
			}

		}
	}
	else
	{
		// check if there is a pin value transient control active
		// when a pin control is active, all existing selection should have been cleared
		TArray<FRigControlElement*> TransientControls = Hierarchy->GetTransientControls();

		if (TransientControls.Num() > 0)
		{
			if (ensure(SelectedKeys.Num() == 0))
			{
				SelectedKeys.Add(TransientControls[0]->GetKey());
			}
		}
	}
	return SelectedKeys;
}

bool FControlRigEditMode::AreRigElementsSelected(uint32 InTypes, UControlRig* InControlRig) const
{
	if (IsInLevelEditor() && InControlRig)
	{
		// no need to look for transient controls when animating in the level editor
		const URigHierarchy* Hierarchy = InControlRig->GetHierarchy();
		if (!Hierarchy)
		{
			return false;
		}
			
		return Hierarchy->HasAnythingSelectedByPredicate([InTypes](const FRigElementKey& InSelectedKey)
		{
			return FRigElementTypeHelper::DoesHave(InTypes, InSelectedKey.Type);
		});
	}
	if (InControlRig == nullptr && GetControlRigs().Num() > 0)
	{
		InControlRig = GetControlRigs()[0].Get();
	}
	const TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(InControlRig);
	for (const FRigElementKey& Ele : SelectedRigElements)
	{
		if (FRigElementTypeHelper::DoesHave(InTypes, Ele.Type))
		{
			return true;
		}
	}

	return false;
}

void FControlRigEditMode::RefreshObjects()
{
	SetObjects_Internal();
}

bool FControlRigEditMode::CanRemoveFromPreviewScene(const USceneComponent* InComponent)
{
	for (auto& ShapeActors : ControlRigShapeActors)
	{
		for (auto& ShapeActor : ShapeActors.Value)
		{
			TInlineComponentArray<USceneComponent*> SceneComponents;
			ShapeActor->GetComponents(SceneComponents, true);
			if (SceneComponents.Contains(InComponent))
			{
				return false;
			}
		}
	}

	// we don't need it 
	return true;
}

ECoordSystem FControlRigEditMode::GetCoordSystemSpace() const
{
	const UControlRigEditModeSettings* Settings = GetSettings();
	if (Settings && Settings->bCoordSystemPerWidgetMode)
	{
		const int32 WidgetMode = static_cast<int32>(GetModeManager()->GetWidgetMode());
		if (CoordSystemPerWidgetMode.IsValidIndex(WidgetMode))
		{
			return CoordSystemPerWidgetMode[WidgetMode];
		}
	}

	return GetModeManager()->GetCoordSystem();	
}

bool FControlRigEditMode::ComputePivotFromEditedShape(UControlRig* InControlRig, FTransform& OutTransform) const
{
	const URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return false;
	}

	if (!ensure(bIsChangingControlShapeTransform))
	{
		return false;
	}
	
	OutTransform = FTransform::Identity;
	
	if (auto* ShapeActors = ControlRigShapeActors.Find(InControlRig))
	{
		// we just want to change the shape transform of one single control.
		const int32 Index = ShapeActors->IndexOfByPredicate([](const TObjectPtr<AControlRigShapeActor>& ShapeActor)
		{
			return IsValid(ShapeActor) && ShapeActor->IsSelected();
		});

		if (Index != INDEX_NONE)
		{
			if (FRigControlElement* ControlElement = InControlRig->FindControl((*ShapeActors)[Index]->ControlName))
			{
				OutTransform = Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentGlobal);
			}				
		}
	}
	
	return true;
}

EControlRigInteractionTransformSpace FControlRigEditMode::GetTransformSpace() const
{
	switch (GetCoordSystemSpace())
	{
		case COORD_World:
			return EControlRigInteractionTransformSpace::World;
		case COORD_Local:
			return EControlRigInteractionTransformSpace::Local;
		case COORD_Parent:
			return EControlRigInteractionTransformSpace::Parent;
		case COORD_Explicit:
			return EControlRigInteractionTransformSpace::Explicit;
		default:
			break;
	}
	return EControlRigInteractionTransformSpace::Local;	
};

FTransform FControlRigEditMode::GetPivotOrientation(
	const FRigElementKey& InControlKey, const UControlRig* InControlRig, URigHierarchy* InHierarchy,
	const EControlRigInteractionTransformSpace InSpace, const FTransform& InComponentTransform) const
{
	FRigControlElement* Control = InHierarchy->Find<FRigControlElement>(InControlKey);
	if (!Control)
	{
		return FTransform::Identity;
	}
		
	switch (InSpace)
	{
	case EControlRigInteractionTransformSpace::World:
	case EControlRigInteractionTransformSpace::Local:
		return InHierarchy->GetTransform(Control, ERigTransformType::CurrentGlobal);
	case EControlRigInteractionTransformSpace::Parent:
		{	
			if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(InControlRig, InControlKey.Name))
			{
				return ConstraintSpace->GetRelativeTransform(InComponentTransform);
			}
			
			const int32 NumParents = InHierarchy->GetNumberOfParents(Control);
			return NumParents > 0 ? InHierarchy->GetParentTransform(Control, ERigTransformType::CurrentGlobal) : InHierarchy->GetTransform(Control, ERigTransformType::CurrentGlobal);
		}
	case EControlRigInteractionTransformSpace::Explicit:
		{
			FRotationContext& RotationContext = GetRotationContext();
					
			const bool bRotating = GetModeManager()->GetWidgetMode() == UE::Widget::EWidgetMode::WM_Rotate;
			const bool bUsePreferredRotationOrder = InHierarchy->GetUsePreferredRotationOrder(Control);

			RotationContext.RotationOrder = bUsePreferredRotationOrder ? InHierarchy->GetControlPreferredEulerRotationOrder(Control) : EEulerRotationOrder::XYZ;
			RotationContext.Rotation = InHierarchy->GetControlPreferredRotator(Control);

			if (bRotating)
			{
				if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(InControlRig, InControlKey.Name))
				{
					RotationContext.Offset = *ConstraintSpace;
				}
				else
				{
					const FTransform Offset = InHierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentGlobal);
					RotationContext.Offset = Offset * InComponentTransform;					
				}
				
				return InHierarchy->GetTransform(Control, ERigTransformType::CurrentGlobal);
			}

			RotationContext.Offset = FTransform::Identity;
			if (IsInLevelEditor())
			{
				if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(InControlRig, InControlKey.Name))
				{
					return ConstraintSpace->GetRelativeTransform(InComponentTransform);
				}
			}
			return InHierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentGlobal);
		}
	default:
		break;
	}

	return FTransform::Identity;
}

bool FControlRigEditMode::ComputePivotFromShapeActors(UControlRig* InControlRig, const bool bEachLocalSpace, const EControlRigInteractionTransformSpace InSpace, FTransform& OutTransform) const
{
	if (!ensure(!bIsChangingControlShapeTransform))
	{
		return false;
	}
	
	URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return false;
	}
	const FTransform ComponentTransform = GetHostingSceneComponentTransform(InControlRig);

	FTransform LastTransform = FTransform::Identity, PivotTransform = FTransform::Identity;

	if (const auto* ShapeActors = ControlRigShapeActors.Find(InControlRig))
	{
		// if in local just use the first selected actor transform
		// otherwise, compute the average location as pivot location
		
		int32 NumSelectedControls = 0;
		FVector PivotLocation = FVector::ZeroVector;
		for (const TObjectPtr<AControlRigShapeActor>& ShapeActor : *ShapeActors)
		{
			if (IsValid(ShapeActor) && ShapeActor->IsSelected())
			{
				const FRigElementKey ControlKey = ShapeActor->GetElementKey();
				const FTransform ShapeTransform = ShapeActor->GetActorTransform().GetRelativeTransform(ComponentTransform);
				LastTransform = GetPivotOrientation(ControlKey, InControlRig, Hierarchy, InSpace, ComponentTransform);
				PivotLocation += ShapeTransform.GetLocation();
				
				++NumSelectedControls;
				if (bEachLocalSpace)
				{
					break;
				}
			}
		}

		if (NumSelectedControls > 1)
		{
			PivotLocation /= static_cast<double>(NumSelectedControls);
		}
		PivotTransform.SetLocation(PivotLocation);
	}

	// Use the last transform's rotation as pivot rotation
	const FTransform WorldTransform = LastTransform * ComponentTransform;
	PivotTransform.SetRotation(WorldTransform.GetRotation());
	
	OutTransform = PivotTransform;
	
	return true;
}

bool FControlRigEditMode::ComputePivotFromElements(UControlRig* InControlRig, FTransform& OutTransform) const
{
	if (!ensure(!bIsChangingControlShapeTransform))
	{
		return false;
	}
	
	const URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return false;
	}
	
	const FTransform ComponentTransform = GetHostingSceneComponentTransform(InControlRig);
	
	int32 NumSelection = 0;
	FTransform LastTransform = FTransform::Identity, PivotTransform = FTransform::Identity;
	FVector PivotLocation = FVector::ZeroVector;
	const TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(InControlRig);
	
	for (int32 Index = 0; Index < SelectedRigElements.Num(); ++Index)
	{
		if (SelectedRigElements[Index].Type == ERigElementType::Control)
		{
			LastTransform = OnGetRigElementTransformDelegate.Execute(SelectedRigElements[Index], false, true);
			PivotLocation += LastTransform.GetLocation();
			++NumSelection;
		}
	}

	if (NumSelection == 1)
	{
		// A single control just uses its own transform
		const FTransform WorldTransform = LastTransform * ComponentTransform;
		PivotTransform.SetRotation(WorldTransform.GetRotation());
	}
	else if (NumSelection > 1)
	{
		PivotLocation /= static_cast<double>(NumSelection);
		PivotTransform.SetRotation(ComponentTransform.GetRotation());
	}
		
	PivotTransform.SetLocation(PivotLocation);
	OutTransform = PivotTransform;

	return true;
}

void FControlRigEditMode::UpdatePivotTransforms()
{
	const UControlRigEditModeSettings* Settings = GetSettings();
	const bool bEachLocalSpace = Settings && Settings->bLocalTransformsInEachLocalSpace;

	PivotTransforms.Reset();

	for (const TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			bool bAdd = false;
			FTransform Pivot = FTransform::Identity;
			
			if (AreRigElementsSelected(ValidControlTypeMask(), ControlRig))
			{
				if (bIsChangingControlShapeTransform)
				{
					bAdd = ComputePivotFromEditedShape(ControlRig, Pivot);
				}
				else
				{
					bAdd = ComputePivotFromShapeActors(ControlRig, bEachLocalSpace, GetTransformSpace(), Pivot);			
				}
			}
			else if (AreRigElementSelectedAndMovable(ControlRig))
			{
				// do we even get in here ?!
				// we will enter the if first as AreRigElementsSelected will return true before AreRigElementSelectedAndMovable does...
				bAdd = ComputePivotFromElements(ControlRig, Pivot);
			}
			
			if (bAdd)
			{
				PivotTransforms.Add(ControlRig, MoveTemp(Pivot));
			}
		}
	}

	bPivotsNeedUpdate = false;

	//If in level editor and the transforms changed we need to force hit proxy invalidate so widget hit testing 
	//doesn't work off of it's last transform.  Similar to what sequencer does on re-evaluation but do to how edit modes and widget ticks happen
	//it doesn't work for control rig gizmo's
	if (IsInLevelEditor() && InteractionScopes.IsEmpty() && !bSequencerPlaying)
	{
		if (HasPivotTransformsChanged())
		{
			for (FEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
			{
				if (LevelVC)
				{
					if (!LevelVC->IsRealtime())
					{
						LevelVC->RequestRealTimeFrames(1);
					}

					if (LevelVC->Viewport)
					{
						LevelVC->Viewport->InvalidateHitProxy();
					}
				}
			}
		}
		LastPivotTransforms = PivotTransforms;
	}
}

void FControlRigEditMode::RequestTransformWidgetMode(UE::Widget::EWidgetMode InWidgetMode)
{
	RequestedWidgetModes.Add(InWidgetMode);
}

bool FControlRigEditMode::HasPivotTransformsChanged() const
{
	if (PivotTransforms.Num() != LastPivotTransforms.Num())
	{
		return true;
	}
	for (const TPair<UControlRig*, FTransform>& Transform : PivotTransforms)
	{
		if (const FTransform* LastTransform = LastPivotTransforms.Find(Transform.Key))
		{
			if (Transform.Value.Equals(*LastTransform, 1e-4f) == false)
			{
				return true;
			}
		}
		else
		{
			return true;
		}
	}
	return false;
}

void FControlRigEditMode::UpdatePivotTransformsIfNeeded(UControlRig* InControlRig, FTransform& InOutTransform) const
{
	if (!bPivotsNeedUpdate)
	{
		return;
	}

	if (!InControlRig || !InControlRig->GetHierarchy())
	{
		return;
	}

	// Update shape actors transforms
	if (auto* ShapeActors = ControlRigShapeActors.Find(InControlRig))
	{
		FTransform ComponentTransform = FTransform::Identity;
		if (!AreEditingControlRigDirectly())
		{
			ComponentTransform = GetHostingSceneComponentTransform(InControlRig);
		}
		for (AControlRigShapeActor* ShapeActor : *ShapeActors)
		{
			if (FRigTransformElement* ControlElement = InControlRig->FindControl(ShapeActor->ControlName))
			{
				const FTransform Transform = InControlRig->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal);
				ShapeActor->SetActorTransform(Transform * ComponentTransform);
			}
		}
	}

	// Update pivot
	if (AreRigElementsSelected(ValidControlTypeMask(), InControlRig))
	{
		if (bIsChangingControlShapeTransform)
		{
			ComputePivotFromEditedShape(InControlRig, InOutTransform);
		}
		else
		{
			const UControlRigEditModeSettings* Settings = GetSettings();
			const bool bEachLocalSpace = Settings && Settings->bLocalTransformsInEachLocalSpace;
			ComputePivotFromShapeActors(InControlRig, bEachLocalSpace, GetTransformSpace(), InOutTransform);			
		}
	}
	else if (AreRigElementSelectedAndMovable(InControlRig))
	{
		ComputePivotFromElements(InControlRig, InOutTransform);
	}
}

void FControlRigEditMode::HandleSelectionChanged()
{
	constexpr bool bFromChildActors = true;
	for (const auto& ShapeActors : ControlRigShapeActors)
	{
		for (const TObjectPtr<AControlRigShapeActor>& ShapeActor : ShapeActors.Value)
		{
			ShapeActor->ForEachComponent<UPrimitiveComponent>(bFromChildActors, [](UPrimitiveComponent* PrimitiveComponent) 
			{
				PrimitiveComponent->PushSelectionToProxy();
			});
		}
	}

	// automatically exit shape transform edit mode if there is no shape selected
	if (bIsChangingControlShapeTransform)
	{
		if (!CanChangeControlShapeTransform())
		{
			bIsChangingControlShapeTransform = false;
		}
	}

	// update the pivot transform of our selected objects (they could be animating)
	UpdatePivotTransforms();
	
	//need to force the redraw also
	if (!AreEditingControlRigDirectly())
	{
		GEditor->RedrawLevelEditingViewports(true);
	}
}

void FControlRigEditMode::BindCommands()
{
	const FControlRigEditModeCommands& Commands = FControlRigEditModeCommands::Get();

	CommandBindings->MapAction(
		Commands.ToggleManipulators,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ToggleManipulators));
	CommandBindings->MapAction(
		Commands.ToggleModuleManipulators,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ToggleModuleManipulators));
	CommandBindings->MapAction(
		Commands.ToggleAllManipulators,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ToggleAllManipulators));
	CommandBindings->MapAction(
		Commands.ToggleControlsAsOverlay,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ToggleControlsAsOverlay));
	CommandBindings->MapAction(
		Commands.ZeroTransforms,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ZeroTransforms, true, false, (UControlRig*) nullptr));
	CommandBindings->MapAction(
		Commands.ZeroAllTransforms,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ZeroTransforms, false, false, (UControlRig*) nullptr));
	CommandBindings->MapAction(
		Commands.InvertTransforms,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::InvertInputPose, true, false));
	CommandBindings->MapAction(
		Commands.InvertAllTransforms,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::InvertInputPose, false, false));
	CommandBindings->MapAction(
		Commands.InvertTransformsAndChannels,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::InvertInputPose, true, true));
	CommandBindings->MapAction(
		Commands.InvertAllTransformsAndChannels,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::InvertInputPose, false, true));
	CommandBindings->MapAction(
		Commands.ClearSelection,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ClearSelection));

	CommandBindings->MapAction(
		Commands.FrameSelection,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::FrameSelection),
		FCanExecuteAction::CreateRaw(this, &FControlRigEditMode::CanFrameSelection)
	);

	CommandBindings->MapAction(
		Commands.IsolateLinkedAnimTracks,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::IsolateLinkedAnimTracksForAllBindings, true));

	CommandBindings->MapAction(
		Commands.UnIsolateLinkedAnimTracks,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::IsolateLinkedAnimTracksForAllBindings, false));

	CommandBindings->MapAction(
		Commands.ToggleControlShapeTransformEdit,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ToggleControlShapeTransformEdit));

	CommandBindings->MapAction(
		Commands.SelectMirroredControls,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SelectMirroredControls));

	CommandBindings->MapAction(
		Commands.AddMirroredControlsToSelection,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::AddMirroredControlsToSelection));

	CommandBindings->MapAction(
		Commands.MirrorSelectedControls,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::MirrorSelectedControls));

	CommandBindings->MapAction(
		Commands.MirrorUnselectedControls,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::MirrorUnselectedControls));

	CommandBindings->MapAction(
		Commands.SelectAllControls,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SelectAllControls));

	CommandBindings->MapAction(
		Commands.SavePose,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SavePose,0));

	CommandBindings->MapAction(
		Commands.SelectPose,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SelectPose,false, 0));

	CommandBindings->MapAction(
		Commands.SelectMirrorPose,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SelectPose, true, 0));

	CommandBindings->MapAction(
		Commands.PastePose,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::PastePose, false, 0));

	CommandBindings->MapAction(
		Commands.PasteMirrorPose,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::PastePose, true, 0));

	CommandBindings->MapAction(
		Commands.SetAnimLayerPassthroughKey,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SetAnimLayerPassthroughKey));

	CommandBindings->MapAction(
		Commands.SelectAnimLayerWeightChannel,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::SelectAnimLayerWeightChannel));

	CommandBindings->MapAction(
		Commands.OpenSpacePickerWidget,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::OpenTabMenu));

	CommandBindings->MapAction(
		Commands.TogglePivotMode,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::TogglePivotMode));

	CommandBindings->MapAction(
		Commands.ToggleMotionTrails,
		FExecuteAction::CreateRaw(this, &FControlRigEditMode::ToggleMotionTrails));
}

void FControlRigEditMode::IsolateLinkedAnimTracksForAllBindings(bool bIsolate)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}
	const UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}
	const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();;
	TArray<FGuid> Guids;
	Guids.SetNum(Bindings.Num());
	for (int32 Index = 0; Index < Bindings.Num(); ++Index)
	{
		Guids[Index] = Bindings[Index].GetObjectGuid();
	}
	UE::Sequencer::FCommonAnimationTrackEditor::IsolateLinkedAnimTracks(Sequencer, Guids, bIsolate);

}

bool FControlRigEditMode::IsControlSelected(const bool bUseShapes) const
{
	static constexpr uint32 ControlType = static_cast<uint32>(ERigElementType::Control);
	
	if (bUseShapes)
	{
		for (const auto& [WeakControlRig, Shapes] : ControlRigShapeActors)
		{
			if (TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
			{
				for (const AControlRigShapeActor* ShapeActor : Shapes)
				{
					if (ShapeActor && ShapeActor->IsSelected())
					{
						return true;
					}
				}

				if (ControlRig && AreRigElementsSelected(ControlType,ControlRig.Get()))
				{
					return true;
				}
			}
		}
	}
	else
	{
		for (const TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				if (AreRigElementsSelected(ControlType,ControlRig))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

bool FControlRigEditMode::CanFrameSelection()
{
	for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			if (GetSelectedRigElements(ControlRig).Num() > 0)
			{
				return true;
			}
		}
	}
	return  false;;
}

void FControlRigEditMode::ClearSelection()
{
	const bool bShouldActuallyTransact = !AreEditingControlRigDirectly() && !GIsTransacting;
	const FScopedTransaction ScopedTransaction(LOCTEXT("AnimMode_ClearSelectionTransaction", "Clear Selection"), bShouldActuallyTransact);
	
	ClearRigElementSelection(FRigElementTypeHelper::ToMask(ERigElementType::All));
	
	if (GEditor)
	{
		GEditor->Exec(GetWorld(), TEXT("SELECT NONE"));
	}
}

void FControlRigEditMode::FrameSelection()
{
	if (IsInLevelEditor())
	{
		TArray<AActor*> Actors;
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
				for (const FRigElementKey& SelectedKey : SelectedRigElements)
				{
					if (SelectedKey.Type == ERigElementType::Control)
					{
						AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig, SelectedKey.Name);
						if (ShapeActor)
						{
							Actors.Add(ShapeActor);
						}
					}
				}
			}
		}

		if (Actors.Num())
		{
			TArray<UPrimitiveComponent*> SelectedComponents;
			GEditor->MoveViewportCamerasToActor(Actors, SelectedComponents, true);
		}
	}
	else
	{
		TArray<FRigElementKey> SelectedRigElements;
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				SelectedRigElements.Append(GetSelectedRigElements(ControlRig));
			}
		}

		FrameItems(SelectedRigElements);
	}
}

void FControlRigEditMode::FrameItems(const TArray<FRigElementKey>& InItems)
{
	if(!OnGetRigElementTransformDelegate.IsBound())
	{
		return;
	}

	if(CurrentViewportClient == nullptr)
	{
		DeferredItemsToFrame = InItems;
		return;
	}

	FBox Box(ForceInit);

	for (int32 Index = 0; Index < InItems.Num(); ++Index)
	{
		static const float Radius = 20.f;
		if (InItems[Index].Type == ERigElementType::Bone || InItems[Index].Type == ERigElementType::Null)
		{
			FTransform Transform = OnGetRigElementTransformDelegate.Execute(InItems[Index], false, true);
			Box += Transform.TransformPosition(FVector::OneVector * Radius);
			Box += Transform.TransformPosition(FVector::OneVector * -Radius);
		}
		else if (InItems[Index].Type == ERigElementType::Control)
		{
			FTransform Transform = OnGetRigElementTransformDelegate.Execute(InItems[Index], false, true);
			Box += Transform.TransformPosition(FVector::OneVector * Radius);
			Box += Transform.TransformPosition(FVector::OneVector * -Radius);
		}
	}

	if(Box.IsValid)
	{
		CurrentViewportClient->FocusViewportOnBox(Box);
	}
}

uint8 FControlRigEditMode::GetInteractionType(const FEditorViewportClient* InViewportClient)
{
	EControlRigInteractionType Result = EControlRigInteractionType::None;
	if (InViewportClient->IsMovingCamera())
	{
		return static_cast<uint8>(Result);
	}
	
	switch (InViewportClient->GetWidgetMode())
	{
		case UE::Widget::WM_Translate:
			EnumAddFlags(Result, EControlRigInteractionType::Translate);
			break;
		case UE::Widget::WM_TranslateRotateZ:
			EnumAddFlags(Result, EControlRigInteractionType::Translate);
			EnumAddFlags(Result, EControlRigInteractionType::Rotate);
			break;
		case UE::Widget::WM_Rotate:
			EnumAddFlags(Result, EControlRigInteractionType::Rotate);
			break;
		case UE::Widget::WM_Scale:
			EnumAddFlags(Result, EControlRigInteractionType::Scale);
			break;
		default:
			break;
	}
	return static_cast<uint8>(Result);
}

void FControlRigEditMode::ToggleControlShapeTransformEdit()
{ 
	if (bIsChangingControlShapeTransform)
	{
		bIsChangingControlShapeTransform = false;
	}
	else if (CanChangeControlShapeTransform())
	{
		bIsChangingControlShapeTransform = true;
	}
}

void FControlRigEditMode::GetAllSelectedControls(TMap<UControlRig*, TArray<FRigElementKey>>& OutSelectedControls) const
{
	for (const TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			if (const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
			{
				TArray<FRigElementKey> SelectedControls = Hierarchy->GetSelectedKeys(ERigElementType::Control);
				if (SelectedControls.Num() > 0)
				{
					OutSelectedControls.Add(ControlRig, SelectedControls);
				}
			}
		}
	}
}

void FControlRigEditMode::SetAnimLayerPassthroughKey()
{
	ISequencer* Sequencer = WeakSequencer.Pin().Get();
	if (Sequencer)
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(Sequencer))
		{
			const FScopedTransaction Transaction(LOCTEXT("SetPassthroughKey_Transaction", "Set Passthrough Key"), !GIsTransacting);
			for (UAnimLayer* AnimLayer : AnimLayers->AnimLayers)
			{
				if (AnimLayer->GetSelectedInList())
				{
					int32 Index = AnimLayers->GetAnimLayerIndex(AnimLayer);
					if (Index != INDEX_NONE)
					{
						AnimLayers->SetPassthroughKey(Sequencer, Index);
					}
				}
			}
		}
	}
}

void FControlRigEditMode::SelectAnimLayerWeightChannel()
{
	ISequencer* Sequencer = WeakSequencer.Pin().Get();
	if (Sequencer)
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(Sequencer))
		{
			const bool bClearSelection = !(FSlateApplication::Get().GetModifierKeys().IsShiftDown());
			for (UAnimLayer* AnimLayer : AnimLayers->AnimLayers)
			{
				if (AnimLayer->GetSelectedInList())
				{
					AnimLayer->SelectWeightChannel(bClearSelection);
				}
			}
		}
	}
}

void FControlRigEditMode::OpenTabMenu()
{
	using namespace UE::ControlRigBlutility;
		
	if (!SupportUtilityActions())
	{
		return OpenSpacePickerWidget();
	}
	
	auto GetViewportWidget = [ModeTools = GetModeManager()]() -> TWeakPtr<SViewport>
	{
		// NOTE: remove ModeTools == &GLevelEditorModeTools() to extend the tab menu in the ControlRig editor as well.
		if (ModeTools && ModeTools == &GLevelEditorModeTools())
		{
			if (FEditorViewportClient* HoveredVPC = ModeTools->GetHoveredViewportClient())
			{
				const TSharedPtr<SEditorViewport>& EditorViewport = HoveredVPC->GetEditorViewportWidget();
				const TSharedPtr<FSceneViewport>& SceneViewport = EditorViewport ? EditorViewport->GetSceneViewport() : nullptr;
				return SceneViewport ? SceneViewport->GetViewportWidget() : nullptr;
			}
		}
		return nullptr;
	};
	
	FControlRigUtilityActions Actions;
	const TWeakPtr<SViewport>& ViewportWidget = GetViewportWidget();
	if (!ViewportWidget.IsValid() ||
		(Actions = FControlRigUtilityActions::GetControlRigUtilityActions(WorldPtr, RuntimeControlRigs), Actions.UtilityAndRigIndices.IsEmpty()))
	{
		return OpenSpacePickerWidget();
	}

	constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	// add space picker widget
	{
		using namespace UE::ControlRigEditor;
		if (const TSharedPtr<SRigSpacePickerMenuEntry> SpacePickerMenuEntry = GetSpacePickerManager().CreateSpacePickerMenuEntry())
		{
			MenuBuilder.BeginSection("SpacePicker", LOCTEXT("SpacePicker", "Switch Space"));
			MenuBuilder.AddWidget(SpacePickerMenuEntry.ToSharedRef(), FText::GetEmpty());
			MenuBuilder.EndSection();
		}
	}

	// add utility actions
	{
		const FBlutilityMenuExtensions::FMenuParams MenuParams({
		   LOCTEXT("ControlRigMode_Actions", "Control Rig Actions"),
		   LOCTEXT("ControlRigMode_Tooltip", "Control Rig Actions Tooltip"),
			   NAME_None,
			   false});

		const FBlutilityMenuExtensions::FObjectActionParams ActionParams({
		MoveTemp(Actions.SupportedRigs),
			MoveTemp(Actions.UtilityAndRigIndices),
			[](const FProperty*) { return true; }
		});
	
		FBlutilityMenuExtensions::CreateObjectBlutilityActionsMenu(MenuBuilder, MenuParams, ActionParams);
	}
	
	// create menu
	FSlateApplication::Get().PushMenu(
			ViewportWidget.Pin().ToSharedRef(),
			FWidgetPath(),
			MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu ) );
}

void FControlRigEditMode::OpenSpacePickerWidget()
{
	GetSpacePickerManager().SummonSpacePickerAtCursor();
}

UE::ControlRigEditor::FRigFloatingSpacePickerManager& FControlRigEditMode::GetSpacePickerManager()
{
	using namespace UE::ControlRigEditor;
	
	if (!FloatingSpacePickerManager)
	{
		FloatingSpacePickerManager = MakePimpl<FRigFloatingSpacePickerManager>(*this);
	}
	
	return *FloatingSpacePickerManager;
}

static void ToggleTool(const FString& InToolName)
{
	if (ILevelEditor* LevelEditor = Private::GetLevelEditor())
	{
		FString ActiveToolName =
			LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveToolName(EToolSide::Left);
		if (ActiveToolName == InToolName)
		{
			LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->DeactivateTool(
				EToolSide::Left, EToolShutdownType::Completed
			);
		}
		else
		{
			LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->SelectActiveToolType(
				EToolSide::Left, InToolName
			);
			LevelEditor->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->ActivateTool(EToolSide::Left);
		}
	}
}

void FControlRigEditMode::TogglePivotMode()
{
	FString Tool = TEXT("SequencerPivotTool");
	ToggleTool(Tool);
}

void FControlRigEditMode::ToggleMotionTrails()
{
	if (UMotionTrailToolOptions* Settings = GetMutableDefault<UMotionTrailToolOptions>())
	{
		Settings->bShowTrails = !Settings->bShowTrails;
		FPropertyChangedEvent ShowTrailEvent(UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails)));
		Settings->PostEditChangeProperty(ShowTrailEvent);
	}
}

FText FControlRigEditMode::GetToggleControlShapeTransformEditHotKey() const
{
	const FControlRigEditModeCommands& Commands = FControlRigEditModeCommands::Get();
	return Commands.ToggleControlShapeTransformEdit->GetInputText();
}

void FControlRigEditMode::ToggleManipulators()
{
	if (!AreEditingControlRigDirectly())
	{
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		GetAllSelectedControls(SelectedControls);
		TArray<UControlRig*> ControlRigs;
		SelectedControls.GenerateKeyArray(ControlRigs);
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("ToggleControlsVisibility", "Toggle Controls Visibility"),!GIsTransacting);
				ControlRig->Modify();
				ControlRig->ToggleControlsVisible();
				if (OnControlRigVisibilityChangedDelegate.IsBound())
				{
					OnControlRigVisibilityChangedDelegate.Broadcast({ControlRig});
				}
			}
		}
	}
	else
	{
		ToggleDisplayControlsEnabled();
	}
}

void FControlRigEditMode::ToggleModuleManipulators()
{
	const UControlRigEditModeSettings* Settings = GetSettings();

	if (!AreEditingControlRigDirectly() && IsDisplayControlsEnabled())
	{
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				if (ControlRig->GetControlsVisible())
				{
					if (UModularRig* ModularRig = Cast<UModularRig>(ControlRig))
					{
						TArray<FString> ModuleNames;
						TArray<UControlRig*> ChangedRigs;
						TArray<FRigElementKey> Selected = GetSelectedRigElements(ControlRig);
						for (const FRigElementKey& Key : Selected)
						{
							const FString ModuleName = ControlRig->GetHierarchy()->GetModuleName(Key);
							ModuleNames.AddUnique(ModuleName);
						}
						for (const FString& ModuleName : ModuleNames)
						{
							if (FRigModuleInstance* Module = ModularRig->FindModule(*ModuleName))
							{
								if (UControlRig* Rig = Module->GetRig())
								{
									Rig->ToggleControlsVisible();
									ChangedRigs.Add(Rig);
								}
							}
						}

						if (OnControlRigVisibilityChangedDelegate.IsBound())
						{
							OnControlRigVisibilityChangedDelegate.Broadcast(ChangedRigs);
						}
					}
				}
			}
		}
	}
}

void FControlRigEditMode::OpenPoseLibraryTab()
{
	if (UsesToolkits() && Toolkit.IsValid())
	{
		StaticCastSharedPtr<FControlRigEditModeToolkit>(Toolkit)->TryInvokeToolkitUI(FControlRigEditModeToolkit::PoseTabName);
	}
}

void FControlRigEditMode::ClosePoseLibraryTab()
{
	if (UsesToolkits() && Toolkit.IsValid())
	{
		StaticCastSharedPtr<FControlRigEditModeToolkit>(Toolkit)->TryCloseToolkitUI(FControlRigEditModeToolkit::PoseTabName);
	}
}

void FControlRigEditMode::ToggleAllManipulators()
{
	ToggleDisplayControlsEnabled();

	//turn on all if in level editor in case any where off
	if (!AreEditingControlRigDirectly() && 
		IsDisplayControlsEnabled())
	{
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				ControlRig->SetControlsVisible(true);
				if (UModularRig* ModularRig = Cast<UModularRig>(ControlRig))
				{
					ModularRig->ForEachModule([](FRigModuleInstance* Module)
					{
						Module->GetRig()->SetControlsVisible(true);
						return true;
					});
				}
			}
		}
	}
}

bool FControlRigEditMode::IsDisplayControlsEnabled() const
{
	return !GetDefault<UControlRigEditModeSettings>()->bHideControlShapes;
}

void FControlRigEditMode::ToggleDisplayControlsEnabled()
{
	UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();
	Settings->bHideControlShapes = !Settings->bHideControlShapes;
	Settings->SaveConfig();
}

bool FControlRigEditMode::IsControlsAsOverlayEnabled() const
{
	return GetDefault<UControlRigEditModeSettings>()->bShowControlsAsOverlay;
}

void FControlRigEditMode::ToggleControlsAsOverlay()
{
	UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();
	Settings->bShowControlsAsOverlay = !Settings->bShowControlsAsOverlay;
	Settings->SaveConfig();
}

bool FControlRigEditMode::IsDisplayNullsEnabled() const
{
	return GetDefault<UControlRigEditModeSettings>()->bDisplayNulls;
}

void FControlRigEditMode::ToggleDisplayNulls()
{
	UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();
	Settings->bDisplayNulls = !Settings->bDisplayNulls;
	Settings->SaveConfig();
}

bool FControlRigEditMode::IsDisplaySocketsEnabled() const
{
	return GetDefault<UControlRigEditModeSettings>()->bDisplaySockets;
}

void FControlRigEditMode::ToggleDisplaySockets()
{
	UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();
	Settings->bDisplaySockets = !Settings->bDisplaySockets;
	Settings->SaveConfig();
}

bool FControlRigEditMode::IsDisplayAxesOnSelectionEnabled() const
{
	return GetDefault<UControlRigEditModeSettings>()->bDisplayAxesOnSelection;
}

void FControlRigEditMode::ToggleDisplayAxesOnSelection()
{
	UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();
	Settings->bDisplayAxesOnSelection = !Settings->bDisplayAxesOnSelection;
	Settings->SaveConfig();
}

float FControlRigEditMode::GetAxesOnSelectionScale() const
{
	return GetDefault<UControlRigEditModeSettings>()->AxisScale;
}

bool FControlRigEditMode::AreControlsVisible() const
{
	if (!AreEditingControlRigDirectly())
	{
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		GetAllSelectedControls(SelectedControls);
		TArray<UControlRig*> ControlRigs;
		SelectedControls.GenerateKeyArray(ControlRigs);
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				if (!ControlRig->bControlsVisible)
				{
					return false;
				}
			}
		}
		return true;
	}
	
	return IsDisplayControlsEnabled();
}

TArray<FRigElementKey> FControlRigEditMode::GetRigElementsForSettingTransforms(UControlRig* ControlRig, bool bSelectionOnly, bool bIncludeChannels)
{
	TArray<FRigElementKey> RigElements;
	if (bSelectionOnly)
	{
		RigElements = GetSelectedRigElements(ControlRig);
		if (ControlRig->IsAdditive())
		{
			// For additive rigs, ignore boolean controls
			RigElements = RigElements.FilterByPredicate([ControlRig](const FRigElementKey& Key)
				{
					if (FRigControlElement* Element = ControlRig->FindControl(Key.Name))
					{
						return Element->CanTreatAsAdditive();
					}
					return true;
				});
		}
	}
	else
	{
		TArray<FRigBaseElement*> Elements = ControlRig->GetHierarchy()->GetElementsOfType<FRigBaseElement>(true);
		for (const FRigBaseElement* Element : Elements)
		{
			// For additive rigs, ignore non-additive controls
			if (const FRigControlElement* Control = Cast<FRigControlElement>(Element))
			{
				if (ControlRig->IsAdditive() && !Control->CanTreatAsAdditive())
				{
					continue;
				}
			}
			RigElements.Add(Element->GetKey());
		}
	}

	if (bIncludeChannels == false)
	{
		RigElements = RigElements.FilterByPredicate([ControlRig](const FRigElementKey& Key)
		{
			if (FRigControlElement* Element = ControlRig->FindControl(Key.Name))
			{
				return Element->IsAnimationChannel() == false;
			}
			return true;
		});
	}
	return RigElements;
}

void FControlRigEditMode::ZeroTransforms(bool bSelectionOnly, bool bIncludeChannels, UControlRig* InControlRig)
{
	// Gather up the control rigs for the selected controls
	TArray<UControlRig*> ControlRigs;
	if (InControlRig)
	{
		ControlRigs = {InControlRig};
	}
	else
	{
		ControlRigs = GetControlRigsBasedOnSelection(bSelectionOnly);
	}
	
	if (ControlRigs.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("HierarchyZeroTransforms", "Zero Transforms"));
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::DoNotCare;

	for (UControlRig* ControlRig : ControlRigs)
	{
		ZeroTransforms(ControlRig, Context, bSelectionOnly, bIncludeChannels);
	}
}

void FControlRigEditMode::ZeroTransforms(UControlRig* ControlRig, const FRigControlModifiedContext& Context,  bool bSelectionOnly, bool bIncludeChannels)
{
	TArray<FRigElementKey> ControlsToReset;
	TArray<FRigElementKey> ControlsInteracting;
	TArray<FRigElementKey> TransformElementsToReset;

	TArray<FRigElementKey> SelectedRigElements = FControlRigEditMode::GetRigElementsForSettingTransforms(ControlRig, bSelectionOnly, bIncludeChannels);
	if (bSelectionOnly)
	{
		ControlsToReset = SelectedRigElements;
		ControlsInteracting = SelectedRigElements;
		TransformElementsToReset = SelectedRigElements;
	}
	else 
	{
		TransformElementsToReset = SelectedRigElements;
		TArray<FRigControlElement*> Controls;
		ControlRig->GetControlsInOrder(Controls);
		ControlsToReset.SetNum(0);
		ControlsInteracting.SetNum(0);
		for (const FRigControlElement* Control : Controls)
		{
			// For additive rigs, ignore boolean controls
			if (ControlRig->IsAdditive() && Control->Settings.ControlType == ERigControlType::Bool)
			{
				continue;
			}
			if (bIncludeChannels == false && Control->IsAnimationChannel())
			{
				continue;
			}
			ControlsToReset.Add(Control->GetKey());
			if(Control->Settings.AnimationType == ERigControlAnimationType::AnimationControl ||
				Control->IsAnimationChannel())
			{
				ControlsInteracting.Add(Control->GetKey());
			}
		}
	}
	bool bHasNonDefaultParent = false;
	TMap<FRigElementKey, FRigElementKey> Parents;
	for (const FRigElementKey& Key : TransformElementsToReset)
	{
		FRigElementKey SpaceKey = ControlRig->GetHierarchy()->GetActiveParent(Key);
		Parents.Add(Key, SpaceKey);
		if (!bHasNonDefaultParent && SpaceKey != ControlRig->GetHierarchy()->GetDefaultParentKey())
		{
			bHasNonDefaultParent = true;
		}
	}

	FControlRigInteractionScope InteractionScope(ControlRig, ControlsInteracting);
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	
	for (const FRigElementKey& ElementToReset : TransformElementsToReset)
	{
		FRigControlElement* ControlElement = nullptr;
		if (ElementToReset.Type == ERigElementType::Control)
		{
			ControlElement = ControlRig->FindControl(ElementToReset.Name);
			if (ControlElement->Settings.bIsTransientControl)
			{
				if(UControlRig::GetNodeNameFromTransientControl(ControlElement->GetKey()).IsEmpty())
				{
					ControlElement = nullptr;
				}
			}
		}
			
		const FTransform InitialLocalTransform = ControlRig->GetInitialLocalTransform(ElementToReset);
		ControlRig->Modify();
		if (bHasNonDefaultParent == true) //possibly not at default parent so switch to it
		{
			Hierarchy->SwitchToDefaultParent(ElementToReset);
		}
		if (ControlElement)
		{
			const FVector InitialAngles = Hierarchy->GetControlPreferredEulerAngles(ControlElement, ControlElement->Settings.PreferredRotationOrder, true);
			Hierarchy->SetControlPreferredEulerAngles(ControlElement, InitialAngles, ControlElement->Settings.PreferredRotationOrder);
			ControlRig->SetControlLocalTransform(ElementToReset.Name, InitialLocalTransform, true, Context, true, true);

			NotifyDrivenControls(ControlRig, ElementToReset, Context);

			if (bHasNonDefaultParent == false)
			{
				ControlRig->ControlModified().Broadcast(ControlRig, ControlElement, EControlRigSetKey::DoNotCare);
			}
		}
		else
		{
			Hierarchy->SetLocalTransform(ElementToReset, InitialLocalTransform, false, true, true);
		}

		//@helge not sure what to do if the non-default parent
		FControlRigAssetStrongReference AssetReference = ControlRig->GetAssetReference();
		if (FControlRigAssetInterfacePtr Asset = AssetReference.GetEditorAsset())
		{
			if (Asset->GetDebuggedControlRig() == ControlRig)
			{
				Asset->GetHierarchy()->SetLocalTransform(ElementToReset, InitialLocalTransform);
			}
		}
	}

	if (bHasNonDefaultParent == true) //now we have the initial pose setup we need to get the global transforms as specified now then set them in the current parent space
	{
		EvaluateRig(ControlRig);

		//get global transforms
		TMap<FRigElementKey, FTransform> GlobalTransforms;
		for (const FRigElementKey& ElementToReset : TransformElementsToReset)
		{
			if (ElementToReset.IsTypeOf(ERigElementType::Control))
			{
				FRigControlElement* ControlElement = ControlRig->FindControl(ElementToReset.Name);
				if (ControlElement && !ControlElement->Settings.bIsTransientControl)
				{
					FTransform GlobalTransform = Hierarchy->GetGlobalTransform(ElementToReset);
					GlobalTransforms.Emplace(ElementToReset, GlobalTransform);
				}
				NotifyDrivenControls(ControlRig, ElementToReset,Context);
			}
			else
			{
				FTransform GlobalTransform = Hierarchy->GetGlobalTransform(ElementToReset);
				GlobalTransforms.Add(ElementToReset, GlobalTransform);
			}
		}
		//switch back to original parent space
		for (const FRigElementKey& ElementToReset : TransformElementsToReset)
		{
			if (const FRigElementKey* SpaceKey = Parents.Find(ElementToReset))
			{
				if (ElementToReset.IsTypeOf(ERigElementType::Control))
				{
					FRigControlElement* ControlElement = ControlRig->FindControl(ElementToReset.Name);
					if (ControlElement && !ControlElement->Settings.bIsTransientControl)
					{
						Hierarchy->SwitchToParent(ElementToReset, *SpaceKey);
					}
				}
				else
				{
					Hierarchy->SwitchToParent(ElementToReset, *SpaceKey);
				}
			}
		}
		
		//set global transforms in this space // do it twice since ControlsInOrder is not really always in order
		const FRigControlModifiedContext DefaultContext;
		constexpr bool bNotify = true, bSetupUndo = true, bNoPython = false, bFixEuler = true;
		
		for (int32 SetHack = 0; SetHack < 2; ++SetHack)
		{
			const TGuardValue<bool> GuardEvaluationType(ControlRig->bEvaluationTriggeredFromInteraction, true);
			ControlRig->Evaluate_AnyThread();
			for (const FRigElementKey& ElementToReset : TransformElementsToReset)
			{
				if (const FTransform* GlobalTransform = GlobalTransforms.Find(ElementToReset))
				{
					if (ElementToReset.IsTypeOf(ERigElementType::Control))
					{
						FRigControlElement* ControlElement = ControlRig->FindControl(ElementToReset.Name);
						if (ControlElement && !ControlElement->Settings.bIsTransientControl)
						{
							ControlRig->SetControlGlobalTransform(ElementToReset.Name, *GlobalTransform, bNotify, DefaultContext, bSetupUndo, bNoPython, bFixEuler);
							ControlRig->Evaluate_AnyThread();
							NotifyDrivenControls(ControlRig, ElementToReset, Context);
						}
					}
					else
					{
						Hierarchy->SetGlobalTransform(ElementToReset, *GlobalTransform, false, true, bSetupUndo);
					}
				}
			}
		}
		
		//send notifies
		for (const FRigElementKey& ControlToReset : ControlsToReset)
		{
			FRigControlElement* ControlElement = ControlRig->FindControl(ControlToReset.Name);
			if (ControlElement && !ControlElement->Settings.bIsTransientControl)
			{
				ControlRig->ControlModified().Broadcast(ControlRig, ControlElement, EControlRigSetKey::DoNotCare);
			}
		}
	}
	else
	{
		// we have to insert the interaction event before we run current events
		TArray<FName> NewEventQueue = {FRigUnit_InteractionExecution::EventName};
		NewEventQueue.Append(ControlRig->EventQueue);
		TGuardValue<TArray<FName>> EventGuard(ControlRig->EventQueue, NewEventQueue);
		const TGuardValue<bool> GuardEvaluationType(ControlRig->bEvaluationTriggeredFromInteraction, true);
		ControlRig->Evaluate_AnyThread();
		for (const FRigElementKey& ControlToReset : ControlsToReset)
		{
			NotifyDrivenControls(ControlRig, ControlToReset, Context);
		}
	}
}

TArray<UControlRig*> FControlRigEditMode::GetControlRigsBasedOnSelection(bool bSelectionOnly) const
{
	TArray<UControlRig*> SelectedControlRigs;
	TArray<UControlRig*> AllControlRigs;
	for (const TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			if (ControlRig->CurrentControlSelection().Num() > 0)
			{
				SelectedControlRigs.Add(ControlRig);
			}
			AllControlRigs.Add(ControlRig);
		}
	}

	if (SelectedControlRigs.Num() > 0 || bSelectionOnly)
	{
		return SelectedControlRigs;
	}
	return AllControlRigs;
}

void FControlRigEditMode::InvertInputPose(bool bSelectionOnly, bool bIncludeChannels)
{
	// Gather up the control rigs for the selected controls
	TArray<UControlRig*> ControlRigs = GetControlRigsBasedOnSelection(bSelectionOnly);
	if (ControlRigs.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("HierarchyInvertTransformsToRestPose", "Invert Transforms to Rest Pose"));
	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::DoNotCare;
	for (UControlRig* ControlRig : ControlRigs)
	{
		InvertInputPose(ControlRig, Context,  bSelectionOnly, bIncludeChannels);
	}
}

void FControlRigEditMode::InvertInputPose(UControlRig * ControlRig, const FRigControlModifiedContext & Context, bool bSelectionOnly, bool bIncludeChannels)
{
	if (!ControlRig->IsAdditive())
	{
		ZeroTransforms(ControlRig, Context, bSelectionOnly, bIncludeChannels);
		return;
	}

	TArray<FRigElementKey> SelectedRigElements;
	if(bSelectionOnly)
	{
		SelectedRigElements = FControlRigEditMode::GetRigElementsForSettingTransforms(ControlRig, bSelectionOnly, bIncludeChannels);
	}

	const TArray<FRigControlElement*> ModifiedElements = ControlRig->InvertInputPose(SelectedRigElements, Context.SetKey);
	EvaluateRig(ControlRig);

	for (FRigControlElement* ControlElement : ModifiedElements)
	{
		ControlRig->ControlModified().Broadcast(ControlRig, ControlElement, Context.SetKey);
	}
}

bool FControlRigEditMode::MouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InX, int32 InY)
{
	// compare with the last cache to track useless synthesized mouse moves
	if (!MouseMoveCache.Update(InViewport, InX, InY))
	{
		return false;
	}
	
	// avoid hit proxy cast as much as possible
	auto HasAnyHoverableShapeActor = [this, InViewportClient]()
	{
		if (RecreateControlShapesRequired != ERecreateControlRigShape::RecreateNone)
		{
			return false;
		}
		
		if (!InteractionScopes.IsEmpty())
		{
			return false;
		}

		if (!InViewportClient || InViewportClient->IsInGameView())
		{
			return false;
		}

		if (bSequencerPlaying)
		{
			return false;
		}
		
		for (auto&[ControlRig, ShapeActors]: ControlRigShapeActors)
		{
			for (const AControlRigShapeActor* ShapeActor : ShapeActors)
			{
				if (ShapeActor && ShapeActor->IsSelectable() && !ShapeActor->IsTemporarilyHiddenInEditor())
				{
					return true;
				}
			}
		}
		return false;
	};
	
	if (HasAnyHoverableShapeActor())
	{	
		HHitProxy* const HitProxy = InViewport->GetHitProxy(InX, InY);
		const HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy);
		const AControlRigShapeActor* HitShape = ActorHitProxy && ActorHitProxy->Actor ? Cast<AControlRigShapeActor>(ActorHitProxy->Actor) : nullptr;
		auto IsHovered = [HitShape](const AControlRigShapeActor* InShapeActor)
		{
			return HitShape ? InShapeActor == HitShape : false;
		};

		for (const auto& [ControlRig, Shapes] : ControlRigShapeActors)
		{
			for (AControlRigShapeActor* ShapeActor : Shapes)
			{
				if (ShapeActor)
				{
					ShapeActor->SetHovered(ShapeActor->IsSelected() ? true : IsHovered(ShapeActor));
				}
			}
		}
	}

	return false;
}

bool FControlRigEditMode::MouseEnter(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InX, int32 InY)
{
	if (PendingFocus.IsEnabled() && InViewportClient)
	{
		const FEditorModeTools* ModeTools = GetModeManager();  
		if (ModeTools && ModeTools == &GLevelEditorModeTools())
		{
			FEditorViewportClient* HoveredVPC = ModeTools->GetHoveredViewportClient();
			if (HoveredVPC == InViewportClient)
			{
				const TWeakPtr<SViewport> ViewportWidget = InViewportClient->GetEditorViewportWidget()->GetSceneViewport()->GetViewportWidget();
				if (ViewportWidget.IsValid())
				{
					PendingFocus.SetPendingFocusIfNeeded(ViewportWidget.Pin()->GetContent());
				}
			}
		}
	}

	return IPersonaEditMode::MouseEnter(InViewportClient, InViewport, InX, InY);
}

bool FControlRigEditMode::MouseLeave(FEditorViewportClient* /*InViewportClient*/, FViewport* /*InViewport*/)
{
	PendingFocus.ResetPendingFocus();
	
	for (auto& ShapeActors : ControlRigShapeActors)
	{
		for (AControlRigShapeActor* ShapeActor : ShapeActors.Value)
		{
			ShapeActor->SetHovered(ShapeActor->IsSelected());
		}
	}

	return false;
}

void FControlRigEditMode::RegisterPendingFocusMode()
{
	if (!IsInLevelEditor())
	{
		return;
	}
	
	static IConsoleVariable* UseFocusMode = Private::GetFocusModeVariable();
	if (ensure(UseFocusMode))
	{
		auto OnFocusModeChanged = [this](IConsoleVariable*)
		{
			PendingFocus.Enable(Private::bFocusMode);
			if (WeakSequencer.IsValid())
			{
				TSharedPtr<ISequencer> PreviousSequencer = WeakSequencer.Pin();
				PreviousSequencer->GetSequencerWidgetInterface().EnablePendingFocusOnHovering(Private::bFocusMode);
			}
		};
		if (!PendingFocusHandle.IsValid())
		{
			PendingFocusHandle = UseFocusMode->OnChangedDelegate().AddLambda(OnFocusModeChanged);
		}		
		OnFocusModeChanged(UseFocusMode);
		UseFocusMode->ClearFlags(ECVF_SetByDeviceProfile);
	}
}

void FControlRigEditMode::UnregisterPendingFocusMode()
{
	constexpr bool bDisable = false;
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->GetSequencerWidgetInterface().EnablePendingFocusOnHovering(bDisable);
	}

	PendingFocus.Enable(bDisable);
	
	if (PendingFocusHandle.IsValid())
	{
		static IConsoleVariable* UseFocusMode = Private::GetFocusModeVariable();
		if (ensure(UseFocusMode))
		{
			UseFocusMode->OnChangedDelegate().Remove(PendingFocusHandle);
		}
		PendingFocusHandle.Reset();
	}
}

void FControlRigEditMode::SetSequencerDelegates(const TWeakPtr<ISequencer>& InWeakSequencer)
{
	if (ensure(InWeakSequencer == WeakSequencer))
	{
		if (WeakSequencer.IsValid())
		{
			DetailKeyFrameCache->SetDelegates(WeakSequencer, this);

			if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
			{
				auto AddSequencerStatusBinding = [this](TMulticastDelegate<void()>& InDelegate)
				{
					if (!InDelegate.IsBoundToObject(this))
					{
						InDelegate.AddRaw(this, &FControlRigEditMode::UpdateSequencerStatus);
					}		
				};

				AddSequencerStatusBinding(Sequencer->OnPlayEvent());
				AddSequencerStatusBinding(Sequencer->OnStopEvent());
				// NOTE this is needed as status changes are not triggered
				AddSequencerStatusBinding(Sequencer->OnGlobalTimeChanged());
				AddSequencerStatusBinding(Sequencer->OnEndScrubbingEvent());
			}
		}
	}
	UpdateSequencerStatus();
}

void FControlRigEditMode::UnsetSequencerDelegates() const
{
	DetailKeyFrameCache->UnsetDelegates();

	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnPlayEvent().RemoveAll(this);
		Sequencer->OnStopEvent().RemoveAll(this);
		Sequencer->OnGlobalTimeChanged().RemoveAll(this);
		Sequencer->OnEndScrubbingEvent().RemoveAll(this);
	}
}

void FControlRigEditMode::UpdateSequencerStatus()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
    const bool bIsSequencerPlaying = Sequencer.IsValid() && Sequencer->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;

    if (bIsSequencerPlaying != bSequencerPlaying)
    {
    	bSequencerPlaying = bIsSequencerPlaying;

    	// update keyframer state
    	const bool bDeferAutokeyOnMouseRelease = !bSequencerPlaying && IsInLevelEditor();

    	// flush any existing pending keyframes
    	const bool HasPendingKeyframes = Keyframer.IsEnabled() && !bDeferAutokeyOnMouseRelease && !InteractionScopes.IsEmpty();
    	if (HasPendingKeyframes)
    	{
    		if (FEditorModeTools* ModeTools = GetModeManager())
    		{
    			const FControlRigInteractionTransformContext TransformContext(ModeTools->GetWidgetMode());
    			for (const auto& [ControlRig, InteractionScope] : InteractionScopes)
    			{
    				if (InteractionScope)
    				{
    					Keyframer.Apply(*InteractionScope, TransformContext);	
    				}
    			}
    			Keyframer.Finalize(ModeTools->GetWorld());
    		}
    	}

    	// set new state
    	Keyframer.Enable(bDeferAutokeyOnMouseRelease);
    }
}

bool FControlRigEditMode::CheckMovieSceneSig()
{
	bool bSomethingChanged = false;
	if (WeakSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
		{
			FGuid CurrentMovieSceneSig = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
			if (LastMovieSceneSig != CurrentMovieSceneSig)
			{
				if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence()))
				{
					TArray<TWeakObjectPtr<UControlRig>> CurrentControlRigs;
					TArray<FControlRigSequencerBindingProxy> Proxies = UControlRigSequencerEditorLibrary::GetControlRigs(LevelSequence);
					for (FControlRigSequencerBindingProxy& Proxy : Proxies)
					{
						if (UControlRig* ControlRig = Proxy.ControlRig.Get())
						{
							CurrentControlRigs.Add(ControlRig);
							if (RuntimeControlRigs.Contains(ControlRig) == false)
							{
								AddControlRigInternal(ControlRig);
								bSomethingChanged = true;
							}
						}
					}
					TArray<TWeakObjectPtr<UControlRig>> ControlRigsToRemove;
					for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
					{
						if (CurrentControlRigs.Contains(RuntimeRigPtr) == false)
						{
							ControlRigsToRemove.Add(RuntimeRigPtr);
						}
					}
					for (TWeakObjectPtr<UControlRig>& OldRuntimeRigPtr : ControlRigsToRemove)
					{
						RemoveControlRig(OldRuntimeRigPtr.Get());
					}
				}
				LastMovieSceneSig = CurrentMovieSceneSig;
				if (bSomethingChanged)
				{
					SetObjects_Internal();
				}
				DetailKeyFrameCache->ResetCachedData();
			}
		}
	}
	return bSomethingChanged;
}

void FControlRigEditMode::PostUndo()
{
	bool bInvalidateViewport = false;
	if (WeakSequencer.IsValid())
	{
		bool bHaveInvalidControlRig = false;
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (RuntimeRigPtr.IsValid() == false)
			{				
				bHaveInvalidControlRig = bInvalidateViewport = true;
				break;
			}
		}
		//if one is invalid we need to clear everything,since no longer have ptr to selectively delete
		if (bHaveInvalidControlRig == true)
		{
			TArray<TWeakObjectPtr<UControlRig>> PreviousRuntimeRigs = RuntimeControlRigs;
			for (int32 PreviousRuntimeRigIndex = 0; PreviousRuntimeRigIndex < PreviousRuntimeRigs.Num(); PreviousRuntimeRigIndex++)
			{
				if (PreviousRuntimeRigs[PreviousRuntimeRigIndex].IsValid())
				{
					RemoveControlRig(PreviousRuntimeRigs[PreviousRuntimeRigIndex].Get());
				}
			}
			RuntimeControlRigs.Reset();
			DestroyShapesActors(nullptr);
			DelegateHelpers.Reset();
			RuntimeControlRigs.Reset();
		}
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence()))
		{
			bool bSomethingAdded = false;
			TArray<FControlRigSequencerBindingProxy> Proxies = UControlRigSequencerEditorLibrary::GetControlRigs(LevelSequence);
			for (FControlRigSequencerBindingProxy& Proxy : Proxies)
			{
				if (UControlRig* ControlRig = Proxy.ControlRig.Get())
				{
					if (RuntimeControlRigs.Contains(ControlRig) == false)
					{
						AddControlRigInternal(ControlRig);
						bSomethingAdded = true;

					}
				}
			}
			if (bSomethingAdded)
			{
				Sequencer->ForceEvaluate();
				SetObjects_Internal();
				bInvalidateViewport = true;
			}
		}
	}
	else
	{
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (RuntimeRigPtr.IsValid() == false)
			{
				DestroyShapesActors(RuntimeRigPtr.Get());
				bInvalidateViewport = true;
			}
		}
	}

	//normal actor undo will force the redraw, so we need to do the same for our transients/controls.
	if (!AreEditingControlRigDirectly() && (bInvalidateViewport || UsesTransformWidget()))
	{
		GEditor->GetTimerManager()->SetTimerForNextTick([this]()
		{
			//due to tick ordering need to manually make sure we get everything done in correct order.
			PostPoseUpdate();
			UpdatePivotTransforms();
			GEditor->RedrawLevelEditingViewports(true);
		});
	}

}

EEditAction::Type FControlRigEditMode::GetActionDragDuplicate()
{
	// Halt edit duplicate from a gizmo alt-drag action.
	return EEditAction::Halt;
}

void FControlRigEditMode::ClearOutHidden(UControlRig* InControlRig)
{
	ControlRigEditMode::Shapes::ClearOutHidden(InControlRig);
}

void FControlRigEditMode::ShowControlRigControls(UControlRig* InControlRig, const TSet<FString>& InNames, bool bVal)
{
	ControlRigEditMode::Shapes::ShowControlRigControls(InControlRig, InNames, bVal);
}

void FControlRigEditMode::RequestToRecreateControlShapeActors(UControlRig* ControlRig)
{ 
	if (ControlRig)
	{
		if (RecreateControlShapesRequired != ERecreateControlRigShape::RecreateAll)
		{
			RecreateControlShapesRequired = ERecreateControlRigShape::RecreateSpecified;
			if (ControlRigsToRecreate.Find(ControlRig) == INDEX_NONE)
			{
				ControlRigsToRecreate.Add(ControlRig);
			}
		}
	}
	else
	{
		RecreateControlShapesRequired = ERecreateControlRigShape::RecreateAll;
	}
}

bool FControlRigEditMode::TryUpdatingControlsShapes(UControlRig* InControlRig)
{
	using namespace ControlRigEditMode::Shapes;
	
	const URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return false;
	}

	// Prevent evaluating the rig while we collect and use raw FRigControlElement* pointers.
	// Evaluation (e.g. construction) can reallocate internal hierarchy storage, dangling those pointers.
	// Both CreateShapeActors and this function call GetControlsEligibleForShapes
	UE::TScopeLock EvaluateLock(InControlRig->GetEvaluateMutex());

	const auto* ShapeActors = ControlRigShapeActors.Find(InControlRig);
	if (!ShapeActors)
	{
		// create the shapes if they don't already exist
		CreateShapeActors(InControlRig);
		return true;
	}

	// get controls which need shapes
	TArray<FRigControlElement*> Controls;
	GetControlsEligibleForShapes(InControlRig, Controls);

	if (Controls.IsEmpty())
	{
		// not control needing shape so clear the shape actors
		DestroyShapesActors(InControlRig);
		ClearOutHidden(InControlRig);
		return true;
	}
	

	const TArray<TObjectPtr<AControlRigShapeActor>>& Shapes = *ShapeActors;
	const int32 NumShapes = Shapes.Num();

	TArray<FRigControlElement*> ControlPerShapeActor;
	ControlPerShapeActor.SetNumZeroed(NumShapes);

	if (Controls.Num() == NumShapes)
	{
		//unfortunately n*n-ish but this should be very rare and much faster than recreating them
		for (int32 ShapeActorIndex = 0; ShapeActorIndex < NumShapes; ShapeActorIndex++)
		{
			if (const AControlRigShapeActor* Actor = Shapes[ShapeActorIndex].Get())
			{
				const int32 ControlIndex = Controls.IndexOfByPredicate([Actor](const FRigControlElement* Control)
				{
					return Control && Control->GetFName() == Actor->ControlName;
				});
				if (ControlIndex != INDEX_NONE)
				{
					ControlPerShapeActor[ShapeActorIndex] = Controls[ControlIndex];
					Controls.RemoveAtSwap(ControlIndex);
				}
			}
			else //no actor just recreate
			{
				return false;
			}
		}
	}

	// Some controls don't have associated shape so recreate them 
	if (!Controls.IsEmpty())
	{
		return false;
	}

	// we have matching controls - we should at least sync their settings.
	// PostPoseUpdate / TickControlShape is going to take care of color, visibility etc.
	// MeshTransform has to be handled here.
	const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>& ShapeLibraries = InControlRig->GetShapeLibraries();
	for (int32 ShapeActorIndex = 0; ShapeActorIndex < NumShapes; ShapeActorIndex++)
	{
		AControlRigShapeActor* ShapeActor = Shapes[ShapeActorIndex].Get();
		FRigControlElement* ControlElement = ControlPerShapeActor[ShapeActorIndex];
		if (ShapeActor && ControlElement)
		{
			const FTransform ShapeTransform = Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
			if (const FControlRigShapeDefinition* ShapeDef = UControlRigShapeLibrary::GetShapeByName(ControlElement->Settings.ShapeName, ShapeLibraries, InControlRig->ShapeLibraryNameMap))
			{
				const FTransform& MeshTransform = ShapeDef->Transform;
				if (UObject* ShapeProxy = ShapeDef->GetShapeProxy())
				{
					UPrimitiveComponent* Component = ShapeActor->GetProxyComponent(ShapeProxy);	
					if (Component && ShapeDef->PostSetupFunction)
					{
						ShapeDef->PostSetupFunction(ShapeActor);
					}
				}

				if (UPrimitiveComponent* ProxyComponent = ShapeActor->ProxyComponent.Get())
				{
					ProxyComponent->SetRelativeTransform(MeshTransform * ShapeTransform);
				}
			}
			else
			{
				if (UPrimitiveComponent* ProxyComponent = ShapeActor->ProxyComponent.Get())
				{
					ProxyComponent->SetRelativeTransform(ShapeTransform);
				}
			}
		}
	}

	// equivalent to PostPoseUpdate for those shapes only
	FTransform ComponentTransform = FTransform::Identity;
	if (!AreEditingControlRigDirectly())
	{
		ComponentTransform = GetHostingSceneComponentTransform(InControlRig);
	}
		
	const FShapeUpdateParams Params(InControlRig, ComponentTransform, IsControlRigSkelMeshVisible(InControlRig), IsInLevelEditor(), !IsDisplayControlsEnabled());
	for (int32 ShapeActorIndex = 0; ShapeActorIndex < NumShapes; ShapeActorIndex++)
	{
		AControlRigShapeActor* ShapeActor = Shapes[ShapeActorIndex].Get();
		FRigControlElement* ControlElement = ControlPerShapeActor[ShapeActorIndex];
		if (ShapeActor && ControlElement)
		{
			UpdateControlShape(ShapeActor, ControlElement, Params);
		}

		// workaround for UE-225122, FPrimitiveSceneProxy currently lazily updates the transform, but due to a thread sync issue,
		// if we are setting the transform to 0 at tick 1 and setting it to the correct value like 100 at tick2, depending on
		// the value of the cached transform, only one of the two sets would be committed. This call clears the cached transform to 0 such that
		// set to 0(here) is always ignored and set to 100(TickControlShape) is always accepted.
		ShapeActor->MarkComponentsRenderStateDirty();
	}

	return true;
}

void FControlRigEditMode::RecreateControlShapeActors()
{
	if (RecreateControlShapesRequired == ERecreateControlRigShape::RecreateAll)
	{
		// recreate all control rigs shape actors
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* RuntimeControlRig = RuntimeRigPtr.Get())
			{
				DestroyShapesActors(RuntimeControlRig);
				CreateShapeActors(RuntimeControlRig);
			}
		}
		RecreateControlShapesRequired = ERecreateControlRigShape::RecreateNone;
		return;
	}

	if (ControlRigsToRecreate.IsEmpty())
	{
		// nothing to update
		return;
	}
	
	// update or recreate all control rigs in ControlRigsToRecreate
	TArray<UControlRig*> ControlRigsCopy = ControlRigsToRecreate;
	for (UControlRig* ControlRig : ControlRigsCopy)
	{
		if(!IsValid(ControlRig))
		{
			continue;
		}
		const bool bUpdated = TryUpdatingControlsShapes(ControlRig);
		if (!bUpdated)
		{
			DestroyShapesActors(ControlRig);
			CreateShapeActors(ControlRig);
		}
	}
	RecreateControlShapesRequired = ERecreateControlRigShape::RecreateNone;
	ControlRigsToRecreate.Empty();
}

void FControlRigEditMode::CreateShapeActors(UControlRig* InControlRig)
{
	if (URigVMHost::IsGarbageOrDestroyed(WorldPtr))
	{
		return;
	}
	if (URigVMHost::IsGarbageOrDestroyed(WorldPtr->GetCurrentLevel()))
	{
		return;
	}
	
	using namespace ControlRigEditMode::Shapes;
	
	if(bShowControlsAsOverlay)
	{
		// enable translucent selection
		GetMutableDefault<UEditorPerProjectUserSettings>()->bAllowSelectTranslucent = true;
	}

	const UControlRigEditModeSettings* Settings = GetSettings();
	const TArray<TSoftObjectPtr<UControlRigShapeLibrary>> ShapeLibraries = InControlRig->GetShapeLibraries();

	const int32 ControlRigIndex = RuntimeControlRigs.Find(InControlRig);
	const URigHierarchy* Hierarchy = InControlRig->GetHierarchy();

	// get controls for which shapes are needed in the editor
	TArray<FRigControlElement*> Controls;
	GetControlsEligibleForShapes(InControlRig, Controls);

	// new shape actors to be created
	TArray<AControlRigShapeActor*> NewShapeActors;
	NewShapeActors.Reserve(Controls.Num());

	TArray<const FControlRigShapeDefinition*> ShapeDefinitions;
	for (FRigControlElement* ControlElement : Controls)
	{
		const FRigControlSettings& ControlSettings = ControlElement->Settings;
		
		FControlShapeActorCreationParam Param;
		Param.ManipObj = InControlRig;
		Param.ControlRigIndex = ControlRigIndex;
		Param.ControlRig = InControlRig;
		Param.ControlName = ControlElement->GetFName();
		Param.ShapeName = ControlSettings.ShapeName;
		Param.SpawnTransform = InControlRig->GetControlGlobalTransform(ControlElement->GetFName());
		Param.ShapeTransform = Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
		Param.bSelectable = ControlSettings.IsSelectable(false);

		const FControlRigShapeDefinition* ShapeDef = UControlRigShapeLibrary::GetShapeByName(ControlSettings.ShapeName, ShapeLibraries, InControlRig->ShapeLibraryNameMap);
		if (ShapeDef)
		{
			Param.MeshTransform = ShapeDef->Transform;
			Param.Proxy = ShapeDef->GetShapeProxy();
			Param.Material = ShapeDef->Library->DefaultMaterial;
			if (bShowControlsAsOverlay)
			{
				TSoftObjectPtr<UMaterial> XRayMaterial = ShapeDef->Library->XRayMaterial;
				if (XRayMaterial.IsPending())
				{
					XRayMaterial.LoadSynchronous();
				}
				if (XRayMaterial.IsValid())
				{
					Param.Material = XRayMaterial;
				}
			}
			Param.ColorParameterName = ShapeDef->Library->MaterialColorParameter;
			Param.HoveredParameterName = ShapeDef->Library->MaterialHoveredParameter;
			Param.HoveredColorParameterName = ShapeDef->Library->MaterialHoveredColorParameter;
			if (Settings)
			{
				Param.HoveredColor = Settings->HoveredColor;
			}
			Param.PostSetupFunction = ShapeDef->PostSetupFunction;
		}

		Param.Color = ControlSettings.ShapeColor;

		// create a new shape actor that will represent that control in the editor
		AControlRigShapeActor* NewShapeActor = FControlRigShapeHelper::CreateDefaultShapeActor(WorldPtr, Param);
		if (NewShapeActor)
		{
			//not drawn in game or in game view.
			NewShapeActor->SetActorHiddenInGame(true);
			NewShapeActors.Add(NewShapeActor);
			ShapeDefinitions.Add(ShapeDef);
		}
	}

	// add or replace shape actors
	auto* ShapeActors = ControlRigShapeActors.Find(InControlRig);
	if (ShapeActors)
	{
		// this shouldn't happen but make sure we destroy any existing shape
		DestroyShapesActorsFromWorld(*ShapeActors);
		*ShapeActors = NewShapeActors;
	}
	else
	{
		ShapeActors = &ControlRigShapeActors.Emplace(InControlRig, ObjectPtrWrap(NewShapeActors));
	}

	// setup shape actors
	if(ensure(ShapeActors))
	{
		const USceneComponent* Component = GetHostingSceneComponent(InControlRig);
		if (AActor* PreviewActor = Component ? Component->GetOwner() : nullptr)
		{
			for (int32 Index = 0; Index < ShapeActors->Num(); Index++)
			{
				if (AControlRigShapeActor* ShapeActor = ShapeActors->operator[](Index))
				{
					// attach to preview actor, so that we can communicate via relative transform from the preview actor
					// use the root component explicitly to avoid circular dependencies for actors where the default
					// attachment component isn't the root
					USceneComponent* ParentComponent = PreviewActor->GetRootComponent();
					if (ParentComponent)
					{
						ShapeActor->AttachToComponent(ParentComponent, FAttachmentTransformRules::KeepWorldTransform);
					}					

					TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
					ShapeActor->GetComponents(PrimitiveComponents, true);
					for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
					{
						PrimitiveComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FControlRigEditMode::ShapeSelectionOverride);
						PrimitiveComponent->PushSelectionToProxy();
					}

					if (const FControlRigShapeDefinition* ShapeDefinition = ShapeDefinitions[Index])
					{
						if (ShapeDefinition->PostSetupFunction)
						{
							ShapeDefinition->PostSetupFunction(ShapeActor);
						}
					}
				}
			}
		}
	}
	
	if (!AreEditingControlRigDirectly())
	{
		if (DetailKeyFrameCache)
		{
			DetailKeyFrameCache->ResetCachedData();
		}
	}

	OnControlRigShapeActorsRecreatedDelegate.Broadcast();
}

FControlRigEditMode* FControlRigEditMode::GetEditModeFromWorldContext(UWorld* InWorldContext)
{
	return nullptr;
}

bool FControlRigEditMode::ShapeSelectionOverride(const UPrimitiveComponent* InComponent) const
{
    //Think we only want to do this in regular editor, in the level editor we are driving selection
	if (AreEditingControlRigDirectly())
	{
	    AControlRigShapeActor* OwnerActor = Cast<AControlRigShapeActor>(InComponent->GetOwner());
	    if (OwnerActor)
	    {
		    // See if the actor is in a selected unit proxy
		    return OwnerActor->IsSelected();
	    }
	}

	return false;
}

void FControlRigEditMode::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	bool bHasAnyRigBeenReplaced = false;
	
	const TArray<TWeakObjectPtr<UControlRig>> PreviousRuntimeRigs(RuntimeControlRigs);
	for (const TWeakObjectPtr<UControlRig>& WeakControlRig: PreviousRuntimeRigs)
	{
		if (UControlRig* OldControlRig = WeakControlRig.Get())
		{
			if (UControlRig* NewControlRig = Cast<UControlRig>(OldToNewInstanceMap.FindRef(OldControlRig)))
			{
				// remove old rig (this will also remove it from RuntimeControlRigs)
				RemoveControlRig(OldControlRig);

				// add new rig
				AddControlRigInternal(NewControlRig);
				NewControlRig->Initialize();

				CopyControlsVisibility(OldControlRig, NewControlRig);

				bHasAnyRigBeenReplaced = true;
			}
		}
	}

	if (bHasAnyRigBeenReplaced)
	{
		SetObjects_Internal();
	}
}

void FControlRigEditMode::CopyControlsVisibility(const UControlRig* SourceRig, UControlRig* TargetRig)
{
	TargetRig->bControlsVisible = SourceRig->bControlsVisible;
    if (const UModularRig* SourceModularRig = Cast<UModularRig>(SourceRig))
    {
    	if (UModularRig* TargetModularRig = Cast<UModularRig>(TargetRig))
    	{
    		for (const FRigModuleInstance& SourceModule : SourceModularRig->Modules)
    		{
    			if (FRigModuleInstance* TargetModule = TargetModularRig->FindModule(SourceModule.Name))
    			{
    				UControlRig* SourceModuleRig = SourceModule.GetRig();
    				UControlRig* TargetModuleRig = TargetModule->GetRig();
    				if (SourceModuleRig && TargetModuleRig)
    				{
    					TargetModuleRig->bControlsVisible = SourceModuleRig->bControlsVisible;
    				}
    			}
    		}
    	}
    }
}

bool FControlRigEditMode::IsTransformDelegateAvailable() const
{
	return (OnGetRigElementTransformDelegate.IsBound() && OnSetRigElementTransformDelegate.IsBound());
}

namespace UE::Private
{

bool IsControlSelectedAndTransformable(const URigHierarchy* InHierarchy, const FRigElementKey& InSelectedKey)
{
    if (!FRigElementTypeHelper::DoesHave(FControlRigEditMode::ValidControlTypeMask(), InSelectedKey.Type))
    {
    	return false;
    }
    
    const FRigControlElement* ControlElement = InHierarchy ? InHierarchy->Find<FRigControlElement>(InSelectedKey) : nullptr;
    if (!ControlElement)
    {
    	return false;
    }
    
	constexpr bool bRespectVisibility = false;
    return ControlElement->Settings.IsSelectable(bRespectVisibility);
}
	
}

bool FControlRigEditMode::AreRigElementSelectedAndMovable(UControlRig* InControlRig) const
{
	if (!InControlRig)
	{
		return false;
	}

	// no need to look for transient controls when animating in the level editor
	if (IsInLevelEditor())
	{
		const URigHierarchy* Hierarchy = InControlRig->GetHierarchy();
		if (!Hierarchy)
		{
			return false;
		}
			
		return Hierarchy->HasAnythingSelectedByPredicate([Hierarchy](const FRigElementKey& InSelectedKey)
		{
			return UE::Private::IsControlSelectedAndTransformable(Hierarchy, InSelectedKey);
		});
	}

	auto IsAnySelectedControlMovable = [this, InControlRig]()
	{
		const TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(InControlRig);

		const URigHierarchy* Hierarchy = InControlRig->GetHierarchy();
		return SelectedRigElements.ContainsByPredicate([Hierarchy](const FRigElementKey& InSelectedKey)
		{
			return UE::Private::IsControlSelectedAndTransformable(Hierarchy, InSelectedKey);
		});
	};
	
	if (!IsAnySelectedControlMovable())
	{
		return false;
	}

	//when in sequencer/level we don't have that delegate so don't check.
	if (AreEditingControlRigDirectly())
	{
		if (!IsTransformDelegateAvailable())
		{
			return false;
		}
	}
	else //do check for the binding though
	{
		// if (GetHostingSceneComponent(ControlRig) == nullptr)
		// {
		// 	return false;
		// }
	}

	return true;
}

void FControlRigEditMode::ReplaceControlRig(UControlRig* OldControlRig, UControlRig* NewControlRig)
{
	if (OldControlRig != nullptr)
	{
		RemoveControlRig(OldControlRig);
	}
	AddControlRigInternal(NewControlRig);
	SetObjects_Internal();
	RequestToRecreateControlShapeActors(NewControlRig);

	CopyControlsVisibility(OldControlRig, NewControlRig);
}
void FControlRigEditMode::OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;

	if(bSuspendHierarchyNotifs)
	{
		return;
	}

	// Recreate shape actors when the hierarchy is reset, even when the element is null.
	// This handles cases where another rig is compiled and somehow affects the shape actors of this rig.
	if (InNotif == ERigHierarchyNotification::HierarchyReset)
	{
		RequestToRecreateControlShapeActors();
	}

	if (!InElement)
	{
		return;
	}

	check(InElement);
	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementAdded:
		case ERigHierarchyNotification::ElementRemoved:
		case ERigHierarchyNotification::ElementRenamed:
		case ERigHierarchyNotification::ElementReordered:
		case ERigHierarchyNotification::HierarchyReset:
		{
			UControlRig* ControlRig = InHierarchy->GetTypedOuter<UControlRig>();
			RequestToRecreateControlShapeActors(ControlRig);
			break;
		}
		case ERigHierarchyNotification::ControlSettingChanged:
		case ERigHierarchyNotification::ControlVisibilityChanged:
		case ERigHierarchyNotification::ControlShapeTransformChanged:
		{			
			const FRigElementKey Key = InElement->GetKey();
			UControlRig* ControlRig = InHierarchy->GetTypedOuter<UControlRig>();
			if (Key.Type == ERigElementType::Control)
			{
				if (const FRigControlElement* ControlElement = Cast<FRigControlElement>(InElement))
				{
					if (AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig,Key.Name))
					{
						// try to lazily apply the changes to the actor
						const bool bHideManipulators = !IsDisplayControlsEnabled();
						if (ShapeActor->UpdateControlSettings(InNotif, ControlRig, ControlElement, bHideManipulators, !AreEditingControlRigDirectly()))
						{
							break;
						}
					}
				}
			}

			if(ControlRig != nullptr)
			{
				// if we can't deal with this lazily, let's fall back to recreating all control shape actors
				RequestToRecreateControlShapeActors(ControlRig);
			}
			break;
		}
		case ERigHierarchyNotification::ControlDrivenListChanged:
		{
			if (!AreEditingControlRigDirectly())
			{
				// to synchronize the selection between the viewport / editmode and the details panel / sequencer
				// we re-select the control. during deselection we recover the previously set driven list
				// and then select the control again with the up2date list. this makes sure that the tracks
				// are correctly selected in sequencer to match what the proxy control is driving.
				if (FRigControlElement* ControlElement = InHierarchy->Find<FRigControlElement>(InElement->GetKey()))
				{
					const UControlRig* ControlRig = InHierarchy->GetTypedOuter<UControlRig>();
					const UAnimDetailsSelection* Selection = AnimDetailsProxyManager ? AnimDetailsProxyManager->GetAnimDetailsSelection() : nullptr;

					if (Selection && Selection->IsControlElementSelected(ControlRig, ControlElement))
					{
						// reselect the control - to affect the details panel / sequencer
						if(URigHierarchyController* Controller = InHierarchy->GetController())
						{
							const FRigElementKey Key = ControlElement->GetKey();
							{
								// Restore the previously selected driven elements
								// so that we can deselect them accordingly.
								TGuardValue<TArray<FRigElementKey>> DrivenGuard(
									ControlElement->Settings.DrivenControls,
									ControlElement->Settings.PreviouslyDrivenControls);
								
								Controller->DeselectElement(Key);
							}

							// now select the proxy control again given the new driven list
							Controller->SelectElement(Key);
						}
					}
				}
			}
			break;
		}
		case ERigHierarchyNotification::ElementSelected:
		case ERigHierarchyNotification::ElementDeselected:
		{
			const FRigElementKey Key = InElement->GetKey();

			switch (InElement->GetType())
			{
				case ERigElementType::Bone:
            	case ERigElementType::Null:
            	case ERigElementType::Curve:
            	case ERigElementType::Control:
            	case ERigElementType::Physics:
            	case ERigElementType::Reference:
            	case ERigElementType::Connector:
            	case ERigElementType::Socket:
				{
					const bool bSelected = InNotif == ERigHierarchyNotification::ElementSelected;
					// users may select gizmo and control rig units, so we have to let them go through both of them if they do
						// first go through gizmo actor
					UControlRig* ControlRig = InHierarchy->GetTypedOuter<UControlRig>();
					if (ControlRig == nullptr)
					{
						if (RuntimeControlRigs.Num() > 0)
						{
							ControlRig = RuntimeControlRigs[0].Get();
						}
					}
					if (ControlRig)
					{
						OnControlRigSelectedDelegate.Broadcast(ControlRig, Key, bSelected);
					}
					else
					{
						return;
					}
					
					// if it's control
					if (Key.Type == ERigElementType::Control)
					{
						FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !AreEditingControlRigDirectly() && !GIsTransacting);
						AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig,Key.Name);
						if (ShapeActor)
						{
							ShapeActor->SetSelected(bSelected);
						}

						const FRigControlElement* ControlElement = ControlRig->GetHierarchy()->Find<FRigControlElement>(Key);
						if (!AreEditingControlRigDirectly() &&
							ControlElement &&
							ControlElement->CanDriveControls())
						{
							const UControlRigEditModeSettings* Settings = GetSettings();

							const TArray<FRigElementKey>& DrivenKeys = ControlElement->Settings.DrivenControls;
							for(const FRigElementKey& DrivenKey : DrivenKeys)
							{
								if (FRigControlElement* DrivenControl = ControlRig->GetHierarchy()->Find<FRigControlElement>(DrivenKey))
								{
									if (AControlRigShapeActor* DrivenShapeActor = GetControlShapeFromControlName(ControlRig,DrivenControl->GetFName()))
									{
										if(bSelected)
										{
											DrivenShapeActor->OverrideColor = Settings->DrivenControlColor;
										}
										else
										{
											DrivenShapeActor->OverrideColor = FLinearColor(0, 0, 0, 0);
										}
									}
								}
							}
						}
					}
						
					if (!bSelectionChanged)
					{
						DirectControl.Reset();
					}
						
					bSelectionChanged = true;
		
					break;
				}
				default:
				{
					ensureMsgf(false, TEXT("Unsupported Type of RigElement: %s"), *Key.ToString());
					break;
				}
			}
		}
		case ERigHierarchyNotification::ParentWeightsChanged:
		{
			break;
		}
		case ERigHierarchyNotification::InteractionBracketOpened:
		case ERigHierarchyNotification::InteractionBracketClosed:
		default:
		{
			break;
		}
	}
}

void FControlRigEditMode::OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if(bSuspendHierarchyNotifs)
	{
		return;
	}

	if(bIsConstructionEventRunning)
	{
		return;
	}

	if(IsInGameThread())
	{
		OnHierarchyModified(InNotif, InHierarchy, InSubject);
		return;
	}

	if (InNotif != ERigHierarchyNotification::ControlSettingChanged &&
		InNotif != ERigHierarchyNotification::ControlVisibilityChanged &&
		InNotif != ERigHierarchyNotification::ControlDrivenListChanged &&
		InNotif != ERigHierarchyNotification::ControlShapeTransformChanged &&
		InNotif != ERigHierarchyNotification::ElementSelected &&
		InNotif != ERigHierarchyNotification::ElementDeselected)
	{
		OnHierarchyModified(InNotif, InHierarchy, InSubject);
		return;
	}
	
	FRigElementKey ElementKey;
	FRigComponentKey ComponentKey;
	if(InSubject.Element)
	{
		ElementKey = InSubject.Element->GetKey();
	}
	else if(InSubject.Component)
	{
		ElementKey = InSubject.Component->GetElementKey();
		ComponentKey = InSubject.Component->GetKey();
	}

	TWeakObjectPtr<URigHierarchy> WeakHierarchy = InHierarchy;

	ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, InNotif, WeakHierarchy, ElementKey, ComponentKey]()
	{
		if(!WeakHierarchy.IsValid())
		{
			return;
		}
		if(const FRigBaseComponent* Component = WeakHierarchy.Get()->FindComponent(ComponentKey))
		{
			OnHierarchyModified(InNotif, WeakHierarchy.Get(), Component);
		}
		else if (const FRigBaseElement* Element = WeakHierarchy.Get()->Find(ElementKey))
		{
			OnHierarchyModified(InNotif, WeakHierarchy.Get(), Element);
		}
		
	});
}

void FControlRigEditMode::OnControlModified(UControlRig* Subject, FRigControlElement* InControlElement, const FRigControlModifiedContext& Context)
{			
	// This makes sure the details panel ui get's updated, don't remove.
	// This may be called from other threads, but only calls on the game thread are not relevant to update the anim details.
	if (IsInGameThread() && AnimDetailsProxyManager)
	{	
		// do not propagate the change to the anim details when playing for performances reasons
		if (!bSequencerPlaying)
		{
			AnimDetailsProxyManager->RequestUpdateProxyValues();
		}
	}

	bPivotsNeedUpdate = true;
}

void FControlRigEditMode::OnPreConstruction_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	bIsConstructionEventRunning = true;
}

void FControlRigEditMode::OnPostConstruction_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	bIsConstructionEventRunning = false;

	const int32 RigIndex = RuntimeControlRigs.Find(InRig);
	if(!LastHierarchyHash.IsValidIndex(RigIndex) || !LastShapeLibraryHash.IsValidIndex(RigIndex))
	{
		return;
	}
	
	const int32 HierarchyHash = InRig->GetHierarchy()->GetTopologyHash(false, true);
	const int32 ShapeLibraryHash = InRig->GetShapeLibraryHash();
	if((LastHierarchyHash[RigIndex] != HierarchyHash) ||
		(LastShapeLibraryHash[RigIndex] != ShapeLibraryHash))
	{
		LastHierarchyHash[RigIndex] = HierarchyHash;
		LastShapeLibraryHash[RigIndex] = ShapeLibraryHash;

		auto Task = [this, InRig]()
		{
			RequestToRecreateControlShapeActors(InRig);
			RecreateControlShapeActors();
			HandleSelectionChanged();
			if (DetailKeyFrameCache)
			{
				DetailKeyFrameCache->ResetCachedData();
			}
		};
				
		if(IsInGameThread())
		{
			Task();
		}
		else
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [Task]()
			{
				Task();
			});
		}
	}
}

void FControlRigEditMode::OnWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode)
{
	const UControlRigEditModeSettings* Settings = GetSettings();
	if (Settings && Settings->bCoordSystemPerWidgetMode)
	{
		TGuardValue<bool> ReentrantGuardSelf(bIsChangingCoordSystem, true);

		FEditorModeTools* ModeManager = GetModeManager();
		int32 WidgetMode = (int32)ModeManager->GetWidgetMode();
		if (WidgetMode >= 0 && WidgetMode < CoordSystemPerWidgetMode.Num())
		{
			ModeManager->SetCoordSystem(CoordSystemPerWidgetMode[WidgetMode]);
		}
	}
}

void FControlRigEditMode::OnCoordSystemChanged(ECoordSystem InCoordSystem)
{
	TGuardValue<bool> ReentrantGuardSelf(bIsChangingCoordSystem, true);

	FEditorModeTools* ModeManager = GetModeManager();
	int32 WidgetMode = (int32)ModeManager->GetWidgetMode();
	ECoordSystem CoordSystem = ModeManager->GetCoordSystem();
	if (WidgetMode >= 0 && WidgetMode < CoordSystemPerWidgetMode.Num())
	{
		CoordSystemPerWidgetMode[WidgetMode] = CoordSystem;
	}
}

bool FControlRigEditMode::CanChangeControlShapeTransform()
{
	if (AreEditingControlRigDirectly())
	{
		for (TWeakObjectPtr<UControlRig> RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* ControlRig = RuntimeRigPtr.Get())
			{
				TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
				// do not allow multi-select
				if (SelectedRigElements.Num() == 1)
				{
					if (AreRigElementsSelected(ValidControlTypeMask(),ControlRig))
					{
						// only enable for a Control with Gizmo enabled and visible
						if (FRigControlElement* ControlElement = ControlRig->GetHierarchy()->Find<FRigControlElement>(SelectedRigElements[0]))
						{
							if (ControlElement->Settings.IsVisible())
							{
								if (AControlRigShapeActor* ShapeActor = GetControlShapeFromControlName(ControlRig,SelectedRigElements[0].Name))
								{
									if (ensure(ShapeActor->IsSelected()))
									{
										return true;
									}
								}
							}
						}
						
					}
				}
			}
		}
	}

	return false;
}

void FControlRigEditMode::OnSettingsChanged(const UControlRigEditModeSettings* InSettings)
{
	if (!InSettings)
	{
		return;
	}
	
	// check if the settings for xray rendering are different for any of the control shape actors
	if(bShowControlsAsOverlay != IsControlsAsOverlayEnabled())
	{
		bShowControlsAsOverlay = IsControlsAsOverlayEnabled();
		for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
		{
			if (UControlRig* RuntimeControlRig = RuntimeRigPtr.Get())
			{
				UpdateSelectabilityOnSkeletalMeshes(RuntimeControlRig, !bShowControlsAsOverlay);
			}
		}
		RequestToRecreateControlShapeActors();
	}
}

void FControlRigEditMode::OnSettingsPropertyChanged(UObject* InObject, FPropertyChangedEvent& PropertyChangedEvent)
{
	UControlRigEditModeSettings* Settings = Cast<UControlRigEditModeSettings>(InObject);
	if (!Settings)
	{
		return;
	}
	
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UControlRigEditModeSettings, HoveredColor))
	{
		if (RecreateControlShapesRequired == ERecreateControlRigShape::RecreateNone)
		{
			for (const auto&[ControlRig, ShapeActors]: ControlRigShapeActors)
			{
				if (ControlRig.IsValid())
				{
					for (AControlRigShapeActor* ShapeActor : ShapeActors)
					{
						if (ShapeActor)
						{
							ShapeActor->SetHoveredColor(Settings->HoveredColor);
						}
					}
				}
			}
		}
	}
}

void FControlRigEditMode::SetControlShapeTransform(
	const AControlRigShapeActor* InShapeActor,
	const FTransform& InGlobalTransform,
	const FTransform& InToWorldTransform,
	const FRigControlModifiedContext& InContext,
	const bool bPrintPython,
	const FControlRigInteractionTransformContext& InTransformContext,
	const bool bFixEulerFlips) const
{
	UControlRig* ControlRig = InShapeActor->ControlRig.Get();
	URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return;
	}

	FRigControlElement* ControlElement = ControlRig->FindControl(InShapeActor->ControlName);
	if (!ControlElement)
	{
		return;
	}

	static constexpr bool bNotify = true, bUndo = true;
	const FExplicitRotationInteraction ExplicitRotation(InTransformContext, ControlRig, Hierarchy, ControlElement, InToWorldTransform);
	const bool bApplyExplicitRotation = ExplicitRotation.IsValid();

	if (AreEditingControlRigDirectly())
	{
		if (bApplyExplicitRotation)
		{
			ExplicitRotation.Apply(InGlobalTransform, InContext, bPrintPython);
		}
		else
		{
			// assumes it's attached to actor
			ControlRig->SetControlGlobalTransform(
				InShapeActor->ControlName, InGlobalTransform, bNotify, InContext, bUndo, bPrintPython, bFixEulerFlips);
		}
		return;
	}

	auto EvaluateRigIfAdditive = [ControlRig]()
	{
		// skip compensation and evaluate the rig to force notifications: auto-key and constraints updates (among others) are based on
		// UControlRig::OnControlModified being broadcast but this only happens on evaluation for additive rigs.
		// constraint compensation is disabled while manipulating in that case to avoid re-entrant evaluations 
		if (ControlRig->IsAdditive())
		{
			TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
			const TGuardValue<float> AbsoluteTimeGuard(ControlRig->AbsoluteTime, ControlRig->AbsoluteTime);
			const TGuardValue<bool> GuardEvaluationType(ControlRig->bEvaluationTriggeredFromInteraction, true);
			ControlRig->Evaluate_AnyThread();
		}
	};
	
	// find the last constraint in the stack (this could be cached on mouse press)
	UWorld* World = ControlRig->GetWorld();
	const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, InShapeActor->ControlName);
	const bool bNeedsConstraintPostProcess = ConstraintsCache.HasAnyActiveConstraint(ControlHash, World);
	static const TArray< TWeakObjectPtr<UTickableConstraint> > EmptyConstraints;
	const TArray< TWeakObjectPtr<UTickableConstraint> >& Constraints = bNeedsConstraintPostProcess ? ConstraintsCache.Get(ControlHash, World) : EmptyConstraints;

	// set the global space, assumes it's attached to actor
	// no need to compensate for constraints here, this will be done after when setting the control in the constraint space
	{
		TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
		if (bApplyExplicitRotation)
		{
			ExplicitRotation.Apply(InGlobalTransform, InContext, bPrintPython, Constraints);
		}
		else
		{
			ControlRig->SetControlGlobalTransform(
					InShapeActor->ControlName, InGlobalTransform, bNotify, InContext, bUndo, bPrintPython, bFixEulerFlips);
			EvaluateRigIfAdditive();
		}
	}

	FTransform LocalTransform = ControlRig->GetControlLocalTransform(InShapeActor->ControlName);

	FControlKeyframeData KeyframeData;
	KeyframeData.LocalTransform = LocalTransform;
	
	if (bNeedsConstraintPostProcess)
	{
		if (!bApplyExplicitRotation)
		{
			// switch to constraint space
			const FTransform WorldTransform = InGlobalTransform * InToWorldTransform;

			const TOptional<FTransform> RelativeTransform =
				UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, LocalTransform, WorldTransform);
			if (RelativeTransform)
			{
				LocalTransform = *RelativeTransform;
				KeyframeData.LocalTransform = LocalTransform;
				KeyframeData.bConstraintSpace = true;
			}

			FRigControlModifiedContext Context = InContext;
			Context.bConstraintUpdate = false;
	
			ControlRig->SetControlLocalTransform(InShapeActor->ControlName, LocalTransform, bNotify, Context, bUndo, bFixEulerFlips);
			EvaluateRigIfAdditive();
		}
		else
		{
			KeyframeData.bConstraintSpace = true;
		}
	}

	if (bNeedsConstraintPostProcess || ConstraintsCache.HasAnyActiveConstraint(GetHostingSceneComponent(ControlRig), World))
	{
		TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.EvaluateAllConstraints();
	}

	Keyframer.Store(ControlHash, MoveTemp(KeyframeData));
}

FTransform FControlRigEditMode::GetControlShapeTransform(const AControlRigShapeActor* ShapeActor)
{
	if (const UControlRig* ControlRig = ShapeActor->ControlRig.Get())
	{
		return ControlRig->GetControlGlobalTransform(ShapeActor->ControlName);
	}
	return FTransform::Identity;
}

bool FControlRigEditMode::MoveControlShapeLocally( AControlRigShapeActor* ShapeActor,
	const FControlRigInteractionTransformContext& InTransformContext, const FTransform& ToWorldTransform,
	const FTransform& InLocal)
{
	if (!ensure(InTransformContext.CanTransform()))
	{
		return false;
	}
	
	UControlRig* ControlRig = ShapeActor ? ShapeActor->ControlRig.Get() : nullptr;
	if (!ensure(ControlRig))
	{
		return  false;
	}

	bool bTransformChanged = false;
	
	FTransform CurrentLocalTransform = ControlRig->GetControlLocalTransform(ShapeActor->ControlName);

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	FRigControlElement* Control = Hierarchy ? Hierarchy->Find<FRigControlElement>(ShapeActor->GetElementKey()) : nullptr;
	
	if (InTransformContext.bRotation)
	{
		FQuat CurrentRotation = CurrentLocalTransform.GetRotation();

		FQuat DeltaRot = InLocal.GetRotation();

		if (ensure(Control))
		{
			switch(InTransformContext.Space)
			{
			case EControlRigInteractionTransformSpace::World:
			case EControlRigInteractionTransformSpace::Explicit:
				ensure(false);
				break;
			case EControlRigInteractionTransformSpace::Local:
				break;
			case EControlRigInteractionTransformSpace::Parent:
				{
					const int32 NumParents = Hierarchy->GetNumberOfParents(Control);
					const FTransform Global = Hierarchy->GetTransform(Control, ERigTransformType::CurrentGlobal);
					const FTransform Parent = NumParents > 0 ? Hierarchy->GetParentTransform(Control, ERigTransformType::CurrentGlobal) : Global;

					FTransform RelativeToParent = Global.GetRelativeTransform(Parent);
					RelativeToParent = RelativeToParent * DeltaRot;
					const FTransform NewGlobal = RelativeToParent * Parent;
					
					DeltaRot = NewGlobal.GetRelativeTransform(Global).GetRotation();
					break;
				}
			}
		}

		CurrentRotation = (CurrentRotation * DeltaRot);
		
		CurrentLocalTransform.SetRotation(CurrentRotation);
		bTransformChanged = true;
	}

	if (InTransformContext.bTranslation)
	{
		FVector CurrentLocation = CurrentLocalTransform.GetLocation();

		FVector Delta = InLocal.GetLocation();

		if (ensure(Control))
		{
			switch(InTransformContext.Space)
			{
			case EControlRigInteractionTransformSpace::World:
				ensure(false);
				break;
			case EControlRigInteractionTransformSpace::Local:
			{
				// in local mode, the incoming delta is expressed in the global space so it has to be switched back to the offset space
				const FTransform Global = Hierarchy->GetTransform(Control, ERigTransformType::CurrentGlobal);
				const FVector GlobalDelta = Global.TransformVector(Delta);
				const FTransform Offset = Hierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentGlobal);
				Delta = Offset.InverseTransformVector(GlobalDelta);
				break;
			}
			case EControlRigInteractionTransformSpace::Parent:
			{
				// in parent mode, the incoming delta is expressed in the parent space so it has to be switched back to the offset space
				FTransform Parent = FTransform::Identity;  
				if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(ControlRig, ShapeActor->ControlName))
				{
					Parent = ConstraintSpace->GetRelativeTransform(ToWorldTransform);
				}
				else
				{
					const int32 NumParents = Hierarchy->GetNumberOfParents(Control);
					Parent = NumParents > 0 ?
						Hierarchy->GetParentTransform(Control, ERigTransformType::CurrentGlobal) :
						Hierarchy->GetTransform(Control, ERigTransformType::CurrentGlobal);
				}
				const FVector GlobalDelta = Parent.TransformVector(Delta);
				const FTransform Offset = Hierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentGlobal);
				Delta = Offset.InverseTransformVector(GlobalDelta);
				break;
			}
			case EControlRigInteractionTransformSpace::Explicit:
			{
				// nothing to do as it has already been computed in the right space
				break;
			}
			}
			
			CurrentLocation = CurrentLocation + Delta;
		}
		CurrentLocalTransform.SetLocation(CurrentLocation);
		bTransformChanged = true;
	}

	if (bTransformChanged)
	{
		ControlRig->InteractionType = InteractionType;
		ControlRig->ElementsBeingInteracted.AddUnique(ShapeActor->GetElementKey());

		const bool bDeferAutokey = Keyframer.IsEnabled();
		const FRigControlModifiedContext Context(bDeferAutokey ? EControlRigSetKey::Never : EControlRigSetKey::DoNotCare);
		ControlRig->SetControlLocalTransform(ShapeActor->ControlName, CurrentLocalTransform,true, Context, true, /*fix eulers*/ true);

		FTransform CurrentTransform  = ControlRig->GetControlGlobalTransform(ShapeActor->ControlName);			// assumes it's attached to actor
		CurrentTransform = ToWorldTransform * CurrentTransform;

		// make the transform relative to the offset transform again.
		// first we'll make it relative to the offset used at the time of starting the drag
		// and then we'll make it absolute again based on the current offset. these two can be
		// different if we are interacting on a control on an animated character
		CurrentTransform = CurrentTransform.GetRelativeTransform(ShapeActor->OffsetTransform);
		CurrentTransform = CurrentTransform * ControlRig->GetHierarchy()->GetGlobalControlOffsetTransform(ShapeActor->GetElementKey(), false);

		// Don't set the global transform to the shape actor to avoid drifting
		//ShapeActor->SetGlobalTransform(CurrentTransform);

		RigsToEvaluateDuringThisTick.AddUnique(ControlRig);

		if (bDeferAutokey)
		{
			FControlKeyframeData KeyframeData;
			KeyframeData.LocalTransform = ControlRig->GetControlLocalTransform(ShapeActor->ControlName);
			const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, ShapeActor->ControlName);
			Keyframer.Store(ControlHash, MoveTemp(KeyframeData));
		}
	}

	return bTransformChanged; 
}

void FControlRigEditMode::MoveControlShape( AControlRigShapeActor* ShapeActor,
	const FControlRigInteractionTransformContext& InTransformContext, const FTransform& ToWorldTransform,
	const bool bUseLocal, const bool bCalcLocal, FTransform* InOutLocal,
	TArray<TFunction<void()>>& OutTasks)
{
	if (!ensure(InTransformContext.CanTransform()))
	{
		return;
	}
	
	UControlRig* ControlRig = ShapeActor ? ShapeActor->ControlRig.Get() : nullptr;
	if (!ensure(ControlRig))
	{
		return;
	}

	// In case for some reason the shape actor was detached, make sure to attach it again to the scene component
	if (!ShapeActor->GetAttachParentActor())
	{
		if (USceneComponent* SceneComponent = GetHostingSceneComponent(ControlRig))
		{
			if (AActor* OwnerActor = SceneComponent->GetOwner())
			{
				// attach to preview actor, so that we can communicate via relative transform from the preview actor
				// use the root component explicitly to avoid circular dependencies for actors where the default
				// attachment component isn't the root
				USceneComponent* ParentComponent = OwnerActor->GetRootComponent();
				if (ParentComponent)
				{
					ShapeActor->AttachToComponent(ParentComponent, FAttachmentTransformRules::KeepWorldTransform);
				}				
			}
		}
	}
	
	// first case is where we do all controls by the local diff.
	bool bTransformChanged = false;
	if (bUseLocal && InOutLocal)
	{
		bTransformChanged = MoveControlShapeLocally(ShapeActor, InTransformContext, ToWorldTransform, *InOutLocal);
		if (bTransformChanged)
		{
			return;
		}
	}
	// else: world, explicit or doing scale.

	FInteractionDependencyCache& Dependencies = GetInteractionDependencies(ControlRig);

	// for readability
	const bool bSolveImmediately = (!bUseLocal && bCalcLocal) ||
		Dependencies.HasDownwardDependencies(ShapeActor->GetElementKey()) ||
		Dependencies.CheckAndUpdateParentsPoseVersion();
	const bool bQueueTasks = !bSolveImmediately;

	if(bQueueTasks)
	{
		RigsToEvaluateDuringThisTick.AddUnique(ControlRig);
	}
	else
	{
		EvaluateRig(ControlRig);
	}

	// Get the global transform from shape actor to avoid drifting
	FTransform CurrentTransform;
	FRigControlElement* ControlElement = ControlRig->FindControl(ShapeActor->ControlName);
	if (ShapeActor->ProxyComponent.Get())
	{
		// Update ShapeActor in case we are moving multiple shapes, which affect one another
		const FTransform Transform = ControlRig->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal);
		ShapeActor->SetActorTransform(Transform * ToWorldTransform);

		if (const AActor* AttachParentActor = ShapeActor->GetAttachParentActor())
		{
			const FTransform& ParentTransform = AttachParentActor->GetTransform();
			CurrentTransform = ShapeActor->GetGlobalTransform() * ParentTransform;
		}
		else
		{
			CurrentTransform = ShapeActor->GetGlobalTransform();
		}
	}
	else
	{
		// If the shape proxy is not valid, we cannot rely on the shape's transform.
		// This happens for FKControlRigs (and other control types)
		// We will need to rely on the information we have in the rig hierarchy
		CurrentTransform = GetControlShapeTransform(ShapeActor) * ToWorldTransform;
	}
	const FTransform GlobalTransform(CurrentTransform);
	
	if (InTransformContext.bRotation)
	{
		FQuat CurrentRotation = CurrentTransform.GetRotation();
		CurrentRotation = (InTransformContext.Rot.Quaternion() * CurrentRotation);
		CurrentTransform.SetRotation(CurrentRotation);
		bTransformChanged = true;
	}

	if (InTransformContext.bTranslation)
	{
		FVector CurrentLocation = CurrentTransform.GetLocation();
		CurrentLocation = CurrentLocation + InTransformContext.Drag;
		CurrentTransform.SetLocation(CurrentLocation);
		bTransformChanged = true;
	}

	if (InTransformContext.bScale)
	{
		FVector CurrentScale = CurrentTransform.GetScale3D();
		CurrentScale = CurrentScale + InTransformContext.Scale;
		CurrentTransform.SetScale3D(CurrentScale);
		bTransformChanged = true;
	}

	if (bTransformChanged)
	{
		ControlRig->InteractionType = InteractionType;
		ControlRig->ElementsBeingInteracted.AddUnique(ShapeActor->GetElementKey());

		FTransform NewTransform = CurrentTransform.GetRelativeTransform(ToWorldTransform);
		
		FRigControlModifiedContext Context;
		Context.EventName = FRigUnit_BeginExecution::EventName;
		Context.bConstraintUpdate = true;
		
		const bool bDeferAutokey = Keyframer.IsEnabled();
		if (bDeferAutokey)
		{
			Context.SetKey = EControlRigSetKey::Never;
		}

		FTransform TransformSpace = FTransform::Identity; 
		if (bCalcLocal && InOutLocal)
		{
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			*InOutLocal = ControlRig->GetControlLocalTransform(ShapeActor->ControlName);

			switch (InTransformContext.Space)
			{
			case EControlRigInteractionTransformSpace::World:
				ensure(false);
				break;
			case EControlRigInteractionTransformSpace::Local:
				TransformSpace = GlobalTransform;
				break;
			case EControlRigInteractionTransformSpace::Parent:
				{
					if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(ControlRig, ShapeActor->ControlName))
					{
						TransformSpace = *ConstraintSpace;
					}
					else
					{
						const int32 NumParents = Hierarchy->GetNumberOfParents(ControlElement);
						if (NumParents > 0)
						{
							TransformSpace = Hierarchy->GetParentTransform(ControlElement, ERigTransformType::CurrentGlobal) * ToWorldTransform;
						}
						else
						{
							TransformSpace = GlobalTransform;
						}
					}
				}
				break;
			case EControlRigInteractionTransformSpace::Explicit:
				if (ensure(InTransformContext.bTranslation))
				{
					if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(ControlRig, ShapeActor->ControlName))
					{
						TransformSpace = *ConstraintSpace;
					}
					else
					{
						TransformSpace = Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentGlobal) * ToWorldTransform;	
					}
				}
				break;
			}
		}

		const UWorld* World = ControlRig->GetWorld();
		const bool bPrintPythonCommands = World ? World->IsPreviewWorld() : false;

		const bool bIsTransientControl = ControlElement ? ControlElement->Settings.bIsTransientControl : false;

		// if we are operating on a PIE instance which is playing we need to reapply the input pose
		// since the hierarchy will also have been brought into the solved pose. by reapplying the
		// input pose we avoid double transformation / double forward solve results.
		if(bIsTransientControl && World && World->IsPlayInEditor() && !World->IsPaused())
		{
			ControlRig->GetHierarchy()->SetPose(ControlRig->InputPoseOnDebuggedRig);
		}

		const FTransform ToWorldTransformCopy = ToWorldTransform;
		
		const TFunction<void()> SetControlShapeTask = [this, ShapeActor, ControlRig, ControlElement, NewTransform, ToWorldTransformCopy, Context,
			bPrintPythonCommands, bIsTransientControl, InTransformContext, bQueueTasks]()
		{
			//fix flips and do rotation orders only if not additive or fk rig
			const bool bFixEulerFlips = (!ControlRig->IsAdditive() || ControlRig->IsA<UFKControlRig>()) && InTransformContext.bRotation;
			SetControlShapeTransform(ShapeActor, NewTransform, ToWorldTransformCopy, Context, bPrintPythonCommands, InTransformContext, bFixEulerFlips);
			NotifyDrivenControls(ControlRig, ShapeActor->GetElementKey(),Context);

			if (ControlElement && !bIsTransientControl)
			{
				if(bQueueTasks)
				{
					RigsToEvaluateDuringThisTick.AddUnique(ControlRig);
				}
				else
				{
					EvaluateRig(ControlRig);
				}
			}
		};

		const TFunction<void()> SetGlobalAndUpdateLocalTask = [ShapeActor, CurrentTransform, ControlRig, bCalcLocal, InOutLocal, InTransformContext, TransformSpace]() {

			// Don't set the global transform to the shape actor to avoid drifting
			//ShapeActor->SetGlobalTransform(CurrentTransform);

			if (bCalcLocal && InOutLocal)
			{
				const FTransform NewLocal = ControlRig->GetControlLocalTransform(ShapeActor->ControlName);
				*InOutLocal = NewLocal.GetRelativeTransform(*InOutLocal);
				
				switch (InTransformContext.Space)
				{
					case EControlRigInteractionTransformSpace::World:
						break;
					case EControlRigInteractionTransformSpace::Local:
						if (InTransformContext.bTranslation)
						{
							InOutLocal->SetLocation(TransformSpace.InverseTransformVector(InTransformContext.Drag));
						}
						break;
					case EControlRigInteractionTransformSpace::Parent:
						if (InTransformContext.bTranslation)
						{
							InOutLocal->SetLocation(TransformSpace.InverseTransformVector(InTransformContext.Drag));
						}
						if (InTransformContext.bRotation)
						{
							FQuat SpaceRotation = TransformSpace.GetRotation();
							SpaceRotation = (InTransformContext.Rot.Quaternion() * SpaceRotation);
							InOutLocal->SetRotation(TransformSpace.InverseTransformRotation(SpaceRotation));
						}
						break;
					case EControlRigInteractionTransformSpace::Explicit:
						if (InTransformContext.bTranslation)
						{
							InOutLocal->SetLocation(TransformSpace.InverseTransformVector(InTransformContext.Drag));
						}
						break;
				}
			}
		};

		if(bQueueTasks)
		{
			OutTasks.Add(SetControlShapeTask);
			OutTasks.Add(SetGlobalAndUpdateLocalTask);
		}
		else
		{
			SetControlShapeTask();
			SetGlobalAndUpdateLocalTask();
		}
	}
}

void FControlRigEditMode::ChangeControlShapeTransform(AControlRigShapeActor* InShapeActor, const FControlRigInteractionTransformContext& InContext, const FTransform& ToWorldTransform)
{
	if (!InContext.CanTransform())
	{
		return;
	}
	
	UControlRig* ControlRig = InShapeActor->ControlRig.Get();
	URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return;
	}

	FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InShapeActor->GetElementKey());
	if (!ControlElement)
	{
		return;
	}

	FTransform CurrentTransform = Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentGlobal);
	CurrentTransform = CurrentTransform * ToWorldTransform;

	if (InContext.bRotation)
	{
		FQuat CurrentRotation = CurrentTransform.GetRotation();
		CurrentRotation = (InContext.Rot.Quaternion() * CurrentRotation);
		CurrentTransform.SetRotation(CurrentRotation);
	}

	if (InContext.bTranslation)
	{
		FVector CurrentLocation = CurrentTransform.GetLocation();
		CurrentLocation = CurrentLocation + InContext.Drag;
		CurrentTransform.SetLocation(CurrentLocation);
	}

	if (InContext.bScale)
	{
		FVector CurrentScale = CurrentTransform.GetScale3D();
		CurrentScale = CurrentScale + InContext.Scale;
		CurrentTransform.SetScale3D(CurrentScale);
	}

	FTransform NewTransform = CurrentTransform.GetRelativeTransform(ToWorldTransform);

	// do not setup undo for this first step since it is just used to calculate the local transform
	Hierarchy->SetControlShapeTransform(ControlElement, NewTransform, ERigTransformType::CurrentGlobal, false);
	const FTransform CurrentLocalShapeTransform = Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
	// this call should trigger an instance-to-BP update in ControlRigEditor
	Hierarchy->SetControlShapeTransform(ControlElement, CurrentLocalShapeTransform, ERigTransformType::InitialLocal, true);

	FTransform MeshTransform = FTransform::Identity;
	FTransform ShapeTransform = CurrentLocalShapeTransform;
	if (const FControlRigShapeDefinition* Gizmo = UControlRigShapeLibrary::GetShapeByName(ControlElement->Settings.ShapeName, ControlRig->GetShapeLibraries(), ControlRig->ShapeLibraryNameMap))
	{
		MeshTransform = Gizmo->Transform;
	}
	
	if (UPrimitiveComponent* ProxyComponent = InShapeActor->ProxyComponent.Get())
	{
		ProxyComponent->SetRelativeTransform(MeshTransform * ShapeTransform);
	}
}


bool FControlRigEditMode::ModeSupportedByShapeActor(const AControlRigShapeActor* ShapeActor, UE::Widget::EWidgetMode InMode) const
{
	if (UControlRig* ControlRig = ShapeActor->ControlRig.Get())
	{
		if (const FRigControlElement* ControlElement = ControlRig->FindControl(ShapeActor->ControlName))
		{
			if (bIsChangingControlShapeTransform)
			{
				return true;
			}

			return ControlRigEditMode::Shapes::IsModeSupported(ControlElement->Settings.ControlType, InMode);
		}
	}
	return false;
}

bool FControlRigEditMode::IsControlRigSkelMeshVisible(const UControlRig* InControlRig) const
{
	if (IsInLevelEditor())
	{
		if (InControlRig)
		{
			if (const USceneComponent* SceneComponent = GetHostingSceneComponent(InControlRig))
			{
				const AActor* Actor = SceneComponent->GetTypedOuter<AActor>();
				return Actor ? (Actor->IsHiddenEd() == false && SceneComponent->IsVisibleInEditor()) : SceneComponent->IsVisibleInEditor();
			}
		}
		return false;
	}
	return true;
}

AControlRigShapeActor* FControlRigEditMode::GetControlShapeFromControlName(UControlRig* InControlRig,const FName& ControlName) const
{
	const auto* ShapeActors = ControlRigShapeActors.Find(InControlRig);
	if (ShapeActors)
	{
		for (AControlRigShapeActor* ShapeActor : *ShapeActors)
		{
			if (ShapeActor->ControlName == ControlName)
			{
				return ShapeActor;
			}
		}
	}

	return nullptr;
}

void FControlRigEditMode::AddControlRigInternal(UControlRig* InControlRig)
{
	RuntimeControlRigs.AddUnique(InControlRig);
	LastHierarchyHash.Add(INDEX_NONE);
	LastShapeLibraryHash.Add(INDEX_NONE);

	InControlRig->SetControlsVisible(true);
	if (UModularRig* ModularRig = Cast<UModularRig>(InControlRig))
	{
		ModularRig->ForEachModule([](FRigModuleInstance* Module)
		{
			if (UControlRig* Rig = Module->GetRig())
			{
				Rig->SetControlsVisible(true);
			}
			return true;
		});
	}
	InControlRig->PostInitInstanceIfRequired();

	InControlRig->GetHierarchy()->OnModified().RemoveAll(this);
	InControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
	
	InControlRig->GetHierarchy()->OnModified().AddSP(this, &FControlRigEditMode::OnHierarchyModified_AnyThread);
	InControlRig->OnPostConstruction_AnyThread().AddSP(this, &FControlRigEditMode::OnPostConstruction_AnyThread);

	//needed for the control rig track editor delegates to get hooked up
	if (WeakSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		Sequencer->ObjectImplicitlyAdded(InControlRig);
	}
	OnControlRigAddedOrRemovedDelegate.Broadcast(InControlRig, true);

	UpdateSelectabilityOnSkeletalMeshes(InControlRig, !bShowControlsAsOverlay);
}

TArrayView<const TWeakObjectPtr<UControlRig>> FControlRigEditMode::GetControlRigs() const
{
	return MakeArrayView(RuntimeControlRigs);
}

TArrayView<TWeakObjectPtr<UControlRig>> FControlRigEditMode::GetControlRigs() 
{
	return MakeArrayView(RuntimeControlRigs);
}

TArray<UControlRig*> FControlRigEditMode::GetControlRigsArray(bool bIsVisible)
{
	TArray < UControlRig*> ControlRigs;
	for (TWeakObjectPtr<UControlRig> ControlRigPtr : RuntimeControlRigs)
	{
		if (ControlRigPtr.IsValid() && ControlRigPtr.Get() != nullptr && (bIsVisible == false ||ControlRigPtr.Get()->GetControlsVisible()))
		{
			ControlRigs.Add(ControlRigPtr.Get());
		}
	}
	return ControlRigs;
}

TArray<const UControlRig*> FControlRigEditMode::GetControlRigsArray(bool bIsVisible) const
{
	TArray<const UControlRig*> ControlRigs;
	for (const TWeakObjectPtr<UControlRig> ControlRigPtr : RuntimeControlRigs)
	{
		if (ControlRigPtr.IsValid() && ControlRigPtr.Get() != nullptr && (bIsVisible == false || ControlRigPtr.Get()->GetControlsVisible()))
		{
			ControlRigs.Add(ControlRigPtr.Get());
		}
	}
	return ControlRigs;
}

void FControlRigEditMode::RemoveControlRig(UControlRig* InControlRig)
{
	if (InControlRig == nullptr)
	{
		return;
	}

	if (!URigVMHost::IsGarbageOrDestroyed(InControlRig))
	{
		InControlRig->ControlModified().RemoveAll(this);
		if (URigHierarchy* Hierarchy = InControlRig->GetHierarchy())
		{
			Hierarchy->OnModified().RemoveAll(this);
		}
		InControlRig->OnPreConstructionForUI_AnyThread().RemoveAll(this);
		InControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
	}
	
	int32 Index = RuntimeControlRigs.Find(InControlRig);
	TStrongObjectPtr<UControlRigEditModeDelegateHelper>* DelegateHelper = DelegateHelpers.Find(InControlRig);
	if (DelegateHelper && DelegateHelper->IsValid())
	{
		DelegateHelper->Get()->RemoveDelegates();
		DelegateHelper->Reset();
		DelegateHelpers.Remove(InControlRig);
	}
	DestroyShapesActors(InControlRig);
	ClearOutHidden(InControlRig);
	if (RuntimeControlRigs.IsValidIndex(Index))
	{
		RuntimeControlRigs.RemoveAt(Index);
	}
	if (LastHierarchyHash.IsValidIndex(Index))
	{
		LastHierarchyHash.RemoveAt(Index);
	}
	if (LastShapeLibraryHash.IsValidIndex(Index))
	{
		LastShapeLibraryHash.RemoveAt(Index);
	}

	ControlRigsToRecreate.Remove(InControlRig);

	//needed for the control rig track editor delegates to get removed
	if (WeakSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		Sequencer->ObjectImplicitlyRemoved(InControlRig);
	}
	OnControlRigAddedOrRemovedDelegate.Broadcast(InControlRig, false);
	
	UpdateSelectabilityOnSkeletalMeshes(InControlRig, true);
}

void FControlRigEditMode::TickManipulatableObjects(const TArray<TWeakObjectPtr<UControlRig>>& InRigs) const
{
	auto UpdateSkeletalMeshComponent = [](USkeletalMeshComponent* InSkeletalMeshComponent)
	{
		if (InSkeletalMeshComponent && !InSkeletalMeshComponent->IsPostEvaluatingAnimation() && InSkeletalMeshComponent->PoseTickedThisFrame())
		{
			// "Copy Pose from Mesh" requires AnimInstance::PreUpdate() to copy the parent bone transforms.
			// have to TickAnimation() to ensure that PreUpdate() is called on all anim instances
			
			constexpr float DeltaTime = 0.f;
			constexpr bool bDontNeedValidRootMotion = false;
			
			InSkeletalMeshComponent->TickAnimation(DeltaTime, bDontNeedValidRootMotion);
			InSkeletalMeshComponent->RefreshBoneTransforms();
			InSkeletalMeshComponent->RefreshFollowerComponents();
			InSkeletalMeshComponent->UpdateComponentToWorld();
			InSkeletalMeshComponent->FinalizeBoneTransform();
			InSkeletalMeshComponent->MarkRenderTransformDirty();
			InSkeletalMeshComponent->MarkRenderDynamicDataDirty();
		}
	};
	
	const TArray<TWeakObjectPtr<UControlRig>>& RigsToTick = InRigs.IsEmpty() ? RuntimeControlRigs : InRigs;
	for (const TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RigsToTick)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			const TGuardValue<bool> GuardEvaluationType(ControlRig->bEvaluationTriggeredFromInteraction, true);
			
			// tick skeletalmeshcomponent, that's how they update their transform from rig change
			USceneComponent* SceneComponent = GetHostingSceneComponent(ControlRig);
			
			if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(SceneComponent))
			{
				ControlRigComponent->Update();
			}
			else if (USkeletalMeshComponent* MeshComponent = Cast<USkeletalMeshComponent>(SceneComponent))
			{
				if(!ControlRig->ContainsSimulation())
				{
					// NOTE: we have to update/tick ALL children skeletal mesh components here because user can attach
					// additional skeletal meshes via the "Copy Pose from Mesh" node.
					//
					// If this is left up to the viewport tick(), the attached meshes will render before they get the latest
					// parent bone transforms resulting in a visible lag on all attached components.

					// get hierarchically ordered list of ALL child skeletal mesh components (recursive)
					const AActor* ThisActor = MeshComponent->GetOwner();
					TArray<USceneComponent*> ChildrenComponents;
					MeshComponent->GetChildrenComponents(true, ChildrenComponents);
					TInlineComponentArray<USkeletalMeshComponent*> SkeletalMeshesToUpdate;
					SkeletalMeshesToUpdate.Add(MeshComponent);
					for (USceneComponent* ChildComponent : ChildrenComponents)
					{
						if (USkeletalMeshComponent* ChildMeshComponent = Cast<USkeletalMeshComponent>(ChildComponent))
						{
							if (ThisActor == ChildMeshComponent->GetOwner())
							{
								SkeletalMeshesToUpdate.Add(ChildMeshComponent);
							}
						}
					}

					// update pose of all children skeletal meshes in this actor
					for (USkeletalMeshComponent* SkeletalMeshToUpdate : SkeletalMeshesToUpdate)
					{
						UpdateSkeletalMeshComponent(SkeletalMeshToUpdate);
					}
				}
			}
		}
	}
	
	PostPoseUpdate(nullptr, InRigs);
}


void FControlRigEditMode::OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources)
{
	// if world gets cleaned up first, we destroy gizmo actors
	if ((WorldPtr == World) || (Owner && Owner->GetWorld() == World))
	{
		DestroyShapesActors(nullptr);
	}

	// WorldPtr is a raw pointer, so if the world it points to is the one being torn down we must
	// clear it here. Otherwise it is left dangling once the world is garbage collected.
	if (WorldPtr == World)
	{
		WorldPtr = nullptr;
	}
}

void FControlRigEditMode::OnEditorClosed()
{
	ControlRigShapeActors.Reset();
	ControlRigsToRecreate.Reset();
}

FControlRigEditMode::FMarqueeDragTool::FMarqueeDragTool()
	: bIsDeletingDragTool(false)
{
}

bool FControlRigEditMode::FMarqueeDragTool::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return (DragTool.IsValid() && InViewportClient->GetCurrentWidgetAxis() == EAxisList::None);
}

bool FControlRigEditMode::FMarqueeDragTool::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (!bIsDeletingDragTool)
	{
		// Ending the drag tool may pop up a modal dialog which can cause unwanted reentrancy - protect against this.
		TGuardValue<bool> RecursionGuard(bIsDeletingDragTool, true);

		// Delete the drag tool if one exists.
		if (DragTool.IsValid())
		{
			if (DragTool->IsDragging())
			{
				DragTool->EndDrag();
			}
			DragTool.Reset();
			return true;
		}
	}
	
	return false;
}

void FControlRigEditMode::FMarqueeDragTool::MakeDragTool(FEditorViewportClient* InViewportClient)
{
	DragTool.Reset();
	if (InViewportClient->IsOrtho())
	{
		DragTool = MakeShareable( new FDragTool_ActorBoxSelect(InViewportClient) );
	}
	else
	{
		DragTool = MakeShareable( new FDragTool_ActorFrustumSelect(InViewportClient) );
	}
}

bool FControlRigEditMode::FMarqueeDragTool::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (DragTool.IsValid() == false || InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
	{
		return false;
	}
	if (DragTool->IsDragging() == false)
	{
		int32 InX = InViewport->GetMouseX();
		int32 InY = InViewport->GetMouseY();
		FVector2D Start(InX, InY);

		DragTool->StartDrag(InViewportClient, GEditor->ClickLocation,Start);
	}
	const bool bUsingDragTool = UsingDragTool();
	if (bUsingDragTool == false)
	{
		return false;
	}

	DragTool->AddDelta(InDrag);
	return true;
}

bool FControlRigEditMode::FMarqueeDragTool::UsingDragTool() const
{
	return DragTool.IsValid() && DragTool->IsDragging();
}

void FControlRigEditMode::FMarqueeDragTool::Render3DDragTool(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (DragTool.IsValid())
	{
		DragTool->Render3D(View, PDI);
	}
}

void FControlRigEditMode::FMarqueeDragTool::RenderDragTool(const FSceneView* View, FCanvas* Canvas)
{
	if (DragTool.IsValid())
	{
		DragTool->Render(View, Canvas);
	}
}

void FControlRigEditMode::DestroyShapesActors(UControlRig* InControlRig)
{
	using namespace ControlRigEditMode::Shapes;
	
	if (!InControlRig)
	{
		// destroy all control rigs shape actors
		for(auto& ShapeActors: ControlRigShapeActors)
		{
			DestroyShapesActorsFromWorld(ShapeActors.Value);
		}
		ControlRigEditMode::Shapes::ClearOutHidden(nullptr);
		ControlRigShapeActors.Reset();
		ControlRigsToRecreate.Reset();
		
		return;
	}

	// only destroy control rigs shape actors related to InControlRig
	ControlRigsToRecreate.Remove(InControlRig);
	if (const auto* ShapeActors = ControlRigShapeActors.Find(InControlRig))
	{
		DestroyShapesActorsFromWorld(*ShapeActors);
		ControlRigShapeActors.Remove(InControlRig);
	}
}

USceneComponent* FControlRigEditMode::GetHostingSceneComponent(const UControlRig* ControlRig) const
{
	const TArrayView<const TWeakObjectPtr<UControlRig>> Rigs = GetControlRigs();
	if (!ControlRig && !Rigs.IsEmpty())
	{
		ControlRig = Rigs[0].Get();
	}

	const TSharedPtr<IControlRigObjectBinding>& ObjectBinding = ControlRig ? ControlRig->GetObjectBinding() : nullptr;
	UObject* BoundObject = ObjectBinding ? ObjectBinding->GetBoundObject() : nullptr;
	if (!BoundObject)
	{
		return nullptr;
	}

	if (USceneComponent* BoundSceneComponent = Cast<USceneComponent>(BoundObject))
	{
		return BoundSceneComponent;
	}

	if (const USkeleton* BoundSkeleton = Cast<USkeleton>(BoundObject))
	{
		// Bound to a Skeleton means we are previewing an Animation Sequence
		if (WorldPtr && WorldPtr->PersistentLevel)
		{
			const TObjectPtr<AActor>* PreviewActor = WorldPtr->PersistentLevel->Actors.FindByPredicate([](const TObjectPtr<AActor>& Actor)
			{
				return Actor && Actor->GetClass() == AAnimationEditorPreviewActor::StaticClass();
			});

			if (PreviewActor)
			{
				if (UDebugSkelMeshComponent* DebugComponent = (*PreviewActor)->FindComponentByClass<UDebugSkelMeshComponent>())
				{
					return DebugComponent;
				}
			}
		}
	}
	
	return nullptr;
}

FTransform FControlRigEditMode::GetHostingSceneComponentTransform(const UControlRig* ControlRig) const
{
	// we care about this transform only in the level,
	// since in the control rig editor the debug skeletal mesh component
	// is set at identity anyway.
	if(IsInLevelEditor())
	{
		if (ControlRig == nullptr && GetControlRigs().Num() > 0)
		{
			ControlRig = GetControlRigs()[0].Get();
		}

		USceneComponent* HostingComponent = GetHostingSceneComponent(ControlRig);
		FTransform Result = HostingComponent ? HostingComponent->GetComponentTransform() : FTransform::Identity;

		// When root motion is extracted from a control rig layer and applied to the actor,
		// the control hierarchy's cached global transforms still include the extracted root
		// motion while the component transform also includes it (via the moved actor).
		// Subtract the root motion offset so gizmos are not drawn at a double offset.
		if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
		{
			const UMovieSceneEntitySystemLinker* EntityLinker =
				Sequencer->GetEvaluationTemplate().GetEntitySystemLinker();
			if (EntityLinker && HostingComponent)
			{
				TArray<IMovieSceneRootMotionOffsetProvider*> Providers =
					IModularFeatures::Get().GetModularFeatureImplementations<IMovieSceneRootMotionOffsetProvider>(
						IMovieSceneRootMotionOffsetProvider::GetModularFeatureName());
				for (IMovieSceneRootMotionOffsetProvider* Provider : Providers)
				{
					if (!Provider)
					{
						continue;
					}
					FTransform Offset = Provider->GetRootMotionOffset(EntityLinker, HostingComponent, ControlRig);
					if (!Offset.Equals(FTransform::Identity))
					{
						Result = Offset.Inverse() * Result;
						break;
					}
				}
			}
		}

		return Result;
	}
	return FTransform::Identity;
}

void FControlRigEditMode::OnPoseInitialized()
{
	OnAnimSystemInitializedDelegate.Broadcast();
}

void FControlRigEditMode::PostPoseUpdate(const FEditorViewportClient* InViewportClient, const TArray<TWeakObjectPtr<UControlRig>>& InRigs) const
{
	if (bDoPostPoseUpdate == false)
	{
		return;
	}

	using namespace ControlRigEditMode::Shapes;

	const bool bIsGameView = InViewportClient ? InViewportClient->IsInGameView() : false;
	if (bIsGameView)
	{
		// no need to update the shape actors in game view (shapes are already hidden in game using SetActorHiddenInGame(true))
		return;
	}

	const bool bAreEditingControlRigDirectly = AreEditingControlRigDirectly();
	auto UpdateShapes = [this, bAreEditingControlRigDirectly](const TWeakObjectPtr<UControlRig> WeakControlRig, const TArray<TObjectPtr<AControlRigShapeActor>>& ShapeActors)
	{
		if(TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
		{
			const FTransform ComponentTransform = bAreEditingControlRigDirectly ? FTransform::Identity : GetHostingSceneComponentTransform(ControlRig.Get()); 

			FShapeUpdateParams Params(ControlRig.Get(), ComponentTransform, IsControlRigSkelMeshVisible(ControlRig.Get()), IsInLevelEditor(), !IsDisplayControlsEnabled());
			for (AControlRigShapeActor* ShapeActor : ShapeActors)
			{
				UpdateControlShape(ShapeActor, ControlRig->FindControl(ShapeActor->ControlName), Params);
			}
		}
	};
	
	if (InRigs.IsEmpty())
	{
		// updates all control shapes properties
		for (const TPair<TWeakObjectPtr<UControlRig>,TArray<TObjectPtr<AControlRigShapeActor>>>& ShapeActors : ControlRigShapeActors)
		{
			UpdateShapes(ShapeActors.Key, ShapeActors.Value);
		}
	}
	else
	{
		for (const TWeakObjectPtr<UControlRig>& RigPtr: InRigs)
		{
			if (TStrongObjectPtr<UControlRig> ControlRig = RigPtr.Pin())
			{
				if (const TArray<TObjectPtr<AControlRigShapeActor>>* ShapeActors = ControlRigShapeActors.Find(ControlRig.Get()))
				{
					UpdateShapes(RigPtr, *ShapeActors);
				}
			}
		}
	}
}

void FControlRigEditMode::NotifyDrivenControls(UControlRig* InControlRig, const FRigElementKey& InKey, const FRigControlModifiedContext& InContext)
{
	// if we are changing a proxy control - we also need to notify the change for the driven controls
	if (FRigControlElement* ControlElement = InControlRig->GetHierarchy()->Find<FRigControlElement>(InKey))
	{
		if(ControlElement->CanDriveControls())
		{
			const bool bFixEulerFlips = InControlRig->IsAdditive() == false ? true : false;
			FRigControlModifiedContext Context(InContext);
			Context.EventName = FRigUnit_BeginExecution::EventName;

			for(const FRigElementKey& DrivenKey : ControlElement->Settings.DrivenControls)
			{
				if(DrivenKey.Type == ERigElementType::Control)
				{
					const FTransform DrivenTransform = InControlRig->GetControlLocalTransform(DrivenKey.Name);
					InControlRig->SetControlLocalTransform(DrivenKey.Name, DrivenTransform, true, Context, false /*undo*/, bFixEulerFlips);
				}
			}
		}
	}
}

void FControlRigEditMode::UpdateSelectabilityOnSkeletalMeshes(UControlRig* InControlRig, bool bEnabled)
{
	if(const USceneComponent* HostingComponent = GetHostingSceneComponent(InControlRig))
	{
		if(const AActor* HostingOwner = HostingComponent->GetOwner())
		{
			for(UActorComponent* ActorComponent : HostingOwner->GetComponents())
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ActorComponent))
				{
					SkeletalMeshComponent->bSelectable = bEnabled;
					SkeletalMeshComponent->MarkRenderStateDirty();
				}
				else if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ActorComponent))
				{
					StaticMeshComponent->bSelectable = bEnabled;
					StaticMeshComponent->MarkRenderStateDirty();
				}
			}
		}
	}
}

bool FControlRigEditMode::IsViewportHitProxySelectable(HHitProxy* const InHitProxy) const
{
	if (!InHitProxy)
	{
		return false;
	}

	if (InHitProxy->IsA(HWidgetAxis::StaticGetType()) || InHitProxy->IsA(HFKRigBoneProxy::StaticGetType()))
	{
		return true;
	}

	if (const HActor* const ActorHitProxy = HitProxyCast<HActor>(InHitProxy))
	{
		return IsViewportObjectSelectable(ActorHitProxy->Actor);
	}

	return false;
}

bool FControlRigEditMode::IsViewportObjectSelectable(UObject* const InObject) const
{
	if (!IsInLevelEditor())
	{
		return true;
	}

	const UControlRigEditModeSettings* const Settings = GetSettings();
	if (!Settings || !Settings->bOnlySelectRigControls)
	{
		return true;
	}

	const AActor* Actor = Cast<AActor>(InObject);
	if (!Actor)
	{
		if (const UActorComponent* const ActorComponent = Cast<UActorComponent>(InObject))
		{
			Actor = ActorComponent->GetOwner();
		}
	}

	return !Actor || Actor->IsA<AControlRigShapeActor>();
}

void FControlRigEditMode::SetOnlySelectRigControls(bool Val)
{
	UControlRigEditModeSettings* Settings = GetMutableSettings();
	Settings->bOnlySelectRigControls = Val;
	Settings->SaveConfig();
}

bool FControlRigEditMode::GetOnlySelectRigControls()const
{
	const UControlRigEditModeSettings* Settings = GetSettings();
	return Settings->bOnlySelectRigControls;
}

static TArray<UControlRig*> GetControlRigsWithSelectedControls(const TArray<UControlRig*>& InControlRigs)
{
	TArray<UControlRig*> SelectedControlRigs;
	for (UControlRig* ControlRig : InControlRigs)
	{
		if (ControlRig && ControlRig->CurrentControlSelection().Num() > 0)
		{
			SelectedControlRigs.Add(ControlRig);
		}
	}
	return SelectedControlRigs;
}

/** Select Mirrored Controls on Current Selection*/
void FControlRigEditMode::SelectMirroredControls()
{
	TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
	ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
	if (ControlRigs.Num() == 0)
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("SelectMirroredControls", "Select Mirrored Controls"), !GIsTransacting);
	FGCScopeGuard Guard;
	UControlRigPoseAsset* TempPose = NewObject<UControlRigPoseAsset>(GetTransientPackage(), NAME_None);
	
	for (UControlRig* ControlRig : ControlRigs)
	{
		if (ControlRig)
		{
			ControlRig->Modify();
			TempPose->SavePose(ControlRig, false, false/*bSaveValues*/);
			TempPose->SelectControls(ControlRig, true);
		}
	}
}

/** Select Mirrored Controls on Current Selection, keeping current selection*/
void FControlRigEditMode::AddMirroredControlsToSelection()
{
	TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
	ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
	if (ControlRigs.Num() == 0)
	{
		return;
	}

	FScopedTransaction ScopedTransaction(LOCTEXT("AddMirroredControlsToSelection", "Add Mirrored Controls to Selection"), !GIsTransacting);
	FGCScopeGuard Guard;
	UControlRigPoseAsset* TempPose = NewObject<UControlRigPoseAsset>(GetTransientPackage(), NAME_None);
	for (UControlRig* ControlRig : ControlRigs)
	{
		if (ControlRig)
		{
			ControlRig->Modify();
			TempPose->SavePose(ControlRig, false, false/*bSaveValues*/);
			TempPose->SelectControls(ControlRig, true, false);
		}
	}
}

/** Put Selected Controls To Mirrored Pose*/
void FControlRigEditMode::MirrorSelectedControls()
{
	TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
	ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
	if (ControlRigs.Num() == 0)
	{
		return;
	}
	FScopedTransaction ScopedTransaction(LOCTEXT("MirrorSelectedControls", "Mirror Selected Controls"), !GIsTransacting);
	FGCScopeGuard Guard;
	UControlRigPoseAsset* TempPose = NewObject<UControlRigPoseAsset>(GetTransientPackage(), NAME_None);

	for (UControlRig* ControlRig : ControlRigs)
	{
		if (ControlRig)
		{
			ControlRig->Modify();
			TempPose->SavePose(ControlRig, true);
			TempPose->PastePose(ControlRig, /*setkey*/false, true);
		}
	}
}

/** Put Unselected Controls To Mirrored selcted controls*/
void FControlRigEditMode::MirrorUnselectedControls()
{
	TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
	ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
	if (ControlRigs.Num() == 0)
	{
		return;
	}
	FScopedTransaction ScopedTransaction(LOCTEXT("MirrorSelectedControls", "Mirror Selected Controls"), !GIsTransacting);
	FGCScopeGuard Guard;
	UControlRigPoseAsset* TempPose = NewObject<UControlRigPoseAsset>(GetTransientPackage(), NAME_None);

	for (UControlRig* ControlRig : ControlRigs)
	{
		if (ControlRig)
		{
			ControlRig->Modify();
			TempPose->SavePose(ControlRig, false);
			TempPose->SelectControls(ControlRig, true); //select mirrored controls
			TempPose->PastePose(ControlRig, /*setkey*/false, true); //paste it
			TempPose->SelectControls(ControlRig, false);//put it back
		}
	}
}

/** Select All Controls*/
void FControlRigEditMode::SelectAllControls()
{
	TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
	ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
	if (ControlRigs.Num() == 0)
	{
		return;
	}
	FScopedTransaction ScopedTransaction(LOCTEXT("SelectAllControls", "Select All Controls"), !GIsTransacting);
	FGCScopeGuard Guard;
	UControlRigPoseAsset* TempPose = NewObject<UControlRigPoseAsset>(GetTransientPackage(), NAME_None);
	for (UControlRig* ControlRig : ControlRigs)
	{
		if (ControlRig)
		{
			ControlRig->Modify();
			TempPose->SavePose(ControlRig, true, false/*bSaveValues*/); //this will save the whole pose
			TempPose->SelectControls(ControlRig);
		}
	}
}

/** Save a pose of selected controls*/
void FControlRigEditMode::SavePose(int32 PoseNum)
{
	if (StoredPose)
	{
		TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
		ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				StoredPose->SavePose(ControlRig, false);
				return;
			}
		}
	}
}

/** Select controls in saved pose*/
void FControlRigEditMode::SelectPose(bool bMirror, int32 PoseNum)
{
	if (StoredPose)
	{
		TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
		if (ControlRigs.Num() < 0)
		{
			return;
		}
		else if (ControlRigs.Num() == 1 && ControlRigs[0])
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("SelectPoseControls", "Select Pose Controls"), !GIsTransacting);
			ControlRigs[0]->Modify();
			StoredPose->SelectControls(ControlRigs[0], bMirror);
			return;
		}
		ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
		if (ControlRigs.Num() == 0)
		{
			return;
		}
		FScopedTransaction ScopedTransaction(LOCTEXT("SelectPoseControls", "Select Pose Controls"), !GIsTransacting);

		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				ControlRig->Modify();
				StoredPose->SelectControls(ControlRig, bMirror);
			}
		}
	}
}

/** Paste saved pose */
void FControlRigEditMode::PastePose(bool bMirror, int32 PoseNum)
{
	if (StoredPose)
	{
		TArray<UControlRig*> ControlRigs = GetControlRigsArray(false);
		ControlRigs = GetControlRigsWithSelectedControls(ControlRigs);
		if (ControlRigs.Num() == 0)
		{
			return;
		}
		FScopedTransaction ScopedTransaction(LOCTEXT("PastePose", "Paste Pose"), !GIsTransacting);
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				ControlRig->Modify();
				StoredPose->PastePose(ControlRig,/*setkey*/false, bMirror);
				return;
			}
		}
	}
}

const UControlRigEditModeSettings* FControlRigEditMode::GetSettings() const
{
	if (!WeakSettings.IsValid())
	{
		WeakSettings = GetMutableDefault<UControlRigEditModeSettings>();
	}
	return WeakSettings.Get();
}

UControlRigEditModeSettings* FControlRigEditMode::GetMutableSettings() const
{
	if (!WeakSettings.IsValid())
	{
		WeakSettings = GetMutableDefault<UControlRigEditModeSettings>();
	}
	return WeakSettings.Get();
}

FRotationContext& FControlRigEditMode::GetRotationContext() const
{
	static FRotationContext DefaultContext;
	return WeakGizmoContext.IsValid() ? WeakGizmoContext->RotationContext : DefaultContext;
}

void FControlRigEditMode::UpdateRotationContext()
{
	if (!WeakGizmoContext.IsValid())
	{
		return;
	}
	
	FRotationContext& RotationContext = GetRotationContext();
	RotationContext = FRotationContext();
			
	const bool bIsExplicitRotation = GetCoordSystemSpace() == COORD_Explicit; 
	const bool bRotating = GetModeManager()->GetWidgetMode() == UE::Widget::EWidgetMode::WM_Rotate;
	RotationContext.bUseExplicitRotator = bIsExplicitRotation && bRotating;

	if (!RotationContext.bUseExplicitRotator)
	{
		return;
	}
	
	for (const auto& [WeakControlRig, Shapes] : ControlRigShapeActors)
	{
		if (TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
		{
			if (const URigHierarchy* Hierarchy = ControlRig ? ControlRig->GetHierarchy() : nullptr)
			{
				if (Hierarchy->UsesPreferredEulerAngles())
				{
					for (const AControlRigShapeActor* ShapeActor : Shapes)
					{
						if (ShapeActor->IsEnabled() && ShapeActor->IsSelected())
						{
							FRigControlElement* Control = ControlRig->FindControl(ShapeActor->ControlName);
							const bool bUsePreferredRotationOrder = Hierarchy->GetUsePreferredRotationOrder(Control);

							RotationContext.RotationOrder = bUsePreferredRotationOrder ? Hierarchy->GetControlPreferredEulerRotationOrder(Control) : EEulerRotationOrder::XYZ;
							RotationContext.Rotation = Hierarchy->GetControlPreferredRotator(Control);

							if (TOptional<FTransform> ConstraintSpace = GetConstraintParentTransform(ControlRig.Get(), ShapeActor->ControlName))
							{
								RotationContext.Offset = *ConstraintSpace;
							}
							else
							{
								const FTransform Offset = Hierarchy->GetControlOffsetTransform(Control, ERigTransformType::CurrentGlobal);
								const FTransform ComponentTransform = GetHostingSceneComponentTransform(ControlRig.Get());
								RotationContext.Offset = Offset * ComponentTransform;
							}

							// only get the first rotation order
							return;
						}
					}
				}
			}
		}
	}
}

FInteractionDependencyCache& FControlRigEditMode::GetInteractionDependencies(UControlRig* InControlRig)
{
	if (FInteractionDependencyCache* DependencyCache = InteractionDependencies.Find(InControlRig))
	{
		return *DependencyCache;
	}

	static FInteractionDependencyCache DummyDependencies;
	
	if (const TArray<TObjectPtr<AControlRigShapeActor>>* ShapeActors = ControlRigShapeActors.Find(InControlRig))
	{
		URigHierarchy* Hierarchy = InControlRig->GetHierarchy();

		// get selected controls 
		TArray<FRigControlElement*> SelectedControls;
		for (const TObjectPtr<AControlRigShapeActor>& ShapeActor: *ShapeActors)
		{
			if (ShapeActor && ShapeActor->IsEnabled() && ShapeActor->IsSelected())
			{
				if (FRigControlElement* Control = Hierarchy->Find<FRigControlElement>(ShapeActor->GetElementKey()))
				{
					SelectedControls.Add(Control);
				}
			}
		}

		if (SelectedControls.IsEmpty())
		{
			return DummyDependencies;
		}

		// build dependencies between selected controls
		FInteractionDependencyCache& NewDependency = InteractionDependencies.Add(InControlRig);
		NewDependency.WeakHierarchy = Hierarchy;

		// NOTE: this is not enough for modular rigs since there are several VMs
		const FRigDependenciesProviderForControlRig DependencyProvider(InControlRig);
		
		for (int32 Index = 0; Index < SelectedControls.Num(); ++Index)
		{
			FRigControlElement* Control = SelectedControls[Index];
			for (int32 NextIndex = Index+1; NextIndex < SelectedControls.Num(); ++NextIndex)
			{
				FRigControlElement* OtherControl = SelectedControls[NextIndex];
				if (Hierarchy->IsParentedTo(Control, OtherControl, DependencyProvider))
				{
					NewDependency.Parents.Add(OtherControl->GetKey());
					NewDependency.Children.Add(Control->GetKey());
				}
				else if (Hierarchy->IsParentedTo(OtherControl, Control, DependencyProvider))
				{
					NewDependency.Parents.Add(Control->GetKey());
					NewDependency.Children.Add(OtherControl->GetKey());
				}
			}

			// store parents' pose versions
			const FRigBaseElementParentArray Parents = Hierarchy->GetParents(Control);
			NewDependency.ParentsPoseVersion.Reserve(Parents.Num());
			for (const FRigBaseElement* Parent: Parents)
			{
				if (const FRigTransformElement* TransformParent = Cast<FRigTransformElement>(Parent))
				{
					NewDependency.ParentsPoseVersion.FindOrAdd(Parent->GetIndex()) = Hierarchy->GetPoseVersion(TransformParent);
				}
			}
		}

		return NewDependency;
	}

	return DummyDependencies;
}

/**
* FDetailKeyFrameCacheAndHandler
*/

bool FDetailKeyFrameCacheAndHandler::IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
{
	const FProperty* Property = InPropertyHandle.GetProperty();
	if (!InObjectClass || !Property)
	{
		return false;
	}
	const FName PropertyName = Property->GetFName();

	TArray<UObject*> OuterObjects;
	InPropertyHandle.GetOuterObjects(OuterObjects);
	for (UObject* OuterObject : OuterObjects)
	{
		if (UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(OuterObject))
		{
			UControlRig* ControlRig = Proxy->GetControlRig();
			FRigControlElement* ControlElement = Proxy->GetControlElement();

			if (ControlRig && ControlElement)
			{
				if (!ControlRig->GetHierarchy()->IsAnimatable(ControlElement))
				{
					return false;
				}
			}
		}
	}

	if (InObjectClass->IsChildOf(UAnimLayer::StaticClass()))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyTransform::StaticClass()) &&
		(PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale)))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyLocation::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyLocation, Location))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyRotation::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyRotation, Rotation))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyScale::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyScale, Scale))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyVector2D::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyVector2D, Vector2D))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyInteger::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyInteger, Integer))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyEnum::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyEnum, Enum))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyBool::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyBool, Bool))
	{
		return true;
	}

	if (InObjectClass->IsChildOf(UAnimDetailsProxyFloat::StaticClass()) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyFloat, Float))
	{
		return true;
	}

	FCanKeyPropertyParams CanKeyPropertyParams(InObjectClass, InPropertyHandle);
	if (WeakSequencer.IsValid() && WeakSequencer.Pin()->CanKeyProperty(CanKeyPropertyParams))
	{
		return true;
	}

	return false;
}

bool FDetailKeyFrameCacheAndHandler::IsPropertyKeyingEnabled() const
{
	if (WeakSequencer.IsValid() &&  WeakSequencer.Pin()->GetFocusedMovieSceneSequence())
	{
		return true;
	}

	return false;
}

bool FDetailKeyFrameCacheAndHandler::IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject* ParentObject) const
{
	if (WeakSequencer.IsValid() && WeakSequencer.Pin()->GetFocusedMovieSceneSequence())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		constexpr bool bCreateHandleIfMissing = false;
		FGuid ObjectHandle = Sequencer->GetHandleToObject(ParentObject, bCreateHandleIfMissing);
		if (ObjectHandle.IsValid())
		{
			UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
			FProperty* Property = PropertyHandle.GetProperty();
			TSharedRef<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();
			PropertyPath->AddProperty(FPropertyInfo(Property));
			FName PropertyName(*PropertyPath->ToString(TEXT(".")));
			TSubclassOf<UMovieSceneTrack> TrackClass; //use empty @todo find way to get the UMovieSceneTrack from the Property type.
			return MovieScene->FindTrack(TrackClass, ObjectHandle, PropertyName) != nullptr;
		}
	}
	return false;
}

void FDetailKeyFrameCacheAndHandler::OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle)
{
	if (!WeakSequencer.IsValid() || !WeakSequencer.Pin()->IsAllowedToChange())
	{
		return;
	}
	FScopedTransaction ScopedTransaction(LOCTEXT("KeyAttribute", "Key Attribute"), !GIsTransacting);
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	TArray<UObject*> Objects;
	KeyedPropertyHandle.GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		if (Object)
		{
			if (UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(Object))
			{
				Proxy->SetKey(KeyedPropertyHandle);
			}
			else if (UAnimLayer* AnimLayer = (Object->GetTypedOuter<UAnimLayer>()))
			{
				AnimLayer->SetKey(SequencerPtr, KeyedPropertyHandle);
			}
		}
	}
}

EPropertyKeyedStatus FDetailKeyFrameCacheAndHandler::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	if (WeakSequencer.IsValid() == false)
	{
		return EPropertyKeyedStatus::NotKeyed;
	}		

	if (const EPropertyKeyedStatus* ExistingKeyedStatus = CachedPropertyKeyedStatusMap.Find(&PropertyHandle))
	{
		return *ExistingKeyedStatus;
	}
	//hack so we can get the reset cache state updated, use ToggleEditable state
	{
		IPropertyHandle* NotConst = const_cast<IPropertyHandle*>(&PropertyHandle);
		NotConst->NotifyPostChange(EPropertyChangeType::ToggleEditable);
	}

	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	UMovieSceneSequence* Sequence = SequencerPtr->GetFocusedMovieSceneSequence();
	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return KeyedStatus;
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle.GetOuterObjects(OuterObjects);
	if (OuterObjects.IsEmpty())
	{
		return EPropertyKeyedStatus::NotKeyed;
	}
	
	for (UObject* Object : OuterObjects)
	{
		if (Object)
		{
			if (UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(Object))
			{
				KeyedStatus = FMath::Max(KeyedStatus, Proxy->GetPropertyKeyedStatus(PropertyHandle));
			}
			else if (UAnimLayer* AnimLayer = (Object->GetTypedOuter<UAnimLayer>()))
			{
				KeyedStatus = FMath::Max(KeyedStatus, AnimLayer->GetPropertyKeyedStatus(SequencerPtr, PropertyHandle));
			}
		}
		//else check to see if it's in sequencer
	}
	CachedPropertyKeyedStatusMap.Add(&PropertyHandle, KeyedStatus);

	return KeyedStatus;
}

void FDetailKeyFrameCacheAndHandler::SetDelegates(TWeakPtr<ISequencer>& InWeakSequencer, FControlRigEditMode* InEditMode)
{
	WeakSequencer = InWeakSequencer;
	EditMode = InEditMode;
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnMovieSceneDataChanged().AddRaw(this, &FDetailKeyFrameCacheAndHandler::OnMovieSceneDataChanged);
		Sequencer->OnGlobalTimeChanged().AddRaw(this, &FDetailKeyFrameCacheAndHandler::OnGlobalTimeChanged);
		Sequencer->OnEndScrubbingEvent().AddRaw(this, &FDetailKeyFrameCacheAndHandler::ResetCachedData);
		Sequencer->OnChannelChangedWithTime().AddRaw(this, &FDetailKeyFrameCacheAndHandler::OnChannelChangedWithTime);
		Sequencer->OnStopEvent().AddRaw(this, &FDetailKeyFrameCacheAndHandler::ResetCachedData);
	}
}

void FDetailKeyFrameCacheAndHandler::UnsetDelegates()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
		Sequencer->OnGlobalTimeChanged().RemoveAll(this);
		Sequencer->OnEndScrubbingEvent().RemoveAll(this);
		Sequencer->OnChannelChangedWithTime().RemoveAll(this);
		Sequencer->OnStopEvent().RemoveAll(this);
	}
}

void FDetailKeyFrameCacheAndHandler::OnGlobalTimeChanged()
{
	// Only reset cached data when not playing
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid() && Sequencer->GetPlaybackStatus() != EMovieScenePlayerStatus::Playing)
	{
		ResetCachedData();
	}
}

void FDetailKeyFrameCacheAndHandler::OnMovieSceneDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	if (DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemAdded
		|| DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemRemoved
		|| DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemsChanged
		|| DataChangeType == EMovieSceneDataChangeType::ActiveMovieSceneChanged
		|| DataChangeType == EMovieSceneDataChangeType::RefreshAllImmediately)
	{
		ResetCachedData();
	}
}

void FDetailKeyFrameCacheAndHandler::OnChannelChangedWithTime(const TArray<FFrameNumber>&, const FMovieSceneChannelMetaData*, UMovieSceneSection*)
{
	ResetCachedData();
}

void FDetailKeyFrameCacheAndHandler::ResetCachedData()
{
	CachedPropertyKeyedStatusMap.Reset();
	bValuesDirty = true;
}

void FDetailKeyFrameCacheAndHandler::UpdateIfDirty()
{
	if (bValuesDirty == true)
	{
		if (FMovieSceneConstraintChannelHelper::bDoNotCompensate == false) //if compensating don't reset this.
		{
			UAnimDetailsProxyManager* ProxyManager = EditMode ? EditMode->GetAnimDetailsProxyManager() : nullptr;
			if (ProxyManager)
			{
				ProxyManager->RequestUpdateProxyValues();
			}

			bValuesDirty = false;
		}
	}
}

FControlRigEditMode::FPendingControlRigEvaluator::~FPendingControlRigEvaluator()
{
	if(--EditMode->RigEvaluationBracket == 0)
	{
		for(UControlRig* ControlRig : EditMode->RigsToEvaluateDuringThisTick)
		{
			FControlRigEditMode::EvaluateRig(ControlRig);
		}
		EditMode->RigsToEvaluateDuringThisTick.Reset();
	}
}

void FControlRigEditMode::EvaluateRig(UControlRig* InControlRig)
{
	if (InControlRig)
	{
		const TGuardValue<float> AbsoluteTimeGuard(InControlRig->AbsoluteTime, InControlRig->AbsoluteTime);
		const TGuardValue<bool> GuardEvaluationType(InControlRig->bEvaluationTriggeredFromInteraction, true);
		InControlRig->Evaluate_AnyThread();
	}
}

TOptional<FTransform> FControlRigEditMode::GetConstraintParentTransform(const UControlRig* InControlRig, const FName& InControlName) const
{
	static const TOptional<FTransform> Dummy;

	if (IsInLevelEditor())
	{
		if (!InControlRig || InControlName == NAME_None)
		{
			return Dummy;
		}
		
		const uint32 ControlHash = UTransformableControlHandle::ComputeHash(InControlRig, InControlName);
		return ConstraintsCache.GetParentTransform(ControlHash, InControlRig->GetWorld());
	}

	return Dummy;
}

void FControlRigEditMode::HandleActorMoving(AActor* InMovedActor)
{
	if (!bUpdateNonInteractingRigs && InMovedActor)
	{
		bUpdateNonInteractingRigs =
			SelectedObjects.Contains(InMovedActor) ||
			SelectedObjects.Contains(InMovedActor->GetRootComponent());

		if (!bUpdateNonInteractingRigs)
		{
			constexpr bool bFromChildActors = true;
			InMovedActor->ForEachComponent<USceneComponent>(
				bFromChildActors,
				[&Objects = SelectedObjects, &Found = bUpdateNonInteractingRigs](USceneComponent* Component) 
				{
					if (!Found)
					{
						Found = Objects.Contains(Component);
					}
				});
		}
	}
}

void FControlRigEditMode::CacheAnyOtherSelectedObjects()
{
	if (!IsInLevelEditor())
	{
		return;
	}

	SelectedObjects.Reset();
	
	// check actor selection (minus control shapes)
	TArray< TWeakObjectPtr<UObject> > Selection;
	GetModeManager()->GetSelectedActors()->GetSelectedObjects(Selection);
	Selection.RemoveAll([](const TWeakObjectPtr<UObject>& WeakActor)
	{
		return !WeakActor.IsValid() || WeakActor->IsA<AControlRigShapeActor>();
	});

	if (!Selection.IsEmpty())
	{
		SelectedObjects.Reserve(Selection.Num());
		for (TWeakObjectPtr<UObject> WeakActor: Selection)
		{
			if (AActor* Actor = Cast<AActor>(WeakActor))
			{
				SelectedObjects.Add(Actor);
			}
		}
		return;
	}
	
	// check component selection (minus bound components)
	GetModeManager()->GetSelectedComponents()->GetSelectedObjects(Selection);
	for (const TWeakObjectPtr<UControlRig>& ControlRigPtr : RuntimeControlRigs)
	{
		if (const TSharedPtr<IControlRigObjectBinding>& Binding = ControlRigPtr.IsValid() ? ControlRigPtr->GetObjectBinding() : nullptr)
		{
			if (UObject* BoundObject = Binding->GetBoundObject())
			{
				Selection.Remove(BoundObject);
			}
		}
	}
	SelectedObjects.Append(Selection);
}

void FControlRigEditMode::CacheNonInteractingRigsToUpdate()
{
	if (!IsInLevelEditor())
	{
		return;
	}

	bUpdateNonInteractingRigs = false;
	NonInteractingRigs.Reset();

	// cache any other rig which is not currently interacting
	if (!InteractionScopes.IsEmpty())
	{
		TSet<TWeakObjectPtr<UControlRig>> InteractingRigs;
		TSet<USceneComponent*> InteractingComponents;

		for (const auto& [InteractingRig, InteractionScope] : InteractionScopes)
		{
			if (InteractingRig)
			{
				InteractingRigs.Add(InteractingRig);
				if (USceneComponent* BoundComponent = GetHostingSceneComponent(InteractingRig))
				{
					InteractingComponents.Add(BoundComponent);
				}
			}
		}

		const TArrayView<const TWeakObjectPtr<UControlRig>> Rigs = GetControlRigs();
		if (InteractingRigs.Num() == Rigs.Num())
		{
			return;
		}
		
		for (const TWeakObjectPtr<UControlRig>& Rig: Rigs)
		{
			if (UControlRig* NonInteractingRig = Rig.Get(); !InteractingRigs.Contains(NonInteractingRig))
			{
				if (USceneComponent* BoundComponent = GetHostingSceneComponent(NonInteractingRig))
				{
					for (USceneComponent* ComponentToTick: InteractingComponents)
					{
						if (ConstraintsCache.HasAnyDependency(BoundComponent, ComponentToTick, WorldPtr))
						{
							NonInteractingRigs.Add(NonInteractingRig);
							break;
						}
					}
				}
			}
		}
		return;
	}

	// cache all the rigs that are related to the external objects that might be interacted with
	if (!SelectedObjects.IsEmpty())
	{
		for (const TWeakObjectPtr<UControlRig>& Rig: GetControlRigs())
		{
			if (UControlRig* NonInteractingRig = Rig.Get())
			{
				if (USceneComponent* BoundComponent = GetHostingSceneComponent(NonInteractingRig))
				{
					FComponentDependency ComponentDependency(BoundComponent, WorldPtr, ConstraintsCache);
					
					for (const TWeakObjectPtr<UObject>& SelectedObject: SelectedObjects)
					{
						if (UObject* Object = SelectedObject.Get())
						{
							if (AActor* Actor = Cast<AActor>(Object))
							{
								const TInlineComponentArray<USceneComponent*> Components(Actor);
								const bool bDepends = Components.ContainsByPredicate([&ComponentDependency](USceneComponent* Component)
								{
									return ComponentDependency.DependsOn(Component);
								});

								if (bDepends)
								{
									NonInteractingRigs.Add(NonInteractingRig);
									break;
								}
							}
							else if (ComponentDependency.DependsOn(Object))
							{
								NonInteractingRigs.Add(NonInteractingRig);
								break;
							}
						}
					}
				}
			}
		}
	}
}

void FControlRigEditMode::BindDirectControl()
{
	DirectControl.Bind(Owner);
	
	auto GetSequencer = [this]()
	{
		return WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	};
	
	auto ShouldSelectControlsOnly = [this]()
	{
		return GetOnlySelectRigControls();
	};
	
	DirectControl.HitProxyFilterFunction = [GetSequencer, ShouldSelectControlsOnly](HHitProxy* HitProxy)
	{
		if (HitProxy)
		{
			const HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy);
			if (!ActorHitProxy || !IsValid(ActorHitProxy->Actor))
			{
				return false;
			}
				
			if (ShouldSelectControlsOnly() && !ActorHitProxy->Actor->IsA<AControlRigShapeActor>())
			{
				return false;
			}
			
			if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
			{
				using SequencerSelectionGuard = TGuardValue_Bitfield_Cleanup<TFunction<void()>>;
				const SequencerSelectionGuard NotificationGuard([Sequencer, bSelectability = Sequencer->IsViewportSelectionLimited()]()
				{
					Sequencer->SetViewportSelectionLimited(bSelectability);
				});
				Sequencer->SetViewportSelectionLimited(true);
				return Sequencer->IsObjectSelectableInViewport(ActorHitProxy->Actor);
			}
				
			return true;
		}

		return false;
	};
	
	DirectControl.SequencerFunction = [this]()
	{
		return WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	};
}

#undef LOCTEXT_NAMESPACE
