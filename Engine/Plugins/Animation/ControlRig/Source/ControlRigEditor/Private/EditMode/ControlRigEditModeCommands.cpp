// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/ControlRigEditModeCommands.h"

#define LOCTEXT_NAMESPACE "ControlRigEditModeCommands"

void FControlRigEditModeCommands::RegisterCommands()
{
	UI_COMMAND(InvertTransformsAndChannels, "Invert Selected Controls And Channels to Rest Pose", "Invert the Selected Controls and Channels to Rest Pose", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InvertAllTransformsAndChannels, "Invert All Controls and Channels to Rest Pose", "Invert all of the Controls And Channels to Rest Pose", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InvertTransforms, "Invert to Rest Pose", "Invert the Controls Transforms to Rest Pose", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Control));
	UI_COMMAND(InvertAllTransforms, "Invert All Transform to Rest Pose", "Invert all of the Controls Transforms to Rest Pose", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Control | EModifierKey::Shift));
	UI_COMMAND(ZeroTransforms, "Zeros Transforms", "Resets Control Transforms to Zero", EUserInterfaceActionType::Button, FInputChord(EKeys::I, EModifierKey::Control));
	UI_COMMAND(ZeroAllTransforms, "Zeros All Transforms", "Resets all of the Control Transforms to Zero", EUserInterfaceActionType::Button, FInputChord(EKeys::I, EModifierKey::Control | EModifierKey::Shift));
	UI_COMMAND(ToggleManipulators, "Toggle Shapes", "Toggles visibility of active control rig shapes in the viewport", EUserInterfaceActionType::Button, FInputChord(EKeys::T, EModifierKey::Control | EModifierKey::Alt));
	UI_COMMAND(ToggleModuleManipulators, "Toggle Module Shapes", "Toggles visibility of active control rig shapes belonging to selected modules in the viewport", EUserInterfaceActionType::Button, FInputChord(EKeys::T, EModifierKey::Control));
	UI_COMMAND(ToggleAllManipulators, "Toggle All Shapes", "Toggles visibility of all control rig shapes in the viewport", EUserInterfaceActionType::Button, FInputChord(EKeys::T));
	UI_COMMAND(ToggleControlsAsOverlay, "Toggle Controls As Overlay", "Toggles showing controls as overlay", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FrameSelection, "Frame Selection", "Focus the viewport on the current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(ClearSelection, "Clear Selection", "Clear Selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));

	UI_COMMAND(SetAnimLayerPassthroughKey,"Set Anim Layer Passthrough Key","Set previous layer value on current selected anim layer", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectAnimLayerWeightChannel, "Select Anim Layer Weight Channel", "Select the weight channel for the selected anim layer", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SelectMirroredControls, "Select Mirrored Controls", " Select mirrored Controls on current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::M, EModifierKey::Control));

	UI_COMMAND(AddMirroredControlsToSelection, "Add Mirrored Controls to Selection", "Select mirrored Controls on current selection, keeping current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::M, EModifierKey::Shift));

	UI_COMMAND(MirrorSelectedControls, "Mirror Selected Controls", "Put selected Controls to mirrored Pose", EUserInterfaceActionType::Button, FInputChord(EKeys::M, EModifierKey::Shift | EModifierKey::Control));

	UI_COMMAND(MirrorUnselectedControls, "Mirror Unselected Controls", "Put unselected Controls to mirrored Selection Pose", EUserInterfaceActionType::Button, FInputChord(EKeys::M, EModifierKey::Alt));


	UI_COMMAND(SelectAllControls, "Select All Controls", "Select All Controls", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape, EModifierKey::Shift));

	UI_COMMAND(SavePose, "Save Current Pose", "Save current Pose", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectPose, "Select Saved Pose", "Select Controls on saved Pose", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PastePose, "Paste Saved Pose", "Paste Controls from saved Pose", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectMirrorPose, "Select Mirrored Saved Pose", "Select mirrored Controls on saved Pose", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteMirrorPose, "Paste Mirrored Saved Pose", "Paste mirrored Controls from saved Pose", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(TogglePivotMode, "Toggle Pivot Mode Tool", "Toggle Pivot Mode Tool", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleMotionTrails, "Toggle Motion Paths Tool", "Toggle Motion paths Tool", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ToggleControlShapeTransformEdit, "Toggle Shape Transform Edit", "Toggle Editing Selected Control's Shape Transform", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Period, EModifierKey::Control)); 
	UI_COMMAND(OpenSpacePickerWidget, "Open the Space Picker", "Allows space switching on the control", EUserInterfaceActionType::Button, FInputChord(EKeys::Tab)); 

	UI_COMMAND(IsolateLinkedAnimTracks, "Isolate Linked Anim Tracks", "Isolate linked anim tracks for every binding in the current Sequencer", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UnIsolateLinkedAnimTracks, "Un-Isolate Linked Anim Tracks", "Un-isolate linked anim tracks for every binding in the current Sequencer", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SummonSelectionSetsWidget, "Move Selection Sets to cursor", "Places the selection sets overlay at your mouse position", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleSelectionSetsWidget, "Show or hide Selection Sets", "Shows the selection sets UI if it's hidden, and hides it if it's visible.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SummonTweenWidget, "Move Tween Slider to cursor", "Places the tween slider at your mouse position", EUserInterfaceActionType::Button, FInputChord(EKeys::U)); 
	UI_COMMAND(ToggleTweenWidget, "Show or hide Tween Slider", "Shows the tween slider if it's hidden, and hides it if it's visible.", EUserInterfaceActionType::Button, FInputChord(EKeys::U, EModifierKey::Alt));
	
	UI_COMMAND(ToggleAnimLayersTab, "Show or hide Anim Layers tab", "Toggles the visibility of the Anim Layers tab.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(TogglePoseLibraryTab, "Show or hide Pose Library tab", "Toggles the visibility of the Pose Library tab.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ToggleConstrainTab, "Show or hide Constrain tab", "Toggles the visibility of the constrain tab.", EUserInterfaceActionType::Button, FInputChord()); 
	UI_COMMAND(OpenSpacesTab, "Open Spaces tab", "Opens the spaces tab in the constrain tab.", EUserInterfaceActionType::Button, FInputChord()); 
	UI_COMMAND(OpenConstraintsTab, "Open Constraints tab", "Opens the constraints tab in the constrain tab.", EUserInterfaceActionType::Button, FInputChord()); 
	UI_COMMAND(OpenSnapperTab, "Open Snapper tab", "Opens the snapper tab in the constrain tab.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ClickAndDrag, "Enable Click and Drag", "Enables Click and Drag manipulation.", EUserInterfaceActionType::None, FInputChord(EKeys::N));
}

#undef LOCTEXT_NAMESPACE
