// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "ControlRigEditorStyle.h"

class FControlRigEditModeCommands : public TCommands<FControlRigEditModeCommands>
{

public:
	FControlRigEditModeCommands() : TCommands<FControlRigEditModeCommands>
	(
		"ControlRigEditMode",
		NSLOCTEXT("Contexts", "RigAnimation", "Rig Animation"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FControlRigEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	

	/** Toggles hiding all manipulators on active control rig in the viewport */
	TSharedPtr< FUICommandInfo > ToggleManipulators;

	/** Toggles hiding all manipulators belonging to the selected module on active control rig in the viewport */
    TSharedPtr< FUICommandInfo > ToggleModuleManipulators;

	/** Toggles hiding all manipulators on all control rigs in the viewport */
	TSharedPtr< FUICommandInfo > ToggleAllManipulators;

	/** Toggles show controls as an overlay in the viewport */
	TSharedPtr< FUICommandInfo > ToggleControlsAsOverlay;

	/** Invert Transforms and channels to Rest Pose */
	TSharedPtr< FUICommandInfo > InvertTransformsAndChannels;

	/** Invert All Transforms and channels  to Rest Pose */
	TSharedPtr< FUICommandInfo > InvertAllTransformsAndChannels;

	/** Invert Transforms to Rest Pose */
	TSharedPtr< FUICommandInfo > InvertTransforms;

	/** Invert All Transforms to Rest Pose */
	TSharedPtr< FUICommandInfo > InvertAllTransforms;

	/** Set Transforms to Zero */
	TSharedPtr< FUICommandInfo > ZeroTransforms;

	/** Set All Transforms to Zero */
	TSharedPtr< FUICommandInfo > ZeroAllTransforms;

	/** Clear Selection*/
	TSharedPtr< FUICommandInfo > ClearSelection;

	/** Frame selected elements */
	TSharedPtr<FUICommandInfo> FrameSelection;

	/** Toggle Shape Transform Edit*/
	TSharedPtr< FUICommandInfo > ToggleControlShapeTransformEdit;

	/** Sets Passthrough Key on selected anim layers */
	TSharedPtr< FUICommandInfo > SetAnimLayerPassthroughKey;

	/** Select weight channel on selected anim layers */
	TSharedPtr< FUICommandInfo > SelectAnimLayerWeightChannel;

	/** Select mirrored Controls on current selection*/
	TSharedPtr< FUICommandInfo > SelectMirroredControls;

	/** Select mirrored Controls on current selection, keeping current selection*/
	TSharedPtr< FUICommandInfo > AddMirroredControlsToSelection;

	/** Put selected Controls to mirrored Pose*/
	TSharedPtr< FUICommandInfo > MirrorSelectedControls;

	/** Put unselected Controls to mirrored selected Controls*/
	TSharedPtr< FUICommandInfo > MirrorUnselectedControls;

	/** Select all Controls*/
	TSharedPtr< FUICommandInfo > SelectAllControls;

	/** Save a pose of selected Controls*/
	TSharedPtr< FUICommandInfo > SavePose;

	/** Select Controls in saved Pose*/
	TSharedPtr< FUICommandInfo > SelectPose;

	/** Paste saved Pose */
	TSharedPtr< FUICommandInfo > PastePose;

	/** Select Controls in saved Pose*/
	TSharedPtr< FUICommandInfo > SelectMirrorPose;

	/** Paste saved mirror Pose */
	TSharedPtr< FUICommandInfo > PasteMirrorPose;

	/**Toggle Pivot Mode*/
	TSharedPtr< FUICommandInfo> TogglePivotMode;

	/**Toggle Motion Trail Mode*/
	TSharedPtr< FUICommandInfo> ToggleMotionTrails;

	/** Opens up the space picker widget */
	TSharedPtr< FUICommandInfo > OpenSpacePickerWidget;

	/** Opens the tween slider widget and positions it at the mouse. */
	TSharedPtr< FUICommandInfo > SummonTweenWidget;

	/** Toggles the visibility of the tween widget. */
	TSharedPtr< FUICommandInfo > ToggleTweenWidget;

	/** Opens the selection sets slider widget and positions it at the mouse. */
	TSharedPtr< FUICommandInfo > SummonSelectionSetsWidget;

	/** Toggles the visibility of the selection sets widget. */
	TSharedPtr< FUICommandInfo > ToggleSelectionSetsWidget;

	/** Toggles the visibility of the anim layers window. */
	TSharedPtr< FUICommandInfo > ToggleAnimLayersTab;
	
	/** Toggles the visibility of the anim pose library window. */
	TSharedPtr< FUICommandInfo > TogglePoseLibraryTab;
	
	/** Toggles the visibility of the constrain window. */
	TSharedPtr< FUICommandInfo > ToggleConstrainTab;

	/** Opens the spaces tab in the constrain window. */
	TSharedPtr< FUICommandInfo > OpenSpacesTab;
	
	/** Opens the constraints tab in the constrain window. */
	TSharedPtr< FUICommandInfo > OpenConstraintsTab;
	
	/** Opens the snapper tab in the constrain window. */
	TSharedPtr< FUICommandInfo > OpenSnapperTab;

	/** Cycle to the next direct manipulation target on the selected node */
	TSharedPtr< FUICommandInfo > CycleDirectManipulationTargetNext;

	/** Cycle to the previous direct manipulation target on the selected node */
	TSharedPtr< FUICommandInfo > CycleDirectManipulationTargetPrev;

	/** Isolate linked anim tracks for every binding in the current Sequencer */
	TSharedPtr< FUICommandInfo > IsolateLinkedAnimTracks;

	/** Un-isolate linked anim tracks for every binding in the current Sequencer */
	TSharedPtr< FUICommandInfo > UnIsolateLinkedAnimTracks;
	
	/** Binding used to validate the click-and-drag pre-click functionality. */
	TSharedPtr< FUICommandInfo > ClickAndDrag;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
