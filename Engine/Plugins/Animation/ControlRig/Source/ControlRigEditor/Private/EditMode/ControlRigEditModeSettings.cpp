// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/ControlRigEditModeSettings.h"
#include "EditorModeManager.h"
#include "EditMode/ControlRigEditMode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigEditModeSettings)

UControlRigEditModeSettings::FOnUpdateSettings UControlRigEditModeSettings::OnSettingsChange;

void UControlRigEditModeSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);
}

void UControlRigEditModeSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
		SaveConfig();
	}
	OnSettingsChange.Broadcast(this);
}

#if WITH_EDITOR
void UControlRigEditModeSettings::PostEditUndo()
{
	OnSettingsChange.Broadcast(this);
}
#endif


