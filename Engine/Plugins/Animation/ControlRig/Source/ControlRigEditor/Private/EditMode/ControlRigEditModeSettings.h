// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Settings/ControlRigUIRestoreStates.h"
#include "UObject/Object.h"
#include "ControlRigEditModeSettings.generated.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FGizmoScaleSet, float /* GizmoScale */);

/** Specifies the selection policy applied when using click-and-drag interactions. */
UENUM(MinimalAPI, meta=(Experimental))
enum class EClickAndDragBehavior : uint8
{
	// click-and-drag interaction is disabled.
	Disabled = 0,
	// temporarily selects the hovered element during the interaction (if nothing is already selected).
	TransientSelection,
	// selects the hovered element and preserves the selection after the interaction ends.
	PersistentSelection
};

/** Settings object used to show useful information in the details panel */
UCLASS(config=EditorPerProjectUserSettings, MinimalAPI, meta=(DisplayName="Control Rig Edit Mode"))
class UControlRigEditModeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UControlRigEditModeSettings()
		: bDisplayHierarchy(false)
		, bDisplayNulls(false)
		, bDisplaySockets(false)
		, bHideControlShapes(false)
		, bShowAllProxyControls(false)
		, bShowControlsAsOverlay(false)
		, bDisplayAxesOnSelection(false)
		, AxisScale(10.f)
		, bCoordSystemPerWidgetMode(true)
		, bOnlySelectRigControls(false)
		, bLocalTransformsInEachLocalSpace(true)
	{
		DrivenControlColor = FLinearColor::White * FLinearColor(FVector::OneVector * 0.8f);
	}

	// UObject interface
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
public:

	static UControlRigEditModeSettings* Get() { return GetMutableDefault<UControlRigEditModeSettings>(); }

	/** Whether to show all bones in the hierarchy */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplayHierarchy;

	/** Whether to show all nulls in the hierarchy */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplayNulls;

	/** Should we show sockets in the viewport */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplaySockets;

	/** Should we always hide control shapes in viewport */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bHideControlShapes;

	/** Should we always hide control shapes in viewport */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings", meta = (EditCondition = "!bHideControlShapes"))
	bool bShowAllProxyControls;

	/** Determins if controls should be rendered on top of other controls */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings", meta = (EditCondition = "!bHideControlShapes"))
	bool bShowControlsAsOverlay;

	/** Indicates a control being driven by a proxy control */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings", meta = (EditCondition = "!bHideControlShapes"))
	FLinearColor DrivenControlColor;

	/** Should we show axes for the selected elements */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bDisplayAxesOnSelection;

	/** The scale for axes to draw on the selection */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	float AxisScale;

	/** If true we restore the coordinate space when changing Widget Modes in the Viewport*/
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bCoordSystemPerWidgetMode;

	/** If true we can only select Rig Controls in the scene not other Actors. */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bOnlySelectRigControls;

	/** If true when we transform multiple selected objects in the viewport they each transforms along their own local transform space */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings")
	bool bLocalTransformsInEachLocalSpace;

	/** Data to restore the UI state control rig had last time it was open. */
	UPROPERTY(config)
	FControlRigUIRestoreStates LastUIStates;

	/** The tint to apply when the tween overlay is out of focus. */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings|Tween")
	FLinearColor TweenOutOfFocusTint = FLinearColor(1.f, 1.f, 1.f, .35f);

	/**
	 * If enabled, the DragAnimSliderTool hotkey should position the widget at the mouse cursor if the widget was visible when you pressed it.
	 * If disabled, the DragAnimSliderTool hotkey will leave the widget where you placed it if the widget was visible when you pressed it.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings|Tween")
	bool bIndirectSliderMovementShouldSnapSliderToMouse = false;

	/**
	 * Enables the tab menu customization so that users can register contextual utility actions.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings|Workflow")
	bool bSupportUtilityActions = true;
	
	/**
	 * Enables or disables click-and-drag selection and defines how it behaves.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Animation Settings|Interaction", meta=(Experimental))
	EClickAndDragBehavior ClickAndDragBehavior = EClickAndDragBehavior::Disabled;
	
	/** Hovered color used when hovering a control shape. */
    UPROPERTY(config, EditAnywhere, Category = "Animation Settings", meta = (EditCondition = "!bHideControlShapes"))
    FLinearColor HoveredColor = FLinearColor(0.5f, 0.5f, 0.5f, 0.25f);
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateSettings, const UControlRigEditModeSettings*);
	static FOnUpdateSettings OnSettingsChange;
};