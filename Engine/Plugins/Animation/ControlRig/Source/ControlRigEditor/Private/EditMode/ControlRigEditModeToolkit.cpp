// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Control Rig Edit Mode Toolkit
*/
#include "EditMode/ControlRigEditModeToolkit.h"

#include "ControlRigEditModeCommands.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/Views/SAnimDetailsView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "EditorModes.h"
#include "Toolkits/BaseToolkit.h"
#include "EditorModeManager.h"
#include "EditMode/SControlRigEditModeTools.h"
#include "EditMode/ControlRigEditMode.h"
#include "Modules/ModuleManager.h"
#include "EditMode/SControlRigBaseListWidget.h"
#include "EditMode/SControlRigSnapper.h"
#include "Tools/SMotionTrailOptions.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Widgets/Docking/SDockTab.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "EditMode/SControlRigOutliner.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"
#include "SLevelViewport.h"
#include "Models/RigSelectionViewModel.h"
#include "Sequencer/AnimLayers/SAnimLayers.h"
#include "Sequencer/AnimLayers/AnimLayers.h"
#include "Sequencer/SelectionSets/SSelectionSets.h"
#include "Sequencer/SelectionSets/SelectionSets.h"
#include "Tools/MotionTrailOptions.h"
#include "UObject/WeakObjectPtr.h"
#include "ISequencer.h"


class SAnimDetailsView;
namespace UE::ControlRigEditor
{
	class SAnimDetailsView;
}

#define LOCTEXT_NAMESPACE "FControlRigEditModeToolkit"

namespace 
{
	static const FName AnimationName(TEXT("Animation")); 
	const TArray<FName> AnimationPaletteNames = { AnimationName };
}

const FName FControlRigEditModeToolkit::PoseTabName = FName(TEXT("PoseTab"));
const FName FControlRigEditModeToolkit::MotionTrailTabName = FName(TEXT("MotionTrailTab"));
const FName FControlRigEditModeToolkit::SnapperTabName = FName(TEXT("SnapperTab"));
const FName FControlRigEditModeToolkit::AnimLayerTabName = FName(TEXT("AnimLayerTab"));
const FName FControlRigEditModeToolkit::TweenOverlayName = FName(TEXT("TweenOverlay"));
const FName FControlRigEditModeToolkit::ConstrainingTabName = FName(TEXT("ConstrainingTab"));

#if SELECTION_SETS_AS_TAB

const FName FControlRigEditModeToolkit::SelectionSetsTabName = FName(TEXT("SelectionSets"));

#else

const FName FControlRigEditModeToolkit::SelectionSetsOverlayName = FName(TEXT("SelectionSets"));

#endif
const FName FControlRigEditModeToolkit::OutlinerTabName = FName(TEXT("ControlRigOutlinerTab"));
const FName FControlRigEditModeToolkit::DetailsTabName = FName(TEXT("ControlRigDetailsTab"));

TSharedPtr<UE::ControlRigEditor::SAnimDetailsView> FControlRigEditModeToolkit::Details = nullptr;
TSharedPtr<SControlRigOutliner> FControlRigEditModeToolkit::Outliner = nullptr;

FControlRigEditModeToolkit::FControlRigEditModeToolkit(FControlRigEditMode& InEditMode)
	: EditMode(InEditMode)
	, SelectionViewModel(MakeShared<UE::ControlRigEditor::FRigSelectionViewModel>())
{}

FControlRigEditModeToolkit::~FControlRigEditModeToolkit()
{
	if (ModeTools)
	{
		ModeTools->Cleanup();
	}
}

void FControlRigEditModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost)
{
	SAssignNew(ModeTools, SControlRigEditModeTools, SelectionViewModel);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bSearchInitialKeyFocus = false;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	BindCommands();
	
	FModeToolkit::Init(InitToolkitHost);
}

void FControlRigEditModeToolkit::GetToolPaletteNames(TArray<FName>& InPaletteName) const
{
	InPaletteName = AnimationPaletteNames;
}

FText FControlRigEditModeToolkit::GetToolPaletteDisplayName(FName PaletteName) const
{
	if (PaletteName == AnimationName)
	{
		return FText::FromName(AnimationName);
	}
	return FText();
}

void FControlRigEditModeToolkit::BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolBarBuilder)
{
	if (PaletteName == AnimationName)
	{
		BuildToolBar(ToolBarBuilder);
	}
}

void FControlRigEditModeToolkit::TryToggleToolkitUI(const FName InName)
{
	if (IsToolkitUIActive(InName))
	{
		TryCloseToolkitUI(InName);
	}
	else
	{
		TryInvokeToolkitUI(InName);
	}
}

namespace UE::ControlRigEditor::ToolkitDetail
{
/**
 * @return The tab ID for one of the logical FControlRigEditModeToolkit::xxxTabName.
 * They mostly coincide but are different for those that re-use the UAssetEditorUISubsystem tab names.
 */
static TOptional<FName> GetTabIdForUI(const FName InName)
{
	if (InName == FControlRigEditModeToolkit::MotionTrailTabName)
	{
		return FControlRigEditModeToolkit::MotionTrailTabName;
	}

	if (InName == FControlRigEditModeToolkit::AnimLayerTabName)
	{
		return FControlRigEditModeToolkit::AnimLayerTabName;
	}

	if (InName == FControlRigEditModeToolkit::PoseTabName)
	{
		return FControlRigEditModeToolkit::PoseTabName;
	}

	if (InName == FControlRigEditModeToolkit::SnapperTabName)
	{
		return FControlRigEditModeToolkit::SnapperTabName;
	}

	if (InName == FControlRigEditModeToolkit::OutlinerTabName)
	{
		return UAssetEditorUISubsystem::TopRightTabID;
	}

	if (InName == FControlRigEditModeToolkit::DetailsTabName)
	{
		return UAssetEditorUISubsystem::BottomRightTabID;
	}

#if SELECTION_SETS_AS_TAB
	if (InName == FControlRigEditModeToolkit::SelectionSetsTabName)
	{
		return FControlRigEditModeToolkit::SelectionSetsTabName;
	}
#endif

	return {};
}
}

void FControlRigEditModeToolkit::TryInvokeToolkitUI(const FName InName)
{
	const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
	const TOptional<FName> TabName = UE::ControlRigEditor::ToolkitDetail::GetTabIdForUI(InName);
	
	if (ModeUILayerPtr && TabName)
	{
		const FTabId TabID(*TabName);
		ModeUILayerPtr->GetTabManager()->TryInvokeTab(TabID, false /*bIsActive*/);
	}
	else if (InName == TweenOverlayName)
	{
		TweenOverlayManager->ShowWidget();
	}
	else if (InName == ConstrainingTabName)
	{
		ConstrainToolsManager->ShowWidget();
	}
#if SELECTION_SETS_AS_TAB == 0
	else if (InName == SelectionSetsOverlayName)
	{
		SelectionSetsOverlayManager->ShowWidget();
	}
#endif
}

void FControlRigEditModeToolkit::TryCloseToolkitUI(const FName InName) const
{
	const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
	const TOptional<FName> TabName = UE::ControlRigEditor::ToolkitDetail::GetTabIdForUI(InName);
	const TSharedPtr<SDockTab> LiveTab = ModeUILayerPtr && TabName ? ModeUILayerPtr->GetTabManager()->FindExistingLiveTab(FTabId(*TabName)) : nullptr;
	if (TabName && LiveTab)
	{
		LiveTab->RequestCloseTab();
	}
	else if (TabName)
	{
		return;
	}
	
	if (InName == TweenOverlayName)
	{
		TweenOverlayManager->HideWidget();
	}
	else if (InName == ConstrainingTabName)
	{
		ConstrainToolsManager->HideWidget();
	}
#if SELECTION_SETS_AS_TAB == 0
	else if (InName == SelectionSetsOverlayName)
	{
		SelectionSetsOverlayManager->HideWidget();
	}
#endif
}

bool FControlRigEditModeToolkit::IsToolkitUIActive(const FName InName) const
{
	if (InName == TweenOverlayName)
	{
		return TweenOverlayManager->IsShowingWidget();
	}
#if SELECTION_SETS_AS_TAB == 0
	else if (InName == SelectionSetsOverlayName)
	{
		return SelectionSetsOverlayManager->IsShowingWidget();
	}
#endif

	if (InName == ConstrainingTabName)
	{
		return ConstrainToolsManager->IsShowingWidget();
	}
	
	const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
	const TOptional<FName> TabName = UE::ControlRigEditor::ToolkitDetail::GetTabIdForUI(InName);
	return ModeUILayerPtr && ensureMsgf(TabName, TEXT("You passed in an unknown UI name!"))
		&& ModeUILayerPtr->GetTabManager()->FindExistingLiveTab(FTabId(*TabName)).IsValid();
}

void FControlRigEditModeToolkit::OnControlsChanged(TConstArrayView<TWeakObjectPtr<UControlRig>> InControlRigs) const
{
	SelectionViewModel->SetControls(InControlRigs);
}

FText FControlRigEditModeToolkit::GetActiveToolDisplayName() const
{
	return FText::FromString(TEXT("Control Rig Editing"));
}

FText FControlRigEditModeToolkit::GetActiveToolMessage() const
{
	return FText::GetEmpty();
}

TSharedRef<SDockTab> SpawnPoseTab(const FSpawnTabArgs& Args, TWeakPtr<FControlRigEditModeToolkit> SharedToolkit)
{
	return SNew(SDockTab)
		[
			SNew(SControlRigBaseListWidget)
			.InSharedToolkit(SharedToolkit)
		];
}

TSharedRef<SDockTab> SpawnSnapperTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		[
			SNew(SControlRigSnapper)
		];
}

TSharedRef<SDockTab> SpawnMotionTrailTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		[
			SNew(SMotionTrailOptions)
		];
}

#if SELECTION_SETS_AS_TAB
	TSharedRef<SDockTab> SpawnSelectionSetsTab(const FSpawnTabArgs& Args, FControlRigEditMode* InEditorMode)
	{
		return SNew(SDockTab)
			[
				SNew(SSelectionSets, *InEditorMode)
			];
	}
#endif
TSharedRef<SDockTab> SpawnAnimLayerTab(const FSpawnTabArgs& Args, FControlRigEditMode* InEditorMode)
{
	return SNew(SDockTab)
		[
			SNew(SAnimLayers, *InEditorMode)
		];
}

TSharedRef<SDockTab> SpawnOutlinerTab(const FSpawnTabArgs& Args, FControlRigEditMode* InEditorMode)
{
	return SNew(SDockTab)
		[
			SAssignNew(FControlRigEditModeToolkit::Outliner, SControlRigOutliner, *InEditorMode)
		];
}

TSharedRef<SDockTab> SpawnDetailsTab(const FSpawnTabArgs& Args, FControlRigEditMode* InEditorMode)
{
	UAnimDetailsProxyManager* ProxyManager = InEditorMode ? InEditorMode->GetAnimDetailsProxyManager() : nullptr;

	if (ProxyManager)
	{
		ProxyManager->SetSuspended(false);

		const TWeakObjectPtr<UAnimDetailsProxyManager> WeakProxyManager = ProxyManager;
		return SNew(SDockTab)
			.OnTabClosed_Lambda([WeakProxyManager](TSharedRef<SDockTab> InTab)
				{
					if (WeakProxyManager.IsValid())
					{
						// For performance reasons, suspend the anim details proxy manager while its view is not spawned
						WeakProxyManager->SetSuspended(true);
					}
				})
			[
				SAssignNew(FControlRigEditModeToolkit::Details, UE::ControlRigEditor::SAnimDetailsView)
			];
	}
	else
	{
		return SNew(SDockTab)
			[
				SNullWidget::NullWidget
			];
	}
}

void FControlRigEditModeToolkit::RequestModeUITabs()
{
	FModeToolkit::RequestModeUITabs();
	if (ModeUILayer.IsValid())
	{
		const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		const TSharedRef<FWorkspaceItem> MenuGroup = ModeUILayerPtr->GetModeMenuCategory().ToSharedRef();
		WorkspaceMenuCategory = MenuGroup;

		FMinorTabConfig DetailTabInfo;
		DetailTabInfo.OnSpawnTab = FOnSpawnTab::CreateStatic(&SpawnDetailsTab, &EditMode);
		DetailTabInfo.TabLabel = LOCTEXT("ControlRigDetailTab", "Anim Details");
		DetailTabInfo.TabTooltip = LOCTEXT("ControlRigDetailTabTooltip", "Show Details For Selected Controls.");
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::BottomRightTabID, DetailTabInfo);

		FMinorTabConfig OutlinerTabInfo;
		OutlinerTabInfo.OnSpawnTab = FOnSpawnTab::CreateStatic(&SpawnOutlinerTab, &EditMode);
		OutlinerTabInfo.TabLabel = LOCTEXT("AnimationOutlinerTab", "Anim Outliner");
		OutlinerTabInfo.TabTooltip = LOCTEXT("AnimationOutlinerTabTooltip", "Control Rig Controls");
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::TopRightTabID, OutlinerTabInfo);
		/* doesn't work as expected
		FMinorTabConfig SpawnSpacePickerTabInfo;
		SpawnSpacePickerTabInfo.OnSpawnTab = FOnSpawnTab::CreateStatic(&SpawnSpacePickerTab, &EditMode);
		SpawnSpacePickerTabInfo.TabLabel = LOCTEXT("ControlRigSpacePickerTab", "Control Rig Space Picker");
		SpawnSpacePickerTabInfo.TabTooltip = LOCTEXT("ControlRigSpacePickerTabTooltip", "Control Rig Space Picker");
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::TopLeftTabID, SpawnSpacePickerTabInfo);
		*/

		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(SnapperTabName);
		ModeUILayerPtr->GetTabManager()->RegisterTabSpawner(SnapperTabName, FOnSpawnTab::CreateStatic(&SpawnSnapperTab))
			.SetDisplayName(LOCTEXT("ControlRigSnapperTab", "Control Rig Snapper"))
			.SetTooltipText(LOCTEXT("ControlRigSnapperTabTooltip", "Snap child objects to a parent object over a set of frames."))
			.SetGroup(MenuGroup)
			.SetIcon(FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.ConstraintTools")));
		ModeUILayerPtr->GetTabManager()->RegisterDefaultTabWindowSize(SnapperTabName, FVector2D(300, 325));

		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(PoseTabName);

		TWeakPtr<FControlRigEditModeToolkit> WeakToolkit = SharedThis(this);
		ModeUILayerPtr->GetTabManager()->RegisterTabSpawner(PoseTabName, FOnSpawnTab::CreateLambda([WeakToolkit](const FSpawnTabArgs& Args)
		{
			return SpawnPoseTab(Args, WeakToolkit);
		}))
			.SetDisplayName(LOCTEXT("ControlRigPoseTab", "Control Rig Pose"))
			.SetTooltipText(LOCTEXT("ControlRigPoseTabTooltip", "Show Poses."))
			.SetGroup(MenuGroup)
			.SetIcon(FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.PoseTool")));
		ModeUILayerPtr->GetTabManager()->RegisterDefaultTabWindowSize(PoseTabName, FVector2D(675, 625));

		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(MotionTrailTabName);
		ModeUILayerPtr->GetTabManager()->RegisterTabSpawner(MotionTrailTabName, FOnSpawnTab::CreateStatic(&SpawnMotionTrailTab))
			.SetDisplayName(LOCTEXT("MotionTrailTab", "Motion Trail"))
			.SetTooltipText(LOCTEXT("MotionTrailTabTooltip", "Display motion trails for animated objects."))
			.SetGroup(MenuGroup)
			.SetIcon(FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.EditableMotionTrails")));
		ModeUILayerPtr->GetTabManager()->RegisterDefaultTabWindowSize(MotionTrailTabName, FVector2D(425, 575));

		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(AnimLayerTabName);
		ModeUILayerPtr->GetTabManager()->RegisterTabSpawner(AnimLayerTabName, FOnSpawnTab::CreateStatic(&SpawnAnimLayerTab,&EditMode))
			.SetDisplayName(LOCTEXT("AnimLayerTab", "Anim Layers"))
			.SetTooltipText(LOCTEXT("AnimationLayerTabTooltip", "Animation layers"))
			.SetGroup(MenuGroup)
			.SetIcon(FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.AnimLayers")));
		ModeUILayerPtr->GetTabManager()->RegisterDefaultTabWindowSize(AnimLayerTabName, FVector2D(425, 200));

#if SELECTION_SETS_AS_TAB
		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(SelectionSetsTabName);
		ModeUILayerPtr->GetTabManager()->RegisterTabSpawner(SelectionSetsTabName, FOnSpawnTab::CreateStatic(&SpawnSelectionSetsTab, &EditMode))
			.SetDisplayName(LOCTEXT("SelectionSetsTab", "Sets"))
			.SetTooltipText(LOCTEXT("SelectionSetsTabTooltip", "Selection Sets"))
			.SetGroup(MenuGroup)
			.SetIcon(FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.SelectionSet")));
		ModeUILayerPtr->GetTabManager()->RegisterDefaultTabWindowSize(SelectionSetsTabName, FVector2D(425, 200));

#endif


	}
};

void FControlRigEditModeToolkit::InvokeUI()
{
	FModeToolkit::InvokeUI();

	const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
	if (!ModeUILayerPtr)
	{
		return;
	}

	ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::TopRightTabID);
	// doesn't work as expected todo ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::TopLeftTabID);
	ModeUILayerPtr->GetTabManager()->TryInvokeTab(UAssetEditorUISubsystem::BottomRightTabID);

	// TweenOverlayManager will automatically restore its last visibility state upon construction.
	TweenOverlayManager = MakeUnique<UE::ControlRigEditor::FTweenOverlayManager>(GetToolkitHost(), GetToolkitCommands(), SharedThis(&EditMode));

	checkf(WorkspaceMenuCategory, TEXT("WorkspaceMenuCategory should have been set in RequestModeUITabs by now."))
	ConstrainToolsManager = MakeUnique<UE::ControlRigEditor::FConstrainToolsManager>(
		ModeUILayerPtr->GetTabManager().ToSharedRef(), GetWorkspaceMenuCategory(), GetToolkitCommands(), EditMode, SelectionViewModel
		);
	
	const FControlRigUIRestoreStates& RestoreStates = UControlRigEditModeSettings::Get()->LastUIStates;
	
#if SELECTION_SETS_AS_TAB
	if (RestoreStates.bSelectionSetsOpen)
	{
		TryInvokeToolkitUI(SelectionSetsTabName);
	}
	/*
	else if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		//wasn't open but there are layers in open level sequence we try to open it
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			if (SelectionSets && SelectionSets->bOpenUIOnOpen)
			{
				TryInvokeToolkitUI(SelectionSetsTabName);
			}
		}
	}
	*/
#else
	SelectionSetsOverlayManager = MakeUnique<UE::AIE::FSelectionSetsOverlayManager>(GetToolkitHost(), GetToolkitCommands(), SharedThis(&EditMode));
	if (RestoreStates.bSelectionSetsOpen)
	{
		SelectionSetsOverlayManager->ShowWidget();
	}
#endif	

	if (RestoreStates.ConstraintsTabState.bWasOpen)
	{
		ConstrainToolsManager->ShowWidget();
	}
	
	if (RestoreStates.bAnimLayerTabOpen)
	{
		TryInvokeToolkitUI(AnimLayerTabName);
	}
	else if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		// Wasn't open but if there are layers in open level sequence we try to open it
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			if (AnimLayers && AnimLayers->bOpenUIOnOpen)
			{
				TryInvokeToolkitUI(AnimLayerTabName);
			}
		}
	}
	
	if (RestoreStates.bSnapperTabOpen)
	{
		TryInvokeToolkitUI(SnapperTabName);
	}
	
	if (RestoreStates.bPoseTabOpen)
	{
		TryInvokeToolkitUI(PoseTabName);
	}

	UMotionTrailToolOptions* Settings = GetMutableDefault<UMotionTrailToolOptions>();
	if (RestoreStates.bMotionTrailsOn && Settings)
	{
		Settings->bShowTrails = true;
		FPropertyChangedEvent ShowTrailEvent(
			UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails))
			);
		Settings->PostEditChangeProperty(ShowTrailEvent);
	}
}

void FControlRigEditModeToolkit::ShutdownUI()
{
	FModeToolkit::ShutdownUI();
	UnregisterAndRemoveFloatingTabs();
}

void FControlRigEditModeToolkit::UnregisterAndRemoveFloatingTabs()
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}

	SaveLayoutState();
	FControlRigUIRestoreStates& RestoreStates = UControlRigEditModeSettings::Get()->LastUIStates;

	if (TweenOverlayManager)
	{
		TweenOverlayManager.Reset();
	}
	
	if (ConstrainToolsManager)
	{
		ConstrainToolsManager.Reset();
	}

#if SELECTION_SETS_AS_TAB == 0
	if (SelectionSetsOverlayManager)
	{
		if (RestoreStates.bSelectionSetsOpen)
		{
			SelectionSetsOverlayManager->DestroyWidget();
		}
		SelectionSetsOverlayManager.Reset();
	}
#endif

	if (const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
	{
		TryCloseToolkitUI(MotionTrailTabName);
		TryCloseToolkitUI(AnimLayerTabName);
		TryCloseToolkitUI(SnapperTabName);
		TryCloseToolkitUI(PoseTabName);
#if SELECTION_SETS_AS_TAB
		TryCloseToolkitUI(SelectionSetsTabName);
#endif

		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(MotionTrailTabName);
		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(AnimLayerTabName);
		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(SnapperTabName);
		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(PoseTabName);
#if SELECTION_SETS_AS_TAB
		ModeUILayerPtr->GetTabManager()->UnregisterTabSpawner(SelectionSetsTabName);
#endif

		if (UMotionTrailToolOptions* Settings = GetMutableDefault<UMotionTrailToolOptions>())
		{
			if (Settings->bShowTrails)
			{
				Settings->bShowTrails = false;
				FPropertyChangedEvent ShowTrailEvent(
					UMotionTrailToolOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMotionTrailToolOptions, bShowTrails))
					);
				Settings->PostEditChangeProperty(ShowTrailEvent);
			}
		}

		const TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
		UAnimLayers* AnimLayers = SequencerPtr ? UAnimLayers::GetAnimLayers(SequencerPtr.Get()) : nullptr;
		if (AnimLayers)
		{
			AnimLayers->bOpenUIOnOpen = RestoreStates.bAnimLayerTabOpen;
		}
	}
}

void FControlRigEditModeToolkit::SaveLayoutState() const
{
	UControlRigEditModeSettings* Settings = UControlRigEditModeSettings::Get();
	FControlRigUIRestoreStates& RestoreStates = Settings->LastUIStates;
	
	RestoreStates.bMotionTrailsOn = IsToolkitUIActive(MotionTrailTabName);
	RestoreStates.bAnimLayerTabOpen = IsToolkitUIActive(AnimLayerTabName);
	RestoreStates.bPoseTabOpen = IsToolkitUIActive(PoseTabName);
	RestoreStates.bSnapperTabOpen = IsToolkitUIActive(SnapperTabName);
	RestoreStates.ConstraintsTabState.bWasOpen = IsToolkitUIActive(ConstrainingTabName);
#if SELECTION_SETS_AS_TAB
	RestoreStates.bSelectionSetsOpen = IsToolkitUIActive(SelectionSetsTabName);
#else
	RestoreStates.bSelectionSetsOpen = IsToolkitUIActive(SelectionSetsOverlayName);
#endif
	
	Settings->SaveConfig();
}

void FControlRigEditModeToolkit::BuildToolBar(FToolBarBuilder& InToolBarBuilder)
{
	// CURVE EDITOR
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateLambda([]()
			{
				const TWeakPtr<ISequencer>& WeakSequencer = UE::AnimationEditMode::GetSequencer();
				if (const TSharedPtr<ISequencer>& Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr)
				{
					Sequencer->SetShowCurveEditor(!Sequencer->GetCurveEditorIsVisible());
				}
			}),
			FCanExecuteAction::CreateLambda([]
			{
				const TWeakPtr<ISequencer>& WeakSequencer = UE::AnimationEditMode::GetSequencer();
				const TSharedPtr<ISequencer>& Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
				return Sequencer != nullptr;
			}),
			FIsActionChecked::CreateLambda([]()
			{
				const TWeakPtr<ISequencer>& WeakSequencer = UE::AnimationEditMode::GetSequencer();
				if (const TSharedPtr<ISequencer>& Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr)
				{
					return Sequencer->GetCurveEditorIsVisible();
				}
				return false;
			})
		),
		NAME_None,
		LOCTEXT("CurveEditor", "Curve Editor"),
		LOCTEXT("CurveEditorTooltip", "Toggle the Curve Editor"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCurveEditor")),
		EUserInterfaceActionType::ToggleButton
	);

	//POSES
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, FControlRigEditModeToolkit::PoseTabName),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigEditModeToolkit::IsToolkitUIActive, FControlRigEditModeToolkit::PoseTabName)
			),
		NAME_None,
		LOCTEXT("Poses", "Poses"),
		LOCTEXT("PosesTooltip", "Show Poses"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.PoseTool")),
		EUserInterfaceActionType::ToggleButton
	);
	
	// Tweens
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, FControlRigEditModeToolkit::TweenOverlayName),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigEditModeToolkit::IsToolkitUIActive, FControlRigEditModeToolkit::TweenOverlayName)
		),		
		NAME_None,
		LOCTEXT("Tweens", "Tweens"),
		LOCTEXT("TweensTooltip", "Create Tweens.\n\nDefault bindings:\nU - While hidden, temporarily bring widget to your cursor\nU+LMB - While visible, indirectly move the slider\nAlt+U - Toggle visibility\nShift+U - Cycle function\nCtrl+U - Toggle Overshoot Mode"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.TweenTool")),
		EUserInterfaceActionType::ToggleButton
	);
	
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, FControlRigEditModeToolkit::ConstrainingTabName),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigEditModeToolkit::IsToolkitUIActive, FControlRigEditModeToolkit::ConstrainingTabName)
		),
		NAME_None,
		LOCTEXT("Constrain", "Constrain"),
		LOCTEXT("SnapperTooltip", "Tools for constraining, such as snapping, space picker, and constraints edition."),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.ConstraintTools")),
		EUserInterfaceActionType::ToggleButton
	);
	
	// Selection Sets
#if SELECTION_SETS_AS_TAB
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateRaw(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, FControlRigEditModeToolkit::SelectionSetsTabName),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FControlRigEditModeToolkit::IsToolkitUIActive, FControlRigEditModeToolkit::SelectionSetsTabName)
		),
		NAME_None,
		LOCTEXT("Sets", "Sets"),
		LOCTEXT("SelectionSetsTooltip", "Open Selection Sets"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.SelectionSet")),
		EUserInterfaceActionType::ToggleButton
	);
#else
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateRaw(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, FControlRigEditModeToolkit::SelectionSetsOverlayName),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FControlRigEditModeToolkit::IsToolkitUIActive, FControlRigEditModeToolkit::SelectionSetsOverlayName)
		),
		NAME_None,
		LOCTEXT("Sets", "Sets"),
		LOCTEXT("SelectionSetsTooltip", "Open Selection Sets"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.SelectionSet")),
		EUserInterfaceActionType::ToggleButton
	);
#endif
	
	/*  leaving this around for a bit to get animator feedback on the new menu system
	// Motion Trail
	ToolBarBuilder.AddToolBarButton(
		FExecuteAction::CreateRaw(this, &FControlRigEditModeToolkit::TryInvokeToolkitUI, FControlRigEditModeToolkit::MotionTrailTabName),
		NAME_None,
		LOCTEXT("MotionTrails", "Trails"),
		LOCTEXT("MotionTrailsTooltip", "Display motion trails for animated objects"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.EditableMotionTrails")),
		EUserInterfaceActionType::Button
	);
	*/
	
	// Anim Layer
	InToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, FControlRigEditModeToolkit::AnimLayerTabName),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigEditModeToolkit::IsToolkitUIActive, FControlRigEditModeToolkit::AnimLayerTabName)
		),
		NAME_None,
		LOCTEXT("Layers", "Layers"),
		LOCTEXT("AnimLayersTooltip", "Display animation layers"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.AnimLayers")),
		EUserInterfaceActionType::ToggleButton
	);
	//Pivot
	/** like motion trail leaving it in around for a bi
	ToolBarBuilder.AddToolBarButton(
		FUIAction(
		FExecuteAction::CreateSP(this, &SControlRigEditModeTools::ToggleEditPivotMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this] {
				if (TSharedPtr<IToolkit> Toolkit = OwningToolkit.Pin())
				{
					if (Toolkit.IsValid())
					{
						const FString ActiveToolName = Toolkit->GetToolkitHost()->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveToolName(EToolSide::Left);
						if (ActiveToolName == TEXT("SequencerPivotTool"))
						{
							return true;
						}
					}
				}
				return false;
			})
		),
		NAME_None,
		LOCTEXT("TempPivot", "Pivot"),
		LOCTEXT("TempPivotTooltip", "Create a temporary pivot to rotate the selected Control"),
		FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.TemporaryPivot")),
		EUserInterfaceActionType::ToggleButton
		);
		*/
}

void FControlRigEditModeToolkit::BindCommands()
{
	FControlRigEditModeCommands& Comands = FControlRigEditModeCommands::Get();
#if SELECTION_SETS_AS_TAB
	ToolkitCommands->MapAction(
		Comands.ToggleSelectionSetsWidget,
		FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, SelectionSetsTabName)
		);
#endif

	ToolkitCommands->MapAction(
		Comands.ToggleAnimLayersTab,
		FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, AnimLayerTabName)
		);

	ToolkitCommands->MapAction(
		Comands.TogglePoseLibraryTab,
		FExecuteAction::CreateSP(this, &FControlRigEditModeToolkit::TryToggleToolkitUI, PoseTabName)
		);
}

#undef LOCTEXT_NAMESPACE
