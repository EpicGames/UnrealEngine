// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "EditorModes.h"
#include "Toolkits/BaseToolkit.h"
#include "EditorModeManager.h"
#include "Constrain/ConstrainToolsManager.h"
#include "EditMode/SControlRigEditModeTools.h"
#include "EditMode/ControlRigEditMode.h"
#include "Sequencer/SelectionSets/SelectionSetsOverlayManager.h"

#include "Tween/ControlRigTweenModel.h"
#include "Tween/TweenOverlayManager.h"

//when true selection sets will show up as a tab, 0 will be an overlay

#ifndef SELECTION_SETS_AS_TAB
#define SELECTION_SETS_AS_TAB 1
#endif

class SAnimDetailsView;
class SControlRigOutliner;
namespace UE::ControlRigEditor 
{ 
	class SAnimDetailsView;
	class SControlRigTweenWidget; 
}

class FControlRigEditModeToolkit : public FModeToolkit
{
	using Super = FModeToolkit;
public:
	friend UE::ControlRigEditor::SControlRigTweenWidget;

	explicit FControlRigEditModeToolkit(FControlRigEditMode& InEditMode);
	virtual ~FControlRigEditModeToolkit() override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override { return FName("AnimationMode"); }
	virtual FText GetBaseToolkitName() const override { return NSLOCTEXT("AnimationModeToolkit", "DisplayName", "Animation"); }
	virtual class FEdMode* GetEditorMode() const override { return &EditMode; }
	virtual TSharedPtr<class SWidget> GetInlineContent() const override { return ModeTools; }
	virtual bool ProcessCommandBindings(const FKeyEvent& InKeyEvent) const override
	{
		return EditMode.GetCommandBindings() && EditMode.GetCommandBindings()->ProcessCommandBindings(InKeyEvent);
	}
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost) override;

	/** Mode Toolbar Palettes **/
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const override;
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder) override;

	/** Modes Panel Header Information **/
	virtual FText GetActiveToolDisplayName() const override;
	virtual FText GetActiveToolMessage() const override;

	/** If the named UI is shown, hide it. If it's hidden, show it. */
	void TryToggleToolkitUI(const FName InName);
	/** Tries to spawn the named UI. If already spawned, and it's a tab, highlight it (little flash). */
	void TryInvokeToolkitUI(const FName InName);
	/** Tries to hide the named UI. */
	void TryCloseToolkitUI(const FName InName) const;
	/** @return Whether the named UI is open */
	bool IsToolkitUIActive(const FName InName) const;

	/** Called when the global array of control rigs changed the user is working on has changed. */
	void OnControlsChanged(TConstArrayView<TWeakObjectPtr<UControlRig>> InControlRigs) const;

	/** @return The constraining UI tools, which is a window containing tabs for spaces, constraining, and snapping. */
	UE::ControlRigEditor::FConstrainToolsManager* GetConstrainToolsManager() const { return ConstrainToolsManager.Get(); }

public:
	
	static const FName PoseTabName;
	static const FName MotionTrailTabName;
	static const FName AnimLayerTabName;
	static const FName TweenOverlayName;
	static const FName ConstrainingTabName;
#if SELECTION_SETS_AS_TAB
	static const FName SelectionSetsTabName;
#else
	static const FName SelectionSetsOverlayName;
#endif
	static const FName SnapperTabName;
	static const FName DetailsTabName;
	static const FName OutlinerTabName;

	static TSharedPtr<UE::ControlRigEditor::SAnimDetailsView> Details;
	static TSharedPtr<SControlRigOutliner> Outliner;

protected:

	/* FModeToolkit Interface */
	virtual void RequestModeUITabs() override;
	virtual void InvokeUI() override;
	virtual void ShutdownUI() override;

	/** Removes all UI, unregisters tab spawners, and saves the state so it can be restored next time. */
	void UnregisterAndRemoveFloatingTabs();
	/** Saves the layout state into config so it can be restored next time the user restarts the editor. */
	void SaveLayoutState() const;

private:
	
	/** The edit mode we are bound to */
	FControlRigEditMode& EditMode;

	/** View model for the controls rigs that the user is working on. Used by UI. */
	const TSharedRef<UE::ControlRigEditor::FRigSelectionViewModel> SelectionViewModel;
	
	/** Manages the tween viewport UI overlay. */
	TUniquePtr<UE::ControlRigEditor::FTweenOverlayManager> TweenOverlayManager;
	/** Manages the constraining UI tools, which is a window containing tabs for spaces, constraining, and snapping. */
	TUniquePtr<UE::ControlRigEditor::FConstrainToolsManager> ConstrainToolsManager;
#if SELECTION_SETS_AS_TAB == 0
	TUniquePtr<UE::AIE::FSelectionSetsOverlayManager> SelectionSetsOverlayManager;
#endif

	/** The tools widget */
	TSharedPtr<SControlRigEditModeTools> ModeTools;

	/** Adds the toolbar items to the toolbar builder. */
	void BuildToolBar(FToolBarBuilder& ToolBarBuilder);

	void BindCommands();
};
