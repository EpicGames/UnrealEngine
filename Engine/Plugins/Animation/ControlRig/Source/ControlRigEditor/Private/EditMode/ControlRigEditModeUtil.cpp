// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/ControlRigEditModeUtil.h"

#include "ControlRig.h"
#include "EditorViewportClient.h"
#include "EngineUtils.h"
#include "LevelEditorViewport.h"
#include "Misc/ConfigCacheIni.h"
#include "Rigs/RigHierarchy.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"
#include "UnrealClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigEditModeUtil)

namespace UE::ControlRigEditMode
{

FExplicitRotationInteraction::FExplicitRotationInteraction(const FControlRigInteractionTransformContext& InContext,
	UControlRig* InControlRig,
	URigHierarchy* InHierarchy,
	FRigControlElement* InControlElement,
	const FTransform& InComponentWorldTransform)
		: TransformContext(InContext)
		, ControlRig(InControlRig)
		, Hierarchy(InHierarchy)
		, ControlElement(InControlElement)
		, ComponentWorldTransform(InComponentWorldTransform)
{}
	
bool FExplicitRotationInteraction::IsValid() const
{
	const bool bIsExplicitRotation = TransformContext.bRotation && TransformContext.Space == EControlRigInteractionTransformSpace::Explicit && !TransformContext.Rot.IsZero();
	if (!bIsExplicitRotation)
	{
		return false;
	}

	if (!ControlRig || !Hierarchy)
	{
		return false;
	}

	if (ControlRig->IsAdditive() || !Hierarchy->UsesPreferredEulerAngles())	
	{
		return false;
	}

	if (!ControlElement)
	{
		return false;
	}
	
	const ERigControlType ControlType = ControlElement->Settings.ControlType;
	return ControlType == ERigControlType::Rotator || ControlType == ERigControlType::EulerTransform;
}

void FExplicitRotationInteraction::Apply(
	const FTransform& InGlobalTransform,
	const FRigControlModifiedContext& InContext,
	const bool bPrintPython,
	const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints) const
{
	static constexpr bool bNotify = true, bUndo = true;

	const FName ControlName = ControlElement->GetKey().Name;
	const ERigControlType ControlType = ControlElement->Settings.ControlType;
	
	const FVector DeltaEulerAngle(TransformContext.Rot.Roll, TransformContext.Rot.Pitch, TransformContext.Rot.Yaw);
	FVector NewEulerAngle = Hierarchy->GetControlSpecifiedEulerAngle(ControlElement);
	NewEulerAngle += DeltaEulerAngle;
	
	if (ControlType == ERigControlType::Rotator)
	{
		const FQuat Quat = Hierarchy->GetControlQuaternion(ControlElement, NewEulerAngle);
		const FRotator Rotator(Quat);
			
		Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, NewEulerAngle);
		ControlRig->SetControlValue<FRotator>(ControlName, Rotator, bNotify, InContext, bUndo, bPrintPython);
	}
	else if (ControlType == ERigControlType::EulerTransform)
	{
		FRigControlModifiedContext Context = InContext;
		
		const FQuat Quat = Hierarchy->GetControlQuaternion(ControlElement, NewEulerAngle);

		FEulerTransform EulerTransform;
		if (InConstraints.IsEmpty())
		{
			FRigControlValue NewValue = ControlRig->GetControlValueFromGlobalTransform(ControlName, InGlobalTransform, ERigTransformType::CurrentGlobal);
			EulerTransform = NewValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
			EulerTransform.Rotation = FRotator(Quat);
		}
		else
		{
			Context.bConstraintUpdate = false;
			
			const FTransform WorldTransform = InGlobalTransform * ComponentWorldTransform;
			FTransform LocalTransform = ControlRig->GetControlLocalTransform(ControlName);

			const TOptional<FTransform> RelativeTransform =
				UE::TransformConstraintUtil::GetConstraintsRelativeTransform(InConstraints, LocalTransform, WorldTransform);
			if (ensure(RelativeTransform))
			{
				LocalTransform = *RelativeTransform; 
			}

			EulerTransform = FEulerTransform(LocalTransform);
			EulerTransform.Rotation = FRotator(Quat);
		}
		
		Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, NewEulerAngle);
		ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlName, EulerTransform, bNotify, Context, bUndo, bPrintPython);
		Hierarchy->SetControlSpecifiedEulerAngle(ControlElement, NewEulerAngle);
	}
}

FSelectionHelper::FSelectionHelper(
	FEditorViewportClient* InViewportClient,
	const TMap<TWeakObjectPtr<UControlRig>, TArray<TObjectPtr<AControlRigShapeActor>>>& InControlRigShapeActors,
	TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>>& OutElements)
	: ViewportClient(InViewportClient)
	, ControlRigShapeActors(InControlRigShapeActors)
	, Elements(OutElements)
	, LevelEditorViewportSettings(GetDefault<ULevelEditorViewportSettings>())
{
	if (ensure(InViewportClient))
	{
		if (InViewportClient->IsLevelEditorClient())
		{
			FLevelEditorViewportClient* LevelViewportClient = static_cast<FLevelEditorViewportClient*>(InViewportClient);
			HiddenLayers = LevelViewportClient->ViewHiddenLayers;
		}
	}
}

bool FSelectionHelper::IsValid() const
{
	return ViewportClient && ViewportClient->Viewport;
}
	
void FSelectionHelper::GetFromFrustum(const FConvexVolume& InFrustum) const
{
	if (!IsValid())
	{
		return;
	}

	// NOTE: occlusion based selection is a level editor property but should probably be by viewport client.
	bool bTransparentBoxSelection = ViewportClient->IsLevelEditorClient() ? LevelEditorViewportSettings->bTransparentBoxSelection : true;
	if (!bTransparentBoxSelection)
	{
		if (const TOptional<FIntRect> Rect = RectangleFromFrustum(InFrustum))
		{
			GetNonOccludedElements(*Rect);
		}
		else
		{
			bTransparentBoxSelection = true;
		}
	}

	if (bTransparentBoxSelection)
	{
		for (const auto& [WeakControlRig, ShapeActors]: ControlRigShapeActors)
		{
			if (TStrongObjectPtr<UControlRig> ControlRig = WeakControlRig.Pin())
			{
				if (ControlRig && ControlRig->GetControlsVisible())
				{
					for (AControlRigShapeActor* ShapeActor : ShapeActors)
					{
						const bool bTreatShape = ShapeActor && ShapeActor->IsSelectable() && !ShapeActor->IsTemporarilyHiddenInEditor();
						if (bTreatShape)
						{
							for (UActorComponent* Component : ShapeActor->GetComponents())
							{
								UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component);
								if (PrimitiveComponent && PrimitiveComponent->IsRegistered() && PrimitiveComponent->IsVisibleInEditor())
								{
									if (PrimitiveComponent->IsShown(ViewportClient->EngineShowFlags) && PrimitiveComponent->ComponentIsTouchingSelectionFrustum(InFrustum, false /*only bsp*/, false/*encompass entire*/))
									{
										TArray<FRigElementKey>& Controls = Elements.FindOrAdd(ControlRig.Get());
										Controls.Add(ShapeActor->GetElementKey());
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

namespace Private
{

uint32 GetHitProxySize()
{
	// see FViewport constructor and FViewport::GetHitProxy(int32 X,int32 Y)
	static uint32 HitProxySize = 0;
	if (HitProxySize == 0)
	{
		HitProxySize = 5;
		if (GIsEditor && GConfig) 
		{
			int32 HitProxySizeFromConfig = 5;
			if (GConfig->GetInt( TEXT("UnrealEd.HitProxy"), TEXT("HitProxySize"), HitProxySizeFromConfig, GEditorIni ))
			{
				HitProxySize = static_cast<uint32>(FMath::Clamp(HitProxySizeFromConfig, 1, MAX_HITPROXYSIZE));
			}
		}
	}
	return HitProxySize;
}

}
	
TOptional<FIntRect> FSelectionHelper::RectangleFromFrustum(const FConvexVolume& InFrustum) const
{
	static TOptional<FIntRect> InvalidBox;

	if (!IsValid())
	{
		return InvalidBox;
	}
	
	const FConvexVolume::FPlaneArray& Planes = InFrustum.Planes;
	const int32 NumPlanes = Planes.Num();
	if (NumPlanes < 4)
	{
		return InvalidBox;
	}

	FSceneInterface* Scene = ViewportClient->GetScene();
	if (!Scene)
	{
		return InvalidBox;
	}
	
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(ViewportClient->Viewport, Scene, ViewportClient->EngineShowFlags ));
	const FSceneView* SceneView = ViewportClient->CalcSceneView(&ViewFamily);
	if (!SceneView)
	{
		return InvalidBox;
	}

	FPlane NearPlane;
	if (!SceneView->ViewMatrices.GetWorldToClip().GetFrustumNearPlane(NearPlane))
	{
		return InvalidBox;
	}

	const FVector& ViewLocation = ViewportClient->GetViewLocation();

	// compute the intersections of side planes with the near plane
	TArray<FVector> Intersections;
	Intersections.Reserve(4);
	constexpr double Threshold = FMath::Square(0.001); // cf. IntersectPlanes2 for threshold
	for (uint8 Index = 0; Index < 4; Index++)
	{
		FVector Direction = FVector::CrossProduct(Planes[Index], Planes[(Index+1)%4]);
		const double DD = Direction.SizeSquared();
		if (DD >= Threshold)
		{
			// planes intersect, compute the intersection with the near plane
			Direction.Normalize();
			Intersections.Add(FMath::RayPlaneIntersection(ViewLocation, Direction, NearPlane));
		}
	}

	if (Intersections.Num() == 4)
	{
		// compute the screen space & pixel projections of those intersections
		FVector2D ScreenPos[4];
		bool bIsProjectionValid = true;
		for (uint8 Index = 0; Index < 4; Index++)
		{
			bIsProjectionValid &= SceneView->ScreenToPixel(SceneView->WorldToScreen(Intersections[Index]), ScreenPos[Index]);
		}

		if (bIsProjectionValid)
		{
			FVector2D& TopLeft = ScreenPos[3], BottomRight = ScreenPos[1];

			const int32 HitProxySize = Private::GetHitProxySize();
			const double dHitProxySize(HitProxySize);

			if (BottomRight.X - TopLeft.X < 2.0 * dHitProxySize)
			{
				const double MidX = 0.5 * (BottomRight.X + TopLeft.X);
				TopLeft.X = MidX - dHitProxySize;
				BottomRight.X = MidX + dHitProxySize;
			}

			if (BottomRight.Y - TopLeft.Y < 2.0 * dHitProxySize)
			{
				const double MidY = 0.5 * (BottomRight.Y + TopLeft.Y);
				TopLeft.Y = MidY - dHitProxySize;
				BottomRight.Y = MidY + dHitProxySize;
			}

			const FIntPoint ViewportSize = ViewportClient->Viewport->GetSizeXY();
			
			const int32 MinX = FMath::Clamp(FMath::FloorToInt32(TopLeft.X), 0, ViewportSize.X-1);
			const int32 MinY = FMath::Clamp(FMath::FloorToInt32(TopLeft.Y), 0, ViewportSize.Y-1);

			const int32 MinXClamp = (MinX + HitProxySize) > (ViewportSize.X-1) ? MinX + 1 : MinX + HitProxySize;
			const int32 MinYClamp = (MinY + HitProxySize) > (ViewportSize.Y-1) ? MinY + 1 : MinY + HitProxySize;
			
			const int32 MaxX = FMath::Clamp(FMath::CeilToInt32(BottomRight.X), MinXClamp, ViewportSize.X);
			const int32 MaxY = FMath::Clamp(FMath::CeilToInt32(BottomRight.Y), MinYClamp, ViewportSize.Y);

			const FIntRect Rect(MinX, MinY, MaxX, MaxY);
			return Rect;
		}
	}

	return InvalidBox;
}

void FSelectionHelper::GetNonOccludedElements(const FIntRect& InRect) const
{
	if (!IsValid())
	{
		return;
	}

	// extend that predicate to filter more hit proxies
	ViewportClient->Viewport->EnumerateHitProxiesInRect(InRect, [this](HHitProxy* HitProxy)
	{
		const HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy);
		if (const AControlRigShapeActor* ShapeActor = ActorHitProxy ? Cast<AControlRigShapeActor>(ActorHitProxy->Actor) : nullptr)
		{
			if (ShapeActor->IsSelectable() && !ShapeActor->IsTemporarilyHiddenInEditor())
			{
				if (UControlRig* ControlRig = ShapeActor->ControlRig.Get())
				{
					TArray<FRigElementKey>& Controls = Elements.FindOrAdd(ControlRig);
					Controls.AddUnique(ShapeActor->GetElementKey());
				}
			}
		}
		return true;
	});
}
	
}
