// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigUtilityActions.h"

#include "ActorActionUtility.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationEditorPreviewActor.h"
#include "BlutilityMenuExtensions.h"
#include "Components/SceneComponent.h"
#include "ControlRig.h"
#include "ControlRigEditModeSettings.h"
#include "ControlRigGizmoActor.h"
#include "EditorUtilityAssetPrototype.h"
#include "IControlRigObjectBinding.h"
#include "GameFramework/Actor.h"
#include "ModularRig.h"
#include "Algo/AnyOf.h"
#include "Engine/Level.h"
#include "HAL/IConsoleManager.h"

namespace UE::ControlRigBlutility
{

bool SupportUtilityActions()
{
	const UControlRigEditModeSettings* Settings = GetDefault<UControlRigEditModeSettings>();
	return Settings ? Settings->bSupportUtilityActions : false;
}

bool IsControlSelectable(const URigHierarchy* InHierarchy, const FRigElementKey& InSelectedKey)
{
	constexpr uint32 CtrlMask = static_cast<uint32>(ERigElementType::Control);
	
	if (!FRigElementTypeHelper::DoesHave(CtrlMask, InSelectedKey.Type))
	{
		return false;
	}

	constexpr bool bDoNotRespectVisibility = false;
	const FRigControlElement* ControlElement = InHierarchy ? InHierarchy->Find<FRigControlElement>(InSelectedKey) : nullptr;
	return ControlElement ? ControlElement->Settings.IsSelectable(bDoNotRespectVisibility) : false;
}
	
struct FSelectedRigs
{
	void TryAdding(UControlRig* ControlRig, const USceneComponent* Bound)
	{
		if (ControlRig)
		{
			if (Bound)
			{
				if (Bound->IsSelectedInEditor())
				{
					return AddSelectedRig(ControlRig);
				}

				if (AActor* Actor = Bound->GetTypedOuter<AActor>(); Actor && Actor->IsSelectedInEditor())
				{
					return AddSelectedRig(ControlRig);
				}
			}

			if (const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
			{
				const bool AnySelectedControl = Hierarchy->HasAnythingSelectedByPredicate([Hierarchy](const FRigElementKey& InSelectedKey)
				{
					return IsControlSelectable(Hierarchy, InSelectedKey);
				});
				
				if (AnySelectedControl)
				{
					return AddSelectedRig(ControlRig);
				}
			}
		}
	}
	
	void AddSelectedRig(UControlRig* ControlRig)
	{
		SelectedRigs.Add(ControlRig);

		// if ControlRig is modular then also store the selected modules
		if (UModularRig* ModularRig = Cast<UModularRig>(ControlRig))
		{
			TSet<UControlRig*> SelectedModules;
			const URigHierarchy* Hierarchy = ModularRig->GetHierarchy();
			for (const FRigElementKey& Control: Hierarchy->GetSelectedKeys(ERigElementType::Control))
			{
				if (const FName ModuleName = Hierarchy->GetModuleFName(Control); ModuleName != NAME_None)
				{
					const FRigModuleInstance* Module = ModularRig->FindModule(ModuleName);
					if (UControlRig* ModuleRig = Module ? Module->GetRig() : nullptr)
					{
						SelectedModules.Add(ModuleRig);
					}
				}
			}
			if (!SelectedModules.IsEmpty())
			{
				SelectedModulesPerRig.Emplace(ControlRig, MoveTemp(SelectedModules));
			}
		}
	}

	TArray<UControlRig*> SelectedRigs;
	TMap<UControlRig*, TSet<UControlRig*>> SelectedModulesPerRig;
};

USceneComponent* GetBoundComponent(const UControlRig* ControlRig, const UWorld* InWorld)
{
	const TSharedPtr<IControlRigObjectBinding>& ObjectBinding = ControlRig ? ControlRig->GetObjectBinding() : nullptr;
	UObject* BoundObject = ObjectBinding ? ObjectBinding->GetBoundObject() : nullptr;
	if (!BoundObject)
	{
		return nullptr;
	}

	if (USceneComponent* BoundSceneComponent = Cast<USceneComponent>(BoundObject))
	{
		return BoundSceneComponent;
	}

	if (const USkeleton* BoundSkeleton = Cast<USkeleton>(BoundObject))
	{
		// Bound to a Skeleton means we are previewing an Animation Sequence
		if (InWorld && InWorld->PersistentLevel)
		{
			const TObjectPtr<AActor>* PreviewActor = InWorld->PersistentLevel->Actors.FindByPredicate([](const TObjectPtr<AActor>& Actor)
			{
				return Actor && Actor->GetClass() == AAnimationEditorPreviewActor::StaticClass();
			});

			if (PreviewActor)
			{
				if (UDebugSkelMeshComponent* DebugComponent = (*PreviewActor)->FindComponentByClass<UDebugSkelMeshComponent>())
				{
					return DebugComponent;
				}
			}
		}
	}

	return nullptr;
}

FControlRigUtilityActions FControlRigUtilityActions::GetControlRigUtilityActions(
	const UWorld* InWorld,
	const TArrayView<TWeakObjectPtr<UControlRig>>& InRigs)
{
	FControlRigUtilityActions Actions;
	if (!SupportUtilityActions())
	{
		return Actions;
	}

	FSelectedRigs Selection;
	for (const TWeakObjectPtr<UControlRig>& WeakControlRig: InRigs)
	{
		if (UControlRig* ControlRig = WeakControlRig.IsValid() ? WeakControlRig.Get() : nullptr)
		{
			const USceneComponent* Bound = GetBoundComponent(ControlRig, InWorld);
			Selection.TryAdding(ControlRig, Bound);
		}
	}
	
	const TArray<UControlRig*> SelectedRigs = MoveTemp(Selection.SelectedRigs);
	const TMap<UControlRig*, TSet<UControlRig*>> SelectedModulesPerRig = MoveTemp(Selection.SelectedModulesPerRig);
	if (SelectedRigs.IsEmpty())
	{
		// nothing selected
		return Actions;
	}
	
	const bool bOnlyGeneric = [&SelectedRigs]()
	{
		if (SelectedRigs.Num() > 1)
		{
			for (int32 Index = 1; Index < SelectedRigs.Num(); Index++)
			{
				if (SelectedRigs[Index] != SelectedRigs[Index-1])
				{
					return true;	
				}
			}
		}
		return false;
	}();

	auto ProcessRigAction = [&Actions, &SelectedRigs, &SelectedModulesPerRig, bOnlyGeneric](const TSharedRef<FAssetActionUtilityPrototype>& ActionUtilityPrototype)
	{
		const TArray<TSoftClassPtr<UObject>> SupportedClasses = [ActionUtilityPrototype]()
		{
			if (ActionUtilityPrototype->IsLatestVersion() && !ActionUtilityPrototype->GetCallableFunctions().IsEmpty())
			{
				return ActionUtilityPrototype->GetSupportedClasses().FilterByPredicate([](const TSoftClassPtr<UObject>& SupportedClass)
				{
					const UClass* SupportedClassPtr = SupportedClass.Get();
					return SupportedClassPtr && SupportedClassPtr->IsChildOf(UControlRig::StaticClass());
				});
			}
			return TArray<TSoftClassPtr<UObject>>();
		}();

		if (SupportedClasses.IsEmpty())
		{
			return;
		}
		
		for (UControlRig* SelectedRig : SelectedRigs)
		{
			const bool bIsRigSupported = Algo::AnyOf(SupportedClasses, [SelectedRig, bOnlyGeneric, &SelectedModulesPerRig](const TSoftClassPtr<UObject>& SupportedClass)
			{
				return bOnlyGeneric ? SupportedClass == UControlRig::StaticClass() : SelectedRig->IsA(SupportedClass.Get());
			});

			if (bIsRigSupported)
			{
				const int32 Index = Actions.SupportedRigs.AddUnique(SelectedRig);
				Actions.UtilityAndRigIndices.FindOrAdd(ActionUtilityPrototype).Add(Index);
			}
			else if (!bOnlyGeneric && SelectedRig->IsModularRig())
			{
				if (UModularRig* ModularRig = Cast<UModularRig>(SelectedRig))
				{
					for (const TSoftClassPtr<UObject>& SupportedClass: SupportedClasses)
					{
						if (const TSet<UControlRig*>* SelectedModules = SelectedModulesPerRig.Find(ModularRig))
						{
							for (UControlRig* SelectedModuleRig: *SelectedModules)
							{
								if (SelectedModuleRig->IsA(SupportedClass.Get()))
								{
									const int32 Index = Actions.SupportedRigs.AddUnique(SelectedModuleRig);
									Actions.UtilityAndRigIndices.FindOrAdd(ActionUtilityPrototype).Add(Index);
								}
							}
						}
					}
				}
			}
		}
	};

	TArray<FAssetData> UtilAssets;
	FBlutilityMenuExtensions::GetBlutilityClasses(UtilAssets, UActorActionUtility::StaticClass()->GetClassPathName());
	for (const FAssetData& UtilAsset : UtilAssets)
	{
		if (const UClass* ParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(UtilAsset))
		{
			if (ParentClass->IsChildOf(UActorActionUtility::StaticClass()))
			{
				ProcessRigAction(MakeShared<FAssetActionUtilityPrototype>(UtilAsset));
			}
		}
	}
	
	return Actions;
}
}
