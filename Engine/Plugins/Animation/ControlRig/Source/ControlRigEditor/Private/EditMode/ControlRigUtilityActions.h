// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AControlRigShapeActor;
class UControlRig;
class FAssetActionUtilityPrototype;

namespace UE::ControlRigBlutility
{

/** Returns true if control rig utility actions are supported. */
bool SupportUtilityActions();
	
/** FControlRigEditorUtilityActions enables the storage and application of utility functions for control rigs. */
struct FControlRigUtilityActions
{
	/** Returns the actions/functions and rigs to which they can be applied. */
	static FControlRigUtilityActions GetControlRigUtilityActions(const UWorld* InWorld, const TArrayView<TWeakObjectPtr<UControlRig>>& InRigs);
	
	// Supported control rig objects
	TArray<UObject*> SupportedRigs;
	// actions and rig indices (in SupportedRigs) that can be applied
	TMap<TSharedRef<FAssetActionUtilityPrototype>, TSet<int32>> UtilityAndRigIndices;
};
	
}

