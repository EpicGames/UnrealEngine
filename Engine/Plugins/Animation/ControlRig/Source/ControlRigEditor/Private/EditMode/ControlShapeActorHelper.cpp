// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlShapeActorHelper.h"

#include "ControlRig.h"
#include "ControlRigEditModeSettings.h"
#include "ControlRigGizmoActor.h"
#include "Engine/World.h"
#include "ModularRig.h"
#include "Rigs/RigHierarchyElements.h"


namespace ControlRigEditMode::Shapes
{

struct FControls
{
	TSet<FString> ControlNames;
};
static TMap <TWeakObjectPtr<UControlRig>, FControls> HiddenControls;


void ClearOutHidden(UControlRig* InControlRig)
{
	if (InControlRig == nullptr)
	{
		HiddenControls.Reset();
	}
	else
	{
		HiddenControls.Remove(InControlRig);
	}
}

void ShowControlRigControls(UControlRig* InControlRig, const TSet<FString>& InNames, bool bVal)
{
	if (InControlRig)
	{
		FControls& Names = HiddenControls.FindOrAdd(InControlRig);
		if (bVal)
		{
			for (const FString& Name : InNames)
			{
				Names.ControlNames.Remove(Name);
			}
		}
		else //hide so we add them
		{
			for (const FString& Name : InNames)
			{
				Names.ControlNames.Add(Name);
			}
		}
	}
}

bool IsSupportedControlType(const ERigControlType& ControlType)
{
	switch (ControlType)
	{
	case ERigControlType::Float:
	case ERigControlType::ScaleFloat:
	case ERigControlType::Integer:
	case ERigControlType::Vector2D:
	case ERigControlType::Position:
	case ERigControlType::Scale:
	case ERigControlType::Rotator:
	case ERigControlType::Transform:
	case ERigControlType::TransformNoScale:
	case ERigControlType::EulerTransform:
		{
			return true;
		}
	default:
		{
			break;
		}
	}

	return false;
}

bool IsModeSupported(const ERigControlType& InControlType, const UE::Widget::EWidgetMode& InMode)
{
	if (IsSupportedControlType(InControlType))
	{
		switch (InMode)
		{
			case UE::Widget::WM_None:
				return true;
			case UE::Widget::WM_Rotate:
			{
				switch (InControlType)
				{
					case ERigControlType::Rotator:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						return true;
					}
					default:
					{
						break;
					}
				}
				break;
			}
			case UE::Widget::WM_Translate:
			{
				switch (InControlType)
				{
					case ERigControlType::Float:
					case ERigControlType::Integer:
					case ERigControlType::Vector2D:
					case ERigControlType::Position:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						return true;
					}
					default:
					{
						break;
					}
				}
				break;
			}
			case UE::Widget::WM_Scale:
			{
				switch (InControlType)
				{
					case ERigControlType::Scale:
					case ERigControlType::ScaleFloat:
					case ERigControlType::Transform:
					case ERigControlType::EulerTransform:
					{
						return true;
					}
					default:
					{
						break;
					}
				}
				break;
			}
			case UE::Widget::WM_TranslateRotateZ:
			{
				switch (InControlType)
				{
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						return true;
					}
					default:
					{
						break;
					}
				}
				break;
			}
		}
	}

	return false;
}
	
void GetControlsEligibleForShapes(const UControlRig* InControlRig, TArray<FRigControlElement*>& OutControls)
{
	OutControls.Reset();

	const URigHierarchy* Hierarchy = InControlRig ? InControlRig->GetHierarchy() : nullptr;
	if (!Hierarchy)
	{
		return;
	}

	OutControls = Hierarchy->GetFilteredElements<FRigControlElement>([](const FRigControlElement* ControlElement)
	{
		const FRigControlSettings& ControlSettings = ControlElement->Settings;
		return ControlSettings.SupportsShape() && IsSupportedControlType(ControlSettings.ControlType);
	});
}
	
void DestroyShapesActorsFromWorld(const TArray<TObjectPtr<AControlRigShapeActor>>& InShapeActorsToDestroy)
{
	// NOTE: should UWorld::EditorDestroyActor really modify the level when removing the shapes?
	// kept for legacy but I guess this should be set to false
	static constexpr bool bShouldModifyLevel = true;

	for (const TObjectPtr<AControlRigShapeActor>& ShapeActorPtr: InShapeActorsToDestroy)
	{
		if (AControlRigShapeActor* ShapeActor = ShapeActorPtr.Get())
		{
			if (UWorld* World = ShapeActor->GetWorld())
			{
				if (ShapeActor->GetAttachParentActor())
				{
					ShapeActor->DetachFromActor(FDetachmentTransformRules::KeepRelativeTransform);
				}
				World->EditorDestroyActor(ShapeActor,bShouldModifyLevel);
			}			
		}
	}
}

FShapeUpdateParams::FShapeUpdateParams(
	const UControlRig* InControlRig, 
	const FTransform& InComponentTransform, 
	const bool InSkeletalMeshVisible, 
	const bool IsInLevelEditor, 
	const bool bInHideControlShapes)
	: ControlRig(InControlRig)
	, Hierarchy(InControlRig->GetHierarchy())
	, Settings(GetDefault<UControlRigEditModeSettings>())
	, ComponentTransform(InComponentTransform)
	, bIsSkeletalMeshVisible(InSkeletalMeshVisible)
	, bIsInLevelEditor(IsInLevelEditor)
	, bHideControlShapes(bInHideControlShapes)
{
	if (IsValid())
	{
		bControlsHiddenInViewport = bHideControlShapes || !ControlRig->GetControlsVisible() || !bIsSkeletalMeshVisible;
	}
}

bool FShapeUpdateParams::IsValid() const
{
	return ControlRig && Hierarchy && Settings;
}

void UpdateControlShape(AControlRigShapeActor* InShapeActor, FRigControlElement* InControlElement, const FShapeUpdateParams& InUpdateParams)
{
	if (!InShapeActor || !InControlElement || !InUpdateParams.IsValid())
	{
		return;
	}

	if (InUpdateParams.bIsGameView)
	{
		InShapeActor->SetIsTemporarilyHiddenInEditor(true);
		return;
	}
	
	// update transform
	const FTransform Transform = InUpdateParams.Hierarchy->GetTransform(InControlElement, ERigTransformType::CurrentGlobal);
	InShapeActor->SetActorTransform(Transform * InUpdateParams.ComponentTransform);

	const FRigControlSettings& ControlSettings = InControlElement->Settings;

	// update visibility & color
	bool bIsVisible = ControlSettings.IsVisible();
	if (const UModularRig* ModularRig = Cast<UModularRig>(InUpdateParams.ControlRig))
	{
		const FString ModuleName = InUpdateParams.Hierarchy->GetModuleName(InControlElement->GetKey());
		if (const FRigModuleInstance* Module = ModularRig->FindModule(*ModuleName))
		{
			bIsVisible &= Module->GetRig()->GetControlsVisible();
		}
	}
	bool bRespectVisibilityForSelection = !InUpdateParams.bIsInLevelEditor;

	const bool bControlsHiddenInViewport =
		InUpdateParams.bHideControlShapes ||
		!InUpdateParams.ControlRig->GetControlsVisible() ||
		!InUpdateParams.bIsSkeletalMeshVisible;
	
	if (!bControlsHiddenInViewport)
	{
		if (ControlSettings.AnimationType == ERigControlAnimationType::ProxyControl)
		{					
			bRespectVisibilityForSelection = false;
			if (InUpdateParams.Settings->bShowAllProxyControls)
			{
				bIsVisible = true;
			}
		}
	}
	//hidden possibly
	if (bIsVisible)
	{
		if (FControls* HiddenControlNames = HiddenControls.Find(InShapeActor->ControlRig))
		{
			bIsVisible = HiddenControlNames->ControlNames.Contains(InControlElement->GetName()) == false;
		}
	}

	InShapeActor->SetIsTemporarilyHiddenInEditor(!bIsVisible || InUpdateParams.bControlsHiddenInViewport);

	// update color
	InShapeActor->SetShapeColor(InShapeActor->OverrideColor.A < SMALL_NUMBER ?
		ControlSettings.ShapeColor : InShapeActor->OverrideColor);
	
	// update selectability
	InShapeActor->SetSelectable( ControlSettings.IsSelectable(bRespectVisibilityForSelection) );
}
	
}
