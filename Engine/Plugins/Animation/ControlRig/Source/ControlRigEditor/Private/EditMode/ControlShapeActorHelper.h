// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "UObject/ObjectPtr.h"
#include "Containers/Array.h"
#include "UnrealWidgetFwd.h"

enum class ERigControlType : uint8;
class UControlRigEditModeSettings;
class URigHierarchy;
class AControlRigShapeActor;
struct FRigControlElement;
class UControlRig;

namespace ControlRigEditMode::Shapes
{
	bool IsSupportedControlType(const ERigControlType& ControlType);

	// Returns true if the control type supports being modified using this widget mode
	bool IsModeSupported(const ERigControlType& InControlType, const UE::Widget::EWidgetMode& InMode);

	// Returns the list of controls for which a shape is expected
	void GetControlsEligibleForShapes(const UControlRig* InControlRig, TArray<FRigControlElement*>& OutControls);

	// Destroys shape actors and removes them from their UWorld
	void DestroyShapesActorsFromWorld(const TArray<TObjectPtr<AControlRigShapeActor>>& InShapeActorsToDestroy);

	// Parameters used to update shape actors (transform, visibility, etc.)
	struct FShapeUpdateParams
	{
		FShapeUpdateParams(
			const UControlRig* InControlRig, 
			const FTransform& InComponentTransform, 
			const bool InSkeletalMeshVisible, 
			const bool IsInLevelEditor, 
			const bool bInHideControlShapes);

		const UControlRig* ControlRig = nullptr;
		const URigHierarchy* Hierarchy = nullptr;
		const UControlRigEditModeSettings* Settings = nullptr;
		const FTransform& ComponentTransform = FTransform::Identity;
		const bool bIsSkeletalMeshVisible = false;
		const bool bIsInLevelEditor = false;
		bool bControlsHiddenInViewport = false;
		bool bHideControlShapes = false;
		bool bIsGameView = false;
		bool IsValid() const;
	};

	// Updates shape actors transform, visibility, etc.
	void UpdateControlShape(AControlRigShapeActor* InShapeActor, FRigControlElement* InControlElement, const FShapeUpdateParams& InUpdateParams);

	//handle per control visibility by keeping list of hidden controls. We won't create these when requested
	//clear out the hidden ones on this control rig or all if InControlRig is null
	void ClearOutHidden(UControlRig* InControlRig);
	// user is responsible to make sure we recreate them at some other point after the map is set up
	void ShowControlRigControls(UControlRig* InControlRig, const TSet<FString>& Names, bool bVal);

}
