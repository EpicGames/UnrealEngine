// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigSelectionViewModel.h"

#include "ControlRig.h"

namespace UE::ControlRigEditor
{
FRigSelectionViewModel::~FRigSelectionViewModel()
{
	CleanupSubscriptions();
}

void FRigSelectionViewModel::SetControls(TConstArrayView<TWeakObjectPtr<UControlRig>> InControlRigs)
{
	CleanupSubscriptions();

	for (const TWeakObjectPtr<UControlRig>& WeakControlRig : InControlRigs)
	{
		if (UControlRig* ControlRig = WeakControlRig.Get())
		{
			RuntimeRigs.AddUnique(ControlRig);
			ControlRig->ControlSelected().AddRaw(this, &FRigSelectionViewModel::BroadcastOnRigElementSelected);
		}
	}

	OnControlsChangedDelegate.Broadcast();
}

void FRigSelectionViewModel::CleanupSubscriptions()
{
	for (const TWeakObjectPtr<UControlRig>& WeakControlRig : RuntimeRigs)
	{
		if (UControlRig* ControlRig = WeakControlRig.Get())
		{
			ControlRig->ControlSelected().RemoveAll(this);
		}
	}
		
	RuntimeRigs.Reset();
}

void FRigSelectionViewModel::BroadcastOnRigElementSelected(
	UControlRig* InSubject, FRigControlElement* InElement, bool bIsSelected
	)
{
	OnControlSelectedDelegate.Broadcast(InSubject, InElement, bIsSelected);
}
}
