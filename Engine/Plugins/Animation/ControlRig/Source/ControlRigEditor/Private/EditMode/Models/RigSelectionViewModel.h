// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRig.h"
#include "Delegates/Delegate.h"

namespace UE::ControlRigEditor
{
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnControlSelected, UControlRig* /*Subject*/, FRigControlElement* /*ControlElement*/, bool /*bSelected*/);
using FOnControlsChanged = FSimpleMulticastDelegate;
	
/**
 * Responsible for invoking an event when control rig selection changes.
 * The intention is for this to be a view model for UI.
 */
class FRigSelectionViewModel : public FNoncopyable
{
public:

	~FRigSelectionViewModel();

	/** Called when the global array of control rigs that the user is working on changes. */
	void SetControls(TConstArrayView<TWeakObjectPtr<UControlRig>> InControlRigs);

	/** The control rigs that the user is working on. */
	TConstArrayView<TWeakObjectPtr<UControlRig>> GetControlRigs() const { return RuntimeRigs; }
	
	/** Invoked when a control rig is selected. */
	FOnControlSelected& OnControlSelected() { return OnControlSelectedDelegate; }
	/** Invoked when the controls rigs change. */
	FOnControlsChanged& OnControlsChanged() { return OnControlsChangedDelegate; }

private:

	/** The control rigs that the user is working on. */
	TArray<TWeakObjectPtr<UControlRig>> RuntimeRigs;

	/** Invoked when a control rig is selected. */
	FOnControlSelected OnControlSelectedDelegate;

	/** Invoked when the controls rigs change. */
	FOnControlsChanged OnControlsChangedDelegate;

	/** Unbinds all subscriptions. */
	void CleanupSubscriptions();

	void BroadcastOnRigElementSelected(UControlRig* InSubject, FRigControlElement* InElement, bool bIsSelected);
};
}

