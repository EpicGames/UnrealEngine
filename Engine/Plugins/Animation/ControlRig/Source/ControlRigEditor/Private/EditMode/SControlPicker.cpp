// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SControlPicker.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "ScopedTransaction.h"
#include "EditMode/ControlRigEditMode.h"
#include "ControlRig.h"
#include "EditorModeManager.h"
#include "EditMode/SEditorUserWidgetHost.h"
#include "ControlRigBlueprintLegacy.h"
#include "RigVMBlueprintGeneratedClass.h"

#define LOCTEXT_NAMESPACE "ControlPicker"

void SControlPicker::Construct(const FArguments& InArgs, UWorld* InWorld)
{
	ChildSlot
	[
		SAssignNew(EditorUserWidgetHost, SEditorUserWidgetHost, InWorld)
		.Visibility(this, &SControlPicker::ShowWidgetHost)
	];
}

void SControlPicker::SetControlRig(UControlRig* InRig)
{
	if (InRig != RigPtr.Get())
	{
		RigPtr = InRig;
	}
}

UControlRig* SControlPicker::GetRig() const
{
	return RigPtr.Get();
}

EVisibility SControlPicker::ShowWidgetHost() const
{
	return (GetRig() != nullptr) ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
