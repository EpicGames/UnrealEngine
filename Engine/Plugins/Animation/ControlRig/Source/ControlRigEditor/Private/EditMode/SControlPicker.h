// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Delegates/Delegate.h"
#include "UObject/WeakObjectPtr.h"

class UControlRig;
class SControlPicker;
class SCanvas;
class SScaleBox;
class SEditorUserWidgetHost;

//////////////////////////////////////////////////////////////////////////

/** 2D visual picker for picking control manipulators within a rig  */
class SControlPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SControlPicker) {}

	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, UWorld* InWorld);

	/** Set the rig to display manipulators for */
	void SetControlRig(UControlRig* InRig);

	/** Set the manipulators that are currently selected */
	void SetSelectedManipulators(const TArray<FName>& Manipulators);

	/** Call when a button is clicked, will fire OnManipulatorsPicked */
	void SelectManipulator(FName ManipulatorName, bool bAddToSelection);
	/** Call when background clicked, will fired OnManipulatorsPicked */
	void ClearSelection();
	/** Select all manipulators */
	void SelectAll();

	/** Returns whether a particular manipulator is selected */
	bool IsManipulatorSelected(FName ManipulatorName);

	/** Returns the rig we are displaying controls for */
	UControlRig* GetRig() const;

	/** See if the limb/spine is in IK mode, or FK */
	bool IsControlIK(FName ControlName) const;

	/** Toggle kinematic mode for a limb/spine, by name */
	void ToggleControlKinematicMode(FName ControlName);

protected:

	EVisibility ShowWidgetHost() const;

	/** Widget host for UMG picker */
	TSharedPtr<SEditorUserWidgetHost> EditorUserWidgetHost;

	/** Rig we are showing controls for */
	TWeakObjectPtr<UControlRig> RigPtr;
};
