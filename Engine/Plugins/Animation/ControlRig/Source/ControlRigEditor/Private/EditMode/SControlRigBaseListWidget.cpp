// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditMode/SControlRigBaseListWidget.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Styling/AppStyle.h"

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"

#include "ScopedTransaction.h"

#include "ControlRig.h"
#include "UnrealEdGlobals.h"
#include "EditMode/ControlRigEditMode.h"
#include "Tools/ControlRigPose.h"
#include "EditorDirectories.h"
#include "EditorModeManager.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserDataModule.h"
#include "ContentBrowserDataSubsystem.h"
#include "Tools/ControlRigPoseProjectSettings.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "EditMode/ControlRigEditModeCommands.h"
#include "Tools/CreateControlAssetRigSettings.h"
#include "FileHelpers.h"
#include "Tools/ControlRigPoseMirrorSettings.h"
#include "ObjectTools.h"
#include "EditMode/SControlRigUpdatePose.h"
#include "EditMode/SControlRigRenamePoseControls.h"
#include "Dialogs/Dialogs.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "Misc/PackageName.h"
#include "AssetToolsModule.h"
#include "ControlRigEditModeToolkit.h"
#include "PropertyEditorModule.h"


#define LOCTEXT_NAMESPACE "ControlRigBaseListWidget"

FString SControlRigBaseListWidget::CurrentlySelectedInternalPath("");

enum class FControlRigAssetType {
	ControlRigPose,
	ControlRigAnimation,
	ControlRigSelectionSet

};

/////////////////////////////////////////////////////
// SControlRigPoseAnimSelectionToolbar

DECLARE_DELEGATE_OneParam(FCreateControlAssetDelegate,
FString);

struct FCreateControlAssetRigDialog
{
	static void GetControlAssetParams(FControlRigAssetType Type, FCreateControlAssetDelegate& Delegate);

};

class SCreateControlAssetRigDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SCreateControlAssetRigDialog) :
		_AssetType(FControlRigAssetType::ControlRigPose) {}

	SLATE_ARGUMENT(FControlRigAssetType, AssetType)

		SLATE_END_ARGS()

		FControlRigAssetType AssetType;

	~SCreateControlAssetRigDialog()
	{
	}

	void Construct(const FArguments& InArgs)
	{
		AssetType = InArgs._AssetType;

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Create Control Asset";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		ChildSlot
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
			.Text(LOCTEXT("CreateControlAssetRig", "Create Asset"))
			.OnClicked(this, &SCreateControlAssetRigDialog::OnCreateControlAssetRig)
			]

			];

		switch (AssetType)
		{
		case FControlRigAssetType::ControlRigPose:
		{
			UCreateControlPoseAssetRigSettings* AssetSettings = GetMutableDefault<UCreateControlPoseAssetRigSettings>();
			DetailView->SetObject(AssetSettings);
			break;
		}
		/*
		case FControlRigAssetType::ControlRigAnimation:
		{
			UCreateControlAnimationAssetRigSettings* AssetSettings = GetMutableDefault<UCreateControlAnimationAssetRigSettings>();
			DetailView->SetObject(AssetSettings);
			break;
		}
		case FControlRigAssetType::ControlRigSelectionSet:
		{
			UCreateControlSelectionSetAssetRigSettings* AssetSettings = GetMutableDefault<UCreateControlSelectionSetAssetRigSettings>();
			DetailView->SetObject(AssetSettings);
			break;
		}
		*/
		};

	}

	void SetDelegate(FCreateControlAssetDelegate& InDelegate)
	{
		Delegate = InDelegate;
	}

private:

	FReply OnCreateControlAssetRig()
	{
		FString AssetName("");
		switch (AssetType)
		{
		case FControlRigAssetType::ControlRigPose:
		{
			UCreateControlPoseAssetRigSettings* AssetSettings = GetMutableDefault<UCreateControlPoseAssetRigSettings>();
			AssetName = AssetSettings->AssetName;
			break;
		}
		/*
		case FControlRigAssetType::ControlRigAnimation:
		{
			UCreateControlAnimationAssetRigSettings* AssetSettings = GetMutableDefault<UCreateControlAnimationAssetRigSettings>();
			AssetName = AssetSettings->AssetName;
			break;
		}
		case FControlRigAssetType::ControlRigSelectionSet:
		{
			UCreateControlSelectionSetAssetRigSettings* AssetSettings = GetMutableDefault<UCreateControlSelectionSetAssetRigSettings>();
			AssetName = AssetSettings->AssetName;
			break;
		}
		*/
		};

		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (Delegate.IsBound())
		{
			Delegate.Execute(AssetName);
		}
		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return FReply::Handled();
	}
	TSharedPtr<IDetailsView> DetailView;
	FCreateControlAssetDelegate  Delegate;

};


void FCreateControlAssetRigDialog::GetControlAssetParams(FControlRigAssetType Type, FCreateControlAssetDelegate& InDelegate)
{
	FText TitleText;
	switch (Type)
	{
	case FControlRigAssetType::ControlRigPose:
		TitleText = LOCTEXT("CreateControlAssetRigPose", "Create Control Rig Pose");
		break;
	case FControlRigAssetType::ControlRigAnimation:
		TitleText = LOCTEXT("CreateControlAssetRigAnimation", "Create Control Rig Animation");
		break;
	case FControlRigAssetType::ControlRigSelectionSet:
		TitleText = LOCTEXT("CreateControlAssetRigSelectionSet", "Create Control Rig Selection Set");
		break;
	};


	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(400.0f, 200.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SCreateControlAssetRigDialog> DialogWidget = SNew(SCreateControlAssetRigDialog).AssetType(Type);
	DialogWidget->SetDelegate(InDelegate);
	Window->SetContent(DialogWidget);

	FSlateApplication::Get().AddWindow(Window);

}


/////////////////////////////////////////////////////
// SControlRigPoseAnimSelectionToolbar

class SControlRigPoseAnimSelectionToolbar : public SCompoundWidget
{

public:
	/** Default constructor. */
	SControlRigPoseAnimSelectionToolbar();

	/** Virtual destructor. */
	virtual ~SControlRigPoseAnimSelectionToolbar();

	SLATE_BEGIN_ARGS(SControlRigPoseAnimSelectionToolbar) {}
	SLATE_ARGUMENT(SControlRigBaseListWidget*, OwningControlRigWidget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void MakeControlRigAssetDialog(FControlRigAssetType Type, bool bSelectAll);
	bool CanExecuteMakeControlRigAsset();
	
	//void ToggleFilter(FControlRigAssetType Type);
	//bool IsOnToggleFilter(FControlRigAssetType Type) const;

	//It's parent so will always be there..
	SControlRigBaseListWidget* OwningControlRigWidget;
};


SControlRigPoseAnimSelectionToolbar::SControlRigPoseAnimSelectionToolbar() :OwningControlRigWidget(nullptr)
{
}

SControlRigPoseAnimSelectionToolbar::~SControlRigPoseAnimSelectionToolbar()
{
}


void SControlRigPoseAnimSelectionToolbar::Construct(const FArguments& InArgs)
{

	OwningControlRigWidget = InArgs._OwningControlRigWidget;

	FToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None, TSharedPtr<FExtender>(), true);
	ToolbarBuilder.SetLabelVisibility(EVisibility::Visible);
	FUIAction CreatePoseDialog(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::MakeControlRigAssetDialog, FControlRigAssetType::ControlRigPose,false),
		FCanExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::CanExecuteMakeControlRigAsset));
	/*
	FUIAction CreatePoseFromAllDialog(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::MakeControlRigAssetDialog, FControlRigAssetType::ControlRigPose,true));
	*/
	/*
	FUIAction CreateAnimationDialog(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::MakeControlRigAssetDialog, FControlRigAssetType::ControlRigAnimation));
	FUIAction CreateSelectionSetDialog(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::MakeControlRigAssetDialog, FControlRigAssetType::ControlRigSelectionSet));
	*/

	ToolbarBuilder.BeginSection("Create");
	{
		ToolbarBuilder.AddToolBarButton(CreatePoseDialog,
			NAME_None,
			LOCTEXT("CreatePose", "Create Pose"),
			LOCTEXT("CreatePoseTooltip", "Create pose asset from selection."),
			FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.CreatePose")),
			EUserInterfaceActionType::Button
		);
		/*
		ToolbarBuilder.AddToolBarButton(CreatePoseFromAllDialog,
			NAME_None,
			LOCTEXT("CreatePoseAll", "Create Pose From All Controls"),
			LOCTEXT("CreatePoseAllTooltip", "Create Pose Asset from all Controls not just selected"),
			FSlateIcon(),
			EUserInterfaceActionType::Button
			
		);
		*/
		/** For now just making pose assets
		ToolbarBuilder.AddToolBarButton(CreateAnimationDialog,
			NAME_None,
			LOCTEXT("CreateAnimation", "Create Animation"),
			LOCTEXT("CreateAnimationTooltip", "Create Control Rig Animation Asset"),
			FSlateIcon(),
			EUserInterfaceActionType::Button
		);
		ToolbarBuilder.AddToolBarButton(CreateSelectionSetDialog,
			NAME_None,
			LOCTEXT("CreateSelectionSet", "Create Selection Set"),
			LOCTEXT("CreateSelectionSetTooltip", "Create Selection Set Asset"),
			FSlateIcon(),
			EUserInterfaceActionType::Button
		);
		*/

	}
	/** TODO if we have multiple asset types we will put back the filters.
	ToolbarBuilder.EndSection();
	FUIAction ToggleFilterPose(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::ToggleFilter, FControlRigAssetType::ControlRigPose),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::IsOnToggleFilter, FControlRigAssetType::ControlRigPose));
	FUIAction ToggleFilterAnimation(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::ToggleFilter, FControlRigAssetType::ControlRigAnimation),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::IsOnToggleFilter, FControlRigAssetType::ControlRigAnimation));
	FUIAction ToggleFilterSelectionSet(
		FExecuteAction::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::ToggleFilter, FControlRigAssetType::ControlRigSelectionSet),
		FCanExecuteAction(),
		FIsActionChecked::CreateRaw(this, &SControlRigPoseAnimSelectionToolbar::IsOnToggleFilter, FControlRigAssetType::ControlRigSelectionSet));

	FToolBarBuilder RightToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None, TSharedPtr<FExtender>(), true);
	ToolbarBuilder.BeginSection("Filters");
	{
		RightToolbarBuilder.AddToolBarButton(ToggleFilterPose,
			NAME_None,
			LOCTEXT("FilterPose", "Filter Pose"),
			LOCTEXT("FilterPoseTooltip", "Toggle to Show or Hide Pose Assets"),
			FSlateIcon(),
			EUserInterfaceActionType::ToggleButton
		);
		RightToolbarBuilder.AddToolBarButton(ToggleFilterAnimation,
			NAME_None,
			LOCTEXT("FilterAnimation", "Filter Animation"),
			LOCTEXT("FilterAnimationTooltip", "Toggle to Show or Hide Animation Assets"),
			FSlateIcon(),
			EUserInterfaceActionType::ToggleButton
		);
		RightToolbarBuilder.AddToolBarButton(ToggleFilterSelectionSet,
			NAME_None,
			LOCTEXT("FilterSelectionSet", "Filter SelectionSet"),
			LOCTEXT("FilterSelectionaTooltip", "Toggle to Show or Hide Selction Set Assets"),
			FSlateIcon(),
			EUserInterfaceActionType::ToggleButton
		);

	}
	ToolbarBuilder.EndSection();
	*/
	// Create the tool bar!
	ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.FillWidth(1.0)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.Padding(0)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		[
			ToolbarBuilder.MakeWidget()
		]
		]
	/*
		+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.0f)
			[
				SNew(SBorder)
				.Padding(0)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			[
				RightToolbarBuilder.MakeWidget()
			]
			]*/
		];
}

void SControlRigPoseAnimSelectionToolbar::MakeControlRigAssetDialog(FControlRigAssetType Type, bool bSelectAll)
{
	FControlRigEditMode* ControlRigEditMode = OwningControlRigWidget->GetEditMode();
	if (!ControlRigEditMode)
	{
		return;
	}

	TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
	ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
	
	if (AllSelectedControls.Num() > 1)
	{
		return;
	}

	if (AllSelectedControls.Num() <= 0)
	{
		FText ConfirmDelete = LOCTEXT("ConfirmNoSelectedControls", "You are saving a Pose with no selected Controls - are you sure?");

		FSuppressableWarningDialog::FSetupInfo Info(ConfirmDelete, LOCTEXT("SavePose", "Save Pose"), "SavePose_Warning");
		Info.ConfirmText = LOCTEXT("SavePose_Yes", "Yes");
		Info.CancelText = LOCTEXT("SavePose_No", "No");

		FSuppressableWarningDialog SavePose(Info);
		if (SavePose.ShowModal() == FSuppressableWarningDialog::Cancel)
		{
			return;
		}
	}

	FCreateControlAssetDelegate GetNameCallback = FCreateControlAssetDelegate::CreateLambda([this, Type, bSelectAll](FString AssetName)
	{
		if (OwningControlRigWidget)
		{
			FString Path = OwningControlRigWidget->GetCurrentlySelectedPath();

			FControlRigEditMode* ControlRigEditMode =  OwningControlRigWidget->GetEditMode();
			if (ControlRigEditMode )
			{
				TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
				ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
				if (AllSelectedControls.Num() == 1)
				{
					TArray<UControlRig*> ControlRigs;
					AllSelectedControls.GenerateKeyArray(ControlRigs);
					UControlRig* ControlRig =ControlRigs[0];
					UObject* NewAsset = nullptr;
					switch (Type)
					{
					case FControlRigAssetType::ControlRigPose:
						NewAsset = FControlRigToolAsset::SaveAsset<UControlRigPoseAsset>(ControlRig, Path, AssetName, bSelectAll);

						break;
					case FControlRigAssetType::ControlRigAnimation:
						break;
					case FControlRigAssetType::ControlRigSelectionSet:
						break;
					default:
						break;
					};
					if (NewAsset)
					{
						FControlRigView::CaptureThumbnail(NewAsset);
						OwningControlRigWidget->SelectThisAsset(NewAsset);
					}
				}
			}
		}
	});

	FCreateControlAssetRigDialog::GetControlAssetParams(Type, GetNameCallback);
}

bool SControlRigPoseAnimSelectionToolbar::CanExecuteMakeControlRigAsset()
{
	FControlRigEditMode* ControlRigEditMode =  OwningControlRigWidget->GetEditMode();
	if (!ControlRigEditMode)
	{
		return false;
	}
	
	TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
	ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);

	if (AllSelectedControls.Num() != 1)
	{
		return false;
	}

	return true;
}
/*
void SControlRigPoseAnimSelectionToolbar::ToggleFilter(FControlRigAssetType Type)
{
	UControlRigPoseProjectSettings* PoseSettings = GetMutableDefault<UControlRigPoseProjectSettings>();
	if (PoseSettings)
	{
		switch (Type)
		{
		case FControlRigAssetType::ControlRigPose:
			PoseSettings->bFilterPoses = PoseSettings->bFilterPoses ? false : true;
			break;
		case FControlRigAssetType::ControlRigAnimation:
			PoseSettings->bFilterAnimations = PoseSettings->bFilterAnimations ? false : true;
			break;
		case FControlRigAssetType::ControlRigSelectionSet:
			PoseSettings->bFilterSelectionSets = PoseSettings->bFilterSelectionSets ? false : true;
			break;
		default:
			break;
		};
	}
	if (OwningControlRigWidget)
	{
		OwningControlRigWidget->FilterChanged();
	}
}

bool SControlRigPoseAnimSelectionToolbar::IsOnToggleFilter(FControlRigAssetType Type) const
{
	const UControlRigPoseProjectSettings* PoseSettings = GetDefault<UControlRigPoseProjectSettings>();
	if (PoseSettings)
	{
		switch (Type)
		{
		case FControlRigAssetType::ControlRigPose:
			return PoseSettings->bFilterPoses;
		case FControlRigAssetType::ControlRigAnimation:
			return PoseSettings->bFilterAnimations;
		case FControlRigAssetType::ControlRigSelectionSet:
			return PoseSettings->bFilterSelectionSets;
		default:
			break;
		};
	}

	return false;
}
*/
/////////////////////////////////////////////////////////////////
//  Helper to Select Path 

class SPathDialogWithAllowList : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SPathDialogWithAllowList)
	{
	}

	SLATE_ARGUMENT(FText, DefaultAssetPath)
		SLATE_END_ARGS()

		SPathDialogWithAllowList()
		: UserResponse(EAppReturnType::Cancel)
	{
	}

	void Construct(const FArguments& InArgs);

public:
	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

protected:
	void OnPathChange(const FString& NewPath);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	bool IsOkButtonEnabled() const;
	EAppReturnType::Type UserResponse;
	FText AssetPath;

};

void SPathDialogWithAllowList::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SPathDialogWithAllowList::OnPathChange);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SPathDialogWithAllowList_Title", "Select Folder To Contain Poses"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectPath", "Select Path"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]

					+SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(3)
					[
						ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SPathDialogWithAllowList::OnButtonClick, EAppReturnType::Ok)
					.IsEnabled(this, &SPathDialogWithAllowList::IsOkButtonEnabled)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SPathDialogWithAllowList::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}

void SPathDialogWithAllowList::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
}

FReply SPathDialogWithAllowList::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	RequestDestroyWindow();

	return FReply::Handled();
}

bool SPathDialogWithAllowList::IsOkButtonEnabled() const
{
	return !AssetPath.IsEmptyOrWhitespace();
}

EAppReturnType::Type SPathDialogWithAllowList::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SPathDialogWithAllowList::GetAssetPath()
{
	return AssetPath.ToString();
}

/////////////////////////////////////////////////////
// SControlRigBaseListWidget, Main Dialog Window holding the path picker, asset view and pose view.

void SControlRigBaseListWidget::Construct(const FArguments& InArgs)
{
	WeakToolkit = InArgs._InSharedToolkit;
	FControlRigEditMode* EditMode = GetEditMode();
	BindCommands();

	const UControlRigPoseProjectSettings* PoseSettings = GetDefault<UControlRigPoseProjectSettings>();

	// Find the asset root of the current map to append relative pose folder paths to
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString RootPath = TEXT("/Game");
	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		const FString EditorWorldPackageName = EditorWorld->GetPackage()->GetName();
		if (!EditorWorldPackageName.StartsWith(TEXT("/Temp/")))
		{
			FString NewPackageFolder = FPackageName::GetLongPackagePath(EditorWorldPackageName);
			if (AssetTools.GetWritableFolderPermissionList()->PassesStartsWithFilter(NewPackageFolder))
			{
				RootPath = NewPackageFolder;
			}
		}
	}

	TArray<FString> PosesDirectories = PoseSettings->GetAssetPaths();
	for (FString& PoseDirectory : PosesDirectories)
	{
		// If relative, make it absolute to the root of the current map
		if (FPaths::IsRelative(PoseDirectory))
		{
			PoseDirectory = RootPath / PoseDirectory;
		}

		if (CurrentlySelectedInternalPath.IsEmpty())
		{
			CurrentlySelectedInternalPath = PoseDirectory;
		}
	}

	FName CurrentlySelectedVirtualPath;
	IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(FStringView(CurrentlySelectedInternalPath), CurrentlySelectedVirtualPath);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	// Configure filter for asset picker
	FAssetPickerConfig AssetPickerConfig;

	//AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.Filter.ClassPaths.Add(UControlRigPoseAsset::StaticClass()->GetClassPathName());

	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowFolders = true;
	AssetPickerConfig.bCanShowRealTimeThumbnails = true;
	AssetPickerConfig.ThumbnailLabel = EThumbnailLabel::AssetName;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
	AssetPickerConfig.Filter.PackagePaths.Add(CurrentlySelectedVirtualPath);
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SControlRigBaseListWidget::OnAssetSelected);
	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SControlRigBaseListWidget::OnAssetsActivated);
	AssetPickerConfig.SaveSettingsName = TEXT("ControlPoseDialog");
	AssetPickerConfig.bCanShowDevelopersFolder = true;
	AssetPickerConfig.OnFolderEntered = FOnPathSelected::CreateSP(this, &SControlRigBaseListWidget::HandleAssetViewFolderEntered);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SControlRigBaseListWidget::OnGetAssetContextMenu);	
	AssetPickerConfig.SetFilterDelegates.Add(&SetFilterDelegate);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.SelectionMode = ESelectionMode::Multi;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.AssetShowWarningText = LOCTEXT("NoPoses_Warning", "No Poses Found, Create One Using Button In Upper Left Corner");
	AssetPickerConfig.OnIsAssetValidForCustomToolTip = FOnIsAssetValidForCustomToolTip::CreateLambda([](const FAssetData& AssetData) {return AssetData.IsAssetLoaded(); });

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.bAddDefaultPath = true;
	PathPickerConfig.bOnPathSelectedPassesVirtualPaths = false;
	PathPickerConfig.DefaultPath = CurrentlySelectedInternalPath;
	CustomFolderPermissionList = PathPickerConfig.CustomFolderPermissionList = MakeShared<FPathPermissionList>();
	for (const FString& Path : PosesDirectories)
	{
		PathPickerConfig.CustomFolderPermissionList.Get()->AddAllowListItem("PoseLibrary", Path);
	}

	PathPickerConfig.OnGetFolderContextMenu = FOnGetFolderContextMenu::CreateSP(this, &SControlRigBaseListWidget::OnGetFolderContextMenu);
	PathPickerConfig.bFocusSearchBoxWhenOpened = false;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SControlRigBaseListWidget::HandlePathSelected);
	PathPickerConfig.SetPathsDelegates.Add(&SetPathsDelegate);
	PathPickerConfig.bAllowContextMenu = true;

	// The root widget in this dialog.
	TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);

	AssetPicker = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
	PathPicker = ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig);

	MainVerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SControlRigPoseAnimSelectionToolbar).OwningControlRigWidget(this)
		];

	// Path/Asset view
	MainVerticalBox->AddSlot()

		.HAlign(HAlign_Fill)
		.FillHeight(0.6)
		.Padding(0, 0, 0, 4)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+ SSplitter::Slot()
			.Value(0.66f)
			[
				SNew(SSplitter)

				+ SSplitter::Slot()
				.Value(0.33f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					PathPicker.ToSharedRef()
				]
				]

			+ SSplitter::Slot()
				.Value(0.66f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						AssetPicker.ToSharedRef()
					]
				]
			]
		+ SSplitter::Slot()
		.Value(0.4f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(ViewContainer, SBox)
				.Padding(FMargin(5.0f, 0, 0, 0))
			]
		]
		];

	ChildSlot
		[
			MainVerticalBox
		];

	CurrentViewType = ESelectedControlAsset::Type::None;
	CreateCurrentView(nullptr);
}

SControlRigBaseListWidget::~SControlRigBaseListWidget()
{
}

void SControlRigBaseListWidget::NotifyUser(FNotificationInfo& NotificationInfo)
{
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	if (Notification.IsValid())
	{
		Notification->SetCompletionState(SNotificationItem::CS_Fail);
	}
}

FControlRigEditMode* SControlRigBaseListWidget::GetEditMode()
{
	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(WeakToolkit.Pin()->GetEditorMode());
	return ControlRigEditMode;
}

FText SControlRigBaseListWidget::GetAssetNameText() const
{
	return FText::FromString(CurrentlyEnteredAssetName);
}

FText SControlRigBaseListWidget::GetPathNameText() const
{
	return FText::FromString(CurrentlySelectedInternalPath);
}

void SControlRigBaseListWidget::SetCurrentlySelectedPath(const FString& NewPath)
{
	CurrentlySelectedInternalPath = NewPath;
	UpdateInputValidity();
}

FString SControlRigBaseListWidget::GetCurrentlySelectedPath() const
{
	return CurrentlySelectedInternalPath;
}
void SControlRigBaseListWidget::SetCurrentlyEnteredAssetName(const FString& NewName)
{
	CurrentlyEnteredAssetName = NewName;
	UpdateInputValidity();
}

//todo to be used with renaming when it comes in reuse. (next thing to do).
void SControlRigBaseListWidget::UpdateInputValidity()
{
	bLastInputValidityCheckSuccessful = true;

	if (CurrentlyEnteredAssetName.IsEmpty())
	{
		bLastInputValidityCheckSuccessful = false;
	}

	if (bLastInputValidityCheckSuccessful)
	{
		if (CurrentlySelectedInternalPath.IsEmpty())
		{
			bLastInputValidityCheckSuccessful = false;
		}
	}

	/**
	const FString ObjectPath = GetObjectPathForSave();
	FText ErrorMessage;
	const bool bAllowExistingAsset = true;

	FName AssetClassName = AssetClassNames.Num() == 1 ? AssetClassNames[0] : NAME_None;
	UClass* AssetClass = AssetClassName != NAME_None ? FindObject<UClass>(nullptr, *AssetClassName.ToString(), EFindObjectFlags::ExactClass) : nullptr;

	if (!ContentBrowserUtils::IsValidObjectPathForCreate(ObjectPath, AssetClass, ErrorMessage, bAllowExistingAsset))
	{
		LastInputValidityErrorText = ErrorMessage;
		bLastInputValidityCheckSuccessful = false;
	}
	*/
}


FString SControlRigBaseListWidget::GetObjectPathForSave() const
{
	return CurrentlySelectedInternalPath / CurrentlyEnteredAssetName + TEXT(".") + CurrentlyEnteredAssetName;
}

void SControlRigBaseListWidget::SelectThisAsset(UObject* Asset)
{
	if (Asset != nullptr)
	{
		if (UControlRigPoseAsset* PoseAsset = Cast<UControlRigPoseAsset>(Asset))
		{
			CurrentViewType = ESelectedControlAsset::Type::Pose;
		}
		else
		{
			CurrentViewType = ESelectedControlAsset::Type::None;
		}
		FString Path = FPaths::GetPath(Asset->GetOutermost()->GetPathName());
		SetCurrentlySelectedPath(Path);
		SetCurrentlyEnteredAssetName(Asset->GetName());
	}
	else
	{
		CurrentViewType = ESelectedControlAsset::Type::None;
	}
	CreateCurrentView(Asset);
}

void SControlRigBaseListWidget::OnAssetSelected(const FAssetData& AssetData)
{
	UObject* Asset = nullptr;
	if (AssetData.IsValid())
	{
		Asset = AssetData.GetAsset();
	}
	SelectThisAsset(Asset);
}

void SControlRigBaseListWidget::OnAssetsActivated(const TArray<FAssetData>& SelectedAssets, EAssetTypeActivationMethod::Type ActivationType)
{
	if (SelectedAssets.Num() == 1 && (ActivationType == EAssetTypeActivationMethod::DoubleClicked))
	{
		UObject* Asset = nullptr;

		if (SelectedAssets[0].IsValid())
		{
			Asset = SelectedAssets[0].GetAsset();
			UControlRigPoseAsset* PoseAsset = Cast<UControlRigPoseAsset>(Asset);
			if (PoseAsset)
			{
				ExecutePastePose(PoseAsset);

				// If alt is down, select controls
				if (FSlateApplication::Get().GetModifierKeys().IsAltDown())
				{
					FControlRigEditMode* ControlRigEditMode = GetEditMode();
					if (ControlRigEditMode)
					{
						TArray<UControlRig*> ControlRigs = ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/);
						if (ControlRigs.Num() > 0)
						{
							const FScopedTransaction Transaction(LOCTEXT("SelectControls", "Select Controls"));
							for (UControlRig* ControlRig : ControlRigs)
							{
								ControlRig->Modify();
								PoseAsset->SelectControls(ControlRig, SControlRigPoseView::IsMirror());
							}
						}
					}
				}
			}
		}
		SelectThisAsset(Asset);
	}
}

void SControlRigBaseListWidget::FilterChanged()
{
	FName CurrentlySelectedVirtualPath;
	IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(FStringView(CurrentlySelectedInternalPath), CurrentlySelectedVirtualPath);

	FARFilter NewFilter;
	NewFilter.PackagePaths.Add(CurrentlySelectedVirtualPath);
	NewFilter.ClassPaths.Add(UControlRigPoseAsset::StaticClass()->GetClassPathName());
	SetFilterDelegate.ExecuteIfBound(NewFilter);
}

void SControlRigBaseListWidget::HandlePathSelected(const FString& NewPath)
{
	SetCurrentlySelectedPath(NewPath);
	FilterChanged();
}

void SControlRigBaseListWidget::HandleAssetViewFolderEntered(const FString& NewPath)
{
	SetCurrentlySelectedPath(NewPath);

	TArray<FString> NewPaths;
	NewPaths.Add(NewPath);
	SetPathsDelegate.Execute(NewPaths);
}

TSharedPtr<SWidget> SControlRigBaseListWidget::OnGetFolderContextMenu(const TArray<FString>& SelectedPaths, FContentBrowserMenuExtender_SelectedPaths InMenuExtender, FOnCreateNewFolder InOnCreateNewFolder)
{

	TSharedPtr<FExtender> Extender;
	if (InMenuExtender.IsBound())
	{
		Extender = InMenuExtender.Execute(SelectedPaths);
	}

	FMenuBuilder MenuBuilder(true /*bInShouldCloseWindowAfterMenuSelection*/, Commands, Extender);
	MenuBuilder.BeginSection("AssetDialogOptions", LOCTEXT("AssetDialogMenuHeading", "Options"));

	if (SelectedPaths.Num() == 1)
	{
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteAddFolder, SelectedPaths[0]));
			const FText Label = LOCTEXT("AddFolder", "Add Folder");
			const FText ToolTipText = LOCTEXT("AddFolder Tooltip", "Add Folder to the current selected folder");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteRenameFolder, SelectedPaths[0]));
			const FText Label = LOCTEXT("RenameFolder", "Rename Folder");
			const FText ToolTipText = LOCTEXT("RenameFolderTooltip", "Rename selected folder");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		MenuBuilder.AddSeparator();
		/*
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteDeleteFolder, SelectedPaths));
			const FText Label = LOCTEXT("DeleteFolder", "Delete Folder");
			const FText ToolTipText = LOCTEXT("DeleteFolderTooltip", "Delete selected folder(s), Note this will delete content.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		*/
	}
	else if (SelectedPaths.Num() > 0)
	{
		/*
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteDeleteFolder, SelectedPaths));
			const FText Label = LOCTEXT("DeleteFolder", "Delete Folder");
			const FText ToolTipText = LOCTEXT("DeleteFolderTooltip", "Delete selected folder(s), Note this will delete content.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		*/
	}
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteAddFolderToView));
		const FText Label = LOCTEXT("AddExistingFolderToView", "Add Existing Folder To View");
		const FText ToolTipText = LOCTEXT("AddExistingFolderToViewTooltip", "Add an existing folder to this view.");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SControlRigBaseListWidget::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	FMenuBuilder MenuBuilder(true /*bInShouldCloseWindowAfterMenuSelection*/, Commands);
	if (SelectedAssets.Num() == 0)
	{
		return nullptr;
	}
	MenuBuilder.BeginSection("PoseDialogOptions", LOCTEXT("Asset", "Asset"));
	{
		if (SelectedAssets.Num() == 1)
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteRenameAssets, SelectedAssets),
			FCanExecuteAction::CreateRaw(this, &SControlRigBaseListWidget::CanExecuteRenameAssets, SelectedAssets));
			const FText Label = LOCTEXT("RenameAssetButton", "Rename Asset");
			const FText ToolTipText = LOCTEXT("RenameAssetButtonTooltip", "Rename the selected asset");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}

		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteSaveAssets, SelectedAssets));
			const FText Label = LOCTEXT("SaveAssetButton", "Save Asset");
			const FText ToolTipText = LOCTEXT("SaveAssetButtonTooltip", "Save the selected assets");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteDeleteAssets, SelectedAssets));
			const FText Label = LOCTEXT("DeleteAssetButton", "Delete Asset");
			const FText ToolTipText = LOCTEXT("DeleteAssetButtonTooltip", "Delete the selected assets.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
	}
	MenuBuilder.EndSection();

	UObject* SelectedAsset = SelectedAssets[0].GetAsset();
	if (SelectedAsset)
	{

		UControlRigPoseAsset* PoseAsset = Cast<UControlRigPoseAsset>(SelectedAsset);
		if (PoseAsset)
		{
			MenuBuilder.BeginSection("PoseDialogOptions", LOCTEXT("PoseDialogMenuHeading", "Paste"));
			{

				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecutePastePose, PoseAsset),
					FCanExecuteAction::CreateRaw(this, &SControlRigBaseListWidget::CanExecutePastePose, PoseAsset));
				const FText Label = LOCTEXT("PastePoseButton", "Paste Pose");
				const FText ToolTipText = LOCTEXT("PastePoseButtonTooltip", "Paste the selected pose");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}

			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecutePasteMirrorPose, PoseAsset),
					FCanExecuteAction::CreateRaw(this, &SControlRigBaseListWidget::CanExecutePasteMirrorPose, PoseAsset));
				const FText Label = LOCTEXT("PasteMirrorPoseButton", "Paste Mirror Pose");
				const FText ToolTipText = LOCTEXT("PasteMirrorPoseButtonTooltip", "Paste the mirror of the pose");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("PoseDialogOptions", LOCTEXT("PoseDialogSelectHeading", "Selection"));
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteSelectControls, PoseAsset));
				const FText Label = LOCTEXT("SelectControls", "Select Controls");
				const FText ToolTipText = LOCTEXT("SelectControlsTooltip", "Select controls in this pose on active control rig");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("PoseDialogOptions", LOCTEXT("PoseDialogUpdateHeading", "Update"));
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteUpdatePose, PoseAsset));
				const FText Label = LOCTEXT("UpdatePose", "Update Pose");
				const FText ToolTipText = LOCTEXT("UpdatePoseTooltip", "Update the pose based upon current control rig pose and selected controls");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.EndSection();
		}
	}

	MenuBuilder.BeginSection("PoseDialogOptions", LOCTEXT("PoseDialogRenameControlsHeading", "Rename Controls"));
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SControlRigBaseListWidget::ExecuteRenamePoseControls, SelectedAssets));
		const FText Label = LOCTEXT("RenameControls", "Rename Controls");
		const FText ToolTipText = LOCTEXT("RenameControlsTooltip", "Rename controls in pose asset");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SControlRigBaseListWidget::BindCommands()
{
	Commands = TSharedPtr< FUICommandList >(new FUICommandList);
}

void SControlRigBaseListWidget::ExecuteRenameFolder(const FString SelectedPath)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().ExecuteRename(PathPicker);
}

void SControlRigBaseListWidget::ExecuteAddFolder(const FString SelectedPath)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().ExecuteAddFolder(PathPicker);
}

void SControlRigBaseListWidget::ExecuteRenameAssets(const TArray<FAssetData> SelectedAssets)
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().ExecuteRename(AssetPicker);
}

bool SControlRigBaseListWidget::CanExecuteRenameAssets(const TArray<FAssetData> SelectedAssets) const
{
	return true;
}

void SControlRigBaseListWidget::ExecuteSaveAssets(const TArray<FAssetData> SelectedAssets)
{
	TArray<UPackage*> PackagesToSave;
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.IsValid() && AssetData.GetPackage())
		{
			PackagesToSave.Add(AssetData.GetPackage());
			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false /*bCheckDirty*/, false /*bPromptToSave*/);
		}
	}
}

void SControlRigBaseListWidget::ExecuteDeleteAssets(const TArray<FAssetData> SelectedAssets)
{
	ObjectTools::DeleteAssets(SelectedAssets);
	SelectThisAsset(nullptr);
}

void SControlRigBaseListWidget::ExecuteAddFolderToView()
{
	TSharedRef<SPathDialogWithAllowList> NewAnimDlg =
		SNew(SPathDialogWithAllowList);

	if (NewAnimDlg->ShowModal() != EAppReturnType::Cancel)
	{
		FString AssetPath = NewAnimDlg->GetAssetPath();
		CustomFolderPermissionList.Get()->AddAllowListItem("PoseLibrary", AssetPath);
	    UControlRigPoseProjectSettings* PoseSettings = GetMutableDefault<UControlRigPoseProjectSettings>();
		FDirectoryPath Path;
		Path.Path = AssetPath;
		PoseSettings->RootSaveDirs.Add(Path);
		PoseSettings->SaveConfig();

		//Need to refresh since the deny list changed
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		ContentBrowserModule.Get().RefreshPathView(PathPicker);
	}
}

void SControlRigBaseListWidget::ExecuteDeleteFolder(const TArray<FString> SelectedFolders)
{
	/*
	// Don't allow asset deletion during PIE
	if (GIsEditor)
	{
		UEditorEngine* Editor = GEditor;
		FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
		if (PIEWorldContext)
		{
			FNotificationInfo Notification(LOCTEXT("CannotDeleteAssetInPIE", "Assets cannot be deleted while in PIE."));
			Notification.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Notification);
			return;
		}
	}
	
	const TArray<FContentBrowserItem> SelectedFiles = AssetPicker->GetAssetView()->GetSelectedFileItems();

	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& SelectedItem : SelectedFiles)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				FText DeleteErrorMsg;
				if (ItemDataSource->CanDeleteItem(ItemData, &DeleteErrorMsg))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
				else
				{
					AssetViewUtils::ShowErrorNotifcation(DeleteErrorMsg);
				}
			}
		}
	}

	// Execute the operation now
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		SourceAndItemsPair.Key->BulkDeleteItems(SourceAndItemsPair.Value);
	}
	
	// If we had any folders selected, ask the user whether they want to delete them 
	// as it can be slow to build the deletion dialog on an accidental click
	if (SelectedFolders.Num() > 0)
	{
		FText Prompt;
		if (SelectedFolders.Num() == 1)
		{
			//			Prompt = FText::Format(LOCTEXT("FolderDeleteConfirm_Single", "Delete folder '{0}'?"), SelectedFolders[0]);
		}
		else
		{
			//		Prompt = FText::Format(LOCTEXT("FolderDeleteConfirm_Multiple", "Delete {0} folders?"), SelectedFolders.Num());
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	
		// Spawn a confirmation dialog since this is potentially a highly destructive operation
		ContentBrowserModule.Get().DisplayConfirmationPopup(
			Prompt,
			LOCTEXT("FolderDeleteConfirm_Yes", "Delete"),
			LOCTEXT("FolderDeleteConfirm_No", "Cancel"),
			ToSharedRef(),
			FOnClicked::CreateSP(this, &SControlRigBaseListWidget::ExecuteDeleteFolderConfirmed)
		);
		
	}
	*/

}

FReply SControlRigBaseListWidget::ExecuteDeleteFolderConfirmed()
{
	/*
	TArray< FString > SelectedFolders = AssetPicker->GetAssetView()->GetSelectedFolders();

	if (SelectedFolders.Num() > 0)
	{
		ContentBrowserUtils::DeleteFolders(SelectedFolders);
	}
	else
	{
		const TArray<FString>& SelectedPaths = PathPicker->GetPaths();

		if (SelectedPaths.Num() > 0)
		{
			if (ContentBrowserUtils::DeleteFolders(SelectedPaths))
			{
				// Since the contents of the asset view have just been deleted, set the selected path to the default "/Game"
				TArray<FString> DefaultSelectedPaths;
				DefaultSelectedPaths.Add(TEXT("/Game"));
				PathPicker->GetPathView()->SetSelectedPaths(DefaultSelectedPaths);

				FAssetViewContentSources DefaultContentSources(FName("/Game"));
				AssetPicker->GetAssetView()->SetContentSources(DefaultContentSources);
			}
		}
	}
	*/
	return FReply::Handled();
}


void SControlRigBaseListWidget::ExecutePastePose(UControlRigPoseAsset* PoseAsset)
{
	if (PoseAsset)
	{
		FControlRigEditMode* ControlRigEditMode = GetEditMode();
		if (ControlRigEditMode)
		{
			TArray<UControlRig*> ControlRigs = ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/);
			if (ControlRigs.Num() > 0)
			{
				const FScopedTransaction Transaction(LOCTEXT("PastePose", "Paste Pose"));
				for (UControlRig* ControlRig : ControlRigs)
				{
					ControlRig->Modify();
					PoseAsset->PastePose(ControlRig);
				}
			}
		}
	}
}

bool SControlRigBaseListWidget::CanExecutePastePose(UControlRigPoseAsset* PoseAsset) const
{
	return PoseAsset != nullptr;
}

void SControlRigBaseListWidget::ExecuteUpdatePose(UControlRigPoseAsset* PoseAsset)
{
	FControlRigUpdatePoseDialog::UpdatePose(PoseAsset);
}

void SControlRigBaseListWidget::ExecuteSelectControls(UControlRigPoseAsset* PoseAsset)
{
	FControlRigEditMode* ControlRigEditMode = GetEditMode();
	if (ControlRigEditMode)
	{
		TArray<UControlRig*> ControlRigs = ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/);
		if (ControlRigs.Num() > 0)
		{
			const FScopedTransaction Transaction(LOCTEXT("SelectControls", "Select Controls"));
			for (UControlRig* ControlRig : ControlRigs)
			{
				ControlRig->Modify();
				PoseAsset->SelectControls(ControlRig);
			}
		}
	}
	else
	{
		ULevelSequence *LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetFocusedLevelSequence();
		if (LevelSequence)
		{
			const FScopedTransaction Transaction(LOCTEXT("SelectControls", "Select Controls"));
			TArray<FControlRigSequencerBindingProxy> Proxies = UControlRigSequencerEditorLibrary::GetControlRigs(LevelSequence);
			for (FControlRigSequencerBindingProxy& Proxy: Proxies)
			{
				if (Proxy.ControlRig)
				{
					Proxy.ControlRig->Modify();
					PoseAsset->SelectControls(Proxy.ControlRig);
				}
			}
		}
	}
}

void SControlRigBaseListWidget::ExecutePasteMirrorPose(UControlRigPoseAsset* PoseAsset)
{
	if (PoseAsset)
	{
		FControlRigEditMode* ControlRigEditMode = GetEditMode();
		if (ControlRigEditMode)
		{
			TArray<UControlRig*> ControlRigs = ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/);
			if (ControlRigs.Num() > 0)
			{
				const FScopedTransaction Transaction(LOCTEXT("PasteMirrorPose", "Paste Mirror Pose"));
				for (UControlRig* ControlRig : ControlRigs)
				{
					ControlRig->Modify();
					PoseAsset->PastePose(ControlRig, false, true);
				}
			}
		}
	}
}

bool SControlRigBaseListWidget::CanExecutePasteMirrorPose(UControlRigPoseAsset* PoseAsset) const
{
	return PoseAsset != nullptr;
}

void SControlRigBaseListWidget::ExecuteRenamePoseControls(const TArray<FAssetData> SelectedAssets)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TArray<UControlRigPoseAsset*> SelectedObjects;
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.IsValid() && AssetData.GetAsset() && AssetData.GetAsset()->IsA<UControlRigPoseAsset>())
		{
			UControlRigPoseAsset* PoseAsset = Cast<UControlRigPoseAsset>(AssetData.GetAsset());
			SelectedObjects.Add(PoseAsset);
		}
	}
	FControlRigRenameControlsDialog::RenameControls(SelectedObjects);
}

void SControlRigBaseListWidget::CreateCurrentView(UObject* Asset)
{
	PoseView.Reset();
	AnimationView.Reset();
	SelectionSetView.Reset();
	EmptyBox.Reset();
	TSharedRef<SWidget> NewView = SNullWidget::NullWidget;
	switch (CurrentViewType)
	{
	case ESelectedControlAsset::Type::Pose:
		PoseView = CreatePoseView(Asset);
		ViewContainer->SetContent(PoseView.ToSharedRef());
		break;
	case ESelectedControlAsset::Type::Animation:
		AnimationView = CreateAnimationView(Asset);

		ViewContainer->SetContent(AnimationView.ToSharedRef());
		break;
	case ESelectedControlAsset::Type::SelectionSet:
		SelectionSetView = CreateSelectionSetView(Asset);

		ViewContainer->SetContent(SelectionSetView.ToSharedRef());
		break;
	case ESelectedControlAsset::Type::None:
		EmptyBox = SNew(SBox);
		ViewContainer->SetContent(EmptyBox.ToSharedRef());
		break;
	}
}

TSharedRef<class SControlRigPoseView> SControlRigBaseListWidget::CreatePoseView(UObject* InObject)
{
	UControlRigPoseAsset* PoseAsset = Cast<UControlRigPoseAsset>(InObject);
	return SNew(SControlRigPoseView)
		.PoseAsset(PoseAsset)
		.OwningWidget(SharedThis(this));
}

TSharedRef<class SControlRigPoseView> SControlRigBaseListWidget::CreateAnimationView(UObject* InObject)
{
	return SNew(SControlRigPoseView)
		.OwningWidget(SharedThis(this));
}

TSharedRef<class SControlRigPoseView> SControlRigBaseListWidget::CreateSelectionSetView(UObject* InObject)
{
	return SNew(SControlRigPoseView)
		.OwningWidget(SharedThis(this));
}

#undef LOCTEXT_NAMESPACE
