// Copyright Epic Games, Inc. All Rights Reserved.

/**
* Main Window and associated classes that holds the Path View, Asset Browswer
* and the Control Views
*/
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "EditorUndoClient.h"
#include "AssetRegistry/AssetData.h"
#include "IAssetTools.h"
#include "ContentBrowserDelegates.h"
#include "EditMode/SControlRigControlViews.h"

class UControlRig;
class FControlRigEditMode;
class UControlRigPoseAsset;
class FControlRigEditModeToolkit;

namespace ESelectedControlAsset
{
	enum Type
	{
		None,
		Pose,
		Animation,
		SelectionSet,

		MAX
	};
}

class SControlRigBaseListWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SControlRigBaseListWidget)

	{}
	SLATE_ARGUMENT(TWeakPtr<FControlRigEditModeToolkit>, InSharedToolkit)
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs);


	SControlRigBaseListWidget() : bLastInputValidityCheckSuccessful(false) {}
	~SControlRigBaseListWidget();

	FControlRigEditMode* GetEditMode();

	/*Notification that filter has changed, this includes new folder selected*/
	void FilterChanged();
	
	/*Current Path we use to save the asset.*/
	FString GetCurrentlySelectedPath()const;

	/* Select this Asset, make sure Views all sync up*/
	void SelectThisAsset(UObject* Asset);
private:

	void BindCommands();

	/** Asset Related Functions*/
	FText GetAssetNameText()const;
	FText GetPathNameText()const;
	void SetCurrentlySelectedPath(const FString& NewPath);
	void SetCurrentlyEnteredAssetName(const FString& NewName);
	void UpdateInputValidity();
	FString GetObjectPathForSave() const;

	/* Asset Browser Functions*/
	void OnAssetSelected(const FAssetData& AssetData);
	void OnAssetsActivated(const TArray<FAssetData>& SelectedAssets, EAssetTypeActivationMethod::Type ActivationType);
	void HandleAssetViewFolderEntered(const FString& NewPath);
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets);
	void HandlePathSelected(const FString& NewPath);
	TSharedPtr<SWidget> OnGetFolderContextMenu(const TArray<FString>& SelectedPaths, FContentBrowserMenuExtender_SelectedPaths InMenuExtender, FOnCreateNewFolder InOnCreateNewFolder);
	FSetARFilterDelegate SetFilterDelegate;
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
	FSetPathPickerPathsDelegate SetPathsDelegate;

	/** Context Menu Functions*/
	void ExecuteDeleteFolder(const TArray<FString> SelectedPaths);
	void ExecuteRenameFolder(const FString SelectedPath);
	void ExecuteAddFolder(const FString SelectedPath);
	FReply ExecuteDeleteFolderConfirmed();
	void ExecuteRenameAssets(const TArray<FAssetData> RenameAssets);
	bool CanExecuteRenameAssets(const TArray<FAssetData> RenameAssets) const;
	void ExecuteSaveAssets(const TArray<FAssetData> SelectedAssets);
	void ExecuteDeleteAssets(const TArray<FAssetData> SelectedAssets);
	void ExecutePastePose(UControlRigPoseAsset* PoseAsset);
	bool CanExecutePastePose(UControlRigPoseAsset* PoseAsset) const;
	void ExecuteSelectControls(UControlRigPoseAsset* PoseAsset);
	void ExecutePasteMirrorPose(UControlRigPoseAsset* PoseAsset);
	bool CanExecutePasteMirrorPose(UControlRigPoseAsset* PoseAsset) const;
	void ExecuteUpdatePose(UControlRigPoseAsset* PoseAsset);
	void ExecuteRenamePoseControls(const TArray<FAssetData> SelectedAssets);
	void ExecuteAddFolderToView();

	/** Create the Views for the Selected Asset*/
	void CreateCurrentView(UObject* Asset);
	TSharedRef<SControlRigPoseView> CreatePoseView(UObject* Asset);
	TSharedRef<SControlRigPoseView> CreateAnimationView(UObject* Asset);
	TSharedRef<SControlRigPoseView> CreateSelectionSetView(UObject* Asset);

	/** View Type Data*/
	ESelectedControlAsset::Type CurrentViewType;
	TSharedPtr<SControlRigPoseView> PoseView;
	TSharedPtr<SControlRigPoseView> AnimationView; //todo
	TSharedPtr<SControlRigPoseView> SelectionSetView; //todo
	TSharedPtr<SBox> EmptyBox;
	TSharedPtr<SBox> ViewContainer;

	TSharedPtr<SWidget> PathPicker;
	TSharedPtr<SWidget> AssetPicker;

	static FString CurrentlySelectedInternalPath;
	FString CurrentlyEnteredAssetName;
	bool bLastInputValidityCheckSuccessful;

	/** Commands handled by this widget */
	TSharedPtr< FUICommandList > Commands;

	/** Deny list held by the Path Picker*/
	TSharedPtr<FPathPermissionList> CustomFolderPermissionList;

	/** Utility function to display notifications to the user */
	void NotifyUser(FNotificationInfo& NotificationInfo);

	TWeakPtr<FControlRigEditModeToolkit> WeakToolkit;
};
