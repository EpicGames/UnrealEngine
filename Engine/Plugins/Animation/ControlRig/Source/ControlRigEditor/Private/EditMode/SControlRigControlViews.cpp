// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditMode/SControlRigControlViews.h"
#include "ControlRig.h"
#include "Tools/ControlRigPose.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Styling/AppStyle.h"
#include "ScopedTransaction.h"
#include "Editor/EditorEngine.h"
#include "AssetViewUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "AssetThumbnail.h"
#include "FileHelpers.h"
#include "Tools/ControlRigPoseMirrorSettings.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Widgets/Input/SButton.h"
#include "LevelEditor.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "PropertyEditorModule.h"
#include "SControlRigBaseListWidget.h"
#include "UnrealClient.h"

#define LOCTEXT_NAMESPACE "ControlRigBaseListWidget"


void FControlRigView::CaptureThumbnail(UObject* Asset)
{
	FViewport* Viewport = GEditor->GetActiveViewport();

	if (GCurrentLevelEditingViewportClient && Viewport)
	{
		//have to re-render the requested viewport
		FLevelEditorViewportClient* OldViewportClient = GCurrentLevelEditingViewportClient;
		//remove selection box around client during render
		GCurrentLevelEditingViewportClient = NULL;
		Viewport->Draw();

		TArray<FAssetData> SelectedAssets;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(Asset));
		SelectedAssets.Emplace(AssetData);
		AssetViewUtils::CaptureThumbnailFromViewport(Viewport, SelectedAssets);

		//redraw viewport to have the yellow highlight again
		GCurrentLevelEditingViewportClient = OldViewportClient;
		Viewport->Draw();
	}
	
}

/** Widget wraps an editable text box for editing name of the asset */
class SControlRigAssetEditableTextBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SControlRigAssetEditableTextBox) {}

		SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Asset)

		SLATE_END_ARGS()

		/**
		 * Construct this widget
		 *
		 * @param	InArgs	The declaration data for this widget
		 */
		void Construct(const FArguments& InArgs);


private:

	/** Getter for the Text attribute of the editable text inside this widget */
	FText GetNameText() const;

	/** Getter for the ToolTipText attribute of the editable text inside this widget */
	FText GetNameTooltipText() const;


	/** Getter for the OnTextCommitted event of the editable text inside this widget */
	void OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	/** Callback to verify a text change */
	void OnTextChanged(const FText& InLabel);

	/** The list of objects whose names are edited by the widget */
	TWeakObjectPtr<UObject> Asset;

	/** The text box used to edit object names */
	TSharedPtr< STextBlock > TextBox;

};

void SControlRigAssetEditableTextBox::Construct(const FArguments& InArgs)
{
	Asset = InArgs._Asset;
	ChildSlot
		[
			SAssignNew(TextBox, STextBlock)
			.Text(this, &SControlRigAssetEditableTextBox::GetNameText)
			// Current Thinking is to not have this be editable here, so removing it, but leaving in case we change our minds again.
			//.ToolTipText(this, &SControlRigAssetEditableTextBox::GetNameTooltipText)
			//.OnTextCommitted(this, &SControlRigAssetEditableTextBox::OnNameTextCommitted)
			//.OnTextChanged(this, &SControlRigAssetEditableTextBox::OnTextChanged)
			//.RevertTextOnEscape(true)
		];
}

FText SControlRigAssetEditableTextBox::GetNameText() const
{
	if (Asset.IsValid())
	{
		FString Result = Asset.Get()->GetName();
		return FText::FromString(Result);
	}
	return FText();
}

FText SControlRigAssetEditableTextBox::GetNameTooltipText() const
{
	FText Result = FText::Format(LOCTEXT("AssetRenameTooltip", "Rename the selected {0}"), FText::FromString(Asset.Get()->GetClass()->GetName()));
	
	return Result;
}


void SControlRigAssetEditableTextBox::OnNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{

	if (InTextCommit != ETextCommit::OnCleared)
	{
		FText TrimmedText = FText::TrimPrecedingAndTrailing(NewText);

		if (!TrimmedText.IsEmpty())
		{

			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
			FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(Asset.Get()));
			const FString PackagePath = FPackageName::GetLongPackagePath(Asset->GetOutermost()->GetName());

			//Need to save asset before renaming else may lose snapshot
			// save existing play list asset
			TArray<UPackage*> PackagesToSave;
			PackagesToSave.Add(Asset->GetPackage());
			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false /*bCheckDirty*/, false /*bPromptToSave*/);

			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

			TArray<FAssetRenameData> AssetsAndNames;
			AssetsAndNames.Emplace(FAssetRenameData(Asset, PackagePath, TrimmedText.ToString()));
			AssetToolsModule.Get().RenameAssetsWithDialog(AssetsAndNames);

		}
			
		// Remove ourselves from the window focus so we don't get automatically reselected when scrolling around the context menu.
		TSharedPtr< SWindow > ParentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
		if (ParentWindow.IsValid())
		{
			ParentWindow->SetWidgetToFocusOnActivate(NULL);
		}
	}

	// Clear Error 
	//TextBox->SetError(FText::GetEmpty());
}

void SControlRigAssetEditableTextBox::OnTextChanged(const FText& InLabel)
{
	const FString PackagePath = FPackageName::GetLongPackagePath(Asset->GetOutermost()->GetName());
	const FString PackageName = PackagePath / InLabel.ToString();
	const FString ObjectPath = FString::Printf(TEXT("%s.%s"), *PackageName, *(InLabel.ToString()));

	FText OutErrorMessage;
	if (!AssetViewUtils::IsValidObjectPathForCreate(ObjectPath, OutErrorMessage))
	{
		//TextBox->SetError(OutErrorMessage);
	}
	else
	{
		//TextBox->SetError(FText::GetEmpty());
	}
}

bool SControlRigPoseView::bIsKey = false;
bool SControlRigPoseView::bIsMirror = false;
bool SControlRigPoseView::bIsAdditive = false;
float SControlRigPoseView::bDynamicMin = 0.0;
float SControlRigPoseView::bDynamicMax = 1.0;
float SControlRigPoseView::SplitterSlotValues[3] = {0.33f, 0.33f, 0.33f};

void SControlRigPoseView::Construct(const FArguments& InArgs)
{
	PoseAsset = InArgs._PoseAsset;
	OwningWidget = InArgs._OwningWidget;

	PoseBlendValue = 0.0f;
	bIsBlending = false;
	bSliderStartedTransaction = false;

	TSharedRef<SWidget> ThumbnailWidget = GetThumbnailWidget();
	TSharedRef <SControlRigAssetEditableTextBox> ObjectNameBox = SNew(SControlRigAssetEditableTextBox).Asset(PoseAsset);

	//Not used currently CreateControlList();

	//for mirror settings
	UControlRigPoseMirrorSettings* MirrorSettings = GetMutableDefault<UControlRigPoseMirrorSettings>();
	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.ViewIdentifier = "Create Control Asset";

	MirrorDetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
	MirrorDetailsView->SetObject(MirrorSettings);
			
	ChildSlot
	[

		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		
			.FillHeight(1)
			.Padding(0, 0, 0, 4)
			[
			SAssignNew(PoseSplitter, SSplitter)
				.OnSplitterFinishedResizing(this, &SControlRigPoseView::OnSplitterFinishedResizing)

				+ SSplitter::Slot()
				.Value(SplitterSlotValues[0])
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(5.f)
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
						[
							ObjectNameBox
						]
						]

					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(5.f)
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
						[
							ThumbnailWidget
						]
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						.Padding(5.f)

						[
							SNew(SButton)
							.ContentPadding(FMargin(10, 5))
						.Text(LOCTEXT("CaptureThmbnail", "Capture Thumbnail"))
						.ToolTipText(LOCTEXT("CaptureThmbnailTooltip", "Captures a thumbnail from the active viewport"))
						.OnClicked(this, &SControlRigPoseView::OnCaptureThumbnail)
						]
					]
				]

			+ SSplitter::Slot()
				.Value(SplitterSlotValues[1])
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(5.f)

							[
							SNew(SButton)
								.ContentPadding(FMargin(10, 5))
							.Text(LOCTEXT("PastePose", "Paste Pose"))
							.OnClicked(this, &SControlRigPoseView::OnPastePose)
							]
						+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(2.5f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Center)
							.Padding(2.5f)
							[
								SNew(SCheckBox)
								.IsChecked(this, &SControlRigPoseView::IsKeyPoseChecked)
							.OnCheckStateChanged(this, &SControlRigPoseView::OnKeyPoseChecked)
							.Padding(2.5f)

							[
								SNew(STextBlock).Text(LOCTEXT("Key", "Key"))

							]
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()

							.HAlign(HAlign_Center)

							.Padding(2.5f)
							[
								SNew(SCheckBox)
								.IsChecked(this, &SControlRigPoseView::IsMirrorPoseChecked)
							.OnCheckStateChanged(this, &SControlRigPoseView::OnMirrorPoseChecked)
							.IsEnabled(this, &SControlRigPoseView::IsMirrorEnabled)
							.Padding(1.0f)
							[
								SNew(STextBlock).Text(LOCTEXT("Mirror", "Mirror"))
								.IsEnabled(this, &SControlRigPoseView::IsMirrorEnabled)
							]
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()

							.HAlign(HAlign_Center)

							.Padding(2.5f)
							[
							SNew(SCheckBox)
							.IsChecked(this, &SControlRigPoseView::IsAdditivePoseChecked)
							.OnCheckStateChanged(this, &SControlRigPoseView::OnAdditivePoseChecked)
							.Padding(1.0f)
							[
								SNew(STextBlock).Text(LOCTEXT("Additive", "Additive"))
							]
							]
							]
						+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(2.5f)
							[

								SNew(SSpinBox<float>)
								// Only allow spinning if we have a single value
								.PreventThrottling(true)
								.Value(this, &SControlRigPoseView::OnGetPoseBlendValue)
								.ToolTipText(LOCTEXT("BlendTooltip", "Blend between current pose and pose asset. Use Alt to extend values"))
								.MinValue(-1.0f)
								.MaxValue(2.0f)
								.MinSliderValue_Lambda([]() ->float
								{
									if (SControlRigPoseView::bIsAdditive)
									{
										return -1.0;
									}
									return SControlRigPoseView::bDynamicMin;

								})
								.MaxSliderValue_Lambda([]() ->float
								{
									if (SControlRigPoseView::bIsAdditive)
									{
										return 2.0f;
									}
									return SControlRigPoseView::bDynamicMax;

								})
								.SliderExponent(1)
								.Delta(0.005f)
								.MinDesiredWidth(100.0f)
								.SupportDynamicSliderMinValue(true)
								.SupportDynamicSliderMaxValue(true)
								.OnDynamicSliderMinValueChanged_Lambda([](float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool bIsOriginator, bool bUpdateOnlyIfLower)
								{
									if (((NewMinSliderValue < SControlRigPoseView::bDynamicMin) && bUpdateOnlyIfLower) || !bUpdateOnlyIfLower)
									{
										SControlRigPoseView::bDynamicMin = NewMinSliderValue;
									}

								})
								.OnDynamicSliderMaxValueChanged_Lambda([](float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool bIsOriginator, bool bUpdateOnlyIfHigher)
								{
									if (((NewMaxSliderValue > SControlRigPoseView::bDynamicMax) && bUpdateOnlyIfHigher) || !bUpdateOnlyIfHigher)
									{
										SControlRigPoseView::bDynamicMax = NewMaxSliderValue;
									}

								})
								.OnValueChanged(this, &SControlRigPoseView::OnPoseBlendChanged)
								.OnValueCommitted(this, &SControlRigPoseView::OnPoseBlendCommited)
								.OnBeginSliderMovement(this,&SControlRigPoseView::OnBeginSliderMovement)
								.OnEndSliderMovement(this,&SControlRigPoseView::OnEndSliderMovement)

							]

							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(15.f)
							[
								SNew(SButton)
								.ContentPadding(FMargin(10, 5))
								.Text(LOCTEXT("SelectControls", "Select Controls"))
								.OnClicked(this, &SControlRigPoseView::OnSelectControls)
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(3.f)
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.Padding(FMargin(3.0f, 2.0f))
								.Visibility(EVisibility::HitTestInvisible)
								[
									SAssignNew(TextStatusBlock1, STextBlock)
								]
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(1.0f)
								[
									SNew(SBorder)
									.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
								.Padding(FMargin(3.0f, 0.0f))
								.Visibility(EVisibility::HitTestInvisible)
								[
									SAssignNew(TextStatusBlock2, STextBlock)
								]
								]
					]
				]
			+ SSplitter::Slot()
				.Value(SplitterSlotValues[2])
				[
					MirrorDetailsView.ToSharedRef()
				]
			/*  todo may want to put this back, it let's you see the controls...
			+ SSplitter::Slot()
				.Value(0.33f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							.Padding(2.f)
							[

							SNew(SButton)
								.ContentPadding(FMargin(5, 5))
								.Text(LOCTEXT("SelectControls", "Select Controls"))
								.ToolTipText(LOCTEXT("SelectControlsTooltip", "Select controls from this asset"))
								.OnClicked(this, &SControlRigPoseView::OnSelectControls)
				
							]
						+ SVerticalBox::Slot()
							.VAlign(VAlign_Fill)
							.HAlign(HAlign_Center)
							.Padding(5.f)

							[
							SNew(SListView< TSharedPtr<FString> >)
								.ItemHeight(24)
								.ListItemsSource(&ControlList)
								.SelectionMode(ESelectionMode::None)
								.OnGenerateRow(this, &SControlRigPoseView::OnGenerateWidgetForList)
							]
					]
				]
				*/
			]
	];
	if (OwningWidget.IsValid())
	{
		if (FControlRigEditMode* EditMode = OwningWidget.Pin()->GetEditMode())
		{
			EditMode->OnControlRigAddedOrRemoved().AddSP(SharedThis(this), &SControlRigPoseView::HandleControlAdded);
			TArray<UControlRig*> ControlRigs = GetControlRigs();
			for (UControlRig* ControlRig : ControlRigs)
			{
				HandleControlAdded(ControlRig, true);
			}
		}
	}
}

SControlRigPoseView::~SControlRigPoseView()
{
	if(OwningWidget.IsValid() && OwningWidget.Pin()->GetEditMode())
	{
		FControlRigEditMode* EditMode = OwningWidget.Pin()->GetEditMode();
		EditMode->OnControlRigAddedOrRemoved().RemoveAll(this);
		TArray<UControlRig*> EditModeRigs = EditMode->GetControlRigsArray(false /*bIsVisible*/);
		for (UControlRig* ControlRig : EditModeRigs)
		{
			if (ControlRig)
			{
				ControlRig->ControlSelected().RemoveAll(this);
			}
		}
	}
	else
	{
		for (TWeakObjectPtr<UControlRig>& CurrentControlRig: CurrentControlRigs)
		{
			if (CurrentControlRig.IsValid())
			{
				(CurrentControlRig.Get())->ControlSelected().RemoveAll(this);
			}
		}
	}
}

void SControlRigPoseView::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	bUpdateStatusBlocks = true;
}

ECheckBoxState SControlRigPoseView::IsKeyPoseChecked() const
{
	if (bIsKey)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SControlRigPoseView::OnKeyPoseChecked(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Checked)
	{
		bIsKey = true;
	}
	else
	{
		bIsKey = false;
	}
}

ECheckBoxState SControlRigPoseView::IsMirrorPoseChecked() const
{
	if (bIsMirror)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SControlRigPoseView::OnMirrorPoseChecked(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Checked)
	{
		bIsMirror = true;
	}
	else
	{
		bIsMirror = false;
	}
	bUpdateStatusBlocks = true;
}

bool SControlRigPoseView::IsMirrorEnabled() const
{
	return true;
}

ECheckBoxState SControlRigPoseView::IsAdditivePoseChecked() const
{
	if (bIsAdditive)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SControlRigPoseView::OnAdditivePoseChecked(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Checked)
	{
		bIsAdditive = true;
	}
	else
	{
		bIsAdditive = false;
	}
	bUpdateStatusBlocks = true;
}

FReply SControlRigPoseView::OnPastePose()
{
	if (PoseAsset.IsValid())
	{
		TArray<UControlRig*> ControlRigs = GetControlRigs();
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				PoseAsset->PastePose(ControlRig, bIsKey, bIsMirror,bIsAdditive);
			}
		}
	}
	return FReply::Handled();
}

FReply SControlRigPoseView::OnSelectControls()
{	
	if (PoseAsset.IsValid())
	{
		bool bIsSelected = true;
		TArray<UControlRig*> ControlRigs = GetControlRigs(bIsSelected);
		if (ControlRigs.Num() == 0)
		{
			bIsSelected = false;
			ControlRigs = GetControlRigs(bIsSelected);
		}
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				PoseAsset->SelectControls(ControlRig, bIsMirror);
			}
		}
	}
	return FReply::Handled();
}

void SControlRigPoseView::OnPoseBlendChanged(float ChangedVal)
{
	auto ControlRigContainsControls = [this] (const UControlRig* InControlRig)
	{
		TArray<FName> ControlNames = PoseAsset->GetControlNames();
		for (FName ControlName : ControlNames)
		{
			if (InControlRig->FindControl(ControlName))
			{
				return true;
			}
		}
		return false;
	};
	
	if (PoseAsset.IsValid())
	{
		TArray<UControlRig*> ControlRigs = GetControlRigs();
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig && ControlRigContainsControls(ControlRig))
			{
				PoseBlendValue = ChangedVal;
				if (!bIsBlending)
				{
					bIsBlending = true;
					PoseAsset->GetCurrentPose(ControlRig, TempPose);
				}

				PoseAsset->BlendWithInitialPoses(TempPose, ControlRig, false, bIsMirror, PoseBlendValue, bIsAdditive);
				PoseAsset->BlendWithInitialPoses(TempPose, ControlRig, false, bIsMirror, PoseBlendValue, bIsAdditive);
			}
		}
	}
}
void SControlRigPoseView::OnBeginSliderMovement()
{
	if (bSliderStartedTransaction == false)
	{
		bSliderStartedTransaction = true;
		GEditor->BeginTransaction(LOCTEXT("PastePoseTransation", "Paste Pose"));
	}
}
void SControlRigPoseView::OnEndSliderMovement(float NewValue)
{
	if (bSliderStartedTransaction)
	{
		GEditor->EndTransaction();
		bSliderStartedTransaction = false;

	}
}

void SControlRigPoseView::OnPoseBlendCommited(float ChangedVal, ETextCommit::Type Type)
{
	if (PoseAsset.IsValid())
	{
		TArray<UControlRig*> ControlRigs = GetControlRigs();
		if (ControlRigs.Num() > 0)
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("PastePoseTransaction", "Paste Pose"));
			for (UControlRig* ControlRig : ControlRigs)
			{
				if (ControlRig)
				{
					PoseBlendValue = ChangedVal;
					PoseAsset->BlendWithInitialPoses(TempPose, ControlRig, bIsKey, bIsMirror, PoseBlendValue, bIsAdditive);
					PoseAsset->BlendWithInitialPoses(TempPose, ControlRig, bIsKey, bIsMirror, PoseBlendValue, bIsAdditive);
					bIsBlending = false;
					PoseBlendValue = 0.0f;

				}
			}
		}
	}
}

FReply SControlRigPoseView::OnCaptureThumbnail()
{
	FControlRigView::CaptureThumbnail(PoseAsset.Get());
	return FReply::Handled();
}

TSharedRef<SWidget> SControlRigPoseView::GetThumbnailWidget()
{
	const int32 ThumbnailSize = 128;
	Thumbnail = MakeShareable(new FAssetThumbnail(PoseAsset.Get(), ThumbnailSize, ThumbnailSize, UThumbnailManager::Get().GetSharedThumbnailPool()));
	FAssetThumbnailConfig ThumbnailConfig;
	ThumbnailConfig.bAllowFadeIn = false;
	ThumbnailConfig.bAllowHintText = false;
	ThumbnailConfig.bAllowRealTimeOnHovered = false; // we use our own OnMouseEnter/Leave for logical asset item
	ThumbnailConfig.bForceGenericThumbnail = false;

	TSharedRef<SOverlay> ItemContentsOverlay = SNew(SOverlay);
	ItemContentsOverlay->AddSlot()
		[
			Thumbnail->MakeThumbnailWidget(ThumbnailConfig)
		];

	return SNew(SBox)
		.Padding(0)
		.WidthOverride(ThumbnailSize)
		.HeightOverride(ThumbnailSize)
		[
			ItemContentsOverlay

		];
}

TArray<UControlRig*> SControlRigPoseView::GetControlRigs(bool bIsSelected)
{
	TArray<UControlRig*> NewControlRigs;
	if (OwningWidget.IsValid())
	{
		FControlRigEditMode* EditMode = OwningWidget.Pin()->GetEditMode();
		if (EditMode)
		{
			NewControlRigs =  EditMode->GetControlRigsArray(false /*bIsVisible*/);
		}
		for (TWeakObjectPtr<UControlRig> ControlRigPtr : CurrentControlRigs)
		{
			if (ControlRigPtr.IsValid())
			{
				if (NewControlRigs.Contains(ControlRigPtr.Get()) == false)
				{
					(ControlRigPtr.Get())->ControlSelected().RemoveAll(this);
				}
			}
		}
		if (EditMode)
		{
			CurrentControlRigs = EditMode->GetControlRigs();
		}
	}
	if (bIsSelected)
	{
		TArray<UControlRig*> SelectedControlRigs;
		for (TWeakObjectPtr<UControlRig> ControlRigPtr : CurrentControlRigs)
		{
			if (ControlRigPtr.IsValid() && ControlRigPtr->CurrentControlSelection().Num() > 0)
			{
				SelectedControlRigs.Add(ControlRigPtr.Get());
			}
		}
		if (SelectedControlRigs.Num() > 0)
		{
			return SelectedControlRigs;
		}
	}
	return NewControlRigs;
}

/* We may want to list the Controls in it (design said no but animators said yes)
void SControlRigPoseView::CreateControlList()
{
	if (PoseAsset.IsValid())
	{
		const TArray<FName>& Controls = PoseAsset.Get()->GetControlNames();
		for (const FName& ControlName : Controls)
		{
			ControlList.Add(MakeShared<FString>(ControlName.ToString()));
		}
	}
}
*/
void SControlRigPoseView::HandleControlAdded(UControlRig* ControlRig, bool bIsAdded)
{
	if (ControlRig)
	{
		if (bIsAdded)
		{
			(ControlRig)->ControlSelected().RemoveAll(this);
			(ControlRig)->ControlSelected().AddRaw(this, &SControlRigPoseView::HandleControlSelected);
		}
		else
		{
			(ControlRig)->ControlSelected().RemoveAll(this);
		}
	}
	bUpdateStatusBlocks = true;
}

void SControlRigPoseView::HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected)
{
	bUpdateStatusBlocks = true;
}

void SControlRigPoseView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bUpdateStatusBlocks)
	{
		UpdateStatusBlocks();
		bUpdateStatusBlocks = false;
	}
}

void SControlRigPoseView::UpdateStatusBlocks()
{
	FText StatusText1;
	FText StatusText2;
	TArray<UControlRig*> ControlRigs = GetControlRigs();
	if (PoseAsset.IsValid() && ControlRigs.Num() > 0)
	{

		FFormatNamedArguments NamedArgs;
		TArray<FName> ControlNames = PoseAsset->GetControlNames();
		NamedArgs.Add("Total", ControlNames.Num());
		int32 TotalSelected = 0;
		uint32 Matching = 0;
		uint32 MirrorMatching = 0;
		for (UControlRig* ControlRig : ControlRigs)
		{
			TArray<FName> SelectedNames = ControlRig->CurrentControlSelection();
			TotalSelected += SelectedNames.Num();
			for (const FName& ControlName : ControlNames)
			{
				for (const FName& SelectedName : SelectedNames)
				{
					if (SelectedName == ControlName)
					{
						++Matching;
						if (SControlRigPoseView::bIsMirror)
						{
							if (PoseAsset->DoesMirrorMatch(ControlRig, ControlName))
							{
								++MirrorMatching;
							}
						}
					}
				}
			}
		}

		NamedArgs.Add("Selected", TotalSelected);//SelectedNames.Num());
		NamedArgs.Add("Matching", Matching);
		NamedArgs.Add("MirrorMatching", MirrorMatching);

		if (SControlRigPoseView::bIsMirror)
		{
			StatusText1 = FText::Format(LOCTEXT("NumberControlsAndMatch", "{Total} Controls Matching {Matching} of {Selected} Selected"), NamedArgs);
			StatusText2 = FText::Format(LOCTEXT("NumberMirroredMatch", " {MirrorMatching} Mirror String Matches"), NamedArgs);
		}
		else
		{
			StatusText1 = FText::Format(LOCTEXT("NumberControlsAndMatch", "{Total} Controls Matching {Matching} of {Selected} Selected"), NamedArgs);
			StatusText2 = FText::GetEmpty();
		}
	}
	else
	{
		StatusText1 = FText::GetEmpty();
		StatusText2 = FText::GetEmpty();
	}
	if (TextStatusBlock1.IsValid())
	{
		TextStatusBlock1->SetText(StatusText1);
	}
	if (TextStatusBlock2.IsValid())
	{
		TextStatusBlock2->SetText(StatusText2);
	}
}

void SControlRigPoseView::OnSplitterFinishedResizing()
{
	if (PoseSplitter.IsValid())
	{
		for (int32 SlotIndex = 0; SlotIndex < 3; ++SlotIndex)
		{
			SplitterSlotValues[SlotIndex] = PoseSplitter->SlotAt(SlotIndex).GetSizeValue();
		}
	}
}

#undef LOCTEXT_NAMESPACE
