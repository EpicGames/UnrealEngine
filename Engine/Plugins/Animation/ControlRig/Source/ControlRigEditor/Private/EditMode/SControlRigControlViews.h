// Copyright Epic Games, Inc. All Rights Reserved.
/**
* Hold the Slate Views for the different Control Rig Asset Views
* This is shown in the Bottom of The SControlRigBaseListWidget
*/
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Tools/ControlRigPose.h"
#include "Misc/NotifyHook.h"

class FAssetThumbnail;
class IDetailsView;
class UControlRig;
class SControlRigBaseListWidget;

//Class to Hold Statics that are shared and externally callable
class FControlRigView
{
public:
	static void CaptureThumbnail(UObject* Asset);
};

class SControlRigPoseView : public SCompoundWidget, public FNotifyHook
{
	SLATE_BEGIN_ARGS(SControlRigPoseView) {}
	SLATE_ARGUMENT(UControlRigPoseAsset*, PoseAsset)
	SLATE_ARGUMENT(TSharedPtr<SControlRigBaseListWidget>, OwningWidget)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SControlRigPoseView();

	//FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
private:
	bool bUpdateStatusBlocks = false;
	void UpdateStatusBlocks();

	/*
	* Delegates and Helpers
	*/
	ECheckBoxState IsKeyPoseChecked() const;
	void OnKeyPoseChecked(ECheckBoxState NewState);
	ECheckBoxState IsMirrorPoseChecked() const;
	void OnMirrorPoseChecked(ECheckBoxState NewState);
	ECheckBoxState IsAdditivePoseChecked() const;
	void OnAdditivePoseChecked(ECheckBoxState NewState);
	bool IsMirrorEnabled() const;
	void OnPoseBlendChanged(float ChangedVal);
	void OnPoseBlendCommited(float ChangedVal, ETextCommit::Type Type);
	void OnBeginSliderMovement();
	void OnEndSliderMovement(float NewValue);
	float OnGetPoseBlendValue()const {return PoseBlendValue;}
	FReply OnPastePose();
	FReply OnSelectControls();
	FReply OnCaptureThumbnail();
	void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected);
	void HandleControlAdded(UControlRig* ControlRig, bool bIsAdded);
	TArray<UControlRig*> GetControlRigs(bool bIsSelected = false);

	static bool bIsKey;
	static bool bIsMirror;
	static bool bIsAdditive;
	static float bDynamicMin;
	static float bDynamicMax;
	static float SplitterSlotValues[3];

	void OnSplitterFinishedResizing();
	TSharedPtr<SSplitter> PoseSplitter;

	float PoseBlendValue;
	bool bIsBlending;
	bool bSliderStartedTransaction;
	FControlRigControlPose TempPose;
	TSharedPtr< FAssetThumbnail > Thumbnail;
	TSharedRef<SWidget> GetThumbnailWidget();

	TWeakObjectPtr<UControlRigPoseAsset> PoseAsset;
	TArray<TWeakObjectPtr<UControlRig>> CurrentControlRigs;

	TSharedPtr<STextBlock> TextStatusBlock1;
	TSharedPtr<STextBlock> TextStatusBlock2;

	TWeakPtr<SControlRigBaseListWidget> OwningWidget;

	/* Mirroring*/
	TSharedPtr<IDetailsView> MirrorDetailsView;

	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{

		return SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
			.Content()
			[
				SNew(SBox)
				.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString((*InItem)))
			]
			];
	}

public:
	static bool IsMirror() { return bIsMirror; }
};

