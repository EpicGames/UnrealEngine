// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SControlRigEditModeTools.h"

#include "AnimDetails/Proxies/AnimDetailsProxyTransform.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "ISequencer.h"
#include "PropertyHandle.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "IDetailRootObjectCustomization.h"
#include "Modules/ModuleManager.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Rigs/FKControlRig.h"
#include "EditMode/SControlRigBaseListWidget.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IControlRigEditorModule.h"
#include "Framework/Docking/TabManager.h"
#include "ControlRigEditorStyle.h"
#include "LevelEditor.h"
#include "EditorModeManager.h"
#include "InteractiveToolManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ControlRigSpaceChannelEditors.h"
#include "IKeyArea.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Transform/TransformConstraint.h"
#include "EditMode/ControlRigEditModeToolkit.h"
#include "Editor/Constraints/SConstraintsWidget.h"
#include "Models/RigSelectionViewModel.h"

#define LOCTEXT_NAMESPACE "ControlRigEditModeTools"

void SControlRigEditModeTools::Cleanup()
{
	// This is required as these hold a shared pointer to THIS OBJECT and make this class not to be destroyed when the parent class releases the shared pointer of this object
	if(FSlateApplication::IsInitialized())
	{
		if (SettingsDetailsView)
		{
			SettingsDetailsView->SetKeyframeHandler(nullptr);
		}
	}
}

void SControlRigEditModeTools::Construct(const FArguments& InArgs, const TSharedRef<UE::ControlRigEditor::FRigSelectionViewModel>& InRigViewModel)
{
	RigViewModel = InRigViewModel;
	RigViewModel->OnControlsChanged().AddSP(this, &SControlRigEditModeTools::UpdateOverridesDetailsView);
	
	// initialize settings view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = true;
		DetailsViewArgs.bCustomNameAreaLocation = true;
		DetailsViewArgs.bCustomFilterAreaLocation = true;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bShowScrollBar = false; // Don't need to show this, as we are putting it in a scroll box
	}

	SettingsDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	OverridesDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	OverridesDetailsView->OnFinishedChangingProperties().AddSP(this, &SControlRigEditModeTools::OnOverrideOptionFinishedChange);
	OverridesDetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SControlRigEditModeTools::ShouldShowOverrideProperty));
	
	ChildSlot
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SettingsDetailsView.ToSharedRef()
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SExpandableArea)
				.InitiallyCollapsed(true)
				.Visibility(this, &SControlRigEditModeTools::GetOverridesExpanderVisibility)
				.AreaTitle(LOCTEXT("OverridesWidget", "Overrides"))
				.AreaTitleFont(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				.BorderBackgroundColor(FLinearColor(.6f, .6f, .6f))
				.HeaderContent()
				[
					SNew(SHorizontalBox)

					// "Overrides" label
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, 0.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OverridesWidget", "Overrides"))
						.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
					]
				]
				.BodyContent()
				[
					OverridesDetailsView.ToSharedRef()
				]
			]
		]
	];
}

SControlRigEditModeTools::~SControlRigEditModeTools()
{
	RigViewModel->OnControlsChanged().RemoveAll(this);
}

void SControlRigEditModeTools::SetSettingsDetailsObject(const TWeakObjectPtr<>& InObject)
{
	if (SettingsDetailsView)
	{
		TArray<TWeakObjectPtr<>> Objects;

		bool bIsControlRigSettings = false;
		if (InObject.IsValid())
		{
			Objects.Add(InObject);
			bIsControlRigSettings = InObject->IsA<UControlRigEditModeSettings>();
		}

		if (bIsControlRigSettings)
		{
			// no need to override anything when as are control rig settings properties are always visible/editable
			SettingsDetailsView->SetKeyframeHandler(nullptr);
			SettingsDetailsView->SetIsPropertyVisibleDelegate(nullptr);
			SettingsDetailsView->SetIsPropertyReadOnlyDelegate(nullptr);
		}
		else
		{
			SettingsDetailsView->SetKeyframeHandler(SharedThis(this));
			SettingsDetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateStatic(&SControlRigEditModeTools::ShouldShowPropertyOnDetailCustomization));
			SettingsDetailsView->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateStatic(&SControlRigEditModeTools::IsReadOnlyPropertyOnDetailCustomization));
		}
		
		SettingsDetailsView->SetObjects(Objects);
	}
}

void SControlRigEditModeTools::SetSequencer(TWeakPtr<ISequencer> InSequencer)
{
	WeakSequencer = InSequencer.Pin();
	UpdateOverridesDetailsView();
}

bool SControlRigEditModeTools::IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
{
	if (InObjectClass && InObjectClass->IsChildOf(UAnimDetailsProxyTransform::StaticClass()))
	{
		return true;
	}
	if (InObjectClass && InObjectClass->IsChildOf(UAnimDetailsProxyTransform::StaticClass()) && InPropertyHandle.GetProperty()
		&& (InPropertyHandle.GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Location) ||
		InPropertyHandle.GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Rotation) ||
		InPropertyHandle.GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimDetailsProxyTransform, Scale)))
	{
		return true;
	}

	FCanKeyPropertyParams CanKeyPropertyParams(InObjectClass, InPropertyHandle);
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid() && Sequencer->CanKeyProperty(CanKeyPropertyParams))
	{
		return true;
	}

	return false;
}

bool SControlRigEditModeTools::IsPropertyKeyingEnabled() const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence())
	{
		return true;
	}

	return false;
}

bool SControlRigEditModeTools::IsPropertyAnimated(const IPropertyHandle& PropertyHandle, UObject *ParentObject) const
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid() && Sequencer->GetFocusedMovieSceneSequence())
	{
		constexpr bool bCreateHandleIfMissing = false;
		FGuid ObjectHandle = Sequencer->GetHandleToObject(ParentObject, bCreateHandleIfMissing);
		if (ObjectHandle.IsValid()) 
		{
			UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
			FProperty* Property = PropertyHandle.GetProperty();
			TSharedRef<FPropertyPath> PropertyPath = FPropertyPath::CreateEmpty();
			PropertyPath->AddProperty(FPropertyInfo(Property));
			FName PropertyName(*PropertyPath->ToString(TEXT(".")));
			TSubclassOf<UMovieSceneTrack> TrackClass; //use empty @todo find way to get the UMovieSceneTrack from the Property type.
			return MovieScene->FindTrack(TrackClass, ObjectHandle, PropertyName) != nullptr;
		}
	}
	return false;
}

void SControlRigEditModeTools::OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle)
{
	if (!WeakSequencer.IsValid() || !WeakSequencer.Pin()->IsAllowedToChange())
	{
		return;
	}

	TArray<UObject*> Objects;
	KeyedPropertyHandle.GetOuterObjects(Objects);
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	for (UObject *Object : Objects)
	{
		UAnimDetailsProxyBase* Proxy = Cast<UAnimDetailsProxyBase>(Object);
		if (Proxy)
	{
			Proxy->SetKey(KeyedPropertyHandle);
		}
	}
}

bool SControlRigEditModeTools::ShouldShowPropertyOnDetailCustomization(const FPropertyAndParent& InPropertyAndParent)
{
	auto ShouldPropertyBeVisible = [](const FProperty& InProperty)
	{
		// Always show settings properties
		if (InProperty.GetOwner<UClass>() == UControlRigEditModeSettings::StaticClass())
		{
			return true;
		}
		
		return InProperty.HasAnyPropertyFlags(CPF_Interp) || InProperty.HasMetaData(FRigVMStruct::InputMetaName) || InProperty.HasMetaData(FRigVMStruct::OutputMetaName);
	};

	if (InPropertyAndParent.Property.IsA<FStructProperty>())
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(&InPropertyAndParent.Property);
		for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct); PropertyIt; ++PropertyIt)
		{
			if (ShouldPropertyBeVisible(**PropertyIt))
			{
				return true;
			}
		}
	}

	return ShouldPropertyBeVisible(InPropertyAndParent.Property) || 
		(InPropertyAndParent.ParentProperties.Num() > 0 && ShouldPropertyBeVisible(*InPropertyAndParent.ParentProperties[0]));
}

bool SControlRigEditModeTools::IsReadOnlyPropertyOnDetailCustomization(const FPropertyAndParent& InPropertyAndParent)
{
	auto ShouldPropertyBeEnabled = [](const FProperty& InProperty)
	{
		// Always show settings properties
		if (InProperty.GetOwner<UClass>() == UControlRigEditModeSettings::StaticClass())
		{
			return true;
		}
		return InProperty.HasAnyPropertyFlags(CPF_Interp) || InProperty.HasMetaData(FRigVMStruct::InputMetaName);
	};

	if (InPropertyAndParent.Property.IsA<FStructProperty>())
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(&InPropertyAndParent.Property);
		for (TFieldIterator<FProperty> PropertyIt(StructProperty->Struct); PropertyIt; ++PropertyIt)
		{
			if (ShouldPropertyBeEnabled(**PropertyIt))
			{
				return false;
			}
		}
	}

	return !(ShouldPropertyBeEnabled(InPropertyAndParent.Property) || 
		(InPropertyAndParent.ParentProperties.Num() > 0 && ShouldPropertyBeEnabled(*InPropertyAndParent.ParentProperties[0])));
}


/* MZ TODO
void SControlRigEditModeTools::MakeSelectionSetDialog()
{

	FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (ControlRigEditMode)
	{
		TSharedPtr<SWindow> ExistingWindow = SelectionSetWindow.Pin();
		if (ExistingWindow.IsValid())
		{
			ExistingWindow->BringToFront();
		}
		else
		{
			ExistingWindow = SNew(SWindow)
				.Title(LOCTEXT("SelectionSets", "Selection Set"))
				.HasCloseButton(true)
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.ClientSize(FVector2D(165, 200));
			TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
			if (RootWindow.IsValid())
			{
				FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), RootWindow.ToSharedRef());
			}
			else
			{
				FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
			}

		}

		ExistingWindow->SetContent(
			SNew(SControlRigBaseListWidget)
		);
		SelectionSetWindow = ExistingWindow;
	}
}
*/

EVisibility SControlRigEditModeTools::GetOverridesExpanderVisibility() const
{
	if(CVarControlRigEnableOverrides.GetValueOnAnyThread())
	{
		if(!OverridesDetailsView->GetSelectedObjects().IsEmpty())
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void SControlRigEditModeTools::OnOverrideOptionFinishedChange(const FPropertyChangedEvent& PropertyChangedEvent)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = OverridesDetailsView->GetSelectedObjects();
	for(const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if(UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SelectedObject.Get()))
		{
			if(UControlRig* ControlRig = CRSection->GetControlRig())
			{
				TGuardValue<bool> _(CRSection->bSuspendOverrideAssetSync, true);
				ControlRig->UnlinkAllOverrideAssets();
				for(const TSoftObjectPtr<UControlRigOverrideAsset>& OverrideAssetPtr : CRSection->OverrideAssets)
				{
					if(UControlRigOverrideAsset* OverrideAsset = OverrideAssetPtr.LoadSynchronous())
					{
						ControlRig->LinkOverrideAsset(OverrideAsset);
					}
				}
				CRSection->UpdateOverrideAssetDelegates();
				CRSection->ReconstructChannelProxy();
			}
		}
	}
}

bool SControlRigEditModeTools::ShouldShowOverrideProperty(const FPropertyAndParent& InPropertyAndParent) const
{
	static const FProperty* OverrideAssetsProperty = UMovieSceneControlRigParameterSection::StaticClass()->FindPropertyByName(
		GET_MEMBER_NAME_CHECKED(UMovieSceneControlRigParameterSection, OverrideAssets));
	return OverrideAssetsProperty == &InPropertyAndParent.Property ||
		InPropertyAndParent.ParentProperties.Contains(OverrideAssetsProperty);
}

void SControlRigEditModeTools::UpdateOverridesDetailsView()
{
	TArray<TWeakObjectPtr<>> Sections;
	if(WeakSequencer.IsValid())
	{
		ISequencer* Sequencer = WeakSequencer.Pin().Get();
		for(const TWeakObjectPtr<UControlRig>& WeakControlRig : RigViewModel->GetControlRigs())
		{
			if(const UControlRig* ControlRig = WeakControlRig.Get())
			{
				if(!ControlRig->CurrentControlSelection().IsEmpty())
				{
					if(UMovieSceneControlRigParameterSection* CRSection = FControlRigSpaceChannelHelpers::GetControlRigSection(Sequencer, ControlRig))
					{
						Sections.Emplace(CRSection);
					}
				}
			}
		}
	}
	OverridesDetailsView->SetObjects(Sections);
}

#undef LOCTEXT_NAMESPACE
