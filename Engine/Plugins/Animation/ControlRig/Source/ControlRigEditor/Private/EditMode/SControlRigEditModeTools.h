// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "IDetailKeyframeHandler.h"
#include "EditMode/ControlRigEditModeUtil.h"
#include "RigVMModel/RigVMGraph.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Widgets/DeclarativeSyntaxSupport.h"


class IDetailsView;
class ISequencer;
class SRigHierarchyTreeView;
namespace UE::ControlRigEditor { class FRigSelectionViewModel; }

/** Shows UI for adding override assets to control rig when the ControlRig.Overrides cvar is enabled. */
class SControlRigEditModeTools : public SCompoundWidget, public IDetailKeyframeHandler
{
public:
	SLATE_BEGIN_ARGS(SControlRigEditModeTools) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSharedRef<UE::ControlRigEditor::FRigSelectionViewModel>& InRigViewModel);
	
	virtual ~SControlRigEditModeTools() override;
	void Cleanup();

	/** Set the objects to be displayed in the details panel */
	void SetSettingsDetailsObject(const TWeakObjectPtr<>& InObject);

	/** Set the sequencer we are bound to */
	void SetSequencer(TWeakPtr<ISequencer> InSequencer);
	
	// IDetailKeyframeHandler interface
	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const class IPropertyHandle& PropertyHandle) const override;
	virtual bool IsPropertyKeyingEnabled() const override;
	virtual void OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle) override;
	virtual bool IsPropertyAnimated(const class IPropertyHandle& PropertyHandle, UObject *ParentObject) const override;

private:

	/** Knows about the control rigs that are selected. */
	TSharedPtr<UE::ControlRigEditor::FRigSelectionViewModel> RigViewModel;

	/** Sequencer we are currently bound to */
	TWeakPtr<ISequencer> WeakSequencer;
	TSharedPtr<IDetailsView> SettingsDetailsView;

	TSharedPtr<IDetailsView> OverridesDetailsView;

	/** Display or edit set up for property */
	static bool ShouldShowPropertyOnDetailCustomization(const struct FPropertyAndParent& InPropertyAndParent);
	static bool IsReadOnlyPropertyOnDetailCustomization(const struct FPropertyAndParent& InPropertyAndParent);

	EVisibility GetOverridesExpanderVisibility() const;
	void OnOverrideOptionFinishedChange(const FPropertyChangedEvent& PropertyChangedEvent);
	bool ShouldShowOverrideProperty(const FPropertyAndParent& InPropertyAndParent) const;
	void UpdateOverridesDetailsView();
};

