// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SControlRigOutliner.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SComboButton.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "ScopedTransaction.h"
#include "ControlRig.h"
#include "ControlRigEditModeCommands.h"
#include "ControlRigHierarchyCommands.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "ISequencer.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "Editor.h"
#include "Widgets/Text/STextBlock.h"
#include "TimerManager.h"
#include "Rigs/RigHierarchyController.h"
#include "ControlRigObjectBinding.h"
#include "ModularRig.h"
#include "MovieScene.h"
#include "MovieSceneNameableTrack.h"
#include "Settings/ControlRigSettings.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor/EditorEngine.h"
#include "ScopedTransaction.h"
#include "Blueprint/WidgetBlueprintLibrary.h"
#include "Editor/Hierarchy/Views/SRigHierarchy.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyItem.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Rigs/FKControlRig.h"
#include "SEnumCombo.h"
#include "Components/SkeletalMeshComponent.h"
#include "Overrides/SOverrideStatusWidget.h"
#include "Async/TaskGraphInterfaces.h"
#include "Engine/Texture2D.h"

#define LOCTEXT_NAMESPACE "ControlRigOutliner"

FRigHierarchyTreeDisplaySettings FMultiRigTreeDelegates::DefaultDisplaySettings;


//////////////////////////////////////////////////////////////
/// FMultiRigData
///////////////////////////////////////////////////////////

uint32 GetTypeHash(const FMultiRigData& Data)
{
	return GetTypeHash(TTuple<FMultiRigData::EMultiRigDataType, FName, const UControlRig*, FRigElementKey> (
		Data.Type,
		Data.GetItemName(),
		Data.WeakControlRig.Get(),
		Data.GetElementKey())
	);
}

FText FMultiRigData::GetName() const
{
	if (IsControlElement())
	{
		return FText::FromName(Key->Name);
	}

	if (IsModule() || IsActor() || IsComponent())
	{
		return FText::FromName(GetItemName());
	}
	
	if (UControlRig* ControlRigPtr = WeakControlRig.Get())
	{
		FString ControlRigName = ControlRigPtr->GetName();
		if (UMovieSceneNameableTrack* Track = Cast<UMovieSceneNameableTrack>(ControlRigPtr->GetOuter()))
		{
			ControlRigName = Track->GetDisplayName().ToString();
		}
		
		return FText::Format(LOCTEXT("ControlTitle", "{0}"), FText::AsCultureInvariant(ControlRigName));
	}
	
	return FText::FromName(NAME_None);
}

FText FMultiRigData::GetDisplayName(const FRigHierarchyTreeDisplaySettings& InSettings) const
{
	if (!CachedDisplayName.IsSet())
	{
		if (Key.IsSet())
		{
			if (const URigHierarchy* Hierarchy = GetHierarchy())
			{
				const FRigElementKey& RigElementKey = Key.GetValue();
				const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(RigElementKey);
				EElementNameDisplayMode ElementNameDisplayMode = InSettings.NameDisplayMode;
				if(ControlElement)
				{
					// animation channels should not show their module name if they belong to a control
					// in the same module.
					if(ControlElement->Settings.AnimationType == ERigControlAnimationType::AnimationChannel)
					{
						const FRigElementKey ParentElementKey = Hierarchy->GetFirstParent(RigElementKey);
						if(Hierarchy->GetModuleFName(ParentElementKey) == Hierarchy->GetModuleFName(RigElementKey))
						{
							ElementNameDisplayMode = EElementNameDisplayMode::ForceShort;
						}
					}
				}
				
				const FText DisplayNameForUI = Hierarchy->GetDisplayNameForUI(RigElementKey, ElementNameDisplayMode);
				if (!DisplayNameForUI.IsEmpty())
				{
					CachedDisplayName = DisplayNameForUI;
				}
				else if (ControlElement)
				{
					if (!ControlElement->Settings.DisplayName.IsNone())
					{
						CachedDisplayName = FText::FromName(ControlElement->Settings.DisplayName);
					}
				}
			}
		}

		if (!CachedDisplayName.IsSet())
		{
			CachedDisplayName = GetName();
		}
	}
	
	return *CachedDisplayName;
}

void FMultiRigData::InvalidateDisplayName() const
{
	CachedDisplayName.Reset();
}

FText FMultiRigData::GetToolTipText(const FRigHierarchyTreeDisplaySettings& Settings) const
{
	if (IsControlElement())
	{
		return FText::Format(LOCTEXT("SMultiRigHierarchyItemControlTooltip", "{0}"), FText::FromName(GetElementKey().Name));
	}
	else
	{
		return FText::Format(LOCTEXT("SMultiRigHierarchyModuleControlTooltip", "{0}\n\nUse Alt+Click to select subtree."), GetDisplayName(Settings));
	}
}

bool FMultiRigData::operator== (const FMultiRigData & Other) const
{
	if (WeakControlRig == Other.WeakControlRig
		&& Type == Other.Type
		&& Key == Other.Key
		&& Name == Other.Name
		&& UniqueID == Other.UniqueID)
	{
		return true;
	}

	return false;
}

bool FMultiRigData::IsValid() const
{
	if (Type >= EMultiRigDataType_Invalid)
	{
		return false;
	}
	
	if (WeakControlRig.IsValid())
	{
		if (Key.IsSet())
		{
			return Key->IsValid();
		}
		if (Name.IsSet())
		{
			return !Name->IsNone();
		}
		return true;
	}
	
	return (IsActor() || IsComponent()) && UniqueID.IsSet();
}

URigHierarchy* FMultiRigData::GetHierarchy() const
{
	if (UControlRig* ControlRigPtr = WeakControlRig.Get())
	{
		return ControlRigPtr->GetHierarchy();
	}
	return nullptr;
}

bool FMultiRigData::IsModularRig() const
{
	if (TStrongObjectPtr<UControlRig> Rig = WeakControlRig.Pin())
	{
		return Rig->IsModularRig();
	}
	return false;
}

FRigModuleInstance* FMultiRigData::GetModuleInstance() const
{
	if (IsModularRig() && IsModule())
	{
		if (TStrongObjectPtr<UControlRig> Rig = WeakControlRig.Pin())
		{
			if (UModularRig* ModularRig = Cast<UModularRig>(Rig.Get()))
			{
				return ModularRig->FindModule(GetItemName());
			}
		}
	}
	return nullptr;
}

//////////////////////////////////////////////////////////////
/// FMultiRigTreeElement
///////////////////////////////////////////////////////////

FMultiRigTreeElement::FMultiRigTreeElement(const FMultiRigData& InData, TWeakPtr<SMultiRigHierarchyTreeView> InTreeView, ERigTreeFilterResult InFilterResult)
{
	Data = InData;
	FilterResult = InFilterResult;

	if (InTreeView.IsValid() && Data.IsValid())
	{
		const FRigHierarchyTreeDisplaySettings& Settings = InTreeView.Pin()->GetTreeDelegates().GetDisplaySettings();
		RefreshDisplaySettings(Data.GetHierarchy(), Settings);
	}
}

TSharedRef<ITableRow> FMultiRigTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMultiRigTreeElement> InRigTreeElement, TSharedPtr<SMultiRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned)
{
	return SNew(SMultiRigHierarchyItem, InOwnerTable, InRigTreeElement, InTreeView, InSettings, bPinned);

}

void FMultiRigTreeElement::RefreshDisplaySettings(const URigHierarchy* InHierarchy, const FRigHierarchyTreeDisplaySettings& InSettings)
{
	TPair<const FSlateBrush*, FSlateColor> Result;
	if (InHierarchy)
	{
		Result = SMultiRigHierarchyItem::GetBrushForElementType(InHierarchy, Data);
	}

	IconBrush = Result.Key;
	IconColor = Result.Value;

	if (FilterResult == ERigTreeFilterResult::Shown)
	{
		IconColor = IconColor.IsColorSpecified() && InSettings.bShowIconColors ? Result.Value : FSlateColor::UseForeground();
		const bool bIsKey = Data.IsControlElement();
		const bool bIsModule = Data.IsModule();
		const bool bIsSuperItem = !bIsModule && !bIsKey;
		if (bIsSuperItem || bIsModule)
		{
			TextColor = FSlateColor(FLinearColor::White);
		}
		else
		{
			TextColor = FSlateColor::UseForeground();
		}
	}
	else
	{
		IconColor = IconColor.IsColorSpecified() && InSettings.bShowIconColors ? FSlateColor(Result.Value.GetSpecifiedColor() * 0.5f) : FSlateColor(FLinearColor::Gray * 0.5f);
		TextColor = FSlateColor::UseForeground();
	}

	Data.InvalidateDisplayName();
}

bool FMultiRigTreeElement::AreControlsVisible() const
{
	switch (Data.Type)
	{
		case FMultiRigData::EMultiRigDataType_ControlRig:
		{
			if (TStrongObjectPtr<UControlRig> Rig = Data.WeakControlRig.Pin())
			{
				return Rig->GetControlsVisible();
			}
			break;
		}
		case FMultiRigData::EMultiRigDataType_Module:
		{
			if (FRigModuleInstance* Module = Data.GetModuleInstance())
			{
				if (UControlRig* ModuleRig = Module->GetRig())
				{
					return ModuleRig->GetControlsVisible();
				}
			}
			break;
		}
	}
	return false;
}

//////////////////////////////////////////////////////////////
/// SMultiRigHierarchyItem
///////////////////////////////////////////////////////////
TMap<FSoftObjectPath, TSharedPtr<FSlateBrush>> SMultiRigHierarchyItem::IconPathToBrush;
void SMultiRigHierarchyItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMultiRigTreeElement> InRigTreeElement, TSharedPtr<SMultiRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned)
{
	WeakRigTreeElement = InRigTreeElement;
	TreeView = InTreeView;
	Delegates = InTreeView->GetTreeDelegates();

	
	if (!InRigTreeElement->Data.IsValid())
	{
		SMultiColumnTableRow<TSharedPtr<FMultiRigTreeElement>>::Construct(
			SMultiColumnTableRow<TSharedPtr<FMultiRigTreeElement>>::FArguments()
			.ShowSelection(false)
			.Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
			.FillHeight(200.f)
			[
				SNew(SSpacer)
			]
			], OwnerTable);
		return;
	}

	SMultiColumnTableRow<TSharedPtr<FMultiRigTreeElement>>::Construct(
		SMultiColumnTableRow<TSharedPtr<FMultiRigTreeElement>>::FArguments()
		.ShowWires(true), OwnerTable);
}

FReply SMultiRigHierarchyItem::OnToggleVisibilityClicked()
{
	if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
	{
		TArray<TSharedPtr<FMultiRigTreeElement>> ElementsToToggle;
		ElementsToToggle.Add(Element);
		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			TArray<TSharedPtr<FMultiRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();

			// If the element toggled belongs to a selection, toggle all the modules selected
			if (SelectedItems.Contains(Element))
			{
				for (TSharedPtr<FMultiRigTreeElement> Selected : SelectedItems)
				{
					if (Selected->Data.IsModule())
					{
						ElementsToToggle.AddUnique(Selected);
					}
				}
			}
			else
			{
				// If the module toggled does not belong to a selection, toggle all submodules
				TArray<TSharedPtr<FMultiRigTreeElement>> Descendants = Element->Children;
				for (int32 i=0; i<Descendants.Num(); i++)
				{
					if (Descendants[i]->Data.IsModule())
					{
						ElementsToToggle.AddUnique(Descendants[i]);
						Descendants.Append(Descendants[i]->Children);
					}
				}
			}
		}

		TArray<UControlRig*> RigsToToggle;

		TOptional<bool> bSetControlsVisible;
		for (TSharedPtr<FMultiRigTreeElement> ElementToToggle : ElementsToToggle)
		{
			if (!ElementToToggle->Data.IsControlElement())
			{
				UControlRig* Rig = nullptr;
				if (ElementToToggle->Data.IsModule())
				{
					if (FRigModuleInstance* Module = ElementToToggle->Data.GetModuleInstance())
					{
						Rig = Module->GetRig();
					}
				}
				else
				{
					Rig  = ElementToToggle->Data.WeakControlRig.Get();
				}
			
				if (Rig)
				{
					if (!bSetControlsVisible.IsSet())
					{
						bSetControlsVisible = !Rig->GetControlsVisible();
					}

					RigsToToggle.AddUnique(Rig);
				}
			}
		}

		if (!RigsToToggle.IsEmpty())
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ToggleControlsVisibility", "Toggle Controls Visibility"), !GIsTransacting);
			for (UControlRig* Rig : RigsToToggle)
			{
				Rig->Modify();
				Rig->SetControlsVisible(bSetControlsVisible.GetValue());
			}
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();  //should flow to selection instead
}

FText SMultiRigHierarchyItem::GetDisplayName() const
{
	const FRigHierarchyTreeDisplaySettings Settings = Delegates.GetDisplaySettings();
	return (WeakRigTreeElement.Pin()->Data.GetDisplayName(Settings));
}

FText SMultiRigHierarchyItem::GetToolTipText() const
{
	const FRigHierarchyTreeDisplaySettings Settings = Delegates.GetDisplaySettings();
	return (WeakRigTreeElement.Pin()->Data.GetToolTipText(Settings));
}

TPair<const FSlateBrush*, FSlateColor> SMultiRigHierarchyItem::GetBrushForElementType(const URigHierarchy* InHierarchy, const FMultiRigData& InData)
{
	const FSlateBrush* Brush = nullptr;
	FSlateColor Color = FSlateColor::UseForeground();
	if (InData.IsControlElement())
	{
		const FRigElementKey Key = InData.GetElementKey();
		return SRigHierarchyItem::GetBrushForElementType(InHierarchy, Key);
	}

	return TPair<const FSlateBrush*, FSlateColor>(Brush, Color);
}

const FSlateBrush* SMultiRigHierarchyItem::GetBorder() const
{
	if (!TreeView.IsValid())
	{
		return STableRow<TSharedPtr<FMultiRigTreeElement>>::GetBorder(); 
	}
	
	TArray<TSharedPtr<FMultiRigTreeElement>> SelectedElements = TreeView->GetSelectedItems();
	if (SelectedElements.Contains(WeakRigTreeElement))
	{
		// item selected
		return &Style->ActiveBrush;
	}
	
	for (TSharedPtr<FMultiRigTreeElement> Selected : SelectedElements)
	{
		TSharedPtr<FMultiRigTreeElement> Cur = Selected;
		bool bAnyDescendantSelected = false;
		while (Cur)
		{
			if (Cur == WeakRigTreeElement)
			{
				bAnyDescendantSelected = true;
				break;
			}
			else
			{
				Cur = TreeView->GetParentElement(Cur);
			}
		}
		if (bAnyDescendantSelected)
		{
			// Descendant selected
			return &Style->InactiveHighlightedBrush;
		}
	}
	return STableRow<TSharedPtr<FMultiRigTreeElement>>::GetBorder();
}

TSharedRef<SWidget> SMultiRigHierarchyItem::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (InColumnName == "Visibility")
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(this, &SMultiRigHierarchyItem::OnToggleVisibilityClicked)
			.OnHovered_Lambda([this]()
			{
				if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
				{
					Element->bIsEyeballIconHovered = true;
				}
			})
			.OnUnhovered_Lambda([this]()
			{
				if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
				{
					Element->bIsEyeballIconHovered = false;
				}
			})
			[
				SNew(SImage)
				.Image_Lambda([this]() -> const FSlateBrush*
				{
					if (TSharedPtr<FMultiRigTreeElement> SharedElement = WeakRigTreeElement.Pin())
					{
						UControlRig* Rig = nullptr;
						// If we have a module name, we are dealing with a module, get the rig from there
						if (SharedElement->Data.IsModule())
						{
							if (FRigModuleInstance* Module = SharedElement->Data.GetModuleInstance())
							{
								Rig = Module->GetRig();
							}
						}
						else if (SharedElement->Data.IsControlRig())
						{
							Rig = SharedElement->Data.WeakControlRig.Get();
						}

						if (Rig)
						{
							if (Rig->GetControlsVisible())
							{
								return (FAppStyle::GetBrush("Level.VisibleIcon16x"));
							}
							else
							{
								return  (FAppStyle::GetBrush("Level.NotVisibleIcon16x"));
							}
						}
					}
					return nullptr;
				})
				.ColorAndOpacity_Lambda([this]()
				{
					if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
					{
						if (Element->AreControlsVisible() && !Element->bIsRowHovered && !Element->bIsEyeballIconHovered)
						{
							return FSlateColor(FLinearColor::Transparent);
						}
						if (Element->bIsEyeballIconHovered)
						{
							return FSlateColor::UseForeground();
						}
					}
					return FSlateColor::UseSubduedForeground();
				})
				.DesiredSizeOverride(FVector2D(16, 16))
			];
	}
	else if (InColumnName == "Name")
	{
		return SNew(SHorizontalBox)
		.ToolTipText(GetToolTipText())
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4, 2, 2, 0)
		.VAlign(VAlign_Fill)
		[
			SNew(SExpanderArrow, SharedThis(this))
			.IndentAmount(12)
			.ShouldDrawWires(true)
		]
			
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			[
				SNew(SImage)
				.Visibility_Lambda([this]() -> EVisibility
				{
					if (TSharedPtr<FMultiRigTreeElement> SharedElement = WeakRigTreeElement.Pin())
					{
						if (SharedElement->Data.IsModule() || SharedElement->Data.IsControlElement())
						{
							return EVisibility::Visible;
						}
					}
					return EVisibility::Collapsed;
				})
				.Image_Lambda([this]() -> const FSlateBrush*
				{
					if (TSharedPtr<FMultiRigTreeElement> SharedElement = WeakRigTreeElement.Pin())
					{
						// If we have a module name, we are dealing with a module, get the rig from there
						if (SharedElement->Data.IsModule())
						{
							UControlRig* Rig = nullptr;
							if (FRigModuleInstance* Module = SharedElement->Data.GetModuleInstance())
							{
								Rig = Module->GetRig();
							}
							
							if (Rig)
							{
								const FSoftObjectPath& IconPath = Rig->GetRigModuleSettings().Icon;
								const TSharedPtr<FSlateBrush>* ExistingBrush = IconPathToBrush.Find(IconPath);
								if(ExistingBrush && ExistingBrush->IsValid())
								{
									return ExistingBrush->Get();
								}
								if(UTexture2D* Icon = Cast<UTexture2D>(IconPath.TryLoad()))
								{
									const TSharedPtr<FSlateBrush> NewBrush = MakeShareable(new FSlateBrush(UWidgetBlueprintLibrary::MakeBrushFromTexture(Icon, 16.0f, 16.0f)));
									IconPathToBrush.FindOrAdd(IconPath) = NewBrush;
									return NewBrush.Get();
								}
							}
						}
					}
					return WeakRigTreeElement.Pin()->IconBrush;
				})
				.ColorAndOpacity_Lambda([this]()
				{
					return WeakRigTreeElement.Pin()->IconColor;
				})
				.DesiredSizeOverride(FVector2D(16, 16))
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.0)
		[
			SNew(SInlineEditableTextBlock)
				.Text(this, &SMultiRigHierarchyItem::GetDisplayName)
				.ToolTipText(this, &SMultiRigHierarchyItem::GetToolTipText)
				.MultiLine(false)
				.Font_Lambda([this]()
				{
					if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
					{
						if (!Element->Data.IsModule() && !Element->Data.IsControlElement())
						{
							return FCoreStyle::GetDefaultFontStyle("Bold", 10);
						}
					}
					return FCoreStyle::GetDefaultFontStyle("Regular", 10);
				})
				.ColorAndOpacity_Lambda([this]()
				{
					if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
					{
						bool bIsSelected = false;
						if (TreeView.IsValid())
						{
							bIsSelected = TreeView->GetSelectedData().Contains(Element->Data);
						}
						return bIsSelected ? FSlateColor(FLinearColor::White) : Element->TextColor;
					}
					return FSlateColor::UseForeground();
				})
		];
	}
	return SNullWidget::NullWidget;
}

void SMultiRigHierarchyItem::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) 
{
	if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
	{
		Element->bIsRowHovered = true;
	}
}

void SMultiRigHierarchyItem::OnMouseLeave(const FPointerEvent& MouseEvent) 
{
	if (TSharedPtr<FMultiRigTreeElement> Element = WeakRigTreeElement.Pin())
	{
		Element->bIsRowHovered = false;
	}
}

//////////////////////////////////////////////////////////////
/// SRigHierarchyTreeView
///////////////////////////////////////////////////////////

SMultiRigHierarchyTreeView::~SMultiRigHierarchyTreeView()
{
	UnregisterPendingRefresh();
}

void SMultiRigHierarchyTreeView::Construct(const FArguments& InArgs)
{
	Delegates = InArgs._RigTreeDelegates;

	STreeView<TSharedPtr<FMultiRigTreeElement>>::FArguments SuperArgs;
	SuperArgs.TreeItemsSource(&RootElements);
	SuperArgs.SelectionMode(ESelectionMode::Multi);
	SuperArgs.OnGenerateRow(this, &SMultiRigHierarchyTreeView::MakeTableRowWidget, false);
	SuperArgs.OnGetChildren(this, &SMultiRigHierarchyTreeView::HandleGetChildrenForTree);
	SuperArgs.HeaderRow(
		SNew(SHeaderRow)

		+ SHeaderRow::Column(FName(TEXT("Visibility")))
		.DefaultLabel(FText::GetEmpty())
		.FixedWidth(30)
		.HAlignCell(HAlign_Left)
		.HAlignHeader(HAlign_Left)
		.VAlignCell(VAlign_Top)

		+ SHeaderRow::Column(FName(TEXT("Name")))
		.DefaultLabel(LOCTEXT("ElementName", "Name"))
		.HAlignCell(HAlign_Fill)
		.HAlignHeader(HAlign_Fill)
	);

	SuperArgs.OnSelectionChanged(FOnMultiRigTreeSelectionChanged::CreateRaw(&Delegates, &FMultiRigTreeDelegates::HandleSelectionChanged));
	SuperArgs.OnContextMenuOpening(Delegates.OnContextMenuOpening);
	SuperArgs.OnMouseButtonClick(FOnMouseButtonClick::CreateSP(this, &SMultiRigHierarchyTreeView::HandleMouseClicked));
	SuperArgs.OnMouseButtonDoubleClick(Delegates.OnMouseButtonDoubleClick);
	SuperArgs.OnSetExpansionRecursive(FOnMultiRigTreeSetExpansionRecursive::CreateSP(this, &SMultiRigHierarchyTreeView::SetExpansionRecursive));
	SuperArgs.HighlightParentNodesForSelection(true);
	SuperArgs.AllowInvisibleItemSelection(true);  //without this we deselect everything when we filter or we collapse

	SuperArgs.ShouldStackHierarchyHeaders_Lambda([]() -> bool {
		return UControlRigEditorSettings::Get()->bShowStackedHierarchy;
		});
	SuperArgs.OnGeneratePinnedRow(this, &SMultiRigHierarchyTreeView::MakeTableRowWidget, true);
	SuperArgs.MaxPinnedItems_Lambda([]() -> int32
		{
			return FMath::Max<int32>(1, UControlRigEditorSettings::Get()->MaxStackSize);
		});

	STreeView<TSharedPtr<FMultiRigTreeElement>>::Construct(SuperArgs);
}

TSharedPtr<FMultiRigTreeElement> SMultiRigHierarchyTreeView::FindElement(const FMultiRigData& InElementData, TSharedPtr<FMultiRigTreeElement> CurrentItem)
{
	if (CurrentItem->Data == InElementData)
	{
		return CurrentItem;
	}

	for (int32 ChildIndex = 0; ChildIndex < CurrentItem->Children.Num(); ++ChildIndex)
	{
		TSharedPtr<FMultiRigTreeElement> Found = FindElement(InElementData, CurrentItem->Children[ChildIndex]);
		if (Found.IsValid())
		{
			return Found;
		}
	}

	return TSharedPtr<FMultiRigTreeElement>();
}

bool SMultiRigHierarchyTreeView::AddElement(const FMultiRigData& InData, const FMultiRigData& InParentData)
{
	if (ElementMap.Contains(InData))
	{
		return false;
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();

	const FString FilteredString = Settings.FilterText.ToString();
	if (FilteredString.IsEmpty() || !InData.IsValid())
	{
		TSharedPtr<FMultiRigTreeElement> NewItem = MakeShared<FMultiRigTreeElement>(InData, SharedThis(this), ERigTreeFilterResult::Shown);

		if (InData.IsValid())
		{
			ElementMap.Add(InData, NewItem);
			if (InParentData.IsValid())
			{
				ParentMap.Add(InData, InParentData);
				TSharedPtr<FMultiRigTreeElement>* FoundItem = ElementMap.Find(InParentData);
				check(FoundItem);
				FoundItem->Get()->Children.Add(NewItem);
			}
			else
			{
				RootElements.Add(NewItem);
			}
		}
		else
		{
			RootElements.Add(NewItem);
		}
	}
	else
	{
		FString FilteredStringUnderScores = FilteredString.Replace(TEXT(" "), TEXT("_"));
		if (InData.GetName().ToString().Contains(FilteredString) || InData.GetName().ToString().Contains(FilteredStringUnderScores))
		{
			TSharedPtr<FMultiRigTreeElement> NewItem = MakeShared<FMultiRigTreeElement>(InData, SharedThis(this), ERigTreeFilterResult::Shown);
			ElementMap.Add(InData, NewItem);
			RootElements.Add(NewItem);

			if (!Settings.bFlattenHierarchyOnFilter && !Settings.bHideParentsOnFilter)
			{
				if (const URigHierarchy* Hierarchy = InData.GetHierarchy())
				{
					if (InData.IsControlElement())
					{
						TSharedPtr<FMultiRigTreeElement> ChildItem = NewItem;
						FRigElementKey ParentKey = Hierarchy->GetFirstParent(InData.GetElementKey());
						while (ParentKey.IsValid())
						{
							FMultiRigData ParentData(InData.WeakControlRig.Get(), ParentKey);
							if (!ElementMap.Contains(ParentData))
							{
								TSharedPtr<FMultiRigTreeElement> ParentItem = MakeShared<FMultiRigTreeElement>(ParentData, SharedThis(this), ERigTreeFilterResult::ShownDescendant);
								ElementMap.Add(ParentData, ParentItem);
								RootElements.Add(ParentItem);

								ReparentElement(ChildItem->Data, ParentData);

								ChildItem = ParentItem;
								ParentKey = Hierarchy->GetFirstParent(ParentKey);
							}
							else
							{
								ReparentElement(ChildItem->Data, ParentData);
								break;
							}
						}
					}
				}
			}
		}
	}

	return true;
}

bool SMultiRigHierarchyTreeView::AddElement(UControlRig* InControlRig, const FRigBaseElement* InElement)
{
	check(InControlRig);
	check(InElement);
	
	FMultiRigData Data(InControlRig, InElement->GetKey());

	if (ElementMap.Contains(Data))
	{
		return false;
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();

	auto IsElementShown = [Settings](const FRigBaseElement* InElement) -> bool
	{
		switch (InElement->GetType())
		{
			case ERigElementType::Bone:
			{
				if (!Settings.bShowBones)
				{
					return false;
				}

				const FRigBoneElement* BoneElement = CastChecked<FRigBoneElement>(InElement);
				if (!Settings.bShowImportedBones && BoneElement->BoneType == ERigBoneType::Imported)
				{
					return false;
				}
				break;
			}
			case ERigElementType::Null:
			{
				if (!Settings.bShowNulls)
				{
					return false;
				}
				break;
			}
			case ERigElementType::Control:
			{
				const FRigControlElement* ControlElement = CastChecked<FRigControlElement>(InElement);
				if (!Settings.bShowControls || ControlElement->Settings.AnimationType == ERigControlAnimationType::VisualCue)
				{
					return false;
				}
				if (ControlElement->Settings.AnimationType == ERigControlAnimationType::AnimationChannel)
				{
					return false;
				}
				break;
			}
			case ERigElementType::Physics:
			{
				return false;
			}
			case ERigElementType::Reference:
			{
				if (!Settings.bShowReferences)
				{
					return false;
				}
				break;
			}
			case ERigElementType::Socket:
			{
				if (!Settings.bShowSockets)
				{
					return false;
				}
				break;
			}
			case ERigElementType::Connector:
			{
				if (!Settings.bShowConnectors)
				{
					return false;
				}
				break;
			}
			case ERigElementType::Curve:
			{
				return false;
			}
			default:
			{
				break;
			}
		}
		return true;
	};

	if(!IsElementShown(InElement))
	{
		return false;
	}

	FMultiRigData ParentData;
	ParentData.WeakControlRig = InControlRig;

	// If only showing selected modules, 
	if (InControlRig->IsModularRig() && Settings.bArrangeByModules)
	{
		if (Settings.OutlinerDisplayMode == EMultiRigTreeDisplayMode::SelectedModules)
		{
			if (const URigHierarchy* Hierarchy = InControlRig->GetHierarchy())
			{
				ParentData.SetItemName(FMultiRigData::EMultiRigDataType_Module, *Hierarchy->GetModuleName(InElement->GetKey()));
				if (!ElementMap.Contains(ParentData))
				{
					return false;
				}
			}
		}
	}

	if (!AddElement(Data, ParentData))
	{
		return false;
	}

	UFKControlRig* FKControlRig = Cast<UFKControlRig>(InControlRig);

	if (ElementMap.Contains(Data))
	{
		if (const URigHierarchy* Hierarchy = InControlRig->GetHierarchy())
		{
			if (InControlRig->IsModularRig() && Settings.bArrangeByModules)
			{
				ParentData.SetItemName(FMultiRigData::EMultiRigDataType_Module, *Hierarchy->GetModuleName(InElement->GetKey()));

				if (TSharedPtr<FMultiRigTreeElement>* ParentElementPtr = ElementMap.Find(ParentData))
				{
					if (Settings.FilterText.IsEmpty() || !Settings.bFlattenHierarchyOnFilter)
					{
						if (ReparentElement(Data, ParentData))
						{
							// Move any rig element to be inserted before any other module
							{
								TSharedPtr<FMultiRigTreeElement> Parent = *ParentElementPtr;
								int32 InsertIndex = INDEX_NONE;
								for (int32 i=Parent->Children.Num()-2; i >= 0; i--)
								{
									if (!Parent->Children[i]->Data.IsModule())
									{
										InsertIndex = i;
										break;
									}
								}
								if (InsertIndex+1 != Parent->Children.Num() - 1)
								{
									TSharedPtr<FMultiRigTreeElement> Element = Parent->Children.Last();
									Parent->Children.RemoveAt(Parent->Children.Num() - 1);
									Parent->Children.Insert(Element, InsertIndex+1);
								}
							}
						}
					}
				}
			}
			else
			{
				FRigElementKey ParentKey = Hierarchy->GetFirstParent(InElement->GetKey());
				if (FKControlRig && 
					ParentKey != URigHierarchy::GetWorldSpaceReferenceKey())
				{
					// Find the parent key for FK Control Rigs
					TArray<FRigElementWeight> ParentWeights = Hierarchy->GetParentWeightArray(InElement->GetKey());
					if (ParentWeights.Num() > 0)
					{
						TArray<FRigElementKey> ParentKeys = Hierarchy->GetParents(InElement->GetKey());
						check(ParentKeys.Num() == ParentWeights.Num());
						for (int32 ParentIndex = 0; ParentIndex < ParentKeys.Num(); ParentIndex++)
						{
							if (ParentWeights[ParentIndex].IsAlmostZero())
							{
								continue;
							}
							ParentKey = ParentKeys[ParentIndex];
							break;
						}
					}

					if (const FRigControlElement* ControlElement = Cast<FRigControlElement>(InElement))
					{
						if (ControlElement->Settings.AnimationType == ERigControlAnimationType::AnimationControl &&
							(ParentKey.Type == ERigElementType::Bone || ParentKey.Type == ERigElementType::Curve))
						{
							const FRigElementKey& ElementKey = InElement->GetKey();
							const FName BoneName = FKControlRig->GetControlTargetName(ElementKey.Name, ParentKey.Type);
							const FRigElementKey ParentBoneKey = Hierarchy->GetFirstParent(FRigElementKey(BoneName, ERigElementType::Bone));
							if (ParentBoneKey.IsValid())
							{
								ParentKey = FRigElementKey(FKControlRig->GetControlName(ParentBoneKey.Name, ParentKey.Type), ElementKey.Type);
							}
						}
					}
				}
				else
				{
					// Find the parent key for Control Rigs
					const TFunction<FRigElementKey(const FRigBaseElement*)> GetParentInTreeRecursive =
						[&](const FRigBaseElement* Element) -> FRigElementKey
						{
							if (FRigBaseElementParentArray Parents = Hierarchy->GetActiveParents(Element); Parents.Num() > 0)
							{
								const FRigBaseElement* ParentElement = Parents[0];
								const bool bIsVisibleInOutliner = Algo::AnyOf(ElementMap,
									[ParentElement, InControlRig](const TTuple<FMultiRigData, TSharedPtr<FMultiRigTreeElement>>& DataToElementPair)
									{
										return
											DataToElementPair.Key.WeakControlRig == InControlRig &&
											DataToElementPair.Key.GetElementKey() == ParentElement->GetKey();
									});

								if (ParentElement &&
									bIsVisibleInOutliner)
								{
									return ParentElement->GetKey();
								}
								else
								{
									return GetParentInTreeRecursive(ParentElement);
								}
							}

							return FRigElementKey();
						};

					ParentKey = GetParentInTreeRecursive(InElement);
				}

				// Add or reparent the element
				if (ParentKey.IsValid())
				{
					if (const FRigBaseElement* ParentElement = Hierarchy->Find(ParentKey))
					{
						AddElement(InControlRig,ParentElement);

						FMultiRigData NewParentData(InControlRig, ParentKey);
						if (ElementMap.Contains(NewParentData))
						{
							ReparentElement(Data, NewParentData);
						}
					}
				}
			}
		}
	}

	return true;
}


bool SMultiRigHierarchyTreeView::ReparentElement(const FMultiRigData& InData, const FMultiRigData& InParentData)
{
	if (!InData.IsValid() || InData == InParentData)
	{
		return false;
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();

	TSharedPtr<FMultiRigTreeElement>* FoundItem = ElementMap.Find(InData);
	if (FoundItem == nullptr)
	{
		return false;
	}

	if (!Settings.FilterText.IsEmpty() && Settings.bFlattenHierarchyOnFilter)
	{
		return false;
	}

	if (const FMultiRigData* ExistingParentKey = ParentMap.Find(InData))
	{
		if (*ExistingParentKey == InParentData)
		{
			return false;
		}

		if (TSharedPtr<FMultiRigTreeElement>* ExistingParent = ElementMap.Find(*ExistingParentKey))
		{
			(*ExistingParent)->Children.Remove(*FoundItem);
		}

		ParentMap.Remove(InData);
	}
	else
	{
		if (!InParentData.IsValid())
		{
			return false;
		}

		RootElements.Remove(*FoundItem);
	}

	if (InParentData.IsValid())
	{
		ParentMap.Add(InData, InParentData);

		TSharedPtr<FMultiRigTreeElement>* FoundParent = ElementMap.Find(InParentData);
		check(FoundParent);
		FoundParent->Get()->Children.Add(*FoundItem);
	}
	else
	{
		RootElements.Add(*FoundItem);
	}

	return true;
}

bool SMultiRigHierarchyTreeView::RemoveElement(const FMultiRigData& InData)
{
	TSharedPtr<FMultiRigTreeElement>* FoundItem = ElementMap.Find(InData);
	if (FoundItem == nullptr)
	{
		return false;
	}

	FMultiRigData EmptyParent(nullptr, FRigElementKey());
	ReparentElement(InData, EmptyParent);

	RootElements.Remove(*FoundItem);
	return ElementMap.Remove(InData) > 0;
}

TSharedPtr<FMultiRigTreeElement> SMultiRigHierarchyTreeView::GetParentElement(TSharedPtr<FMultiRigTreeElement> InElement) const
{
	if (!InElement.IsValid())
	{
		return nullptr;
	}

	if (const FMultiRigData* Parent = ParentMap.Find(InElement->Data))
	{
		if (const TSharedPtr<FMultiRigTreeElement>* ParentElement = ElementMap.Find(*Parent))
		{
			return *ParentElement;
		}
	}

	return nullptr;
}

void SMultiRigHierarchyTreeView::ScrollToElement(const FMultiRigData& InElement)
{
	if (bRefreshPending)
	{
		ScrollToElementAfterRefresh = InElement;
	}
	else
	{
		if (TSharedPtr<FMultiRigTreeElement>* Element = ElementMap.Find(InElement))
		{
			// Expand parents
			TSharedPtr<FMultiRigTreeElement> Parent = GetParentElement(*Element);
			while (Parent)
			{
				SetItemExpansion(Parent, true);
				Parent = GetParentElement(Parent);
			}
			
			RequestScrollIntoView(*Element);
		}
	}
}

void SMultiRigHierarchyTreeView::RequestTreeViewRefresh()
{
	if (bRefreshPending)
	{
		return;
	}
	
	UnregisterPendingRefresh();

	bRefreshPending = true;
	
	PendingTreeViewRefreshHandle = RegisterActiveTimer( 0.f,
		FWidgetActiveTimerDelegate::CreateLambda([this](double, float)
		{
			RefreshTreeView();
			return EActiveTimerReturnType::Stop;
		})
	);
}

void SMultiRigHierarchyTreeView::UnregisterPendingRefresh()
{
	if (PendingTreeViewRefreshHandle.IsValid())
	{
		if (TSharedPtr<FActiveTimerHandle> ActiveTimerHandle = PendingTreeViewRefreshHandle.Pin())
		{
			UnRegisterActiveTimer(ActiveTimerHandle.ToSharedRef());
		}
		PendingTreeViewRefreshHandle.Reset();
	}
}

void SMultiRigHierarchyTreeView::RefreshTreeView()
{
	bRefreshPending = false;
	
	// store expansion state
	TMap<FMultiRigData, bool> ExpansionState;
	for (TPair<FMultiRigData, TSharedPtr<FMultiRigTreeElement>> Pair : ElementMap)
	{
		ExpansionState.FindOrAdd(Pair.Key) = IsItemExpanded(Pair.Value);
	}

	// internally save expansion states before rebuilding the tree, so the states can be restored later
	SaveAndClearSparseItemInfos();

	RootElements.Reset();
	ElementMap.Reset();
	ParentMap.Reset();

	// rebuild elements
	static FMultiRigData EmptyParentData(nullptr, FRigElementKey());

	FControlRigEditMode* EditMode = Delegates.GetEditMode();
	const TWeakPtr<ISequencer>& WeakSequencer = EditMode->GetWeakSequencer();
	
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();

	TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
	TArray<FRigModuleInstance*> SelectedModules;
	if (Settings.OutlinerDisplayMode != EMultiRigTreeDisplayMode::All)
	{
		EditMode->GetAllSelectedControls(SelectedControls);
		if (Settings.OutlinerDisplayMode == EMultiRigTreeDisplayMode::SelectedModules)
		{
			// If showing current module, always expand
			ExpansionState.Reset();
			
			for (TTuple<UControlRig*, TArray<FRigElementKey>> RigControls : SelectedControls)
			{
				if (UModularRig* ModularRig = Cast<UModularRig>(RigControls.Key))
				{
					URigHierarchy* Hierarchy = RigControls.Key->GetHierarchy();
					for (FRigElementKey& Control : RigControls.Value)
					{
						// Do not change selected modules because of animation channels or visual cues
						if (FRigControlElement* ControlElement = Cast<FRigControlElement>(Hierarchy->Find(Control)))
						{
							const ERigControlAnimationType& Type = ControlElement->Settings.AnimationType;
							if (Type == ERigControlAnimationType::AnimationChannel || Type == ERigControlAnimationType::VisualCue)
							{
								continue;
							}
						}
						
						FString ModuleName = Hierarchy->GetModuleName(Control);
						if (FRigModuleInstance* Module = ModularRig->FindModule(*ModuleName))
						{
							SelectedModules.AddUnique(Module);
						}
					}
				}
			}
		}
	}

	auto GetAscestorName = [Sequencer](UObject* Object, FString& InOutName)
	{
		if (Sequencer && Object)
		{
			FGuid Id = Sequencer->FindCachedObjectId(*Object, Sequencer->GetFocusedTemplateID());
			if (UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
			{
				InOutName = MovieScene->GetObjectDisplayName(Id).ToString();
			}
		}
	};
	
	for (const TWeakObjectPtr<UControlRig>& ControlRigPtr : ControlRigs)
	{
		if (UControlRig* ControlRig  = ControlRigPtr.Get())
		{
			if (Settings.OutlinerDisplayMode != EMultiRigTreeDisplayMode::All)
			{
				if (!SelectedControls.Contains(ControlRig))
				{
					continue;
				}
			}
			
			if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRigPtr->GetObjectBinding())
			{
				TArray<FMultiRigData> DataHierarchy;
				UObject* ParentObject = ObjectBinding->GetBoundObject();
				while (ParentObject)
				{
					if (USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(ParentObject))
					{
						if (!Component->HasAnyFlags(RF_DefaultSubObject))
						{
							FMultiRigData Data;

							FString ComponentName = Component->GetName();
							GetAscestorName(Component, ComponentName);
							
							Data.SetItemName(FMultiRigData::EMultiRigDataType_Component, *ComponentName);
							Data.SetUniqueID(Component->GetUniqueID());
							DataHierarchy.Add(Data);
						}
					}
					else if (AActor* Actor = Cast<AActor>(ParentObject))
					{
						FMultiRigData Data;

						FString ActorName = Actor->GetActorLabel();
						GetAscestorName(Actor, ActorName);
	
						Data.SetItemName(FMultiRigData::EMultiRigDataType_Actor, *ActorName);
						Data.SetUniqueID(Actor->GetUniqueID());
						DataHierarchy.Add(Data);
					}
					ParentObject = ParentObject->GetOuter();
				}

				FMultiRigData* ParentData = &EmptyParentData;
				for (int32 i=DataHierarchy.Num()-1; i>=0; --i)
				{
					AddElement(DataHierarchy[i], *ParentData);
					ParentData = &DataHierarchy[i];
				}

				FMultiRigData CRData;
				CRData.Type = FMultiRigData::EMultiRigDataType_ControlRig;
				CRData.WeakControlRig = ControlRig; //leave key unset so it's valid

				// add root element
				AddElement(CRData, *ParentData);

				if (ControlRig->IsModularRig() && Settings.bArrangeByModules)
				{
					if (UModularRig* ModularRig = Cast<UModularRig>(ControlRig))
					{
						ModularRig->ForEachModule([&](FRigModuleInstance* Module)
						{
							if (Settings.OutlinerDisplayMode == EMultiRigTreeDisplayMode::SelectedModules)
							{
								if (!SelectedModules.Contains(Module))
								{
									return true;
								}
							}
							FMultiRigData ParentData;
							ParentData.WeakControlRig = ControlRig;
							if (Settings.bFlattenModules || Module->ParentModuleName.IsNone() ||
								Settings.OutlinerDisplayMode == EMultiRigTreeDisplayMode::SelectedModules)
							{
								ParentData = CRData;
							}
							else
							{
								ParentData.SetItemName(FMultiRigData::EMultiRigDataType_Module, Module->ParentModuleName);
							}

							FMultiRigData NewData;
							NewData.WeakControlRig = ControlRig;
							NewData.SetItemName(FMultiRigData::EMultiRigDataType_Module, Module->Name);
							AddElement(NewData, ParentData);
							return true;
						});
					}
				}

				// add children
				if (const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
				{
					constexpr bool bTowardsChildren = true;
					constexpr bool bIngoreInactiveRelations = true;

					Hierarchy->Traverse([&](FRigBaseElement* Element, bool& bContinue)
					{
						AddElement(ControlRig, Element);
						bContinue = true;
					}, bTowardsChildren, bIngoreInactiveRelations);
				}

				// remove empty modules
				if (ControlRig->IsModularRig() && Settings.bArrangeByModules)
				{
					TArray<FMultiRigData> ToRemove;
					for (const TPair<FMultiRigData, TSharedPtr<FMultiRigTreeElement>>& Pair : ElementMap)
					{
						if (Pair.Key.IsModule() && Pair.Value->Children.IsEmpty())
						{
							ToRemove.Add(Pair.Key);
						}
					}
				
					for (int32 i=0; i<ToRemove.Num(); i++)
					{
						FMultiRigData Parent;
						if (FMultiRigData* ParentDataPtr = ParentMap.Find(ToRemove[i]))
						{
							Parent = *ParentDataPtr;
						}
					
						RemoveElement(ToRemove[i]);

						// If the parent has no children, remove it
						if (Parent.IsModule())
						{
							if (TSharedPtr<FMultiRigTreeElement>* ParentElement = ElementMap.Find(Parent))
							{
								if ((*ParentElement)->Children.IsEmpty())
								{
									ToRemove.AddUnique(Parent);
								}
							}
						}
					}
				}
			}
		}
	}

	// expand all elements upon the initial construction of the tree
	if (ExpansionState.Num() == 0)
	{
		for (TSharedPtr<FMultiRigTreeElement> RootElement : RootElements)
		{
			SetExpansionRecursive(RootElement, false, true);
		}
	}
	else if (ExpansionState.Num() < ElementMap.Num())
	{
		for (const TPair<FMultiRigData, TSharedPtr<FMultiRigTreeElement>>& Element : ElementMap)
		{
			if (!ExpansionState.Contains(Element.Key))
			{
				SetItemExpansion(Element.Value, true);
			}
		}
	}

	// restore infos	
	for (const auto& Pair : ElementMap)
	{
		RestoreSparseItemInfos(Pair.Value);
	}

	RequestTreeRefresh();

	// update selection
	
	TMap<UControlRig*, TArray<FRigElementKey>> RigAndSelection;
	for (const TWeakObjectPtr<UControlRig>& ControlRigPtr : ControlRigs)
	{
		if (UControlRig* ControlRig = ControlRigPtr.Get())
		{
			if (const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
			{
				TArray<FRigElementKey> Selection = Hierarchy->GetSelectedKeys();
				if (!Selection.IsEmpty())
				{
					RigAndSelection.Emplace(ControlRig, MoveTemp(Selection));
				}
			}
		}
	}

	TGuardValue<bool> Guard(Delegates.bIsChangingRigHierarchy, true);
	ClearSelection();

	for (const TPair<UControlRig*, TArray<FRigElementKey>>& RigAndKeys : RigAndSelection)
	{
		UControlRig* ControlRig = RigAndKeys.Key;
		
		// look for the root item referencing this rig
		TSharedPtr<FMultiRigTreeElement> RootElement = nullptr;
		for (TSharedPtr<FMultiRigTreeElement> Element : RootElements)
		{
			if (!RootElement.IsValid())
			{
				TArray<TSharedPtr<FMultiRigTreeElement>> Descendants = Element->Children;
				for (int32 i=0; i<Descendants.Num(); ++i)
				{
					TSharedPtr<FMultiRigTreeElement>& Child = Descendants[i];
					if (Child->Data.IsControlRig())
					{
						if (Child->Data.WeakControlRig == ControlRig)
						{
							RootElement = Child;
							break;
						}
					}
					else
					{
						// Only add descendants of super elements (like actors or components)
						Descendants.Append(Child->Children);
					}
				}
			}
			
			if (RootElement.IsValid())
			{
				break;
			}
		}

		if (RootElement)
		{
			// look for the child item referencing this key
			for (const FRigElementKey& Key : RigAndKeys.Value)
			{
				const FMultiRigData Data(ControlRig, Key);
				const TSharedPtr<FMultiRigTreeElement> Found = FindElement(Data, RootElement);
				if (Found.IsValid())
				{
					SetItemSelection(Found, true, ESelectInfo::OnNavigation);
				}
			}
		}
		else 
		{
			// otherwise, iterate thru the elements if there's a filter as the root might have been skipped because of it  
			const FString FilteredString = Settings.FilterText.ToString();
			if (!FilteredString.IsEmpty())
			{
				for (const FRigElementKey& Key : RigAndKeys.Value)
				{
					const FMultiRigData Data(ControlRig, Key);
					TSharedPtr<FMultiRigTreeElement>* Found = ElementMap.Find(Data);
					if (Found && Found->IsValid())
					{
						SetItemSelection(*Found, true, ESelectInfo::OnNavigation);
					}
				}
			}
		}
	}

	if (ScrollToElementAfterRefresh.IsValid())
	{
		ScrollToElement(ScrollToElementAfterRefresh);
		ScrollToElementAfterRefresh.Reset();
	}
}

void SMultiRigHierarchyTreeView::SetExpansionRecursive(TSharedPtr<FMultiRigTreeElement> InElement, bool bShouldBeExpanded)
{
	SetExpansionRecursive(InElement, false, bShouldBeExpanded);
}

void SMultiRigHierarchyTreeView::SetExpansionRecursive(TSharedPtr<FMultiRigTreeElement> InElement, bool bTowardsParent,
	bool bShouldBeExpanded)
{
	SetItemExpansion(InElement, bShouldBeExpanded);

	if (bTowardsParent)
	{
		if (const FMultiRigData* ParentKey = ParentMap.Find(InElement->Data))
		{
			if (TSharedPtr<FMultiRigTreeElement>* ParentItem = ElementMap.Find(*ParentKey))
			{
				SetExpansionRecursive(*ParentItem, bTowardsParent, bShouldBeExpanded);
			}
		}
	}
	else
	{
		for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
		{
			SetExpansionRecursive(InElement->Children[ChildIndex], bTowardsParent, bShouldBeExpanded);
		}
	}
}

TSharedRef<ITableRow> SMultiRigHierarchyTreeView::MakeTableRowWidget(TSharedPtr<FMultiRigTreeElement> InItem,
	const TSharedRef<STableViewBase>& OwnerTable, bool bPinned)
{
	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();
	return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this), Settings, bPinned);
}

void SMultiRigHierarchyTreeView::HandleGetChildrenForTree(TSharedPtr<FMultiRigTreeElement> InItem,
	TArray<TSharedPtr<FMultiRigTreeElement>>& OutChildren)
{
	OutChildren = InItem.Get()->Children;
}

TSharedPtr<FMultiRigTreeElement> SMultiRigHierarchyTreeView::FindElement(const FMultiRigData& InData)
{
	if (TSharedPtr<FMultiRigTreeElement>* Element = ElementMap.Find(InData))
	{
		return *Element;
	}
	return nullptr;
}

void SMultiRigHierarchyTreeView::HandleMouseClicked(TSharedPtr<FMultiRigTreeElement> InElement)
{
	// When alt+clicking an element that is already selected, the actual selection is not changed, so HandleSelectionChanged is not called
	// However, we want to handle this case to select the subtree of that element
	if (FSlateApplication::Get().GetModifierKeys().IsAltDown())
	{
		Delegates.HandleSelectionChanged(InElement, ESelectInfo::Type::OnMouseClick);
	}
}

TArray<FMultiRigData> SMultiRigHierarchyTreeView::GetSelectedData() const
{
	TArray<FMultiRigData> Keys;
	TArray<TSharedPtr<FMultiRigTreeElement>> SelectedElements = GetSelectedItems();
	for (const TSharedPtr<FMultiRigTreeElement>& SelectedElement : SelectedElements)
	{
		if (SelectedElement)
		{
			Keys.Add(SelectedElement->Data);
		}
	}
	return Keys;
}

TArray<URigHierarchy*> SMultiRigHierarchyTreeView::GetHierarchy() const
{
	TArray<URigHierarchy*> RigHierarchy;
	for (const TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
	{
		if (ControlRig.IsValid())
		{
			RigHierarchy.Add(ControlRig.Get()->GetHierarchy());
		}
	}
	return RigHierarchy;
}

void SMultiRigHierarchyTreeView::SetControlRigs(const TArrayView<TWeakObjectPtr<UControlRig>>& InControlRigs)
{
	ControlRigs.SetNum(0);
	for (TWeakObjectPtr<UControlRig>& ControlRig : InControlRigs)
	{
		if (ControlRig.IsValid())
		{
			ControlRigs.AddUnique(ControlRig);
		}
	}
	
	RequestTreeViewRefresh();
}

//////////////////////////////////////////////////////////////
/// SSearchableMultiRigHierarchyTreeView
///////////////////////////////////////////////////////////

void SSearchableMultiRigHierarchyTreeView::Construct(const FArguments& InArgs)
{
	FMultiRigTreeDelegates TreeDelegates = InArgs._RigTreeDelegates;
	SuperGetRigTreeDisplaySettings = TreeDelegates.OnGetDisplaySettings;
	GetEditMode = TreeDelegates.OnGetEditMode;

	CommandList = MakeShared<FUICommandList>();
	BindCommands();

	TreeDelegates.OnGetDisplaySettings.BindSP(this, &SSearchableMultiRigHierarchyTreeView::GetDisplaySettings);
	GetMutableDefault<UControlRigEditorSettings>()->OnSettingChanged().AddSP(this, &SSearchableMultiRigHierarchyTreeView::OnSettingChanged);

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
			
					+SHorizontalBox::Slot()
					.Padding(0.0f, 0.0f)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(SComboButton)
					   .ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
					   .ForegroundColor(FSlateColor::UseStyle())
					   .ToolTipText(LOCTEXT("OptionsToolTip", "Open the Options Menu ."))
					   .OnGetMenuContent(this, &SSearchableMultiRigHierarchyTreeView::OnGetOptionsMenu)
					   .ContentPadding(FMargin(1, 0))
					   .ButtonContent()
					   [
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
							.ColorAndOpacity(FSlateColor::UseForeground())
					   ]
					]
					+SHorizontalBox::Slot()
					[
						SNew(SSearchBox)
						.InitialText(InArgs._InitialFilterText)
						.OnTextChanged(this, &SSearchableMultiRigHierarchyTreeView::OnFilterTextChanged)
					]
				]

			+ SVerticalBox::Slot()
				.FillHeight(1.f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f)
				[
					SNew(SBorder)
					.Padding(2.0f)
					.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
					[
						SAssignNew(TreeView, SMultiRigHierarchyTreeView)
						.RigTreeDelegates(TreeDelegates)
					]
				]
		];
}

const FRigHierarchyTreeDisplaySettings& SSearchableMultiRigHierarchyTreeView::GetDisplaySettings()
{
	if (SuperGetRigTreeDisplaySettings.IsBound())
	{
		Settings = SuperGetRigTreeDisplaySettings.Execute();
	}
	Settings.FilterText = FilterText;
	Settings.bArrangeByModules = UControlRigEditorSettings::Get()->bArrangeByModules;
	Settings.bFlattenModules = UControlRigEditorSettings::Get()->bFlattenModules;
	Settings.bFocusOnSelection = UControlRigEditorSettings::Get()->bFocusOnSelection;
	Settings.NameDisplayMode = UControlRigEditorSettings::Get()->ElementNameDisplayMode;
	Settings.OutlinerDisplayMode = UControlRigEditorSettings::Get()->OutlinerMultiRigDisplayMode;
	return Settings;
}

TSharedRef<SWidget> SSearchableMultiRigHierarchyTreeView::OnGetOptionsMenu()
{
	
	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.BeginSection("FilterOptions", LOCTEXT("FilterOptions", "Filter Options"));
	{
		const FControlRigHierarchyCommands& Actions = FControlRigHierarchyCommands::Get();
		const FControlRigEditModeCommands& EditModeActions = FControlRigEditModeCommands::Get();
		MenuBuilder.AddMenuEntry(Actions.ArrangeByModules);
		MenuBuilder.AddMenuEntry(Actions.FlattenModules);
		
		MenuBuilder.AddWidget(
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3)
			[
				SNew(SEnumComboBox, StaticEnum<EMultiRigTreeDisplayMode>())
				.CurrentValue_Lambda([this]() -> int32
				{
					return static_cast<int32>(GetMultiRigMode());
				})
				.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type)
				{
					SetMultiRigMode(static_cast<EMultiRigTreeDisplayMode>(InEnumValue));
				})
			]
			,
			LOCTEXT("MultiRigMode", "MultiRig Display Mode")
		);

		MenuBuilder.AddWidget(
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0)
				[
					SNew(SEnumComboBox, StaticEnum<EElementNameDisplayMode>())
					.CurrentValue_Lambda([this]() -> int32
					{
						return static_cast<int32>(GetElementNameDisplayMode());
					})
					.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type)
					{
						SetElementNameDisplayMode(static_cast<EElementNameDisplayMode>(InEnumValue));
					})
				]
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0, 0, 0)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(SOverrideStatusWidget)
					.Status_Lambda([this]() -> EOverrideWidgetStatus::Type
					{
						return GetElementNameDisplayMode() == EElementNameDisplayMode::AssetDefault ?
							EOverrideWidgetStatus::None : EOverrideWidgetStatus::ChangedHere;
					})
					.MenuContent_Lambda([this]() -> TSharedRef<SWidget>
					{
						if(GetElementNameDisplayMode() != EElementNameDisplayMode::AssetDefault)
						{
							FMenuBuilder MenuBuilder(true, nullptr);
							MenuBuilder.AddMenuEntry(
								LOCTEXT("RemoveOverride", "Remove Override"),
								LOCTEXT("RemoveElementNameOverrideTooltip", "Removes the override from the element name mode and uses the AssetDefault option."),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([this]()
									{
										SetElementNameDisplayMode(EElementNameDisplayMode::AssetDefault);
									})
								)
							);
							return MenuBuilder.MakeWidget();
						}
						return SNullWidget::NullWidget;
					})
				]
			]
			,
			LOCTEXT("ElementNameDisplayMode", "Name Mode")
		);
		
		MenuBuilder.AddMenuEntry(Actions.FocusOnSelection);
		MenuBuilder.AddMenuEntry(EditModeActions.ToggleModuleManipulators);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SSearchableMultiRigHierarchyTreeView::BindCommands()
{
	const FControlRigHierarchyCommands& Commands = FControlRigHierarchyCommands::Get();

	CommandList->MapAction(
		Commands.ArrangeByModules,
		FExecuteAction::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::ToggleArrangeByModules),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::IsArrangedByModules));
		
	CommandList->MapAction(
		Commands.FlattenModules,
		FExecuteAction::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::ToggleFlattenModules),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::IsShowingFlatModules));

	CommandList->MapAction(
		Commands.FocusOnSelection,
		FExecuteAction::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::ToggleFocusOnSelection),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::IsFocusingOnSelection));
	
	CommandList->MapAction(
		FControlRigEditModeCommands::Get().ToggleModuleManipulators,
		FExecuteAction::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::ToggleModuleManipulators),
		FCanExecuteAction::CreateSP(this, &SSearchableMultiRigHierarchyTreeView::CanToggleModuleManipulators));
}

void SSearchableMultiRigHierarchyTreeView::OnSettingChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	TreeView->RequestTreeViewRefresh();
}

bool SSearchableMultiRigHierarchyTreeView::IsArrangedByModules() const
{
	return UControlRigEditorSettings::Get()->bArrangeByModules;
}

void SSearchableMultiRigHierarchyTreeView::ToggleArrangeByModules()
{
	UControlRigEditorSettings* EditorSettings = UControlRigEditorSettings::Get();
	EditorSettings->bArrangeByModules = !EditorSettings->bArrangeByModules;

	if (FProperty* Property = UControlRigEditorSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, bArrangeByModules)))
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		EditorSettings->PostEditChangeProperty(PropertyChangedEvent);
	}
	TreeView->RequestTreeViewRefresh();
}

bool SSearchableMultiRigHierarchyTreeView::IsShowingFlatModules() const
{
	return UControlRigEditorSettings::Get()->bFlattenModules;
}

void SSearchableMultiRigHierarchyTreeView::ToggleFlattenModules()
{
	UControlRigEditorSettings* EditorSettings = UControlRigEditorSettings::Get();
	EditorSettings->bFlattenModules = !EditorSettings->bFlattenModules;

	if (FProperty* Property = UControlRigEditorSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, bFlattenModules)))
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		EditorSettings->PostEditChangeProperty(PropertyChangedEvent);
	}
	TreeView->RequestTreeViewRefresh();
}

EElementNameDisplayMode SSearchableMultiRigHierarchyTreeView::GetElementNameDisplayMode() const
{
	UControlRigEditorSettings* EditorSettings = UControlRigEditorSettings::Get();
	return EditorSettings->ElementNameDisplayMode;
}

void SSearchableMultiRigHierarchyTreeView::SetElementNameDisplayMode(EElementNameDisplayMode InElementNameDisplayMode)
{
	UControlRigEditorSettings* EditorSettings = UControlRigEditorSettings::Get();
	EditorSettings->ElementNameDisplayMode = InElementNameDisplayMode;

	if (FProperty* Property = UControlRigEditorSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, ElementNameDisplayMode)))
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		EditorSettings->PostEditChangeProperty(PropertyChangedEvent);
	}
	TreeView->RequestTreeViewRefresh();
}

void SSearchableMultiRigHierarchyTreeView::ToggleModuleManipulators()
{
	if (FControlRigEditMode* EditMode = GetEditMode.Execute())
	{
		EditMode->ToggleModuleManipulators();
	}
}

bool SSearchableMultiRigHierarchyTreeView::CanToggleModuleManipulators() const
{
	const TArray<FMultiRigData> Selection = TreeView->GetSelectedData();
	for (const FMultiRigData& Data : Selection)
	{
		const UControlRig* ControlRig = Data.WeakControlRig.Get();
		if (ControlRig && ControlRig->IsModularRig())
		{
			return true;
		}
	}
	return false;
}

bool SSearchableMultiRigHierarchyTreeView::IsFocusingOnSelection() const
{
	return UControlRigEditorSettings::Get()->bFocusOnSelection;
}

void SSearchableMultiRigHierarchyTreeView::ToggleFocusOnSelection()
{
	UControlRigEditorSettings* EditorSettings = UControlRigEditorSettings::Get();
	EditorSettings->bFocusOnSelection = !EditorSettings->bFocusOnSelection;

	if (FProperty* Property = UControlRigEditorSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, bFocusOnSelection)))
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		EditorSettings->PostEditChangeProperty(PropertyChangedEvent);
	}
}

void SSearchableMultiRigHierarchyTreeView::SetMultiRigMode(const EMultiRigTreeDisplayMode InMode)
{
	UControlRigEditorSettings* EditorSettings = UControlRigEditorSettings::Get();
	EditorSettings->OutlinerMultiRigDisplayMode = InMode;

	if (FProperty* Property = UControlRigEditorSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, bFocusOnSelection)))
	{
		FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
		EditorSettings->PostEditChangeProperty(PropertyChangedEvent);
	}

	GetTreeView()->RequestTreeViewRefresh();
}

bool SSearchableMultiRigHierarchyTreeView::IsMultiRigMode(const EMultiRigTreeDisplayMode InMode)
{
	return UControlRigEditorSettings::Get()->OutlinerMultiRigDisplayMode == InMode;
}

EMultiRigTreeDisplayMode SSearchableMultiRigHierarchyTreeView::GetMultiRigMode()
{
	return UControlRigEditorSettings::Get()->OutlinerMultiRigDisplayMode;
}

void SSearchableMultiRigHierarchyTreeView::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;
	GetTreeView()->RequestTreeViewRefresh();
}

//////////////////////////////////////////////////////////////
/// SControlRigOutliner
///////////////////////////////////////////////////////////

void SControlRigOutliner::Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode)
{
	bIsChangingRigHierarchy = false;

	DisplaySettings.bShowBones = false;
	DisplaySettings.bShowControls = true;
	DisplaySettings.bShowNulls = false;
	DisplaySettings.bShowReferences = false;
	DisplaySettings.bShowSockets = false;
	DisplaySettings.bShowComponents = false;
	DisplaySettings.bHideParentsOnFilter = true;
	DisplaySettings.bFlattenHierarchyOnFilter = true;
	DisplaySettings.bShowConnectors = false;
	DisplaySettings.bArrangeByModules = true;
	DisplaySettings.bFlattenModules = false;
	DisplaySettings.bFocusOnSelection = true;
	DisplaySettings.NameDisplayMode = EElementNameDisplayMode::AssetDefault;
	DisplaySettings.OutlinerDisplayMode = EMultiRigTreeDisplayMode::All;

	FMultiRigTreeDelegates RigTreeDelegates;
	RigTreeDelegates.OnGetDisplaySettings = FOnGetRigTreeDisplaySettings::CreateSP(this, &SControlRigOutliner::GetDisplaySettings);
	RigTreeDelegates.OnSelectionChanged = FOnMultiRigTreeSelectionChanged::CreateSP(this, &SControlRigOutliner::HandleSelectionChanged);
	RigTreeDelegates.OnGetEditMode = FOnMultiRigTreeGetEditMode::CreateSP(this, &SControlRigOutliner::GetEditMode);
	
	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(HierarchyTreeView, SSearchableMultiRigHierarchyTreeView)
				.RigTreeDelegates(RigTreeDelegates)
			]
		];
	SetEditMode(InEditMode);

	if (TSharedPtr<ISequencer> Sequencer = InEditMode.GetWeakSequencer().Pin())
	{
		Sequencer->OnMovieSceneDataChanged().AddSP(this, &SControlRigOutliner::OnSequencerTreeViewChanged);
	}
}

void SControlRigOutliner::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	//if there's a control rig recreate the tree, controls may have changed
	bool bNewControlRig = false;
	for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
	{
		if(Pair.Key && Pair.Value)
		{
			if (Pair.Key->IsA<UControlRig>() && Pair.Value->IsA<UControlRig>())
			{
				bNewControlRig = false;
				break;
			}
		}
	}
	if (bNewControlRig)
	{
		HierarchyTreeView->GetTreeView()->RequestTreeViewRefresh();
	}
}

void SControlRigOutliner::OnSequencerTreeViewChanged(EMovieSceneDataChangeType MovieSceneDataChange)
{
	if (MovieSceneDataChange == EMovieSceneDataChangeType::MovieSceneStructureItemAdded ||
		MovieSceneDataChange == EMovieSceneDataChangeType::MovieSceneStructureItemRemoved ||
		MovieSceneDataChange == EMovieSceneDataChangeType::MovieSceneStructureItemsChanged)
	{
		HierarchyTreeView->GetTreeView()->RequestTreeViewRefresh();
	}
}

SControlRigOutliner::SControlRigOutliner()
{
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &SControlRigOutliner::OnObjectsReplaced);
}

SControlRigOutliner::~SControlRigOutliner()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	for(TWeakObjectPtr<UControlRig>& ControlRig: BoundControlRigs)
	{ 
		if (ControlRig.IsValid())
		{
			ControlRig.Get()->ControlRigBound().RemoveAll(this);
			ControlRig.Get()->OnPostConstruction_AnyThread().RemoveAll(this);
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig.Get()->GetObjectBinding();
			if (Binding)
			{
				Binding->OnControlRigBind().RemoveAll(this);
			}
			if(URigHierarchy* Hierarchy = ControlRig.Get()->GetHierarchy())
			{
				Hierarchy->OnModified().RemoveAll(this);
			}
		}
	}
	BoundControlRigs.SetNum(0);
}

void SControlRigOutliner::HandleControlSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected)
{
	FControlRigBaseDockableView::HandleControlSelected(Subject, ControlElement, bSelected);

	if (ControlElement->IsAnimationChannel())
	{
		// This control is not present in the outliner, so no need to handle
		return;
	}

	if (bIsChangingRigHierarchy)
	{
		// This action was initiated by the outliner
		return;
	}
	const EMultiRigTreeDisplayMode Mode = HierarchyTreeView->GetDisplaySettings().OutlinerDisplayMode;
	if (Mode != EMultiRigTreeDisplayMode::All)
	{
		if (!bSelected)
		{
			HierarchyTreeView->GetTreeView()->RequestTreeViewRefresh();
			return;
		}
		
		FMultiRigData RigData;
		RigData.WeakControlRig = Subject;
		RigData.Type = FMultiRigData::EMultiRigDataType_ControlRig;
		TSharedPtr<FMultiRigTreeElement> Element = HierarchyTreeView->GetTreeView()->FindElement(RigData);
		if (!Element.IsValid())
		{
			HierarchyTreeView->GetTreeView()->RequestTreeViewRefresh();
		}
		else if (Mode == EMultiRigTreeDisplayMode::SelectedModules)
		{
			if (UModularRig* ModularRig = Cast<UModularRig>(Subject))
			{
				FMultiRigData ModuleData;
				ModuleData.WeakControlRig = Subject;
				ModuleData.SetItemName(FMultiRigData::EMultiRigDataType_Module, *ModularRig->GetHierarchy()->GetModuleName(ControlElement->GetKey()));
				TSharedPtr<FMultiRigTreeElement> ModuleElement = HierarchyTreeView->GetTreeView()->FindElement(ModuleData);
				if (!ModuleElement.IsValid())
				{
					HierarchyTreeView->GetTreeView()->RequestTreeViewRefresh();
				}
			}
		}
	}
	
	const FRigElementKey Key = ControlElement->GetKey();
	FMultiRigData Data(Subject, Key);

	bool bScrollToElement = UControlRigEditorSettings::Get()->bFocusOnSelection;
	
	TSharedPtr<FMultiRigTreeElement> Found = HierarchyTreeView->GetTreeView()->FindElement(Data);
	if (Found.IsValid())
	{
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		HierarchyTreeView->GetTreeView()->SetItemSelection(Found, bSelected, ESelectInfo::Direct);
	}
	
	if (bSelected && bScrollToElement)
	{
		HierarchyTreeView->GetTreeView()->ScrollToElement(Data);
	}
}

void SControlRigOutliner::HandleRigVisibilityChanged(TArray<UControlRig*> InControlRigs)
{
	FControlRigBaseDockableView::HandleRigVisibilityChanged(InControlRigs);

	// If only modules or super items are selected, we might have to toggle the visibility ourselves (instead of relying on the edit mode)
	TArray<FMultiRigData> Selected = HierarchyTreeView->GetTreeView()->GetSelectedData();
	
	for (const FMultiRigData& Data : Selected)
	{
		if (Data.IsModule())
		{
			if (FRigModuleInstance* Module = Data.GetModuleInstance())
			{
				if (!InControlRigs.Contains(Module->GetRig()))
				{
					Module->GetRig()->ToggleControlsVisible();
				}
			}
		}
	}
}

void SControlRigOutliner::HandleHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy,
	const FRigNotificationSubject& InSubject)
{
	if(InHierarchy == nullptr)
	{
		return;
	}
	
	UControlRig* ControlRig = InHierarchy->GetTypedOuter<UControlRig>();
	if(ControlRig == nullptr)
	{
		return;
	}
	
	if(InNotification == ERigHierarchyNotification::ControlSettingChanged)
	{
		if(const FRigControlElement* ControlElement = Cast<FRigControlElement>(InSubject.Element))
		{
			const FRigElementKey Key = ControlElement->GetKey();
			const FMultiRigData Data(ControlRig, Key);

			for (int32 RootIndex = 0; RootIndex < HierarchyTreeView->GetTreeView()->GetRootElements().Num(); ++RootIndex)
			{
				TSharedPtr<FMultiRigTreeElement> Found = HierarchyTreeView->GetTreeView()->FindElement(Data, HierarchyTreeView->GetTreeView()->GetRootElements()[RootIndex]);
				if (Found.IsValid())
				{
					Found->RefreshDisplaySettings(InHierarchy, GetDisplaySettings());
				}
			}
		}
	}
}

void SControlRigOutliner::HandleSelectionChanged(TSharedPtr<FMultiRigTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}

	const TArray<FMultiRigData> NewSelection = HierarchyTreeView->GetTreeView()->GetSelectedData();
	TMap<UControlRig*, TArray<FRigElementKey>> SelectedRigAndKeys;
	TArray<TSharedPtr<FMultiRigTreeElement>> AddToSelection;
	auto SelectRigDescendants = [this, &SelectedRigAndKeys, &AddToSelection](const FMultiRigData& Data)
	{
		if (!Data.IsControlElement())
		{
			TArray<FRigElementKey>& Keys = SelectedRigAndKeys.FindOrAdd(Data.WeakControlRig.Get());
			const FRigHierarchyTreeDisplaySettings& Settings = HierarchyTreeView->GetDisplaySettings();
			if (Settings.FilterText.IsEmpty() || !Settings.bFlattenHierarchyOnFilter)
			{
				TSharedPtr<FMultiRigTreeElement> Element = HierarchyTreeView->GetTreeView()->FindElement(Data);
				TArray<TSharedPtr<FMultiRigTreeElement>> Descendants = Element->Children;
				for (int32 i=0; i<Descendants.Num(); ++i)
				{
					TSharedPtr<FMultiRigTreeElement>& Child = Descendants[i];
					if (Child->Data.IsControlElement())
					{
						Keys.AddUnique(Child->Data.GetElementKey());
					}
						
					AddToSelection.AddUnique(Child);
					Descendants.Append(Child->Children);
				}
			}
			else if (Data.IsModule())
			{
				// If we have flatten the hierarchy due to a search, we cannot rely on the children of the module
				Data.GetHierarchy()->ForEach<FRigControlElement>([&](FRigControlElement* ControlElement)
				{
					if (Data.GetHierarchy()->GetModuleName(ControlElement->GetKey()) == Data.GetItemName())
					{
						Keys.AddUnique(ControlElement->GetKey());

						FMultiRigData ChildData(Data.WeakControlRig.Get(), ControlElement->GetKey());
						TSharedPtr<FMultiRigTreeElement> ChildElement = HierarchyTreeView->GetTreeView()->FindElement(ChildData);
						AddToSelection.AddUnique(ChildElement);
					}
					return true;
				});
			}
			else
			{
				// If the key is not set, and the module name is not set, we are selecting the root of the rig. Select all controls
				for (const FRigElementKey ControlKey : Data.GetHierarchy()->GetControlKeys())
				{
					Keys.AddUnique(ControlKey);

					FMultiRigData ChildData(Data.WeakControlRig.Get(), ControlKey);
					TSharedPtr<FMultiRigTreeElement> ChildElement = HierarchyTreeView->GetTreeView()->FindElement(ChildData);
					AddToSelection.AddUnique(ChildElement);
				}
			}
		}
	};
	
	
	for (const FMultiRigData& Data : NewSelection)
	{
		if (Data.IsControlElement() && Data.IsValid())
		{
			SelectedRigAndKeys.FindOrAdd(Data.WeakControlRig.Get()).AddUnique(Data.GetElementKey());
		}

		if (FSlateApplication::Get().GetModifierKeys().IsAltDown())
		{
			if (Data.IsActor() || Data.IsComponent())
			{
				TSharedPtr<FMultiRigTreeElement> Element = HierarchyTreeView->GetTreeView()->FindElement(Data);
				TArray<TSharedPtr<FMultiRigTreeElement>> Descendants = Element->Children;
				for (int32 i=0; i<Descendants.Num(); ++i)
				{
					TSharedPtr<FMultiRigTreeElement> Child = Descendants[i];
					if (Child->Data.IsControlRig())
					{
						SelectRigDescendants(Child->Data);
					}
					else
					{
						Descendants.Append(Child->Children);
					}
					AddToSelection.AddUnique(Child);
				}
			}
			else
			{
				SelectRigDescendants(Data);
			}
		}
	}

	TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
	
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName));
	bool bEndTransaction = false;
	if (GEditor && !GIsTransacting && EditMode && EditMode->IsInLevelEditor())
	{
		GEditor->BeginTransaction(LOCTEXT("SelectControl", "Select Control"));
		bEndTransaction = true;
	}

	if (!AddToSelection.IsEmpty())
	{
		TArray<TSharedPtr<FMultiRigTreeElement>> NewSelectedElements = HierarchyTreeView->GetTreeView()->GetSelectedItems();
		NewSelectedElements.Append(AddToSelection);
		HierarchyTreeView->GetTreeView()->SetItemSelection(AddToSelection, true);
	}
	
	const bool bSetupUndo = bEndTransaction;
	//due to how Sequencer Tree View will redo selection on next tick if we aren't keeping or toggling selection we need to clear it out
	if (FSlateApplication::Get().GetModifierKeys().IsShiftDown() == false || FSlateApplication::Get().GetModifierKeys().IsControlDown() == false)
	{
		if (EditMode)
		{		
			TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
			EditMode->GetAllSelectedControls(SelectedControls);
			for (TPair<UControlRig*, TArray<FRigElementKey>>& CurrentSelection : SelectedControls)
			{
				if (CurrentSelection.Key)
				{
					CurrentSelection.Key->ClearControlSelection(bSetupUndo);
				}
			}
			if (GEditor)
			{
				// Replicating the UEditorEngine::HandleSelectCommand, without the transaction to avoid ensure(!GIsTransacting)
				GEditor->SelectNone(true, true);
				GEditor->RedrawLevelEditingViewports();
			}
			const TWeakPtr<ISequencer>& WeakSequencer = EditMode->GetWeakSequencer();
			//also need to clear explicitly in sequencer
			if (WeakSequencer.IsValid())
			{
				if (ISequencer* SequencerPtr = WeakSequencer.Pin().Get())
				{
					SequencerPtr->GetViewModel()->GetSelection()->Empty();
				}
			}
		}
	}

	for(TPair<UControlRig*, TArray<FRigElementKey>>& RigAndKeys: SelectedRigAndKeys)
	{ 
		const URigHierarchy* Hierarchy = RigAndKeys.Key->GetHierarchy();
		if (Hierarchy)
		{
			URigHierarchyController* Controller = ((URigHierarchy*)Hierarchy)->GetController(true);
			check(Controller);
			Controller->SetSelection(RigAndKeys.Value, false, bSetupUndo);
		}
	}
	if (bEndTransaction)
	{
		GEditor->EndTransaction();
	}
}

void SControlRigOutliner::SetEditMode(FControlRigEditMode& InEditMode)
{
	FControlRigBaseDockableView::SetEditMode(InEditMode);
	ModeTools = InEditMode.GetModeManager();
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TArrayView<TWeakObjectPtr<UControlRig>> ControlRigs = EditMode->GetControlRigs();
		for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
		{
			if (ControlRig.IsValid())
			{
				if (!ControlRig.Get()->ControlRigBound().IsBoundToObject(this))
				{
					ControlRig.Get()->ControlRigBound().AddRaw(this, &SControlRigOutliner::HandleOnControlRigBound);
					BoundControlRigs.Add(ControlRig);
				}
				if(!ControlRig.Get()->OnPostConstruction_AnyThread().IsBoundToObject(this))
				{
					ControlRig.Get()->OnPostConstruction_AnyThread().AddRaw(this, &SControlRigOutliner::HandlePostConstruction);
				}
				const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig.Get()->GetObjectBinding();
				if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
				{
					Binding->OnControlRigBind().AddRaw(this, &SControlRigOutliner::HandleOnObjectBoundToControlRig);
				}
				if(URigHierarchy* Hierarchy = ControlRig.Get()->GetHierarchy())
				{
					Hierarchy->OnModified().AddRaw(this, &SControlRigOutliner::HandleHierarchyModified);
				}
			}
		}
		HierarchyTreeView->GetTreeView()->SetControlRigs(ControlRigs); //will refresh tree
	}
}

void SControlRigOutliner::HandleControlAdded(UControlRig* ControlRig, bool bIsAdded)
{
	FControlRigBaseDockableView::HandleControlAdded(ControlRig, bIsAdded);
	if (ControlRig)
	{
		if (bIsAdded == true )
		{
			if (!ControlRig->ControlRigBound().IsBoundToObject(this))
			{
				ControlRig->ControlRigBound().AddRaw(this, &SControlRigOutliner::HandleOnControlRigBound);
				BoundControlRigs.Add(ControlRig);
			}
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
			{
				Binding->OnControlRigBind().AddRaw(this, &SControlRigOutliner::HandleOnObjectBoundToControlRig);
			}
			if(!ControlRig->OnPostConstruction_AnyThread().IsBoundToObject(this))
			{
				ControlRig->OnPostConstruction_AnyThread().AddRaw(this, &SControlRigOutliner::HandlePostConstruction);
			}
		}
		else
		{
			BoundControlRigs.Remove(ControlRig);
			ControlRig->ControlRigBound().RemoveAll(this);
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			if (Binding)
			{
				Binding->OnControlRigBind().RemoveAll(this);
			}
			if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
			{
				Hierarchy->OnModified().RemoveAll(this);
			}
			ControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
		}
	}
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TArrayView<TWeakObjectPtr<UControlRig>> ControlRigs = EditMode->GetControlRigs();
		HierarchyTreeView->GetTreeView()->SetControlRigs(ControlRigs); //will refresh tree
	}
}

void SControlRigOutliner::HandleOnControlRigBound(UControlRig* InControlRig)
{
	if (!InControlRig)
	{
		return;
	}

	const TSharedPtr<IControlRigObjectBinding> Binding = InControlRig->GetObjectBinding();

	if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
	{
		Binding->OnControlRigBind().AddRaw(this, &SControlRigOutliner::HandleOnObjectBoundToControlRig);
	}
}


void SControlRigOutliner::HandleOnObjectBoundToControlRig(UObject* InObject)
{
	//just refresh the views, but do so on next tick since with FK control rig's the controls aren't set up
	//until AFTER we are bound.
	TWeakPtr<SControlRigOutliner> WeakPtr = StaticCastSharedRef<SControlRigOutliner>(AsShared()).ToWeakPtr();
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakPtr]()
	{
		if (WeakPtr.IsValid())
		{
			TSharedPtr<SControlRigOutliner> StrongThis = WeakPtr.Pin();
			if(StrongThis)
			{
				if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(StrongThis->ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
				{
					TArrayView<TWeakObjectPtr<UControlRig>> ControlRigs = EditMode->GetControlRigs();
					if(StrongThis->HierarchyTreeView)
					{
						StrongThis->HierarchyTreeView->GetTreeView()->SetControlRigs(ControlRigs); //will refresh tree
					}
				}
			}
		}
	});
}

void SControlRigOutliner::HandlePostConstruction(UControlRig* InControlRig, const FName& InEventName)
{
	// rely on the code above to refresh the views
	HandleOnObjectBoundToControlRig(nullptr);
}

#undef LOCTEXT_NAMESPACE
