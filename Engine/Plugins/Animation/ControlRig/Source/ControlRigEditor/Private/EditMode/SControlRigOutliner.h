// Copyright Epic Games, Inc. All Rights Reserved.
/**
* View for holding ControlRig Animation Outliner
*/
#pragma once

#include "CoreMinimal.h"
#include "EditMode/ControlRigBaseDockableView.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "ControlRig.h"
#include "ISequencer.h"
#include "Rigs/RigHierarchy.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Widgets/SBoxPanel.h"

#define UE_API CONTROLRIGEDITOR_API


class ISequencer;
class SExpandableArea;
class SRigHierarchySearchableTreeView;

class SMultiRigHierarchyTreeView;
class SMultiRigHierarchyItem;
class FMultiRigTreeElement;
class FMultiRigData;

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnMultiRigTreeCompareKeys, const FMultiRigData& /*A*/, const FMultiRigData& /*B*/);
DECLARE_DELEGATE_RetVal(FControlRigEditMode*, FOnMultiRigTreeGetEditMode);

typedef STreeView<TSharedPtr<FMultiRigTreeElement>>::FOnSelectionChanged FOnMultiRigTreeSelectionChanged;
typedef STreeView<TSharedPtr<FMultiRigTreeElement>>::FOnMouseButtonClick FOnMultiRigTreeMouseButtonClick;
typedef STreeView<TSharedPtr<FMultiRigTreeElement>>::FOnMouseButtonDoubleClick FOnMultiRigTreeMouseButtonDoubleClick;
typedef STreeView<TSharedPtr<FMultiRigTreeElement>>::FOnSetExpansionRecursive FOnMultiRigTreeSetExpansionRecursive;

uint32 GetTypeHash(const FMultiRigData& Data);

struct FMultiRigTreeDelegates
{
	FOnGetRigTreeDisplaySettings OnGetDisplaySettings;
	FOnMultiRigTreeSelectionChanged OnSelectionChanged;
	FOnContextMenuOpening OnContextMenuOpening;
	FOnMultiRigTreeMouseButtonClick OnMouseButtonClick;
	FOnMultiRigTreeMouseButtonDoubleClick OnMouseButtonDoubleClick;
	FOnRigTreeCompareKeys OnCompareKeys;
	FOnMultiRigTreeGetEditMode OnGetEditMode;
	

	FMultiRigTreeDelegates()
	{
		bIsChangingRigHierarchy = false;
	}


	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings() const
	{
		if (OnGetDisplaySettings.IsBound())
		{
			return OnGetDisplaySettings.Execute();
		}
		return DefaultDisplaySettings;
	}

	FControlRigEditMode* GetEditMode() const
	{
		if (OnGetEditMode.IsBound())
		{
			return OnGetEditMode.Execute();
		}
		return nullptr;
	}

	void HandleSelectionChanged(TSharedPtr<FMultiRigTreeElement> Selection, ESelectInfo::Type SelectInfo)
	{
		if (bIsChangingRigHierarchy)
		{
			return;
		}
		TGuardValue<bool> Guard(bIsChangingRigHierarchy, true);
		OnSelectionChanged.ExecuteIfBound(Selection, SelectInfo);
	}

	static UE_API FRigHierarchyTreeDisplaySettings DefaultDisplaySettings;
	bool bIsChangingRigHierarchy;
};

/** Data for the tree*/
class FMultiRigData
{

public:
	enum EMultiRigDataType : uint8
	{
		EMultiRigDataType_ControlRig,
		EMultiRigDataType_Element,
		EMultiRigDataType_Module,
		EMultiRigDataType_Component,
		EMultiRigDataType_Actor,

		EMultiRigDataType_MAX,
		EMultiRigDataType_Invalid = EMultiRigDataType_MAX,
	};
	
	FMultiRigData() { Type = EMultiRigDataType_ControlRig; };
	FMultiRigData(UControlRig* InControlRig, FRigElementKey InKey) : WeakControlRig(InControlRig), Type(EMultiRigDataType_Element), Key(InKey) {};
	FMultiRigData(UControlRig* InControlRig, EMultiRigDataType InType, FName InName) : WeakControlRig(InControlRig), Type(InType), Name(InName) {};
	FText GetName() const;
	FText GetDisplayName(const FRigHierarchyTreeDisplaySettings& DisplaySettings) const;
	void InvalidateDisplayName() const;
	FText GetToolTipText(const FRigHierarchyTreeDisplaySettings& Settings) const;
	bool operator == (const FMultiRigData & Other) const;
	bool IsValid() const;
	void Reset() { Type = EMultiRigDataType_Invalid; }
	URigHierarchy* GetHierarchy() const;
	bool IsModularRig() const;
	FRigModuleInstance* GetModuleInstance() const;

	bool IsControlElement() const { return Type == EMultiRigDataType_Element && Key.IsSet() && Key.GetValue().IsValid(); }
	bool IsModule() const { return Type == EMultiRigDataType_Module && Name.IsSet() && !Name.GetValue().IsNone(); }
	bool IsControlRig() const { return Type == EMultiRigDataType_ControlRig && WeakControlRig.IsValid(); }
	bool IsActor() const { return Type == EMultiRigDataType_Actor && !Name->IsNone(); }
	bool IsComponent() const { return Type == EMultiRigDataType_Component && !Name->IsNone(); }

	FRigElementKey GetElementKey() const { return Key.IsSet() ? Key.GetValue() : FRigElementKey(); }
	FName GetItemName() const { return Name.IsSet() ? Name.GetValue() : NAME_None; }
	
	void SetElementKey(const FRigElementKey& InKey) { Type = EMultiRigDataType_Element; Key = InKey; }
	void SetItemName(EMultiRigDataType InType, const FName& InName) { Type = InType; Name = InName; }
	void SetUniqueID(const uint32 InID) { UniqueID = InID; }
	
	TWeakObjectPtr<UControlRig> WeakControlRig;
	EMultiRigDataType Type;

private:
	
	TOptional<FName> Name;
	TOptional<FRigElementKey> Key;
	TOptional<uint32> UniqueID;
	
	mutable TOptional<FText> CachedDisplayName;
};

/** An item in the tree */
class FMultiRigTreeElement : public TSharedFromThis<FMultiRigTreeElement>
{
public:
	FMultiRigTreeElement(const FMultiRigData& InData, TWeakPtr<SMultiRigHierarchyTreeView> InTreeView,ERigTreeFilterResult InFilterResult);

	/** Element Data to display */
	FMultiRigData Data;
	TArray<TSharedPtr<FMultiRigTreeElement>> Children;

	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMultiRigTreeElement> InRigTreeElement, TSharedPtr<SMultiRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned);

	void RefreshDisplaySettings(const URigHierarchy* InHierarchy, const FRigHierarchyTreeDisplaySettings& InSettings);
	bool AreControlsVisible() const;

	/** The current filter result */
	ERigTreeFilterResult FilterResult;

	/** The brush to use when rendering an icon */
	const FSlateBrush* IconBrush;

	/** The color to use when rendering an icon */
	FSlateColor IconColor;

	/** The color to use when rendering the label text */
	FSlateColor TextColor;

	/** Whether or not this row is being hovered */
	bool bIsRowHovered = false;

	/** Whether or not the eyeball icon is being hovered */
	bool bIsEyeballIconHovered = false;

};

class SMultiRigHierarchyItem : public SMultiColumnTableRow<TSharedPtr<FMultiRigTreeElement>>
{
public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMultiRigTreeElement> InRigTreeElement, TSharedPtr<SMultiRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned);
	static TPair<const FSlateBrush*, FSlateColor> GetBrushForElementType(const URigHierarchy* InHierarchy, const FMultiRigData& InData);
	virtual const FSlateBrush* GetBorder() const override;
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& InColumnName ) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

private:
	TWeakPtr<FMultiRigTreeElement> WeakRigTreeElement;
	TSharedPtr<SMultiRigHierarchyTreeView> TreeView;
	FMultiRigTreeDelegates Delegates;

	FText GetDisplayName() const;
	FText GetToolTipText() const;
	FReply OnToggleVisibilityClicked();

	static TMap<FSoftObjectPath, TSharedPtr<FSlateBrush>> IconPathToBrush;

};

class SMultiRigHierarchyTreeView : public STreeView<TSharedPtr<FMultiRigTreeElement>>
{
public:

	SLATE_BEGIN_ARGS(SMultiRigHierarchyTreeView) {}
	SLATE_ARGUMENT(FMultiRigTreeDelegates, RigTreeDelegates)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SMultiRigHierarchyTreeView();

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		FReply Reply = STreeView<TSharedPtr<FMultiRigTreeElement>>::OnFocusReceived(MyGeometry, InFocusEvent);
		return Reply;
	}

	/** Save a snapshot of the internal map that tracks item expansion before tree reconstruction */
	void SaveAndClearSparseItemInfos()
	{
		// Only save the info if there is something to save (do not overwrite info with an empty map)
		if (!SparseItemInfos.IsEmpty())
		{
			OldSparseItemInfos = SparseItemInfos;
		}
		ClearExpandedItems();
	}

	/** Restore the expansion infos map from the saved snapshot after tree reconstruction */
	void RestoreSparseItemInfos(TSharedPtr<FMultiRigTreeElement> ItemPtr)
	{
		for (const auto& Pair : OldSparseItemInfos)
		{
			if (Pair.Key->Data == ItemPtr->Data)
			{
				// the SparseItemInfos now reference the new element, but keep the same expansion state
				SparseItemInfos.Add(ItemPtr, Pair.Value);
				break;
			}
		}
	}

	static TSharedPtr<FMultiRigTreeElement> FindElement(const FMultiRigData& InData, TSharedPtr<FMultiRigTreeElement> CurrentItem);

	bool AddElement(const FMultiRigData& InData, const FMultiRigData& InParentData);
	bool AddElement(UControlRig* InControlRig, const FRigBaseElement* InElement);
	bool ReparentElement(const FMultiRigData& InData, const FMultiRigData& InParentData);
	bool RemoveElement(const FMultiRigData& InData);
	void SetExpansionRecursive(TSharedPtr<FMultiRigTreeElement> InElement, bool bShouldBeExpanded);
	void SetExpansionRecursive(TSharedPtr<FMultiRigTreeElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);
	TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FMultiRigTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bPinned);
	void HandleGetChildrenForTree(TSharedPtr<FMultiRigTreeElement> InItem, TArray<TSharedPtr<FMultiRigTreeElement>>& OutChildren);
	TSharedPtr<FMultiRigTreeElement> FindElement(const FMultiRigData& InData);
	void HandleMouseClicked(TSharedPtr<FMultiRigTreeElement> InElement);

	TArray<FMultiRigData> GetSelectedData() const;
	const TArray<TSharedPtr<FMultiRigTreeElement>>& GetRootElements() const { return RootElements; }
	FMultiRigTreeDelegates& GetTreeDelegates() { return Delegates; }

	TArray<URigHierarchy*> GetHierarchy() const;
	void SetControlRigs(const TArrayView<TWeakObjectPtr<UControlRig>>& InControlRigs);

	/** Requests a tree view refresh. The actual refresh will be done lazily. */
	void RequestTreeViewRefresh();

	TSharedPtr<FMultiRigTreeElement> GetParentElement(TSharedPtr<FMultiRigTreeElement> InElement) const;

	void ScrollToElement(const FMultiRigData& InElement);
	
private:

	/** A temporary snapshot of the SparseItemInfos in STreeView, used during RefreshTreeView() */
	TSparseItemMap OldSparseItemInfos;

	/** Backing array for tree view */
	TArray<TSharedPtr<FMultiRigTreeElement>> RootElements;

	/** A map for looking up items based on their key */
	TMap<FMultiRigData, TSharedPtr<FMultiRigTreeElement>> ElementMap;

	/** A map for looking up a parent based on their key */
	TMap<FMultiRigData, FMultiRigData> ParentMap;

	FMultiRigTreeDelegates Delegates;

	friend class SRigHierarchy;

	TArray <TWeakObjectPtr<UControlRig>> ControlRigs;

	/** Pending function to refresh the tree view. */
	TWeakPtr<FActiveTimerHandle> PendingTreeViewRefreshHandle;
	bool bRefreshPending = false;
	FMultiRigData ScrollToElementAfterRefresh;
	void UnregisterPendingRefresh();
	void RefreshTreeView();
};

class SSearchableMultiRigHierarchyTreeView : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSearchableMultiRigHierarchyTreeView)
	{}
		SLATE_ARGUMENT(FMultiRigTreeDelegates, RigTreeDelegates)
		SLATE_ARGUMENT(FText, InitialFilterText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SSearchableMultiRigHierarchyTreeView() = default;
	TSharedRef<SMultiRigHierarchyTreeView> GetTreeView() const { return TreeView.ToSharedRef(); }
	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings();
	TSharedRef<SWidget> OnGetOptionsMenu();
	void BindCommands();
	void OnSettingChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	bool IsArrangedByModules() const;
	void ToggleArrangeByModules();
	bool IsShowingFlatModules() const;
	void ToggleFlattenModules();
	EElementNameDisplayMode GetElementNameDisplayMode() const;
	void SetElementNameDisplayMode(EElementNameDisplayMode InElementNameDisplayMode);
	void ToggleModuleManipulators();
	bool CanToggleModuleManipulators() const;
	bool IsFocusingOnSelection() const;
	void ToggleFocusOnSelection();
	void SetMultiRigMode(const EMultiRigTreeDisplayMode InMode);
	bool IsMultiRigMode(const EMultiRigTreeDisplayMode InMode);
	EMultiRigTreeDisplayMode GetMultiRigMode();

private:

	void OnFilterTextChanged(const FText& SearchText);

	/** Command list we bind to */
	TSharedPtr<FUICommandList> CommandList;

	FOnGetRigTreeDisplaySettings SuperGetRigTreeDisplaySettings;
	FOnMultiRigTreeGetEditMode GetEditMode;
	FText FilterText;
	FRigHierarchyTreeDisplaySettings Settings;
	TSharedPtr<SMultiRigHierarchyTreeView> TreeView;
};

class SControlRigOutliner : public FControlRigBaseDockableView, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SControlRigOutliner){}
	SLATE_END_ARGS()
	SControlRigOutliner();
	~SControlRigOutliner();

	void Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode);

	//FControlRigBaseDockableView overrides
	virtual TSharedRef<FControlRigBaseDockableView> AsSharedWidget() override { return SharedThis(this); }
	virtual void SetEditMode(FControlRigEditMode& InEditMode) override;
private:
	virtual void HandleControlAdded(UControlRig* ControlRig, bool bIsAdded) override;
	virtual void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected) override;
	virtual void HandleRigVisibilityChanged(TArray<UControlRig*> InControlRigs) override;
	virtual void HandleHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);

	//control rig delegates
	void HandleOnControlRigBound(UControlRig* InControlRig);
	void HandleOnObjectBoundToControlRig(UObject* InObject);
	void HandlePostConstruction(UControlRig* InControlRig, const FName& InEventName);

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);
	void OnSequencerTreeViewChanged(EMovieSceneDataChangeType MovieSceneDataChange);
	
	void HandleSelectionChanged(TSharedPtr<FMultiRigTreeElement> Selection, ESelectInfo::Type SelectInfo);

	/** Hierarchy picker for controls*/
	TSharedPtr<SSearchableMultiRigHierarchyTreeView> HierarchyTreeView;
	FRigHierarchyTreeDisplaySettings DisplaySettings;
	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings() const { return DisplaySettings; }
	bool bIsChangingRigHierarchy = false;

	//set of control rigs we are bound too and need to clear delegates from
	TArray<TWeakObjectPtr<UControlRig>> BoundControlRigs;
};

#undef UE_API
