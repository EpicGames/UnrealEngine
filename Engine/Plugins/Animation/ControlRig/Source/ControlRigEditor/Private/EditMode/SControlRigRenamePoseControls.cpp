// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SControlRigRenamePoseControls.h"
#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "EditMode/ControlRigEditMode.h"
#include "Tools/ControlRigPose.h"
#include "EditorModeManager.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "ControlRigRenamePoseControls"

///// SControlRigRenamePoseControls /////

struct SControlRigRenamePoseControls : public  SCompoundWidget
{
public:
	struct FListItem
	{
		FString OriginalName;
		FString NewName;
	};

	class SControlRenameListRow : public SMultiColumnTableRow<TSharedPtr<FListItem>>
	{
	public:
		SLATE_BEGIN_ARGS(SControlRenameListRow) {}
		SLATE_ARGUMENT(TSharedPtr<FListItem>, Item)
			SLATE_END_ARGS()

		static const FName NAME_Original;
		static const FName NAME_New;

		TSharedPtr<FListItem> Item;

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
		{
			Item = InArgs._Item;

			FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments();
			FSuperRowType::Construct(SuperArgs, OwnerTable);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			
			if (ColumnName == NAME_Original)
			{
				return SNew(SBox)
					.Padding(2.0f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(Item->OriginalName))
					];
			}
			else
			{
				return SNew(SBox)
					.Padding(2.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(SEditableTextBox)
							.Text(FText::FromString(Item->NewName))
							.OnTextCommitted_Lambda([this](const FText& Val, ETextCommit::Type TextCommitType)
								{
									Item->NewName = Val.ToString();
								})
							.ToolTipText(LOCTEXT("NewControlName", "New control name"))
						];
					}
		}
	};

	SLATE_BEGIN_ARGS(SControlRigRenamePoseControls) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& Args, const TArray<UControlRigPoseAsset*>& InPoseAssets)
	{
		bResult = false;
		SetPoseAssets(InPoseAssets);

		for (const FName& Name: CommonControlNames)
		{
			TSharedRef<SControlRigRenamePoseControls::FListItem> Item = MakeShared<SControlRigRenamePoseControls::FListItem>();
			Item->OriginalName = Name.ToString();
			Item->NewName = Name.ToString();
			Items.Add(Item);
		}

		ChildSlot
			[
				SNew(SBox)
				.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(20.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RenamePrompt", "Enter a new name for the Controls you wish to rename."))
				]
			+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				.FillHeight(1.0f)
				[
					SNew(SBorder)
					[
						SAssignNew(RenameList, SListView<TSharedPtr<FListItem>>)
						.ListItemsSource(&Items)
				.OnGenerateRow(this, &SControlRigRenamePoseControls::OnGenerateRow)
				.HeaderRow
				(
					SNew(SHeaderRow)

					+ SHeaderRow::Column(SControlRenameListRow::NAME_Original)
					.FillWidth(0.4)
					.DefaultLabel(LOCTEXT("Original", "Original"))

					+ SHeaderRow::Column(SControlRenameListRow::NAME_New)
					.FillWidth(0.6)
					.DefaultLabel(LOCTEXT("New", "New"))
				)
					]
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(5.0f, 0.0f)
					[
						SNullWidget::NullWidget
					]
				+ SHorizontalBox::Slot()
					.Padding(5.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SButton)
						.ContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
					.OnClicked(this, &SControlRigRenamePoseControls::OnButtonClick_Continue)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Rename", "Rename"))
					]
				]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
						.OnClicked(this, &SControlRigRenamePoseControls::OnButtonClick_Cancel)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Cancel", "Cancel"))
						]
					]
				]
			]
		];	
	}

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FListItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SControlRenameListRow, OwnerTable)
			.Item(Item);
	}

	FReply OnButtonClick_Continue()
	{
		bResult = true;
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		for (TWeakObjectPtr<UControlRigPoseAsset>& PoseAsset : PoseAssets)
		{
			if (PoseAsset.IsValid() )
			{
				const FScopedTransaction Transaction(NSLOCTEXT("ControlRig", "RenameControls", "Rename Controls"));
				PoseAsset->Modify();
				for (TSharedPtr<FListItem>&Item : Items)
				{
					PoseAsset->ReplaceControlName(FName(*Item->OriginalName),FName(*Item->NewName));
				}
			}
			if (Window.IsValid())
			{
				Window->RequestDestroyWindow();
			}
		}
		return FReply::Handled();
	}

	FReply OnButtonClick_Cancel()
	{
		bResult = false;
		FSlateApplication::Get().FindWidgetWindow(AsShared())->RequestDestroyWindow();
		return FReply::Handled();
	}

	void SetPoseAssets(const TArray<UControlRigPoseAsset*>& InPoseAssets)
	{
		CommonControlNames.Reset();
		PoseAssets.Reset();

		for (UControlRigPoseAsset* PoseAsset : InPoseAssets)
		{
			if (PoseAsset)
			{
				PoseAssets.Add(PoseAsset);
			}
		}
		if (PoseAssets.Num() == 1)
		{
			CommonControlNames = PoseAssets[0].Get()->Pose.GetControlNames();
		}
		else
		{
			TArray<FName> PossibleControlNames = PoseAssets[0].Get()->Pose.GetControlNames();

			for (FName& ControlName : PossibleControlNames)
			{
				int32 Index = 1;
				for (; Index < PoseAssets.Num(); ++Index)
				{
					if (PoseAssets[Index].Get()->Pose.ContainsName(ControlName) == false)
					{
						break;
					}
				}
				if (Index == PoseAssets.Num())
				{
					CommonControlNames.Add(ControlName);
				}
			}
		}
	}

	bool bResult;

	TArray<TSharedPtr<FListItem>> Items;
	TSharedPtr<SListView<TSharedPtr<FListItem>>> RenameList;

	TArray<TWeakObjectPtr<UControlRigPoseAsset>> PoseAssets;
	TArray<FName> CommonControlNames;
};

const FName SControlRigRenamePoseControls::SControlRenameListRow::NAME_Original = FName(TEXT("Original"));
const FName SControlRigRenamePoseControls::SControlRenameListRow::NAME_New = FName(TEXT("New"));


void FControlRigRenameControlsDialog::RenameControls(const TArray<UControlRigPoseAsset*>& PoseAssets)
{
	const FText TitleText = NSLOCTEXT("ControlRig", "RenameControlsOnSelectedAssets", "Rename Controls On Selected Assets");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(500.0f, 600.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SControlRigRenamePoseControls> DialogWidget = SNew(SControlRigRenamePoseControls, PoseAssets);
	Window->SetContent(DialogWidget);

	FSlateApplication::Get().AddWindow(Window);
}

#undef LOCTEXT_NAMESPACE
