// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class UControlRigPoseAsset;

struct FControlRigRenameControlsDialog
{
	static void RenameControls(const TArray<UControlRigPoseAsset*>& PoseAssets);
};