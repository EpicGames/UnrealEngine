// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SControlRigSnapper.h"
#include "EditMode/SComponentPickerPopup.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Styling/AppStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "ScopedTransaction.h"
#include "ControlRig.h"
#include "UnrealEdGlobals.h"
#include "EditMode/ControlRigEditMode.h"
#include "Tools/ControlRigPose.h"
#include "EditorModeManager.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "Selection.h"
#include "Editor.h"
#include "Tools/ControlRigSnapSettings.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"
#include "SSocketChooser.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Tools/ControlRigSnapSettings.h"
#include "MovieScene.h"
#include "SActionButton.h"

#define LOCTEXT_NAMESPACE "ControlRigSnapper"


//function to keep framenumber from beign too large do to user error, it can cause a crash if we snap too many frames.
static void KeepFrameInRange(TSharedPtr<ISequencer>& SequencerPtr, FFrameNumber& KeepFrameInRange, UMovieScene* MovieScene)
{
	if (MovieScene && SequencerPtr.IsValid())
	{
		//limit values to 10x times the range from the end points.
		TOptional<TRange<FFrameNumber>> OptionalRange = SequencerPtr->GetSubSequenceRange();
		FFrameNumber StartFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetLowerBoundValue() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
		FFrameNumber EndFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetUpperBoundValue() : MovieScene->GetPlaybackRange().GetUpperBoundValue();

		if (EndFrame < StartFrame)
		{
			FFrameNumber Temp = StartFrame;
			StartFrame = EndFrame;
			EndFrame = Temp;
		}
		const FFrameNumber Max = (EndFrame - StartFrame) * 10;
		if (KeepFrameInRange < (StartFrame - Max))
		{
			KeepFrameInRange = (StartFrame - Max);
		}
		else if (KeepFrameInRange > (EndFrame + Max))
		{
			KeepFrameInRange = (EndFrame + Max);
		}
	}
}

void SControlRigSnapper::Construct(const FArguments& InArgs)
{
	ClearActors();
	SetStartEndFrames();

	//for snapper settings
	UControlRigSnapSettings* SnapperSettings = GetMutableDefault<UControlRigSnapSettings>();
	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = "ControlRigSnapper";

	SnapperDetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
	SnapperDetailsView->SetObject(SnapperSettings);


	ChildSlot
		[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(10.f)
					.VAlign(VAlign_Center)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						[

							SNew(SBox)
							.Padding(0.0f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("Children", "Children"))
							]

						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Fill)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.ContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
							.OnClicked(this, &SControlRigSnapper::OnActorToSnapClicked)
							[
								SNew(STextBlock)
								.ToolTipText(LOCTEXT("ActorToSnapTooltip","Select child object(s) you want to snap over the interval range"))
								.Text_Lambda([this]()
									{
										return GetActorToSnapText();
									})
							]
						]

					]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(10.f)
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Center)
							[

								SNew(SBox)
								.Padding(0.0f)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("Parent", "Parent"))
								]

							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Fill)
							[
								SNew(SButton)
									.HAlign(HAlign_Center)
									.ContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
									.OnClicked(this, &SControlRigSnapper::OnParentToSnapToClicked)
									[
										SNew(STextBlock)
										.ToolTipText(LOCTEXT("ParentToSnapTooltip", "Select parent object you want children to snap to. If one is not selected it will snap to World Location at the start."))
										.Text_Lambda([this]()
											{
												return GetParentToSnapText();
											})
									]
							]

						]
			 ]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(10.f)
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
							.ContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
							.OnClicked(this, &SControlRigSnapper::OnStartFrameClicked)
							[
								SNew(SEditableTextBox)
								.ToolTipText(LOCTEXT("GetStartFrameTooltip", "Set first frame to snap"))
								.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
									{
										TSharedPtr<ISequencer> SequencerPtr = Snapper.GetSequencer().Pin();
										if (SequencerPtr.IsValid() && SequencerPtr->GetFocusedMovieSceneSequence())
										{
											TOptional<double> NewFrameTime = SequencerPtr->GetNumericTypeInterface()->FromString(InText.ToString(), 0);
											if (NewFrameTime.IsSet())
											{
												StartFrame = FFrameNumber((int32)NewFrameTime.GetValue());
												KeepFrameInRange(SequencerPtr, StartFrame, SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene());
											}
										}
									})
								.Text_Lambda([this]()
									{
										return GetStartFrameToSnapText();
									})
							]

						]
						+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(10.f)
							.VAlign(VAlign_Center)
						[

							SNew(SButton)
								.HAlign(HAlign_Center)
								.ContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
								.OnClicked(this, &SControlRigSnapper::OnEndFrameClicked)
								[
									SNew(SEditableTextBox)
									.ToolTipText(LOCTEXT("GetEndFrameTooltip", "Set end frame to snap"))
									.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type TextCommitType)
									{
										TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
										if (Sequencer.IsValid() && Sequencer.Pin()->GetFocusedMovieSceneSequence())
										{
											TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
											TOptional<double> NewFrameTime = SequencerPtr->GetNumericTypeInterface()->FromString(InText.ToString(), 0);
											if (NewFrameTime.IsSet())
											{
												EndFrame = FFrameNumber((int32)NewFrameTime.GetValue());
												KeepFrameInRange(SequencerPtr, EndFrame, SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene());
											}
										}
									})
									.Text_Lambda([this]()
										{
											return GetEndFrameToSnapText();
										})
								]

						]
				 ]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				[
					SnapperDetailsView.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Bottom)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(5.f)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SActionButton)
						.Text(LOCTEXT("SnapAnimation", "Snap Animation"))
						.ButtonContentPadding(FMargin(10.0f, 2.0f, 10.0f, 2.0f))
						.OnClicked(this, &SControlRigSnapper::OnSnapAnimationClicked)
					]
				]
			]
		];

	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid())
	{
		Sequencer.Pin()->OnActivateSequence().AddRaw(this, &SControlRigSnapper::OnActivateSequenceChanged);
	}
}

SControlRigSnapper::~SControlRigSnapper()
{
	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid())
	{
		Sequencer.Pin()->OnActivateSequence().RemoveAll(this);
	}
}

void SControlRigSnapper::OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID)
{
	if(!GIsTransacting)
	{ 
		ClearActors();
	}
}


FReply SControlRigSnapper::OnActorToSnapClicked()
{
	ActorToSnap = GetSelection(true);
	return FReply::Handled();
}

FReply SControlRigSnapper::OnParentToSnapToClicked()
{
	ParentToSnap = GetSelection(false);
	return FReply::Handled();
}

FText SControlRigSnapper::GetActorToSnapText()
{
	if (ActorToSnap.IsValid())
	{
		return (ActorToSnap.GetName());
	}

	return LOCTEXT("SelectActor", "Select Actor");
}

FText SControlRigSnapper::GetParentToSnapText()
{
	if (ParentToSnap.IsValid())
	{
		return (ParentToSnap.GetName());
	}

	return LOCTEXT("World", "World");
}


FReply SControlRigSnapper::OnStartFrameClicked()
{
	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid())
	{
		const FFrameRate TickResolution = Sequencer.Pin()->GetFocusedTickResolution();
		const FFrameTime FrameTime = Sequencer.Pin()->GetLocalTime().ConvertTo(TickResolution);
		StartFrame = FrameTime.GetFrame();
	}
	return FReply::Handled();
}

FReply SControlRigSnapper::OnEndFrameClicked()
{
	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid())
	{
		const FFrameRate TickResolution = Sequencer.Pin()->GetFocusedTickResolution();
		const FFrameTime FrameTime = Sequencer.Pin()->GetLocalTime().ConvertTo(TickResolution);
		EndFrame = FrameTime.GetFrame();
	}
	return FReply::Handled();
}

FReply SControlRigSnapper::OnSnapAnimationClicked()
{
	const UControlRigSnapSettings* SnapSettings = GetDefault<UControlRigSnapSettings>();
	Snapper.SnapIt(StartFrame, EndFrame, ActorToSnap, ParentToSnap,SnapSettings);
	return FReply::Handled();
}

FText SControlRigSnapper::GetStartFrameToSnapText()
{
	FText Value;
	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid() && Sequencer.Pin()->GetFocusedMovieSceneSequence())
	{
		Value = FText::FromString(Sequencer.Pin()->GetNumericTypeInterface()->ToString(StartFrame.Value));
	}
	return Value;
}

FText SControlRigSnapper::GetEndFrameToSnapText()
{
	FText Value;
	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid() && Sequencer.Pin()->GetFocusedMovieSceneSequence())
	{
		Value = FText::FromString(Sequencer.Pin()->GetNumericTypeInterface()->ToString(EndFrame.Value));
	}
	return Value;
}

void SControlRigSnapper::ClearActors()
{
	ActorToSnap.Clear();
	ParentToSnap.Clear();
}

void SControlRigSnapper::SetStartEndFrames()
{
	TWeakPtr<ISequencer> Sequencer = Snapper.GetSequencer();
	if (Sequencer.IsValid()  && Sequencer.Pin()->GetFocusedMovieSceneSequence())
	{
		UMovieScene* MovieScene = Sequencer.Pin()->GetFocusedMovieSceneSequence()->GetMovieScene();
		TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer.Pin()->GetSubSequenceRange();
		StartFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetLowerBoundValue() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
		EndFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetUpperBoundValue() : MovieScene->GetPlaybackRange().GetUpperBoundValue();

	}
}

FControlRigSnapperSelection SControlRigSnapper::GetSelection(bool bGetAll) 
{
	FControlRigSnapperSelection Selection;
	TArray<UControlRig*> ControlRigs;
	GetControlRigs(ControlRigs);
	for (UControlRig* ControlRig : ControlRigs)
	{
		if (ControlRig)
		{
			if (const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
			{
				TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
				if (SelectedControls.Num() > 0)
				{
					FControlRigForWorldTransforms ControlRigAndSelection;
					ControlRigAndSelection.ControlRig = ControlRig;
					if (bGetAll == false)
					{
						//make sure to only use first control that is snappable(that has a shape)
						for (const FName& ControlName : SelectedControls)
						{
							if (const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
							{
								if (ControlElement->Settings.SupportsShape())
								{
									ControlRigAndSelection.ControlNames.Add(ControlName);
									break;
								}
							}
						}
						Selection.ControlRigs.Add(ControlRigAndSelection);
						return Selection;
					}
					else
					{
						for (const FName& ControlName : SelectedControls)
						{
							if (const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
							{
								if (ControlElement->Settings.SupportsShape())
								{
									ControlRigAndSelection.ControlNames.Add(ControlName);
								}
							}
						}
						Selection.ControlRigs.Add(ControlRigAndSelection);

					}
				}
			}
		}
	}
	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			FActorForWorldTransforms ActorSelection;
			ActorSelection.Actor = Actor;
			Selection.Actors.Add(ActorSelection);
			if (bGetAll == false)
			{
				ActorParentPicked(ActorSelection);
				return Selection;
			}
		}
	}
	return Selection;
}

void SControlRigSnapper::GetControlRigs(TArray<UControlRig*>& OutControlRigs) const
{
	OutControlRigs.SetNum(0);
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TArrayView<TWeakObjectPtr<UControlRig>> ControlRigPtrs = EditMode->GetControlRigs();
		for (TWeakObjectPtr<UControlRig>& ControlRigPtr : ControlRigPtrs)
		{
			if (UControlRig* ControlRig = ControlRigPtr.Get())
			{
				OutControlRigs.Add(ControlRig);
			}
		}
	}
}


void SControlRigSnapper::ActorParentSocketPicked(const FName SocketPicked, FActorForWorldTransforms Selection) 
{
	ParentToSnap.Actors.SetNum(0);
	Selection.SocketName = SocketPicked;
	ParentToSnap.Actors.Add(Selection);
	//ParentToSnap
}

void SControlRigSnapper::ActorParentPicked(FActorForWorldTransforms Selection) 
{
	TArray<USceneComponent*> ComponentsWithSockets;
	if (Selection.Actor.IsValid())
	{
		TInlineComponentArray<USceneComponent*> Components(Selection.Actor.Get());

		for (USceneComponent* Component : Components)
		{
			if (Component->HasAnySockets())
			{
				ComponentsWithSockets.Add(Component);
			}
		}
	}

	if (ComponentsWithSockets.Num() == 0)
	{
		FSlateApplication::Get().DismissAllMenus();
		ActorParentSocketPicked(NAME_None, Selection);
		return;
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;
	if (ComponentsWithSockets.Num() > 1)
	{
		MenuWidget =
			SNew(SComponentPickerPopup)
			.Actor(Selection.Actor.Get())
			.OnComponentChosen(this, &SControlRigSnapper::ActorParentComponentPicked,Selection);

		// Create as context menu
		FSlateApplication::Get().PushMenu(
			LevelEditor.ToSharedRef(),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}
	else
	{
		ActorParentComponentPicked(ComponentsWithSockets[0]->GetFName(),Selection);
	}
}


void SControlRigSnapper::ActorParentComponentPicked(FName ComponentName, FActorForWorldTransforms Selection) 
{
	USceneComponent* ComponentWithSockets = nullptr;
	if (Selection.Actor.IsValid())
	{
		AActor* Actor = Selection.Actor.Get();
		TInlineComponentArray<USceneComponent*> Components(Actor);

		for (USceneComponent* Component : Components)
		{
			if (Component->GetFName() == ComponentName)
			{
				ComponentWithSockets = Component;
				break;
			}
		}
	}

	if (ComponentWithSockets == nullptr)
	{
		return;
	}
	Selection.Component = ComponentWithSockets;
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();

	TSharedPtr<SWidget> MenuWidget;

	MenuWidget =
		SNew(SSocketChooserPopup)
		.SceneComponent(ComponentWithSockets)
		.OnSocketChosen(this, &SControlRigSnapper::ActorParentSocketPicked, Selection);

	// Create as context menu
	FSlateApplication::Get().PushMenu(
		LevelEditor.ToSharedRef(),
		FWidgetPath(),
		MenuWidget.ToSharedRef(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
}



#undef LOCTEXT_NAMESPACE
