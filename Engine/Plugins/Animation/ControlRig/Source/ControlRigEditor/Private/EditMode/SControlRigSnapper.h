// Copyright Epic Games, Inc. All Rights Reserved.
/**
* Hold the View for the Snapper Widget
*/
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Tools/ControlRigSnapper.h"
#include "Misc/FrameNumber.h"
#include "IDetailsView.h"
#include "MovieSceneSequenceID.h"


class UControlRig;

class SControlRigSnapper : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SControlRigSnapper) {}

	SLATE_END_ARGS()
	~SControlRigSnapper();

	void Construct(const FArguments& InArgs);


private:

	void GetControlRigs(TArray<UControlRig*>& OutControlRigs) const;

	/*
	* Delegates and Helpers
	*/
	void ActorParentPicked(FActorForWorldTransforms Selection);
	void ActorParentSocketPicked(const FName SocketName, FActorForWorldTransforms Selection);
	void ActorParentComponentPicked(FName ComponentName, FActorForWorldTransforms Selection);
	void OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID);

	FReply OnActorToSnapClicked();
	FReply OnParentToSnapToClicked();
	FText GetActorToSnapText();
	FText GetParentToSnapText();

	FReply OnStartFrameClicked();
	FReply OnEndFrameClicked();
	FText GetStartFrameToSnapText();
	FText GetEndFrameToSnapText();

	void ClearActors();
	void SetStartEndFrames();
	FReply OnSnapAnimationClicked();

	FControlRigSnapperSelection GetSelection(bool bGetAll);

	//FControlRigSnapper  ControlRigSnapper;
	TSharedPtr<IDetailsView> SnapperDetailsView;

	FControlRigSnapperSelection ActorToSnap;
	FControlRigSnapperSelection ParentToSnap;

	FFrameNumber StartFrame;
	FFrameNumber EndFrame;

	FControlRigSnapper Snapper;

};

