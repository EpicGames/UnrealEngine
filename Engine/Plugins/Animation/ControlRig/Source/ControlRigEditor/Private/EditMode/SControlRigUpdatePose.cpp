// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SControlRigUpdatePose.h"
#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Editor.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "EditMode/ControlRigEditMode.h"
#include "Tools/ControlRigPose.h"
#include "EditorModeManager.h"
#include "ScopedTransaction.h"

class SControlRigUpdatePoseDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SControlRigUpdatePoseDialog) {}
	SLATE_END_ARGS()
	~SControlRigUpdatePoseDialog()
	{
	}
	
	void Construct(const FArguments& InArgs)
	{

		ChildSlot
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(5.f)
				[
				SNew(STextBlock)
				.Text(NSLOCTEXT("ControlRig", "UpdatePoseWithSelectedControls", "Update Pose With Selected Controls?"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(5.f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.Padding(10)
					[
						SNew(SButton)
						.ContentPadding(FMargin(10, 5))
						.Text(NSLOCTEXT("ControlRig", "Yes", "Yes"))
						.OnClicked(this, &SControlRigUpdatePoseDialog::UpdatePose)
						.IsEnabled(this, &SControlRigUpdatePoseDialog::CanUpdatePose)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(10)
					[
						SNew(SButton)
						.ContentPadding(FMargin(10, 5))
						.Text(NSLOCTEXT("ControlRig", "No", "No"))
						.OnClicked(this, &SControlRigUpdatePoseDialog::CancelUpdatePose)
					]
				]
			];
	}

	void SetPose(UControlRigPoseAsset* InPoseAsset)
	{
		PoseAsset = InPoseAsset;
	}

private:

	bool CanUpdatePose() const
	{
		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (PoseAsset.IsValid() && ControlRigEditMode)
		{
			TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
			ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
			return (AllSelectedControls.Num() > 0);
		}
		return false;

	}
	FReply UpdatePose()
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (PoseAsset.IsValid() && ControlRigEditMode)
		{
			TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
			ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
			if (AllSelectedControls.Num() == 1)
			{
				TArray<UControlRig*> ControlRigs;
				AllSelectedControls.GenerateKeyArray(ControlRigs);
				const FScopedTransaction Transaction(NSLOCTEXT("ControlRig", "UpdatePose", "Update Pose"));
				PoseAsset->Modify();
				PoseAsset->SavePose(ControlRigs[0], false);
			}
			else
			{

			}
		}
		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply CancelUpdatePose()
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	TSharedPtr<IDetailsView> DetailView;
	TWeakObjectPtr<UControlRigPoseAsset> PoseAsset;
};


void FControlRigUpdatePoseDialog::UpdatePose(UControlRigPoseAsset* PoseAsset)
{
	const FText TitleText = NSLOCTEXT("ControlRig", "UpdateControlRigPose", "Update Control Rig Pose");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(400.0f, 100.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SControlRigUpdatePoseDialog> DialogWidget = SNew(SControlRigUpdatePoseDialog);
	DialogWidget->SetPose(PoseAsset);
	Window->SetContent(DialogWidget);

	FSlateApplication::Get().AddWindow(Window);
}

