// Copyright Epic Games, Inc. All Rights Reserved.


#include "EditMode/SCurveControlContainer.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "PropertyCustomizationHelpers.h"
#include "Framework/Commands/GenericCommands.h"
#include "Editor/ControlRigEditor.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "ControlRig.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "SCurveControlContainer"

static const FName ColumnId_CurveControlNameLabel( "Curve" );
static const FName ColumnID_CurveControlValueLabel( "Value" );

//////////////////////////////////////////////////////////////////////////
// SCurveControlListRow

void SCurveControlListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	OnSetCurveControlValue = InArgs._OnSetCurveControlValue;
	OnGetCurveControlValue = InArgs._OnGetCurveControlValue;
	OnGetFilterText = InArgs._OnGetFilterText;

	check( Item.IsValid() );

	SMultiColumnTableRow< FDisplayedCurveControlInfoPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SCurveControlListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == ColumnId_CurveControlNameLabel )
	{
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SAssignNew(Item->EditableText, SInlineEditableTextBlock)
				.ColorAndOpacity(this, &SCurveControlListRow::GetItemTextColor)
				.IsSelected(this, &SCurveControlListRow::IsSelected)
				.Text(this, &SCurveControlListRow::GetItemName)
				.HighlightText(this, &SCurveControlListRow::GetFilterText)
			];
	}
	else if ( ColumnName == ColumnID_CurveControlValueLabel )
	{
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 1.0f )
			.VAlign( VAlign_Center )
			[
				SNew( SSpinBox<float> )
				.Value( this, &SCurveControlListRow::GetValue )
				.OnValueChanged( this, &SCurveControlListRow::OnCurveControlValueChanged )
				.OnValueCommitted( this, &SCurveControlListRow::OnCurveControlValueValueCommitted )
				.IsEnabled(true)
			];
	}

	return SNullWidget::NullWidget;
}

void SCurveControlListRow::OnCurveControlValueChanged( float NewValue )
{
	Item->Value = NewValue;

	OnSetCurveControlValue.ExecuteIfBound(Item->CurveName, NewValue);
}

void SCurveControlListRow::OnCurveControlValueValueCommitted( float NewValue, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		OnCurveControlValueChanged(NewValue);
	}
}


FText SCurveControlListRow::GetItemName() const
{
	return FText::FromName(Item->CurveName);
}

FText SCurveControlListRow::GetFilterText() const
{
	if (OnGetFilterText.IsBound())
	{
		return OnGetFilterText.Execute();
	}
	else
	{
		return FText::GetEmpty();
	}
}

FSlateColor SCurveControlListRow::GetItemTextColor() const
{
	// If row is selected, show text as black to make it easier to read
	if (IsSelected())
	{
		return FLinearColor(0, 0, 0);
	}

	return FLinearColor(1, 1, 1);
}

float SCurveControlListRow::GetValue() const 
{ 
	if (OnGetCurveControlValue.IsBound())
	{
		return OnGetCurveControlValue.Execute(Item->CurveName);
	}

	return 0.f;
}

//////////////////////////////////////////////////////////////////////////
// SCurveControlContainer

void SCurveControlContainer::Construct(const FArguments& InArgs, UControlRig* InControlRig)
{
	ChildSlot
	[
		SNew( SVerticalBox )
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth( 1 )
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SCurveControlContainer::OnFilterTextChanged )
				.OnTextCommitted( this, &SCurveControlContainer::OnFilterTextCommitted )
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( CurveControlListView, SCurveControlListType )
			.ListItemsSource( &CurveControlList )
			.OnGenerateRow( this, &SCurveControlContainer::GenerateCurveControlRow )
			.SelectionMode(ESelectionMode::Multi)
			.OnSelectionChanged( this, &SCurveControlContainer::OnSelectionChanged )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_CurveControlNameLabel )
				.FillWidth(1.f)
				.DefaultLabel( LOCTEXT( "CurveControlNameLabel", "Curve" ) )

				+ SHeaderRow::Column( ColumnID_CurveControlValueLabel )
				.FillWidth(1.f)
				.DefaultLabel( LOCTEXT( "CurveControlValueLabel", "Value" ) )
			)
		]
	];

	SetControlRig(InControlRig);
}


void SCurveControlContainer::SetControlRig(UControlRig* InControlRig)
{

	if (ControlRig.IsValid())
	{
		ControlRig->ControlSelected().RemoveAll(this);
	}
	ControlRig = InControlRig;
	if (ControlRig.IsValid())
	{
		ControlRig->ControlSelected().AddRaw(this, &SCurveControlContainer::OnRigElementSelected);
	}
	RefreshCurveList();
}


SCurveControlContainer::~SCurveControlContainer()
{
	if (ControlRig.IsValid())
	{
		ControlRig->ControlSelected().RemoveAll(this);
	}
}

void SCurveControlContainer::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	RefreshCurveList();
}

void SCurveControlContainer::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SCurveControlContainer::GenerateCurveControlRow(FDisplayedCurveControlInfoPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InInfo.IsValid());

	return
		SNew(SCurveControlListRow, OwnerTable)
		.Item(InInfo)
		.OnSetCurveControlValue(this, &SCurveControlContainer::SetCurveValue)
		.OnGetCurveControlValue(this, &SCurveControlContainer::GetCurveValue)
		.OnGetFilterText(this, &SCurveControlContainer::GetFilterText);
}


void SCurveControlContainer::CreateCurveControlList( const FString& SearchText )
{
	URigHierarchy* Hierarchy = GetHierarchy();
	if (Hierarchy)
	{
		CurveControlList.Reset();

		const FString CTRLName(TEXT("CTRL_"));

		// Iterate through all curves..
		Hierarchy->ForEach<FRigCurveElement>([CTRLName, this](FRigCurveElement* CurveElement) -> bool
        {
			const FString Name = CurveElement->GetName();
			if (Name.Contains(CTRLName))
			{
				const FString CurveString = CurveElement->GetName();

				// See if we pass the search filter
				if (!FilterText.IsEmpty() && !CurveString.Contains(*FilterText.ToString()))
				{
					return true;
				}

				const TSharedRef<FDisplayedCurveControlInfo> NewItem = FDisplayedCurveControlInfo::Make(CurveElement->GetFName());
				CurveControlList.Add(NewItem);
			}

			return true;
		});

		// Sort final list
		struct FSortNamesAlphabetically
		{
			bool operator()(const FDisplayedCurveControlInfoPtr& A, const FDisplayedCurveControlInfoPtr& B) const
			{
				return (A.Get()->CurveName.Compare(B.Get()->CurveName) < 0);
			}
		};

		CurveControlList.Sort(FSortNamesAlphabetically());
	}
	CurveControlListView->RequestListRefresh();

	if (Hierarchy)
	{
		TArray<FRigElementKey> Selection = Hierarchy->GetSelectedKeys(ERigElementType::Control);

		for (const FRigElementKey& SelectedKey : Selection)
		{
			if(FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(SelectedKey))
			{
				OnRigElementSelected(ControlRig.Get(), ControlElement, true);
				break;
			}
		}
	}
}

void SCurveControlContainer::RefreshCurveList()
{
	CreateCurveControlList(FilterText.ToString());
}

void SCurveControlContainer::SetCurveValue(const FName& CurveName, float CurveValue)
{
	if (ControlRig.IsValid())
	{
		FRigControlElement* ControlElement = ControlRig->FindControl(CurveName);
		if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::Float)
		{
			ControlRig->SetControlValue<float>(CurveName, CurveValue,true, EControlRigSetKey::Always);
		}

	}
}

float SCurveControlContainer::GetCurveValue(const FName& CurveName)
{
	URigHierarchy* Hierarchy = GetHierarchy();
	if (Hierarchy)
	{
		return Hierarchy->GetCurveValue(FRigElementKey(CurveName, ERigElementType::Curve));
	}
	return 0.f;
}

void SCurveControlContainer::OnSelectionChanged(FDisplayedCurveControlInfoPtr Selection, ESelectInfo::Type SelectInfo)
{
	if (ControlRig.IsValid())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !GIsTransacting);

		URigHierarchy* Hierarchy = GetHierarchy();
		TArray<FRigElementKey> OldSelection = Hierarchy->GetSelectedKeys();
		TArray<FRigElementKey> NewSelection;

		TArray<FDisplayedCurveControlInfoPtr> SelectedItems = CurveControlListView->GetSelectedItems();
		for (const FDisplayedCurveControlInfoPtr& SelectedItem : SelectedItems)
		{
			NewSelection.Add(FRigElementKey(SelectedItem->CurveName, ERigElementType::Curve));
		}

		for (const FRigElementKey& PreviouslySelected : OldSelection)
		{
			if (NewSelection.Contains(PreviouslySelected))
			{
				continue;
			}
			ControlRig->SelectControl(PreviouslySelected.Name, false);
		}

		for (const FRigElementKey& NewlySelected : NewSelection)
		{
			ControlRig->SelectControl(NewlySelected.Name, true);
		}
	}
}
void SCurveControlContainer::OnRigElementSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected)
{
	for(const FDisplayedCurveControlInfoPtr& Item : CurveControlList)
	{
		if (Item->CurveName == ControlElement->GetFName())
		{
			CurveControlListView->SetItemSelection(Item, bSelected);
			break;
		}
	}
}

URigHierarchy* SCurveControlContainer::GetHierarchy() const
{
	if (ControlRig.IsValid())
	{
		return ControlRig->GetHierarchy();
	}

	return nullptr;
}
#undef LOCTEXT_NAMESPACE
