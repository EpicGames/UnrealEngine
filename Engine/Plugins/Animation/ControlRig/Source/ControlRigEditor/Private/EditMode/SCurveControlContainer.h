// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Rigs/RigHierarchyContainer.h"

//////////////////////////////////////////////////////////////////////////
// FDisplayedCurveControlInfo

class SInlineEditableTextBlock;
struct FAssetData;
class UControlRig;

class FDisplayedCurveControlInfo
{
public:
	FName CurveName;
	float Value;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedCurveControlInfo> Make(const FName& InCurveName)
	{
		return MakeShareable(new FDisplayedCurveControlInfo(InCurveName));
	}

	// editable text
	TSharedPtr<SInlineEditableTextBlock> EditableText;

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedCurveControlInfo(const FName& InCurveName)
		: CurveName(InCurveName)
		, Value( 0 )
	{}
};

typedef TSharedPtr< FDisplayedCurveControlInfo > FDisplayedCurveControlInfoPtr;
typedef SListView< FDisplayedCurveControlInfoPtr > SCurveControlListType;

//////////////////////////////////////////////////////////////////////////
// SCurveControlListRow

DECLARE_DELEGATE_TwoParams(FSetCurveControlValue, const FName&, float);
DECLARE_DELEGATE_RetVal_OneParam(float, FGetCurveControlValue, const FName&);
DECLARE_DELEGATE_RetVal(FText&, FGetFilterText);

class SCurveControlListRow : public SMultiColumnTableRow< FDisplayedCurveControlInfoPtr >
{
public:

	SLATE_BEGIN_ARGS(SCurveControlListRow) {}

		/** The item for this row **/
		SLATE_ARGUMENT(FDisplayedCurveControlInfoPtr, Item)


		/** set the value delegate */
		SLATE_EVENT(FSetCurveControlValue, OnSetCurveControlValue)

		/** set the value delegate */
		SLATE_EVENT(FGetCurveControlValue, OnGetCurveControlValue)

		/** get filter text */
		SLATE_EVENT(FGetFilterText, OnGetFilterText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/**
	* Called when the user changes the value of the SSpinBox
	*
	* @param NewValue - The new number the SSpinBox is set to
	* this may not work right if the rig is being evaluated
	* this just overwrite the data when it's written
	*
	*/
	void OnCurveControlValueChanged(float NewValue);

	/**
	* Called when the user types the value and enters
	*
	* @param NewValue - The new number the SSpinBox is set to
	*
	*/
	void OnCurveControlValueValueCommitted(float NewValue, ETextCommit::Type CommitType);

	/** Returns the Value of this curve */
	float GetValue() const;
	/** Returns name of this curve */
	FText GetItemName() const;
	/** Get text we are filtering for */
	FText GetFilterText() const;
	/** Return color for text of item */
	FSlateColor GetItemTextColor() const;

	/** The name and Value of the morph target */
	FDisplayedCurveControlInfoPtr	Item;

	FSetCurveControlValue OnSetCurveControlValue;
	FGetCurveControlValue OnGetCurveControlValue;
	FGetFilterText OnGetFilterText;
};

//////////////////////////////////////////////////////////////////////////
// SCurveControlContainer

class SCurveControlContainer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCurveControlContainer )
	{}
	
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct( const FArguments& InArgs, UControlRig* InControlRig);

	/**
	* Destructor - resets the animation curve
	*
	*/
	virtual ~SCurveControlContainer();

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged( const FText& SearchText );


	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GenerateCurveControlRow(FDisplayedCurveControlInfoPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	void RefreshCurveList();
	void SetControlRig(UControlRig* InControlRig);
private:

	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreateCurveControlList( const FString& SearchText = FString() );

	// delegate for changing info
	void SetCurveValue(const FName& CurveName, float CurveValue);
	float GetCurveValue(const FName& CurveName);

	void OnSelectionChanged(FDisplayedCurveControlInfoPtr Selection, ESelectInfo::Type SelectInfo);

	void OnRigElementSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected);

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** A list of animation curve. Used by the CurveControlListView. */
	TArray< FDisplayedCurveControlInfoPtr > CurveControlList;

	/** Widget used to display the list of animation curve */
	TSharedPtr<SCurveControlListType> CurveControlListView;

	/** ControlRig */
	TWeakObjectPtr<UControlRig> ControlRig;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;

	URigHierarchy* GetHierarchy() const;

	friend class SCurveControlListRow;
	friend class SCurveControlTypeList;
};