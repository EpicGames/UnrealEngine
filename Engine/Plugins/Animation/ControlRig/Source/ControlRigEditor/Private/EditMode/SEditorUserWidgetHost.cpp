// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMode/SEditorUserWidgetHost.h"
#include "Engine/World.h"

void SEditorUserWidgetHost::Construct(const FArguments& InArgs, UWorld* InWorld)
{
	check(InWorld);
	World = InWorld;
	
	ChildSlot
	[
		SNullWidget::NullWidget
	];
}

void SEditorUserWidgetHost::AddReferencedObjects(FReferenceCollector& Collector)
{
}
