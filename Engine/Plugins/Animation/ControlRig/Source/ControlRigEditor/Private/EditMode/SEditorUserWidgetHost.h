// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SEditorUserWidgetHost : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SEditorUserWidgetHost) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UWorld* InWorld);

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SEditorUserWidgetHost");
	}

private:
	/** The world we create widgets with */
	TWeakObjectPtr<UWorld> World;
};