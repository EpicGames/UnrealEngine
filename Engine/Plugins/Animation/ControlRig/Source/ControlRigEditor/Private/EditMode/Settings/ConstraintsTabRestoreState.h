// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "ConstraintsTabRestoreState.generated.h"

UENUM()
enum class EControlRigConstrainTab : uint8
{
	Spaces,
	Constraints,
	Snapper
};

USTRUCT()
struct FControlRigConstraintsTabRestoreState
{
	GENERATED_BODY()

	UPROPERTY()
	bool bWasOpen = false;

	UPROPERTY() 
	EControlRigConstrainTab LastOpenInlineTab = EControlRigConstrainTab::Spaces;
};