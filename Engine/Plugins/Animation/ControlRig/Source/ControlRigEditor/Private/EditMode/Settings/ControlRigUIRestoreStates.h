// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConstraintsTabRestoreState.h"
#include "Misc/Flyout/FlyoutSavedState.h"
#include "Overlay/DragBoxPosition.h"
#include "ControlRigUIRestoreStates.generated.h"

/** Data to restore the UI state control rig had last time it was open. */
USTRUCT()
struct FControlRigUIRestoreStates
{
	GENERATED_BODY()

	UPROPERTY()
	bool bMotionTrailsOn = false;
	
	UPROPERTY()
	bool bAnimLayerTabOpen = false;
	
	UPROPERTY()
	bool bPoseTabOpen = false;
	
	UPROPERTY()
	bool bSnapperTabOpen = false;
	
	UPROPERTY()
	FControlRigConstraintsTabRestoreState ConstraintsTabState;

	/** Whether the selection set was open when SELECTION_SETS_AS_TAB == 1 */
	UPROPERTY()
	bool bSelectionSetsOpen = false;

	/** Selection set view state if SELECTION_SETS_AS_TAB == 0 */
	UPROPERTY()
	FToolWidget_FlyoutSavedState SelectionSetOverlayState;

	UPROPERTY()
	FToolWidget_FlyoutSavedState TweenOverlayState;

	explicit FControlRigUIRestoreStates()
		: SelectionSetOverlayState(FToolWidget_DragBoxPosition(FVector2f(20.f, 30.f), HAlign_Right, VAlign_Top), false)
		, TweenOverlayState(FToolWidget_DragBoxPosition(FVector2f(20.f, 30.f), HAlign_Right, VAlign_Bottom), false)
	{}
};