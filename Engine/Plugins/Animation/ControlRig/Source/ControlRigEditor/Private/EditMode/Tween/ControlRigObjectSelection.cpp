// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigObjectSelection.h"

#include "CurveDataAbstraction.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EditMode/ControlRigEditMode.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tools/EvaluateSequencerTools.h"


namespace UE::ControlRigEditor
{
bool FControlRigObjectSelection::Setup(const TWeakPtr<ISequencer>& InSequencer, const TWeakPtr<FControlRigEditMode>& InEditMode)
{
	ChannelsArray.Reset();
	const TArray<UControlRig*> ControlRigs = GetControlRigs(InEditMode);
	return Setup(ControlRigs, InSequencer);
}
	
bool FControlRigObjectSelection::Setup(const TArray<UControlRig*>& SelectedControlRigs, const TWeakPtr<ISequencer>& InSequencer)
{
	using namespace UE::AIE;
	ChannelsArray.Reset();
	if (InSequencer.IsValid() == false)
	{
		return false;
	}
	ISequencer* Sequencer = InSequencer.Pin().Get();

	const FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();
	const UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

	//get selected controls and objects
	TArray<FGuid> SelectedGuids;
	SelectedGuids = FSequencerSelected::GetSelectedOutlinerGuids(Sequencer);


	TArray<UMovieSceneControlRigParameterSection*> HandledSections;
	auto SetupControlRigTrackChannels = [this, &CurrentTime, &HandledSections](const UMovieSceneControlRigParameterTrack* Track)
	{
		check(Track);		
		for (UMovieSceneSection* MovieSection : Track->GetAllSections())
		{
			UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(MovieSection);
			if (Section && Section->IsActive() && Section->GetRange().Contains(CurrentTime.Time.GetFrame()) && !HandledSections.Contains(Section))
			{
				HandledSections.Add(Section);				
				UControlRig* ControlRig = Track->GetControlRig();
				TArray<FRigControlElement*> CurrentControls;
				ControlRig->GetControlsInOrder(CurrentControls);

				FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
				TArrayView<FMovieSceneFloatChannel*> Channels = ChannelProxy.GetChannels<FMovieSceneFloatChannel>();
				//reuse these arrays
				TArray<FFrameNumber> KeyTimes;
				TArray<FKeyHandle> Handles;
				for (const FRigControlElement* ControlElement : CurrentControls)
				{
					if (ControlRig->GetHierarchy()->IsAnimatable(ControlElement) &&  ControlRig->IsControlSelected(ControlElement->GetFName()))
					{
						FControlRigObjectSelection::FObjectChannels ObjectChannels;
						ObjectChannels.Section = Section;
						KeyTimes.SetNum(0);
						Handles.SetNum(0);
						const FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlElement->GetFName());
						if (pChannelIndex == nullptr)
						{
							continue;
						}

						const int32 NumChannels = [&ControlElement]()
						{
							switch (ControlElement->Settings.ControlType)
							{
								case ERigControlType::Float:
								case ERigControlType::ScaleFloat:
								{
									return 1;
								}
								case ERigControlType::Vector2D:
								{
									return 2;
								}
								case ERigControlType::Position:
								case ERigControlType::Scale:
								case ERigControlType::Rotator:
								{
									return 3;
								}
								case ERigControlType::TransformNoScale:
								{
									return 6;
								}
								case ERigControlType::Transform:
								case ERigControlType::EulerTransform:
								{
									return 9;
								}
							default:
									return 0;
							}
						}();
					
						int32 BoundIndex = 0;
						int32 NumValidChannels = 0;
						ObjectChannels.KeyBounds.SetNum(NumChannels);
						for (int32 ChannelIdx = pChannelIndex->ChannelIndex; ChannelIdx < (pChannelIndex->ChannelIndex + NumChannels); ++ChannelIdx)
						{
							FMovieSceneFloatChannel* Channel = Channels[ChannelIdx];
							SetupChannel(CurrentTime.Time.GetFrame(), KeyTimes, Handles, Channel, nullptr,
							ObjectChannels.KeyBounds[BoundIndex]);
							if (ObjectChannels.KeyBounds[BoundIndex].bValid)
							{
								++NumValidChannels;
							}
							++BoundIndex;
						}
						if (NumValidChannels > 0)
						{
							ChannelsArray.Add(ObjectChannels);
						}
					}
				}
			}
		}
	};
	
	// Handle MovieScene bindings
	const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		if (SelectedGuids.Num() > 0 && SelectedGuids.Contains(Binding.GetObjectGuid()))
		{
			const TArray<UMovieSceneTrack*>& Tracks = Binding.GetTracks();
			for (const UMovieSceneTrack* Track : Tracks)
			{
				if (Track && Track->IsA<UMovieSceneControlRigParameterTrack>() == false)
				{
					const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
					for (UMovieSceneSection* Section : Sections)
					{
						if (Section && Section->IsActive())
						{
							FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
							TArrayView<FMovieSceneFloatChannel*> FloatChannels = ChannelProxy.GetChannels<FMovieSceneFloatChannel>();
							TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = ChannelProxy.GetChannels<FMovieSceneDoubleChannel>();

							//reuse these arrays
							TArray<FFrameNumber> KeyTimes;
							TArray<FKeyHandle> Handles;
							FControlRigObjectSelection::FObjectChannels ObjectChannels;
							ObjectChannels.Section = Section;
							KeyTimes.SetNum(0);
							Handles.SetNum(0);
							const int32 NumFloatChannels = FloatChannels.Num();
							const int32 NumDoubleChannels = DoubleChannels.Num();

							ObjectChannels.KeyBounds.SetNum(NumFloatChannels + NumDoubleChannels);
							int32 NumValidChannels = 0;
							for (int32 ChannelIdx = 0;ChannelIdx < NumFloatChannels; ++ChannelIdx)
							{
								FMovieSceneFloatChannel* FloatChannel = FloatChannels[ChannelIdx];
								SetupChannel(CurrentTime.Time.GetFrame(), KeyTimes, Handles, FloatChannel, nullptr,
									ObjectChannels.KeyBounds[ChannelIdx]);
								if (ObjectChannels.KeyBounds[ChannelIdx].bValid)
								{
									++NumValidChannels;
								}
							}
							for (int32 ChannelIdx = 0; ChannelIdx < NumDoubleChannels; ++ChannelIdx)
							{
								FMovieSceneDoubleChannel* DoubleChannel = DoubleChannels[ChannelIdx];
								SetupChannel(CurrentTime.Time.GetFrame(), KeyTimes, Handles, nullptr, DoubleChannel,
									ObjectChannels.KeyBounds[ChannelIdx]);
								if (ObjectChannels.KeyBounds[ChannelIdx].bValid)
								{
									++NumValidChannels;
								}
							}
							if (NumValidChannels > 0)
							{
								ChannelsArray.Add(ObjectChannels);
							}
						}
					}
				}
			}
		}

		const TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(
			UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None
			);
		for (const UMovieSceneTrack* Track : Tracks)
		{
			const UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
			if (ControlRigTrack && ControlRigTrack->GetControlRig() && SelectedControlRigs.Contains(ControlRigTrack->GetControlRig()))
			{
				SetupControlRigTrackChannels(ControlRigTrack);
			}
		}
	}
	
	// Handle movie tracks in general (for non-binding, USkeleton, ControlRig tracks)
	for (const UMovieSceneTrack* MovieTrack : MovieScene->GetTracks())
	{
		if (const UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(MovieTrack))
		{
			if (Track && Track->GetControlRig() && SelectedControlRigs.Contains(Track->GetControlRig()))
			{
				SetupControlRigTrackChannels(Track);
			}
		}
	}
	
	return ChannelsArray.Num() > 0;
}

void FControlRigObjectSelection::SetupChannel(FFrameNumber CurrentFrame, TArray<FFrameNumber>& KeyTimes, TArray<FKeyHandle>& Handles, 
	FMovieSceneFloatChannel* FloatChannel, FMovieSceneDoubleChannel* DoubleChannel,
	FChannelKeyBounds& KeyBounds)
{
	KeyBounds.FloatChannel = FloatChannel;
	KeyBounds.DoubleChannel = DoubleChannel;
	KeyBounds.CurrentFrame = CurrentFrame;
	KeyBounds.PreviousIndex = INDEX_NONE;
	KeyBounds.NextIndex = INDEX_NONE;
	KeyTimes.SetNum(0);
	Handles.SetNum(0);
	if (FloatChannel)
	{
		FloatChannel->GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		float OutValue;
		FloatChannel->Evaluate(FFrameTime(CurrentFrame), OutValue);
		KeyBounds.CurrentValue = OutValue;
	}
	else if (DoubleChannel)
	{
		DoubleChannel->GetKeys(TRange<FFrameNumber>(), &KeyTimes, &Handles);
		double OutValue;
		DoubleChannel->Evaluate(FFrameTime(CurrentFrame), OutValue);
		KeyBounds.CurrentValue = OutValue;
	}

	if (KeyTimes.Num() > 0)
	{
		TArrayView<const FMovieSceneFloatValue> FloatValues;
		TArrayView<const FMovieSceneDoubleValue> DoubleValues;
		if (FloatChannel)
		{
			FloatValues = FloatChannel->GetValues();
		}
		else if (DoubleChannel)
		{
			DoubleValues = DoubleChannel->GetValues();

		}
		for (int32 Index = 0; Index < KeyTimes.Num(); Index++)
		{
			const FFrameNumber FrameNumber = KeyTimes[Index];
			if (FrameNumber < CurrentFrame || (FrameNumber == CurrentFrame && KeyBounds.PreviousIndex == INDEX_NONE))
			{
				KeyBounds.PreviousIndex = Index;
				KeyBounds.PreviousFrame = FrameNumber;
				if (FloatChannel)
				{
					KeyBounds.PreviousValue = FloatValues[Index].Value;
				}
				else if (DoubleChannel)
				{
					KeyBounds.PreviousValue = DoubleValues[Index].Value;
				}
			}
			else if (FrameNumber > CurrentFrame || (FrameNumber == CurrentFrame && Index == KeyTimes.Num() -1))
			{
				KeyBounds.NextIndex = Index;
				KeyBounds.NextFrame = FrameNumber;
				if (FloatChannel)
				{
					KeyBounds.NextValue = FloatValues[Index].Value;
				}
				else if (DoubleChannel)
				{
					KeyBounds.NextValue = DoubleValues[Index].Value;
				}
				break;
			}
		}
	}
	KeyBounds.bValid = (KeyBounds.PreviousIndex != INDEX_NONE && KeyBounds.NextIndex != INDEX_NONE
		&& KeyBounds.PreviousIndex != KeyBounds.NextIndex) ? true : false;
}

TArray<UControlRig*> FControlRigObjectSelection::GetControlRigs(const TWeakPtr<FControlRigEditMode>& InEditMode)
{
	TArray<UControlRig*> ControlRigs;
	if (const FControlRigEditMode* EditMode = InEditMode.Pin().Get())
	{
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		EditMode->GetAllSelectedControls(SelectedControls);
		for (TPair<UControlRig*, TArray<FRigElementKey>>& Selected : SelectedControls)
		{
			ControlRigs.Add(Selected.Key);
		}
	}
	return ControlRigs;
}
}