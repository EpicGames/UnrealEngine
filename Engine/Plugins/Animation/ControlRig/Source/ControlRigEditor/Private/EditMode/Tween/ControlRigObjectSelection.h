// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Misc/FrameNumber.h"
#include "Templates/SharedPointerFwd.h"

class FControlRigEditMode;
class ISequencer;
class UControlRig;
class UMovieSceneSection;
struct FKeyHandle;
struct FMovieSceneDoubleChannel;
struct FMovieSceneFloatChannel;

namespace UE::ControlRigEditor
{
/** Key bounds and their values used to blend with Controls and Actor float channels */
struct FChannelKeyBounds
{
	FChannelKeyBounds()
		: bValid(false), FloatChannel(nullptr), DoubleChannel(nullptr)
		, PreviousIndex(INDEX_NONE), NextIndex(INDEX_NONE)
		, PreviousFrame(0), NextFrame(0), CurrentFrame(0), PreviousValue(0.0), NextValue(0.0), CurrentValue(0.0)
	{}
	
	bool bValid;
	FMovieSceneFloatChannel* FloatChannel;
	FMovieSceneDoubleChannel* DoubleChannel;
	int32 PreviousIndex;
	int32 NextIndex;
	FFrameNumber PreviousFrame;
	FFrameNumber NextFrame;
	FFrameNumber CurrentFrame;
	double PreviousValue;
	double NextValue;
	double CurrentValue;
};

/** Contains the selection state for a set of Control Rig Controls to blend with the anim slider */
struct FControlRigObjectSelection
{
	//set of possible float channels
	struct FObjectChannels
	{
		FObjectChannels() : Section(nullptr) {};
		TArray<FChannelKeyBounds>  KeyBounds;
		UMovieSceneSection* Section;
	};

	bool Setup(const TWeakPtr<ISequencer>& InSequencer, const TWeakPtr<FControlRigEditMode>& InEditMode);
	bool Setup(const TArray<UControlRig*>& SelectedControlRigs, const TWeakPtr<ISequencer>& InSequencer);

	TArray<FObjectChannels>  ChannelsArray;

private:

	// Either float or double will be non-null
	void SetupChannel(
		FFrameNumber CurrentFrame, TArray<FFrameNumber>& KeyTimes, TArray<FKeyHandle>& Handles,
		FMovieSceneFloatChannel* FloatChannel, FMovieSceneDoubleChannel* DoubleChannel, FChannelKeyBounds& KeyBounds
		);

	TArray<UControlRig*> GetControlRigs(const TWeakPtr<FControlRigEditMode>& InEditMode);
};
}
