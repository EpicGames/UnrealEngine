// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "ControlRigObjectSelection.h"
#include "ISequencer.h"
#include "Math/Abstraction/KeyBlendingAbstraction.h"
#include "Math/Abstraction/TweenRangeTemplates.h"
#include "Math/ContiguousKeyMapping.h"
#include "Math/CurveBlending.h"
#include "Math/Models/TweenModel.h"
#include "Misc/Attribute.h"
#include "Misc/SequencerUtils.h"
#include "Templates/SharedPointer.h"

namespace UE::ControlRigEditor
{
/**
 * Implements tweening in ControlRig
 *
 * Priority of tweening:
 *	1. If keys are selected in the curve editor, tween those keys. 
 *	2. Otherwise, if a control rig element is selected, insert a key at the current scrub position into the curves corresponding to the control rig
 *	and then interpolate those keys.
 */
template<TweeningUtilsEditor::EBlendFunction>
class TControlRigTweenModel : public TweeningUtilsEditor::FTweenModel
{
public:

	explicit TControlRigTweenModel(TAttribute<TWeakPtr<ISequencer>> InSequencerAttr, TWeakPtr<FControlRigEditMode> InControlRigMode)
		: SequencerAttr(MoveTemp(InSequencerAttr))
		, ControlRigMode(MoveTemp(InControlRigMode))
	{}
	explicit TControlRigTweenModel(TAttribute<TWeakPtr<ISequencer>> InSequencerAttr)
		: TControlRigTweenModel(MoveTemp(InSequencerAttr), nullptr)
	{}

	/** Does a blend and returns whether any values were actually blended. */
	bool BlendSingle(float InNormalizedValue);
	/** Does a blend on specified control rigs and returns whether any values were actually blended. */
	bool BlendSingleWithControlRigs(float InNormalizedValue, const TArray<UControlRig*>& InControlRigs);

	/** Inits this model with control rigs. */
	void StartBlendOperationWithControlRigs(const TArray<UControlRig*>& InControlRigs);

	//~ Begin FTweenModel Interface
	virtual void StartBlendOperation() override;
	virtual void BlendValues(float InNormalizedValue) override { DoBlend(InNormalizedValue); }
	//~ End FTweenModel Interface

private:

	/** Gets the sequencer used by Control Rig. Can return nullptr (e.g. when user has closed the sequencer), in which case blending does nothing. */
	const TAttribute<TWeakPtr<ISequencer>> SequencerAttr;
	/** The control rig mode that owns this tweener. Can be nullptr, in which case only the curve editor key selection is blended. */
	const TWeakPtr<FControlRigEditMode> ControlRigMode;

	/** The key selection to blend. Created in StartBlendOperation and used for the entirety of the blend operation. */
	TweeningUtilsEditor::FContiguousKeyMapping KeySelection;
	/**
	 * The control rig selection to blend if ContiguousKeySelection is empty.
	 * Created in StartBlendOperation and used for the entirety of the blend operation.
	 */
	FControlRigObjectSelection ObjectSelection; // TODO: Extract this to UE::ControlRigEditor namespace and deprecate the global one.

	/** @return Whether any blended occured. */
	bool DoBlend(float InNormalizedValue);
	bool BlendObjectSelection(const TSharedPtr<ISequencer>& SequencerPin, float InScaledBlendValue);
};
}

namespace UE::ControlRigEditor
{
template <TweeningUtilsEditor::EBlendFunction BlendFunction>
bool TControlRigTweenModel<BlendFunction>::BlendSingle(float InNormalizedValue)
{
	StartBlendOperation();
	const bool bBlended = DoBlend(InNormalizedValue);
	StopBlendOperation(); // Technically unneeded but it's expected by the API so we do it in case base version changes in the future
	return bBlended;
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
bool TControlRigTweenModel<BlendFunction>::BlendSingleWithControlRigs(float InNormalizedValue, const TArray<UControlRig*>& InControlRigs)
{
	StartBlendOperationWithControlRigs(InControlRigs);
	const bool bBlended = DoBlend(InNormalizedValue);
	StopBlendOperation(); // Technically unneeded but it's expected by the API so we do it in case base version changes in the future
	return bBlended;
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TControlRigTweenModel<BlendFunction>::StartBlendOperationWithControlRigs(const TArray<UControlRig*>& InControlRigs)
{
	const TWeakPtr<ISequencer> Sequencer = SequencerAttr.Get();
	if (const TSharedPtr<ISequencer> SequencerPin = Sequencer.Pin())
	{
		ObjectSelection.Setup(InControlRigs, Sequencer);
	}
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
void TControlRigTweenModel<BlendFunction>::StartBlendOperation()
{
	KeySelection.KeyMap.Empty(KeySelection.KeyMap.Num());
	ObjectSelection.ChannelsArray.Empty(ObjectSelection.ChannelsArray.Num());
	
	const TWeakPtr<ISequencer> Sequencer = SequencerAttr.Get();
	const TSharedPtr<ISequencer> SequencerPin = Sequencer.Pin();
	const TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditorFromSequencer(SequencerPin);
	if (!CurveEditor)
	{
		return;
	}

	KeySelection = TweeningUtilsEditor::FContiguousKeyMapping(*CurveEditor);
	if (KeySelection.KeyMap.IsEmpty())
	{
		ObjectSelection.Setup(Sequencer, ControlRigMode);
	}
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
bool TControlRigTweenModel<BlendFunction>::DoBlend(float InNormalizedValue)
{
	
	const TWeakPtr<ISequencer> Sequencer = SequencerAttr.Get();
	const TSharedPtr<ISequencer> SequencerPin = Sequencer.Pin();
	const TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditorFromSequencer(SequencerPin);
	if (!CurveEditor)
	{
		return false;
	}

	const float ScaledBlendValue = ScaleBlendValue(InNormalizedValue); 
	if (!KeySelection.KeyMap.IsEmpty())
	{
		using namespace UE::TweeningUtilsEditor;
		BlendCurves_BySingleKey(*CurveEditor, KeySelection,
			[this, ScaledBlendValue](
				const FCurveModelID&,
				const FContiguousKeyMapping::FContiguousKeysArray& AllBlendedKeys, const FContiguousKeys& CurrentBlendRange,
				int32 InCurrentKeyIndex
			)
		{
			return TweeningUtilsEditor::TweenRange<BlendFunction>(ScaledBlendValue, AllBlendedKeys, CurrentBlendRange, InCurrentKeyIndex);
		});
	}

	if (!ObjectSelection.ChannelsArray.IsEmpty())
	{
		BlendObjectSelection(SequencerPin, ScaledBlendValue);
	}

	return false;
}

template <TweeningUtilsEditor::EBlendFunction BlendFunction>
bool TControlRigTweenModel<BlendFunction>::BlendObjectSelection(const TSharedPtr<ISequencer>& SequencerPin, float InScaledBlendValue)
{
	using namespace UE::TweeningUtilsEditor;
	
	const FFrameTime FrameTime = SequencerPin->GetLocalTime().Time;
	const FFrameRate TickResolution = SequencerPin->GetFocusedTickResolution();
	
	bool bDidBlend = false;
	for (const FControlRigObjectSelection::FObjectChannels& ObjectChannels : ObjectSelection.ChannelsArray)
	{
		if (ObjectChannels.Section)
		{
			ObjectChannels.Section->Modify();
		}
		for (int32 Index = 0; Index < ObjectChannels.KeyBounds.Num(); ++Index)
		{
			if (ObjectChannels.KeyBounds[Index].bValid)
			{
				const double PreviousValue = ObjectChannels.KeyBounds[Index].PreviousValue;
				const double PreviousTime = TickResolution.AsSeconds(FFrameTime(ObjectChannels.KeyBounds[Index].PreviousFrame));
				const double NextValue = ObjectChannels.KeyBounds[Index].NextValue;
				const double NextTime = TickResolution.AsSeconds(FFrameTime(ObjectChannels.KeyBounds[Index].NextFrame));
				const double CurrentValue = ObjectChannels.KeyBounds[Index].CurrentValue;
				const double CurrentTime = TickResolution.AsSeconds(FFrameTime(ObjectChannels.KeyBounds[Index].CurrentFrame));

				const FVector2d BeforeBlendRange { PreviousTime, PreviousValue };
				const FVector2d AfterBlendRange { NextTime, NextValue };
				const FVector2d Current { CurrentTime, CurrentValue };
				
				FBlendRangesData AllKeys({ BeforeBlendRange, Current, AfterBlendRange});
				AllKeys.AddBlendRange({ 1 });
				const double NewValue = TweenRange<BlendFunction>(InScaledBlendValue, AllKeys, AllKeys.KeysArray[0], 0);
				
				using namespace UE::MovieScene;
				if (ObjectChannels.KeyBounds[Index].FloatChannel)
				{
					const EMovieSceneKeyInterpolation KeyInterpolation = GetInterpolationMode(ObjectChannels.KeyBounds[Index].FloatChannel, FrameTime.GetFrame(), SequencerPin->GetKeyInterpolation());
					AddKeyToChannel(ObjectChannels.KeyBounds[Index].FloatChannel, FrameTime.GetFrame(), (float)NewValue, KeyInterpolation);
				}
				else if (ObjectChannels.KeyBounds[Index].DoubleChannel)
				{
					const EMovieSceneKeyInterpolation KeyInterpolation = GetInterpolationMode(ObjectChannels.KeyBounds[Index].DoubleChannel, FrameTime.GetFrame(), SequencerPin->GetKeyInterpolation());
					AddKeyToChannel(ObjectChannels.KeyBounds[Index].DoubleChannel, FrameTime.GetFrame(), NewValue, KeyInterpolation);
				}
				bDidBlend = true;
			}
		}
	}
	return bDidBlend;
}
}

