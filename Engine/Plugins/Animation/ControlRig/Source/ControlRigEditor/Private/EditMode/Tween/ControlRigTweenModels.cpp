// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigTweenModels.h"

#include "ControlRigTweenModel.h"
#include "Math/Abstraction/TweenRangeTemplates.h"
#include "Math/Models/CurveTimeOffsetTweenModel.h"
#include "Math/Models/EditorTweenModel.h"
#include "Misc/SequencerUtils.h"

namespace UE::ControlRigEditor
{
namespace TweenDetail
{
static TAttribute<TWeakPtr<FCurveEditor>> MakeCurveEditorAttribute(TAttribute<TWeakPtr<ISequencer>> InSequencerAttr)
{
	return TAttribute<TWeakPtr<FCurveEditor>>::CreateLambda([SequencerAttr = InSequencerAttr]
	{
		const TSharedPtr<ISequencer> SequencerPin = SequencerAttr.Get().Pin();
		return GetCurveEditorFromSequencer(SequencerPin);
	});
}

static TArray<TweeningUtilsEditor::FTweenModelUIEntry> MakeTweenModels(
	const TAttribute<TWeakPtr<ISequencer>>& InSequencerAttr, const TSharedRef<FControlRigEditMode>& InOwningEditMode
	)
{
	using namespace TweeningUtilsEditor;

	TArray<FTweenModelUIEntry> Result;
	const TAttribute<TWeakPtr<FCurveEditor>> CurveEditorAttr = MakeCurveEditorAttribute(InSequencerAttr);
	const auto AddCurveTweenable = [&InSequencerAttr, &InOwningEditMode, &CurveEditorAttr, &Result]<EBlendFunction TBlendFunction>
	{
		using FModel = TEditorTweenModel<TControlRigTweenModel<TBlendFunction>>;
		TUniquePtr<FModel> TweenModel = MakeUnique<FModel>(
			CurveEditorAttr,										// TTransactionalTweenModelProxy
			CurveEditorAttr,										// TTangentFlatteningTweenProxy
			InSequencerAttr, InOwningEditMode						// FControlRigTweenModel
			);
		Result.Emplace(MoveTemp(TweenModel), FTweenModelDisplayInfo(TBlendFunction));
	};
	static_assert(static_cast<int32>(EBlendFunction::Num) == 7, "You probably want to add the new tween function here.");

	// Could use TweeningUtilsEditor::ForEachCurveTweenable, but the functions are supposed to be listed in this specific order.
	AddCurveTweenable.template operator()<EBlendFunction::BlendNeighbor>();
	AddCurveTweenable.template operator()<EBlendFunction::PushPull>();
	AddCurveTweenable.template operator()<EBlendFunction::BlendEase>();
	AddCurveTweenable.template operator()<EBlendFunction::BlendRelative>();
	Result.Emplace(
		MakeUnique<TEditorTweenModel<FCurveTimeOffsetTweenModel>>(
			CurveEditorAttr,		// TTransactionalTweenModelProxy
			CurveEditorAttr,		// TTangentFlatteningTweenProxy
			CurveEditorAttr			// FCurveTimeOffsetTweenModel
			),
		FTweenModelDisplayInfo(EBlendFunction::TimeOffset)
	);
	AddCurveTweenable.template operator()<EBlendFunction::SmoothRough>();
	AddCurveTweenable.template operator()<EBlendFunction::ControlsToTween>();

	return Result;
}
}

FControlRigTweenModels::FControlRigTweenModels(
	const TAttribute<TWeakPtr<ISequencer>>& InSequencerAttr, const TSharedRef<FControlRigEditMode>& InOwningEditMode
	)
	: FTweenModelArray(TweenDetail::MakeTweenModels(InSequencerAttr, InOwningEditMode))
{}
}
