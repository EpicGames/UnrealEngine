// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Abstraction/TweenModelArray.h"
#include "Templates/SharedPointerFwd.h"

class ISequencer;
class FControlRigEditMode;
class FCurveEditor;
template<typename ObjectType>class TAttribute;

namespace UE::ControlRigEditor
{
/** Knows of the tween models used in control rig. */
class FControlRigTweenModels : public TweeningUtilsEditor::FTweenModelArray
{
public:

	explicit FControlRigTweenModels(const TAttribute<TWeakPtr<ISequencer>>& InSequencerAttr, const TSharedRef<FControlRigEditMode>& InOwningEditMode);
};
}