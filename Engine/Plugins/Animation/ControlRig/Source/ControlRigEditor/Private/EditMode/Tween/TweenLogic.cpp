// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweenLogic.h"

#include "ControlRigTweenModel.h"
#include "ControlRigTweenModels.h"
#include "EditMode/ControlRigEditMode.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"

namespace UE::ControlRigEditor
{
FTweenLogic::FTweenLogic(const TAttribute<TWeakPtr<ISequencer>>& InSequencerAttr, const TSharedRef<FControlRigEditMode>& InOwningEditMode)
	: CommandList(InOwningEditMode->GetCommandBindings().ToSharedRef())
	, TweenModels(MakeShared<FControlRigTweenModels>(InSequencerAttr, InOwningEditMode))
	, Controllers(CommandList, TweenModels, TEXT("ControlRigTween"))
{}

FTweenLogicWidgets FTweenLogic::MakeWidget() const
{
	// Visually move the slider widget when the user uses the U+LMB button command to indirectly move the mouse
	TAttribute<TOptional<float>> OverrideSliderPosition =
		TAttribute<TOptional<float>>::CreateLambda([this]()
		{
			return Controllers.GetCurrentSliderPosition();
		});
	
	FSlimHorizontalToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None);
	ToolBarBuilder.SetStyle(&FAppStyle::Get(), TEXT("ControlRigTweenToolbar"));
	const TweeningUtilsEditor::FTweenToolbarController::FAddToToolbarResult ToolbarWidgets = Controllers.ToolbarController.AddToToolbar(
		ToolBarBuilder, { MoveTemp(OverrideSliderPosition) }
		);
	return FTweenLogicWidgets(ToolBarBuilder.MakeWidget(), ToolbarWidgets.TweenSlider);
}
}
