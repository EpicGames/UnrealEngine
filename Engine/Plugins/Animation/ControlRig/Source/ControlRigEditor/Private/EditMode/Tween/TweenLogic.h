// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Templates/SharedPointerFwd.h"
#include "Widgets/MVC/TweenControllers.h"

class FControlRigEditMode;
class FUICommandList;
class ISequencer;

namespace UE::ControlRigEditor
{
class FControlRigTweenModels;

struct FTweenLogicWidgets
{
	TSharedPtr<SWidget> RootWidget;
	TSharedPtr<TweeningUtilsEditor::STweenSlider> TweenSlider;

	explicit FTweenLogicWidgets(TSharedPtr<SWidget> InRootWidget, TSharedPtr<TweeningUtilsEditor::STweenSlider> InTweenSlider)
		: RootWidget(MoveTemp(InRootWidget)), TweenSlider(MoveTemp(InTweenSlider))
	{}
};

/** Creates the tweening model specific to ControlRig and provides an interface for interacting with it. */
class FTweenLogic
{
public:

	explicit FTweenLogic(const TAttribute<TWeakPtr<ISequencer>>& InSequencerAttr, const TSharedRef<FControlRigEditMode>& InOwningEditMode);

	/** Constructs the content for the viewport tweening widget. */
	FTweenLogicWidgets MakeWidget() const;

	TweeningUtilsEditor::FTweenControllers& GetControllers() { return Controllers; }
	
private:

	/** Owning control rig's command list. */
	const TSharedRef<FUICommandList> CommandList;

	/** Holds the used tweening functions and info about how to display them in UI. */
	const TSharedRef<FControlRigTweenModels> TweenModels;

	/** Common functionality that should be shared consistent with other editor modules, such as Curve Editor. */
	TweeningUtilsEditor::FTweenControllers Controllers;
};
}
