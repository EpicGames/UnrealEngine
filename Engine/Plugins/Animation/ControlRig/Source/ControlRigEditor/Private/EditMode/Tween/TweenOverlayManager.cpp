// Copyright Epic Games, Inc. All Rights Reserved.

#include "TweenOverlayManager.h"

#include "EditMode/ControlRigEditModeCommands.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "Editor/ControlRigEditorCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "ILevelSequenceEditorToolkit.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "TweeningUtilsStyle.h"
#include "Misc/Flyout/Host/ToolkitBasedWidgetHost.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/STweenSlider.h"
#include "Widgets/TweenSliderStyle.h"

namespace UE::ControlRigEditor
{
namespace Private
{
static void SaveWidgetState(FToolWidget_FlyoutSavedState State)
{
	UControlRigEditModeSettings* ControlRigEditModeSettings = GetMutableDefault<UControlRigEditModeSettings>();
	ControlRigEditModeSettings->LastUIStates.TweenOverlayState = State;
	ControlRigEditModeSettings->SaveConfig();
}
}
	
FTweenOverlayManager::FTweenOverlayManager(
	const TSharedRef<IToolkitHost>& InToolkitHost,
	const TSharedRef<FUICommandList>& InToolkitCommandList,
	const TSharedRef<FControlRigEditMode>& InOwningEditMode
	)
	: TweenControllers(
		TAttribute<TWeakPtr<ISequencer>>::CreateLambda([]
		{
			ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
			IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, false);
			ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
			return LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
		}),
		InOwningEditMode
		)
	, WidgetHierarchy(TweenControllers.MakeWidget())
	, FlyoutWidgetManager(
		FFlyoutWidgetArgs(
			WidgetHierarchy.RootWidget.ToSharedRef(),
			MakeShared<FToolkitBasedWidgetHost>(InToolkitHost)
			)
			.SetStateToRestoreFrom(GetMutableDefault<UControlRigEditModeSettings>()->LastUIStates.TweenOverlayState)
			.SetSaveStateDelegate(FSaveFlyoutState::CreateStatic(&Private::SaveWidgetState))
			.SetCanSubdue(TAttribute<bool>::CreateLambda([this]{ return !TweenControllers.GetControllers().MouseSlidingController.IsSliding(); }))
			.SetSubduedOpacity(TAttribute<FLinearColor>::CreateLambda([]{ return GetMutableDefault<UControlRigEditModeSettings>()->TweenOutOfFocusTint; }))
			.SetAbsOffsetFromCursor(TAttribute<FVector2f>::CreateRaw(this, &FTweenOverlayManager::ComputeOffsetFromWidgetCenterToSliderPosition))
		)
	, FlyoutCommands(FlyoutWidgetManager, 
		FFlyoutCommandArgs(InToolkitCommandList)
		.SetToggleVisibility(FControlRigEditModeCommands::Get().ToggleTweenWidget)
		.SetSummonToCursor(FControlRigEditModeCommands::Get().SummonTweenWidget)
		)
{
	TweeningUtilsEditor::FTweenControllers& Controllers = TweenControllers.GetControllers();
	Controllers.ToolbarController.OnOvershootModeCommandInvoked().AddRaw(this, &FTweenOverlayManager::OnTweenCommandInvoked);
	Controllers.CycleFunctionController.OnFunctionCycleCommandInvoked().AddRaw(this, &FTweenOverlayManager::OnTweenCommandInvoked);
	
	TweeningUtilsEditor::FTweenMouseSlidingController& MouseSlider = Controllers.MouseSlidingController;
	MouseSlider.OnStartSliding().AddRaw(this, &FTweenOverlayManager::OnStartIndirectSliding);
	MouseSlider.OnStopSliding().AddRaw(this, &FTweenOverlayManager::OnStopIndirectSliding);

	FlyoutWidgetManager.OnVisibilityChanged().AddRaw(this, &FTweenOverlayManager::OnVisibilityChanged);
	OnVisibilityChanged(IsShowingWidget());
}

FVector2f FTweenOverlayManager::ComputeOffsetFromWidgetCenterToSliderPosition() const
{
	const FGeometry& RootGeometry = WidgetHierarchy.RootWidget->GetTickSpaceGeometry();
	const FGeometry& TweenSliderGeometry = WidgetHierarchy.TweenSlider->GetTickSpaceGeometry();
	const bool bHasEverBeenPainted = !RootGeometry.GetLocalSize().IsNearlyZero() && !TweenSliderGeometry.GetLocalSize().IsNearlyZero();
	if (!ensure(bHasEverBeenPainted)) // FlyoutWidgetManager should have made sure that the widget was painted at least once.
	{
		return FVector2f::ZeroVector;
	}
	
	const float AbsCenterOfRoot = RootGeometry.GetAbsolutePosition().X + RootGeometry.GetAbsoluteSize().X / 2.f;
	const float AbsCenterOfSlider = TweenSliderGeometry.GetAbsolutePosition().X + TweenSliderGeometry.GetAbsoluteSize().X / 2.f;
	const float CenterOfRootToCenterOfSlider = AbsCenterOfSlider - AbsCenterOfRoot;
	
	// No DPI scaling required because we are already computing in absolute screen space.
	return FVector2f(-CenterOfRootToCenterOfSlider, 0.f);
}

void FTweenOverlayManager::OnStartIndirectSliding()
{
	// The widget should be fully visible while indirectly driving the slider
	FlyoutWidgetManager.UnsubdueContent();
	
	if (GetMutableDefault<UControlRigEditModeSettings>()->bIndirectSliderMovementShouldSnapSliderToMouse)
	{
		// For indirect slider manipulation, the user is allowed to have the cursor anywhere in the editor.
		// We'll just snap the widget to the viewport border if that is the case.
		constexpr ETemporaryFlyoutPositionFlags Flags = ETemporaryFlyoutPositionFlags::AllowCursorOutsideOfParent;
		IndirectSlidePositionOverride = FlyoutWidgetManager.TryTemporarilyPositionWidgetAtCursor(Flags);
	}
}

void FTweenOverlayManager::OnStopIndirectSliding()
{
	IndirectSlidePositionOverride.Reset();
	
	// Subdue the content again, but don't force it in case the user was already hovering the widget when they started indirectly sliding it.
	FlyoutWidgetManager.TrySubdueContent();
}

void FTweenOverlayManager::OnVisibilityChanged(bool bIsVisible)
{
	// FControlRigEditModeCommands::SummonTweenWidget is bound to U (default)
	// FTweeningUtilsCommands::DragAnimSliderTool is bound to U (default), as well. While pressed, once you LMB, you can indirectly manipulate the slider.
	//
	// When visible: Indirect slider manipulation SHOULD occur. SummonTweenWidget will place the widget under the cursor such that the slider is directly under the mouse.
	// When not visible: Indirect slider manipulation SHOULD NOT occur. DragAnimSliderTool will take precedence over SummonTweenWidget.
	if (bIsVisible && !FlyoutWidgetManager.IsTemporarilyRepositioned())
	{
		TweenControllers.GetControllers().MouseSlidingController.BindCommand();
		FlyoutCommands.UnbindSummonCommand();
	}
	else
	{
		TweenControllers.GetControllers().MouseSlidingController.UnbindCommand();
		FlyoutCommands.BindSummonCommand();
	}
}

void FTweenOverlayManager::OnTweenCommandInvoked()
{
	// If the widget is hidden when a command is used on it, temporarily show the widget and move it to the cursor until the user moves the mouse away.
	// That allows the user to "prepare" the tween widget while they've decided to hide it.
	if (!IsShowingWidget())
	{
		FlyoutWidgetManager.TryTemporarilyPositionWidgetAtCursor(ETemporaryFlyoutPositionFlags::HideAtEnd);
	}
}
}
