// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditMode/ControlRigBaseDockableView.h"
#include "Misc/Flyout/FlyoutOverlayManager.h"
#include "TweenLogic.h"
#include "Misc/Flyout/Command/FlyoutCommandBinder.h"

namespace UE::ControlRigEditor
{
/**
 * Manages all logic around the tween overlay widget.
 * 
 * This class is responsible for holding any Control Rig specific state and injects it into a FFlyoutOverlayManager, which handles all the UI
 * logic for docking the tween slider to the viewport, etc.
 *
 * ========== Command precedence ==========
 * Both FControlRigEditModeCommands::SummonTweenWidget and FTweeningUtilsCommands::DragAnimSliderTool are bound to U by default.
 * While the flyout widget is visible, DragAnimSliderTool takes precedence. This is achieved by binding:
 * - DragAnimSliderTool to FControlRigEditModeToolkit::GetCommandBindings().
 * - SummonTweenWidget to FControlRigEditMode::GetCommandBindings().
 *
 * Sequence of input events in the engine is:
 * 1. FEditorModeTools::InputKey, which invokes FControlRigEditModeToolkit::GetCommandBindings()->ProcessCommandBindings
 * 2. SEditorViewport::OnKeyDown
 * 3. SLevelViewport::OnKeyDown
 * 4. SLevelEditor::OnKeyDown, which invokes FControlRigEditModeToolkit::ProcessCommandBindings
 * There are other commands bound to e.g. shift+U in SEditorViewport, so it's important the toolkit's tween commands are processed first. So they
 * must go into FControlRigEditModeToolkit::GetCommandBindings(). This would be fixed if FEditorModeTools::InputKey called
 * FControlRigEditModeToolkit::ProcessCommandBindings instead.
 * ========================================
 */
class FTweenOverlayManager : public FNoncopyable
{
public:
	
	explicit FTweenOverlayManager(
		const TSharedRef<IToolkitHost>& InToolkitHost,
		const TSharedRef<FUICommandList>& InToolkitCommandList,
		const TSharedRef<FControlRigEditMode>& InOwningEditMode
		);

	void ToggleVisibility() { FlyoutWidgetManager.ToggleVisibility(); }
	void ShowWidget() { FlyoutWidgetManager.ShowWidget(); }
	void HideWidget() { FlyoutWidgetManager.HideWidget(); }
	bool IsShowingWidget() const { return FlyoutWidgetManager.IsShowingWidget(); }

private:
	
	/** Manages logic for the tweening widget. */
	FTweenLogic TweenControllers;

	/** The widget hierarchy in the flyout widget manager. */
	const FTweenLogicWidgets WidgetHierarchy;
	/** Contains the tween slider widget. Allows it to be dragged around, commands, etc.  */
	FFlyoutOverlayManager FlyoutWidgetManager;
	/** Process the command bindings that affect FlyoutWidgetManager. */
	FFlyoutCommandBinder FlyoutCommands;
	
	/** Set while the user is sliding using U+LMB. */
	TSharedPtr<FFlyoutTemporaryPositionOverride> IndirectSlidePositionOverride;

	/** @return Computes the offset the widget should have from the mouse cursor so the center of the slider is under it. */
	FVector2f ComputeOffsetFromWidgetCenterToSliderPosition() const;

	// When using U+LMB, show move the tween overlay to the mouse.
	void OnStartIndirectSliding();
	void OnStopIndirectSliding();

	/**
	 * While invisible, gives precedence to SummonTweenWidget by deactivating indirect sliding (DragAnimSliderTool, U+LMB)
	 * because by default both commands are bound to the same hotkey.
	 */
	void OnVisibilityChanged(bool bIsVisible);

	/** If the widget is hidden, temporarily shows the widget and snaps it to the cursor until the user moves the mouse away. */
	void OnTweenCommandInvoked();
};
}
