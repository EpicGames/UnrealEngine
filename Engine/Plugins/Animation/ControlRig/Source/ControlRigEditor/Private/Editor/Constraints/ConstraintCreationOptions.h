// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "UObject/Object.h"

#include "ConstraintCreationOptions.generated.h"

/**
 * UConstraintCreationOptions is used to pass options for creating constraints.
 */

UCLASS(Blueprintable)
class UConstraintCreationOptions : public UObject
{
	GENERATED_BODY()

public:

	/* Creation options related to sequencer. */
	UPROPERTY(EditAnywhere, Category=Options)
	FSequencerCreationOptions SequencerOptions;
};
