// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConstraintsWidget.h"

#include "ActorPickerMode.h"
#include "ConstraintCreationOptions.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "ConstraintsActor.h"
#include "ControlRigEditorStyle.h"
#include "DetailLayoutBuilder.h"
#include "LevelEditorViewport.h"
#include "PropertyCustomizationHelpers.h"
#include "SlateOptMacros.h"
#include "LevelEditor.h"
#include "SSocketChooser.h"
#include "Modules/ModuleManager.h"
#include "Misc/ScopeExit.h"
#include "Widgets/Input/SButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyEditorModule.h"
#include "DetailsViewArgs.h"
#include "Tools/ConstraintBaker.h"
#include "ScopedTransaction.h"
#include "ISequencer.h"
#include "ConstraintsManager.h"
#include "ControlRigGizmoActor.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "FrameNumberDetailsCustomization.h"
#include "SActionButton.h"
#include "SAssetDropTarget.h"
#include "SControlRigDismissDependencyDialog.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Sequencer/EditModeAnimationUtil.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"

#define LOCTEXT_NAMESPACE "SConstraintsWidget"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

const TArray< const FSlateBrush* >& FConstraintInfo::GetBrushes()
{
	static const TArray< const FSlateBrush* > Brushes({
		FAppStyle::Get().GetBrush("EditorViewport.TranslateMode"),
		FAppStyle::Get().GetBrush("EditorViewport.RotateMode"),
		FAppStyle::Get().GetBrush("EditorViewport.ScaleMode"),
		FAppStyle::Get().GetBrush("Icons.ConstraintManager.ParentHierarchy"),
		FAppStyle::Get().GetBrush("Icons.ConstraintManager.LookAt")
		});
	return Brushes;
}

const TMap< UClass*, ETransformConstraintType >& FConstraintInfo::GetConstraintToType()
{
	static const TMap< UClass*, ETransformConstraintType > ConstraintToType({
		{UTickableTranslationConstraint::StaticClass(), ETransformConstraintType::Translation},
		{UTickableRotationConstraint::StaticClass(), ETransformConstraintType::Rotation},
		{UTickableScaleConstraint::StaticClass(), ETransformConstraintType::Scale},
		{UTickableParentConstraint::StaticClass(), ETransformConstraintType::Parent},
		{UTickableLookAtConstraint::StaticClass(), ETransformConstraintType::LookAt}
		});
	return ConstraintToType;
}

const TArray< UTickableTransformConstraint* >& FConstraintInfo::GetConfigurableConstraints()
{
	using namespace UE::TransformConstraintUtil;
	static TArray< UTickableTransformConstraint* > ConfigurableConstraints({
		GetConfig(UTickableTranslationConstraint::StaticClass()),
		GetConfig(UTickableRotationConstraint::StaticClass()),
		GetConfig(UTickableScaleConstraint::StaticClass()),
		GetConfig(UTickableParentConstraint::StaticClass()),
		GetConfig(UTickableLookAtConstraint::StaticClass()),
	});
	return ConfigurableConstraints;
}

const FSlateBrush* FConstraintInfo::GetBrush(uint8 InType)
{
	static const UEnum* ETransformConstraintTypeEnum = StaticEnum<ETransformConstraintType>();
	if (ETransformConstraintTypeEnum->IsValidEnumValue(InType))
	{
		return GetBrushes()[InType];
	}

	return FAppStyle::Get().GetDefaultBrush();
}

int8 FConstraintInfo::GetType(UClass* InClass)
{
	if (const ETransformConstraintType* TransformConstraint = GetConstraintToType().Find(InClass))
	{
		return static_cast<int8>(*TransformConstraint); 
	}
	return -1;
}

UTickableTransformConstraint* FConstraintInfo::GetConfigurable(ETransformConstraintType InType)
{
	static const UEnum* ETransformConstraintTypeEnum = StaticEnum<ETransformConstraintType>();
	const uint8 ConstraintType = static_cast<uint8>(InType);
	if (ETransformConstraintTypeEnum->IsValidEnumValue(ConstraintType))
	{
		return GetConfigurableConstraints()[ConstraintType];
	}
	return nullptr;
}

namespace UE::Private
{
	UWorld* GetCurrentWorld()
	{
		return GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	}
	
	// NOTE we use this function to get the current selection as control actors are Temporary Editor Actors so won't be
	// pushed added to the selection list.
	TArray<AActor*> GetCurrentSelection()
	{
		const UWorld* World = GetCurrentWorld();
		const ULevel* CurrentLevel = IsValid(World) ? World->GetCurrentLevel() : nullptr;
		if (!CurrentLevel)
		{
			static const TArray<AActor*> DummyArray;
			return DummyArray;
		}

		return ObjectPtrDecay(CurrentLevel->Actors).FilterByPredicate( [](const AActor* Actor)
		{
			return IsValid(Actor) && Actor->IsSelected();
		});	
	}

	static TWeakPtr<ISequencer> GetSequencerChecked()
	{
		const TWeakPtr<ISequencer> WeakSequencer = AnimationEditMode::GetSequencer();
		if (!WeakSequencer.IsValid() || !WeakSequencer.Pin()->GetFocusedMovieSceneSequence())
		{
			return nullptr;
		}
	
		const UMovieScene* MovieScene = WeakSequencer.Pin()->GetFocusedMovieSceneSequence()->GetMovieScene();
		if (!MovieScene)
		{
			return nullptr;
		}

		return WeakSequencer;
	}

	void GetConstrainable(AActor* InActor, FConstrainable& InOutConstrainable)
	{
		InOutConstrainable = FConstrainable();

		if (!InActor)
		{
			return;
		}
	
		InOutConstrainable.Actor = InActor;
		InOutConstrainable.Object = InActor;
	
		// gather sub components with sockets
		const TInlineComponentArray<USceneComponent*> Components = InActor->IsA<AControlRigShapeActor>() ?
			TInlineComponentArray<USceneComponent*>() : TInlineComponentArray<USceneComponent*>(InActor);
		const TArray<USceneComponent*> ComponentsWithSockets = Components.FilterByPredicate([](const USceneComponent* Component)
		{
			return Component->HasAnySockets();
		});

		// if no component socket available then return the actor
		const int32 NumComponentsWithSockets = ComponentsWithSockets.Num();
		if (NumComponentsWithSockets == 0)
		{
			return;
		}

		// creates a menu encapsulating InContent  
		auto CreateMenu = [](const TSharedRef<SWidget>& InContent, const FVector2D& InLocation)
		{
			const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			const TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();
			FSlateApplication::Get().PushMenu(
				LevelEditor.ToSharedRef(),
				FWidgetPath(),
				InContent,
			InLocation,
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu ) );
		};

		// creates a new socket chooser popup widget
		auto GetSocketChooserWidget = [&InOutConstrainable](USceneComponent* Component)
		{
			return SNew(SSocketChooserPopup)
			.SceneComponent( Component )
			.OnSocketChosen_Lambda([&InOutConstrainable, Component](FName InSocketName)
			{
				InOutConstrainable.Object = Component;
				InOutConstrainable.Socket = InSocketName;
			});
		};

		// decal the menu 
		const FVector2D SummonLocation = FSlateApplication::Get().GetCursorPos();
	
		// if one component with sockets then constrain the component selecting the socket
		if (NumComponentsWithSockets == 1)
		{
			return CreateMenu( GetSocketChooserWidget(ComponentsWithSockets[0]), FSlateApplication::Get().GetCursorPos());
		}

		// if there are several of them, then build a component chooser first then the socket chooser
		static constexpr bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder(CloseAfterSelection, nullptr);
		MenuBuilder.BeginSection("ChooseComp", LOCTEXT("ChooseComponentSection", "Choose Component"));
		{
			for (USceneComponent* Component: ComponentsWithSockets)
			{
				MenuBuilder.AddMenuEntry(FText::FromName(Component->GetFName()),
				FText(),
					FSlateIconFinder::FindIconForClass(Component->GetClass(), TEXT("SCS.Component")),
					FUIAction(FExecuteAction::CreateLambda([CreateMenu, GetSocketChooserWidget, Component]()
					{
						CreateMenu(GetSocketChooserWidget(Component), FSlateApplication::Get().GetCursorPos());
					})),
					NAME_None,
				   EUserInterfaceActionType::Button);
			}
		}
		MenuBuilder.EndSection();
		CreateMenu(MenuBuilder.MakeWidget(), SummonLocation);
	}

	bool CreateConstraint(const TArray<AActor*>& InChildren, const FConstrainable& InParent, const ETransformConstraintType InConstraintType, const bool bUseDefault)
	{
		UObject* Parent = InParent.Object.Get();
		UWorld* World = GetCurrentWorld();
	
		if (!Parent || !IsValid(World))
		{
			return false;
		}
	
		if (InChildren.IsEmpty())
		{
			return false;
		}

		const TWeakPtr<ISequencer> WeakSequencer = GetSequencerChecked();
	
		bool bCreated = false;
		for (AActor* Child: InChildren)
		{
			if (Child != InParent.Object)
			{

				FScopedTransaction Transaction(LOCTEXT("CreateConstraintKey", "Create Constraint Key"));

				TUniquePtr<FControlRigDismissDependencyDialogGuard> DependencyDialogGuard;
				TFunction<void()> ValidDependency;
				const AControlRigShapeActor* ChildActor = Cast<AControlRigShapeActor>(Child);
				if (const UControlRig* ControlRig = ChildActor ? ChildActor->ControlRig.Get() : nullptr)
				{
					// enable the dependency dialog when creating a constraint for children that are control rig controls
					DependencyDialogGuard = MakeUnique<FControlRigDismissDependencyDialogGuard>(ControlRig->GetHierarchy());
					ValidDependency = [&DependencyDialogGuard]()
					{
						DependencyDialogGuard.Reset();
					};
				}
				
				UTickableTransformConstraint* Constraint = TransformConstraintUtil::CreateAndAddFromObjects(
					World, Parent, InParent.Socket, Child, NAME_None, InConstraintType, true, bUseDefault, ValidDependency);

				if (Constraint)
				{
					bCreated = true;

					if (WeakSequencer.IsValid())
					{
						const UConstraintCreationOptions* Options = bUseDefault ? GetDefault<UConstraintCreationOptions>() : nullptr;
						FControlRigParameterTrackEditor::AddConstraintToSequencer(WeakSequencer.Pin(), Constraint, Options);
					}
					else
					{
						FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
						Controller.StaticConstraintCreated(Constraint);
					}		
				}
			}
		}

		return bCreated;
	}

	bool IsActorConstrainable(const AActor* InActor)
	{
		if (!InActor)
		{
			return false;
		}

		if (!InActor->GetRootComponent())
		{
			return false;
		}
		
		return true;
	}
	
	const TCHAR* HandleLabelDelimiter = TEXT("|");
}

/**
 * SConstraintItem
 */

void SConstraintMenuEntry::Construct(
		const FArguments& InArgs,
		const ETransformConstraintType& InType)
{
	ConstraintType = InType;
	OnConstraintCreated = InArgs._OnConstraintCreated;
	OnGetParent = InArgs._OnGetParent;
	
	const FButtonStyle& ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>( "Menu.Button" );

	// enum to string
	const UEnum* ETransformConstraintTypeEnum = StaticEnum<ETransformConstraintType>();
	const FString TypeStr = ETransformConstraintTypeEnum->GetNameStringByValue((uint8)ConstraintType);

	// tooltip
	const FString ToolTipStr = FString::Printf(TEXT("Create new %s constraint."), *TypeStr);
	const TSharedPtr<IToolTip> ToolTip = FSlateApplicationBase::Get().MakeToolTip(FText::FromString(ToolTipStr));

	const float MenuIconSize = FAppStyle::Get().GetFloat("Menu.MenuIconSize");

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew( SHorizontalBox )

		+ SHorizontalBox::Slot()
		[
			SNew(SBorder)
			.BorderImage_Lambda( [this, &ButtonStyle]
			{
				if (bIsPressed) { return &ButtonStyle.Pressed; }
				if (IsHovered()) { return &ButtonStyle.Hovered; }
				return &ButtonStyle.Normal;
			})
			.ForegroundColor(FLinearColor::White)
			.ToolTip( ToolTip )
			[
				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.Padding(14.0f, 0.f, 10.f, 0.0f)
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(MenuIconSize)
					.HeightOverride(MenuIconSize)
					[
						SNew(SImage)
						.Image(FConstraintInfo::GetBrush((uint8)ConstraintType))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(1.f, 0.f, 25.f, 0.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew( STextBlock )
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Text_Lambda( [TypeStr] { return FText::FromString(TypeStr); } )
				]
			]
		]
		
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(FMargin(0.f, 1.f, 0.f, 1.f))
		.AutoWidth()
		[
			SNew(SComboButton)
			.ComboButtonStyle(FControlRigEditorStyle::Get(), "ConstraintManager.ComboButton")
			.MenuContent()
			[
				GenerateConstraintDefaultWidget()
			]
		]
	];
}

FReply SConstraintMenuEntry::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsPressed = true;
		static constexpr bool bUseMutableDefault = false;
		return CreateFromSelectionOrPicker(bUseMutableDefault);
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SConstraintMenuEntry::GenerateConstraintDefaultWidget()
{
	// static constexpr bool CloseAfterSelection = true;
	// FMenuBuilder MenuBuilder(CloseAfterSelection, nullptr);

	static FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bCustomFilterAreaLocation = true;
		DetailsViewArgs.bCustomNameAreaLocation = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.ColumnWidth = 0.45f;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
	}
	
	TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

	ConfigurableConstraint = FConstraintInfo::GetConfigurable(ConstraintType);

	if (!ensure(ConfigurableConstraint))
	{
		return SNullWidget::NullWidget;
	}
	
	// manage properties visibility
	const bool bIsLookAtConstraint = ConfigurableConstraint->GetClass() == UTickableLookAtConstraint::StaticClass();
	const auto PropertyVisibility = FIsPropertyVisible::CreateLambda(
		[bIsLookAtConstraint](const FPropertyAndParent& InPropertyAndParent)
		{
			const FName PropertyName = InPropertyAndParent.Property.GetFName();
		
			// hide active property
			static const FName ActivePropName = GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, Active);
			if (PropertyName == ActivePropName)
			{
				return false;
			}

			// hide offset properties for look at constraints
			if (bIsLookAtConstraint)
			{
				static const FName MaintainOffsetPropName = GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bMaintainOffset);
				static const FName DynamicOffsetPropName = GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bDynamicOffset);
				if (PropertyName == MaintainOffsetPropName || PropertyName == DynamicOffsetPropName)
				{
					return false;
				}
			}

			static const FName SequencerOptionsPropName = GET_MEMBER_NAME_CHECKED(UConstraintCreationOptions, SequencerOptions);
			if (PropertyName == SequencerOptionsPropName)
			{
				TWeakPtr<ISequencer> WeakSequencer = UE::Private::GetSequencerChecked();
				return UE::Private::GetSequencerChecked().IsValid();
			}
			
			return true;
		});
	DetailsView->SetIsPropertyVisibleDelegate(PropertyVisibility);

	TArray<UObject*> Objects(
	{
		ConfigurableConstraint,
		GetMutableDefault<UConstraintCreationOptions>()
	});
	DetailsView->SetObjects(Objects);

	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign( VAlign_Center )
			[
				SNew(SBox)
				.WidthOverride(350.f) // Make all the menus have a consistent size & prevent details from being too narrow to read the property texts. 
				[
					DetailsView
				]
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 1.0f )
			.VAlign( VAlign_Center )
			[
				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Fill)
				[
					SNew(SSpacer)
				]
		
				+SHorizontalBox::Slot()
				.Padding(16.f, 1.f)
				.AutoWidth()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("CreateButtonLabel", "Create"))
					.OnClicked_Lambda([this]()
					{
						static constexpr bool bUseMutableDefault = true;
						return CreateFromSelectionOrPicker(bUseMutableDefault);
					})
				]
			]
		];
}

FReply SConstraintMenuEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = false;
	}

	return FReply::Unhandled();
}

FReply SConstraintMenuEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsPressed = false;
	return FReply::Handled();
}

FReply SConstraintMenuEntry::CreateFromSelectionOrPicker(const bool bUseDefault) const
{
	// We need to hide the creation menu. This destroys the menu that is keeping this SConstraintMenuEntry alive.
	// So close only at the end because we're still accessing members in this call (undefined behaviour).
	ON_SCOPE_EXIT { 	FSlateApplication::Get().DismissAllMenus(); };
	
	// use the selected parent if any
	const FConstrainable Parent = OnGetParent.IsBound() ? OnGetParent.Execute() : FConstrainable();
	if (Parent.Object.IsValid())
	{
		const TArray<AActor*> Children = UE::Private::GetCurrentSelection();
		const bool bCreated = UE::Private::CreateConstraint(Children, Parent, ConstraintType, bUseDefault);
		if (!bCreated)
		{
			return FReply::Unhandled();
		}

		// update list
		if (OnConstraintCreated.IsBound())
		{
			OnConstraintCreated.Execute();
		}

		return FReply::Handled();
	}

	// pick the parent
	return CreateSelectionPicker(bUseDefault);
}

FReply SConstraintMenuEntry::CreateSelectionPicker(const bool bUseDefault) const
{
	struct FCreationArgs
	{
		FOnConstraintCreated CreationDelegate;
		ETransformConstraintType ConstraintType = ETransformConstraintType::Parent;
		bool bUseDefault = false;
	};

	FCreationArgs Args;
	Args.CreationDelegate = OnConstraintCreated;
	Args.ConstraintType = ConstraintType;
	Args.bUseDefault = bUseDefault;
	
	static const FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");

	ActorPickerMode.BeginActorPickingMode(
		FOnGetAllowedClasses(), 
		FOnShouldFilterActor(), 
		FOnActorSelected::CreateLambda([Args](AActor* InActor)
		{
			OnParentPicked(InActor, Args.CreationDelegate, Args.ConstraintType, Args.bUseDefault);
		}) );
	
	return FReply::Handled();
}

void SConstraintMenuEntry::OnParentPicked(
	AActor* InParent,
	const FOnConstraintCreated& InCreationDelegate,
	const ETransformConstraintType InConstraintType,
	const bool bUseDefault)
{
	using namespace UE::Private;
	
	if (!InParent)
	{
		return;	
	}

	const UEnum* ETransformConstraintTypeEnum = StaticEnum<ETransformConstraintType>();
	if (!ETransformConstraintTypeEnum->IsValidEnumValue(static_cast<int64>(InConstraintType)))
	{
		return;
	}
	
	// get selected actors
	const TArray<AActor*> Selection = GetCurrentSelection();
	if (Selection.IsEmpty())
	{
		return;	
	}

	// create constraints
	auto CreateConstraint = [InCreationDelegate, InConstraintType, bUseDefault](
		const TArray<AActor*>& Selection, UObject* InParent, const FName& InSocketName)
	{
		FConstrainable Parent;
		Parent.Object = InParent;
		Parent.Socket = InSocketName;

		const bool bCreated = UE::Private::CreateConstraint(Selection, Parent, InConstraintType, bUseDefault);
	
		// update list
		if (bCreated && InCreationDelegate.IsBound())
		{
			InCreationDelegate.Execute();
		}
	};

	// gather sub components with sockets
	const TInlineComponentArray<USceneComponent*> Components = InParent->IsA<AControlRigShapeActor>() ?
		TInlineComponentArray<USceneComponent*>() : TInlineComponentArray<USceneComponent*>(InParent);
	const TArray<USceneComponent*> ComponentsWithSockets = Components.FilterByPredicate([](const USceneComponent* Component)
	{
		return Component->HasAnySockets();
	});
	const int32 NumComponentsWithSockets = ComponentsWithSockets.Num();

	// if no component socket available then constrain the whole actor 
	if (NumComponentsWithSockets == 0)
	{
		return CreateConstraint(Selection, InParent, NAME_None);
	}

	// creates a menu encapsulating InContent  
	auto CreateMenu = [](const TSharedRef<SWidget>& InContent, const FVector2D& InLocation)
	{
		const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		const TSharedPtr< ILevelEditor > LevelEditor = LevelEditorModule.GetFirstLevelEditor();
		FSlateApplication::Get().PushMenu(
			LevelEditor.ToSharedRef(),
			FWidgetPath(),
			InContent,
		InLocation,
		FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu ) );
	};

	// creates a new socket chooser popup widget
	auto GetSocketChooserWidget = [CreateConstraint, Selection](USceneComponent* Component)
	{
		return SNew(SSocketChooserPopup)
		.SceneComponent( Component )
		.OnSocketChosen_Lambda([CreateConstraint, Selection, Component](FName InSocketName)
		{
			CreateConstraint(Selection, Component, InSocketName);
		});
	};

	// decal the menu 
	FVector2D SummonLocation = FSlateApplication::Get().GetCursorPos();
	SummonLocation.Y += 4 * FSlateApplication::Get().GetCursorSize().Y;
	
	// if one component with sockets then constrain the component selecting the socket
	if (NumComponentsWithSockets == 1)
	{
		return CreateMenu( GetSocketChooserWidget(ComponentsWithSockets[0]), SummonLocation);
	}

	// if there are several of them, then build a component chooser first then the socket chooser
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, nullptr);
	MenuBuilder.BeginSection("ChooseComp", LOCTEXT("ChooseComponentSection", "Choose Component"));
	{
		for (USceneComponent* Component: ComponentsWithSockets)
		{
			MenuBuilder.AddMenuEntry(FText::FromName(Component->GetFName()),
			FText(),
				FSlateIconFinder::FindIconForClass(Component->GetClass(), TEXT("SCS.Component")),
				FUIAction(FExecuteAction::CreateLambda([CreateMenu, GetSocketChooserWidget, Component]()
				{
					CreateMenu(GetSocketChooserWidget(Component), FSlateApplication::Get().GetCursorPos());
				})),
				NAME_None,
			   EUserInterfaceActionType::Button);
		}
	}
	MenuBuilder.EndSection();
	CreateMenu(MenuBuilder.MakeWidget(), SummonLocation);
}

/**
 * SEditableConstraintItem
 */

void SEditableConstraintItem::Construct(
	const FArguments& InArgs,
	const TSharedPtr<FEditableConstraintItem>& InItem,
	TSharedPtr<SConstraintsEditionWidget> InConstraintsWidget)
{
	using namespace UE::Private;
	
	ConstraintItem = InItem;
	ConstraintsWidget = InConstraintsWidget;

	static const FSlateBrush* RoundedBoxBrush = FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.SpacePicker.RoundedRect"));
	
	// enum to string
	const uint8 ConstraintType = static_cast<uint8>(InItem->Type);

	const FSimpleDelegate OnConstraintRemoved = FSimpleDelegate::CreateLambda([this]()
	{
		if(ConstraintsWidget.IsValid())
		{
			ConstraintsWidget.Pin()->RemoveItem(ConstraintItem);
		}
	});

	// constraint
	auto GetConstraint = [this]() -> UTickableConstraint*
	{
		UWorld* World = GetCurrentWorld();
		if (!IsValid(World))
		{
			return nullptr;
		}
		
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		UTickableConstraint* Constraint = ConstraintItem->Constraint.Get();
		int32 Index = Controller.GetConstraintsArray().Find(Constraint);
		if (Index == INDEX_NONE)
		{
			return nullptr;
		}
		return Constraint;
	};
	TWeakObjectPtr<UTickableConstraint> Constraint = GetConstraint();

	UWorld* World = GetCurrentWorld();

	// sequencer
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerChecked();

	//manager for static check
	UConstraintsManager* Manager = UConstraintsManager::Find(World);

	// labels
	FString ParentLabel(TEXT("undefined")), ChildLabel(TEXT("undefined"));
	FString ParentFullLabel = ParentLabel, ChildFullLabel = ChildLabel;
	if (IsValid(Constraint.Get()))
	{
		Constraint->GetFullLabel().Split(HandleLabelDelimiter, &ParentFullLabel, &ChildFullLabel);
	}

	// widgets
	ChildSlot
	.Padding(FMargin(8.f, 2.f, 12.f, 2.f))
	[
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			SNew(SBorder)
			.Padding(FMargin(5.0, 2.0, 5.0, 2.0))
			.BorderImage(RoundedBoxBrush)
			.BorderBackgroundColor_Lambda([Manager, World,Constraint]()
			{
				if (!Constraint.IsValid() || !IsValid(Constraint.Get()))
				{
					return FStyleColors::Transparent;
				}
				if (Manager && Manager->IsStaticConstraint(Constraint.Get()))
				{
					return FSlateColor(FLinearColor::Green * 0.5);
				}
				return Constraint->IsFullyActive() ? FStyleColors::Select : FStyleColors::Transparent;
			})
			.Content()
			[
				SNew(SHorizontalBox)

				// constraint icon
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
				[
					SNew(SImage)
					.Image(FConstraintInfo::GetBrush(ConstraintType))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]

				// constraint name
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(0)
				[
					SNew( STextBlock )
					.Text_Lambda( [Constraint, ParentLabel, ChildLabel]()
					{
						FString CurrentParentLabel = ParentLabel, CurrentChildLabel = ChildLabel;
						if (Constraint.IsValid() && IsValid(Constraint.Get()) && Constraint->IsValid())
						{
							FString Label = Constraint->GetLabel();
							Label.Split(HandleLabelDelimiter, &CurrentParentLabel, &CurrentChildLabel);
						}
						static constexpr TCHAR LabelFormat[] = TEXT("%s <-- %s");
						const FString Label = FString::Printf(LabelFormat, *CurrentParentLabel, *CurrentChildLabel);
						return FText::FromString(Label);
					})
					.Font_Lambda([Constraint]()
					{
						if (!Constraint.IsValid() || !IsValid(Constraint.Get()))
						{
							return IDetailLayoutBuilder::GetDetailFont();
						}
						return Constraint->Active ? IDetailLayoutBuilder::GetDetailFont() : IDetailLayoutBuilder::GetDetailFontItalic();
					})
					.ToolTipText_Lambda( [Constraint, ParentFullLabel, ChildFullLabel]()
					{
						if (!Constraint.IsValid() || !IsValid(Constraint.Get()))
						{
							return FText();
						}

						static constexpr TCHAR ToolTipFormat[] = TEXT("%s constraint between parent '%s' and child '%s'.");
						const FString TypeLabel = Constraint->GetTypeLabel();
						const FString FullLabel = FString::Printf(ToolTipFormat, *TypeLabel, *ParentFullLabel, *ChildFullLabel);
						return FText::FromString(FullLabel);
					})
				]
			]
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		]

		// add key
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ContentPadding(0)
			.OnClicked_Lambda(	[Constraint, WeakSequencer]()
			{
				if (!Constraint.IsValid() || !IsValid(Constraint.Get()))
				{
					return FReply::Handled();
				}

				if (!WeakSequencer.IsValid())
				{
					return FReply::Handled();
				}
				
				if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint))
				{
					FScopedTransaction Transaction(LOCTEXT("CreateConstraintKey", "Create Constraint Key"));
					FMovieSceneConstraintChannelHelper::SmartConstraintKey(WeakSequencer.Pin(), TransformConstraint,TOptional<bool>(), TOptional<FFrameNumber>());
				}
				return FReply::Handled();
			})
			.IsEnabled_Lambda([WeakSequencer]()
			{
				return WeakSequencer.IsValid();
			})
			.Visibility_Lambda([WeakSequencer]()
			{
				const bool bIsKeyframingAvailable = WeakSequencer.IsValid();
				return bIsKeyframingAvailable ? EVisibility::Visible : EVisibility::Hidden;
			})
			.ToolTipText(LOCTEXT("KeyConstraintToolTip", "Add an active keyframe for that constraint."))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Sequencer.AddKey.Details"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
		
		// move up
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ContentPadding(0)
			.OnClicked_Lambda(	[this, InItem]()
			{
				if (ConstraintsWidget.IsValid())
				{
					ConstraintsWidget.Pin()->MoveItemUp(InItem);
				}
				return FReply::Handled();
			})
			.IsEnabled_Lambda([this]()
			{
				return ConstraintsWidget.IsValid() ? ConstraintsWidget.Pin()->CanMoveUp(ConstraintItem) : false;
			})
			.ToolTipText(LOCTEXT("MoveConstraintUp", "Move this constraint up in the list."))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.ChevronUp"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// move down
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(0)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ContentPadding(0)
			.OnClicked_Lambda([this, InItem]()
			{
				if (ConstraintsWidget.IsValid())
				{
					ConstraintsWidget.Pin()->MoveItemDown(InItem);
				}	
				return FReply::Handled();
			})
			.IsEnabled_Lambda([this]()
			{
				return ConstraintsWidget.IsValid() ? ConstraintsWidget.Pin()->CanMoveDown(ConstraintItem) : false;
			})
			.ToolTipText(LOCTEXT("MoveConstraintDown", "Move this constraint down in the list."))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]

		// deletion
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(0)
		[
			 PropertyCustomizationHelpers::MakeClearButton(OnConstraintRemoved, LOCTEXT("DeleteConstraint", "Remove this constraint."), true)
		]
	];
}

//////////////////////////////////////////////////////////////
/// FBaseConstraintListWidget
///////////////////////////////////////////////////////////


FBaseConstraintListWidget::~FBaseConstraintListWidget()
{
	UnregisterNotifications();
}

void FBaseConstraintListWidget::RegisterNotifications()
{
	// selection changed
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		FLevelEditorModule::FActorSelectionChangedEvent& ActorSelectionChangedEvent = LevelEditor.OnActorSelectionChanged();

		// unregister previous one
		if (OnSelectionChangedHandle.IsValid())
		{
			ActorSelectionChangedEvent.Remove(OnSelectionChangedHandle);
			OnSelectionChangedHandle.Reset();
		}
		

		// register
		OnSelectionChangedHandle = ActorSelectionChangedEvent.AddRaw(this, &FBaseConstraintListWidget::OnActorSelectionChanged);
	}
	

	// constraints controller updated
	{
		if (ConstraintsNotificationHandle.IsValid())
		{
			FConstraintsManagerController::GetNotifyDelegate().Remove(ConstraintsNotificationHandle);
			ConstraintsNotificationHandle.Reset();
		}
		
		ConstraintsNotificationHandle =
			FConstraintsManagerController::GetNotifyDelegate().AddLambda([this](EConstraintsManagerNotifyType InNotifyType, UObject* InObject)
		{
			if (InNotifyType == EConstraintsManagerNotifyType::GraphUpdated)
			{
				InvalidateConstraintList();
			}
		});
	}
}

void FBaseConstraintListWidget::UnregisterNotifications()
{
	if (OnSelectionChangedHandle.IsValid())
	{
		if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditor->OnActorSelectionChanged().Remove(OnSelectionChangedHandle);
		}
		OnSelectionChangedHandle.Reset();
	}
	
	if (ConstraintsNotificationHandle.IsValid())
	{
		FConstraintsManagerController::GetNotifyDelegate().Remove(ConstraintsNotificationHandle);
		ConstraintsNotificationHandle.Reset();
	}
}

void FBaseConstraintListWidget::PostUndo(bool bSuccess)
{
	InvalidateConstraintList();
}

void FBaseConstraintListWidget::PostRedo(bool bSuccess)
{
	InvalidateConstraintList();
}

void FBaseConstraintListWidget::InvalidateConstraintList()
{
	bNeedsRefresh = true;

	// NOTE: this is a hack to disable the picker if the selection changes. The picker should trigger the escape key
	// but this is an editor wide change
	static const FActorPickerModeModule& ActorPickerMode = FModuleManager::Get().GetModuleChecked<FActorPickerModeModule>("ActorPickerMode");
	if (ActorPickerMode.IsInActorPickingMode())
	{
		ActorPickerMode.EndActorPickingMode();
	}
}

FBaseConstraintListWidget::EShowConstraints FBaseConstraintListWidget::ShowConstraints = FBaseConstraintListWidget::EShowConstraints::ShowSelected;

FText FBaseConstraintListWidget::GetShowConstraintsText(EShowConstraints Index) const
{
	FText Text = FText::GetEmpty();
	switch (Index)
	{
		case EShowConstraints::ShowSelected:
		{
			Text = FText(LOCTEXT("Selected", "Selected"));
		}
		break;
		case EShowConstraints::ShowLevelSequence:
		{
			Text = FText(LOCTEXT("LevelSequence", "Level Sequence"));
		}
		break;
		case EShowConstraints::ShowValid:
		{
			Text = FText(LOCTEXT("Current", "Current"));
		}
		break;
		case EShowConstraints::ShowAll:
		{
			Text = FText(LOCTEXT("All", "All"));
		}
		break;
	}
	return Text;
}

FText FBaseConstraintListWidget::GetShowConstraintsTooltip(EShowConstraints Index) const
{
	FText Text = FText::GetEmpty();
	switch (Index)
	{
		case EShowConstraints::ShowSelected:
		{
			Text = FText(LOCTEXT("SelectedTooltip", "Show constraints on selected"));
		}
		break;
		case EShowConstraints::ShowLevelSequence:
		{
			Text = FText(LOCTEXT("LevelSequenceTooltip", "Show constraints on active level sequence"));
		}
		break;
		case EShowConstraints::ShowValid:
		{
			Text = FText(LOCTEXT("CurrentTooltip", "Show constraints that are current in level sequences or static"));
		}
		break;
		case EShowConstraints::ShowAll:
		{
			Text = FText(LOCTEXT("AllTooltip", "Show all constraints"));
		}
		break;
	}
	return Text;
}

int32 FBaseConstraintListWidget::RefreshConstraintList()
{
	using namespace UE::Private;
	
	// get constraints
	UWorld* World = GetCurrentWorld();
	if (!IsValid(World))
	{
		return 0;
	}

	static constexpr bool bSorted = true;
	
	const TArray<AActor*> Selection = GetCurrentSelection();

	const bool bIsConstraintsActor = Selection.Num() == 1 && Selection[0]->IsA<AConstraintsActor>();

	TArray< TWeakObjectPtr<UTickableConstraint> > Constraints;
	if (ShowConstraints == EShowConstraints::ShowSelected)
	{
		if (bIsConstraintsActor)
		{
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			Constraints = Controller.GetStaticConstraints(bSorted);
		}
		else
		{
			for (const AActor* Actor : Selection)
			{
				UE::TransformConstraintUtil::GetParentConstraints(World, Actor, Constraints);
			}
			
			//remove if not active...
			for (int32 Index = Constraints.Num() - 1; Index >= 0; --Index)
			{
				if (Constraints[Index].Get() == nullptr || Constraints[Index].Get()->IsValid() == false)
				{
					Constraints.RemoveAt(Index);
				}
			}
		}
	}
	else
	{
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Constraints = Controller.GetAllConstraints(bSorted);
		if (ShowConstraints == EShowConstraints::ShowLevelSequence)
		{
			TWeakPtr<ISequencer> WeakSequencer = GetSequencerChecked();
			if (WeakSequencer.IsValid())
			{
				UMovieScene* MovieScene = WeakSequencer.Pin()->GetFocusedMovieSceneSequence()->GetMovieScene();
				for (int32 Index = Constraints.Num() - 1; Index >= 0; --Index)
				{
					UMovieScene* OuterMovieScene = Constraints[Index].Get() ? Constraints[Index]->GetTypedOuter<UMovieScene>() : nullptr;
					if (OuterMovieScene != MovieScene)
					{
						Constraints.RemoveAt(Index);
					}
				}
			}
			else
			{
				Constraints.Empty();
			}
		}
		else if (ShowConstraints == EShowConstraints::ShowValid)
		{
			for (int32 Index = Constraints.Num() - 1; Index >= 0; --Index)
			{
				if (Constraints[Index].Get() == nullptr || Constraints[Index].Get()->IsValid() == false)
				{
					Constraints.RemoveAt(Index);
				}
			}
		}
	}

	// rebuild item list
	ListItems.Empty();

	const UEnum* ETransformConstraintTypeEnum = StaticEnum<ETransformConstraintType>();
	for (const TWeakObjectPtr<UTickableConstraint>& Constraint : Constraints)
	{
		if (Constraint.IsValid())
		{
			const int8 Type = FConstraintInfo::GetType(Constraint->GetClass());
			if (ETransformConstraintTypeEnum->IsValidEnumValue(Type))
			{
				const ETransformConstraintType ConstraintType = static_cast<ETransformConstraintType>(Type);
				ListItems.Emplace(FEditableConstraintItem::Make(Constraint.Get(), ConstraintType));
			}
		}
	}

	// refresh tree
	ListView->RequestListRefresh();

	return ListItems.Num();
}

void FBaseConstraintListWidget::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	// NOTE we use this delegate to trigger an tree update, however, control actors are not selected as they are 
	// Temporary Editor Actors so NewSelection won't contain the controls
	InvalidateConstraintList();
}


//////////////////////////////////////////////////////////////
/// SConstraintsEditionWidget
///////////////////////////////////////////////////////////

void SConstraintsEditionWidget::ResetParent()
{
	ConstrainableParent = FConstrainable();
}

const FConstrainable& SConstraintsEditionWidget::GetParent() const
{
	return ConstrainableParent;
}

bool SConstraintsEditionWidget::IsParentValid() const
{
	return ConstrainableParent.Actor.IsValid();
}

void SConstraintsEditionWidget::UpdateParent(AActor* InActor)
{
	UE::Private::GetConstrainable(InActor, ConstrainableParent);
}

void SConstraintsEditionWidget::Construct(const FArguments& InArgs)
{
	using namespace UE::Private;
	
	WeakSequencer = GetSequencerChecked();

	auto GetParentWidget = [this, &InArgs]()
	{
		TSharedRef<SSplitter> Splitter = SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(1.0f)
		.HitDetectionSplitterHandleSize(5.0f)

		// Parent label 
		+ SSplitter::Slot()
		.Value(0.4f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				InArgs._LeftSplitterContent.Widget
			]

			+SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(20.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("PreSelectedParent_Label", "Pre Selected Parent"))
				]
			]
		]

		// Parent chooser 
		+ SSplitter::Slot()
		.Value(0.6f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				InArgs._MiddleSplitterContent.Widget
			]

			+SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				SNew(SAssetDropTarget)
				.bSupportsMultiDrop(true)
				.OnAreAssetsAcceptableForDrop_Lambda([](TArrayView<FAssetData> InAssets)
				{
					const bool bHasAnyNotConstrainable = InAssets.ContainsByPredicate([](const FAssetData& Data)
					{
						return !IsActorConstrainable(Cast<AActor>(Data.GetAsset()));	
					});
					return !bHasAnyNotConstrainable;
				})
				.OnAssetsDropped_Lambda( [this](const FDragDropEvent&, TArrayView<FAssetData> InAssets)
				{
					UpdateParent(InAssets.IsEmpty() ? nullptr : Cast<AActor>(InAssets[0].GetAsset()));
				})
				[
					SNew(SHorizontalBox)
												
					// actor chooser widget
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					[
						GetActorChooserWidget().ToSharedRef()
					]
				
					// use selected widget
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.Padding(FMargin(4.0f,0.0f))
						[
							GetUseSelectedActorWidget()
						]
					]
				]
			]
		]

		// Reset button
		+ SSplitter::Slot()
		.SizeRule(SSplitter::SizeToContent)
		.Value(26.f)
		.MinSize(26.f)
		.Resizable(false)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				InArgs._RightSplitterContent.Widget
			]

			+SVerticalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
				.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Panel"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(22.0f)
					.HeightOverride(22.0f)
					.ToolTipText(LOCTEXT("ResetParent_Tooltip", "Reset parent."))
					[
						SNew(SButton)
						.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
						.OnClicked_Lambda([this]()
						{
							ResetParent();
							return FReply::Handled();
						})
						.Visibility_Lambda([this]()
						{
							return IsParentValid() ? EVisibility::Visible : EVisibility::Hidden;
						})
						.ContentPadding(0.0f)
						.IsFocusable(false)
						[ 
							SNew( SImage )
							.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
							.ColorAndOpacity( FSlateColor::UseForeground() )
						]
					]
				]
			]
		];
		
		return SNew( SBorder )
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.GridLine"))
		.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Panel"))
		.Padding(FMargin(0.0f,0.0f,0.0f,1.0f))
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SBox)
			.MinDesiredHeight(26.f)
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew( SBorder )
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
					.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Panel"))
					.Padding(0.0f)
					[
						Splitter
					]
				]
			]
		];
	};

	auto GetConstraintListWidget = [this]()
	{
		return SNew(SOverlay)
	
		+ SOverlay::Slot()
		[
			SAssignNew(ListView, ConstraintItemListView)
			.SelectionMode(ESelectionMode::Single)
			.ListItemsSource( &ListItems )
			.OnGenerateRow(this, &SConstraintsEditionWidget::OnGenerateWidgetForItem)
			.OnContextMenuOpening(this, &SConstraintsEditionWidget::CreateContextMenu)
			.OnMouseButtonDoubleClick(this, &SConstraintsEditionWidget::OnItemDoubleClicked)
		];
	};

	auto GetBakingWidget = [this, &InArgs]()
	{
		return SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			InArgs._LeftButtonArea.Widget
		]
		
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SSpacer)
		]
			
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(1.f)
		[
			SNew(SActionButton)
			.Text(LOCTEXT("BakeButton", "Bake..."))
			.OnClicked(this, &SConstraintsEditionWidget::OnBakeClicked)
			.IsEnabled_Lambda([this]() { return ListItems.Num() > 0; })
			.ToolTipText(LOCTEXT("BakeConstraintButtonToolTip", "Bake selected constraints"))
		];
	};
	
	ChildSlot
	[
		SNew( SVerticalBox )
	
		// parent chooser widget
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 0.f))
		[
			GetParentWidget()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(8.f))
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SNew( SVerticalBox )
		
				// constraints list widget
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.f))
				[
					GetConstraintListWidget()
				]
		
				// baking widget
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.f))
				[
					GetBakingWidget()
				]
			]
		]
	];

	RefreshConstraintList();
	RegisterNotifications();
}

bool SConstraintsEditionWidget::SequencerTimeChanged() 
{
	if (WeakSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		const bool bTimeChanged = (SequencerLastTime.Rate != Sequencer->GetLocalTime().Rate ||
			SequencerLastTime.Time != Sequencer->GetLocalTime().Time);
		SequencerLastTime = Sequencer->GetLocalTime();
		return bTimeChanged;
	}
	return false;
}

void SConstraintsEditionWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if(bNeedsRefresh || SequencerTimeChanged())
	{
		RefreshConstraintList();
		bNeedsRefresh = false;
	}
}

TSharedRef<SWidget> SConstraintsEditionWidget::OnGetParentMenuContent()
{
	static constexpr bool bAllowClear = true, bAllowPicking = false, bUseSelected = true, bShowTransient = true;

	FOnShouldFilterActor FilterActorDelegate = FOnShouldFilterActor::CreateLambda([](const AActor* InActor)
	{
		return UE::Private::IsActorConstrainable(InActor);
	});
	
	FOnActorSelected SelectedActorDelegate = FOnActorSelected::CreateLambda([this](AActor* InActor)
	{
		UpdateParent(InActor);
	});

	FSimpleDelegate OnCloseDelegate = FSimpleDelegate::CreateLambda([this]
	{
		GetActorChooserWidget()->SetIsOpen(false);
	});

	FSimpleDelegate UseSelectedDelegate = FSimpleDelegate::CreateLambda([this]
	{
		const TArray<AActor*> Actors = UE::Private::GetCurrentSelection();
		UpdateParent(Actors.IsEmpty() ? nullptr : Actors[0]);
	});
	
	return PropertyCustomizationHelpers::MakeActorPickerWithMenu(
		ConstrainableParent.Actor.Get(),
		bAllowClear,
		bAllowPicking,
		FilterActorDelegate,
		SelectedActorDelegate,
		OnCloseDelegate,
		UseSelectedDelegate,
		bUseSelected,
		bShowTransient);
}

void SConstraintsEditionWidget::OnParentMenuOpenChanged(bool bOpened)
{
	if (!bOpened)
	{
		GetActorChooserWidget()->SetMenuContent(SNullWidget::NullWidget);
	}
}

TSharedPtr<SComboButton> SConstraintsEditionWidget::GetActorChooserWidget()
{
	if (!ActorSelectionButtons.IsValid())
	{
		auto GetParentLabel = [this]()
		{
			FString Label;
			if (ConstrainableParent.Actor.IsValid())
			{
				Label = ConstrainableParent.Actor->GetActorLabel();
				if (ConstrainableParent.Socket != NAME_None)
				{
					Label = Label + ":" + ConstrainableParent.Socket.ToString();
				}
			}
			return Label;
		};
		
		SAssignNew(ActorSelectionButtons, SComboButton)
		.ToolTipText_Lambda([this, GetParentLabel]()
		{
			if (ConstrainableParent.Actor.IsValid())
			{
				const FString Tooltip = FString::Printf(TEXT("'%s' will be used as the constraint's parent."), *GetParentLabel());
				return FText::FromString(Tooltip);
			}
			
			return LOCTEXT("SelectParent_Tooltip", "Select a parent actor.");
		})
		.OnGetMenuContent(this, &SConstraintsEditionWidget::OnGetParentMenuContent)
		.OnMenuOpenChanged(this, &SConstraintsEditionWidget::OnParentMenuOpenChanged)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.Text_Lambda([this, GetParentLabel]()
				{
					if (ConstrainableParent.Actor.IsValid())
					{
						return FText::FromString(GetParentLabel()); 
					}
					return LOCTEXT("SelectConstraintParent", "Select Parent");
				})
			]
		];
	}
	
	return ActorSelectionButtons;
}

TSharedRef<SWidget> SConstraintsEditionWidget::GetUseSelectedActorWidget()
{
	static constexpr bool bIsActor = true, bEnabled = true;
	
	return SNew( SHorizontalBox )

	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.AutoWidth()
	.Padding( 2.0f, 0.0f )
	[
		PropertyCustomizationHelpers::MakeUseSelectedButton(
			FSimpleDelegate::CreateLambda([this]()
			{
				const TArray<AActor*> Actors = UE::Private::GetCurrentSelection();
				UpdateParent(Actors.IsEmpty() ? nullptr : Actors[0]);
			}),
			LOCTEXT("SetFromSelection", "Set from current selection."),
			bEnabled, bIsActor)
	];
}

TSharedRef<ITableRow> SConstraintsEditionWidget::OnGenerateWidgetForItem(
	ItemSharedPtr InItem,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<ItemSharedPtr>, OwnerTable)
		.Style(&FAppStyle::Get(), "PlacementBrowser.PlaceableItemRow")
		[
			SNew(SEditableConstraintItem, InItem.ToSharedRef(), SharedThis(this))
		];
}

bool SConstraintsEditionWidget::CanMoveUp(const TSharedPtr<FEditableConstraintItem>& Item) const
{
	const int32 NumItems = ListItems.Num();
	if (NumItems < 2)
	{
		return false;
	}
	
	return ListItems[0] != Item; 
}

bool SConstraintsEditionWidget::CanMoveDown(const TSharedPtr<FEditableConstraintItem>& Item) const
{
	const int32 NumItems = ListItems.Num();
	if (NumItems < 2)
	{
		return false;
	}
	
	return ListItems.Last() != Item;
}

void SConstraintsEditionWidget::MoveItemUp(const TSharedPtr<FEditableConstraintItem>& Item)
{
	if (!CanMoveUp(Item))
	{
		return;
	}
	
	const int32 Index = ListItems.IndexOfByKey(Item);
	if (Index > 0 && ListItems.IsValidIndex(Index))
	{
		UWorld* World = UE::Private::GetCurrentWorld();
		if (!IsValid(World))
		{
			return;
		}
		
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);

		// the current item needs to tick before the previous item 
		const FName BeforeName(ListItems[Index]->GetName().ToString());
		const FName AfterName(ListItems[Index-1]->GetName().ToString());
		Controller.SetConstraintsDependencies(BeforeName, AfterName);
		
		RefreshConstraintList();
	}
}

void SConstraintsEditionWidget::MoveItemDown(const TSharedPtr<FEditableConstraintItem>& Item)
{
	if (!CanMoveDown(Item))
	{
		return;
	}
	
	const int32 Index = ListItems.IndexOfByKey(Item);
	if (ListItems.IsValidIndex(Index))
	{
		UWorld* World = UE::Private::GetCurrentWorld();
		if (!IsValid(World))
		{
			return;
		}
		
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);

		// the current item needs to tick after the next item 
		const FName BeforeName(ListItems[Index+1]->GetName().ToString());
		const FName AfterName(ListItems[Index]->GetName().ToString());
		Controller.SetConstraintsDependencies(BeforeName, AfterName);
		
		RefreshConstraintList();
	}
}

void SConstraintsEditionWidget::RemoveItem(const TSharedPtr<FEditableConstraintItem>& Item)
{
	UWorld* World = UE::Private::GetCurrentWorld();
	if (!IsValid(World))
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("RemoveConstraint", "Remove Constraint"));
	
	FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	
	Controller.RemoveConstraint(Item->Constraint.Get());

	RefreshConstraintList();
}

void SConstraintsEditionWidget::InvalidateConstraintList()
{
	FBaseConstraintListWidget::InvalidateConstraintList();
	UpdateSequencer();
}

void SConstraintsEditionWidget::SequencerChanged(const TWeakPtr<ISequencer>& InNewSequencer)
{
	if (WeakSequencer != InNewSequencer)
	{
		WeakSequencer = InNewSequencer;
		InvalidateConstraintList();
	}
}

void SConstraintsEditionWidget::UpdateSequencer()
{
	if (!WeakSequencer.IsValid())
	{
		WeakSequencer = UE::Private::GetSequencerChecked();
	}
}

FReply SConstraintsEditionWidget::OnBakeClicked()
{
	UpdateSequencer();
	if (!WeakSequencer.IsValid() || ListItems.Num() < 1)
	{
		return FReply::Unhandled();
	}
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	TSharedRef<SConstraintBakeWidget> BakeWidget =
		SNew(SConstraintBakeWidget)
		.Sequencer(Sequencer);

	return BakeWidget->OpenDialog(true);
}

TSharedPtr<SWidget> SConstraintsEditionWidget::CreateContextMenu()
{
	const TArray<TSharedPtr<FEditableConstraintItem>> Selection = ListView->GetSelectedItems();
	if (Selection.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	const int32 Index = ListItems.IndexOfByKey(Selection[0]);
	if (!ListItems.IsValidIndex(Index))
	{
		return SNullWidget::NullWidget;
	}

	UWorld* World = UE::Private::GetCurrentWorld();
	if (!IsValid(World))
	{
		return SNullWidget::NullWidget;
	}

	UpdateSequencer();
	
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	UTickableConstraint* Constraint = ListItems[Index]->Constraint.Get();
	int32 ContIndex = Controller.GetConstraintsArray().Find(Constraint);
	if (ContIndex == INDEX_NONE)
	{
		return SNullWidget::NullWidget;
	}

	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, nullptr);

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bCustomFilterAreaLocation = true;
		DetailsViewArgs.bCustomNameAreaLocation = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.ColumnWidth = 0.35f;
	}
	
	TSharedRef<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);

	// manage properties visibility
	const bool bIsLookAtConstraint = Constraint->GetClass() == UTickableLookAtConstraint::StaticClass();
	FIsPropertyVisible PropertyVisibility = FIsPropertyVisible::CreateLambda(
		[bIsLookAtConstraint](const FPropertyAndParent& InPropertyAndParent)
		{
			const FName PropertyName = InPropertyAndParent.Property.GetFName();
				
			if (!bIsLookAtConstraint)
			{
				if (PropertyName == GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bUseCurrentOffset))
				{
					return false;
				}
				return true;
			}

			// hide offset properties for look at constraints
			static const FName MaintainOffsetPropName = GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bMaintainOffset);
			static const FName DynamicOffsetPropName = GET_MEMBER_NAME_CHECKED(UTickableTransformConstraint, bDynamicOffset);
						
			if (PropertyName == MaintainOffsetPropName || PropertyName == DynamicOffsetPropName)
			{
				return false;
			}

			return true;
		});
	DetailsView->SetIsPropertyVisibleDelegate(PropertyVisibility);

	const UConstraintsManager* Manager = UConstraintsManager::Find(World);
	const bool bIsStatic = Manager ? Manager->IsStaticConstraint(Constraint) : false;
	auto IsPropertyReadOnly = [bIsStatic](const FProperty* InProperty)
	{
		if (InProperty && InProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
		{
			return !bIsStatic;
		}
		return false;
	};
	
	FIsPropertyReadOnly PropertyReadonly = FIsPropertyReadOnly::CreateLambda([IsPropertyReadOnly](const FPropertyAndParent& InPropertyAndParent)
	{
		if (IsPropertyReadOnly(&InPropertyAndParent.Property))
		{
			return true;
		}

		const bool bIsParentReadOnly = InPropertyAndParent.ParentProperties.ContainsByPredicate([IsPropertyReadOnly](const FProperty* Parent)
		{
			return IsPropertyReadOnly(Parent);
		});
		
		return bIsParentReadOnly;
	});
	DetailsView->SetIsPropertyReadOnlyDelegate(PropertyReadonly);
	
	TArray<TWeakObjectPtr<UObject>> ConstrainsToEdit;
	ConstrainsToEdit.Add(Constraint);
	
	DetailsView->SetObjects(ConstrainsToEdit);

	// constraint details
	MenuBuilder.BeginSection("EditConstraint", LOCTEXT("EditConstraintHeader", "Edit Constraint"));
	{
		MenuBuilder.AddWidget(
			// Summary: This makes sure the menu anchor position is realigned when you open the menu near the edge of your screen, so the menu does
			// not spill over to the next screen.
			// Slate calls Prepass and then uses the widget's desired size. However, the details view always defers until next tick to compute it.
			// The width override forces a desired size.
			SNew(SBox).WidthOverride(480.f) [ DetailsView ],
			FText::GetEmpty(), true
			);
	}
	MenuBuilder.EndSection();

	// transform constraint options
	if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint))
	{
		const FText ConstraintLabel = FText::FromName(ListItems[Index]->GetName());
	
		// baking (note that this will probably be moved)
		MenuBuilder.BeginSection("BakeConstraint", LOCTEXT("BakeConstraintHeader", "Bake Constraint"));
		{
			MenuBuilder.AddMenuEntry(
			LOCTEXT("BakeConstraintLabel", "Bake"),
			FText::Format(LOCTEXT("BakeConstraintDoItTooltip", "Bake {0} transforms."), ConstraintLabel),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, Constraint, World]()
			{
				const TSharedPtr<ISequencer>& Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
				UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint);
				if (TransformConstraint && Sequencer)
				{
					FScopedTransaction Transaction(LOCTEXT("BakeConstraint", "Bake Constraint"));
					FConstraintBaker::Bake(World, TransformConstraint, WeakSequencer.Pin(), TOptional< FBakingAnimationKeySettings>(), TOptional<TArray<FFrameNumber>>());
				}
			})),
			NAME_None,
			EUserInterfaceActionType::Button);
		}
		MenuBuilder.EndSection();

		// keys section
		FCanExecuteAction IsCompensationEnabled = FCanExecuteAction::CreateLambda([TransformConstraint]()
		{
			return TransformConstraint->bDynamicOffset;
		});

		if (!bIsLookAtConstraint && WeakSequencer.IsValid())
		{
			MenuBuilder.BeginSection("KeyConstraint", LOCTEXT("KeyConstraintHeader", "Keys"));
			{
				MenuBuilder.AddMenuEntry(
				LOCTEXT("CompensateKeyLabel", "Compensate Key"),
				FText::Format(LOCTEXT("CompensateKeyTooltip", "Compensate transform key for {0}."), ConstraintLabel),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([TransformConstraint, this]()
				{
					const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
					const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
					const FFrameTime FrameTime = Sequencer->GetLocalTime().ConvertTo(TickResolution);
					const FFrameNumber Time = FrameTime.GetFrame();
					FMovieSceneConstraintChannelHelper::Compensate(WeakSequencer.Pin(), TransformConstraint, TOptional<FFrameNumber>(Time), true /*bCompPreviousTick*/);
				}), IsCompensationEnabled),
				NAME_None,
				EUserInterfaceActionType::Button);

				MenuBuilder.AddMenuEntry(
				LOCTEXT("CompensateAllKeysLabel", "Compensate All Keys"),
				FText::Format(LOCTEXT("CompensateAllKeysTooltip", "Compensate all transform keys for {0}."), ConstraintLabel),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([TransformConstraint, this]()
				{
					FMovieSceneConstraintChannelHelper::Compensate(WeakSequencer.Pin(), TransformConstraint, TOptional<FFrameNumber>(), true /*bCompPreviousTick*/);
				}), IsCompensationEnabled),
				NAME_None,
				EUserInterfaceActionType::Button);
			}
			MenuBuilder.EndSection();
		}
	}

	return MenuBuilder.MakeWidget();
}

void SConstraintsEditionWidget::OnItemDoubleClicked(ItemSharedPtr InItem)
{
	const int32 Index = ListItems.IndexOfByKey(InItem);
	if (!ListItems.IsValidIndex(Index))
	{
		return;
	}

	UWorld* World = UE::Private::GetCurrentWorld();
	if (!IsValid(World))
	{
		return;
	}
	
	const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
	UTickableConstraint* Constraint = InItem->Constraint.Get();
	int32 ContIndex = Controller.GetConstraintsArray().Find(Constraint);
	if (ContIndex != INDEX_NONE)
	{
		Constraint->SetActive(!Constraint->Active);
	}
}

//////////////////////////////////////////////////////////////
/// SConstraintBakeWidget
///////////////////////////////////////////////////////////

TOptional<FBakingAnimationKeySettings> SConstraintBakeWidget::BakeConstraintSettings;

void SConstraintBakeWidget::Construct(const FArguments& InArgs)
{
	check(InArgs._Sequencer);
	Sequencer = InArgs._Sequencer;

	if (BakeConstraintSettings.IsSet() == false)
	{
		BakeConstraintSettings = FBakingAnimationKeySettings();
		const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		const FFrameTime FrameTime = Sequencer->GetLocalTime().ConvertTo(TickResolution);
		FFrameNumber CurrentTime = FrameTime.GetFrame();

		TRange<FFrameNumber> Range = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
		TArray<FFrameNumber> Keys;
		TArray < FKeyHandle> KeyHandles;

		BakeConstraintSettings.GetValue().StartFrame = Range.GetLowerBoundValue();
		BakeConstraintSettings.GetValue().EndFrame = Range.GetUpperBoundValue();
	}


	Settings = MakeShared<TStructOnScope<FBakingAnimationKeySettings>>();
	Settings->InitializeAs<FBakingAnimationKeySettings>(BakeConstraintSettings.GetValue());

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	DetailsView = PropertyEditor.CreateStructureDetailView(ViewArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());
	DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber",
		FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.ToSharedRef(), &ISequencer::MakeFrameNumberDetailsCustomization));
	DetailsView->SetStructureData(Settings);


	ChildSlot
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.f))
			[
				SAssignNew(ListView, ConstraintItemListView)
				.SelectionMode(ESelectionMode::Multi)
				.ListItemsSource(&ListItems)
				.OnGenerateRow(this, &SConstraintBakeWidget::OnGenerateWidgetForItem)

			]
			
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 8.f, 0.f, 0.f)
				[
					DetailsView->GetWidget().ToSharedRef()
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 16.f, 0.f, 8.f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SSpacer)
					]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked_Lambda([this, InArgs]()
						{
							BakeSelected();
							CloseDialog();
							return FReply::Handled();

						})
						.IsEnabled_Lambda([this]()
						{
							return (ListItems.Num() > 0 && Settings.IsValid());
						})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(8.f, 0.f, 16.f, 0.f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked_Lambda([this]()
					{
						CloseDialog();
					return FReply::Handled();
					})
				]
			]
		]
	];
	RefreshConstraintList();
	RegisterNotifications();
}

TSharedRef<ITableRow> SConstraintBakeWidget::OnGenerateWidgetForItem(
	ItemSharedPtr InItem,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<ItemSharedPtr>, OwnerTable)
		[
			SNew(SBakeConstraintItem, InItem.ToSharedRef(), SharedThis(this))
		];
		
}

void SConstraintBakeWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bNeedsRefresh)
	{
		RefreshConstraintList();
		bNeedsRefresh = false;
	}
}

//for the bake we auto select averything when selection or something chagnes
int32 SConstraintBakeWidget::RefreshConstraintList()
{
	int32 Num = FBaseConstraintListWidget::RefreshConstraintList();
	ListView->SetItemSelection(ListItems, true, ESelectInfo::Direct);
	return Num;
}

void SConstraintBakeWidget::BakeSelected()
{
	TArray<TSharedPtr<FEditableConstraintItem>> Selection = ListView->GetSelectedItems();
	TArray<UTickableTransformConstraint*> Constraints;
	for (TSharedPtr<FEditableConstraintItem>& Item : Selection)
	{
		if (UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Item->Constraint.Get()))
		{
			Constraints.Add(TransformConstraint);
		}
	}
	UWorld* World = UE::Private::GetCurrentWorld();

	FBakingAnimationKeySettings* BakeSettings = Settings->Get();
	
	FScopedTransaction Transaction(LOCTEXT("BakeConstraints", "Bake Constraints"));
	FConstraintBaker::BakeMultiple(World,Constraints, Sequencer,*BakeSettings );
	BakeConstraintSettings = *BakeSettings;
	InvalidateConstraintList();
}

class SConstraintBakedWidgetWindow : public SWindow
{
};



FReply SConstraintBakeWidget::OpenDialog(bool bModal)
{
	check(!DialogWindow.IsValid());

	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

	TSharedRef<SConstraintBakedWidgetWindow> Window = SNew(SConstraintBakedWidgetWindow)
		.Title(LOCTEXT("SConstraintsBakeWidgetTitle", "Bake Constraints"))
		.CreateTitleBar(true)
		.Type(EWindowType::Normal)
		.SizingRule(ESizingRule::Autosized)
		.ScreenPosition(CursorPos)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		[
			AsShared()
		];

	Window->SetWidgetToFocusOnActivate(AsShared());

	DialogWindow = Window;

	Window->MoveWindowTo(CursorPos);

	if (bModal)
	{
		GEditor->EditorAddModalWindow(Window);
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}

	return FReply::Handled();
}

void SConstraintBakeWidget::CloseDialog()
{
	if (DialogWindow.IsValid())
	{
		DialogWindow.Pin()->RequestDestroyWindow();
		DialogWindow.Reset();
	}
}

//////////////////////////////////////////////////////////////
/// SBakeConstraintItem
///////////////////////////////////////////////////////////


void SBakeConstraintItem::Construct(
	const FArguments& InArgs,
	const TSharedPtr<FEditableConstraintItem>& InItem,
	TSharedPtr<SConstraintBakeWidget> InConstraintsWidget)
{
	using namespace UE::Private;
	
	ConstraintItem = InItem;
	ConstraintsWidget = InConstraintsWidget;

	static const FSlateBrush* RoundedBoxBrush = FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.SpacePicker.RoundedRect"));

	// enum to string
	const uint8 ConstraintType = static_cast<uint8>(InItem->Type);


	// constraint
	auto GetConstraint = [this]() -> UTickableConstraint*
	{
		UWorld* World = GetCurrentWorld();
		if (!IsValid(World))
		{
			return nullptr;
		}

		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		UTickableConstraint* Constraint = ConstraintItem->Constraint.Get();
		int32 Index = Controller.GetConstraintsArray().Find(Constraint);
		if (Index == INDEX_NONE)
		{
			return nullptr;
		}
		return Constraint;
	};
	TWeakObjectPtr<UTickableConstraint> Constraint = GetConstraint();

	// sequencer
	TWeakPtr<ISequencer> WeakSequencer = GetSequencerChecked();

	// labels
	FString ParentLabel(TEXT("undefined")), ChildLabel(TEXT("undefined"));
	

	FString ParentFullLabel = ParentLabel, ChildFullLabel = ChildLabel;
	if (IsValid(Constraint.Get()))
	{
		Constraint->GetFullLabel().Split(HandleLabelDelimiter, &ParentFullLabel, &ChildFullLabel);
	}

	// widgets
	ChildSlot
		.Padding(FMargin(8.f, 2.f, 12.f, 2.f))
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(0)
			[
			SNew(SBorder)
			.Padding(FMargin(5.0, 2.0, 5.0, 2.0))
			.BorderImage(RoundedBoxBrush)
			.BorderBackgroundColor_Lambda([Constraint]()
				{
					return FStyleColors::Transparent;
				})	
				.Content()
				[
					SNew(SHorizontalBox)

					// constraint icon
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
					[
						SNew(SImage)
						.Image(FConstraintInfo::GetBrush(ConstraintType))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

				// constraint name
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(0)
					[
						SNew(STextBlock)
						.Text_Lambda([Constraint, ParentLabel, ChildLabel]()
							{
								FString CurrentParentLabel = ParentLabel, CurrentChildLabel = ChildLabel;
								if (Constraint.IsValid() && IsValid(Constraint.Get()) && Constraint->IsValid())
								{
									FString Label = Constraint->GetLabel();
									Label.Split(HandleLabelDelimiter, &CurrentParentLabel, &CurrentChildLabel);
								}
								static constexpr TCHAR LabelFormat[] = TEXT("%s <-- %s");
								const FString Label = FString::Printf(LabelFormat, *CurrentParentLabel, *CurrentChildLabel);
								return FText::FromString(Label);
							})
					.Font_Lambda([Constraint]()
						{
							if (!Constraint.IsValid() || !IsValid(Constraint.Get()))
							{
								return IDetailLayoutBuilder::GetDetailFont();
							}
							return Constraint->Active ? IDetailLayoutBuilder::GetDetailFont() : IDetailLayoutBuilder::GetDetailFontItalic();
						})
								.ToolTipText_Lambda([Constraint, ParentFullLabel, ChildFullLabel]()
									{
										if (!Constraint.IsValid() || !IsValid(Constraint.Get()))
										{
											return FText();
										}

						static constexpr TCHAR ToolTipFormat[] = TEXT("%s constraint between parent '%s' and child '%s'.");
						const FString TypeLabel = Constraint->GetTypeLabel();
						const FString FullLabel = FString::Printf(ToolTipFormat, *TypeLabel, *ParentFullLabel, *ChildFullLabel);
						return FText::FromString(FullLabel);
									})
					]
					]
		]

	
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
