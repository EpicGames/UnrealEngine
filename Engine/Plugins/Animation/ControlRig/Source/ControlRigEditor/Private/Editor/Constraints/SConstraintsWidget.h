// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Constraint.h"
#include "EditorUndoClient.h"
#include "IStructureDetailsView.h"
#include "BakingAnimationKeySettings.h"
#include "Misc/QualifiedFrameTime.h"
#include "ConstraintsManager.h"

#define UE_API CONTROLRIGEDITOR_API

class UTickableTransformConstraint;
class AActor;
class SConstraintsCreationWidget;
class SConstraintsEditionWidget;
class SComboButton;
class ISequencer;

DECLARE_DELEGATE(FOnConstraintCreated);

/**
 * FConstrainable is a UObject/Socket wrapper used to represent the future transformable handle 
 */
struct FConstrainable
{
	TWeakObjectPtr<AActor> Actor = nullptr;
	TWeakObjectPtr<UObject> Object = nullptr;
	FName Socket = NAME_None;
};
DECLARE_DELEGATE_RetVal(FConstrainable, FOnGetConstrainable);

/**
 * FConstraintInfo
 */

class FConstraintInfo
{
public:
	static const FSlateBrush* GetBrush(uint8 InType);
	static int8 GetType(UClass* InClass);
	static UTickableTransformConstraint* GetConfigurable(ETransformConstraintType InType);
	
private:
	static const TArray< const FSlateBrush* >& GetBrushes();
	static const TMap< UClass*, ETransformConstraintType >& GetConstraintToType();
	static const TArray< UTickableTransformConstraint* >& GetConfigurableConstraints();
};

/**
 * The classes below are used to implement a constraint creation widget.
 *
 * It represents the different constraints type that can be created between two objects. At this stage, it only
 * transform constraints (as described in EConstType but can basically represents any type  of constraint.
 * The resulting widget is a tree filled with drag & droppable items that can be dropped on actors.
 * The selection represents the child and the picked actor will be the parent of the constraint. 
 */

/**
 * FDroppableConstraintItem
 */

struct FDroppableConstraintItem
{
	static TSharedRef<FDroppableConstraintItem> Make(ETransformConstraintType InType)
	{
		return MakeShareable(new FDroppableConstraintItem(InType));
	}
	
	ETransformConstraintType Type = ETransformConstraintType::Parent;

private:
	FDroppableConstraintItem(ETransformConstraintType InType)
		: Type(InType)
	{}
	FDroppableConstraintItem() = default;
};

/**
 * SConstraintMenuEntry
 */

class SConstraintMenuEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SConstraintMenuEntry){}

		SLATE_EVENT(FOnConstraintCreated, OnConstraintCreated)
		SLATE_EVENT(FOnGetConstrainable, OnGetParent)
		SLATE_ARGUMENT(TWeakPtr<ISequencer>, WeakSequencer)
	
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs and the actual tree item. */
	void Construct(
		const FArguments& InArgs,
		const ETransformConstraintType& InType);

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	// End of SWidget interface

private:

	/** Uses the current selected parent or pick a new one if none has been set. */
	FReply CreateFromSelectionOrPicker(const bool bUseDefault) const;
	
	/** Activates the ActorPickerMode to pick a parent in the viewport. */
	FReply CreateSelectionPicker(const bool bUseDefault) const;

	/** Delegate triggered when a new constraint has been created. */
	FOnConstraintCreated OnConstraintCreated;

	/** Delegate to get constrainable parent data. */
	FOnGetConstrainable OnGetParent;

	/** Creates a widget to edit the constraint default properties. */
	TSharedRef<SWidget> GenerateConstraintDefaultWidget();
	
	/** Creates the constraint between the current selection and the picked actor. */
	static void OnParentPicked(
		AActor* InParent,
		const FOnConstraintCreated& InDelegate,
		const ETransformConstraintType InConstraintType,
		const bool bUseDefault);

	/** TSharedPtr to the tree item. */
	TSharedPtr<const FDroppableConstraintItem> ConstraintItem;

	ETransformConstraintType ConstraintType = ETransformConstraintType::Parent;

	/** State on the item. */
	bool bIsPressed = false;

	/** object ptr to the configurable constraint. */
	TObjectPtr<UTickableConstraint> ConfigurableConstraint = nullptr;
};

/**
 * The classes below are used to display a list of constraints. 
 */

/**
 * FEditableConstraintItem
 */

class FEditableConstraintItem
{
public:
	static TSharedRef<FEditableConstraintItem> Make(
		UTickableConstraint* InConstraint,
		ETransformConstraintType InType)
	{
		return MakeShareable(new FEditableConstraintItem(InConstraint, InType));
	}
	
	TWeakObjectPtr<UTickableConstraint> Constraint = nullptr;
	ETransformConstraintType Type = ETransformConstraintType::Parent;

	FName GetName() const
	{
		if (Constraint.IsValid())
		{
			return Constraint->GetFName();
		}
		return NAME_None;
	}
	FString GetLabel() const
	{
		if (Constraint.IsValid())
		{
			return Constraint->GetLabel();
		}
		return FString();
	}

private:
	FEditableConstraintItem(UTickableConstraint* InConstraint, ETransformConstraintType InType)
		: Constraint(InConstraint)
		, Type(InType)
	{}
	FEditableConstraintItem() {}
};

/**
 * SEditableConstraintItem
 */

class SEditableConstraintItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEditableConstraintItem){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs and the actual tree item. */
	void Construct(
		const FArguments& InArgs,
		const TSharedPtr<FEditableConstraintItem>& InItem,
		TSharedPtr<SConstraintsEditionWidget> InConstraintsWidget);
	
private:
	/** TSharedPtr to the tree item. */
	TSharedPtr<FEditableConstraintItem> ConstraintItem;
	TWeakPtr<SConstraintsEditionWidget> ConstraintsWidget;
};

/**
* Constraint List Shared by Edit Widget and Bake Widget
*/

class FBaseConstraintListWidget : public FEditorUndoClient
{
public:

	/**
	*  Constraints to show in Widget
	*/
	enum class EShowConstraints
	{
		ShowSelected = 0x0,
		ShowLevelSequence = 0x1,
		ShowValid = 0x2,
		ShowAll = 0x3,
	};
public:

	UE_API virtual ~FBaseConstraintListWidget() override;
	/* FEditorUndoClient interface */
	UE_API virtual void PostUndo(bool bSuccess);
	UE_API virtual void PostRedo(bool bSuccess);
	/* End FEditorUndoClient interface */

	/** Invalidates the constraint list for further rebuild. */
	UE_API virtual void InvalidateConstraintList();

	/** Rebuild the constraint list based on the current selection. */
	UE_API virtual int32 RefreshConstraintList();

	/** Triggers a constraint list invalidation when selection in the level viewport. */
	UE_API void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

	/** Which constraints to show  */
	EShowConstraints GetShowConstraints() const { return ShowConstraints; }
	void SetShowConstraints(EShowConstraints InShowConstraints)
	{
		if (InShowConstraints != ShowConstraints)
		{
			ShowConstraints = InShowConstraints;
			InvalidateConstraintList();
		}
	}
	UE_API FText GetShowConstraintsText(EShowConstraints Index) const;
	UE_API FText GetShowConstraintsTooltip(EShowConstraints Index) const;

protected:
	/** Types */
	using ItemSharedPtr = TSharedPtr<FEditableConstraintItem>;
	using ConstraintItemListView = SListView< ItemSharedPtr >;

	void RegisterNotifications();
	void UnregisterNotifications();

	/** List view that shows constraint types*/
	TSharedPtr< ConstraintItemListView > ListView;

	/** List of constraint types */
	TArray< ItemSharedPtr > ListItems;

	/** Boolean used to handle the items list refresh. */
	bool bNeedsRefresh = false;

	FDelegateHandle OnSelectionChangedHandle;
	FDelegateHandle ConstraintsNotificationHandle;

public:

	static UE_API EShowConstraints ShowConstraints;

};

/**
 * SConstraintsEditionWidget
 */

class SConstraintsEditionWidget : public SCompoundWidget, public FBaseConstraintListWidget
{
public:
	SLATE_BEGIN_ARGS(SConstraintsEditionWidget)	{}
		
		// This is a temporary way to allow you to add "rows" above the details for this panel. This will be revisited in 5.8.
		SLATE_NAMED_SLOT(FArguments, LeftSplitterContent)
		SLATE_NAMED_SLOT(FArguments, MiddleSplitterContent)
		SLATE_NAMED_SLOT(FArguments, RightSplitterContent)
		
		/** Allows adding extra buttons on the left of the bake button. */
		SLATE_NAMED_SLOT(FArguments, LeftButtonArea)
	SLATE_END_ARGS()
	
	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	/** Override for SWidget::Tick */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**  */
	UE_API bool CanMoveUp(const TSharedPtr<FEditableConstraintItem>& Item) const;
	UE_API bool CanMoveDown(const TSharedPtr<FEditableConstraintItem>& Item) const;

	/**  */
	UE_API void MoveItemUp(const TSharedPtr<FEditableConstraintItem>& Item);
	UE_API void MoveItemDown(const TSharedPtr<FEditableConstraintItem>& Item);

	/**  */
	UE_API void RemoveItem(const TSharedPtr<FEditableConstraintItem>& Item);

	// FBaseConstraintListWidget overrides
	UE_API virtual void InvalidateConstraintList() override;

	/** Notify from sequencer changed. */
	UE_API void SequencerChanged(const TWeakPtr<ISequencer>& InNewSequencer);

	/** Resets the selected parent data. */
	UE_API void ResetParent();

	/** Returns the parent that has been set using the chooser widget. */
	UE_API const FConstrainable& GetParent() const;
	UE_API bool IsParentValid() const;

private:

	/** Updates the current constrainable parent data. */
	UE_API void UpdateParent(AActor* InActor);

	/** Constrainable parent data to be sued to create a new constraint. */
	FConstrainable ConstrainableParent;

	/** Widgets to set the constrainable parent without using the picker. */
	UE_API TSharedRef<SWidget> OnGetParentMenuContent();
	UE_API TSharedPtr<SComboButton> GetActorChooserWidget();
	UE_API TSharedRef<SWidget> GetUseSelectedActorWidget();
	UE_API void OnParentMenuOpenChanged(bool bOpened);
	
	/** Generates a widget for the specified item */
	UE_API TSharedRef<ITableRow> OnGenerateWidgetForItem(ItemSharedPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Types */
	using ItemSharedPtr = TSharedPtr<FEditableConstraintItem>;
	using ConstraintItemListView = SListView< ItemSharedPtr >;

	/** @todo documentation. */
	UE_API TSharedPtr< SWidget > CreateContextMenu();
	UE_API void OnItemDoubleClicked(ItemSharedPtr InItem);

	UE_API FReply OnBakeClicked();

	UE_API void UpdateSequencer();

	//sequencer and it's time
	TWeakPtr<ISequencer> WeakSequencer;
	UE_API bool SequencerTimeChanged();
	FQualifiedFrameTime SequencerLastTime;

	TSharedPtr<SComboButton> ActorSelectionButtons;
};

/** Widget allowing baking of constraints */
class SConstraintBakeWidget : public SCompoundWidget, public FBaseConstraintListWidget
{
public:

	SLATE_BEGIN_ARGS(SConstraintBakeWidget)
		: _Sequencer(nullptr)
	{}

	SLATE_ARGUMENT(TSharedPtr<ISequencer>, Sequencer)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	virtual ~SConstraintBakeWidget() override = default;

	UE_API FReply OpenDialog(bool bModal = true);
	UE_API void CloseDialog();

	//SWidget overrides
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//FBaseConstraintListWidget overrides
	UE_API virtual int32 RefreshConstraintList() override;

private:
	UE_API void BakeSelected();

	/** Generates a widget for the specified item */
	UE_API TSharedRef<ITableRow> OnGenerateWidgetForItem(ItemSharedPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<ISequencer> Sequencer;
	//static to be reused
	static UE_API TOptional<FBakingAnimationKeySettings> BakeConstraintSettings;
	//structonscope for details panel
	TSharedPtr < TStructOnScope<FBakingAnimationKeySettings>> Settings;
	TWeakPtr<SWindow> DialogWindow;
	TSharedPtr<IStructureDetailsView> DetailsView;

};

/**
 * SBakeConstraintItem
 */

class SBakeConstraintItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBakeConstraintItem) {}
	SLATE_END_ARGS()

		/** Constructs this widget with InArgs and the actual tree item. */
		void Construct(
			const FArguments& InArgs,
			const TSharedPtr<FEditableConstraintItem>& InItem,
			TSharedPtr<SConstraintBakeWidget> InConstraintsWidget);

private:
	/** TSharedPtr to the tree item. */
	TSharedPtr<FEditableConstraintItem> ConstraintItem;
	TWeakPtr<SConstraintBakeWidget> ConstraintsWidget;
};

#undef UE_API
