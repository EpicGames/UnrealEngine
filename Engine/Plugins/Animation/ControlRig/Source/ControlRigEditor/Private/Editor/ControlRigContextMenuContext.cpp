// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigContextMenuContext.h"

#include "Editor/ControlRigEditor.h"
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigContextMenuContext)

FString FControlRigRigHierarchyToGraphDragAndDropContext::GetSectionTitle() const
{
	TArray<FString> NameStrings;
	for (const FRigHierarchyKey& HierarchyKey: DraggedHierarchyKeys)
	{
		NameStrings.Add(HierarchyKey.GetName());
	}
	FString SectionTitle = FString::Join(NameStrings, TEXT(","));
	if(SectionTitle.Len() > 64)
	{
		SectionTitle = SectionTitle.Left(61) + TEXT("...");
	}
	return SectionTitle;
}

void UControlRigContextMenuContext::Init(TWeakPtr<IControlRigBaseEditor> InControlRigEditor, const FControlRigMenuSpecificContext& InMenuSpecificContext)
{
	WeakControlRigEditor = InControlRigEditor;
	MenuSpecificContext = InMenuSpecificContext;
}

FControlRigAssetInterfacePtr UControlRigContextMenuContext::GetControlRigAssetInterface() const
{
	if (const TSharedPtr<IControlRigBaseEditor> Editor = WeakControlRigEditor.Pin())
	{
		return Editor->GetControlRigAssetInterface();
	}
	
	return nullptr;
}

UControlRig* UControlRigContextMenuContext::GetControlRig() const
{
	if (FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if (UControlRig* ControlRig = Cast<UControlRig>(RigBlueprint->GetObjectBeingDebugged()))
		{
			return ControlRig;
		}
	}
	return nullptr;
}

bool UControlRigContextMenuContext::IsAltDown() const
{
	return FSlateApplication::Get().GetModifierKeys().IsAltDown();
}

FControlRigRigHierarchyDragAndDropContext UControlRigContextMenuContext::GetRigHierarchyDragAndDropContext()
{
	return MenuSpecificContext.RigHierarchyDragAndDropContext;
}

FControlRigGraphNodeContextMenuContext UControlRigContextMenuContext::GetGraphNodeContextMenuContext()
{
	return MenuSpecificContext.GraphNodeContextMenuContext;
}

FControlRigRigHierarchyToGraphDragAndDropContext UControlRigContextMenuContext::GetRigHierarchyToGraphDragAndDropContext()
{
	return MenuSpecificContext.RigHierarchyToGraphDragAndDropContext;
}

SRigHierarchy* UControlRigContextMenuContext::GetRigHierarchyPanel() const
{
	if (const TSharedPtr<SRigHierarchy> RigHierarchyPanel = MenuSpecificContext.RigHierarchyPanel.Pin())
	{
		return RigHierarchyPanel.Get();
	}
	return nullptr;
}

SModularRigHierarchy* UControlRigContextMenuContext::GetModularRigModelPanel() const
{
	if (const TSharedPtr<SModularRigHierarchy> ModularRigModelPanel = MenuSpecificContext.ModularRigModelPanel.Pin())
	{
		return ModularRigModelPanel.Get();
	}
	return nullptr;
}

IControlRigBaseEditor* UControlRigContextMenuContext::GetControlRigEditor() const
{
	if (const TSharedPtr<IControlRigBaseEditor> Editor = WeakControlRigEditor.Pin())
	{
		return Editor.Get();
	}
	return nullptr;
}

