// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigEditorAsset.h"
#include "ControlRigBlueprintLegacy.h"
#include "EdGraph/EdGraph.h"
#include "Rigs/RigHierarchyDefines.h"

#include "ControlRigContextMenuContext.generated.h"

class UControlRig;
class UControlRigBlueprint;
class IControlRigBaseEditor;
class URigVMGraph;
class URigVMNode;
class URigVMPin;
class SRigHierarchy;
class SModularRigHierarchy;

USTRUCT(BlueprintType)
struct FControlRigRigHierarchyDragAndDropContext
{
	GENERATED_BODY()
	
	FControlRigRigHierarchyDragAndDropContext() = default;
	
	FControlRigRigHierarchyDragAndDropContext(const TArray<FRigHierarchyKey> InDraggedHierarchyKeys, FRigHierarchyKey InTargetHierarchyKey)
		: DraggedHierarchyKeys(InDraggedHierarchyKeys)
		, TargetHierarchyKey(InTargetHierarchyKey)
	{
	}
	
	UPROPERTY(BlueprintReadOnly, Category = ControlRigEditorExtensions)
	TArray<FRigHierarchyKey> DraggedHierarchyKeys;

	UPROPERTY(BlueprintReadOnly, Category = ControlRigEditorExtensions)
	FRigHierarchyKey TargetHierarchyKey;
};

USTRUCT(BlueprintType)
struct FControlRigGraphNodeContextMenuContext
{
	GENERATED_BODY()

	FControlRigGraphNodeContextMenuContext()
		: Graph(nullptr)
		, Node(nullptr)
		, Pin(nullptr)
	{
	}
	
	FControlRigGraphNodeContextMenuContext(TObjectPtr<const URigVMGraph> InGraph, TObjectPtr<const URigVMNode> InNode, TObjectPtr<const URigVMPin> InPin)
		: Graph(InGraph)
		, Node(InNode)
		, Pin(InPin)
	{
	}
	
	/** The graph associated with this context. */
	UPROPERTY(BlueprintReadOnly, Category = ControlRigEditorExtensions)
	TObjectPtr<const URigVMGraph> Graph;

	/** The node associated with this context. */
	UPROPERTY(BlueprintReadOnly, Category = ControlRigEditorExtensions)
	TObjectPtr<const URigVMNode> Node;

	/** The pin associated with this context; may be NULL when over a node. */
	UPROPERTY(BlueprintReadOnly, Category = ControlRigEditorExtensions)
	TObjectPtr<const URigVMPin> Pin;
};

USTRUCT(BlueprintType)
struct FControlRigRigHierarchyToGraphDragAndDropContext
{
	GENERATED_BODY()

	FControlRigRigHierarchyToGraphDragAndDropContext() = default;
	
	FControlRigRigHierarchyToGraphDragAndDropContext(const TArray<FRigHierarchyKey>& InDraggedHierarchyKeys, UEdGraph* InGraph, const FVector2D& InNodePosition)
		:DraggedHierarchyKeys(InDraggedHierarchyKeys)
		,Graph(InGraph)
		,NodePosition(InNodePosition)
	{ };

	UPROPERTY(BlueprintReadOnly, Category = ControlRigEditorExtensions)
	TArray<FRigHierarchyKey> DraggedHierarchyKeys;

	TWeakObjectPtr<UEdGraph> Graph;

	FVector2D NodePosition;

	FString GetSectionTitle() const;
};

struct FControlRigMenuSpecificContext
{
	TWeakPtr<SRigHierarchy> RigHierarchyPanel;
	
	FControlRigRigHierarchyDragAndDropContext RigHierarchyDragAndDropContext;

	TWeakPtr<SModularRigHierarchy> ModularRigModelPanel;
	
	FControlRigGraphNodeContextMenuContext GraphNodeContextMenuContext;

	FControlRigRigHierarchyToGraphDragAndDropContext RigHierarchyToGraphDragAndDropContext;
};

UCLASS(BlueprintType)
class UControlRigContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	/**
	 *	Initialize the Context
	 * @param InControlRigEditor 	    The Control Rig Editor hosting the menus
	 * @param InMenuSpecificContext 	Additional context for specific menus
	*/
	void Init(TWeakPtr<IControlRigBaseEditor> InControlRigEditor, const FControlRigMenuSpecificContext& InMenuSpecificContext = FControlRigMenuSpecificContext());

	/** Get the control rig blueprint that we are editing */
	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetControlRigAssetInterface instead."))
    UControlRigBlueprint* GetControlRigBlueprint() const { return Cast<UControlRigBlueprint>(GetControlRigAssetInterface().GetObject()); }

	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions)
	TScriptInterface<IControlRigEditorAssetInterface> GetControlRigAssetInterface() const;
	
	/** Get the active control rig instance in the viewport */
	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions)
    UControlRig* GetControlRig() const;

	/** Returns true if either alt key is down */
	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions)
	bool IsAltDown() const;
	
	/** Returns context for a drag & drop action that contains source and target element keys */ 
	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions)
	FControlRigRigHierarchyDragAndDropContext GetRigHierarchyDragAndDropContext();

	/** Returns context for graph node context menu */
	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions)
	FControlRigGraphNodeContextMenuContext GetGraphNodeContextMenuContext();
	
	/** Returns context for the menu that shows up when drag and drop from Rig Hierarchy to the Rig Graph*/
	UFUNCTION(BlueprintCallable, Category = ControlRigEditorExtensions)
    FControlRigRigHierarchyToGraphDragAndDropContext GetRigHierarchyToGraphDragAndDropContext();

	SRigHierarchy* GetRigHierarchyPanel() const;

	SModularRigHierarchy* GetModularRigModelPanel() const;
	
	IControlRigBaseEditor* GetControlRigEditor() const;

private:
	TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;

	FControlRigMenuSpecificContext MenuSpecificContext;
};