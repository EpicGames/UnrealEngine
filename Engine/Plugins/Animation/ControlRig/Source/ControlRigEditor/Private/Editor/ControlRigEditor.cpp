// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigEditor.h"
#include "Modules/ModuleManager.h"
#include "ControlRigEditorModule.h"
#include "ControlRigBlueprintLegacy.h"
#include "Editor/ControlRigViewportToolbarExtensionControlRig.h"
#include "SBlueprintEditorToolbar.h"
#include "Editor/ControlRigEditorMode.h"
#include "SEnumCombo.h"
#include "SceneView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Framework/Commands/GenericCommands.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Graph/ControlRigGraph.h"
#include "BlueprintActionDatabase.h"
#include "ControlRigEditorCommands.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "IPersonaToolkit.h"
#include "PersonaModule.h"
#include "Editor/ControlRigEditorEditMode.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "EditorModeManager.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "AnimCustomInstanceHelper.h"
#include "Sequencer/ControlRigLayerInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "IPersonaPreviewScene.h"
#include "Animation/AnimData/BoneMaskFilter.h"
#include "ControlRig.h"
#include "ModularRig.h"
#include "Editor/ControlRigSkeletalMeshComponent.h"
#include "ControlRigObjectBinding.h"
#include "RigVMBlueprintUtils.h"
#include "EditorViewportClient.h"
#include "AnimationEditorPreviewActor.h"
#include "Misc/MessageDialog.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ControlRigEditorStyle.h"
#include "Editor/RigVMEditorStyle.h"
#include "EditorFontGlyphs.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Units/Hierarchy/RigUnit_BoneName.h"
#include "Units/Hierarchy/RigUnit_GetTransform.h"
#include "Units/Hierarchy/RigUnit_SetTransform.h"
#include "Units/Hierarchy/RigUnit_GetRelativeTransform.h"
#include "Units/Hierarchy/RigUnit_SetRelativeTransform.h"
#include "Units/Hierarchy/RigUnit_OffsetTransform.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Units/Hierarchy/RigUnit_GetControlTransform.h"
#include "Units/Hierarchy/RigUnit_SetControlTransform.h"
#include "Units/Hierarchy/RigUnit_ControlChannel.h"
#include "Units/Execution/RigUnit_Collection.h"
#include "Units/Highlevel/Hierarchy/RigUnit_TransformConstraint.h"
#include "Units/Hierarchy/RigUnit_GetControlTransform.h"
#include "Units/Hierarchy/RigUnit_SetCurveValue.h"
#include "Units/Hierarchy/RigUnit_AddBoneTransform.h"
#include "Units/Hierarchy/RigUnit_Component.h"
#include "Units/Hierarchy/RigUnit_Metadata.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphUnitNodeSpawner.h"
#include "Graph/ControlRigGraphSchema.h"
#include "ControlRigObjectVersion.h"
#include "EdGraphUtilities.h"
#include "EdGraphNode_Comment.h"
#include "HAL/PlatformApplicationMisc.h"
#include "SNodePanel.h"
#include "SMyBlueprint.h"
#include "SBlueprintEditorSelectedDebugObjectWidget.h"
#include "Exporters/Exporter.h"
#include "UnrealExporter.h"
#include "ControlRigElementDetails.h"
#include "PropertyEditorModule.h"
#include "PropertyCustomizationHelpers.h"
#include "Settings/ControlRigSettings.h"
#include "Widgets/Docking/SDockTab.h"
#include "BlueprintCompilationManager.h"
#include "AssetEditorModeManager.h"
#include "IPersonaEditorModeManager.h"
#include "BlueprintEditorTabs.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "IMessageLogListing.h"
#include "Widgets/SRigVMGraphFunctionLocalizationWidget.h"
#include "Widgets/SRigVMGraphFunctionBulkEditWidget.h"
#include "Widgets/SRigVMGraphBreakLinksWidget.h"
#include "Widgets/SRigVMGraphChangePinType.h"
#include "SGraphPanel.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Components/StaticMeshComponent.h"
#include "RigVMFunctions/Execution/RigVMFunction_Sequence.h"
#include "Editor/ControlRigContextMenuContext.h"
#include "Types/ISlateMetaData.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "Kismet/KismetSystemLibrary.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MaterialDomain.h"
#include "RigVMFunctions/RigVMFunction_ControlFlow.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "AnimationEditorViewportClient.h"
#include "ControlRigRuntimeAsset.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor/RigVMEditorTools.h"
#include "SchematicGraphPanel/SSchematicGraphPanel.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "Widgets/SRigVMSwapAssetReferencesWidget.h"
#include "Widgets/SRigVMBulkEditDialog.h"
#include "ControlRigTestData.h"
#include "SEditorViewport.h"
#include "Editor/Persona/Private/SAnimationEditorViewport.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "ITransportControl.h"
#include "PropertyPath.h"
#include "Editor/SRigVMDetailsInspector.h"
#include "Overrides/OverrideStatusDetailsObjectFilter.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyElementDragDropOp.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#define LOCTEXT_NAMESPACE "ControlRigEditor"

TAutoConsoleVariable<bool> CVarControlRigShowTestingToolbar(TEXT("ControlRig.Test.EnableTestingToolbar"), false, TEXT("When true we'll show the testing toolbar in Control Rig Editor."));
TAutoConsoleVariable<bool> CVarShowSchematicPanelOverlay(TEXT("ControlRig.Preview.ShowSchematicPanelOverlay"), true, TEXT("When true we'll add an overlay to the persona viewport to show modular rig information."));

const FName FControlRigEditorModes::ControlRigEditorMode = TEXT("Rigging");
const TArray<FName> FControlRigBaseEditor::ForwardsSolveEventQueue = {FRigUnit_BeginExecution::EventName};
const TArray<FName> FControlRigBaseEditor::BackwardsSolveEventQueue = {FRigUnit_InverseExecution::EventName};
const TArray<FName> FControlRigBaseEditor::ConstructionEventQueue = {FRigUnit_PrepareForExecution::EventName};
const TArray<FName> FControlRigBaseEditor::BackwardsAndForwardsSolveEventQueue = {FRigUnit_InverseExecution::EventName, FRigUnit_BeginExecution::EventName};

FControlRigBaseEditor::FControlRigBaseEditor()
	: PreviewInstance(nullptr)
	, ActiveController(nullptr)
	, bExecutionControlRig(true)
	, RigHierarchyTabCount(0)
	, ModularRigHierarchyTabCount(0)
	, bIsConstructionEventRunning(false)
	, LastHierarchyHash(INDEX_NONE)
	, bRefreshDirectionManipulationTargetsRequired(false)
{
	SchematicModel = MakeShared<FControlRigSchematicModel>();
}

FControlRigBaseEditor* FControlRigBaseEditor::GetFromAssetEditorInstance(IAssetEditorInstance* Instance)
{
	FControlRigBaseEditor* Editor = nullptr;
	FWorkflowCentricApplication* App = static_cast<FWorkflowCentricApplication*>(Instance);

	TSharedRef<FAssetEditorToolkit> SharedApp = App->AsShared();
#if WITH_RIGVMLEGACYEDITOR
	if (SharedApp->IsBlueprintEditor())
	{
		TSharedPtr<FControlRigLegacyEditor> LegacyEditor = StaticCastSharedPtr<FControlRigLegacyEditor>(App->AsShared().ToSharedPtr());
		Editor = LegacyEditor.Get();
	}
	else
	{
		TSharedPtr<FControlRigEditor> NewEditor = StaticCastSharedPtr<FControlRigEditor>(App->AsShared().ToSharedPtr());
		Editor = NewEditor.Get();
	}
#else
	TSharedPtr<FControlRigEditor> NewEditor = StaticCastSharedPtr<FControlRigEditor>(App->AsShared().ToSharedPtr());
	Editor = NewEditor.Get();
#endif
	return Editor;
}

UObject* FControlRigBaseEditor::GetOuterForHostImpl() const
{
	UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
	if(EditorSkelComp)
	{
		return EditorSkelComp;
	}
	return GetOuterForHostSuper();
}

UClass* FControlRigBaseEditor::GetDetailWrapperClassImpl() const
{
	return UControlRigWrapperObject::StaticClass();
}



FReply FControlRigBaseEditor::OnViewportDropImpl(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const FReply SuperReply = OnViewportDropSuper(MyGeometry, DragDropEvent);
	if(SuperReply.IsEventHandled())
	{
		return SuperReply;
	}

	if(IsModularRig())
	{
		const TSharedPtr<FAssetDragDropOp> AssetDragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
		if (AssetDragDropOperation)
		{
			for (const FAssetData& AssetData : AssetDragDropOperation->GetAssets())
			{
				FControlRigAssetStrongReference AssetObject;
				if(const FControlRigAssetInterfacePtr AssetBlueprint = AssetData.GetAsset())
				{
					if (AssetBlueprint->IsControlRigModule())
					{
						AssetObject = AssetBlueprint->GetControlRigAssetReference();
					}
				}
				else if (UControlRigBlueprintGeneratedClass* GeneratedClass = Cast<UControlRigBlueprintGeneratedClass>(AssetData.GetAsset()))
				{
					if (GeneratedClass->IsControlRigModule())
					{
						AssetObject.Set(TSubclassOf<UControlRig>(GeneratedClass));
					}
				}
				else if(UControlRigRuntimeAsset* RuntimeAsset = Cast<UControlRigRuntimeAsset>(AssetData.GetAsset()))
				{
					if (RuntimeAsset->IsRigModule())
					{
						AssetObject.Set(RuntimeAsset);
					}
				}
				FControlRigAssetStrongReference ControlRigClass(AssetObject);
				if(ControlRigClass.IsValid())
				{
					FSlateApplication::Get().DismissAllMenus();

					UModularRigController* Controller = GetControlRigAssetInterface()->GetModularRigController();
					FString ClassName = ControlRigClass.Get()->GetName();
					ClassName.RemoveFromEnd(TEXT("_C"));
					const FRigName DesiredModuleName = Controller->GetSafeNewName(FRigName(ClassName));
					const FName ModuleName = Controller->AddModuleFromAssetReference(DesiredModuleName, ControlRigClass, NAME_None);
					if (!ModuleName.IsNone())
					{
						return FReply::Handled();
					}
				}
			}
		}
	}

	return FReply::Unhandled();
}

void FControlRigBaseEditor::CreateEmptyGraphContentImpl(URigVMController* InController)
{
	// If the EdGraph does not exist, create it
	if (FControlRigAssetInterfacePtr ControlRigEditorAsset = GetControlRigAssetInterface())
	{
		if (FRigVMEditorAssetInterfacePtr EditorAsset = ControlRigEditorAsset->GetRigVMAssetInterface())
		{
			if (URigVMGraph* Graph = InController->GetGraph())
			{
				UEdGraph* EdGraph = EditorAsset->GetEdGraph(Graph);
				if (!EdGraph)
				{
					EditorAsset->CreateEdGraph(Graph);
				}
			}
		}
		
	}
	
	if (!InController->GetAllEventNames().Contains(FRigUnit_BeginExecution::EventName))
	{
		URigVMNode* Node = InController->AddUnitNode(FRigUnit_BeginExecution::StaticStruct(), FRigUnit::GetMethodName(), FVector2D::ZeroVector, FString(), false);
		if (Node)
		{
			TArray<FName> NodeNames;
			NodeNames.Add(Node->GetFName());
			InController->SetNodeSelection(NodeNames, false);
		
		}
	}
}

UControlRigBlueprint* FControlRigBaseEditor::GetControlRigBlueprint() const
{
	return Cast<UControlRigBlueprint>(GetRigVMAssetInterface().GetObject());
}

FControlRigAssetInterfacePtr FControlRigBaseEditor::GetControlRigAssetInterface() const
{
	return GetRigVMAssetInterface().GetObject();
}

UControlRig* FControlRigBaseEditor::GetControlRig() const
{
	return Cast<UControlRig>(GetRigVMHost());
}

URigHierarchy* FControlRigBaseEditor::GetHierarchyBeingDebugged() const
{
	if(FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if(UControlRig* RigBeingDebugged = Cast<UControlRig>(RigBlueprint->GetObjectBeingDebugged()))
		{
			return RigBeingDebugged->GetHierarchy();
		}
		return RigBlueprint->GetHierarchy();
	}
	return nullptr;
}

void FControlRigBaseEditor::InitRigVMEditorImpl(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	InitRigVMEditorSuper(Mode, InitToolkitHost, InRigVMBlueprint);

	FControlRigAssetInterfacePtr ControlRigBlueprint(InRigVMBlueprint->GetObject());

	CreatePersonaToolKitIfRequired();
	IControlRigEditorAssetInterface::sCurrentlyOpenedRigBlueprints.AddUnique(ControlRigBlueprint);

	// When the asset has no preview mesh, the default viewport camera doesn't auto-focus on anything visible
	// (the rig's control shapes can be far from world origin -- Face_ControlBoard_CtrlRig is the canonical
	// example). Mark a one-shot request to frame the visible controls once the rig has populated. The actual
	// framing is performed in TickImpl, which gives the hierarchy a chance to compile and the viewport client
	// to be available.
	bPendingInitialViewportFocus = (ControlRigBlueprint->GetPreviewMesh() == nullptr);
	PendingInitialViewportFocusTickCount = 0;

	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->OnGetRigElementTransform() = FOnGetRigElementTransform::CreateSP(SharedRef(), &FControlRigBaseEditor::GetRigElementTransform);
		EditMode->OnSetRigElementTransform() = FOnSetRigElementTransform::CreateSP(SharedRef(), &FControlRigBaseEditor::SetRigElementTransform);
		EditMode->OnGetContextMenu() = FOnGetContextMenu::CreateSP(SharedRef(), &FControlRigBaseEditor::HandleOnGetViewportContextMenuDelegate);
		EditMode->OnContextMenuCommands() = FNewMenuCommandsDelegate::CreateSP(SharedRef(), &FControlRigBaseEditor::HandleOnViewportContextMenuCommandsDelegate);
		EditMode->OnAnimSystemInitialized().Add(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FControlRigBaseEditor::OnAnimInitialized));
	
		PersonaToolkit->GetPreviewScene()->SetRemoveAttachedComponentFilter(FOnRemoveAttachedComponentFilter::CreateSP(EditMode, &FControlRigEditMode::CanRemoveFromPreviewScene));
	}

	{
		// listening to the BP's event instead of BP's Hierarchy's Event ensure a propagation order of
		// 1. Hierarchy change in BP
		// 2. BP propagate to instances
		// 3. Editor forces propagation again, and reflects hierarchy change in either instances or BP
		// 
		// if directly listening to BP's Hierarchy's Event, this ordering is not guaranteed due to multicast,
		// a problematic order we have encountered looks like:
		// 1. Hierarchy change in BP
		// 2. FControlRigEditor::OnHierarchyModified performs propagation from BP to instances, refresh UI
		// 3. BP performs propagation again in UControlRigBlueprint::HandleHierarchyModified, invalidates the rig element
		//    that the UI is observing
		// 4. Editor UI shows an invalid rig element
		ControlRigBlueprint->OnHierarchyModified().AddSP(SharedRef(), &FControlRigBaseEditor::OnHierarchyModified);

		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			ControlRigBlueprint->OnHierarchyModified().AddSP(EditMode, &FControlRigEditMode::OnHierarchyModified_AnyThread);
		}

		if (ControlRigBlueprint->IsModularRig())
		{
			SchematicModel->SetEditor(SharedRef());
			ControlRigBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().AddSP(SchematicModel.Get(), &FControlRigSchematicModel::OnSetObjectBeingDebugged);
			ControlRigBlueprint->GetModularRigController()->OnModified().AddSP(SchematicModel.Get(), &FControlRigSchematicModel::HandleModularRigModified);
		}

		ControlRigBlueprint->OnRigTypeChanged().AddSP(SharedRef(), &FControlRigBaseEditor::HandleRigTypeChanged);
		if (ControlRigBlueprint->IsModularRig())
		{
			ControlRigBlueprint->GetModularRigController()->OnModified().AddSP(SharedRef(), &FControlRigBaseEditor::HandleModularRigModified);
			ControlRigBlueprint->OnModularRigCompiled().AddSP(SharedRef(), &FControlRigBaseEditor::HandlePostCompileModularRigs);
		}
	}

	CreateRigHierarchyToGraphDragAndDropMenu();

	if(SchematicViewport.IsValid())
	{
		SchematicModel->UpdateControlRigContent();
	}
}

void FControlRigBaseEditor::CreatePersonaToolKitIfRequired()
{
	if(PersonaToolkit.IsValid())
	{
		return;
	}
	
	FControlRigAssetInterfacePtr ControlRigBlueprint = GetControlRigAssetInterface();

	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");

	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(SharedRef(), &FControlRigBaseEditor::HandlePreviewSceneCreated);
	PersonaToolkitArgs.bPreviewMeshCanUseDifferentSkeleton = true;
	USkeleton* Skeleton = nullptr;
	if(USkeletalMesh* PreviewMesh = ControlRigBlueprint->GetPreviewMesh())
	{
		Skeleton = PreviewMesh->GetSkeleton();
	}
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(ControlRigBlueprint.GetObject(), PersonaToolkitArgs, Skeleton);

	// set delegate prior to setting mesh
	// otherwise, you don't get delegate
	PersonaToolkit->GetPreviewScene()->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(SharedRef(), &FControlRigBaseEditor::HandlePreviewMeshChanged));
	
	// Set a default preview mesh, if any
	TGuardValue<bool> AutoResolveGuard(ControlRigBlueprint->GetModularRigSettings().bAutoResolve, false);
	PersonaToolkit->SetPreviewMesh(ControlRigBlueprint->GetPreviewMesh(), false);
}

const FName FControlRigBaseEditor::GetEditorAppNameImpl() const
{
	static const FName ControlRigEditorAppName(TEXT("ControlRigEditorApp"));
	return ControlRigEditorAppName;
}

const FName FControlRigBaseEditor::GetEditorModeNameImpl() const
{
	if(IsModularRig())
	{
		return FModularRigEditorEditMode::ModeName;
	}
	return FControlRigEditorEditMode::ModeName;
}

const FSlateBrush* FControlRigBaseEditor::GetDefaultTabIconImpl() const
{
	static const FSlateIcon TabIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.Editor.TabIcon");
	return TabIcon.GetIcon();
}

FText FControlRigBaseEditor::GetReplayAssetName() const
{
	if(ReplayStrongPtr.IsValid())
	{
		return FText::FromString(ReplayStrongPtr->GetName());
	}

	static const FText NoReplayAsset = LOCTEXT("NoReplayAsset", "No Replay Asset");
	return NoReplayAsset;
}

FText FControlRigBaseEditor::GetReplayAssetTooltip() const
{
	if(ReplayStrongPtr.IsValid())
	{
		return FText::FromString(ReplayStrongPtr->GetPathName());
	}
	static const FText NoReplayTooltip = LOCTEXT("NoReplayAssetTooltip", "Click the record button to the left to record a new replay");
	return NoReplayTooltip;
}

bool FControlRigBaseEditor::SetReplayAssetPath(const FString& InAssetPath)
{
	if(ReplayStrongPtr.IsValid())
	{
		if(ReplayStrongPtr->GetPathName().Equals(InAssetPath, ESearchCase::CaseSensitive))
		{
			return false;
		}
	}

	if(ReplayStrongPtr.IsValid())
	{
		ReplayStrongPtr->StopReplay();
		ReplayStrongPtr.Reset();
	}

	if(!InAssetPath.IsEmpty())
	{
		if(UControlRigReplay* Replay = LoadObject<UControlRigReplay>(GetControlRigAssetInterface().GetObject(), *InAssetPath))
		{
			ReplayStrongPtr = TStrongObjectPtr<UControlRigReplay>(Replay);
		}
	}
	return true;
}

TSharedRef<SWidget> FControlRigBaseEditor::GenerateReplayAssetModeMenuContent()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());

	MenuBuilder.BeginSection(TEXT("Default"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearReplay", "Clear"),
		LOCTEXT("ClearReplay_ToolTip", "Clears the test asset"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				SetReplayAssetPath(FString());
			}
		)	
	));
	MenuBuilder.EndSection();

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FAssetData> ReplayAssets;
	FARFilter AssetFilter;
	AssetFilter.ClassPaths.Add(UControlRigReplay::StaticClass()->GetClassPathName());
	AssetRegistryModule.Get().GetAssets(AssetFilter, ReplayAssets);

	const FString CurrentObjectPath = GetControlRigAssetInterface().GetObject()->GetPathName();
	ReplayAssets.RemoveAll([CurrentObjectPath](const FAssetData& InAssetData)
	{
		const FString ControlRigObjectPath = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UControlRigReplay, ControlRigObjectPath));
		return ControlRigObjectPath != CurrentObjectPath;
	});

	if(!ReplayAssets.IsEmpty())
	{
		MenuBuilder.BeginSection(TEXT("Assets"));
		for(const FAssetData& ReplayAsset : ReplayAssets)
		{
			const FString ReplayObjectPath = ReplayAsset.GetObjectPathString();
			FString Right;
			if(ReplayObjectPath.Split(TEXT("."), nullptr, &Right))
			{
				MenuBuilder.AddMenuEntry(FText::FromString(Right), FText::FromString(ReplayObjectPath), FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, ReplayObjectPath]()
						{
							SetReplayAssetPath(ReplayObjectPath);
						}
					)	
				));
			}
		}
		MenuBuilder.EndSection();
	}
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FControlRigBaseEditor::GenerateReplayAssetRecordMenuContent()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ReplayReplayRecordSingleFrame", "Single Frame"),
		LOCTEXT("ReplayReplayRecordSingleFrame_ToolTip", "Records a single frame into the replay asset"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				RecordReplay(0);
			}
		)	
	));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ReplayReplayRecordOneSecond", "1 Second"),
		LOCTEXT("ReplayRecordOneSecond_ToolTip", "Records 1 second of animation into the replay asset"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				RecordReplay(1);
			}
		)	
	));

	MenuBuilder.AddMenuEntry(
	LOCTEXT("ReplayRecordFiveSeconds", "5 Seconds"),
	LOCTEXT("ReplayRecordFiveSeconds_ToolTip", "Records 5 seconds of animation into the replay asset"),
	FSlateIcon(),
	FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				RecordReplay(5);
			}
		)	
	));

	MenuBuilder.AddMenuEntry(
	LOCTEXT("ReplayRecordTenSeconds", "10 Seconds"),
	LOCTEXT("ReplayRecordTenSeconds_ToolTip", "Records 10 seconds of animation into the replay asset"),
	FSlateIcon(),
	FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				RecordReplay(10);
			}
		)	
	));

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FControlRigBaseEditor::GenerateReplayAssetPlaybackMenuContent()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());

	MenuBuilder.AddMenuEntry(
	UControlRigReplay::LiveStatus,
	UControlRigReplay::LiveStatusTooltip,
	FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ClassIcon.ControlRigBlueprint"),
	FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				if(ReplayStrongPtr.IsValid())
				{
					ReplayStrongPtr->StopReplay();
				}
			}
		)	
	));

	MenuBuilder.AddMenuEntry(
	UControlRigReplay::ReplayInputsStatus,
	UControlRigReplay::ReplayInputsStatusTooltip,
	FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ClassIcon.ControlRigSequence"),
	FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				if(ReplayStrongPtr.IsValid())
				{
					ReplayStrongPtr->StartReplay(GetControlRig(), EControlRigReplayPlaybackMode::ReplayInputs);
				}
			}
		)	
	));

	MenuBuilder.AddMenuEntry(
	UControlRigReplay::GroundTruthStatus,
	UControlRigReplay::GroundTruthStatusTooltip,
	FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ClassIcon.ControlRigSequence"),
	FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				if(ReplayStrongPtr.IsValid())
				{
					ReplayStrongPtr->StartReplay(GetControlRig(), EControlRigReplayPlaybackMode::GroundTruth);
				}
			}
		)	
	));

	return MenuBuilder.MakeWidget();
}

bool FControlRigBaseEditor::RecordReplay(double InRecordingDuration)
{
	if(GetControlRig() == nullptr)
	{
		return false;
	}

	// create a new test asset
	static const FString Folder = TEXT("/Game/Animation/ControlRig/NoCook/");
	const FString DesiredPackagePath =  FString::Printf(TEXT("%s/%s_Replay"), *Folder, *GetControlRigAssetInterface().GetObject()->GetName());

	if(UControlRigReplay* Replay = UControlRigReplay::CreateNewAsset(DesiredPackagePath, GetControlRigAssetInterface().GetObject()->GetPathName(), UControlRigReplay::StaticClass()))
	{
		SetReplayAssetPath(Replay->GetPathName());
		if(const USkeletalMesh* PreviewSkeletalMesh = GetControlRigAssetInterface()->GetPreviewMesh())
		{
			Replay->PreviewSkeletalMeshObjectPath = FSoftObjectPath(PreviewSkeletalMesh);
		}
	}

	if(UControlRigReplay* Replay = ReplayStrongPtr.Get())
	{
		Replay->Modify();
		Replay->DesiredRecordingDuration = InRecordingDuration;

		if(InRecordingDuration <= SMALL_NUMBER)
		{
			Replay->StartRecording(GetControlRig());
		}
		else
		{
			// 3 second preroll
			TSharedPtr<int32> TimeLeft = MakeShared<int32>(4);
			GEditor->GetTimerManager()->SetTimer(RecordReplayTimerHandle, FTimerDelegate::CreateLambda([this, Replay, TimeLeft]()
			{
				int32& SecondsLeft = *TimeLeft.Get();
				SecondsLeft--;

				if(SecondsLeft == 0)
				{
					Replay->StartRecording(GetControlRig());
					GEditor->GetTimerManager()->ClearTimer(RecordReplayTimerHandle);
				}
				else
				{
#if WITH_RIGVMLEGACYEDITOR
					static constexpr TCHAR PrintStringFormat[] = TEXT("Recording starts in... %d");
					UKismetSystemLibrary::PrintString(GetPreviewScene()->GetWorld(), FString::Printf(PrintStringFormat, SecondsLeft), true, false, FLinearColor::Red, 1.f);
#endif
				}
			}), 1.f, true, 0.01f);
		}
	}
	return true;
}

void FControlRigBaseEditor::ToggleReplay()
{
	if(ReplayStrongPtr.IsValid())
	{
		switch(ReplayStrongPtr->GetPlaybackMode())
		{
			case EControlRigReplayPlaybackMode::ReplayInputs:
			{
				ReplayStrongPtr->StartReplay(GetControlRig(), EControlRigReplayPlaybackMode::GroundTruth);
				break;
			}
			case EControlRigReplayPlaybackMode::GroundTruth:
			{
				ReplayStrongPtr->StopReplay();
				break;
			}
			default:
			{
				ReplayStrongPtr->StartReplay(GetControlRig(), EControlRigReplayPlaybackMode::ReplayInputs);
				break;
			}
		}
	}
}

EControlRigReplayPlaybackMode FControlRigBaseEditor::GetReplayPlaybackMode() const
{
	if (ReplayStrongPtr.IsValid())
	{
		return ReplayStrongPtr->GetPlaybackMode();
	}
	return EControlRigReplayPlaybackMode::Live;
}

void FControlRigBaseEditor::FillToolbarImpl(FToolBarBuilder& ToolbarBuilder, bool bEndSection)
{
	FillToolbarSuper(ToolbarBuilder, false);
	
	{
		if(CVarControlRigHierarchyEnableModules.GetValueOnAnyThread())
		{
			TWeakInterfacePtr<IControlRigEditorAssetInterface> WeakBlueprint = GetControlRigAssetInterface();
			ToolbarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateLambda([WeakBlueprint]()
					{
						if(WeakBlueprint.IsValid())
						{
							if(WeakBlueprint->IsControlRigModule())
							{
								WeakBlueprint->TurnIntoStandaloneRig();
							}
							else
							{
								if(!WeakBlueprint->CanTurnIntoControlRigModule(false))
								{
									static const FText Message(LOCTEXT("TurnIntoControlRigModuleMessage", "This rig requires some changes to the hierarchy to turn it into a module.\n\nWe'll try to recreate the hierarchy by relying on nodes in the construction event instead.\n\nDo you want to continue?"));
									EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo, Message);
									if(Ret != EAppReturnType::Yes)
									{
										return;
									}
								}
								WeakBlueprint->TurnIntoControlRigModule(true);
							}
						}
					}),
					FCanExecuteAction::CreateLambda([WeakBlueprint]
					{
						if(WeakBlueprint.IsValid())
						{
							if(WeakBlueprint->IsControlRigModule())
							{
								return WeakBlueprint->CanTurnIntoStandaloneRig();
							}
							return WeakBlueprint->CanTurnIntoControlRigModule(true);
						}
						return false;
					})
				),
				NAME_None,
				TAttribute<FText>::CreateLambda([WeakBlueprint]()
				{
					static const FText StandaloneRig = LOCTEXT("SwitchToRigModule", "Switch to Rig Module"); 
					static const FText RigModule = LOCTEXT("SwitchToStandaloneRig", "Switch to Standalone Rig");
					if(WeakBlueprint.IsValid())
					{
						if(WeakBlueprint->IsControlRigModule())
						{
							return RigModule;
						}
					}
					return StandaloneRig;
				}),
				TAttribute<FText>::CreateLambda([WeakBlueprint]()
				{
					static const FText StandaloneRigTooltip = LOCTEXT("StandaloneRigTooltip", "A standalone control rig."); 
					static const FText RigModuleTooltip = LOCTEXT("RigModuleTooltip", "A rig module used to build rigs.");
					if(WeakBlueprint.IsValid())
					{
						if(!WeakBlueprint->IsControlRigModule())
						{
							FString FailureReason;
							if(!WeakBlueprint->CanTurnIntoControlRigModule(true, &FailureReason))
							{
								return FText::Format(
									LOCTEXT("StandaloneRigTooltipFormat", "{0}\n\nThis rig cannot be turned into a module:\n\n{1}"),
									StandaloneRigTooltip,
									FText::FromString(FailureReason)
								);
							}
							return StandaloneRigTooltip;
						}
					}
					return RigModuleTooltip;
				}),
				TAttribute<FSlateIcon>::CreateLambda([WeakBlueprint]()
				{
					static const FSlateIcon ModuleIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.Tree.Connector");
					static const FSlateIcon RigIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ClassIcon.ControlRigBlueprint"); 
					if(WeakBlueprint.IsValid())
					{
						if(WeakBlueprint->IsControlRigModule())
						{
							return ModuleIcon;
						}
					}
					return RigIcon;
				}),
				EUserInterfaceActionType::Button
			);
		}

		if(CVarControlRigShowTestingToolbar.GetValueOnAnyThread())
		{
			ToolbarBuilder.AddSeparator();

			const FCanExecuteAction OnlyWhenNotRecordingAction = FCanExecuteAction::CreateLambda([this]()
			{
				if(ReplayStrongPtr.IsValid())
				{
					return !ReplayStrongPtr->IsReplaying() && !ReplayStrongPtr->IsRecording() && !RecordReplayTimerHandle.IsValid();
				}
				return true;
			});

			const FCanExecuteAction OnlyWithValidReplayAction = FCanExecuteAction::CreateLambda([this]()
			{
				if(ReplayStrongPtr.IsValid())
				{
					return !ReplayStrongPtr->IsRecording() && !RecordReplayTimerHandle.IsValid();
				}
				return false;
			});

			const FUIAction EmptyOnlyWhenNotRecordingAction(FExecuteAction(), OnlyWhenNotRecordingAction);
			const FUIAction EmptyOnlyWithValidReplayAction(FExecuteAction(), OnlyWithValidReplayAction);

			ToolbarBuilder.AddComboButton(
				EmptyOnlyWhenNotRecordingAction,
				FOnGetContent::CreateSP(SharedRef(), &FControlRigBaseEditor::GenerateReplayAssetModeMenuContent),
				TAttribute<FText>::CreateRaw(this, &FControlRigBaseEditor::GetReplayAssetName),
				TAttribute<FText>::CreateRaw(this, &FControlRigBaseEditor::GetReplayAssetTooltip),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "AutomationTools.TestAutomation"),
				false);

			ToolbarBuilder.AddToolBarButton(FUIAction(
					FExecuteAction::CreateLambda([this]()
					{
						RecordReplay(0.0);
					}),
					OnlyWhenNotRecordingAction
				),
				NAME_None,
				LOCTEXT("ReplayRecordButton", "Record"),
				LOCTEXT("ReplayRecordButton_Tooltip", "Records a replay\nA replay asset will be created if necessary."),
				FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ControlRig.Replay.Record")
				);
			ToolbarBuilder.AddComboButton(
				EmptyOnlyWhenNotRecordingAction,
				FOnGetContent::CreateSP(SharedRef(), &FControlRigBaseEditor::GenerateReplayAssetRecordMenuContent),
				LOCTEXT("ReplayRecordMenu_Label", "Recording Modes"),
				LOCTEXT("ReplayRecordMenu_ToolTip", "Pick between different modes for recording"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile"),
				true);

			ToolbarBuilder.AddToolBarButton(FUIAction(
					FExecuteAction::CreateLambda([this]()
					{
						ToggleReplay();
					}),
					OnlyWithValidReplayAction
				),
				NAME_None,
				TAttribute<FText>::CreateLambda([this]()
				{
					if(ReplayStrongPtr.IsValid())
					{
						switch(ReplayStrongPtr->GetPlaybackMode())
						{
							case EControlRigReplayPlaybackMode::ReplayInputs:
							{
								return UControlRigReplay::ReplayInputsStatus;
							}
							case EControlRigReplayPlaybackMode::GroundTruth:
							{
								return UControlRigReplay::GroundTruthStatus;
							}
							default:
							{
								break;
							}
						}
					}
					return UControlRigReplay::LiveStatus;
				}),
				TAttribute<FText>::CreateLambda([this]()
				{
					if(ReplayStrongPtr.IsValid())
					{
						switch(ReplayStrongPtr->GetPlaybackMode())
						{
							case EControlRigReplayPlaybackMode::ReplayInputs:
							{
								return UControlRigReplay::ReplayInputsStatusTooltip;
							}
							case EControlRigReplayPlaybackMode::GroundTruth:
							{
								return UControlRigReplay::GroundTruthStatusTooltip;
							}
							default:
							{
								break;
							}
						}
					}
					return UControlRigReplay::LiveStatusTooltip;
				}),
				TAttribute<FSlateIcon>::CreateLambda([this]()
				{
					static const FSlateIcon LiveIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ClassIcon.ControlRigBlueprint"); 
					static const FSlateIcon ReplayIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(),"ClassIcon.ControlRigSequence"); 
					if(ReplayStrongPtr.IsValid() && ReplayStrongPtr->IsReplaying())
					{
						return ReplayIcon;
					}
					return LiveIcon;
				}),
				EUserInterfaceActionType::Button
			);

			ToolbarBuilder.AddComboButton(
				EmptyOnlyWithValidReplayAction,
				FOnGetContent::CreateSP(SharedRef(), &FControlRigBaseEditor::GenerateReplayAssetPlaybackMenuContent),
				LOCTEXT("ReplayPlaybackModeMenu_Label", "Playback Modes"),
				LOCTEXT("ReplayPlaybackModeMenu_ToolTip", "Pick between different modes for playback"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile"),
				true);
		}
	}
	
	if(bEndSection)
	{
		ToolbarBuilder.EndSection();
	}
}

TArray<FName> FControlRigBaseEditor::GetDefaultEventQueueImpl() const
{
	return ForwardsSolveEventQueue;
}

void FControlRigBaseEditor::SetEventQueueImpl(TArray<FName> InEventQueue, bool bCompile)
{
	if (GetEventQueue() == InEventQueue)
	{
		return;
	}

	TArray<FRigElementKey> PreviousSelection;
	if (FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if(bCompile)
		{
			if (RigBlueprint->GetRigVMAssetInterface()->GetAutoVMRecompile())
			{
				RigBlueprint->GetRigVMAssetInterface()->RequestAutoVMRecompilation();
			}
			RigBlueprint->GetValidator()->SetControlRig(GetControlRig());
		}
		
		// need to clear selection before remove transient control
		// because active selection will trigger transient control recreation after removal	
		PreviousSelection = GetHierarchyBeingDebugged()->GetSelectedKeys();
		RigBlueprint->GetHierarchyController()->ClearSelection();
		
		// need to copy here since the removal changes the iterator
		if (GetControlRig())
		{
			RigBlueprint->ClearTransientControls();
		}
	}

	SetEventQueueSuper(InEventQueue, bCompile);

	if (UControlRig* ControlRig = GetControlRig())
	{
		if (InEventQueue.Num() > 0)
		{
			if (FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
			{
				RigBlueprint->GetValidator()->SetControlRig(ControlRig);

				if (GetLastEventQueue() == ConstructionEventQueue)
				{
					// This will propagate any user bone transformation done during construction to the preview instance
					ResetAllBoneModification();
				}
			}
		}

		// Reset transforms only for construction and forward solve to not interrupt any animation that might be playing
		if (InEventQueue.Contains(FRigUnit_PrepareForExecution::EventName) ||
			InEventQueue.Contains(FRigUnit_BeginExecution::EventName))
		{
			if(UControlRigEditorSettings::Get()->bResetPoseWhenTogglingEventQueue)
			{
				ControlRig->GetHierarchy()->ResetPoseToInitial(ERigElementType::All);
			}
		}
	}

	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->RecreateControlShapeActors();

		UControlRigEditorSettings* EditorSettings = GetMutableDefault<UControlRigEditorSettings>();
		EditorSettings->bDisplayNulls = IsConstructionModeEnabled();
	}

	if (PreviousSelection.Num() > 0)
	{
		GetHierarchyBeingDebugged()->GetController(true)->SetSelection(PreviousSelection);
		SetDetailViewForRigElements();
	}
}

int32 FControlRigBaseEditor::GetEventQueueComboValueImpl() const
{
	const TArray<FName> EventQueue = GetEventQueue();
	if(EventQueue == ForwardsSolveEventQueue)
	{
		return 0;
	}
	if(EventQueue == ConstructionEventQueue)
	{
		return 1;
	}
	if(EventQueue == BackwardsSolveEventQueue)
	{
		return 2;
	}
	if(EventQueue == BackwardsAndForwardsSolveEventQueue)
	{
		return 3;
	}
	return GetEventQueueComboValueSuper();
}

FText FControlRigBaseEditor::GetEventQueueLabelImpl() const
{
	TArray<FName> EventQueue = GetEventQueue();

	if(EventQueue == ConstructionEventQueue)
	{
		return FRigUnit_PrepareForExecution::StaticStruct()->GetDisplayNameText();
	}
	if(EventQueue == ForwardsSolveEventQueue)
	{
		return FRigUnit_BeginExecution::StaticStruct()->GetDisplayNameText();
	}
	if(EventQueue == BackwardsSolveEventQueue)
	{
		return FRigUnit_InverseExecution::StaticStruct()->GetDisplayNameText();
	}
	if(EventQueue == BackwardsAndForwardsSolveEventQueue)
	{
		return FText::FromString(FString::Printf(TEXT("%s and %s"),
			*FRigUnit_InverseExecution::StaticStruct()->GetDisplayNameText().ToString(),
			*FRigUnit_BeginExecution::StaticStruct()->GetDisplayNameText().ToString()));
	}

	if(EventQueue.Num() == 1)
	{
		FString EventName = EventQueue[0].ToString();
		if(!EventName.EndsWith(TEXT("Event")))
		{
			EventName += TEXT(" Event");
		}
		return FText::FromString(EventName);
	}
	
	return LOCTEXT("CustomEventQueue", "Custom Event Queue");
}

FSlateIcon FControlRigBaseEditor::GetEventQueueIconImpl(const TArray<FName>& InEventQueue) const
{
	if(InEventQueue == ConstructionEventQueue)
	{
		return FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.ConstructionMode");
	}
	if(InEventQueue == ForwardsSolveEventQueue)
	{
		return FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.ForwardsSolveEvent");
	}
	if(InEventQueue == BackwardsSolveEventQueue)
	{
		return FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.BackwardsSolveEvent");
	}
	if(InEventQueue == BackwardsAndForwardsSolveEventQueue)
	{
		return FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.BackwardsAndForwardsSolveEvent");
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
}

void FControlRigBaseEditor::HandleSetObjectBeingDebuggedImpl(UObject* InObject)
{
	HandleSetObjectBeingDebuggedSuper(InObject);
	
	UControlRig* DebuggedControlRig = Cast<UControlRig>(InObject);
	if(UControlRig* PreviouslyDebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
	{
		if(!URigVMHost::IsGarbageOrDestroyed(PreviouslyDebuggedControlRig))
		{
			PreviouslyDebuggedControlRig->GetHierarchy()->OnModified().RemoveAll(this);
			PreviouslyDebuggedControlRig->OnPreForwardsSolve_AnyThread().RemoveAll(this);
			PreviouslyDebuggedControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
			PreviouslyDebuggedControlRig->OnPreConstructionForUI_AnyThread().RemoveAll(this);
			PreviouslyDebuggedControlRig->OnPreConstruction_AnyThread().RemoveAll(this);
			PreviouslyDebuggedControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
			PreviouslyDebuggedControlRig->OnEarlyExit_AnyThread().RemoveAll(this);
			PreviouslyDebuggedControlRig->ControlModified().RemoveAll(this);
		}
	}

	if (FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		RigBlueprint->GetValidator()->SetControlRig(DebuggedControlRig);
	}

	if (DebuggedControlRig)
	{
		const bool bShouldExecute = ShouldExecuteControlRig(DebuggedControlRig);
		FControlRigAssetInterfacePtr Asset = GetControlRigAssetInterface();
		Asset->GetHierarchy()->HierarchyForSelectionPtr = DebuggedControlRig->DynamicHierarchy;

		UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
		if (EditorSkelComp)
		{
			UControlRigLayerInstance* AnimInstance = Cast<UControlRigLayerInstance>(EditorSkelComp->GetAnimInstance());
			if (AnimInstance)
			{
				FControlRigIOSettings IOSettings = FControlRigIOSettings::MakeEnabled();
				IOSettings.bUpdatePose = bShouldExecute;
				IOSettings.bUpdateCurves = bShouldExecute;

				// we might want to move this into another method
				FInputBlendPose Filter;
				AnimInstance->ResetControlRigTracks();
				AnimInstance->AddControlRigTrack(0, DebuggedControlRig);
				AnimInstance->UpdateControlRigTrack(0, 1.0f, IOSettings, bShouldExecute);
				AnimInstance->RecalcRequiredBones();

				// since rig has changed, rebuild draw skeleton
				EditorSkelComp->SetControlRigBeingDebugged(DebuggedControlRig);

				if (FControlRigEditMode* EditMode = GetEditMode())
				{
					EditMode->SetObjects(DebuggedControlRig, EditorSkelComp,nullptr);
				}
			}
			
			// get the bone intial transforms from the preview skeletal mesh
			if(bShouldExecute)
			{
				DebuggedControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(EditorSkelComp);
				if(FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
				{
					// copy the initial transforms back to the blueprint
					// no need to call modify here since this code only modifies the bp if the preview mesh changed
					RigBlueprint->GetHierarchy()->CopyPose(DebuggedControlRig->GetHierarchy(), false, true, false);
				}
			}
		}

		DebuggedControlRig->GetHierarchy()->OnModified().RemoveAll(this);
		DebuggedControlRig->OnPreForwardsSolve_AnyThread().RemoveAll(this);
		DebuggedControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
		DebuggedControlRig->OnPreConstructionForUI_AnyThread().RemoveAll(this);
		DebuggedControlRig->OnPreConstruction_AnyThread().RemoveAll(this);
		DebuggedControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
		DebuggedControlRig->OnEarlyExit_AnyThread().RemoveAll(this);
		DebuggedControlRig->ControlModified().RemoveAll(this);

		DebuggedControlRig->GetHierarchy()->OnModified().AddSP(SharedRef(), &FControlRigBaseEditor::OnHierarchyModified_AnyThread);
		DebuggedControlRig->OnPreForwardsSolve_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPreForwardsSolve_AnyThread);
		DebuggedControlRig->OnPostForwardsSolve_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPostForwardsSolve_AnyThread);
		DebuggedControlRig->OnPreConstructionForUI_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPreConstructionForUI_AnyThread);
		DebuggedControlRig->OnPreConstruction_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPreConstruction_AnyThread);
		DebuggedControlRig->OnPostConstruction_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPostConstruction_AnyThread);
		DebuggedControlRig->OnEarlyExit_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnEarlyExit_AnyThread);
		DebuggedControlRig->ControlModified().AddSP(SharedRef(), &FControlRigBaseEditor::HandleOnControlModified);
		
		LastHierarchyHash = INDEX_NONE;

		if(EditorSkelComp)
		{
			EditorSkelComp->SetComponentToWorld(FTransform::Identity);
		}

		if(!bShouldExecute)
		{
			if (FControlRigEditMode* EditMode = GetEditMode())
			{
				EditMode->RequestToRecreateControlShapeActors();
			}
		}
	}
	else
	{
		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			EditMode->SetObjects(nullptr,  nullptr,nullptr);
		}
	}
}

void FControlRigBaseEditor::SetDetailViewForRigElements()
{
	URigHierarchy* HierarchyBeingDebugged = GetHierarchyBeingDebugged();
	SetDetailViewForRigElements(HierarchyBeingDebugged->GetSelectedHierarchyKeys());
}

void FControlRigBaseEditor::SetDetailViewForRigElements(const TArray<FRigHierarchyKey>& InKeys)
{
	if(IsDetailsPanelRefreshSuspended())
	{
		return;
	}

	TArray<FRigHierarchyKey> Keys = InKeys;
	if(Keys.IsEmpty())
	{
		TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjects();
		for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
		{
			if (SelectedObject.IsValid())
			{
				if(const URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
				{
					if(const UScriptStruct* WrappedStruct = WrapperObject->GetWrappedStruct())
					{
						if (WrappedStruct->IsChildOf(FRigBaseElement::StaticStruct()))
						{
							Keys.Add(WrapperObject->GetContent<FRigBaseElement>().Key);
						}
						if (WrappedStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
						{
							Keys.Add(WrapperObject->GetContent<FRigBaseComponent>().Key);
						}
					}
				}
			}
		}
	}

	ClearDetailObject();

	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	URigHierarchy* HierarchyBeingDebugged = GetHierarchyBeingDebugged();
	TArray<UObject*> Objects;

	for(const FRigHierarchyKey& Key : Keys)
	{
		if(Key.IsElement())
		{
			FRigBaseElement* Element = HierarchyBeingDebugged->Find(Key.GetElement());
			if (Element == nullptr)
			{
				continue;
			}

			URigVMDetailsViewWrapperObject* WrapperObject = URigVMDetailsViewWrapperObject::MakeInstance(GetDetailWrapperClass(), GetRigVMAssetInterface()->GetObject(), Element->GetScriptStruct(), (uint8*)Element, HierarchyBeingDebugged);
			WrapperObject->GetWrappedPropertyChangedChainEvent().AddSP(SharedRef(), &FControlRigBaseEditor::OnWrappedPropertyChangedChainEvent);
			WrapperObject->AddToRoot();

			Objects.Add(WrapperObject);
		}
		if(Key.IsComponent())
		{
			FRigBaseComponent* Component = HierarchyBeingDebugged->FindComponent(Key.GetComponent());
			if (Component == nullptr)
			{
				continue;
			}

			URigVMDetailsViewWrapperObject* WrapperObject = URigVMDetailsViewWrapperObject::MakeInstance(GetDetailWrapperClass(), GetRigVMAssetInterface()->GetObject(), Component->GetScriptStruct(), (uint8*)Component, HierarchyBeingDebugged);
			WrapperObject->GetWrappedPropertyChangedChainEvent().AddSP(SharedRef(), &FControlRigBaseEditor::OnWrappedPropertyChangedChainEvent);
			WrapperObject->AddToRoot();

			Objects.Add(WrapperObject);
		}
	}
	
	SetDetailObjects(Objects);
}

void FControlRigBaseEditor::SetDetailObjectsImpl(const TArray<UObject*>& InObjects)
{
	// if no modules should be selected - we need to deselect all modules
	if (!InObjects.ContainsByPredicate([](const UObject* InObject) -> bool
	{
		return IsValid(InObject) && InObject->IsA<UControlRig>();
	}))
	{
		ModulesSelected.Reset();
	}
	
	SetDetailObjectsSuper(InObjects);
}

void FControlRigBaseEditor::RefreshDetailViewImpl()
{
	if(DetailViewShowsAnyRigElement())
	{
		SetDetailViewForRigElements();
		return;
	}
	else if(!ModulesSelected.IsEmpty())
	{
		SetDetailViewForRigModules();
		return;
	}

	RefreshDetailViewSuper();
}

bool FControlRigBaseEditor::DetailViewShowsAnyRigElement() const
{
	return DetailViewShowsStruct(FRigBaseElement::StaticStruct()) || DetailViewShowsStruct(FRigBaseComponent::StaticStruct());
}

bool FControlRigBaseEditor::DetailViewShowsRigElement(FRigHierarchyKey InKey) const
{
	TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjectsFromDetailView();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				if (const UScriptStruct* WrappedStruct = WrapperObject->GetWrappedStruct())
				{
					if (WrappedStruct->IsChildOf(FRigBaseElement::StaticStruct()))
					{
						if(WrapperObject->GetContent<FRigBaseElement>().GetKey() == InKey)
						{
							return true;
						}
					}
					if (WrappedStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
					{
						if(WrapperObject->GetContent<FRigBaseComponent>().GetKey() == InKey)
						{
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

TArray<FRigHierarchyKey> FControlRigBaseEditor::GetSelectedRigElementsFromDetailView() const
{
	TArray<FRigHierarchyKey> Keys;
	
	TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjectsFromDetailView();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				if (const UScriptStruct* WrappedStruct = WrapperObject->GetWrappedStruct())
				{
					if (WrappedStruct->IsChildOf(FRigBaseElement::StaticStruct()))
					{
						Keys.Add(WrapperObject->GetContent<FRigBaseElement>().GetKey());
					}
					if (WrappedStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
					{
						Keys.Add(WrapperObject->GetContent<FRigBaseComponent>().GetKey());
					}
				}
			}
		}
	}

	return Keys;
}

TArray<TWeakObjectPtr<UObject>> FControlRigBaseEditor::GetSelectedObjectsFromDetailView() const
{
	TSharedPtr<class SWidget> Inspector = GetInspector();
	if (Inspector.IsValid())
	{
#if WITH_RIGVMLEGACYEDITOR
		if (IsControlRigLegacyEditor())
		{
			TSharedPtr<class SKismetInspector> KismetInspector = StaticCastSharedPtr<SKismetInspector>(Inspector);
			return KismetInspector->GetSelectedObjects();
		}
		
#endif
		TSharedPtr<class SRigVMDetailsInspector> RigVMInspector = StaticCastSharedPtr<SRigVMDetailsInspector>(Inspector);
		return RigVMInspector->GetSelectedObjects();
	}
	return TArray<TWeakObjectPtr<UObject>>();
}

void FControlRigBaseEditor::SetDetailViewForRigModules()
{
	SetDetailViewForRigModules(ModulesSelected);
}

void FControlRigBaseEditor::SetDetailViewForRigModules(const TArray<FName> InModuleNames)
{
	if(IsDetailsPanelRefreshSuspended())
	{
		return;
	}

	ClearDetailObject();

	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	UModularRig* RigBeingDebugged = Cast<UModularRig>(RigBlueprint->GetDebuggedControlRig());

	if (!RigBeingDebugged)
	{
		return;
	}

	ModulesSelected = InModuleNames;
	TArray<UObject*> Objects;

	for(const FName& ModuleName : InModuleNames)
	{
		const FRigModuleInstance* Element = RigBeingDebugged->FindModule(ModuleName);
		if (Element == nullptr)
		{
			continue;
		}

		if(UControlRig* ModuleInstance = Element->GetRig())
		{
			Objects.Add(ModuleInstance);
		}
	}

	if(!Objects.IsEmpty() && CVarControlRigEnableOverrides.GetValueOnAnyThread())
	{
		TSharedPtr<FOverrideStatusDetailsViewObjectFilter> ObjectFilter = FOverrideStatusDetailsViewObjectFilter::Create();

		ObjectFilter->OnCanMergeObjects().BindLambda([](const UObject* InObjectA, const UObject* InObjectB) -> bool
		{
			if(InObjectA && InObjectB)
			{
				return InObjectA->IsA<UControlRig>() && InObjectB->IsA<UControlRig>(); 
			}
			return false;
		});

		ObjectFilter->OnCanCreateWidget().BindLambda([](const FOverrideStatusSubject& InSubject) -> bool
		{
			static const TSet<FName> CategoriesToIgnore = {
				TEXT("General"),
				TEXT("Connections"),
			};
			return InSubject.Contains<UControlRig>() && !CategoriesToIgnore.Contains(InSubject.GetCategory());
		});

		ObjectFilter->OnGetStatus().BindLambda([RigBlueprint](const FOverrideStatusSubject& InSubject)
		{
			const FString PropertyPath = InSubject.GetPropertyPathString();
			return InSubject.GetStatus<UControlRig>(
				[PropertyPath, RigBlueprint](const FOverrideStatusObjectHandle<UControlRig>& InModuleRig) -> TOptional<EOverrideWidgetStatus::Type>
				{
					if(const FRigModuleReference* ModuleReference = RigBlueprint->GetModularRigModel().FindModule(InModuleRig->GetFName()))
					{
						if(PropertyPath.IsEmpty())
						{
							if(!ModuleReference->ConfigOverrides.IsEmpty())
							{
								return EOverrideWidgetStatus::ChangedInside; 
							}
						}
						else
						{
							if(ModuleReference->ConfigOverrides.Contains(PropertyPath, ModuleReference->Name))
							{
								return EOverrideWidgetStatus::ChangedHere;
							}

							if(ModuleReference->ConfigOverrides.ContainsChildPathOf(PropertyPath, ModuleReference->Name))
							{
								return EOverrideWidgetStatus::ChangedInside;
							}
							if(ModuleReference->ConfigOverrides.ContainsParentPathOf(PropertyPath, ModuleReference->Name))
							{
								return EOverrideWidgetStatus::ChangedOutside;
							}
						}
						return EOverrideWidgetStatus::None;
					}
					return {};
				}
			).Get(EOverrideWidgetStatus::Mixed);
		});

		ObjectFilter->OnAddOverride().BindLambda([RigBlueprint](const FOverrideStatusSubject& InSubject)
			{
				FScopedTransaction Transaction(LOCTEXT("AddOverride", "Add Override"));
				RigBlueprint->Modify();

				InSubject.ForEach<UControlRig>(
					[RigBlueprint, InSubject](const FOverrideStatusObjectHandle<UControlRig>& InModuleRig)
					{
						if(InSubject.HasPropertyPath())
						{
							const FControlRigOverrideValue Value(InSubject.GetPropertyPathString(), InModuleRig->GetVariablesStruct(), const_cast<UControlRig*>(InModuleRig.GetObject())->GetVariablesMemory());
							if(Value.IsValid())
							{
								RigBlueprint->GetModularRigController()->SetConfigValueInModule(InModuleRig->GetFName(), Value);
							}
						}
						else
						{
							const TArray<FRigVMExternalVariable> Variables = InModuleRig->GetPublicVariables();
							for(const FRigVMExternalVariable& Variable : Variables)
							{
								const FString PropertyPath = Variable.GetName().ToString();
								const FControlRigOverrideValue Value(PropertyPath, InModuleRig->GetVariablesStruct(), const_cast<UControlRig*>(InModuleRig.GetObject())->GetVariablesMemory());
								RigBlueprint->GetModularRigController()->SetConfigValueInModule(InModuleRig->GetFName(), Value);
							}
						}
					}
				);

				return FReply::Handled();
			});

		ObjectFilter->OnClearOverride().BindLambda(
			[this, RigBlueprint](const FOverrideStatusSubject& InSubject)
			{
				FScopedTransaction Transaction(LOCTEXT("ClearOverride", "Clear Override"));
				RigBlueprint->Modify();

				InSubject.ForEach<UControlRig>(
					[RigBlueprint, InSubject](const FOverrideStatusObjectHandle<UControlRig>& InModuleRig)
					{
						if(InSubject.HasPropertyPath())
						{
							RigBlueprint->GetModularRigController()->ResetConfigValueInModule(
								InModuleRig->GetFName(), InSubject.GetPropertyPathString(), true);
						}
						else
						{
							if(const FRigModuleReference* ModuleReference = RigBlueprint->GetModularRigModel().FindModule(InModuleRig->GetFName()))
							{
								TArray<FString> PathsToClear;
								for(const FControlRigOverrideValue& Override : ModuleReference->ConfigOverrides)
								{
									if(Override.IsValid())
									{
										PathsToClear.Add(Override.GetPath());
									}
								}
								for(const FString& Path : PathsToClear)
								{
									RigBlueprint->GetModularRigController()->ResetConfigValueInModule(ModuleReference->GetFName(), Path, true);
								}
							}
						}
					}
				);

				return FReply::Handled();
			});

		ObjectFilter->OnResetToDefault().BindLambda(
			[this, RigBlueprint](const FOverrideStatusSubject& InSubject)
			{
				FScopedTransaction Transaction(LOCTEXT("ResetConfigValue", "Reset Config Value"));
				RigBlueprint->Modify();

				InSubject.ForEach<UControlRig>(
					[RigBlueprint, InSubject](const FOverrideStatusObjectHandle<UControlRig>& InModuleRig)
					{
						if(InSubject.HasPropertyPath())
						{
							RigBlueprint->GetModularRigController()->ResetConfigValueInModule(
								InModuleRig->GetFName(), InSubject.GetPropertyPathString(), false);
						}
						else
						{
							if(const FRigModuleReference* ModuleReference = RigBlueprint->GetModularRigModel().FindModule(InModuleRig->GetFName()))
							{
								TArray<FString> PathsToClear;
								for(const FControlRigOverrideValue& Override : ModuleReference->ConfigOverrides)
								{
									if(Override.IsValid())
									{
										PathsToClear.Add(Override.GetPath());
									}
								}
								for(const FString& Path : PathsToClear)
								{
									RigBlueprint->GetModularRigController()->ResetConfigValueInModule(ModuleReference->GetFName(), Path, false);
								}
							}
						}
					}
				);

				return FReply::Handled();
			});

		ObjectFilter->OnValueDiffersFromDefault().BindLambda(
			[this, RigBlueprint](const FOverrideStatusSubject& InSubject)
			{
				return InSubject.GetCommonValue<bool, UControlRig>(
					[RigBlueprint, InSubject](const FOverrideStatusObjectHandle<UControlRig>& InModuleRig)
					{
						if(const FRigModuleReference* ModuleReference = RigBlueprint->GetModularRigModel().FindModule(InModuleRig->GetFName()))
						{
							TArray<TSharedPtr<const FPropertyPath>> PropertyPathsToCheck;
							if(InSubject.HasPropertyPath())
							{
								PropertyPathsToCheck.Add(InSubject.GetPropertyPath());
							}
							else
							{
								const TArray<FRigVMExternalVariable> Variables = InModuleRig->GetPublicVariables();
								for(const FRigVMExternalVariable& Variable : Variables)
								{
									if(FProperty* Property = InModuleRig->GetClass()->FindPropertyByName(Variable.GetName()))
									{
										PropertyPathsToCheck.Add(FPropertyPath::Create(TWeakFieldPtr<FProperty>(Property)));
									}
								}
							}

							FControlRigAssetStrongReference AssetReference = ModuleReference->ControlRigAssetReference.LoadStrongReference();
							uint8* CDO = AssetReference.GetVariablesMemory();
							
							for(const TSharedPtr<const FPropertyPath>& PropertyPath : PropertyPathsToCheck)
							{
								const FString PropertyPathString = PropertyPath->ToString();
								const FString PropertyPathPrefix = PropertyPathString + TEXT("->");
								for(const FControlRigOverrideValue& Override : ModuleReference->ConfigOverrides)
								{
									if(!Override.IdenticalValueInSubject(CDO))
									{
										const FString& ValuePathString = Override.GetPath();
										if(ValuePathString == PropertyPathString || ValuePathString.StartsWith(PropertyPathPrefix))
										{
											return true;
										}
									}
								}
							}
						}
						return false;
					}
				).Get(false);
			});

		SetDetailObjectFilter(ObjectFilter);
	}

	SetDetailObjects(Objects);

	// In case the modules selected are still not available, lets set them again
	if (Objects.IsEmpty())
	{
		ModulesSelected = InModuleNames;
	}
}

bool FControlRigBaseEditor::DetailViewShowsAnyRigModule() const
{
	return DetailViewShowsStruct(FRigModuleInstance::StaticStruct());
}

bool FControlRigBaseEditor::DetailViewShowsRigModule(FName InModuleName) const
{
	TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjectsFromDetailView();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				if (const UScriptStruct* WrappedStruct = WrapperObject->GetWrappedStruct())
				{
					if (WrappedStruct->IsChildOf(FRigModuleInstance::StaticStruct()))
					{
						if(WrapperObject->GetContent<FRigModuleInstance>().Name == InModuleName)
						{
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

void FControlRigBaseEditor::CompileBaseImpl()
{	
	// Compiling can change the event queue. Restore the editor state after compiling.
	struct FScopedRestoreEventQueue : FNoncopyable
	{
		FScopedRestoreEventQueue(FControlRigBaseEditor& InEditor)
			: Editor(InEditor)
			, RestoreEventQueue(Editor.GetEventQueue())
		{
		}
		~FScopedRestoreEventQueue()
		{
			constexpr bool bCompile = false;
			Editor.SetEventQueue(RestoreEventQueue, bCompile);
		}
	private:
		FControlRigBaseEditor& Editor;
		const TArray<FName> RestoreEventQueue;
	};
	
	TUniquePtr<const FScopedRestoreEventQueue> ScopedRestoreEventQueue;
	if (!UControlRigEditorSettings::Get()->bCompileResetsEventQueue)
	{
		ScopedRestoreEventQueue = MakeUnique<FScopedRestoreEventQueue>(*this);
	}

	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

		TUniquePtr<IControlRigEditorAssetInterface::FControlValueScope> ValueScope;
		if (!UControlRigEditorSettings::Get()->bResetControlsOnCompile) // if we need to retain the controls
		{
			ValueScope = MakeUnique<IControlRigEditorAssetInterface::FControlValueScope>(GetControlRigAssetInterface());
		}

		FControlRigAssetInterfacePtr ControlRigBlueprint = GetControlRigAssetInterface();
		if (ControlRigBlueprint == nullptr)
		{
			return;
		}

		const TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjectsFromDetailView();
		const TArray<FRigHierarchyKey> SelectedHierarchyKeysInDetailsView = GetSelectedRigElementsFromDetailView();
		const TArray<FRigHierarchyKey> SelectedHierarchyKeysInHierarchy = ControlRigBlueprint->GetHierarchy()->GetSelectedHierarchyKeys();

		if(IsConstructionModeEnabled())
		{
			SetEventQueue(ForwardsSolveEventQueue, false);
		}

		// clear transient controls such that we don't leave
		// a phantom shape in the viewport
		// have to do this before compile() because during compile
		// a new control rig instance is created without the transient controls
		// so clear is never called for old transient controls
		ControlRigBlueprint->ClearTransientControls();

		// default to always reset all bone modifications 
		ResetAllBoneModification();

		// remove all cached transforms from modified constrols
		if (FControlRigEditorEditMode* EditoMode = GetEditMode())
		{
			EditoMode->ModifiedRigElements.Reset();
		}

		{
			CompileSuper();
		}

		ControlRigBlueprint->RecompileModularRig();

		// ensure the skeletal mesh is still bound
		UControlRigSkeletalMeshComponent* SkelMeshComponent = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
		if (SkelMeshComponent)
		{
			bool bWasCreated = false;
			FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UControlRigLayerInstance>(SkelMeshComponent, bWasCreated);
			if (bWasCreated)
			{
				OnAnimInitialized();
			}
		}

		if (UControlRigEditorSettings::Get()->bResetControlTransformsOnCompile)
		{
			ControlRigBlueprint->GetHierarchy()->ForEach<FRigControlElement>([ControlRigBlueprint](FRigControlElement* ControlElement) -> bool
            {
				const FTransform Transform = ControlRigBlueprint->GetHierarchy()->GetInitialLocalTransform(ControlElement->GetIndex());

				/*/
				if (UControlRig* ControlRig = GetControlRig())
				{
					ControlRig->Modify();
					ControlRig->GetControlHierarchy().SetLocalTransform(Control.Index, Transform);
					ControlRig->ControlModified().Broadcast(ControlRig, Control, EControlRigSetKey::DoNotCare);
				}
				*/

				ControlRigBlueprint->GetHierarchy()->SetLocalTransform(ControlElement->GetIndex(), Transform);
				return true;
			});
		}

		ControlRigBlueprint->PropagatePoseFromBPToInstances();

		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			EditMode->RecreateControlShapeActors();
		}
		
		if (!SelectedHierarchyKeysInDetailsView.IsEmpty())
		{
			SetDetailViewForRigElements(SelectedHierarchyKeysInDetailsView);
		}
		else if (!SelectedObjects.IsEmpty())
		{
			RefreshDetailView();
		}
		if(!SelectedHierarchyKeysInHierarchy.IsEmpty())
		{
			ControlRigBlueprint->GetHierarchyController()->SetHierarchySelection(SelectedHierarchyKeysInHierarchy);
		}
	}
}

void FControlRigBaseEditor::HandleModifiedEventImpl(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	HandleModifiedEventSuper(InNotifType, InGraph, InSubject);

	if(InNotifType == ERigVMGraphNotifType::NodeSelected)
	{
		if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InSubject))
		{
			SetDirectionManipulationSubject(UnitNode);
		}
	}
	else if(InNotifType == ERigVMGraphNotifType::NodeSelectionChanged)
	{
		bool bNeedsToClearManipulationSubject = true;
		const TArray<FName> SelectedNodes = InGraph->GetSelectNodes();
		if(SelectedNodes.Num() == 1)
		{
			if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InGraph->FindNodeByName(SelectedNodes[0])))
			{
				SetDirectionManipulationSubject(UnitNode);
				bNeedsToClearManipulationSubject = false;
			}
		}

		if(bNeedsToClearManipulationSubject)
		{
			ClearDirectManipulationSubject();
		}
	}
	else if(InNotifType == ERigVMGraphNotifType::PinDefaultValueChanged)
	{
		if(const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
		{
			if(Pin->GetNode() == DirectManipulationSubject.Get())
			{
				bRefreshDirectionManipulationTargetsRequired = true;
			}
		}
	}
	else if(InNotifType == ERigVMGraphNotifType::LinkAdded ||
		InNotifType == ERigVMGraphNotifType::LinkRemoved)
	{
		if(const URigVMLink* Link = Cast<URigVMLink>(InSubject))
		{
			if((Link->GetSourceNode() == DirectManipulationSubject.Get()) ||
				(Link->GetTargetNode() == DirectManipulationSubject.Get()))
			{
				bRefreshDirectionManipulationTargetsRequired = true;
			}
		}
	}
}

void FControlRigBaseEditor::OnCreateGraphEditorCommandsImpl(TSharedPtr<FUICommandList> GraphEditorCommandsList)
{
	OnCreateGraphEditorCommandsSuper(GraphEditorCommandsList);

	TSharedRef<FControlRigBaseEditor> EditorRef = SharedRef();

	GraphEditorCommandsList->MapAction(
		FControlRigEditorCommands::Get().RequestDirectManipulationPosition,
		FExecuteAction::CreateSP(EditorRef, &FControlRigBaseEditor::HandleRequestDirectManipulationPosition));
	GraphEditorCommandsList->MapAction(
		FControlRigEditorCommands::Get().RequestDirectManipulationRotation,
		FExecuteAction::CreateSP(EditorRef, &FControlRigBaseEditor::HandleRequestDirectManipulationRotation));
	GraphEditorCommandsList->MapAction(
		FControlRigEditorCommands::Get().RequestDirectManipulationScale,
		FExecuteAction::CreateSP(EditorRef, &FControlRigBaseEditor::HandleRequestDirectManipulationScale));
}

void FControlRigBaseEditor::HandleVMCompiledEventImpl(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext)
{
	HandleVMCompiledEventSuper(InCompiledObject, InVM, InContext);

	if(bRefreshDirectionManipulationTargetsRequired)
	{
		RefreshDirectManipulationTextList();
		bRefreshDirectionManipulationTargetsRequired = false;
	}

	if(FControlRigAssetInterfacePtr ControlRigBlueprint = GetControlRigAssetInterface())
	{
		if(UControlRig* ControlRig = InVM->GetTypedOuter<UControlRig>())
		{
			ControlRigBlueprint->UpdateElementKeyRedirector(ControlRig);
		}
		else
		{
			ControlRigBlueprint->UpdateElementKeyRedirector(nullptr);
		}
	}
}

void FControlRigBaseEditor::SaveAsset_ExecuteImpl()
{
	SaveAsset_ExecuteSuper();

	// Save the new state of the hierarchy in the default object, so that it has the correct values on load
	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	if(const UControlRig* ControlRig = GetControlRig())
	{
		if (RigBlueprint->GetRigVMAssetInterface()->IsBlueprintAsset())
		{
			UControlRig* CDO = ControlRig->GetClass()->GetDefaultObject<UControlRig>();
			CDO->DynamicHierarchy->CopyHierarchy(RigBlueprint->GetHierarchy());
			RigBlueprint->UpdateElementKeyRedirector(CDO);
		}
	}

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.ClearAssetActions(UControlRigEditorAssetInterface::StaticClass());
	ActionDatabase.RefreshClassActions(UControlRigEditorAssetInterface::StaticClass());
}

void FControlRigBaseEditor::SaveAssetAs_ExecuteImpl()
{
	SaveAssetAs_ExecuteSuper();

	// Save the new state of the hierarchy in the default object, so that it has the correct values on load
	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	if(const UControlRig* ControlRig = GetControlRig())
	{
		UControlRig* CDO = ControlRig->GetClass()->GetDefaultObject<UControlRig>();
		CDO->DynamicHierarchy->CopyHierarchy(RigBlueprint->GetHierarchy());
		RigBlueprint->UpdateElementKeyRedirector(CDO);
	}

	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.ClearAssetActions(UControlRigEditorAssetInterface::StaticClass());
	ActionDatabase.RefreshClassActions(UControlRigEditorAssetInterface::StaticClass());
}

bool FControlRigBaseEditor::IsModularRig() const
{
	if(FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		return RigBlueprint->IsModularRig();
	}
	return false;
}

bool FControlRigBaseEditor::IsRigModule() const
{
	if(FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		return RigBlueprint->IsControlRigModule();
	}
	return false;
}

FName FControlRigBaseEditor::GetToolkitFNameImpl() const
{
	return FName("ControlRigEditor");
}

FText FControlRigBaseEditor::GetBaseToolkitNameImpl() const
{
	return LOCTEXT("AppLabel", "Control Rig Editor");
}

FString FControlRigBaseEditor::GetWorldCentricTabPrefixImpl() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Control Rig Editor ").ToString();
}

FReply FControlRigBaseEditor::OnSpawnGraphNodeByShortcutImpl(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph)
{
	const FReply SuperReply = OnSpawnGraphNodeByShortcutSuper(InChord, InPosition, InGraph);
	if(SuperReply.IsEventHandled())
	{
		return SuperReply;
	}

	if(!InChord.HasAnyModifierKeys())
	{
		if(UControlRigGraph* RigGraph = Cast<UControlRigGraph>(InGraph))
		{
			if(URigVMController* Controller = RigGraph->GetController())
			{
				FDeprecateSlateVector2D Position = InPosition;
				if(InChord.Key == EKeys::S)
				{
					Controller->AddUnitNode(FRigVMFunction_Sequence::StaticStruct(), FRigUnit::GetMethodName(), Position, FString(), true, true);
				}
				else if(InChord.Key == EKeys::One)
				{
					Controller->AddUnitNode(FRigUnit_GetTransform::StaticStruct(), FRigUnit::GetMethodName(), Position, FString(), true, true);
				}
				else if(InChord.Key == EKeys::Two)
				{
					Controller->AddUnitNode(FRigUnit_SetTransform::StaticStruct(), FRigUnit::GetMethodName(), Position, FString(), true, true);
				}
				else if(InChord.Key == EKeys::Three)
				{
					Controller->AddUnitNode(FRigUnit_ParentConstraint::StaticStruct(), FRigUnit::GetMethodName(), Position, FString(), true, true);
				}
				else if(InChord.Key == EKeys::Four)
				{
					Controller->AddUnitNode(FRigUnit_GetControlFloat::StaticStruct(), FRigUnit::GetMethodName(), Position, FString(), true, true);
				}
				else if(InChord.Key == EKeys::Five)
				{
					Controller->AddUnitNode(FRigUnit_SetCurveValue::StaticStruct(), FRigUnit::GetMethodName(), Position, FString(), true, true);
				}
			}
		}
	}

	return FReply::Unhandled();
}

void FControlRigBaseEditor::PostTransactionImpl(bool bSuccess, const FTransaction* Transaction, bool bIsRedo)
{
	if(Transaction == nullptr)
	{
		return;
	}

	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	if(RigBlueprint == nullptr)
	{
		return;
	}
	
    const UPackage* Package = RigBlueprint.GetObject()->GetPackage();

    TArray<UObject*> TransactedObjects;
    Transaction->GetTransactionObjects(TransactedObjects);

	bool bAffectsThisAsset = false;
	for(const UObject* Object : TransactedObjects)
	{
		if (Object == RigBlueprint->GetObjectBeingDebugged())
		{
			bAffectsThisAsset = true;
			break;
		}
		if (Object->GetPackage() == Package)
		{
			bAffectsThisAsset = true;
			break;
		}
	}

	if(!bAffectsThisAsset)
	{
		return;
	}

	EnsureValidRigElementsInDetailPanel();

	// Do not compile here. ControlRigBlueprint::PostTransacted decides when it is necessary to compile depending
	// on the properties that are affected.
	//Compile();

	UpdateRigVMHost();

	// SetPreviewMesh sometimes destroys the editor. Lets make sure we still have a valid editor.
	TSharedPtr<FControlRigBaseEditor> SharedPointer = SharedRef().ToSharedPtr();
	USkeletalMesh* PreviewMesh = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMesh();
	if (PreviewMesh != RigBlueprint->GetPreviewMesh())
	{
		RigBlueprint->SetPreviewMesh(PreviewMesh);
		GetPersonaToolkit()->SetPreviewMesh(PreviewMesh, true);
	}

	if (SharedPointer.IsValid())
	{
		if (UControlRig* DebuggedControlRig = Cast<UControlRig>(RigBlueprint->GetObjectBeingDebugged()))
		{
			if(URigHierarchy* Hierarchy = DebuggedControlRig->GetHierarchy())
			{
				if(Hierarchy->Num() == 0)
				{
					OnHierarchyChanged();
				}
			}
		}

		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			EditMode->RequestToRecreateControlShapeActors();
		}
	}
}

void FControlRigBaseEditor::EnsureValidRigElementsInDetailPanel()
{
	FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface();
	URigHierarchy* Hierarchy = ControlRigBP->GetHierarchy(); 

	TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjectsFromDetailView();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				if(const UScriptStruct* WrappedStruct = WrapperObject->GetWrappedStruct())
				{
					if (WrappedStruct->IsChildOf(FRigBaseElement::StaticStruct()))
					{
						FRigElementKey Key = WrapperObject->GetContent<FRigBaseElement>().GetKey();
						if(!Hierarchy->Contains(Key))
						{
							ClearDetailObject();
						}
					}
				}
			}
		}
	}
}

void FControlRigBaseEditor::OnAnimInitialized()
{
	UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
	if (EditorSkelComp)
	{
		EditorSkelComp->bRequiredBonesUpToDateDuringTick = 0;

		UControlRigLayerInstance* AnimInstance = Cast<UControlRigLayerInstance>(EditorSkelComp->GetAnimInstance());
		if (AnimInstance && GetControlRig())
		{
			// update control rig data to anim instance since animation system has been reinitialized
			FInputBlendPose Filter;
			AnimInstance->ResetControlRigTracks();
			AnimInstance->AddControlRigTrack(0, GetControlRig());
			AnimInstance->UpdateControlRigTrack(0, 1.0f, FControlRigIOSettings::MakeEnabled(), bExecutionControlRig);
		}
	}
}

void FControlRigBaseEditor::HandleVMExecutedEventImpl(URigVMHost* InHost, const FName& InEventName)
{
	HandleVMExecutedEventSuper(InHost, InEventName);

	if (FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface())
	{
		URigHierarchy* Hierarchy = GetHierarchyBeingDebugged(); 

		TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjectsFromDetailView();
		for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
		{
			if (SelectedObject.IsValid())
			{
				if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
				{
					if (const UScriptStruct* Struct = WrapperObject->GetWrappedStruct())
					{
						if(Struct->IsChildOf(FRigBaseElement::StaticStruct()))
						{
							const FRigElementKey Key = WrapperObject->GetContent<FRigBaseElement>().GetKey();

							FRigBaseElement* Element = Hierarchy->Find(Key);
							if(Element == nullptr)
							{
								ClearDetailObject();
								break;
							}

							if(FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
							{
								// compute all transforms
								Hierarchy->GetTransform(ControlElement, ERigTransformType::CurrentGlobal);
								Hierarchy->GetTransform(ControlElement, ERigTransformType::CurrentLocal);
								Hierarchy->GetTransform(ControlElement, ERigTransformType::InitialGlobal);
								Hierarchy->GetTransform(ControlElement, ERigTransformType::InitialLocal);
								Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentGlobal);
								Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentLocal);
								Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialGlobal);
								Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::InitialLocal);
								Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentGlobal);
								Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::CurrentLocal);
								Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialGlobal);
								Hierarchy->GetControlShapeTransform(ControlElement, ERigTransformType::InitialLocal);

								WrapperObject->SetContent<FRigControlElement>(*ControlElement);
							}
							else if(FRigTransformElement* TransformElement = Cast<FRigTransformElement>(Element))
							{
								// compute all transforms
								Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentGlobal);
								Hierarchy->GetTransform(TransformElement, ERigTransformType::CurrentLocal);
								Hierarchy->GetTransform(TransformElement, ERigTransformType::InitialGlobal);
								Hierarchy->GetTransform(TransformElement, ERigTransformType::InitialLocal);

								WrapperObject->SetContent<FRigTransformElement>(*TransformElement);
							}
							else
							{
								WrapperObject->SetContent<FRigBaseElement>(*Element);
							}
						}
					}
				}
			}
		}

		// update transient controls on nodes / pins
		if(UControlRig* DebuggedControlRig = Cast<UControlRig>(ControlRigBP->GetObjectBeingDebugged()))
		{
			if(!DebuggedControlRig->RigUnitManipulationInfos.IsEmpty())
			{
				const FRigHierarchyRedirectorGuard RedirectorGuard(DebuggedControlRig);
				FControlRigExecuteContext& ExecuteContext = DebuggedControlRig->GetRigVMExtendedExecuteContext().GetPublicDataSafe<FControlRigExecuteContext>();
				
				for(const TSharedPtr<FRigDirectManipulationInfo>& ManipulationInfo : DebuggedControlRig->RigUnitManipulationInfos)
				{
					if(const URigVMUnitNode* Node = ManipulationInfo->Node.Get())
					{
						const UScriptStruct* ScriptStruct = Node->GetScriptStruct();
						if(ScriptStruct == nullptr)
						{
							continue;
						}

						TSharedPtr<FStructOnScope> NodeInstance = Node->ConstructLiveStructInstance(DebuggedControlRig);
						if(!NodeInstance.IsValid() || !NodeInstance->IsValid())
						{
							continue;
						}

						// if we are not manipulating right now - reset the info so that it can follow the hierarchy
						if (FControlRigEditorEditMode* EditMode = GetEditMode())
						{
							if(!EditMode->bIsTracking)
							{
								ManipulationInfo->Reset();
							}
						}
				
						FRigUnit* UnitInstance = UControlRig::GetRigUnitInstanceFromScope(NodeInstance);
						UnitInstance->UpdateHierarchyForDirectManipulation(Node, NodeInstance, ExecuteContext, ManipulationInfo);
						ManipulationInfo->bInitialized = true;
						UnitInstance->PerformDebugDrawingForDirectManipulation(Node, NodeInstance, ExecuteContext, ManipulationInfo);
					}
				}
			}
		}		
	}
}

void FControlRigBaseEditor::CreateEditorModeManagerImpl()
{
	SetEditorModeManager(MakeShareable(FModuleManager::LoadModuleChecked<FPersonaModule>("Persona").CreatePersonaEditorModeManager()));
}

void FControlRigBaseEditor::TickImpl(float DeltaTime)
{
	TickSuper(DeltaTime);

	bool bDrawHierarchyBones = false;

	// tick the control rig in case we don't have skeletal mesh
	if (FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
	{
		UControlRig* ControlRig = GetControlRig();
		if (Blueprint->GetPreviewMesh() == nullptr && 
			ControlRig != nullptr && 
			bExecutionControlRig)
		{
			{
				// prevent transient controls from getting reset
				UControlRig::FTransientControlPoseScope	PoseScope(ControlRig);
				// reset transforms here to prevent additive transforms from accumulating to INF
				ControlRig->GetHierarchy()->ResetPoseToInitial(ERigElementType::Bone);
			}

			if (PreviewInstance)
			{
				// since we don't have a preview mesh the anim instance cannot deal with the modify bone
				// functionality. we need to perform this manually to ensure the pose is kept.
				const TArray<FAnimNode_ModifyBone>& BoneControllers = PreviewInstance->GetBoneControllers();
				for(const FAnimNode_ModifyBone& ModifyBone : BoneControllers)
				{
					const FRigElementKey BoneKey(ModifyBone.BoneToModify.BoneName, ERigElementType::Bone);
					const FTransform BoneTransform(ModifyBone.Rotation, ModifyBone.Translation, ModifyBone.Scale);
					ControlRig->GetHierarchy()->SetLocalTransform(BoneKey, BoneTransform);
				}
			}
			
			ControlRig->SetDeltaTime(DeltaTime);
			ControlRig->Evaluate_AnyThread();
			bDrawHierarchyBones = true;
		}
	}

	if (FControlRigEditorEditMode* EditMode = GetEditMode())
	{
		if (bDrawHierarchyBones)
		{
			EditMode->bDrawHierarchyBones = bDrawHierarchyBones;
		}
	}

	// Initial viewport focus for assets opened with no preview mesh. We want to make sure the rig's visible
	// controls are on screen by default; without this the camera can sit in empty space and the user has no
	// idea where the rig content is (Face_ControlBoard_CtrlRig is the canonical example).
	// We do this in the tick (rather than HandleViewportCreated) because controls only exist after the
	// blueprint has compiled. EditMode::FrameItems() handles the deferral if the viewport client isn't ready
	// yet by storing the keys and processing them on the next EditMode::Tick.
	if (bPendingInitialViewportFocus)
	{
		// Safety bail: if the hierarchy never populates (e.g. compile error, modular rig with no resolved
		// connectors) we don't want to keep retrying every tick forever. Give it a small budget.
		static constexpr int32 MaxPendingFocusTicks = 60;
		++PendingInitialViewportFocusTickCount;

		// Skip the first tick: this gives the rig at least one frame to run its construction event and
		// evaluate, so control transforms reflect their resolved positions and not stale defaults. From
		// the second tick onward we attempt the framing each tick until it succeeds or we hit the budget.
		if (PendingInitialViewportFocusTickCount > 1)
		{
			if (FControlRigEditorEditMode* EditMode = GetEditMode())
			{
				if (URigHierarchy* Hierarchy = GetHierarchyBeingDebugged())
				{
					TArray<FRigElementKey> VisibleControlKeys;
					Hierarchy->ForEach<FRigControlElement>([&VisibleControlKeys](const FRigControlElement* ControlElement) -> bool
					{
						if (!ControlElement || !ControlElement->Settings.IsVisible())
						{
							return true;
						}

						// Proxy controls with BasedOnSelection visibility are currently hidden until the user
						// selects something they're proxying for; including them would pull the camera off the
						// actually-visible part of the rig.
						if (ControlElement->Settings.AnimationType == ERigControlAnimationType::ProxyControl &&
							ControlElement->Settings.ShapeVisibility == ERigControlVisibility::BasedOnSelection)
						{
							return true;
						}

						VisibleControlKeys.Add(ControlElement->GetKey());
						return true;
					});

					if (VisibleControlKeys.Num() > 0)
					{
						EditMode->FrameItems(VisibleControlKeys);
						bPendingInitialViewportFocus = false;
					}
				}
			}
		}

		if (PendingInitialViewportFocusTickCount >= MaxPendingFocusTicks)
		{
			bPendingInitialViewportFocus = false;
		}
	}

	if(WeakGroundActorPtr.IsValid())
	{
		const TSharedRef<IPersonaPreviewScene> CurrentPreviewScene = GetPersonaToolkit()->GetPreviewScene();
		const float FloorOffset = CurrentPreviewScene->GetFloorOffset();
		const FTransform FloorTransform(FRotator(0, 0, 0), FVector(0, 0, -(FloorOffset)), FVector(4.0f, 4.0f, 1.0f));
		WeakGroundActorPtr->GetStaticMeshComponent()->SetRelativeTransform(FloorTransform);
	}
}

void FControlRigBaseEditor::HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	PreviewViewport = InViewport;

	// TODO: this is duplicated code from FAnimBlueprintEditor, would be nice to consolidate. 
	auto GetCompilationStateText = [this]()
	{
		if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
		{
			switch (Blueprint->GetAssetStatus())
			{
			case RVMA_UpToDate:
			case RVMA_UpToDateWithWarnings:
				// Fall thru and return empty string
				break;
			case RVMA_Dirty:
				return LOCTEXT("ControlRigBP_Dirty", "Preview out of date");
			case RVMA_Error:
				return LOCTEXT("ControlRigBP_CompileError", "Compile Error");
			default:
				return LOCTEXT("ControlRigBP_UnknownStatus", "Unknown Status");
			}
		}

		return FText::GetEmpty();
	};

	auto GetCompilationStateVisibility = [this]()
	{
		if (const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
		{
			if(Blueprint->IsModularRig())
			{
				if(Blueprint->GetPreviewMesh() == nullptr)
				{
					return EVisibility::Collapsed;
				}
			}
			const bool bUpToDate = (Blueprint->GetRigVMAssetInterface()->GetAssetStatus() == RVMA_UpToDate) || (Blueprint->GetRigVMAssetInterface()->GetAssetStatus() == RVMA_UpToDateWithWarnings);
			return bUpToDate ? EVisibility::Collapsed : EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	};

	auto GetCompileButtonVisibility = [this]()
	{
		if (const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
		{
			return (Blueprint->GetRigVMAssetInterface()->GetAssetStatus() == RVMA_Dirty) ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	};

	auto CompileBlueprint = [this]()
	{
		if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
		{
			if (!Blueprint->IsUpToDate())
			{
				Compile();
			}
		}

		return FReply::Handled();
	};

	auto GetErrorSeverity = [this]()
	{
		if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
		{
			return (Blueprint->GetAssetStatus() == RVMA_Error) ? EMessageSeverity::Error : EMessageSeverity::Warning;
		}

		return EMessageSeverity::Warning;
	};

	auto GetIcon = [this]()
	{
		if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
		{
			return (Blueprint->GetAssetStatus() == RVMA_Error) ? FEditorFontGlyphs::Exclamation_Triangle : FEditorFontGlyphs::Eye;
		}

		return FEditorFontGlyphs::Eye;
	};

	auto GetChangingShapeTransformText = [this]()
	{
		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			FText HotKeyText = EditMode->GetToggleControlShapeTransformEditHotKey();

			if (!HotKeyText.IsEmpty())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("HotKey"), HotKeyText);
				return FText::Format(LOCTEXT("ControlRigBPViewportShapeTransformEditNotificationPress", "Currently Manipulating Shape Transform - Press {HotKey} to Exit"), Args);
			}
		}
		
		return LOCTEXT("ControlRigBPViewportShapeTransformEditNotificationAssign", "Currently Manipulating Shape Transform - Assign a Hotkey and Use It to Exit");
	};

	auto GetChangingShapeTransformTextVisibility = [this]()
	{
		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			return EditMode->bIsChangingControlShapeTransform ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	};

	{
		FPersonaViewportNotificationOptions DirectManipulationNotificationOptions(TAttribute<EVisibility>::CreateRaw(this, &FControlRigBaseEditor::GetDirectManipulationVisibility));
		DirectManipulationNotificationOptions.OnGetBrushOverride = TAttribute<const FSlateBrush*>(FControlRigEditorStyle::Get().GetBrush("ControlRig.Viewport.Notification.DirectManipulation"));

		InViewport->AddNotification(
			EMessageSeverity::Info,
			false,
			SNew(SHorizontalBox)
			.Visibility(SharedRef(), &FControlRigBaseEditor::GetDirectManipulationVisibility)
			.ToolTipText(LOCTEXT("DirectManipulation", "Direct Manipulation"))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Crosshairs)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew(DirectManipulationCombo, SComboBox<TSharedPtr<FString>>)
					.ContentPadding(FMargin(4.0f, 2.0f))
					.OptionsSource(&DirectManipulationTextList)
					.OnGenerateWidget_Lambda([this](TSharedPtr<FString> Item)
					{ 
						return SNew(SBox)
							.MaxDesiredWidth(600.0f)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
								.Text(FText::FromString(*Item))
							];
					} )	
					.OnSelectionChanged(SharedRef(), &FControlRigBaseEditor::OnDirectManipulationChanged)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
						.Text(SharedRef(), &FControlRigBaseEditor::GetDirectionManipulationText)
					]
				]
			],
			DirectManipulationNotificationOptions
		);
	}

	{
		InViewport->AddNotification(
			EMessageSeverity::Warning,
			false,
			SNew(SHorizontalBox)
			.Visibility(SharedRef(), &FControlRigBaseEditor::GetConnectorWarningVisibility)
			.ToolTipText(LOCTEXT("ConnectorWarningTooltip", "This rig has unresolved connectors."))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Exclamation_Triangle)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Text(SharedRef(), &FControlRigBaseEditor::GetConnectorWarningText)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				[
					SNew(SButton)
					.ForegroundColor(FSlateColor::UseForeground())
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Primary")
					.ToolTipText(LOCTEXT("ConnectorWarningNavigateTooltip", "Navigate to the first unresolved connector in the hierarchy"))
					.OnClicked(SharedRef(), &FControlRigBaseEditor::OnNavigateToConnectorWarning)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
							.Text(FEditorFontGlyphs::Cog)
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
							.Text(LOCTEXT("ConnectorWarningNavigateButtonLabel", "Discover"))
						]
					]
				]
			],
			FPersonaViewportNotificationOptions(TAttribute<EVisibility>::CreateRaw(this, &FControlRigBaseEditor::GetConnectorWarningVisibility))
		);
	}

	{
		FPersonaViewportNotificationOptions PreviewingNodeNotificationOptions(TAttribute<EVisibility>::CreateRaw(this, &FControlRigBaseEditor::GetPreviewingNodeVisibility));
		PreviewingNodeNotificationOptions.OnGetBrushOverride = TAttribute<const FSlateBrush*>(FControlRigEditorStyle::Get().GetBrush("ControlRig.Viewport.Notification.PreviewingNode"));

		InViewport->AddNotification(
			EMessageSeverity::Info,
			false,
			SNew(SHorizontalBox)
			.Visibility(SharedRef(), &FControlRigBaseEditor::GetPreviewingNodeVisibility)
			.ToolTipText(LOCTEXT("PreviewingNodeTooltip", "Previewing Node in the graph,\nthe full rig is not running.\nClick here to jump to the node."))
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4.0f, 4.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Eye)
				]
				
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked_Lambda([this]()
					{
						OnPreviewingNodeJumpTo();
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
						.Text(LOCTEXT("PreviewingNode", "Previewing Node"))
					]
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("CancelPreviewingNodeTooltip", "Stop previewing and run the full rig."))
					.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
					.OnClicked_Lambda([this]()
					{
						OnPreviewingNodeCancel();
						return FReply::Handled();
					})
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush("Icons.X"))
					]
				]
			],
			PreviewingNodeNotificationOptions
		);
	}

	if(CVarControlRigShowTestingToolbar.GetValueOnAnyThread())
	{
		{
			FPersonaViewportNotificationOptions ReplayValidationNotificationOptions(TAttribute<EVisibility>::CreateRaw<>(this, &FControlRigBaseEditor::GetReplayValidationErrorVisibility));
			ReplayValidationNotificationOptions.OnGetBrushOverride = TAttribute<const FSlateBrush*>(FControlRigEditorStyle::Get().GetBrush("ControlRig.Viewport.Notification.ReplayValidation"));

			InViewport->AddNotification(
				EMessageSeverity::Info,
				false,
				SNew(SHorizontalBox)
				.Visibility(SharedRef(), &FControlRigBaseEditor::GetReplayValidationErrorVisibility)
				.ToolTipText(SharedRef(), &FControlRigBaseEditor::GetReplayValidationErrorTooltip)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(4.0f, 4.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
						.Text(LOCTEXT("ReplayValidationErrorButtonText", "Replay Validation Error"))
					]
				],
				ReplayValidationNotificationOptions
			);
		}
	}

	InViewport->AddNotification(MakeAttributeLambda(GetErrorSeverity),
		false,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetCompilationStateVisibility)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(4.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			.ToolTipText_Lambda(GetCompilationStateText)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text_Lambda(GetIcon)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text_Lambda(GetCompilationStateText)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.Visibility_Lambda(GetCompileButtonVisibility)
			.ToolTipText(LOCTEXT("ControlRigBPViewportCompileButtonToolTip", "Compile this Animation Blueprint to update the preview to reflect any recent changes."))
			.OnClicked_Lambda(CompileBlueprint)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Cog)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Text(LOCTEXT("ControlRigBPViewportCompileButtonLabel", "Compile"))
				]
			]
		],
		FPersonaViewportNotificationOptions(TAttribute<EVisibility>::Create(GetCompilationStateVisibility))
	);

	FPersonaViewportNotificationOptions ChangePreviewMeshNotificationOptions;
	ChangePreviewMeshNotificationOptions.OnGetVisibility = IsModularRig() ? EVisibility::Visible : EVisibility::Collapsed;
	//ChangePreviewMeshNotificationOptions.OnGetBrushOverride = TAttribute<const FSlateBrush*>(FControlRigEditorStyle::Get().GetBrush("ControlRig.Viewport.Notification.ChangeShapeTransform"));

	// notification to allow to change the preview mesh directly in the viewport
	InViewport->AddNotification(TAttribute<EMessageSeverity::Type>::CreateLambda([this]()
		{
			if(const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
			{
				if(Blueprint->GetPreviewMesh() == nullptr)
				{
					return EMessageSeverity::Warning;
				}
			}
			return EMessageSeverity::Info;
		}),
		false,
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.0f, 4.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MissingPreviewMesh", "Please choose a preview mesh!"))
			.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			.Visibility_Lambda([this]()
			{
				if(const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
				{
					if(Blueprint->GetPreviewMesh())
					{
						return EVisibility::Collapsed;
					}
				}
				return EVisibility::Visible;
			})
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 4.0f)
		[
			SNew(SObjectPropertyEntryBox)
			.ObjectPath_Lambda([this]()
			{
				if(const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
				{
					if(const USkeletalMesh* PreviewMesh = Blueprint->GetPreviewMesh())
					{
						return PreviewMesh->GetPathName();
					}
				}
				return FString();
			})
			.AllowedClass(USkeletalMesh::StaticClass())
			.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
			{
				if(const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
				{
					if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InAssetData.GetAsset()))
					{
						const TSharedRef<IPersonaPreviewScene> CurrentPreviewScene = GetPersonaToolkit()->GetPreviewScene();
						CurrentPreviewScene->SetPreviewMesh(SkeletalMesh);
					}
				}
			})
			.AllowCreate(false)
			.AllowClear(false)
			.DisplayUseSelected(false)
			.DisplayBrowse(false)
			.NewAssetFactories(TArray<UFactory*>())
		],
		ChangePreviewMeshNotificationOptions
	);

	FPersonaViewportNotificationOptions ChangeShapeTransformNotificationOptions;
	ChangeShapeTransformNotificationOptions.OnGetVisibility = TAttribute<EVisibility>::Create(GetChangingShapeTransformTextVisibility);
	ChangeShapeTransformNotificationOptions.OnGetBrushOverride = TAttribute<const FSlateBrush*>(FControlRigEditorStyle::Get().GetBrush("ControlRig.Viewport.Notification.ChangeShapeTransform"));

	// notification that shows when users enter the mode that allows them to change shape transform
	InViewport->AddNotification(EMessageSeverity::Type::Info,
		false,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetChangingShapeTransformTextVisibility)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(4.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			.ToolTipText_Lambda(GetChangingShapeTransformText)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda(GetChangingShapeTransformText)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		],
		ChangeShapeTransformNotificationOptions
	);

	UE::ControlRigEditor::PopulateControlRigViewportToolbarControlRigSubmenu();

	auto GetBorderColorAndOpacity = [this]()
	{
		FLinearColor Color = FLinearColor::Transparent;
		const TArray<FName> EventQueue = GetEventQueue();
		if(EventQueue == ConstructionEventQueue)
		{
			Color = UControlRigEditorSettings::Get()->ConstructionEventBorderColor;
		}
		if(EventQueue == BackwardsSolveEventQueue)
		{
			Color = UControlRigEditorSettings::Get()->BackwardsSolveBorderColor;
		}
		if(EventQueue == BackwardsAndForwardsSolveEventQueue)
		{
			Color = UControlRigEditorSettings::Get()->BackwardsAndForwardsBorderColor;
		}
		return Color;
	};

	auto GetBorderVisibility = [this]()
	{
		EVisibility Visibility = EVisibility::Collapsed;
		if (GetEventQueueComboValue() != 0)
		{
			Visibility = EVisibility::HitTestInvisible;
		}
		return Visibility;
	};
	
	InViewport->AddOverlayWidget(
		SNew(SBorder)
        .BorderImage(FControlRigEditorStyle::Get().GetBrush( "ControlRig.Viewport.Border"))
        .BorderBackgroundColor_Lambda(GetBorderColorAndOpacity)
        .Visibility_Lambda(GetBorderVisibility)
        .Padding(0.0f)
        .ShowEffectWhenDisabled(false)
	);

	if (CVarShowSchematicPanelOverlay->GetBool())
	{
		if (FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
		{
			if (Blueprint->IsModularRig())
			{
				SchematicViewport = SNew(SSchematicGraphPanel)
										.GraphDataModel(SchematicModel)
										.IsOverlay(true)
										.PaddingLeft(30)
										.PaddingRight(30)
										.PaddingTop(60)
										.PaddingBottom(60)
										.PaddingInterNode(5)
										.Visibility(SharedRef(), &FControlRigBaseEditor::GetSchematicOverlayVisibility)
				;
				InViewport->AddOverlayWidget(SchematicViewport.ToSharedRef());
			}
		}
	}
	
	InViewport->GetKeyDownDelegate().BindLambda([&](const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) -> FReply {
		if (OnKeyDownDelegate.IsBound())
		{
			FReply Reply = OnKeyDownDelegate.Execute(MyGeometry, InKeyEvent);
			if(Reply.IsEventHandled())
			{
				return Reply;
			}
		}
		if(GetToolkitCommands()->ProcessCommandBindings(InKeyEvent.GetKey(), InKeyEvent.GetModifierKeys(), false))
		{
			return FReply::Handled();
		}
		return FReply::Unhandled();
	});

	// register callbacks to allow control rig asset to store the Bone Size viewport setting
	FEditorViewportClient& ViewportClient = InViewport->GetViewportClient();
	if (FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(&ViewportClient))
	{
		
		TWeakPtr<FControlRigBaseEditor> WeakThis = SharedRef();
		AnimViewportClient->OnSetBoneSize.BindLambda([WeakThis](float InBoneSize)
		{
			if (TSharedPtr<FControlRigBaseEditor> PinnedThis = WeakThis.Pin())
			{
				if (FControlRigAssetInterfacePtr RigBlueprint = PinnedThis->GetControlRigAssetInterface())
				{
					RigBlueprint->Modify();
					RigBlueprint->GetDebugBoneRadius() = InBoneSize;
				}
			}
		});
		
		AnimViewportClient->OnGetBoneSize.BindLambda([WeakThis]() -> float
		{
			if (TSharedPtr<FControlRigBaseEditor> PinnedThis = WeakThis.Pin())
			{
				if (FControlRigAssetInterfacePtr RigBlueprint = PinnedThis->GetControlRigAssetInterface())
				{
					return RigBlueprint->GetDebugBoneRadius();
				}
			}

			return 1.0f;
		});
	}
}

void FControlRigBaseEditor::SetNextSolveMode()
{
	// DEPRECATED 5.8
	SetNextEventQueue();
}

void FControlRigBaseEditor::SetNextEventQueue()
{
	const TArray<FName> EventQueue = GetEventQueue();

	constexpr bool bCompile = false;
	if (EventQueue == ConstructionEventQueue)
	{
		SetEventQueueSuper(ForwardsSolveEventQueue, bCompile);
	}
	if (EventQueue == ForwardsSolveEventQueue)
	{
		SetEventQueueSuper(BackwardsSolveEventQueue, bCompile);
	}
	if (EventQueue == BackwardsSolveEventQueue)
	{
		SetEventQueueSuper(BackwardsAndForwardsSolveEventQueue, bCompile);
	}
	if (EventQueue == BackwardsAndForwardsSolveEventQueue)
	{
		SetEventQueueSuper(ConstructionEventQueue, bCompile);
	}
}

void FControlRigBaseEditor::SetPriorEventQueue()
{
	const TArray<FName> PriorEventQueue = GetLastEventQueue();

	if (!PriorEventQueue.IsEmpty())
	{
		constexpr bool bCompile = false;
		SetEventQueueSuper(PriorEventQueue, bCompile);
	}
}

void FControlRigBaseEditor::ToggleForwardsSolveAndConstruction()
{
	const TArray<FName> EventQueue = GetEventQueue();

	constexpr bool bCompile = false;
	if (EventQueue == ForwardsSolveEventQueue)
	{
		SetEventQueueSuper(ConstructionEventQueue, bCompile);
	}
	else
	{
		SetEventQueueSuper(ForwardsSolveEventQueue, bCompile);
	}
}

void FControlRigBaseEditor::HandleToggleControlVisibility()
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->ToggleDisplayControlsEnabled();
	}
}

bool FControlRigBaseEditor::AreControlsVisible() const
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		return EditMode->IsDisplayControlsEnabled();
	}
	return false;
}

void FControlRigBaseEditor::HandleToggleControlsAsOverlay()
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->ToggleControlsAsOverlay();
	}
}

bool FControlRigBaseEditor::AreControlsAsOverlay() const
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		return EditMode->IsControlsAsOverlayEnabled();
	}
	return false;
}

void FControlRigBaseEditor::HandleToggleSchematicViewport()
{
	if(SchematicViewport.IsValid())
	{
		SchematicModel->UpdateControlRigContent();
		UControlRigEditorSettings::Get()->bShowSchematicViewInModularRig = !UControlRigEditorSettings::Get()->bShowSchematicViewInModularRig;
		UControlRigEditorSettings::Get()->SaveConfig();
	}
}

bool FControlRigBaseEditor::IsSchematicViewportActive() const
{
	if (SchematicViewport.IsValid())
	{
		return SchematicViewport->GetVisibility() != EVisibility::Hidden;
	}
	return false;
}

void FControlRigBaseEditor::HandleToggleSchematicViewportShowEmptySocketsOnly()
{
	if (SchematicViewport.IsValid())
	{
		UControlRigEditorSettings::Get()->bSchematicViewportShowEmptySocketsOnly = !UControlRigEditorSettings::Get()->bSchematicViewportShowEmptySocketsOnly;
		UControlRigEditorSettings::Get()->SaveConfig();

		SchematicModel->UpdateControlRigContent();
	}
}

bool FControlRigBaseEditor::IsSchematicViewportShowEmptySocketsOnlyActive() const
{
	return 
		IsSchematicViewportActive() &&
		UControlRigEditorSettings::Get()->bSchematicViewportShowEmptySocketsOnly;
}

void FControlRigBaseEditor::HandleToggleCompileResetsEventQueue()
{
	UControlRigEditorSettings* EditorSettings = GetMutableDefault<UControlRigEditorSettings>();
	EditorSettings->bCompileResetsEventQueue = !EditorSettings->bCompileResetsEventQueue;

	EditorSettings->SaveConfig();
}

bool FControlRigBaseEditor::IsToggleCompileResetsEventQueueActive() const
{
	return UControlRigEditorSettings::Get()->bCompileResetsEventQueue;
}

EVisibility FControlRigBaseEditor::GetSchematicOverlayVisibility() const
{
	if(!UControlRigEditorSettings::Get()->bShowSchematicViewInModularRig)
	{
		return EVisibility::Hidden;
	}
	
	if(const URigHierarchy* Hierarchy = GetHierarchyBeingDebugged())
	{
		TArray<const FRigBaseElement*> SelectedElements = Hierarchy->GetSelectedElements();
		if(SelectedElements.ContainsByPredicate([](const FRigBaseElement* InSelectedElement) -> bool
		{
			return InSelectedElement->IsA<FRigControlElement>();
		}))
		{
			return EVisibility::Hidden;
		}
	}
	return EVisibility::SelfHitTestInvisible;
}

bool FControlRigBaseEditor::IsRunningForwardSolve() const
{
	return GetEventQueue() == ForwardsSolveEventQueue;
}

void FControlRigBaseEditor::OnRunGraphOnceToggledImpl(bool bRunGraphOnceEnabled)
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		if (bRunGraphOnceEnabled)
		{
			ControlRig->SetEvaluationsLeft(1);
		}
		else
		{
			ControlRig->ResetEvaluationsLeft();
		}
	}	
}

bool FControlRigBaseEditor::GetToolbarDrawAxesOnSelection() const
{
	if (const UControlRigEditorSettings* Settings = GetDefault<UControlRigEditorSettings>())
	{
		return Settings->bDisplayAxesOnSelection;
	}
	return false;
}

void FControlRigBaseEditor::HandleToggleToolbarDrawAxesOnSelection()
{
	if (UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>())
	{
		Settings->bDisplayAxesOnSelection = !Settings->bDisplayAxesOnSelection;
		Settings->SaveConfig();
	}
}

bool FControlRigBaseEditor::IsToolbarDrawNullsEnabled() const
{
	if (const UControlRig* ControlRig = GetControlRig())
	{
		if (!ControlRig->IsConstructionModeEnabled())
		{
			return true;
		}
	}
	return false;
}

bool FControlRigBaseEditor::GetToolbarDrawNulls() const
{
	const FControlRigEditMode* EditMode = GetEditMode();
	return EditMode ? EditMode->IsDisplayNullsEnabled() : false;
}

void FControlRigBaseEditor::HandleToggleToolbarDrawNulls()
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->ToggleDisplayNulls();
	}
}

bool FControlRigBaseEditor::IsToolbarDrawSocketsEnabled() const
{
	if (const UControlRig* ControlRig = GetControlRig())
	{
		if (!ControlRig->IsConstructionModeEnabled())
		{
			return true;
		}
	}
	return false;
}

bool FControlRigBaseEditor::GetToolbarDrawSockets() const
{
	const FControlRigEditMode* EditMode = GetEditMode();
	return EditMode ? EditMode->IsDisplaySocketsEnabled() : false;
}

void FControlRigBaseEditor::HandleToggleToolbarDrawSockets()
{
	if (FControlRigEditMode* EditMode = GetEditMode())
	{
		EditMode->ToggleDisplaySockets();
	}
}

bool FControlRigBaseEditor::IsConstructionModeEnabled() const
{
	return GetEventQueue() == ConstructionEventQueue;
}

bool FControlRigBaseEditor::IsDebuggingExternalControlRig(const UControlRig* InControlRig) const
{
	if(InControlRig == nullptr)
	{
		if(const FControlRigAssetInterfacePtr ControlRigBlueprint = GetControlRigAssetInterface())
		{
			InControlRig = Cast<UControlRig>(ControlRigBlueprint->GetObjectBeingDebugged());
		}
	}
	return InControlRig != GetControlRig();
}

bool FControlRigBaseEditor::ShouldExecuteControlRig(const UControlRig* InControlRig) const
{
	return (!IsDebuggingExternalControlRig(InControlRig)) && bExecutionControlRig;
}

void FControlRigBaseEditor::HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// load a ground mesh
	static const TCHAR* GroundAssetPath = TEXT("/Engine/MapTemplates/SM_Template_Map_Floor.SM_Template_Map_Floor");
	if (UStaticMesh* FloorMesh = Cast<UStaticMesh>(StaticLoadObject(UStaticMesh::StaticClass(), NULL, GroundAssetPath, NULL, LOAD_None, NULL)))
	{
		UMaterial* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		ensureMsgf(DefaultMaterial, TEXT("Control Rig editor default material can not be created"));

		// create ground mesh actor
		AStaticMeshActor* GroundActor = InPersonaPreviewScene->GetWorld()->SpawnActor<AStaticMeshActor>(AStaticMeshActor::StaticClass(), FTransform::Identity);
		GroundActor->SetFlags(RF_Transient);
		GroundActor->GetStaticMeshComponent()->SetStaticMesh(FloorMesh);
		if (DefaultMaterial)
		{
			GroundActor->GetStaticMeshComponent()->SetMaterial(0, DefaultMaterial);
		}
		GroundActor->SetMobility(EComponentMobility::Static);
		GroundActor->GetStaticMeshComponent()->SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
		GroundActor->GetStaticMeshComponent()->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
		GroundActor->GetStaticMeshComponent()->bSelectable = false;
		// this will be an invisible collision box that users can use to test traces
		GroundActor->GetStaticMeshComponent()->SetVisibility(false);

		WeakGroundActorPtr = GroundActor;
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Control Rig editor ground asset not found :%ls", GroundAssetPath);
	}

	// leave some metadata on the world used for debug object labeling
	if (FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(InPersonaPreviewScene->GetWorld()))
	{
		static constexpr TCHAR Format[] = TEXT("ControlRigEditor (%s)");
		WorldContext->CustomDescription = FString::Printf(Format, *GetRigVMAssetInterface()->GetObject()->GetName());
	}

	AAnimationEditorPreviewActor* Actor = InPersonaPreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
	Actor->SetFlags(RF_Transient);
	InPersonaPreviewScene->SetActor(Actor);

	// Create the preview component
	UControlRigSkeletalMeshComponent* EditorSkelComp = NewObject<UControlRigSkeletalMeshComponent>(Actor);
	EditorSkelComp->SetSkeletalMesh(InPersonaPreviewScene->GetPersonaToolkit()->GetPreviewMesh());
	InPersonaPreviewScene->SetPreviewMeshComponent(EditorSkelComp);
	bool bWasCreated = false;
	FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UControlRigLayerInstance>(EditorSkelComp, bWasCreated);
	InPersonaPreviewScene->AddComponent(EditorSkelComp, FTransform::Identity);

	// set root component, so we can attach to it. 
	Actor->SetRootComponent(EditorSkelComp);
	EditorSkelComp->bSelectable = false;
	EditorSkelComp->MarkRenderStateDirty();
	
	InPersonaPreviewScene->SetAllowMeshHitProxies(false);
	InPersonaPreviewScene->SetAdditionalMeshesSelectable(false);

	PreviewInstance = nullptr;
	if (UControlRigLayerInstance* ControlRigLayerInstance = Cast<UControlRigLayerInstance>(EditorSkelComp->GetAnimInstance()))
	{
		PreviewInstance = Cast<UAnimPreviewInstance>(ControlRigLayerInstance->GetSourceAnimInstance());
	}
	else
	{
		PreviewInstance = Cast<UAnimPreviewInstance>(EditorSkelComp->GetAnimInstance());
	}

	// remove the preview scene undo handling - it has unwanted side effects
	InPersonaPreviewScene->UnregisterForUndo();
}

void FControlRigBaseEditor::UpdateRigVMHostImpl()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	UpdateRigVMHostSuper();

	FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface();
	// For backwards compatibility, we cannot assume the runtime asset interface is a IControlRigRuntimeAssetInterface (older assets created URigVMBlueprintGeneratedClass instead 
	// of UControlRigBlueprintGeneratedClass)
	TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = Blueprint->GetRuntimeAssetInterface().GetObject();
	if (!RuntimeAsset)
	{
		if (UControlRigBlueprint* CRBP = Cast<UControlRigBlueprint>(Blueprint.GetObject()))
		{
			RuntimeAsset = CRBP->GetRigVMBlueprintGeneratedClass();
		}
	}
	if (RuntimeAsset)
	{
		UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
		UControlRigLayerInstance* AnimInstance = Cast<UControlRigLayerInstance>(EditorSkelComp->GetAnimInstance());
		UControlRig* ControlRig = GetControlRig();

		if (AnimInstance && ControlRig)
		{
 			PreviewInstance = Cast<UAnimPreviewInstance>(AnimInstance->GetSourceAnimInstance());
			ControlRig->PreviewInstance = PreviewInstance;

			if (TScriptInterface<IControlRigRuntimeAssetInterface> CRRuntimeAsset = Blueprint->GetRuntimeAssetInterface())
			{
				CRRuntimeAsset->GetShapeLibraries() = Blueprint->GetShapeLibraries();
			}
			else
			{
				if (UControlRig* CDO = Cast<UControlRig>(RuntimeAsset.GetObject()->GetClass()->GetDefaultObject()))
				{
					CDO->ShapeLibraries = GetControlRigAssetInterface()->GetShapeLibraries();
				}
			}

			CacheNameLists();

			// When the control rig is re-instanced on compile, it loses its binding, so we refresh it here if needed
			if (!ControlRig->GetObjectBinding().IsValid())
			{
				ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
			}

			// initialize is moved post reinstance
			AnimInstance->ResetControlRigTracks();
			AnimInstance->AddControlRigTrack(0, ControlRig);
			AnimInstance->UpdateControlRigTrack(0, 1.0f, FControlRigIOSettings::MakeEnabled(), bExecutionControlRig);
			AnimInstance->RecalcRequiredBones();

			// since rig has changed, rebuild draw skeleton
			EditorSkelComp->RebuildDebugDrawSkeleton();
			if (FControlRigEditMode* EditMode = GetEditMode())
			{
				EditMode->SetObjects(ControlRig, EditorSkelComp,nullptr);
			}

			ControlRig->OnPreForwardsSolve_AnyThread().RemoveAll(this);
			ControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
			ControlRig->OnEarlyExit_AnyThread().RemoveAll(this);
			ControlRig->ControlModified().RemoveAll(this);

			ControlRig->OnPreForwardsSolve_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPreForwardsSolve_AnyThread);
			ControlRig->OnPostForwardsSolve_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnPostForwardsSolve_AnyThread);
			ControlRig->OnEarlyExit_AnyThread().AddSP(SharedRef(), &FControlRigBaseEditor::OnEarlyExit_AnyThread);
			ControlRig->ControlModified().AddSP(SharedRef(), &FControlRigBaseEditor::HandleOnControlModified);
		}

		if(IsModularRig() && ControlRig)
		{
			if(SchematicModel->ControlRigBlueprint.IsValid())
			{
				SchematicModel->OnSetObjectBeingDebugged(ControlRig);
			}
		}
	}
}

void FControlRigBaseEditor::UpdateRigVMHost_PreClearOldHostImpl(URigVMHost* InPreviousHost)
{
	if(ReplayStrongPtr.IsValid())
	{
		ReplayStrongPtr->StopReplay();
	}
}

void FControlRigBaseEditor::CacheNameListsImpl()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	CacheNameListsSuper();

	if (FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface())
	{
		TArray<UEdGraph*> EdGraphs;
		ControlRigBP->GetRigVMAssetInterface()->GetAllEdGraphs(EdGraphs);

		URigHierarchy* Hierarchy = GetHierarchyBeingDebugged();
		for (UEdGraph* Graph : EdGraphs)
		{
			UControlRigGraph* RigGraph = Cast<UControlRigGraph>(Graph);
			if (RigGraph == nullptr)
			{
				continue;
			}

			const TArray<TSoftObjectPtr<UControlRigShapeLibrary>>* ShapeLibraries = &ControlRigBP->GetShapeLibraries();
			if(const UControlRig* DebuggedControlRig = Hierarchy->GetTypedOuter<UControlRig>())
			{
				ShapeLibraries = &DebuggedControlRig->GetShapeLibraries();
			}
			RigGraph->CacheNameLists(Hierarchy, &ControlRigBP->GetDrawContainer(), *ShapeLibraries);
		}
	}
}

FVector2D FControlRigBaseEditor::ComputePersonaProjectedScreenPos(const FVector& InWorldPos, bool bClampToScreenRectangle)
{
	if (PreviewViewport.IsValid())
	{
		FEditorViewportClient& Client = PreviewViewport->GetViewportClient();
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
							Client.Viewport,
							Client.GetScene(),
							Client.EngineShowFlags));
		// SceneView is deleted with the ViewFamily
		FSceneView* SceneView = Client.CalcSceneView(&ViewFamily);
	
		// Compute the MinP/MaxP in pixel coord, relative to View.ViewRect.Min
		const FMatrix& WorldToView = SceneView->ViewMatrices.GetWorldToView();
		const FMatrix& ViewToProj = SceneView->ViewMatrices.GetViewToClip();
		const float NearClippingDistance = SceneView->NearClippingDistance + SMALL_NUMBER;
		const FIntRect ViewRect = SceneView->UnconstrainedViewRect;

		// Clamp position on the near plane to get valid rect even if bounds' points are behind the camera
		FPlane P_View = WorldToView.TransformFVector4(FVector4(InWorldPos, 1.f));
		if (P_View.Z <= NearClippingDistance)
		{
			P_View.Z = NearClippingDistance;
		}

		// Project from view to projective space
		FVector2D MinP(FLT_MAX, FLT_MAX);
		FVector2D MaxP(-FLT_MAX, -FLT_MAX);
		FVector2D ScreenPos;
		const bool bIsValid = FSceneView::ProjectWorldToScreen(P_View, ViewRect, ViewToProj, ScreenPos);

		// Clamp to pixel border
		ScreenPos = FIntPoint(FMath::FloorToInt(ScreenPos.X), FMath::FloorToInt(ScreenPos.Y));

		// Clamp to screen rect
		if(bClampToScreenRectangle)
		{
			ScreenPos.X = FMath::Clamp(ScreenPos.X, ViewRect.Min.X, ViewRect.Max.X);
			ScreenPos.Y = FMath::Clamp(ScreenPos.Y, ViewRect.Min.Y, ViewRect.Max.Y);
		}

		return FVector2D(ScreenPos.X, ScreenPos.Y);
	}
	return FVector2D::ZeroVector;
}

void FControlRigBaseEditor::FindReferencesOfItem(const FRigHierarchyKey& InKey)
{
	if(InKey.IsElement())
	{
		static constexpr TCHAR Format[] = TEXT("Type,%s,Name,%s");
		static const UEnum* TypeEnum = StaticEnum<ERigElementType>();
		const FText TypeText = TypeEnum->GetDisplayNameTextByValue((int64)InKey.GetElement().Type);
		const FString Query = FString::Printf(Format, *TypeText.ToString(), *InKey.GetName()); 
		SummonSearchUI(true, Query, true);
	}
	if(InKey.IsComponent())
	{
		static constexpr TCHAR Format[] = TEXT("Name,%s");
		const FString Query = FString::Printf(Format, *InKey.GetName()); 
		SummonSearchUI(true, Query, true);
	}
}

void FControlRigBaseEditor::HandlePreviewMeshChanged(USkeletalMesh* InOldSkeletalMesh, USkeletalMesh* InNewSkeletalMesh)
{
	RebindToSkeletalMeshComponent();

	if (GetObjectsCurrentlyBeingEdited()->Num() > 0)
	{
		if (FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface())
		{
			const bool bTransact = !GIsTransacting && IsEditorInitialized();
			const FScopedTransaction Transaction(NSLOCTEXT("ControlRigEditor", "SetPreviewMesh", "Set Preview Mesh"), bTransact);
			
			ControlRigBP->SetPreviewMesh(InNewSkeletalMesh);
			URigHierarchy* BPHierarchy = ControlRigBP->GetHierarchy();

			FModularRigConnections PreviousConnections;
			if(IsModularRig())
			{
				PreviousConnections = ControlRigBP->GetModularRigModel().Connections;
				{
					TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBP->GetRigVMAssetInterface()->bSuspendAllNotifications, true);
					if(URigHierarchyController* Controller = ControlRigBP->GetHierarchyController())
					{
						// remove all connectors / sockets. keeping them around may mess up the order of the elements
						// in the hierarchy, such as [bone,bone,bone,connector,connector,bone,bone,bone].
						// if the element is manually created, remember it to create it after importing the skeleton element
						TArray<FRigElementKey> ConnectorsAndSockets = Controller->GetHierarchy()->GetConnectorKeys();
						ConnectorsAndSockets.Append(Controller->GetHierarchy()->GetSocketKeys());

						TArray<TTuple<FRigElementKey, FRigElementKey, FTransform>> ConnectorsAndSocketsToParents;
						ConnectorsAndSocketsToParents.Reserve(ConnectorsAndSockets.Num());
						
						for(const FRigElementKey& Key : ConnectorsAndSockets)
						{
							// Remember manually created elements to apply them again
							if (BPHierarchy->GetModuleFName(Key).IsNone())
							{
								const FRigElementKey& Parent = BPHierarchy->GetDefaultParent(Key);
								ConnectorsAndSocketsToParents.Emplace(Key, Parent, BPHierarchy->GetLocalTransform(Key));
							}
							(void)Controller->RemoveElement(Key, bTransact, true);
						}
						
						USkeleton* Skeleton = InNewSkeletalMesh ? InNewSkeletalMesh->GetSkeleton() : nullptr;
						Controller->ImportBones(Skeleton, NAME_None, true, true, false, bTransact, true);
						if(InNewSkeletalMesh)
						{
							Controller->ImportCurvesFromSkeletalMesh(InNewSkeletalMesh, NAME_None, false, bTransact, true);
							Controller->ImportSocketsFromSkeletalMesh(InNewSkeletalMesh, NAME_None, true, true, false, bTransact, true);
						}
						else
						{
							Controller->ImportCurves(Skeleton, NAME_None, false, bTransact, true);
						}

						// Recreate manually created elements
						for (const TTuple<FRigElementKey, FRigElementKey, FTransform>& Tuple : ConnectorsAndSocketsToParents)
						{
							const FRigElementKey& Key = Tuple.Get<0>();
							const FRigElementKey& Parent = Tuple.Get<1>();
							const FTransform& Transform = Tuple.Get<2>();
							
							if (!Parent.IsValid() || BPHierarchy->Contains(Parent))
							{
								switch (Key.Type)
								{
									case ERigElementType::Socket: Controller->AddSocket(Key.Name, Parent, Transform, false, FLinearColor::White, FString(), bTransact); break;
									case ERigElementType::Connector: Controller->AddConnector(Key.Name, FRigConnectorSettings(), bTransact); break;
								}
							}
						}
					}
				}
				ControlRigBP->PropagateHierarchyFromBPToInstances();
			}
			
			UpdateRigVMHost();
			
			if(UControlRig* DebuggedControlRig = Cast<UControlRig>(ControlRigBP->GetObjectBeingDebugged()))
			{
				DebuggedControlRig->GetHierarchy()->Notify(ERigHierarchyNotification::HierarchyReset, {});
				DebuggedControlRig->Initialize(true);
			}

			Compile();

			if(IsModularRig())
			{
				if(UControlRig* DebuggedControlRig = Cast<UControlRig>(ControlRigBP->GetObjectBeingDebugged()))
				{
					DebuggedControlRig->RequestConstruction();
					DebuggedControlRig->Execute(FRigUnit_PrepareForExecution::EventName);
					
					if(URigHierarchy* Hierarchy = DebuggedControlRig->GetHierarchy())
					{
						FModularRigModel* Model = &ControlRigBP->GetModularRigModel();
						
						// try to reestablish the connections.
						UModularRigController* ModularRigController = ControlRigBP->GetModularRigController();
						const bool bAutoResolve = ControlRigBP->GetModularRigSettings().bAutoResolve;
						Model->ForEachModule(
							[&Model, Hierarchy, ModularRigController, &PreviousConnections, bAutoResolve, bTransact]
							(const FRigModuleReference* Module) -> bool
							{
								bool bContinueResolval;
								TArray<uint32> AttemptedTargets;
								do
								{
									bContinueResolval = false;
									
									const TArray<const FRigConnectorElement*> Connectors = Module->FindConnectors(Hierarchy);
									TArray<FRigElementKey> PrimaryConnectors, SecondaryConnectors, OptionalConnectors;
									for(const FRigConnectorElement* ExistingConnector : Connectors)
									{
										if(ExistingConnector->IsPrimary())
										{
											PrimaryConnectors.Add(ExistingConnector->GetKey());
										}
										else if(ExistingConnector->IsOptional())
										{
											OptionalConnectors.Add(ExistingConnector->GetKey());
										}
										else
										{
											SecondaryConnectors.Add(ExistingConnector->GetKey());
										}
									}
									TArray<FRigElementKey> ConnectorKeys;
									ConnectorKeys.Append(PrimaryConnectors);
									ConnectorKeys.Append(SecondaryConnectors);
									ConnectorKeys.Append(OptionalConnectors);
									
									for(const FRigElementKey& ConnectorKey : ConnectorKeys)
									{
										const bool bIsPrimary = ConnectorKey == ConnectorKeys[0];
										const bool bIsSecondary = !bIsPrimary;
										
										if(!Model->Connections.HasConnection(ConnectorKey, Hierarchy))
										{
											// try to reapply the connection
											if(PreviousConnections.HasConnection(ConnectorKey, Hierarchy))
											{
												const FRigElementKey Target = PreviousConnections.FindTargetFromConnector(ConnectorKey);
												if(ModularRigController->ConnectConnectorToElement(ConnectorKey, Target, bTransact))
												{
													bContinueResolval = true;
												}
											}

											// try to auto resolve it
											if(!bContinueResolval && bIsSecondary && bAutoResolve)
											{
												if(ModularRigController->AutoConnectSecondaryConnectors({ConnectorKey}, true, bTransact))
												{
													bContinueResolval = true;
												}
											}

											// only do one connector at a time
											break;
										}
									}

									// Avoid looping forever
									if (bContinueResolval)
									{
										uint32 Attempt = 0;
										for(const FRigElementKey& ConnectorKey : ConnectorKeys)
										{
											const FString ConnectionStr = FString::Printf(TEXT("%s -> %s"), *ConnectorKey.ToString(), *Model->Connections.FindTargetFromConnector(ConnectorKey).ToString());
											const uint32 ConnectionHash = HashCombine(GetTypeHash(ConnectorKey), GetTypeHash(Model->Connections.FindTargetFromConnector(ConnectorKey)));
											Attempt = HashCombine(Attempt, ConnectionHash);
										}
										if (AttemptedTargets.Contains(Attempt))
										{
										   bContinueResolval = false;
										}
										else
										{
										   AttemptedTargets.Add(Attempt);
										}
									}
								}
								while (bContinueResolval);

								return true; // continue to the next module
							}
						);
					}
				}
			}
		}
	}
}

void FControlRigBaseEditor::RebindToSkeletalMeshComponent()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	UDebugSkelMeshComponent* MeshComponent = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		bool bWasCreated = false;
		FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UControlRigLayerInstance>(MeshComponent , bWasCreated);
	}
}

void FControlRigBaseEditor::GenerateEventQueueMenuContentImpl(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("Events"));
    MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().ConstructionEvent, TEXT("Setup"), TAttribute<FText>(), TAttribute<FText>(), GetEventQueueIconImpl(ConstructionEventQueue));
    MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().ForwardsSolveEvent, TEXT("Forwards Solve"), TAttribute<FText>(), TAttribute<FText>(), GetEventQueueIconImpl(ForwardsSolveEventQueue));
    MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().BackwardsSolveEvent, TEXT("Backwards Solve"), TAttribute<FText>(), TAttribute<FText>(), GetEventQueueIconImpl(BackwardsSolveEventQueue));
    MenuBuilder.EndSection();

    MenuBuilder.BeginSection(TEXT("Validation"));
    MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().BackwardsAndForwardsSolveEvent, TEXT("BackwardsAndForwards"), TAttribute<FText>(), TAttribute<FText>(), GetEventQueueIconImpl(BackwardsAndForwardsSolveEventQueue));
    MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("Settings"), LOCTEXT("EventQueueSettings", "Settings"));
	MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().ToggleCompileResetsEventQueue, TEXT("ToggleCompileResetsEventQueue"), TAttribute<FText>(), TAttribute<FText>(), GetEventQueueIconImpl(BackwardsAndForwardsSolveEventQueue));
	MenuBuilder.EndSection();

    if (const FControlRigAssetInterfacePtr RigBlueprint = GetRigVMAssetInterface()->GetObject())
    {
    	URigVMEdGraphSchema* Schema = CastChecked<URigVMEdGraphSchema>(RigBlueprint->GetRigVMEdGraphSchemaClass()->GetDefaultObject());
    	
    	bool bFoundUserDefinedEvent = false;
    	const TArray<FName> EntryNames = RigBlueprint->GetRigVMClient()->GetEntryNames();
    	for(const FName& EntryName : EntryNames)
    	{
    		if(Schema->IsRigVMDefaultEvent(EntryName))
    		{
    			continue;
    		}

    		if(!bFoundUserDefinedEvent)
    		{
    			MenuBuilder.AddSeparator();
    			bFoundUserDefinedEvent = true;
    		}

    		FString EventNameStr = EntryName.ToString();
    		if(!EventNameStr.EndsWith(TEXT("Event")))
    		{
    			EventNameStr += TEXT(" Event");
    		}

    		MenuBuilder.AddMenuEntry(
    			FText::FromString(EventNameStr),
    			FText::FromString(FString::Printf(TEXT("Runs the user defined %s"), *EventNameStr)),
    			GetEventQueueIconImpl({EntryName}),
    			FUIAction(
    			FExecuteAction::CreateSP(SharedRef(), &FControlRigBaseEditor::SetEventQueueSuper, TArray<FName>({EntryName})),
    				FCanExecuteAction()
    			)
    		);
    	}
    }
}

void FControlRigBaseEditor::FilterDraggedKeys(TArray<FRigElementKey>& Keys, bool bRemoveNameSpace)
{
	// if the keys being dragged contain something mapped to a connector - use that instead
	if(FControlRigAssetInterfacePtr ControlRigBlueprint = GetControlRigAssetInterface())
	{
		TArray<FRigElementKey> FilteredKeys;
		FilteredKeys.Reserve(Keys.Num());
		for (FRigElementKey Key : Keys)
		{
			for(const FModularRigSingleConnection& Connection : ControlRigBlueprint->GetModularRigModel().Connections)
			{
				if(Connection.Targets.Contains(Key))
				{
					Key = Connection.Connector;
					break;
				}
			}

			if(bRemoveNameSpace)
			{
				const FString Name = Key.Name.ToString();
				int32 LastCharIndex = INDEX_NONE;
				if(Name.FindLastChar(FRigHierarchyModulePath::ModuleNameSuffixChar, LastCharIndex))
				{
					Key.Name = *Name.Mid(LastCharIndex+1);
				}
			}
			else
			{
				if(const UControlRig* DebuggedControlRig = Cast<UControlRig>(ControlRigBlueprint->GetObjectBeingDebugged()))
				{
					if(!DebuggedControlRig->GetHierarchy()->Contains(Key))
					{
						const FString ModulePrefix = DebuggedControlRig->GetRigModulePrefix();
						if(!ModulePrefix.IsEmpty())
						{
							Key.Name = *(ModulePrefix + Key.Name.ToString());
						}
					}
				}
			}
			FilteredKeys.Add(Key);
		}
		Keys = FilteredKeys;
	}
}

FTransform FControlRigBaseEditor::GetRigElementTransform(const FRigElementKey& InElement, bool bLocal, bool bOnDebugInstance) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if(bOnDebugInstance)
	{
		UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged());
		if (DebuggedControlRig == nullptr)
		{
			DebuggedControlRig = GetControlRig();
		}

		if (DebuggedControlRig)
		{
			if (bLocal)
			{
				return DebuggedControlRig->GetHierarchy()->GetLocalTransform(InElement);
			}
			return DebuggedControlRig->GetHierarchy()->GetGlobalTransform(InElement);
		}
	}

	if (bLocal)
	{
		return GetHierarchyBeingDebugged()->GetLocalTransform(InElement);
	}
	return GetHierarchyBeingDebugged()->GetGlobalTransform(InElement);
}

void FControlRigBaseEditor::SetRigElementTransform(const FRigElementKey& InElement, const FTransform& InTransform, bool bLocal)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FScopedTransaction Transaction(LOCTEXT("Move Bone", "Move Bone transform"));
	FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface();
	ControlRigBP->Modify();

	switch (InElement.Type)
	{
		case ERigElementType::Bone:
		case ERigElementType::Connector:
		case ERigElementType::Socket:
		{
			FTransform Transform = InTransform;
			if (bLocal)
			{
				FTransform ParentTransform = FTransform::Identity;
				FRigElementKey ParentKey = ControlRigBP->GetHierarchy()->GetFirstParent(InElement);
				if (ParentKey.IsValid())
				{
					ParentTransform = GetRigElementTransform(ParentKey, false, false);
				}
				Transform = Transform * ParentTransform;
				Transform.NormalizeRotation();
			}

			ControlRigBP->GetHierarchy()->SetInitialGlobalTransform(InElement, Transform);
			ControlRigBP->GetHierarchy()->SetGlobalTransform(InElement, Transform);
			OnHierarchyChanged();
			break;
		}
		case ERigElementType::Control:
		{
			FTransform LocalTransform = InTransform;
			FTransform GlobalTransform = InTransform;
			if (!bLocal)
			{
				ControlRigBP->GetHierarchy()->SetGlobalTransform(InElement, InTransform);
				LocalTransform = ControlRigBP->GetHierarchy()->GetLocalTransform(InElement);
			}
			else
			{
				ControlRigBP->GetHierarchy()->SetLocalTransform(InElement, InTransform);
				GlobalTransform = ControlRigBP->GetHierarchy()->GetGlobalTransform(InElement);
			}
			ControlRigBP->GetHierarchy()->SetInitialLocalTransform(InElement, LocalTransform);
			ControlRigBP->GetHierarchy()->SetGlobalTransform(InElement, GlobalTransform);
			OnHierarchyChanged();
			break;
		}
		case ERigElementType::Null:
		{
			FTransform LocalTransform = InTransform;
			FTransform GlobalTransform = InTransform;
			if (!bLocal)
			{
				ControlRigBP->GetHierarchy()->SetGlobalTransform(InElement, InTransform);
				LocalTransform = ControlRigBP->GetHierarchy()->GetLocalTransform(InElement);
			}
			else
			{
				ControlRigBP->GetHierarchy()->SetLocalTransform(InElement, InTransform);
				GlobalTransform = ControlRigBP->GetHierarchy()->GetGlobalTransform(InElement);
			}

			ControlRigBP->GetHierarchy()->SetInitialLocalTransform(InElement, LocalTransform);
			ControlRigBP->GetHierarchy()->SetGlobalTransform(InElement, GlobalTransform);
			OnHierarchyChanged();
			break;
		}
		default:
		{
			ensureMsgf(false, TEXT("Unsupported RigElement Type : %d"), EnumToUnderlyingType(InElement.Type));
			break;
		}
	}
	
	UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
	if (EditorSkelComp)
	{
		EditorSkelComp->RebuildDebugDrawSkeleton();
	}
}

void FControlRigBaseEditor::OnFinishedChangingPropertiesImpl(const FPropertyChangedEvent& PropertyChangedEvent)
{
	OnFinishedChangingPropertiesSuper(PropertyChangedEvent);
	
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface();

	if (ControlRigBP)
	{
		if (PropertyChangedEvent.Property->GetNameCPP() == GET_MEMBER_NAME_STRING_CHECKED(FRigHierarchySettings, ElementNameDisplayMode))
		{
			Compile();
		}
		
		else if (PropertyChangedEvent.MemberProperty->GetNameCPP() == TEXT("HierarchySettings"))
		{
			ControlRigBP->PropagateHierarchyFromBPToInstances();
		}

		else if (PropertyChangedEvent.MemberProperty->GetNameCPP() == TEXT("DrawContainer"))
		{
			ControlRigBP->PropagateDrawInstructionsFromBPToInstances();
		}

		else if (PropertyChangedEvent.MemberProperty->GetNameCPP() == TEXT("RigModuleSettings"))
		{
			ControlRigBP->PropagateHierarchyFromBPToInstances();
		}
	}
}

void FControlRigBaseEditor::OnWrappedPropertyChangedChainEventImpl(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent)
{
	OnWrappedPropertyChangedChainEventSuper(InWrapperObject, InPropertyPath, InPropertyChangedChainEvent);
	
	check(InWrapperObject);
	check(!GetWrapperObjects().IsEmpty());

	TGuardValue<bool> SuspendDetailsPanelRefresh(GetSuspendDetailsPanelRefreshFlag(), true);

	FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface();

	FString PropertyPath = InPropertyPath;
	if(UScriptStruct* WrappedStruct = InWrapperObject->GetWrappedStruct())
	{
		if(WrappedStruct->IsChildOf(FRigBaseElement::StaticStruct()))
		{
			check(WrappedStruct == GetWrapperObjects()[0]->GetWrappedStruct());

			URigHierarchy* Hierarchy = CastChecked<URigHierarchy>(InWrapperObject->GetSubject());
			const FRigBaseElement WrappedElement = InWrapperObject->GetContent<FRigBaseElement>();
			const FRigBaseElement FirstWrappedElement = GetWrapperObjects()[0]->GetContent<FRigBaseElement>();
			const FRigElementKey& Key = WrappedElement.GetKey();
			if(!Hierarchy->Contains(Key))
			{
				return;
			}

			static constexpr TCHAR PropertyChainElementFormat[] = TEXT("%s->");
			static const FString PoseString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigTransformElement, PoseStorage));
			static const FString OffsetString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigControlElement, OffsetStorage));
			static const FString ShapeString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigControlElement, ShapeStorage));
			static const FString SettingsString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigControlElement, Settings));

			struct Local
			{
				static ERigTransformType::Type GetTransformTypeFromPath(FString& PropertyPath)
				{
					static const FString InitialString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigCurrentAndInitialTransform, Initial));
					static const FString CurrentString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigCurrentAndInitialTransform, Current));
					static const FString GlobalString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigLocalAndGlobalTransform, Global));
					static const FString LocalString = FString::Printf(PropertyChainElementFormat, GET_MEMBER_NAME_STRING_CHECKED(FRigLocalAndGlobalTransform, Local));

					ERigTransformType::Type TransformType = ERigTransformType::CurrentLocal;

					if(PropertyPath.RemoveFromStart(InitialString))
					{
						TransformType = ERigTransformType::MakeInitial(TransformType);
					}
					else
					{
						verify(PropertyPath.RemoveFromStart(CurrentString));
						TransformType = ERigTransformType::MakeCurrent(TransformType);
					}

					if(PropertyPath.RemoveFromStart(GlobalString))
					{
						TransformType = ERigTransformType::MakeGlobal(TransformType);
					}
					else
					{
						verify(PropertyPath.RemoveFromStart(LocalString));
						TransformType = ERigTransformType::MakeLocal(TransformType);
					}

					return TransformType;
				}
			};

			bool bIsInitial = false;
			if(PropertyPath.RemoveFromStart(PoseString))
			{
				const ERigTransformType::Type TransformType = Local::GetTransformTypeFromPath(PropertyPath);
				bIsInitial = bIsInitial || ERigTransformType::IsInitial(TransformType);
				
				if(ERigTransformType::IsInitial(TransformType) || IsConstructionModeEnabled())
				{
					Hierarchy = ControlRigBP->GetHierarchy();
				}

				FRigTransformElement* TransformElement = Hierarchy->Find<FRigTransformElement>(WrappedElement.GetKey());
				if(TransformElement == nullptr)
				{
					return;
				}

				const FTransform Transform = InWrapperObject->GetContent<FRigTransformElement>().GetTransform().Get(TransformType);

				if(ERigTransformType::IsLocal(TransformType) && TransformElement->IsA<FRigControlElement>())
				{
					FRigControlElement* ControlElement = Cast<FRigControlElement>(TransformElement);
							
					FRigControlValue Value;
					Value.SetFromTransform(Transform, ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
							
					if(ERigTransformType::IsInitial(TransformType) || IsConstructionModeEnabled())
					{
						Hierarchy->SetControlValue(ControlElement, Value, ERigControlValueType::Initial, true, true, true);
					}
					Hierarchy->SetControlValue(ControlElement, Value, ERigControlValueType::Current, true, true, true);
				}
				else
				{
					Hierarchy->SetTransform(TransformElement, Transform, TransformType, true, true, false, true);
				}
			}
			else if(PropertyPath.RemoveFromStart(OffsetString))
			{
				FRigControlElement* ControlElement = ControlRigBP->GetHierarchy()->Find<FRigControlElement>(WrappedElement.GetKey());
				if(ControlElement == nullptr)
				{
					return;
				}
				
				ERigTransformType::Type TransformType = Local::GetTransformTypeFromPath(PropertyPath);
				bIsInitial = bIsInitial || ERigTransformType::IsInitial(TransformType);

				const FTransform Transform = GetWrapperObjects()[0]->GetContent<FRigControlElement>().GetOffsetTransform().Get(TransformType);
				
				ControlRigBP->GetHierarchy()->SetControlOffsetTransform(ControlElement, Transform, ERigTransformType::MakeInitial(TransformType), true, true, false, true);
			}
			else if(PropertyPath.RemoveFromStart(ShapeString))
			{
				FRigControlElement* ControlElement = ControlRigBP->GetHierarchy()->Find<FRigControlElement>(WrappedElement.GetKey());
				if(ControlElement == nullptr)
				{
					return;
				}

				ERigTransformType::Type TransformType = Local::GetTransformTypeFromPath(PropertyPath);
				bIsInitial = bIsInitial || ERigTransformType::IsInitial(TransformType);

				const FTransform Transform = GetWrapperObjects()[0]->GetContent<FRigControlElement>().GetShapeTransform().Get(TransformType);
				
				ControlRigBP->GetHierarchy()->SetControlShapeTransform(ControlElement, Transform, ERigTransformType::MakeInitial(TransformType), true, false, true);
			}
			else if(PropertyPath.RemoveFromStart(SettingsString))
			{
				if(Key.Type == ERigElementType::Control)
				{
					const FRigControlSettings Settings  = InWrapperObject->GetContent<FRigControlElement>().Settings;

					FRigControlElement* ControlElement = ControlRigBP->GetHierarchy()->Find<FRigControlElement>(WrappedElement.GetKey());
					if(ControlElement == nullptr)
					{
						return;
					}

					ControlRigBP->GetHierarchy()->SetControlSettings(ControlElement, Settings, true, false, true);
				}
				else if(Key.Type == ERigElementType::Connector)
				{
					const FRigConnectorSettings Settings  = InWrapperObject->GetContent<FRigConnectorElement>().Settings;

					FRigConnectorElement* ConnectorElement = ControlRigBP->GetHierarchy()->Find<FRigConnectorElement>(WrappedElement.GetKey());
					if(ConnectorElement == nullptr)
					{
						return;
					}

					ControlRigBP->GetHierarchy()->SetConnectorSettings(ConnectorElement, Settings, true, false, true);
				}
			}

			if(IsConstructionModeEnabled() || bIsInitial)
			{
				ControlRigBP->PropagatePoseFromBPToInstances();
				ControlRigBP->Modify();
				ControlRigBP.GetObject()->MarkPackageDirty();
			}
		}
		else if(WrappedStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
		{
			FStructOnScope Content(WrappedStruct);
			InWrapperObject->GetContent(Content.GetStructMemory(), Content.GetStruct());

			const FRigBaseComponent* WrappedComponent = reinterpret_cast<const FRigBaseComponent*>(Content.GetStructMemory());
			const FRigComponentState State = WrappedComponent->GetState();
			(void)ControlRigBP->GetHierarchyController()->SetComponentState(WrappedComponent->GetKey(), State, true);
		}
	}
}

void FControlRigBaseEditor::OnClose()
{
	if (ControlRigEditorClosedDelegate.IsBound())
	{
		ControlRigEditorClosedDelegate.Broadcast(this, GetControlRigAssetInterface());
	}
	OnCloseSuper();
}

bool FControlRigBaseEditor::HandleRequestDirectManipulation(ERigControlType InControlType) const
{
	TArray<FRigDirectManipulationTarget> Targets = GetDirectManipulationTargets();
	for(const FRigDirectManipulationTarget& Target : Targets)
	{
		if(Target.ControlType == InControlType || Target.ControlType == ERigControlType::EulerTransform)
		{
			if (FControlRigEditorEditMode* EditMode = GetEditMode())
			{
				switch(InControlType)
				{
					case ERigControlType::Position:
					{
						EditMode->RequestTransformWidgetMode(UE::Widget::WM_Translate);
						break;
					}
					case ERigControlType::Rotator:
					{
						EditMode->RequestTransformWidgetMode(UE::Widget::WM_Rotate);
						break;
					}
					case ERigControlType::Scale:
					{
						EditMode->RequestTransformWidgetMode(UE::Widget::WM_Scale);
						break;
					}
					default:
					{
						break;
					}
				}
			}

			if(FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
			{
				// Mark this node as interactive so the compiler gives it its own literal
				// registers rather than sharing them with other nodes that have the same
				// default values. Only recompile if the node is not already in the set.
				if(DirectManipulationSubject.IsValid())
				{
					auto& ASTSettings = Blueprint->GetRigVMAssetInterface()->GetVMCompileSettings().ASTSettings;
					if(!ASTSettings.InteractiveNodes.Contains(FObjectKey(DirectManipulationSubject.Get())))
					{
						ASTSettings.InteractiveNodes.Add(FObjectKey(DirectManipulationSubject.Get()));
						Blueprint->GetRigVMAssetInterface()->RecompileVM();
					}
				}
				Blueprint->AddTransientControl(DirectManipulationSubject.Get(), Target);
			}
			return true;
		}
	}
	return false;
}

bool FControlRigBaseEditor::SetDirectionManipulationSubject(const URigVMUnitNode* InNode)
{
	if(DirectManipulationSubject.Get() == InNode)
	{
		return false;
	}
	if(FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
	{
		Blueprint->ClearTransientControls();
	}
	DirectManipulationSubject = InNode;

	// update the direct manipulation target list
	RefreshDirectManipulationTextList();
	return true;
}

bool FControlRigBaseEditor::IsDirectManipulationEnabled() const
{
	return !GetDirectManipulationTargets().IsEmpty();
}

EVisibility FControlRigBaseEditor::GetDirectManipulationVisibility() const
{
	return IsDirectManipulationEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FControlRigBaseEditor::GetDirectionManipulationText() const
{
	if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
	{
		if (URigHierarchy* Hierarchy = DebuggedControlRig->GetHierarchy())
		{
			TArray<FRigControlElement*> TransientControls = Hierarchy->GetTransientControls();
			for(const FRigControlElement* TransientControl : TransientControls)
			{
				const FString Target = UControlRig::GetTargetFromTransientControl(TransientControl->GetKey());
				if(!Target.IsEmpty())
				{
					return FText::FromString(Target);
				}
			}
		}
	}
	static const FText DefaultText = LOCTEXT("ControlRigDirectManipulation", "Direct Manipulation");
	return DefaultText;
}

void FControlRigBaseEditor::OnDirectManipulationChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	if(!NewValue.IsValid())
	{
		return;
	}
	
	const URigVMUnitNode* UnitNode = DirectManipulationSubject.Get();
	if(UnitNode == nullptr)
	{
		return;
	}
	
	FControlRigAssetInterfacePtr ControlRigBlueprint(GetRigVMAssetInterface().GetObject());
	if(ControlRigBlueprint == nullptr)
	{
		return;
	}

	// Mark this node as interactive so the compiler gives it its own literal
	// registers rather than sharing them with other nodes that have the same
	// default values. Only recompile if the node is not already in the set.
	{
		auto& ASTSettings = ControlRigBlueprint->GetRigVMAssetInterface()->GetVMCompileSettings().ASTSettings;
		if(!ASTSettings.InteractiveNodes.Contains(FObjectKey(UnitNode)))
		{
			ASTSettings.InteractiveNodes.Add(FObjectKey(UnitNode));
			ControlRigBlueprint->GetRigVMAssetInterface()->RecompileVM();
		}
	}

	const FString& DesiredTarget = *NewValue.Get();
	const TArray<FRigDirectManipulationTarget> Targets = GetDirectManipulationTargets();
	for(const FRigDirectManipulationTarget& Target : Targets)
	{
		if(Target.Name.Equals(DesiredTarget, ESearchCase::CaseSensitive))
		{
			// run the task after a bit so that the rig has the opportunity to run first
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [ControlRigBlueprint, UnitNode, Target]()
			{
				ControlRigBlueprint->AddTransientControl(UnitNode, Target);
			});
			break;
		}
	}
}

const TArray<FRigDirectManipulationTarget> FControlRigBaseEditor::GetDirectManipulationTargets() const
{
	if(DirectManipulationSubject.IsValid())
	{
		if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
		{
			if (DebuggedControlRig->GetVM() && !DebuggedControlRig->GetVM()->IsNativized())
			{
				if(const URigVMUnitNode* Node = DirectManipulationSubject.Get())
				{
					if(Node->IsPartOfRuntime(DebuggedControlRig))
					{
						const TSharedPtr<FStructOnScope> NodeInstance = Node->ConstructLiveStructInstance(DebuggedControlRig);
						if(NodeInstance.IsValid() && NodeInstance->IsValid())
						{
							if(const FRigUnit* UnitInstance = UControlRig::GetRigUnitInstanceFromScope(NodeInstance))
							{
								TArray<FRigDirectManipulationTarget> Targets;
								if(UnitInstance->GetDirectManipulationTargets(Node, NodeInstance, DebuggedControlRig->GetHierarchy(), Targets, nullptr))
								{
									return Targets;
								}
							}
						}
					}
				}
			}
		}
	}

	static const TArray<FRigDirectManipulationTarget> EmptyTargets;
	return EmptyTargets;
}

const TArray<TSharedPtr<FString>>& FControlRigBaseEditor::GetDirectManipulationTargetTextList() const
{
	return DirectManipulationTextList;
}

void FControlRigBaseEditor::RefreshDirectManipulationTextList()
{
	if (DirectManipulationCombo.IsValid())
	{
		DirectManipulationTextList.Reset();
		for (const FRigDirectManipulationTarget& Target : GetDirectManipulationTargets())
		{
			DirectManipulationTextList.Emplace(new FString(Target.Name));
		}

		DirectManipulationCombo->ClearSelection();
		DirectManipulationCombo->RefreshOptions();
	}
}

EVisibility FControlRigBaseEditor::GetPreviewingNodeVisibility() const
{
	if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
	{
		if (DebuggedControlRig->GetDebugInfo().GetStepCondition().OriginInstruction != INDEX_NONE)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void FControlRigBaseEditor::OnPreviewingNodeJumpTo()
{
	check(GetRigVMAssetInterface());

	UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged());
	if (!DebuggedControlRig)
	{
		return;
	}
	const int32 InstructionToJumpTo = DebuggedControlRig->GetDebugInfo().GetStepCondition().OriginInstruction;
	if (InstructionToJumpTo == INDEX_NONE)
	{
		return;
	}

	URigVM* VM = DebuggedControlRig->GetVM();
	if (!VM)
	{
		return;
	}

	const URigVMNode* Subject = Cast<URigVMNode>(VM->GetByteCode().GetSubjectForInstruction(InstructionToJumpTo));
	if (!Subject)
	{
		return;
	}
	
	FRigVMClient* Client = GetRigVMAssetInterface()->GetRigVMClient();
	if (!Client)
	{
		return;
	}

	URigVMController* Controller = Client->GetOrCreateController(Subject->GetGraph());
	if (!Controller)
	{
		return;
	}

	Controller->RequestJumpToHyperLink(Subject);
}

void FControlRigBaseEditor::OnPreviewingNodeCancel()
{
	check(GetRigVMAssetInterface());
	GetRigVMAssetInterface()->ResetEarlyExitInstruction();
}

EVisibility FControlRigBaseEditor::GetConnectorWarningVisibility() const
{
	if(GetConnectorWarningText().IsEmpty())
	{
		return EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}

FText FControlRigBaseEditor::GetConnectorWarningText() const
{
	if (const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
	{
		if(Blueprint->IsControlRigModule())
		{
			if(UControlRig* ControlRig = GetControlRig())
			{
				FString FailureReason;
				if(!ControlRig->AllConnectorsAreResolved(&FailureReason))
				{
					if(FailureReason.IsEmpty())
					{
						static const FText ConnectorWarningDefault = LOCTEXT("ConnectorWarningDefault", "This rig has unresolved connectors.");
						return ConnectorWarningDefault;
					}
					return FText::FromString(FailureReason);
				}
			}
		}
	}
	return FText();
}

FReply FControlRigBaseEditor::OnNavigateToConnectorWarning() const
{
	RequestNavigateToConnectorWarningDelegate.Broadcast();
	return FReply::Handled();
}

void FControlRigBaseEditor::BindCommandsImpl()
{
	BindCommandsSuper();

	TSharedRef<FControlRigBaseEditor> SharedEditor = SharedRef();

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ConstructionEvent,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SetEventQueueSuper, TArray<FName>(ConstructionEventQueue)),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ForwardsSolveEvent,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SetEventQueueSuper, TArray<FName>(ForwardsSolveEventQueue)),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().BackwardsSolveEvent,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SetEventQueueSuper, TArray<FName>(BackwardsSolveEventQueue)),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().BackwardsAndForwardsSolveEvent,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SetEventQueueSuper, TArray<FName>(BackwardsAndForwardsSolveEventQueue)),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().SetNextEventQueue,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SetNextEventQueue),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().SetPriorEventQueue,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SetPriorEventQueue),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleForwardsSolveAndConstruction,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::ToggleForwardsSolveAndConstruction),
		FCanExecuteAction());

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleControlVisibility,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleControlVisibility),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::AreControlsVisible));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleControlsAsOverlay,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleControlsAsOverlay),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::AreControlsAsOverlay));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleDrawNulls,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleToolbarDrawNulls),
		FCanExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::IsToolbarDrawNullsEnabled),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::GetToolbarDrawNulls));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleDrawSockets,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleToolbarDrawSockets),
		FCanExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::IsToolbarDrawSocketsEnabled),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::GetToolbarDrawSockets));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleDrawAxesOnSelection,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleToolbarDrawAxesOnSelection),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::GetToolbarDrawAxesOnSelection));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleSchematicViewportVisibility,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleSchematicViewport),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::IsSchematicViewportActive));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleSchematicViewportShowEmptySocketsOnly,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleSchematicViewportShowEmptySocketsOnly),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::IsSchematicViewportShowEmptySocketsOnlyActive));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().SwapModuleWithinAsset,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SwapModuleWithinAsset),
		FCanExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::IsModularRig));

	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().SwapModuleAcrossProject,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::SwapModuleAcrossProject),
		FCanExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::IsRigModule));
		
	GetToolkitCommands()->MapAction(
		FControlRigEditorCommands::Get().ToggleCompileResetsEventQueue,
		FExecuteAction::CreateSP(SharedEditor, &FControlRigBaseEditor::HandleToggleCompileResetsEventQueue),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(SharedEditor, &FControlRigBaseEditor::IsToggleCompileResetsEventQueueActive));
}

void FControlRigBaseEditor::UnbindCommandsImpl()
{
	UnbindCommandsSuper();
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ConstructionEvent);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ForwardsSolveEvent);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().BackwardsSolveEvent);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().BackwardsAndForwardsSolveEvent);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().SetNextEventQueue);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().SetPriorEventQueue);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleForwardsSolveAndConstruction);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleControlVisibility);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleControlsAsOverlay);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleDrawNulls);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleDrawSockets);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleDrawAxesOnSelection);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleSchematicViewportVisibility);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().SwapModuleWithinAsset);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().SwapModuleAcrossProject);
	GetToolkitCommands()->UnmapAction(FControlRigEditorCommands::Get().ToggleCompileResetsEventQueue);
}

FMenuBuilder FControlRigBaseEditor::GenerateBulkEditMenuImpl()
{
	FMenuBuilder MenuBuilder = GenerateBulkEditMenuSuper();
	MenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("Asset", "Asset"));
	if (FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface())
	{
		if (Blueprint->IsModularRig())
		{
			MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().SwapModuleWithinAsset, TEXT("SwapModuleWithinAsset"), TAttribute<FText>(), TAttribute<FText>(), FSlateIcon());
		}
		else if (Blueprint->IsControlRigModule())
		{
			MenuBuilder.AddMenuEntry(FControlRigEditorCommands::Get().SwapModuleAcrossProject, TEXT("SwapModuleAcrossProject"), TAttribute<FText>(), TAttribute<FText>(), FSlateIcon());
		}
	}
	MenuBuilder.EndSection();
	return MenuBuilder;
}

void FControlRigBaseEditor::OnHierarchyChanged()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (FControlRigAssetInterfacePtr ControlRigBP = GetControlRigAssetInterface())
	{
		{
			TGuardValue<bool> GuardNotifs(ControlRigBP->GetRigVMAssetInterface()->bSuspendAllNotifications, true);
			ControlRigBP->PropagateHierarchyFromBPToInstances();
		}

		GetControlRigAssetInterface()->GetRigVMAssetInterface()->MarkAssetAsModified();
		
		TArray<const FRigBaseElement*> SelectedElements = GetHierarchyBeingDebugged()->GetSelectedElements();
		for(const FRigBaseElement* SelectedElement : SelectedElements)
		{
			ControlRigBP->GetHierarchy()->OnModified().Broadcast(ERigHierarchyNotification::ElementSelected, ControlRigBP->GetHierarchy(), SelectedElement);
		}
		GetControlRigAssetInterface()->GetRigVMAssetInterface()->RequestAutoVMRecompilation();

		SynchronizeViewportBoneSelection();

		UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
		// since rig has changed, rebuild draw skeleton
		if (EditorSkelComp)
		{ 
			EditorSkelComp->RebuildDebugDrawSkeleton(); 
		}

		RefreshDetailView();
	}
	else
	{
		ClearDetailObject();
	}
	
	CacheNameLists();
}


void FControlRigBaseEditor::OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	if(RigBlueprint == nullptr)
	{
		return;
	}

	if (RigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications)
	{
		return;
	}

	if(InHierarchy != RigBlueprint->GetHierarchy())
	{
		return;
	}

	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;

	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementAdded:
		{
			if (!RigBlueprint->IsModularRig())
			{
				if(InElement->GetType() == ERigElementType::Connector)
				{
					if(InHierarchy->GetConnectors().Num() == 1)
					{
						FNotificationInfo Info(LOCTEXT("FirstConnectorEncountered", "Looks like you have added the first connector. This rig will now be configured as a module, settings can be found in the class settings Hierarchy -> Module Settings."));
						Info.bFireAndForget = true;
						Info.FadeOutDuration = 5.0f;
						Info.ExpireDuration = 5.0f;

						TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
						NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);

						RigBlueprint->TurnIntoControlRigModule();
					}
				}
			}
			// no break - fall through
		}
		case ERigHierarchyNotification::ParentChanged:
		case ERigHierarchyNotification::HierarchyReset:
		{
			OnHierarchyChanged();
			break;
		}
		case ERigHierarchyNotification::ElementRemoved:
		{
			UEnum* RigElementTypeEnum = StaticEnum<ERigElementType>();
			if (RigElementTypeEnum == nullptr)
			{
				return;
			}

			CacheNameLists();

			const FString RemovedElementName = InElement->GetName();
			const ERigElementType RemovedElementType = InElement->GetType();

			TArray<UEdGraph*> EdGraphs;
			RigBlueprint->GetRigVMAssetInterface()->GetAllEdGraphs(EdGraphs);

			for (UEdGraph* Graph : EdGraphs)
			{
				UControlRigGraph* RigGraph = Cast<UControlRigGraph>(Graph);
				if (RigGraph == nullptr)
				{
					continue;
				}

				for (UEdGraphNode* Node : RigGraph->Nodes)
				{
					if (UControlRigGraphNode* RigNode = Cast<UControlRigGraphNode>(Node))
					{
						if (URigVMNode* ModelNode = RigNode->GetModelNode())
						{
							TArray<URigVMPin*> ModelPins = ModelNode->GetAllPinsRecursively();
							for (URigVMPin* ModelPin : ModelPins)
							{
								if ((ModelPin->GetCPPType() == TEXT("FName") && ModelPin->GetCustomWidgetName() == TEXT("BoneName") && RemovedElementType == ERigElementType::Bone) ||
									(ModelPin->GetCPPType() == TEXT("FName") && ModelPin->GetCustomWidgetName() == TEXT("ControlName") && RemovedElementType == ERigElementType::Control) ||
									(ModelPin->GetCPPType() == TEXT("FName") && ModelPin->GetCustomWidgetName() == TEXT("SpaceName") && RemovedElementType == ERigElementType::Null) ||
									(ModelPin->GetCPPType() == TEXT("FName") && ModelPin->GetCustomWidgetName() == TEXT("CurveName") && RemovedElementType == ERigElementType::Curve) ||
									(ModelPin->GetCPPType() == TEXT("FName") && ModelPin->GetCustomWidgetName() == TEXT("ConnectorName") && RemovedElementType == ERigElementType::Connector))
								{
									if (ModelPin->GetDefaultValue() == RemovedElementName)
									{
										RigNode->ReconstructNode();
										break;
									}
								}
								else if (ModelPin->GetCPPTypeObject() == FRigElementKey::StaticStruct())
								{
									if (URigVMPin* TypePin = ModelPin->FindSubPin(TEXT("Type")))
									{
										FString TypeStr = TypePin->GetDefaultValue();
										int64 TypeValue = RigElementTypeEnum->GetValueByNameString(TypeStr);
										if (TypeValue == (int64)RemovedElementType)
										{
											if (URigVMPin* NamePin = ModelPin->FindSubPin(TEXT("Name")))
											{
												FString NameStr = NamePin->GetDefaultValue();
												if (NameStr == RemovedElementName)
												{
													RigNode->ReconstructNode();
													break;
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}

			OnHierarchyChanged();
			break;
		}
		case ERigHierarchyNotification::ElementRenamed:
		{
			OnHierarchyChanged();

			break;
		}
		case ERigHierarchyNotification::ComponentAdded:
		case ERigHierarchyNotification::ComponentRemoved:
		case ERigHierarchyNotification::ComponentRenamed:
		case ERigHierarchyNotification::ComponentReparented:
		{
			OnHierarchyChanged();
			break;
		}
		default:
		{
			break;
		}
	}
}

void FControlRigBaseEditor::OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if(bIsConstructionEventRunning)
	{
		return;
	}

	if(SchematicViewport)
	{
		SchematicModel->OnHierarchyModified(InNotif, InHierarchy, InSubject);
	}
	
	FRigElementKey Key;
	FName ComponentName = NAME_None;
	if(InSubject.Element)
	{
		Key = InSubject.Element->GetKey();
	}
	else if(InSubject.Component)
	{
		Key = InSubject.Component->GetElementKey();
		ComponentName = InSubject.Component->GetFName();
	}

	if(IsInGameThread())
	{
		FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
		check(RigBlueprint);

		if(RigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications)
		{
			return;
		}
	}

	TWeakObjectPtr<URigHierarchy> WeakHierarchy = InHierarchy;
	auto Task = [this, InNotif, WeakHierarchy, Key, ComponentName]()
    {
		if(!WeakHierarchy.IsValid())
    	{
    		return;
    	}

		FRigBaseComponent* Component = nullptr;
        FRigBaseElement* Element = WeakHierarchy.Get()->Find(Key);
		if(Element)
		{
			Component = Element->FindComponent(ComponentName);
		}

		switch(InNotif)
		{
			case ERigHierarchyNotification::ElementSelected:
			case ERigHierarchyNotification::ElementDeselected:
			{
				if(Element)
				{
					const bool bSelected = InNotif == ERigHierarchyNotification::ElementSelected;

					if (Element->GetType() == ERigElementType::Bone)
					{
						SynchronizeViewportBoneSelection();
					}

					if (bSelected)
					{
						SetDetailViewForRigElements();
					}
					else
					{
						TArray<FRigElementKey> CurrentSelection = GetHierarchyBeingDebugged()->GetSelectedKeys();
						if (CurrentSelection.Num() > 0)
						{
							if(FRigBaseElement* LastSelectedElement = WeakHierarchy.Get()->Find(CurrentSelection.Last()))
							{
								OnHierarchyModified(ERigHierarchyNotification::ElementSelected,  WeakHierarchy.Get(), LastSelectedElement);
							}
						}
						else
						{
							// only clear the details if we are not looking at a transient control
							if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
							{
								if(DebuggedControlRig->RigUnitManipulationInfos.IsEmpty())
								{
									ClearDetailObject();
								}
							}
						}
					}
				}
				break;
			}
			case ERigHierarchyNotification::ElementAdded:
			case ERigHierarchyNotification::ElementRemoved:
			case ERigHierarchyNotification::ElementRenamed:
			{
				if (Key.IsValid() && Key.Type == ERigElementType::Connector)
				{
					FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
					check(RigBlueprint);
					RigBlueprint->UpdateExposedModuleConnectors();
					RigBlueprint->RecompileModularRig();
				}
				// Fallthrough to next case
			}
			case ERigHierarchyNotification::ParentChanged:
            case ERigHierarchyNotification::HierarchyReset:
			{
				CacheNameLists();
				break;
			}
			case ERigHierarchyNotification::ControlSettingChanged:
			{
				if(DetailViewShowsRigElement(Key))
				{
					FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
					check(RigBlueprint);

					const FRigControlElement* SourceControlElement = Cast<FRigControlElement>(Element);
					FRigControlElement* TargetControlElement = RigBlueprint->GetHierarchy()->Find<FRigControlElement>(Key);

					if(SourceControlElement && TargetControlElement)
					{
						TargetControlElement->Settings = SourceControlElement->Settings;
					}
				}
				break;
			}
			case ERigHierarchyNotification::ControlShapeTransformChanged:
			{
				if(DetailViewShowsRigElement(Key))
				{
					FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
					check(RigBlueprint);

					FRigControlElement* SourceControlElement = Cast<FRigControlElement>(Element);
					if(SourceControlElement)
					{
						FTransform InitialShapeTransform = WeakHierarchy.Get()->GetControlShapeTransform(SourceControlElement, ERigTransformType::InitialLocal);

						// set current shape transform = initial shape transform so that the viewport reflects this change
						WeakHierarchy.Get()->SetControlShapeTransform(SourceControlElement, InitialShapeTransform, ERigTransformType::CurrentLocal, false); 

						RigBlueprint->GetHierarchy()->SetControlShapeTransform(Key, WeakHierarchy.Get()->GetControlShapeTransform(SourceControlElement, ERigTransformType::InitialLocal), true);
						RigBlueprint->GetHierarchy()->SetControlShapeTransform(Key, WeakHierarchy.Get()->GetControlShapeTransform(SourceControlElement, ERigTransformType::CurrentLocal), false);

						RigBlueprint->Modify();
						RigBlueprint.GetObject()->MarkPackageDirty();
					}
				}
				break;
			}
			case ERigHierarchyNotification::ConnectorSettingChanged:
			{
				FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
				check(RigBlueprint);
				RigBlueprint->UpdateExposedModuleConnectors();
				RigBlueprint->RecompileModularRig();
				break;
			}
			default:
			{
				break;
			}
		}
		
    };

	if(IsInGameThread())
	{
		Task();
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [Task]()
		{
			Task();
		});
	}
}

void FControlRigBaseEditor::HandleRigTypeChanged(FControlRigAssetInterfacePtr InBlueprint)
{
	// todo: fire a notification.
	// todo: reapply the preview mesh and react to it accordingly.

	Compile();
}

void FControlRigBaseEditor::HandleModularRigModified(EModularRigNotification InNotification, const FRigModuleReference* InModule)
{
	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	if(RigBlueprint == nullptr)
	{
		return;
	}
	
	UModularRigController* ModularRigController = RigBlueprint->GetModularRigController();
	if(ModularRigController == nullptr)
	{
		return;
	}

	switch(InNotification)
	{
		case EModularRigNotification::ModuleAdded:
		{
			ModularRigController->SelectModule(InModule->Name);
			break;
		}
		case EModularRigNotification::ModuleRemoved:
		{
			if (DetailViewShowsAnyRigModule())
			{
				ClearDetailObject();
			}

			// todo: update SchematicGraph
			break;
		}
		case EModularRigNotification::ModuleReparented:
		case EModularRigNotification::ModuleReordered:
		case EModularRigNotification::ModuleRenamed:
		{
			break;
		}
		case EModularRigNotification::ConnectionChanged:
		{
			// todo: update SchematicGraph
			break;
		}
		case EModularRigNotification::ModuleSelected:
		case EModularRigNotification::ModuleDeselected:
		{
			ModulesSelected = ModularRigController->GetSelectedModules();
			SetDetailViewForRigModules(ModulesSelected);
			break;
		}
	}
}

void FControlRigBaseEditor::HandlePostCompileModularRigs(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	RefreshDetailView();
}

void FControlRigBaseEditor::SwapModuleWithinAsset()
{
	const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface();
	const FAssetData Asset = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(GetRigVMAssetInterface()->GetObject()->GetPathName(), true);
	SRigVMSwapAssetReferencesWidget::FArguments WidgetArgs;

	FRigVMAssetDataFilter FilterModules = FRigVMAssetDataFilter::CreateLambda([](const FAssetData& AssetData)
	{
		return IControlRigEditorAssetInterface::GetRigType(AssetData) == EControlRigType::RigModule;
	});
	FRigVMAssetDataFilter FilterSourceModules = FRigVMAssetDataFilter::CreateLambda([Blueprint](const FAssetData& AssetData)
	{
		if (Blueprint)
		{
			return !Blueprint->GetModularRigModel().FindModuleInstancesOfClass(AssetData).IsEmpty();
		}
		return false;
	});

	TArray<FRigVMAssetDataFilter> SourceFilters = {FilterModules, FilterSourceModules};
	TArray<FRigVMAssetDataFilter> TargetFilters = {FilterModules};
	
	WidgetArgs
		.EnableUndo(true)
		.CloseOnSuccess(true)
		.OnGetReferences_Lambda([Blueprint, Asset](const FAssetData& InReferencedAsset) -> TArray<FSoftObjectPath>
		{
			TArray<FSoftObjectPath> Result;
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			IAssetRegistry& AssetRegistry = AssetRegistryModule.GetRegistry();

			FControlRigAssetStrongReference ReferencedAsset;
			if (FControlRigAssetInterfacePtr ReferencedBlueprint = InReferencedAsset.GetAsset())
			{
				ReferencedAsset = ReferencedBlueprint->GetControlRigAssetReference();
			}

			if (Blueprint)
			{
				if (Blueprint->IsModularRig())
				{
					TArray<const FRigModuleReference*> Modules = Blueprint->GetModularRigModel().FindModuleInstancesOfClass(ReferencedAsset.GetEditorAsset());
					for (const FRigModuleReference* Module : Modules)
					{
						FSoftObjectPath ModulePath = Asset.GetSoftObjectPath();
						ModulePath.SetSubPathString(Module->GetModulePath().GetPath());
						Result.Add(ModulePath);
					}
				}
			}
			
			return Result;
		})
		.OnSwapReference_Lambda([](const FSoftObjectPath& ModulePath, const FAssetData& NewModuleAsset) -> bool
		{
			FControlRigAssetStrongReference NewModuleClass;
			if (const FControlRigAssetInterfacePtr ModuleBlueprint = NewModuleAsset.GetAsset())
			{
				NewModuleClass = ModuleBlueprint->GetControlRigAssetReference();
			}
			else if (UControlRigBlueprintGeneratedClass* GeneratedClass = Cast<UControlRigBlueprintGeneratedClass>(NewModuleAsset.GetAsset()))
			{
				NewModuleClass.Set(TSubclassOf<UControlRig>(GeneratedClass));
			}
			else if (UControlRigRuntimeAsset* RuntimeAsset = Cast<UControlRigRuntimeAsset>(NewModuleAsset.GetAsset()))
			{
				NewModuleClass.Set(RuntimeAsset);
			}
			if (NewModuleClass.IsValid())
			{
				if (FControlRigAssetInterfacePtr RigBlueprint = ModulePath.GetWithoutSubPath().ResolveObject())
				{
					if(const FRigModuleReference* Module = RigBlueprint->GetModularRigModel().FindModuleByPath(ModulePath.GetSubPathString()))
					{
						return RigBlueprint->GetModularRigController()->SwapModuleSource(Module->Name, NewModuleClass);
					}
				}
			}
			return false;
		})
		.SourceAssetFilters(SourceFilters)
		.TargetAssetFilters(TargetFilters);

	const TSharedRef<SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>> SwapModulesDialog =
		SNew(SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>)
		.WindowSize(FVector2D(800.0f, 640.0f))
		.WidgetArgs(WidgetArgs);
	
	SwapModulesDialog->ShowNormal();
}

void FControlRigBaseEditor::SwapModuleAcrossProject()
{
	const FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface();
	const FAssetData Asset = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(GetRigVMAssetInterface()->GetObject()->GetPathName(), true);
	SRigVMSwapAssetReferencesWidget::FArguments WidgetArgs;

	FRigVMAssetDataFilter FilterModules = FRigVMAssetDataFilter::CreateLambda([](const FAssetData& AssetData)
	{
		return IControlRigEditorAssetInterface::GetRigType(AssetData) == EControlRigType::RigModule;
	});

	TArray<FRigVMAssetDataFilter> TargetFilters = {FilterModules};
	
	WidgetArgs
		.EnableUndo(false)
		.CloseOnSuccess(true)
		.OnGetReferences_Lambda([Blueprint, Asset](const FAssetData& ReferencedAsset) -> TArray<FSoftObjectPath>
		{
			return IControlRigEditorAssetInterface::GetReferencesToRigModule(Asset);
		})
		.OnSwapReference_Lambda([](const FSoftObjectPath& ModulePath, const FAssetData& NewModuleAsset) -> bool
		{
			FControlRigAssetStrongReference NewModuleClass;
			if (const FControlRigAssetInterfacePtr ModuleBlueprint = NewModuleAsset.GetAsset())
			{
				NewModuleClass = ModuleBlueprint->GetControlRigAssetReference();
			}
			else if (UControlRigBlueprintGeneratedClass* GeneratedClass = Cast<UControlRigBlueprintGeneratedClass>(NewModuleAsset.GetAsset()))
			{
				NewModuleClass.Set(TSubclassOf<UControlRig>(GeneratedClass));
			}
			else if (UControlRigRuntimeAsset* RuntimeAsset = Cast<UControlRigRuntimeAsset>(NewModuleAsset.GetAsset()))
			{
				NewModuleClass.Set(RuntimeAsset);
			}
			if (NewModuleClass.IsValid())
			{
				if (FControlRigAssetInterfacePtr RigBlueprint = ModulePath.GetWithoutSubPath().ResolveObject())
				{
					return RigBlueprint->GetModularRigController()->SwapModuleSource(*ModulePath.GetSubPathString(), NewModuleClass);
				}
			}
			return false;
		})
		.Source(Asset)
		.TargetAssetFilters(TargetFilters);

	const TSharedRef<SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>> SwapModulesDialog =
		SNew(SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>)
		.WindowSize(FVector2D(800.0f, 640.0f))
		.WidgetArgs(WidgetArgs);
	
	SwapModulesDialog->ShowNormal();
}

void FControlRigBaseEditor::SynchronizeViewportBoneSelection()
{
	FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface();
	if (RigBlueprint == nullptr)
	{
		return;
	}

	UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
	if (EditorSkelComp)
	{
		EditorSkelComp->BonesOfInterest.Reset();

		TArray<const FRigBaseElement*> SelectedBones = GetHierarchyBeingDebugged()->GetSelectedElements(ERigElementType::Bone);
		for (const FRigBaseElement* SelectedBone : SelectedBones)
		{
 			const int32 BoneIndex = EditorSkelComp->GetReferenceSkeleton().FindBoneIndex(SelectedBone->GetFName());
			if(BoneIndex != INDEX_NONE)
			{
				EditorSkelComp->BonesOfInterest.AddUnique(BoneIndex);
			}
		}
	}
}

void FControlRigBaseEditor::UpdateBoneModification(FName BoneName, const FTransform& LocalTransform)
{
	if (UControlRig* ControlRig = GetControlRig())
	{ 
		if (PreviewInstance)
		{ 
			if (FAnimNode_ModifyBone* Modify = PreviewInstance->FindModifiedBone(BoneName))
			{
				Modify->Translation = LocalTransform.GetTranslation();
				Modify->Rotation = LocalTransform.GetRotation().Rotator();
				Modify->TranslationSpace = EBoneControlSpace::BCS_ParentBoneSpace;
				Modify->RotationSpace = EBoneControlSpace::BCS_ParentBoneSpace; 
			}
		}
		
		TMap<FName, FTransform>* TransformOverrideMap = &ControlRig->TransformOverrideForUserCreatedBones;
		if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
		{
			TransformOverrideMap = &DebuggedControlRig->TransformOverrideForUserCreatedBones;
		}

		if (FTransform* Transform = TransformOverrideMap->Find(BoneName))
		{
			*Transform = LocalTransform;
		}
	}
}

void FControlRigBaseEditor::RemoveBoneModification(FName BoneName)
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		if (PreviewInstance)
		{
			PreviewInstance->RemoveBoneModification(BoneName);
		}

		TMap<FName, FTransform>* TransformOverrideMap = &ControlRig->TransformOverrideForUserCreatedBones;
		if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
		{
			TransformOverrideMap = &DebuggedControlRig->TransformOverrideForUserCreatedBones;
		}

		TransformOverrideMap->Remove(BoneName);
	}
}

void FControlRigBaseEditor::ResetAllBoneModification()
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		if (IsValid(PreviewInstance))
		{
			PreviewInstance->ResetModifiedBone();
		}

		TMap<FName, FTransform>* TransformOverrideMap = &ControlRig->TransformOverrideForUserCreatedBones;
		if (UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
		{
			TransformOverrideMap = &DebuggedControlRig->TransformOverrideForUserCreatedBones;
		}

		TransformOverrideMap->Reset();
	}
}

FControlRigEditorEditMode* FControlRigBaseEditor::GetEditMode() const
{
	if(IsModularRig())
	{
		return static_cast<FModularRigEditorEditMode*>(GetEditorModeManagerImpl().GetActiveMode(GetEditorModeNameImpl()));
	}
	return static_cast<FControlRigEditorEditMode*>(GetEditorModeManagerImpl().GetActiveMode(GetEditorModeNameImpl()));
}


void FControlRigBaseEditor::OnCurveContainerChanged()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	ClearDetailObject();

	GetControlRigAssetInterface()->GetRigVMAssetInterface()->MarkAssetAsModified();
	
	UControlRigSkeletalMeshComponent* EditorSkelComp = Cast<UControlRigSkeletalMeshComponent>(GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent());
	if (EditorSkelComp)
	{
		// restart animation 
		EditorSkelComp->InitAnim(true);
		UpdateRigVMHost();
	}
	CacheNameLists();

	// notification
	FNotificationInfo Info(LOCTEXT("CurveContainerChangeHelpMessage", "CurveContainer has been successfully modified."));
	Info.bFireAndForget = true;
	Info.FadeOutDuration = 5.0f;
	Info.ExpireDuration = 5.0f;

	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);
}

void FControlRigBaseEditor::CreateRigHierarchyToGraphDragAndDropMenu() const
{
	const FName MenuName = RigHierarchyToGraphDragAndDropMenuName;
	UToolMenus* ToolMenus = UToolMenus::Get();
	
	if(!ensure(ToolMenus))
	{
		return;
	}

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName);

		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UControlRigContextMenuContext* MainContext = InMenu->FindContext<UControlRigContextMenuContext>();
				
				if (IControlRigBaseEditor* ControlRigEditor = MainContext->GetControlRigEditor())
				{
					const FControlRigRigHierarchyToGraphDragAndDropContext& DragDropContext = MainContext->GetRigHierarchyToGraphDragAndDropContext();

					URigHierarchy* Hierarchy = ControlRigEditor->GetHierarchyBeingDebugged();
					TArray<FRigElementKey> DraggedElements;
					TArray<FRigComponentKey> DraggedComponents;
					for(const FRigHierarchyKey& DraggedHierarchyKey : DragDropContext.DraggedHierarchyKeys)
					{
						if(DraggedHierarchyKey.IsElement())
						{
							DraggedElements.Add(DraggedHierarchyKey.GetElement());
						}
						if(DraggedHierarchyKey.IsComponent())
						{
							DraggedComponents.Add(DraggedHierarchyKey.GetComponent());
						}
					}
					
					UEdGraph* Graph = DragDropContext.Graph.Get();
					const FVector2D& NodePosition = DragDropContext.NodePosition;

					if(DraggedComponents.Num() > 0)
					{
						const FText SectionText = FText::FromString(DragDropContext.GetSectionTitle());
						FToolMenuSection& Section = InMenu->AddSection(NAME_None, SectionText);

						FText GetterLabel = LOCTEXT("GetComponent","Get Component");
						FText GetterTooltip = LOCTEXT("GetComponent_ToolTip", "Getter For Component");
						FText SetterLabel = LOCTEXT("SetComponent","Set Component");
						FText SetterTooltip = LOCTEXT("SetComponent_ToolTip", "Setter For Component");

						FToolMenuEntry GetComponentsEntry = FToolMenuEntry::InitMenuEntry(
							TEXT("GetComponents"),
							GetterLabel,
							GetterTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeComponentGetterSetter, true, DraggedComponents, Graph, NodePosition),
								FCanExecuteAction()
							)
						);
						GetComponentsEntry.InsertPosition.Name = NAME_None;
						GetComponentsEntry.InsertPosition.Position = EToolMenuInsertType::First;

						Section.AddEntry(GetComponentsEntry);

						FToolMenuEntry SetComponentsEntry = FToolMenuEntry::InitMenuEntry(
							TEXT("SetComponents"),
							SetterLabel,
							SetterTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeComponentGetterSetter, false, DraggedComponents, Graph, NodePosition),
								FCanExecuteAction()
							)
						);
						SetComponentsEntry.InsertPosition.Name = GetComponentsEntry.Name;
						SetComponentsEntry.InsertPosition.Position = EToolMenuInsertType::After;	

						Section.AddEntry(SetComponentsEntry);
					}

					if(DraggedElements.Num() > 0)
					{
						ControlRigEditor->FilterDraggedKeys(DraggedElements, true);
					
						// if multiple types are selected, we show Get Elements/Set Elements
						bool bMultipleTypeSelected = false;

						ERigElementType LastType = DraggedElements[0].Type;
				
						uint8 DraggedTypes = 0;
						uint8 DraggedAnimationTypes = 2;
						for (const FRigElementKey& DraggedKey : DraggedElements)
						{
							if (DraggedKey.Type != LastType)
							{
								bMultipleTypeSelected = true;
							}
							else if(DraggedKey.Type == ERigElementType::Control)
							{
								if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(DraggedKey))
								{
									const uint8 DraggedAnimationType = ControlElement->IsAnimationChannel() ? 1 : 0; 
									if(DraggedAnimationTypes == 2)
									{
										DraggedAnimationTypes = DraggedAnimationType;
									}
									else
									{
										if(DraggedAnimationTypes != DraggedAnimationType)
										{
											bMultipleTypeSelected = true;
										}
									}
								}
							}
					
							DraggedTypes = DraggedTypes | (uint8)DraggedKey.Type;
						}
						
						const FText SectionText = FText::FromString(DragDropContext.GetSectionTitle());
						FToolMenuSection& Section = InMenu->AddSection(NAME_None, SectionText);

						FText GetterLabel = LOCTEXT("GetElement","Get Element");
						FText GetterTooltip = LOCTEXT("GetElement_ToolTip", "Getter For Element");
						FText SetterLabel = LOCTEXT("SetElement","Set Element");
						FText SetterTooltip = LOCTEXT("SetElement_ToolTip", "Setter For Element");
						// if multiple types are selected, we show Get Elements/Set Elements
						if (bMultipleTypeSelected)
						{
							GetterLabel = LOCTEXT("GetElements","Get Elements");
							GetterTooltip = LOCTEXT("GetElements_ToolTip", "Getter For Elements");
							SetterLabel = LOCTEXT("SetElements","Set Elements");
							SetterTooltip = LOCTEXT("SetElements_ToolTip", "Setter For Elements");
						}
						else
						{
							// otherwise, we show "Get Bone/NUll/Control"
							if ((DraggedTypes & (uint8)ERigElementType::Bone) != 0)
							{
								GetterLabel = LOCTEXT("GetBone","Get Bone");
								GetterTooltip = LOCTEXT("GetBone_ToolTip", "Getter For Bone");
								SetterLabel = LOCTEXT("SetBone","Set Bone");
								SetterTooltip = LOCTEXT("SetBone_ToolTip", "Setter For Bone");
							}
							else if ((DraggedTypes & (uint8)ERigElementType::Null) != 0)
							{
								GetterLabel = LOCTEXT("GetNull","Get Null");
								GetterTooltip = LOCTEXT("GetNull_ToolTip", "Getter For Null");
								SetterLabel = LOCTEXT("SetNull","Set Null");
								SetterTooltip = LOCTEXT("SetNull_ToolTip", "Setter For Null");
							}
							else if ((DraggedTypes & (uint8)ERigElementType::Control) != 0)
							{
								if(DraggedAnimationTypes == 0)
								{
									GetterLabel = LOCTEXT("GetControl","Get Control");
									GetterTooltip = LOCTEXT("GetControl_ToolTip", "Getter For Control");
									SetterLabel = LOCTEXT("SetControl","Set Control");
									SetterTooltip = LOCTEXT("SetControl_ToolTip", "Setter For Control");
								}
								else
								{
									GetterLabel = LOCTEXT("GetAnimationChannel","Get Animation Channel");
									GetterTooltip = LOCTEXT("GetAnimationChannel_ToolTip", "Getter For Animation Channel");
									SetterLabel = LOCTEXT("SetAnimationChannel","Set Animation Channel");
									SetterTooltip = LOCTEXT("SetAnimationChannel_ToolTip", "Setter For Animation Channel");
								}
							}
							else if ((DraggedTypes & (uint8)ERigElementType::Connector) != 0)
							{
								GetterLabel = LOCTEXT("GetConnector","Get Connector");
								GetterTooltip = LOCTEXT("GetConnector_ToolTip", "Getter For Connector");
								SetterLabel = LOCTEXT("SetConnector","Set Connector");
								SetterTooltip = LOCTEXT("SetConnector_ToolTip", "Setter For Connector");
							}
						}

						FToolMenuEntry GetElementsEntry = FToolMenuEntry::InitMenuEntry(
							TEXT("GetElements"),
							GetterLabel,
							GetterTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Transform, true, DraggedElements, Graph, NodePosition),
								FCanExecuteAction()
							)
						);
						GetElementsEntry.InsertPosition.Name = NAME_None;
						GetElementsEntry.InsertPosition.Position = EToolMenuInsertType::First;
						

						Section.AddEntry(GetElementsEntry);

						FToolMenuEntry SetElementsEntry = FToolMenuEntry::InitMenuEntry(
							TEXT("SetElements"),
							SetterLabel,
							SetterTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Transform, false, DraggedElements, Graph, NodePosition),
								FCanExecuteAction()
							)
						);
						SetElementsEntry.InsertPosition.Name = GetElementsEntry.Name;
						SetElementsEntry.InsertPosition.Position = EToolMenuInsertType::After;	

						Section.AddEntry(SetElementsEntry);

						if (((DraggedTypes & (uint8)ERigElementType::Bone) != 0) ||
							((DraggedTypes & (uint8)ERigElementType::Control) != 0) ||
							((DraggedTypes & (uint8)ERigElementType::Null) != 0) ||
							((DraggedTypes & (uint8)ERigElementType::Connector) != 0))
						{
							FToolMenuEntry& RotationTranslationSeparator = Section.AddSeparator(TEXT("RotationTranslationSeparator"));
							RotationTranslationSeparator.InsertPosition.Name = SetElementsEntry.Name;
							RotationTranslationSeparator.InsertPosition.Position = EToolMenuInsertType::After;

							FToolMenuEntry SetRotationEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("SetRotation"),
								LOCTEXT("SetRotation","Set Rotation"),
								LOCTEXT("SetRotation_ToolTip","Setter for Rotation"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Rotation, false, DraggedElements, Graph, NodePosition),
									FCanExecuteAction()
								)
							);
							
							SetRotationEntry.InsertPosition.Name = RotationTranslationSeparator.Name;
							SetRotationEntry.InsertPosition.Position = EToolMenuInsertType::After;		
							Section.AddEntry(SetRotationEntry);

							FToolMenuEntry SetTranslationEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("SetTranslation"),
								LOCTEXT("SetTranslation","Set Translation"),
								LOCTEXT("SetTranslation_ToolTip","Setter for Translation"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Translation, false, DraggedElements, Graph, NodePosition),
									FCanExecuteAction()
								)
							);

							SetTranslationEntry.InsertPosition.Name = SetRotationEntry.Name;
							SetTranslationEntry.InsertPosition.Position = EToolMenuInsertType::After;		
							Section.AddEntry(SetTranslationEntry);

							FToolMenuEntry AddOffsetEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("AddOffset"),
								LOCTEXT("AddOffset","Add Offset"),
								LOCTEXT("AddOffset_ToolTip","Setter for Offset"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Offset, false, DraggedElements, Graph, NodePosition),
									FCanExecuteAction()
								)
							);
							
							AddOffsetEntry.InsertPosition.Name = SetTranslationEntry.Name;
							AddOffsetEntry.InsertPosition.Position = EToolMenuInsertType::After;						
							Section.AddEntry(AddOffsetEntry);

							FToolMenuEntry& RelativeTransformSeparator = Section.AddSeparator(TEXT("RelativeTransformSeparator"));
							RelativeTransformSeparator.InsertPosition.Name = AddOffsetEntry.Name;
							RelativeTransformSeparator.InsertPosition.Position = EToolMenuInsertType::After;
							
							FToolMenuEntry GetRelativeTransformEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("GetRelativeTransformEntry"),
								LOCTEXT("GetRelativeTransform", "Get Relative Transform"),
								LOCTEXT("GetRelativeTransform_ToolTip", "Getter for Relative Transform"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Relative, true, DraggedElements, Graph, NodePosition),
									FCanExecuteAction()
								)
							);
							
							GetRelativeTransformEntry.InsertPosition.Name = RelativeTransformSeparator.Name;
							GetRelativeTransformEntry.InsertPosition.Position = EToolMenuInsertType::After;	
							Section.AddEntry(GetRelativeTransformEntry);
							
							FToolMenuEntry SetRelativeTransformEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("SetRelativeTransformEntry"),
								LOCTEXT("SetRelativeTransform", "Set Relative Transform"),
								LOCTEXT("SetRelativeTransform_ToolTip", "Setter for Relative Transform"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(ControlRigEditor->SharedRef(), &FControlRigBaseEditor::HandleMakeElementGetterSetter, ERigElementGetterSetterType_Relative, false, DraggedElements, Graph, NodePosition),
									FCanExecuteAction()
								)
							);
							SetRelativeTransformEntry.InsertPosition.Name = GetRelativeTransformEntry.Name;
							SetRelativeTransformEntry.InsertPosition.Position = EToolMenuInsertType::After;		
							Section.AddEntry(SetRelativeTransformEntry);
						}

						if (Hierarchy != nullptr)
						{
							FToolMenuEntry& ItemArraySeparator = Section.AddSeparator(TEXT("ItemArraySeparator"));
							ItemArraySeparator.InsertPosition.Name = TEXT("SetRelativeTransformEntry"),
							ItemArraySeparator.InsertPosition.Position = EToolMenuInsertType::After;
							
							FToolMenuEntry CreateItemArrayEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("CreateItemArray"),
								LOCTEXT("CreateItemArray", "Create Item Array"),
								LOCTEXT("CreateItemArray_ToolTip", "Creates an item array from the selected elements in the hierarchy"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([ControlRigEditor, DraggedElements, NodePosition]()
										{
											if (URigVMController* Controller = ControlRigEditor->GetFocusedController())
											{
												Controller->OpenUndoBracket(TEXT("Create Item Array From Selection"));

												const FRigVMDispatchFactory* ConstantFactory = FRigVMRegistry::Get().FindOrAddDispatchFactory(FRigVMDispatch_Constant::StaticStruct());
												check(ConstantFactory);
												
												if (URigVMNode* ItemsNode = Controller->AddTemplateNode(ConstantFactory->GetTemplateNotation(), NodePosition, TEXT("ItemArray"), true, true))
												{
													if (URigVMPin* ItemsPin = ItemsNode->FindPin(TEXT("Value")))
													{
														const TRigVMTypeIndex RigElementKeyType = FRigVMRegistry::Get().FindOrAddType(FRigElementKey::StaticStruct());
														const TRigVMTypeIndex RigElementKeyArrayType = FRigVMRegistry::Get().GetArrayTypeFromBaseTypeIndex(RigElementKeyType);
														Controller->ResolveWildCardPin(ItemsPin->GetPinPath(), RigElementKeyArrayType, true, true);
														Controller->SetArrayPinSize(ItemsPin->GetPinPath(), DraggedElements.Num());

														TArray<URigVMPin*> ItemPins = ItemsPin->GetSubPins();
														if(ensure(ItemPins.Num() == DraggedElements.Num()))
														{
															for (int32 ItemIndex = 0; ItemIndex < DraggedElements.Num(); ItemIndex++)
															{
																FString DefaultValue;
																FRigElementKey::StaticStruct()->ExportText(DefaultValue, &DraggedElements[ItemIndex], nullptr, nullptr, PPF_None, nullptr);
																Controller->SetPinDefaultValue(ItemPins[ItemIndex]->GetPinPath(), DefaultValue, true, true, false, true);
																Controller->SetPinExpansion(ItemPins[ItemIndex]->GetPinPath(), true, true, true);
															}
															Controller->SetPinExpansion(ItemsPin->GetPinPath(), true, true, true);
														}

														Controller->SelectNodeByName(ItemsNode->GetFName());
													}

													if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->GetEdGraph(ItemsNode->GetGraph())))
													{
														if(URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(EdGraph->FindNodeForModelNodeName(ItemsNode->GetFName())))
														{
															EdGraphNode->RequestRename();
														}
													}
												}

												Controller->CloseUndoBracket();
											}
										}
									)
								)
							);
							
							CreateItemArrayEntry.InsertPosition.Name = ItemArraySeparator.Name,
							CreateItemArrayEntry.InsertPosition.Position = EToolMenuInsertType::After;
							Section.AddEntry(CreateItemArrayEntry);
						}

						// Meta data 
						{
							FToolMenuEntry& MetadataSeparator = Section.AddSeparator(TEXT("MetadataSeparator"));
							MetadataSeparator.InsertPosition.Name = TEXT("CreateItemArray"),
							MetadataSeparator.InsertPosition.Position = EToolMenuInsertType::After;

							constexpr bool bGetMetadataEntry_IsGetter = true;
							FToolMenuEntry GetMetadataEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("GetMetadataEntry"),
								LOCTEXT("GetMetadata", "Get Metadata"),
								LOCTEXT("GetMetadata_ToolTip", "Getter for Metadata"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(
										ControlRigEditor->SharedRef(),
										&FControlRigBaseEditor::HandleMakeMetadataGetterSetter,
										bGetMetadataEntry_IsGetter, 
										DraggedElements,
										Graph, 
										NodePosition),
									FCanExecuteAction()
								)
							);

							GetMetadataEntry.InsertPosition.Name = MetadataSeparator.Name;
							GetMetadataEntry.InsertPosition.Position = EToolMenuInsertType::After;
							Section.AddEntry(GetMetadataEntry);

							constexpr bool bSetMetadataEntry_IsGetter = false;
							FToolMenuEntry SetMetadataEntry = FToolMenuEntry::InitMenuEntry(
								TEXT("SetMetadataEntry"),
								LOCTEXT("SetMetadata", "Set Metadata"),
								LOCTEXT("SetMetadata_ToolTip", "Setter for Metadata"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(
										ControlRigEditor->SharedRef(),
										&FControlRigBaseEditor::HandleMakeMetadataGetterSetter,
										bSetMetadataEntry_IsGetter,
										DraggedElements,
										Graph,
										NodePosition),
									FCanExecuteAction()
								)
							);

							SetMetadataEntry.InsertPosition.Name = GetMetadataEntry.Name;
							SetMetadataEntry.InsertPosition.Position = EToolMenuInsertType::After;
							Section.AddEntry(SetMetadataEntry);
						}
					}
				}
			})
		);
	}
}

void FControlRigBaseEditor::OnGraphNodeDropToPerformImpl(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InDragDropOp->IsOfType<FRigHierarchyElementDragDropOp>())
	{
		TSharedPtr<FRigHierarchyElementDragDropOp> RigHierarchyOp = StaticCastSharedPtr<FRigHierarchyElementDragDropOp>(InDragDropOp);

		if (RigHierarchyOp->GetElements().Num() > 0 && GetFocusedGraphEd().IsValid())
		{
			const FName MenuName = RigHierarchyToGraphDragAndDropMenuName;
			
			UControlRigContextMenuContext* MenuContext = NewObject<UControlRigContextMenuContext>();
			FControlRigMenuSpecificContext MenuSpecificContext;
			MenuSpecificContext.RigHierarchyToGraphDragAndDropContext =
				FControlRigRigHierarchyToGraphDragAndDropContext(
					RigHierarchyOp->GetElements(),
					InGraph,
					FDeprecateSlateVector2D(InNodePosition)
				);
			MenuContext->Init(StaticCastSharedRef<FControlRigBaseEditor>(SharedControlRigEditorRef()), MenuSpecificContext);
			
			UToolMenus* ToolMenus = UToolMenus::Get();
			TSharedRef<SWidget>	MenuWidget = ToolMenus->GenerateWidget(MenuName, FToolMenuContext(MenuContext));
			
			TSharedRef<SWidget> GraphEditorPanel = GetFocusedGraphEd().Pin().ToSharedRef();

			// Show menu to choose getter vs setter
			FSlateApplication::Get().PushMenu(
				GraphEditorPanel,
				FWidgetPath(),
				MenuWidget,
				InScreenPosition,
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

		}
		return;
	}

	OnGraphNodeDropToPerformSuper(InDragDropOp, InGraph, InNodePosition, InScreenPosition);
}

void FControlRigBaseEditor::HandleMakeElementGetterSetter(ERigElementGetterSetterType Type, bool bIsGetter, TArray<FRigElementKey> Keys, UEdGraph* Graph, FVector2D NodePosition)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Keys.Num() == 0)
	{
		return;
	}

	URigHierarchy* Hierarchy = GetHierarchyBeingDebugged();
	if (Hierarchy == nullptr)
	{
		return;
	}
	if (GetFocusedController() == nullptr)
	{
		return;
	}

	GetFocusedController()->OpenUndoBracket(TEXT("Adding Nodes from Hierarchy"));

	struct FNewNodeData
	{
		FName Name;
		FName ValuePinName;
		ERigControlType ValueType;
		FRigControlValue Value;
	};
	TArray<FNewNodeData> NewNodes;

	TArray<FRigElementKey> KeysIncludingNameSpace = Keys;
	FilterDraggedKeys(KeysIncludingNameSpace, false);

	for (int32 Index = 0; Index < Keys.Num(); Index++)
	{
		const FRigElementKey& Key = Keys[Index];
		const FRigElementKey& KeyIncludingNameSpace = KeysIncludingNameSpace[Index];
		
		UScriptStruct* StructTemplate = nullptr;

		FNewNodeData NewNode;
		NewNode.Name = NAME_None;
		NewNode.ValuePinName = NAME_None;

		TArray<FName> ItemPins;
		ItemPins.Add(TEXT("Item"));

		FName NameValue = Key.Name;
		FName ChannelValue = Key.Name;
		TArray<FName> NamePins;
		TArray<FName> ChannelPins;
		TMap<FName, int32> PinsToResolve; 

		if(FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(KeyIncludingNameSpace))
		{
			if(ControlElement->IsAnimationChannel())
			{
				ChannelValue = ControlElement->GetDisplayName();
				
				if(const FRigControlElement* ParentControlElement = Cast<FRigControlElement>(Hierarchy->GetFirstParent(ControlElement)))
				{
					NameValue = ParentControlElement->GetFName();
				}
				else
				{
					NameValue = NAME_None;
				}

				ItemPins.Reset();
				NamePins.Add(TEXT("Control"));
				ChannelPins.Add(TEXT("Channel"));
				static const FName ValueName = GET_MEMBER_NAME_CHECKED(FRigUnit_GetBoolAnimationChannel, Value);

				switch (ControlElement->Settings.ControlType)
				{
					case ERigControlType::Bool:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetBoolAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetBoolAnimationChannel::StaticStruct();
						}
						PinsToResolve.Add(ValueName, RigVMTypeUtils::TypeIndex::Bool);
						break;
					}
					case ERigControlType::Float:
					case ERigControlType::ScaleFloat:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetFloatAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetFloatAnimationChannel::StaticStruct();
						}
						PinsToResolve.Add(ValueName, RigVMTypeUtils::TypeIndex::Float);
						break;
					}
					case ERigControlType::Integer:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetIntAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetIntAnimationChannel::StaticStruct();
						}
						PinsToResolve.Add(ValueName, RigVMTypeUtils::TypeIndex::Int32);
						break;
					}
					case ERigControlType::Vector2D:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetVector2DAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetVector2DAnimationChannel::StaticStruct();
						}

						UScriptStruct* ValueStruct = TBaseStructure<FVector2D>::Get();
						const FRigVMTemplateArgumentType TypeForStruct(*RigVMTypeUtils::GetUniqueStructTypeName(ValueStruct), ValueStruct);
						const int32 TypeIndex = FRigVMRegistry::Get().GetTypeIndex(TypeForStruct);
						PinsToResolve.Add(ValueName, TypeIndex);
						break;
					}
					case ERigControlType::Position:
					case ERigControlType::Scale:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetVectorAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetVectorAnimationChannel::StaticStruct();
						}
						UScriptStruct* ValueStruct = TBaseStructure<FVector>::Get();
						const FRigVMTemplateArgumentType TypeForStruct(*RigVMTypeUtils::GetUniqueStructTypeName(ValueStruct), ValueStruct);
						const int32 TypeIndex = FRigVMRegistry::Get().GetTypeIndex(TypeForStruct);
						PinsToResolve.Add(ValueName, TypeIndex);
						break;
					}
					case ERigControlType::Rotator:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetRotatorAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetRotatorAnimationChannel::StaticStruct();
						}
						UScriptStruct* ValueStruct = TBaseStructure<FRotator>::Get();
						const FRigVMTemplateArgumentType TypeForStruct(*ValueStruct->GetStructCPPName(), ValueStruct);
						const int32 TypeIndex = FRigVMRegistry::Get().GetTypeIndex(TypeForStruct);
						PinsToResolve.Add(ValueName, TypeIndex);
						break;
					}
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						if(bIsGetter)
						{
							StructTemplate = FRigUnit_GetTransformAnimationChannel::StaticStruct();
						}
						else
						{
							StructTemplate = FRigUnit_SetTransformAnimationChannel::StaticStruct();
						}
						UScriptStruct* ValueStruct = TBaseStructure<FTransform>::Get();
						const FRigVMTemplateArgumentType TypeForStruct(*RigVMTypeUtils::GetUniqueStructTypeName(ValueStruct), ValueStruct);
						const int32 TypeIndex = FRigVMRegistry::Get().GetTypeIndex(TypeForStruct);
						PinsToResolve.Add(ValueName, TypeIndex);
						break;
					}
					default:
					{
						break;
					}
				}
			}
		}

		if (bIsGetter && StructTemplate == nullptr)
		{
			switch (Type)
			{
				case ERigElementGetterSetterType_Transform:
				{
					if (Key.Type == ERigElementType::Control)
					{
						FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(KeyIncludingNameSpace);
						if(ControlElement == nullptr)
						{
							return;
						}
						
						switch (ControlElement->Settings.ControlType)
						{
							case ERigControlType::Bool:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_GetControlBool::StaticStruct();
								break;
							}
							case ERigControlType::Float:
							case ERigControlType::ScaleFloat:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_GetControlFloat::StaticStruct();
								break;
							}
							case ERigControlType::Integer:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_GetControlInteger::StaticStruct();
								break;
							}
							case ERigControlType::Vector2D:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_GetControlVector2D::StaticStruct();
								break;
							}
							case ERigControlType::Position:
							case ERigControlType::Scale:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_GetControlVector::StaticStruct();
								break;
							}
							case ERigControlType::Rotator:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_GetControlRotator::StaticStruct();
								break;
							}
							case ERigControlType::Transform:
							case ERigControlType::TransformNoScale:
							case ERigControlType::EulerTransform:
							{
								StructTemplate = FRigUnit_GetTransform::StaticStruct();
								break;
							}
							default:
							{
								break;
							}
						}
					}
					else
					{
						StructTemplate = FRigUnit_GetTransform::StaticStruct();
					}
					break;
				}
				case ERigElementGetterSetterType_Initial:
				{
					StructTemplate = FRigUnit_GetTransform::StaticStruct();
					break;
				}
				case ERigElementGetterSetterType_Relative:
				{
					StructTemplate = FRigUnit_GetRelativeTransformForItem::StaticStruct();
					ItemPins.Reset();
					ItemPins.Add(TEXT("Child"));
					ItemPins.Add(TEXT("Parent"));
					break;
				}
				default:
				{
					break;
				}
			}
		}
		else if(StructTemplate == nullptr)
		{
			switch (Type)
			{
				case ERigElementGetterSetterType_Transform:
				{
					if (Key.Type == ERigElementType::Control)
					{
						FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(KeyIncludingNameSpace);
						if(ControlElement == nullptr)
						{
							return;
						}

						switch (ControlElement->Settings.ControlType)
						{
							case ERigControlType::Bool:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlBool::StaticStruct();
								break;
							}
							case ERigControlType::Float:
							case ERigControlType::ScaleFloat:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlFloat::StaticStruct();
								break;
							}
							case ERigControlType::Integer:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlInteger::StaticStruct();
								break;
							}
							case ERigControlType::Vector2D:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlVector2D::StaticStruct();
								break;
							}
							case ERigControlType::Position:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlVector::StaticStruct();
								NewNode.ValuePinName = TEXT("Vector");
								NewNode.ValueType = ERigControlType::Position;
								NewNode.Value = FRigControlValue::Make<FVector>(Hierarchy->GetGlobalTransform(Key).GetLocation());
								break;
							}
							case ERigControlType::Scale:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlVector::StaticStruct();
								NewNode.ValuePinName = TEXT("Vector");
								NewNode.ValueType = ERigControlType::Scale;
								NewNode.Value = FRigControlValue::Make<FVector>(Hierarchy->GetGlobalTransform(Key).GetScale3D());
								break;
							}
							case ERigControlType::Rotator:
							{
								NamePins.Add(TEXT("Control"));
								StructTemplate = FRigUnit_SetControlRotator::StaticStruct();
								NewNode.ValuePinName = TEXT("Rotator");
								NewNode.ValueType = ERigControlType::Rotator;
								NewNode.Value = FRigControlValue::Make<FRotator>(Hierarchy->GetGlobalTransform(Key).Rotator());
								break;
							}
							case ERigControlType::Transform:
							case ERigControlType::TransformNoScale:
							case ERigControlType::EulerTransform:
							{
								StructTemplate = FRigUnit_SetTransform::StaticStruct();
								NewNode.ValuePinName = TEXT("Transform");
								NewNode.ValueType = ERigControlType::Transform;
								NewNode.Value = FRigControlValue::Make<FTransform>(Hierarchy->GetGlobalTransform(Key));
								break;
							}
							default:
							{
								break;
							}
						}
					}
					else
					{
						StructTemplate = FRigUnit_SetTransform::StaticStruct();
						NewNode.ValuePinName = TEXT("Transform");
						NewNode.ValueType = ERigControlType::Transform;
						NewNode.Value = FRigControlValue::Make<FTransform>(Hierarchy->GetGlobalTransform(Key));
					}
					break;
				}
				case ERigElementGetterSetterType_Relative:
				{
					StructTemplate = FRigUnit_SetRelativeTransformForItem::StaticStruct();
					ItemPins.Reset();
					ItemPins.Add(TEXT("Child"));
					ItemPins.Add(TEXT("Parent"));
					break;
				}
				case ERigElementGetterSetterType_Rotation:
				{
					StructTemplate = FRigUnit_SetRotation::StaticStruct();
					NewNode.ValuePinName = TEXT("Rotation");
					NewNode.ValueType = ERigControlType::Rotator;
					NewNode.Value = FRigControlValue::Make<FRotator>(Hierarchy->GetGlobalTransform(Key).Rotator());
					break;
				}
				case ERigElementGetterSetterType_Translation:
				{
					StructTemplate = FRigUnit_SetTranslation::StaticStruct();
					NewNode.ValuePinName = TEXT("Translation");
					NewNode.ValueType = ERigControlType::Position;
					NewNode.Value = FRigControlValue::Make<FVector>(Hierarchy->GetGlobalTransform(Key).GetLocation());
					break;
				}
				case ERigElementGetterSetterType_Offset:
				{
					StructTemplate = FRigUnit_OffsetTransformForItem::StaticStruct();
					break;
				}
				default:
				{
					break;
				}
			}
		}

		if (StructTemplate == nullptr)
		{
			return;
		}

		FVector2D NodePositionIncrement(0.f, 120.f);
		if (!bIsGetter)
		{
			NodePositionIncrement = FVector2D(380.f, 0.f);
		}

		FName Name = FRigVMBlueprintUtils::ValidateName(GetControlRigAssetInterface()->GetRigVMAssetInterface(), StructTemplate->GetName());
		if (URigVMUnitNode* ModelNode = GetFocusedController()->AddUnitNode(StructTemplate, FRigUnit::GetMethodName(), NodePosition, FString(), true, true))
		{
			FString ItemTypeStr = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Key.Type).ToString();
			NewNode.Name = ModelNode->GetFName();
			NewNodes.Add(NewNode);

			for (const TPair<FName, int32>& PinToResolve : PinsToResolve)
			{
				if(URigVMPin* Pin = ModelNode->FindPin(PinToResolve.Key.ToString()))
				{
					GetFocusedController()->ResolveWildCardPin(Pin, PinToResolve.Value, true, true);
				}
			}

			for (const FName& ItemPin : ItemPins)
			{
				GetFocusedController()->SetPinDefaultValue(FString::Printf(TEXT("%s.%s.Name"), *ModelNode->GetName(), *ItemPin.ToString()), Key.Name.ToString(), true, true, false, true);
				GetFocusedController()->SetPinDefaultValue(FString::Printf(TEXT("%s.%s.Type"), *ModelNode->GetName(), *ItemPin.ToString()), ItemTypeStr, true, true, false, true);
			}

			for (const FName& NamePin : NamePins)
			{
				const FString PinPath = FString::Printf(TEXT("%s.%s"), *ModelNode->GetName(), *NamePin.ToString());
				GetFocusedController()->SetPinDefaultValue(PinPath, NameValue.ToString(), true, true, false, true);
			}

			for (const FName& ChannelPin : ChannelPins)
			{
				const FString PinPath = FString::Printf(TEXT("%s.%s"), *ModelNode->GetName(), *ChannelPin.ToString());
				GetFocusedController()->SetPinDefaultValue(PinPath, ChannelValue.ToString(), true, true, false, true);
			}

			if (!NewNode.ValuePinName.IsNone())
			{
				FString DefaultValue;

				switch (NewNode.ValueType)
				{
					case ERigControlType::Position:
					case ERigControlType::Scale:
					{
						DefaultValue = NewNode.Value.ToString<FVector>();
						break;
					}
					case ERigControlType::Rotator:
					{
						DefaultValue = NewNode.Value.ToString<FRotator>();
						break;
					}
					case ERigControlType::Transform:
					{
						DefaultValue = NewNode.Value.ToString<FTransform>();
						break;
					}
					default:
					{
						break;
					}
				}
				if (!DefaultValue.IsEmpty())
				{
					GetFocusedController()->SetPinDefaultValue(FString::Printf(TEXT("%s.%s"), *ModelNode->GetName(), *NewNode.ValuePinName.ToString()), DefaultValue, true, true, false, true);
				}
			}

			URigVMEdGraphUnitNodeSpawner::HookupMutableNode(ModelNode, GetControlRigAssetInterface()->GetRigVMAssetInterface());
		}

		NodePosition += NodePositionIncrement;
	}

	if (NewNodes.Num() > 0)
	{
		TArray<FName> NewNodeNames;
		for (const FNewNodeData& NewNode : NewNodes)
		{
			NewNodeNames.Add(NewNode.Name);
		}
		GetFocusedController()->SetNodeSelection(NewNodeNames);
		GetFocusedController()->CloseUndoBracket();
	}
	else
	{
		GetFocusedController()->CancelUndoBracket();
	}
}

void FControlRigBaseEditor::HandleMakeMetadataGetterSetter(const bool bIsGetter, const TArray<FRigElementKey> Keys, UEdGraph* Graph, FVector2D NodePosition)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigHierarchy* Hierarchy = GetHierarchyBeingDebugged();
	URigVMController* Controller = GetFocusedController();
	if (Keys.IsEmpty() ||
		!Hierarchy ||
		!Controller)
	{
		return;
	}

	const FRigVMDispatchFactory* GetterDispatchFactory = FRigVMRegistry::Get().FindOrAddDispatchFactory(FRigDispatch_GetMetadata::StaticStruct());
	const FRigVMDispatchFactory* SetterDispatchFactory = FRigVMRegistry::Get().FindOrAddDispatchFactory(FRigDispatch_SetMetadata::StaticStruct());
	if (!ensureMsgf(GetterDispatchFactory && SetterDispatchFactory, TEXT("Cannot find factory for meta data getter and setter nodes, skip creating nodes.")))
	{
		return;
	}

	const FRigVMDispatchFactory* Factory = bIsGetter ? GetterDispatchFactory : SetterDispatchFactory;
	check(Factory);

	const FVector2D NodePositionIncrement = bIsGetter ? FVector2D(0.0, 200.0) : FVector2D(360.0, 0.0);

	Controller->OpenUndoBracket(TEXT("Adding Nodes from Hierarchy"));

	TArray<FName> NewNodes;
	for (int32 Index = 0; Index < Keys.Num(); Index++)
	{
		const FRigElementKey Key = Keys[Index];

		constexpr const TCHAR* NodeNameOverride = TEXT("");
		constexpr bool bSetupUndoRedo_ForAddTemplateNode = true;
		constexpr bool bPrintPythonCommand_ForAddTemplateNode = true;
		if (URigVMTemplateNode* ModelNode = Controller->AddTemplateNode(
			Factory->GetTemplateNotation(),
			NodePosition,
			NodeNameOverride,
			bSetupUndoRedo_ForAddTemplateNode,
			bPrintPythonCommand_ForAddTemplateNode))
		{
			NewNodes.Add(ModelNode->GetFName());

			constexpr bool bResizeArrays = true;
			constexpr bool bSetupUndoRedo = true;
			constexpr bool bPrintPythonCommand_ForSetPinDefaultValue = false;
			constexpr bool bSetValueOnLinkedPin_ForSetPinDefaultValue = true;

			// Set default item type
			{
				const FString ItemTypePinPath = FString::Printf(TEXT("%s.%s.Type"), *ModelNode->GetName(), TEXT("Item"));
				const FString ItemTypeStr = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)Key.Type).ToString();
				Controller->SetPinDefaultValue(
					ItemTypePinPath,
					ItemTypeStr,
					bResizeArrays,
					bSetupUndoRedo,
					bPrintPythonCommand_ForSetPinDefaultValue,
					bSetValueOnLinkedPin_ForSetPinDefaultValue);
			}

			// Set default item name
			{
				const FString ItemNamePinPath = FString::Printf(TEXT("%s.%s.Name"), *ModelNode->GetName(), TEXT("Item"));
				Controller->SetPinDefaultValue(
					ItemNamePinPath,
					Key.Name.ToString(),
					bResizeArrays,
					bSetupUndoRedo,
					bPrintPythonCommand_ForSetPinDefaultValue,
					bSetValueOnLinkedPin_ForSetPinDefaultValue);
			}

			URigVMEdGraphUnitNodeSpawner::HookupMutableNode(ModelNode, GetControlRigAssetInterface()->GetRigVMAssetInterface());

			NewNodes.Add(ModelNode->GetFName());
		}

		NodePosition += NodePositionIncrement;
	}

	if (NewNodes.Num() > 0)
	{
		Controller->SetNodeSelection(NewNodes);
		Controller->CloseUndoBracket();
	}
	else
	{
		Controller->CancelUndoBracket();
	}
}

void FControlRigBaseEditor::HandleMakeComponentGetterSetter(bool bIsGetter, TArray<FRigComponentKey> Keys, UEdGraph* Graph, FVector2D NodePosition)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Keys.Num() == 0)
	{
		return;
	}

	URigHierarchy* Hierarchy = GetHierarchyBeingDebugged();
	if (Hierarchy == nullptr)
	{
		return;
	}
	if (GetFocusedController() == nullptr)
	{
		return;
	}

	GetFocusedController()->OpenUndoBracket(TEXT("Adding Nodes from Hierarchy"));

	static const FRigVMDispatchFactory* GetComponentFactory = FRigVMRegistry_RWLock::Get().FindDispatchFactory(FRigDispatch_GetComponentContent().GetFactoryName());
	static const FRigVMDispatchFactory* SetComponentFactory = FRigVMRegistry_RWLock::Get().FindDispatchFactory(FRigDispatch_SetComponentContent().GetFactoryName());

	TArray<FName> NewNodes;

	for (int32 Index = 0; Index < Keys.Num(); Index++)
	{
		const FRigComponentKey& Key = Keys[Index];
		const FRigBaseComponent* Component = Hierarchy->FindComponent(Key);
		if(Component == nullptr)
		{
			continue;
		}

		const TRigVMTypeIndex ComponentTypeIndex = FRigVMRegistry_RWLock::Get().GetTypeIndex(Component->GetScriptStruct());
		if(ComponentTypeIndex == INDEX_NONE)
		{
			continue;
		}

		const FRigVMDispatchFactory* Factory = nullptr;
		Factory = bIsGetter ? GetComponentFactory : SetComponentFactory;

		check(Factory);

		FVector2D NodePositionIncrement(0.f, 120.f);
		if (!bIsGetter)
		{
			NodePositionIncrement = FVector2D(380.f, 0.f);
		}

		if (URigVMTemplateNode* ModelNode = GetFocusedController()->AddTemplateNode(Factory->GetTemplateNotation(), NodePosition, FString(), true, true))
		{
			if(URigVMPin* ComponentPin = ModelNode->FindPin(FRigDispatch_ComponentBase::ComponentArgName.ToString()))
			{
				GetFocusedController()->ResolveWildCardPin(ComponentPin, ComponentTypeIndex, true, true);
				if(!bIsGetter)
				{
					GetFocusedController()->SetPinDefaultValue(ComponentPin->GetPinPath(), Component->GetContentAsText(), true, true, false, true);
				}
			}

			if(const URigVMPin* NamePin = ModelNode->FindPin(FRigDispatch_ComponentBase::NameArgName.ToString()))
			{
				GetFocusedController()->SetPinDefaultValue(NamePin->GetPinPath(), Component->GetName(), true, true, false, true);
			}

			if(const URigVMPin* KeyPin = ModelNode->FindPin(FRigDispatch_ComponentBase::KeyArgName.ToString()))
			{
				FString DefaultValue;
				FRigComponentKey::StaticStruct()->ExportText(DefaultValue, &Key, nullptr, nullptr, PPF_None, nullptr);
				GetFocusedController()->SetPinDefaultValue(KeyPin->GetPinPath(), DefaultValue, true, true, false, true);
			}

			URigVMEdGraphUnitNodeSpawner::HookupMutableNode(ModelNode, GetControlRigAssetInterface()->GetRigVMAssetInterface());

			NewNodes.Add(ModelNode->GetFName());
		}

		NodePosition += NodePositionIncrement;
	}

	if (NewNodes.Num() > 0)
	{
		GetFocusedController()->SetNodeSelection(NewNodes);
		GetFocusedController()->CloseUndoBracket();
	}
	else
	{
		GetFocusedController()->CancelUndoBracket();
	}
}

void FControlRigBaseEditor::HandleOnControlModified(UControlRig* Subject, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context)
{
	UControlRig* DebuggedControlRig = Cast<UControlRig>(GetRigVMAssetInterface()->GetObjectBeingDebugged());
	if (Subject != DebuggedControlRig)
	{
		return;
	}

	FControlRigAssetInterfacePtr Blueprint = GetControlRigAssetInterface();
	if (Blueprint == nullptr)
	{
		return;
	}

	URigHierarchy* Hierarchy = Subject->GetHierarchy();

	if (ControlElement->Settings.bIsTransientControl && !GIsTransacting)
	{
		const URigVMUnitNode* UnitNode = nullptr;
		const FString NodeName = UControlRig::GetNodeNameFromTransientControl(ControlElement->GetKey());
		const FString PoseTarget = UControlRig::GetTargetFromTransientControl(ControlElement->GetKey());
		TSharedPtr<FStructOnScope> NodeInstance;
		TSharedPtr<FRigDirectManipulationInfo> ManipulationInfo;

		// try to find the direct manipulation info on the rig. if there's no matching information
		// the manipulation is likely happening on a bone instead.
		if(DebuggedControlRig && !NodeName.IsEmpty() && !PoseTarget.IsEmpty())
		{
			UnitNode = Cast<URigVMUnitNode>(GetFocusedModel()->FindNode(NodeName));
			if(UnitNode)
			{
				if(UnitNode->GetScriptStruct())
				{
					NodeInstance = UnitNode->ConstructStructInstance(false);
					ManipulationInfo = DebuggedControlRig->GetRigUnitManipulationInfoForTransientControl(ControlElement->GetKey());
				}
				else
				{
					UnitNode = nullptr;
				}
			}
		}
		
		if (UnitNode && NodeInstance.IsValid() && ManipulationInfo.IsValid())
		{
			FRigUnit* UnitInstance = DebuggedControlRig->GetRigUnitInstanceFromScope(NodeInstance);
			check(UnitInstance);

			const FRigPose Pose = DebuggedControlRig->GetHierarchy()->GetPose();

			// update the node based on the incoming pose. once that is done we'll need to compare the node instance
			// with the settings on the node in the graph and update them accordingly.
			FControlRigExecuteContext& ExecuteContext = DebuggedControlRig->GetRigVMExtendedExecuteContext().GetPublicDataSafe<FControlRigExecuteContext>();
			const FRigHierarchyRedirectorGuard RedirectorGuard(DebuggedControlRig);
			if(UnitInstance->UpdateDirectManipulationFromHierarchy(UnitNode, NodeInstance, ExecuteContext, ManipulationInfo))
			{
				UnitNode->UpdateHostFromStructInstance(DebuggedControlRig, NodeInstance);
				DebuggedControlRig->GetHierarchy()->SetPose(Pose);
				
				URigVMController* Controller = Blueprint->GetRigVMAssetInterface()->GetOrCreateController(UnitNode->GetGraph());
				TMap<FString, FString> PinPathToNewDefaultValue;
				UnitNode->ComputePinValueDifferences(NodeInstance, PinPathToNewDefaultValue);
				if(!PinPathToNewDefaultValue.IsEmpty())
				{
					// we'll disable compilation since the control rig editor module will have disabled folding of literals
					// so each register is free to be edited directly.
					TGuardValue<bool> DisableBlueprintNotifs(Blueprint->GetRigVMAssetInterface()->bSuspendModelNotificationsForSelf, true);

					if(PinPathToNewDefaultValue.Num() > 1)
					{
						Controller->OpenUndoBracket(TEXT("Set pin defaults during manipulation"));
					}
					bool bChangedSomething = false;

					for(const TPair<FString, FString>& Pair : PinPathToNewDefaultValue)
					{
						if(const URigVMPin* Pin = UnitNode->FindPin(Pair.Key))
						{
							if(Controller->SetPinDefaultValue(Pin->GetPinPath(), Pair.Value, true, true, true, false, false))
							{
								bChangedSomething = true;
							}
						}
					}

					if(PinPathToNewDefaultValue.Num() > 1)
					{
						if(bChangedSomething)
						{
							Controller->CloseUndoBracket();
						}
						else
						{
							Controller->CancelUndoBracket();
						}
					}
				}

			}
		}
		else
		{
			FRigControlValue ControlValue = Hierarchy->GetControlValue(ControlElement, ERigControlValueType::Current);
			const FRigElementKey ElementKey = UControlRig::GetElementKeyFromTransientControl(ControlElement->GetKey());

			if (ElementKey.Type == ERigElementType::Bone)
			{
				const FTransform CurrentValue = ControlValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
				const FTransform Transform = CurrentValue * Hierarchy->GetControlOffsetTransform(ControlElement, ERigTransformType::CurrentLocal);
				Blueprint->GetHierarchy()->SetLocalTransform(ElementKey, Transform);
				Hierarchy->SetLocalTransform(ElementKey, Transform);

				if (IsConstructionModeEnabled())
				{
					Blueprint->GetHierarchy()->SetInitialLocalTransform(ElementKey, Transform);
					Hierarchy->SetInitialLocalTransform(ElementKey, Transform);
				}
				else
				{ 
					UpdateBoneModification(ElementKey.Name, Transform);
				}
			}
			else if (ElementKey.Type == ERigElementType::Null)
			{
				const FTransform GlobalTransform = GetControlRig()->GetControlGlobalTransform(ControlElement->GetFName());
				Blueprint->GetHierarchy()->SetGlobalTransform(ElementKey, GlobalTransform);
				Hierarchy->SetGlobalTransform(ElementKey, GlobalTransform);
				if (IsConstructionModeEnabled())
				{
					Blueprint->GetHierarchy()->SetInitialGlobalTransform(ElementKey, GlobalTransform);
					Hierarchy->SetInitialGlobalTransform(ElementKey, GlobalTransform);
				}
			}
		}
	}
	else if (IsConstructionModeEnabled())
	{
		FRigControlElement* SourceControlElement = Hierarchy->Find<FRigControlElement>(ControlElement->GetKey());
		FRigControlElement* TargetControlElement = Blueprint->GetHierarchy()->Find<FRigControlElement>(ControlElement->GetKey());
		if(SourceControlElement && TargetControlElement)
		{
			TargetControlElement->Settings = SourceControlElement->Settings;

			// only fire the setting change if the interaction is not currently ongoing
			if(!Subject->ElementsBeingInteracted.Contains(ControlElement->GetKey()))
			{
				Blueprint->GetHierarchy()->OnModified().Broadcast(ERigHierarchyNotification::ControlSettingChanged, Blueprint->GetHierarchy(), TargetControlElement);
			}

			// we copy the pose including the weights since we want the topology to align during construction mode.
			// i.e. dynamic reparenting should be reset here.
			TargetControlElement->CopyPose(SourceControlElement, true, true, true);
		}
	}
}

void FControlRigBaseEditor::HandleRefreshEditorFromBlueprintImpl(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	OnHierarchyChanged();
	HandleRefreshEditorFromBlueprintSuper(InBlueprint);
}

UToolMenu* FControlRigBaseEditor::HandleOnGetViewportContextMenuDelegate()
{
	if (OnGetViewportContextMenuDelegate.IsBound())
	{
		return OnGetViewportContextMenuDelegate.Execute();
	}
	return nullptr;
}

TSharedPtr<FUICommandList> FControlRigBaseEditor::HandleOnViewportContextMenuCommandsDelegate()
{
	if (OnViewportContextMenuCommandsDelegate.IsBound())
	{
		return OnViewportContextMenuCommandsDelegate.Execute();
	}
	return TSharedPtr<FUICommandList>();
}

void FControlRigBaseEditor::OnPreForwardsSolve_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	// make sure to reapply this in case
	if (SharedRigVMEditorRef()->IsRunGraphOnceOn())
	{
		OnRunGraphOnceToggledImpl(true);
	}
	
	// if we are debugging a PIE instance, we need to remember the input pose on the
	// rig so we can perform multiple evaluations. this is to avoid double transforms / double forward solve results.
	if(InRig->GetWorld()->IsPlayInEditor())
	{
		if(!InRig->GetWorld()->IsPaused())
		{
			// store the pose while PIE is running
			InRig->InputPoseOnDebuggedRig = InRig->GetHierarchy()->GetPose(false, false);
		}
		else
		{
			// reapply the pose as PIE is paused. during pause the rig won't be updated with the input pose
			// from the animbp / client thus we need to reset the pose to avoid double transformation.
			InRig->GetHierarchy()->SetPose(InRig->InputPoseOnDebuggedRig);
		}
	}
}

void FControlRigBaseEditor::OnPostForwardsSolve_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	const UControlRig* ControlRig = GetControlRig();
	if (!ControlRig || ControlRig != InRig)
	{
		return;
	}

	// this is the last evaluation
	if (ControlRig->GetEvaluationsLeft() == 1)
	{
		if (const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			static const FRigElementKeyCollection EmptyCollection;
			BonePoseFromLastEvaluation = Hierarchy->GetPose(false, ERigElementType::Bone, EmptyCollection, false);
		}
	}
	else
	{
		BonePoseFromLastEvaluation.Reset();
	}
}

void FControlRigBaseEditor::OnPreConstructionForUI_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	bIsConstructionEventRunning = true;

	if(ShouldExecuteControlRig(InRig))
	{
		PreConstructionPose.Reset();
		if (FControlRigEditorEditMode* EditMode = GetEditMode())
		{
			if (!EditMode->ModifiedRigElements.IsEmpty())
			{
				PreConstructionPose = InRig->GetHierarchy()->GetPose(false, ERigElementType::ToResetAfterConstructionEvent, TArrayView<FRigElementKey>(EditMode->ModifiedRigElements));
			}
		}

		if(const FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
		{
			if(RigBlueprint->IsControlRigModule())
			{
				SocketStates = InRig->GetHierarchy()->GetSocketStates();
				ConnectorStates = RigBlueprint->GetHierarchy()->GetConnectorStates();
			}
		}
	}
}

void FControlRigBaseEditor::OnPreConstruction_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	if(FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if(RigBlueprint->IsControlRigModule())
		{
			if(USkeletalMesh* PreviewSkeletalMesh = RigBlueprint->GetPreviewSkeletalMesh().Get())
			{
				if(URigHierarchy* Hierarchy = InRig->GetHierarchy())
				{
					if(URigHierarchyController* Controller = Hierarchy->GetController(true))
					{
						Controller->ImportPreviewSkeletalMesh(PreviewSkeletalMesh, false, false, false, false);
					}
				}

				if(ShouldExecuteControlRig(InRig))
				{
					RigBlueprint->GetHierarchy()->RestoreSocketsFromStates(SocketStates);
					InRig->GetHierarchy()->RestoreSocketsFromStates(SocketStates);
				}
			}
		}
	}
}

void FControlRigBaseEditor::OnPostConstruction_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	bIsConstructionEventRunning = false;
	const bool bShouldExecute = ShouldExecuteControlRig(InRig);

	if(FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if(bShouldExecute && RigBlueprint->IsControlRigModule())
		{
			RigBlueprint->GetHierarchy()->RestoreConnectorsFromStates(ConnectorStates);
		}

		if(bShouldExecute && RigBlueprint->IsModularRig())
		{
			// auto resolve the root module's primary connector
			if(RigBlueprint->GetModularRigModel().Connections.IsEmpty() && RigBlueprint->GetModularRigModel().Modules.Num() == 1 && RigBlueprint->GetHierarchy()->Num(ERigElementType::Bone) > 0)
			{
				const FRigModuleReference& RootModule = RigBlueprint->GetModularRigModel().Modules[0];

				FSoftObjectPath DefaultRootModulePath = UControlRigSettings::Get()->DefaultRootModule;
				UObject* DefaultRootModuleObj = DefaultRootModulePath.TryLoad();
				if (!DefaultRootModuleObj)
				{
					DefaultRootModulePath = FString::Printf(TEXT("%s_C"), *UControlRigSettings::Get()->DefaultRootModule.GetAssetPathString());
					DefaultRootModuleObj = DefaultRootModulePath.TryLoad();
				}
				FControlRigAssetStrongReference ControlRigAssetReference;
				if (DefaultRootModuleObj)
				{
					ControlRigAssetReference = FControlRigAssetStrongReference(DefaultRootModuleObj);
				}
				
				if(ControlRigAssetReference.IsValid())
				{
					if(ControlRigAssetReference.Get() == RootModule.ControlRigAssetReference.Get())
					{
						if(const FRigConnectorElement* PrimaryConnector = RootModule.FindPrimaryConnector(RigBlueprint->GetHierarchy()))
						{
							if(const FRigBoneElement* RootBone = RigBlueprint->GetHierarchy()->GetBones()[0])
							{
								if(UModularRigController* ModularRigController = RigBlueprint->GetModularRigModel().GetController())
								{
									(void)ModularRigController->ConnectConnectorToElement(PrimaryConnector->GetKey(), RootBone->GetKey(), false, false, false);
								}
							}
						}
					}
				}
			}
		}
	}
	
	const uint32 HierarchyHash = InRig->GetHierarchy()->GetTopologyHash(false);
	if(LastHierarchyHash != HierarchyHash)
	{
		LastHierarchyHash = HierarchyHash;
		
		auto Task = [this, InRig]()
		{
			CacheNameLists();
			SynchronizeViewportBoneSelection();
			RebindToSkeletalMeshComponent();
			if(DetailViewShowsAnyRigElement())
			{
				const TArray<FRigHierarchyKey> Keys = GetSelectedRigElementsFromDetailView();
				SetDetailViewForRigElements(Keys);
			}
			
			if (FControlRigEditorEditMode* EditMode = GetEditMode())
            {
				if (InRig)
            	{
            		EditMode->bDrawHierarchyBones = !InRig->GetHierarchy()->GetBones().IsEmpty();
            	}
            }
		};
				
		if(IsInGameThread())
		{
			Task();
		}
		else
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [Task]()
			{
				Task();
			});
		}
	}
	else if(bShouldExecute)
	{
		InRig->GetHierarchy()->SetPose(PreConstructionPose, ERigTransformType::CurrentGlobal);
	}
}

void FControlRigBaseEditor::OnEarlyExit_AnyThread(URigVMHost* InHost, URigVMHost::EEarlyExitReason InReason)
{
	if (InReason != URigVMHost::EEarlyExitReason_NoEvaluationsLeft)
	{
		return;
	}
	
	UControlRig* ControlRig = GetControlRig();
	if (!ControlRig || ControlRig != InHost)
	{
		return;
	}

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (!Hierarchy)
	{
		return;
	}

	if (BonePoseFromLastEvaluation.Num() == 0)
	{
		return;
	}

	Hierarchy->SetPose(BonePoseFromLastEvaluation, ERigTransformType::CurrentGlobal);
}

void FControlRigBaseEditor::SetupTimelineDelegates(FAnimationScrubPanelDelegates& InOutDelegates)
{
	TSharedRef<FControlRigBaseEditor> SharedEditor = SharedRef();
	InOutDelegates.IsRecordingActiveDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayIsRecordingActive);
	InOutDelegates.GetRecordingVisibilityDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleGetReplayRecordButtonVisibility);
	InOutDelegates.StartRecordingDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayStartRecording);
	InOutDelegates.StopRecordingDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayStopRecording);
	InOutDelegates.GetPlaybackModeDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayGetPlaybackMode);
	InOutDelegates.SetPlaybackModeDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplaySetPlaybackMode);
	InOutDelegates.GetPlaybackTimeDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayGetPlaybackTime);
	InOutDelegates.SetPlaybackTimeDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplaySetPlaybackTime);
	InOutDelegates.StepForwardDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayStepForward);
	InOutDelegates.StepBackwardDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayStepBackward);
	InOutDelegates.GetIsLoopingDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayGetIsLooping);
	InOutDelegates.SetIsLoopingDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplaySetIsLooping);
	InOutDelegates.GetPlaybackTimeRangeDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayGetPlaybackTimeRange);
	InOutDelegates.GetNumberOfKeysDelegate.BindSP(SharedEditor, &FControlRigBaseEditor::HandleReplayGetNumberOfKeys);
}

bool FControlRigBaseEditor::ShowReplayOnTimeline() const
{
	if(ReplayStrongPtr.IsValid())
	{
		if(GetControlRig()->Replay.Get() == ReplayStrongPtr.Get())
		{
			return !ReplayStrongPtr->IsRecording();
		}
	}
	return false;
}

TOptional<bool> FControlRigBaseEditor::HandleReplayIsRecordingActive() const
{
	if(!ShowReplayOnTimeline())
	{
		return TOptional<bool>();
	}
	return ReplayStrongPtr->IsRecording();
}

TOptional<EVisibility> FControlRigBaseEditor::HandleGetReplayRecordButtonVisibility() const
{
	if(ReplayStrongPtr.IsValid())
	{
		return EVisibility::Collapsed;
	}
	return TOptional<EVisibility>();
}

bool FControlRigBaseEditor::HandleReplayStartRecording()
{
	// for now this button won't be supported from here.
	return false;
}

bool FControlRigBaseEditor::HandleReplayStopRecording()
{
	// for now this button won't be supported from here.
	return false;
}

TOptional<int32> FControlRigBaseEditor::HandleReplayGetPlaybackMode() const
{
	if(!ShowReplayOnTimeline())
	{
		return TOptional<int32>();
	}
	return (ReplayStrongPtr->IsReplaying() && !ReplayStrongPtr->IsPaused()) ? (int32)EPlaybackMode::PlayingForward : (int32)EPlaybackMode::Stopped; 
}

bool FControlRigBaseEditor::HandleReplaySetPlaybackMode(int32 InPlaybackMode)
{
	if(!ShowReplayOnTimeline())
	{
		return false;
	}
	if(InPlaybackMode == (int32)EPlaybackMode::Stopped)
	{
		ReplayStrongPtr->PauseReplay();
	}
	else
	{
		ReplayStrongPtr->StartReplay(GetControlRig());
	}
	return true;
}

TOptional<float> FControlRigBaseEditor::HandleReplayGetPlaybackTime() const
{
	if(!ShowReplayOnTimeline())
	{
		return TOptional<float>();
	}
	return GetControlRig()->GetAbsoluteTime();
}

bool FControlRigBaseEditor::HandleReplaySetPlaybackTime(float InTime, bool bStopPlayback)
{
	if(!ShowReplayOnTimeline())
	{
		return false;
	}
	if(bStopPlayback)
	{
		HandleReplaySetPlaybackMode(EPlaybackMode::Stopped);
	}

	const int32 TimeIndex = ReplayStrongPtr->InputTracks.GetTimeIndex(InTime);
	if(TimeIndex != INDEX_NONE)
	{
		GetControlRig()->SetReplayTimeIndex(TimeIndex);
	}
	return true;
}

bool FControlRigBaseEditor::HandleReplayStepForward()
{
	if(!ShowReplayOnTimeline())
	{
		return false;
	}

	HandleReplaySetPlaybackMode(EPlaybackMode::Stopped);

	const int32 PreviousTimeIndex = GetControlRig()->GetReplayTimeIndex();
	if(PreviousTimeIndex < ReplayStrongPtr->InputTracks.GetNumTimes() - 1)
	{
		GetControlRig()->SetReplayTimeIndex(PreviousTimeIndex + 1);
	}
	return true;
}

bool FControlRigBaseEditor::HandleReplayStepBackward()
{
	if(!ShowReplayOnTimeline())
	{
		return false;
	}
	
	HandleReplaySetPlaybackMode(EPlaybackMode::Stopped);

	const int32 PreviousTimeIndex = GetControlRig()->GetReplayTimeIndex();
	if(PreviousTimeIndex > 0)
	{
		GetControlRig()->SetReplayTimeIndex(PreviousTimeIndex - 1);
	}
	return true;
}

TOptional<bool> FControlRigBaseEditor::HandleReplayGetIsLooping() const
{
	if(!ShowReplayOnTimeline())
	{
		return TOptional<bool>();
	}
	// we are always set to loop
	return true;
}

bool FControlRigBaseEditor::HandleReplaySetIsLooping(bool bIsLooping)
{
	// we are always set to loop for now.
	return true;
}

TOptional<FVector2f> FControlRigBaseEditor::HandleReplayGetPlaybackTimeRange() const
{
	if(!ShowReplayOnTimeline())
	{
		return TOptional<FVector2f>();
	}
	return FVector2f(ReplayStrongPtr->GetTimeRange());
}

TOptional<uint32> FControlRigBaseEditor::HandleReplayGetNumberOfKeys() const
{
	if(!ShowReplayOnTimeline())
	{
		return TOptional<uint32>();
	}
	return static_cast<uint32>(ReplayStrongPtr->InputTracks.GetNumTimes());
}

EVisibility FControlRigBaseEditor::GetReplayValidationErrorVisibility() const
{
	if(ReplayStrongPtr.IsValid())
	{
		if(ReplayStrongPtr->IsReplaying() &&
			ReplayStrongPtr->GetPlaybackMode() == EControlRigReplayPlaybackMode::ReplayInputs && 
			ReplayStrongPtr->HasValidationErrors())
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

FText FControlRigBaseEditor::GetReplayValidationErrorTooltip() const
{
	if(ReplayStrongPtr.IsValid())
	{
		const TArray<FString>& ValidationErrors = ReplayStrongPtr->GetValidationErrors();
		if(!ValidationErrors.IsEmpty())
		{
			static const FText Format = LOCTEXT("ReplayValidationErrorTooltipFormat", "The results from the rig don't match the expected values in the replay.\nSwitch to ground truth to compare or check the output log.\n{0}");
			const TArrayView<const FString> ClampedValidationErrors(ValidationErrors.GetData(), FMath::Min(20, ValidationErrors.Num()));
			return FText::Format(Format, FText::FromString(FString::Join(ClampedValidationErrors, TEXT("\n"))));
		}
	}
	return FText();
}

#undef LOCTEXT_NAMESPACE
