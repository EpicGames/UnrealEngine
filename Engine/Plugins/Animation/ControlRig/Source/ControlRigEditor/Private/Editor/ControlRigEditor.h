// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IControlRigEditor.h"
#include "Editor/ControlRigEditorEditMode.h"
#include "AssetEditorModeManager.h"
#include "DragAndDrop/GraphNodeDragDropOp.h"
#include "ControlRigDefines.h"
#include "Units/RigUnitContext.h"
#include "IPersonaViewport.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMCore/RigVM.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Styling/SlateTypes.h"
#include "AnimPreviewInstance.h"
#include "ScopedTransaction.h"
#include "Graph/ControlRigGraphNode.h"
#include "RigVMModel/RigVMController.h"
#include "Editor/RigVMDetailsViewWrapperObject.h"
#include "ControlRigReplay.h"
#include "ModularRigController.h"
#include "RigVMHost.h"
#include "SchematicGraphPanel/SSchematicGraphPanel.h"
#include "Units/RigUnit.h"
#include "ControlRigSchematicModel.h"
#include "PersonaModule.h"

class UControlRigBlueprint;
class IPersonaToolkit;
class SWidget;
class SBorder;
class USkeletalMesh;
class FStructOnScope;
class UToolMenu;
class FControlRigBaseEditor;

struct FControlRigEditorModes
{
	// Mode constants
	static const FName ControlRigEditorMode;
	static FText GetLocalizedMode(const FName InMode)
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add(ControlRigEditorMode, NSLOCTEXT("ControlRigEditorModes", "ControlRigEditorMode", "Rigging"));
		}

		check(InMode != NAME_None);
		const FText* OutDesc = LocModes.Find(InMode);
		check(OutDesc);
		return *OutDesc;
	}
private:
	FControlRigEditorModes() {}
};

class IControlRigBaseEditor
{
public:
	virtual TSharedRef<IRigVMEditor> SharedRigVMEditorRef() = 0;
	virtual TSharedRef<IControlRigBaseEditor> SharedControlRigEditorRef() = 0;
	virtual TSharedRef<FControlRigBaseEditor> SharedRef() = 0;	
	virtual TSharedRef<const IRigVMEditor> SharedRigVMEditorRef() const = 0;

	UE_DEPRECATED(5.7, "Use GetControlRigAssetInterface instead.")
	virtual UControlRigBlueprint* GetControlRigBlueprint() const = 0;
	virtual FControlRigAssetInterfacePtr GetControlRigAssetInterface() const = 0;
	virtual TSharedPtr<FAssetEditorToolkit> GetHostingApp() = 0;
	virtual UControlRig* GetControlRig() const = 0;

	virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const = 0;

	virtual void Compile() = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FControlRigEditorClosed, IControlRigBaseEditor*, FControlRigAssetInterfacePtr);
	virtual FControlRigEditorClosed& OnEditorClosed() = 0;
	virtual FPersonaViewportKeyDownDelegate& GetKeyDownDelegate() = 0;
	virtual FOnGetContextMenu& OnGetViewportContextMenu() = 0;
	virtual FNewMenuCommandsDelegate& OnViewportContextMenuCommands() = 0;
	virtual FSimpleMulticastDelegate& OnRequestNavigateToConnectorWarning() = 0;
	virtual FControlRigEditorEditMode* GetEditMode() const = 0;
	virtual int32 GetEventQueueComboValue() const = 0;
	virtual void OnHierarchyChanged() = 0;

	virtual void SetDetailViewForRigElements() = 0;
	virtual void SetDetailViewForRigElements(const TArray<FRigHierarchyKey>& InKeys) = 0;
	virtual void ClearDetailObject(bool bChangeUISelectionState = true) = 0;
	virtual void FindReferencesOfItem(const FRigHierarchyKey& InKey) = 0;
	
	virtual int32 GetRigHierarchyTabCount() const = 0;
	virtual void IncreaseRigHierarchyTabCount() = 0;
	virtual void DecreaseRigHierarchyTabCount() = 0;

	virtual int32 GetModularRigHierarchyTabCount() const = 0;
	virtual void IncreaseModularRigHierarchyTabCount() = 0;
	virtual void DecreaseModularRigHierarchyTabCount() = 0;
	
	virtual bool& GetSuspendDetailsPanelRefreshFlag() = 0;
	virtual EControlRigReplayPlaybackMode GetReplayPlaybackMode() const = 0;
	virtual TArray<FName> GetSelectedModules() const = 0;

	virtual void RefreshDetailView() = 0;
	virtual FVector2D ComputePersonaProjectedScreenPos(const FVector& InWorldPos, bool bClampToScreenRectangle = false) = 0;
	virtual void SetDetailViewForRigModules(const TArray<FName> InModuleNames) = 0;

	virtual UAnimPreviewInstance* GetPreviewInstance() const = 0;
	virtual void SetPreviewInstance(UAnimPreviewInstance* InPreviewInstance) = 0;
	
	virtual void RemoveBoneModification(FName BoneName) = 0;
	virtual URigHierarchy* GetHierarchyBeingDebugged() const = 0;

	virtual void FilterDraggedKeys(TArray<FRigElementKey>& Keys, bool bRemoveNameSpace) = 0;
	virtual URigVMController* GetFocusedController() const = 0;
	
protected:

	virtual void InitRigVMEditorSuper(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint) = 0;

	virtual bool IsControlRigLegacyEditor() const = 0;
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const = 0;
	virtual URigVMHost* GetRigVMHost() const = 0;
	virtual bool IsEditorInitialized() const = 0;
	
	virtual TSharedRef<FUICommandList> GetToolkitCommands() = 0;
	virtual FPreviewScene* GetPreviewScene() = 0;
	virtual bool IsDetailsPanelRefreshSuspended() const = 0;
	virtual TArray<TWeakObjectPtr<UObject>> GetSelectedObjects() const = 0;
	virtual UClass* GetDetailWrapperClass() const = 0;
	virtual void OnWrappedPropertyChangedChainEvent(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent) = 0;
	virtual void SetDetailObjects(const TArray<UObject*>& InObjects) = 0;
	virtual void SetDetailObjectsSuper(const TArray<UObject*>& InObjects) = 0;
	virtual void SetDetailObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InObjectFilter) = 0;
	virtual bool DetailViewShowsStruct(UScriptStruct* InStruct) const = 0;
	virtual TSharedPtr<class SWidget> GetInspector() const = 0;
	virtual TArray<FName> GetEventQueue() const = 0;
	virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false) = 0;
	virtual const TArray< UObject* >* GetObjectsCurrentlyBeingEdited() const = 0;
	virtual void UpdateRigVMHost() = 0;
	virtual void RefreshDetailViewSuper() = 0;
	virtual void CacheNameLists() = 0;
	virtual FEditorModeTools& GetEditorModeManagerImpl() const = 0;
	virtual const FName GetEditorModeNameImpl() const = 0;
	virtual URigVMGraph* GetFocusedModel() const = 0;
	virtual UObject* GetOuterForHostSuper() const = 0;
	virtual void CompileSuper() = 0;
	virtual void HandleModifiedEventSuper(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) = 0;
	virtual void OnCreateGraphEditorCommandsSuper(TSharedPtr<FUICommandList> GraphEditorCommandsList) = 0;
	virtual void HandleVMCompiledEventSuper(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext) = 0;
	virtual FReply OnViewportDropSuper(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) = 0;
	virtual void FillToolbarSuper(FToolBarBuilder& ToolbarBuilder, bool bEndSection = true) = 0;
	virtual TArray<FName> GetLastEventQueue() const = 0;
	virtual int32 GetEventQueueComboValueSuper() const = 0;
	virtual void HandleSetObjectBeingDebuggedSuper(UObject* InObject) = 0;
	virtual void SetEventQueue(TArray<FName> InEventQueue, bool bCompile) = 0;
	virtual void SetEventQueueSuper(TArray<FName> InEventQueue, bool bCompile) = 0;
	virtual void SetEventQueueSuper(TArray<FName> InEventQueue) = 0;
	virtual void SaveAsset_ExecuteSuper() = 0;
	virtual void SaveAssetAs_ExecuteSuper() = 0;
	virtual FReply OnSpawnGraphNodeByShortcutSuper(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph) = 0;
	virtual void HandleVMExecutedEventSuper(URigVMHost* InHost, const FName& InEventName) = 0;
	virtual void SetEditorModeManager(TSharedPtr<FEditorModeTools> InManager) = 0;
	virtual void TickSuper(float DeltaTime) = 0;
	virtual void UpdateRigVMHostSuper() = 0;
	virtual void CacheNameListsSuper() = 0;
	virtual void OnFinishedChangingPropertiesSuper(const FPropertyChangedEvent& PropertyChangedEvent) = 0;
	virtual void OnWrappedPropertyChangedChainEventSuper(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent) = 0;
	virtual const TArray<TStrongObjectPtr<URigVMDetailsViewWrapperObject>>& GetWrapperObjects() const = 0;
	virtual void BindCommandsSuper() = 0;
	virtual void UnbindCommandsSuper() = 0;
	virtual FMenuBuilder GenerateBulkEditMenuSuper() = 0;
	virtual TWeakPtr<class SGraphEditor> GetFocusedGraphEd() const = 0;
	virtual void OnGraphNodeDropToPerformSuper(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition) = 0;
	virtual void HandleRefreshEditorFromBlueprintSuper(FRigVMEditorAssetInterfacePtr InBlueprint) = 0;
	virtual void OnGraphNodeDropToPerform(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition) = 0;
	
	virtual void OnCloseSuper() = 0;

	friend class SModularRigEventQueueView;
};

class FControlRigBaseEditor : public IControlRigBaseEditor
{
public:
	static FControlRigBaseEditor* GetFromAssetEditorInstance(IAssetEditorInstance* Instance);
	virtual TSharedRef<FControlRigBaseEditor> SharedRef() override { return StaticCastSharedRef<FControlRigBaseEditor>(SharedControlRigEditorRef());}

	// Gets the Control Rig Blueprint being edited/viewed
	UE_DEPRECATED(5.7, "Use GetControlRigAssetInterface instead")
	virtual UControlRigBlueprint* GetControlRigBlueprint() const override;
	virtual FControlRigAssetInterfacePtr GetControlRigAssetInterface() const override;

protected:
	void InitRigVMEditorImpl(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint);

	virtual const FName GetEditorAppNameImpl() const;
	virtual const FName GetEditorModeNameImpl() const;
	virtual const FSlateBrush* GetDefaultTabIconImpl() const;


	FControlRigBaseEditor();
	virtual ~FControlRigBaseEditor() = default;

	virtual UObject* GetOuterForHostImpl() const;

	virtual UClass* GetDetailWrapperClassImpl() const;
	virtual void CompileBaseImpl();
	virtual void HandleModifiedEventImpl(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);
	virtual void OnCreateGraphEditorCommandsImpl(TSharedPtr<FUICommandList> GraphEditorCommandsList);
	virtual void HandleVMCompiledEventImpl(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext);
	virtual bool ShouldOpenGraphByDefaultImpl() const { return !IsModularRig(); }
	virtual FReply OnViewportDropImpl(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	// allows the editor to fill an empty graph
	virtual void CreateEmptyGraphContentImpl(URigVMController* InController);

	virtual int32 GetRigHierarchyTabCount() const override { return RigHierarchyTabCount; }
	virtual void IncreaseRigHierarchyTabCount() override { RigHierarchyTabCount++; }
	virtual void DecreaseRigHierarchyTabCount() override { RigHierarchyTabCount--; }
	virtual int32 GetModularRigHierarchyTabCount() const override { return ModularRigHierarchyTabCount; }
	virtual void IncreaseModularRigHierarchyTabCount() override { ModularRigHierarchyTabCount++; }
	virtual void DecreaseModularRigHierarchyTabCount() override { ModularRigHierarchyTabCount--; }

	bool IsModularRig() const;
	bool IsRigModule() const;


	
	// IToolkit Interface
	virtual FName GetToolkitFNameImpl() const;
	virtual FText GetBaseToolkitNameImpl() const;
	virtual FString GetWorldCentricTabPrefixImpl() const;
	virtual FString GetDocumentationLinkImpl() const 
	{
		return FString(TEXT("Engine/Animation/ControlRig"));
	}

	// BlueprintEditor interface
	virtual FReply OnSpawnGraphNodeByShortcutImpl(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph);

	virtual void PostTransactionImpl(bool bSuccess, const FTransaction* Transaction, bool bIsRedo);

	void EnsureValidRigElementsInDetailPanel();

	//  FTickableEditorObject Interface
	virtual void TickImpl(float DeltaTime);

	// returns the hierarchy being debugged
	virtual UControlRig* GetControlRig() const override;

	// returns the hierarchy being debugged
	virtual URigHierarchy* GetHierarchyBeingDebugged() const override;

	virtual void SetDetailViewForRigElements() override;
	virtual void SetDetailViewForRigElements(const TArray<FRigHierarchyKey>& InKeys) override;
	bool DetailViewShowsAnyRigElement() const;
	bool DetailViewShowsRigElement(FRigHierarchyKey InKey) const;
	TArray<FRigHierarchyKey> GetSelectedRigElementsFromDetailView() const;
	TArray< TWeakObjectPtr<UObject> > GetSelectedObjectsFromDetailView() const;

	void SetDetailViewForRigModules();
	virtual void SetDetailViewForRigModules(const TArray<FName> InModuleNames) override;
	bool DetailViewShowsAnyRigModule() const;
	bool DetailViewShowsRigModule(FName InModuleName) const;

	TArray<FName> ModulesSelected;
	virtual TArray<FName> GetSelectedModules() const override { return ModulesSelected; }

	virtual void SetDetailObjectsImpl(const TArray<UObject*>& InObjects);
	virtual void RefreshDetailViewImpl();

	void CreatePersonaToolKitIfRequired();

public:
	/** Get the persona toolkit */
	virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }
protected:

	/** Get the edit mode */
	virtual FControlRigEditorEditMode* GetEditMode() const override;

	void OnCurveContainerChanged();

	void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);

	void HandleRigTypeChanged(FControlRigAssetInterfacePtr InBlueprint);

	void HandleModularRigModified(EModularRigNotification InNotification, const FRigModuleReference* InModule);
	void HandlePostCompileModularRigs(FRigVMEditorAssetInterfacePtr InBlueprint);
	void SwapModuleWithinAsset();
	void SwapModuleAcrossProject();

	const FName RigHierarchyToGraphDragAndDropMenuName = TEXT("ControlRigEditor.RigHierarchyToGraphDragAndDropMenu");
	void CreateRigHierarchyToGraphDragAndDropMenu() const;
	virtual void OnGraphNodeDropToPerformImpl(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition);

	virtual FPersonaViewportKeyDownDelegate& GetKeyDownDelegate() override { return OnKeyDownDelegate; }

	virtual FOnGetContextMenu& OnGetViewportContextMenu() override { return OnGetViewportContextMenuDelegate; }
	virtual FNewMenuCommandsDelegate& OnViewportContextMenuCommands() override { return OnViewportContextMenuCommandsDelegate; }

	// DirectManipulation functionality
	void HandleRequestDirectManipulationPosition() const { (void)HandleRequestDirectManipulation(ERigControlType::Position); }
	void HandleRequestDirectManipulationRotation() const { (void)HandleRequestDirectManipulation(ERigControlType::Rotator); }
	void HandleRequestDirectManipulationScale() const { (void)HandleRequestDirectManipulation(ERigControlType::Scale); }
	bool HandleRequestDirectManipulation(ERigControlType InControlType) const;
	bool SetDirectionManipulationSubject(const URigVMUnitNode* InNode);
	bool IsDirectManipulationEnabled() const;
	EVisibility GetDirectManipulationVisibility() const;
	FText GetDirectionManipulationText() const;
	void OnDirectManipulationChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);
	const TArray<FRigDirectManipulationTarget> GetDirectManipulationTargets() const;
	const TArray<TSharedPtr<FString>>& GetDirectManipulationTargetTextList() const;
	bool ClearDirectManipulationSubject() { return SetDirectionManipulationSubject(nullptr); }
	void RefreshDirectManipulationTextList();
	EVisibility GetPreviewingNodeVisibility() const;
	void OnPreviewingNodeJumpTo();
	void OnPreviewingNodeCancel();

	// Rig connector functionality
	EVisibility GetConnectorWarningVisibility() const;
	FText GetConnectorWarningText() const;
	FReply OnNavigateToConnectorWarning() const;
	virtual FSimpleMulticastDelegate& OnRequestNavigateToConnectorWarning() override { return RequestNavigateToConnectorWarningDelegate; }

	virtual FVector2D ComputePersonaProjectedScreenPos(const FVector& InWorldPos, bool bClampToScreenRectangle = false) override;

	virtual void FindReferencesOfItem(const FRigHierarchyKey& InKey) override;
	


	virtual void BindCommandsImpl();
	virtual void UnbindCommandsImpl();
	virtual FMenuBuilder GenerateBulkEditMenuImpl();

	virtual void OnHierarchyChanged() override;

	void SynchronizeViewportBoneSelection();

	virtual void SaveAsset_ExecuteImpl();
	virtual void SaveAssetAs_ExecuteImpl();

	// update the cached modification value
	void UpdateBoneModification(FName BoneName, const FTransform& Transform);

	// remove a single bone modification across all instance
	virtual void RemoveBoneModification(FName BoneName) override;

	// reset all bone modification across all instance
	void ResetAllBoneModification();

	virtual void HandleVMExecutedEventImpl(URigVMHost* InHost, const FName& InEventName);

	// FBaseToolKit overrides
	virtual void CreateEditorModeManagerImpl();

	/** Fill the toolbar with content */
	virtual void FillToolbarImpl(FToolBarBuilder& ToolbarBuilder, bool bEndSection = true);

	virtual TArray<FName> GetDefaultEventQueueImpl() const;
	virtual void SetEventQueueImpl(TArray<FName> InEventQueue, bool bCompile);
	virtual int32 GetEventQueueComboValueImpl() const;
	virtual FText GetEventQueueLabelImpl() const;
	virtual FSlateIcon GetEventQueueIconImpl(const TArray<FName>& InEventQueue) const;
	virtual void HandleSetObjectBeingDebuggedImpl(UObject* InObject);


	/** Handle preview scene setup */
	void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);
public:
	void HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport);
protected:
	/** Sets the next solve mode, for example Backwards Solve if the last entry in the Queue is ForwardsSolve */
	UE_DEPRECATED(5.8, "Instead use SetNextEventQueue")
	void SetNextSolveMode();
	
	/** Sets the next Event Queue, for example Backwards Solve if the last entry in the Queue is ForwardsSolve */
	void SetNextEventQueue();

	/** Sets the Event Queue that was set prior to the current one */
	void SetPriorEventQueue();

	/** Toggles between forwards solve and construction */
	void ToggleForwardsSolveAndConstruction();

	void HandleToggleControlVisibility();
	bool AreControlsVisible() const;
	void HandleToggleControlsAsOverlay();
	bool AreControlsAsOverlay() const;
	bool IsToolbarDrawNullsEnabled() const;
	bool GetToolbarDrawNulls() const;
	void HandleToggleToolbarDrawNulls();
	bool IsToolbarDrawSocketsEnabled() const;
	bool GetToolbarDrawSockets() const;
	void HandleToggleToolbarDrawSockets();
	bool GetToolbarDrawAxesOnSelection() const;
	void HandleToggleToolbarDrawAxesOnSelection();
	void HandleToggleSchematicViewport();
	bool IsSchematicViewportActive() const;
	void HandleToggleSchematicViewportShowEmptySocketsOnly();
	bool IsSchematicViewportShowEmptySocketsOnlyActive() const;
	void HandleToggleCompileResetsEventQueue();
	bool IsToggleCompileResetsEventQueueActive() const;
	EVisibility GetSchematicOverlayVisibility() const;

	bool IsRunningForwardSolve() const;
	void OnRunGraphOnceToggledImpl(bool bRunGraphOnceEnabled);

	/** Handle switching skeletal meshes */
	void HandlePreviewMeshChanged(USkeletalMesh* InOldSkeletalMesh, USkeletalMesh* InNewSkeletalMesh);

	/** Push a newly compiled/opened control rig to the edit mode */
	virtual void UpdateRigVMHostImpl();
	virtual void UpdateRigVMHost_PreClearOldHostImpl(URigVMHost* InPreviousHost);

	/** Update the name lists for use in name combo boxes */
	virtual void CacheNameListsImpl();

	/** Rebind our anim instance to the preview's skeletal mesh component */
	void RebindToSkeletalMeshComponent();

	virtual void GenerateEventQueueMenuContentImpl(FMenuBuilder& MenuBuilder);

	enum ERigElementGetterSetterType
	{
		ERigElementGetterSetterType_Transform,
		ERigElementGetterSetterType_Rotation,
		ERigElementGetterSetterType_Translation,
		ERigElementGetterSetterType_Initial,
		ERigElementGetterSetterType_Relative,
		ERigElementGetterSetterType_Offset,
		ERigElementGetterSetterType_Name
	};

	virtual void FilterDraggedKeys(TArray<FRigElementKey>& Keys, bool bRemoveNameSpace) override;
	void HandleMakeElementGetterSetter(ERigElementGetterSetterType Type, bool bIsGetter, TArray<FRigElementKey> Keys, UEdGraph* Graph, FVector2D NodePosition);
	void HandleMakeMetadataGetterSetter(const bool bIsGetter, const TArray<FRigElementKey> Keys, UEdGraph* Graph, FVector2D NodePosition);
	void HandleMakeComponentGetterSetter(bool bIsGetter, TArray<FRigComponentKey> Keys, UEdGraph* Graph, FVector2D NodePosition);

	void HandleOnControlModified(UControlRig* Subject, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context);

	virtual void HandleRefreshEditorFromBlueprintImpl(FRigVMEditorAssetInterfacePtr InBlueprint);

	FText GetReplayAssetName() const;
	FText GetReplayAssetTooltip() const;
	bool SetReplayAssetPath(const FString& InAssetPath);
	TSharedRef<SWidget> GenerateReplayAssetModeMenuContent();
	TSharedRef<SWidget> GenerateReplayAssetRecordMenuContent();
	TSharedRef<SWidget> GenerateReplayAssetPlaybackMenuContent();
	bool RecordReplay(double InRecordingDuration);
	void ToggleReplay();


	virtual EControlRigReplayPlaybackMode GetReplayPlaybackMode() const override;


	/** Persona toolkit used to support skeletal mesh preview */
	TSharedPtr<IPersonaToolkit> PersonaToolkit;

	/** Preview instance inspector widget */
	TSharedPtr<IPersonaViewport> PreviewViewport;

	/** preview scene */
	TSharedPtr<IPersonaPreviewScene> PreviewScene;

	/** preview animation instance */
	UAnimPreviewInstance* PreviewInstance;
	virtual UAnimPreviewInstance* GetPreviewInstance() const override { return PreviewInstance; }
	virtual void SetPreviewInstance(UAnimPreviewInstance* InPreviewInstance) override { PreviewInstance = InPreviewInstance; }

	/** Model for the schematic views */
	TSharedPtr<FControlRigSchematicModel> SchematicModel;

	/** Delegate to deal with key down evens in the viewport / editor */
	FPersonaViewportKeyDownDelegate OnKeyDownDelegate;

	/** Delgate to build the context menu for the viewport */
	FOnGetContextMenu OnGetViewportContextMenuDelegate;
	UToolMenu* HandleOnGetViewportContextMenuDelegate();
	FNewMenuCommandsDelegate OnViewportContextMenuCommandsDelegate;
	TSharedPtr<FUICommandList> HandleOnViewportContextMenuCommandsDelegate();

	/** Bone Selection related */
	FTransform GetRigElementTransform(const FRigElementKey& InElement, bool bLocal, bool bOnDebugInstance) const;
	void SetRigElementTransform(const FRigElementKey& InElement, const FTransform& InTransform, bool bLocal);
	
	/** delegate for changing property */
	virtual void OnFinishedChangingPropertiesImpl(const FPropertyChangedEvent& PropertyChangedEvent);
	virtual void OnWrappedPropertyChangedChainEventImpl(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent);

	URigVMController* ActiveController;

	/** Currently executing ControlRig or not - later maybe this will change to enum for whatever different mode*/
	bool bExecutionControlRig;

	/**
	 * When true, the editor will try to frame the rig's visible controls in the preview viewport on the next
	 * tick. Set during init when no preview mesh is assigned (e.g. Face_ControlBoard_CtrlRig) so the camera
	 * doesn't sit somewhere with no rig content visible. Cleared once the framing has been requested or after
	 * a small number of attempts (in case the rig hierarchy never populates).
	 * See: MH issue "Control Rigs displaced out of their Blueprint Viewports".
	 */
	bool bPendingInitialViewportFocus = false;
	int32 PendingInitialViewportFocusTickCount = 0;

	void OnAnimInitialized();

	bool IsConstructionModeEnabled() const;
	bool IsDebuggingExternalControlRig(const UControlRig* InControlRig = nullptr) const;
	bool ShouldExecuteControlRig(const UControlRig* InControlRig = nullptr) const;

	int32 RigHierarchyTabCount;
	int32 ModularRigHierarchyTabCount;
	TWeakObjectPtr<AStaticMeshActor> WeakGroundActorPtr;

	void OnPreForwardsSolve_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnPostForwardsSolve_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnPreConstructionForUI_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnPreConstruction_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnPostConstruction_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnEarlyExit_AnyThread(URigVMHost* InHost, URigVMHost::EEarlyExitReason InReason);
	FRigPose PreConstructionPose;
	TArray<FRigSocketState> SocketStates;
	TArray<FRigConnectorState> ConnectorStates;

	bool bIsConstructionEventRunning;
	uint32 LastHierarchyHash;

	TStrongObjectPtr<UControlRigReplay> ReplayStrongPtr;

	TWeakObjectPtr<const URigVMUnitNode> DirectManipulationSubject;
	TArray<TSharedPtr<FString>> DirectManipulationTextList;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> DirectManipulationCombo;
	bool bRefreshDirectionManipulationTargetsRequired;
	FSimpleMulticastDelegate RequestNavigateToConnectorWarningDelegate;
	TSharedPtr<SSchematicGraphPanel> SchematicViewport;
	FTimerHandle RecordReplayTimerHandle;
	FRigPose BonePoseFromLastEvaluation;

public:
	void SetupTimelineDelegates(FAnimationScrubPanelDelegates& InOutDelegates);
protected:
	bool ShowReplayOnTimeline() const;
	TOptional<bool> HandleReplayIsRecordingActive() const;
	TOptional<EVisibility> HandleGetReplayRecordButtonVisibility() const;
	bool HandleReplayStartRecording();
	bool HandleReplayStopRecording();
	TOptional<int32> HandleReplayGetPlaybackMode() const;
	bool HandleReplaySetPlaybackMode(int32 InPlaybackMode);
	TOptional<float> HandleReplayGetPlaybackTime() const;
	bool HandleReplaySetPlaybackTime(float InTime, bool bStopPlayback);
	bool HandleReplayStepForward();
	bool HandleReplayStepBackward();
	TOptional<bool> HandleReplayGetIsLooping() const;
	bool HandleReplaySetIsLooping(bool bIsLooping);
	TOptional<FVector2f> HandleReplayGetPlaybackTimeRange() const;
	TOptional<uint32> HandleReplayGetNumberOfKeys() const;
	
	EVisibility GetReplayValidationErrorVisibility() const;
	FText GetReplayValidationErrorTooltip() const;

	virtual FControlRigEditorClosed& OnEditorClosed() override { return ControlRigEditorClosedDelegate; }
	FControlRigEditorClosed ControlRigEditorClosedDelegate;

	void OnClose();

	static const TArray<FName> ForwardsSolveEventQueue;
	static const TArray<FName> BackwardsSolveEventQueue;
	static const TArray<FName> ConstructionEventQueue;
	static const TArray<FName> BackwardsAndForwardsSolveEventQueue;

	friend class FControlRigEditorMode;
	friend class FModularRigEditorMode;
	friend class SControlRigStackView;
	friend class SRigHierarchy;
	friend class SModularRigHierarchy;
	friend struct FRigHierarchyTabSummoner;
	friend struct FModularRigModelTabSummoner;
};
