// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigEditorCommands.h"

#define LOCTEXT_NAMESPACE "ControlRigEditorCommands"

void FControlRigEditorCommands::RegisterCommands()
{
	UI_COMMAND(ConstructionEvent, "Construction Event", "Enable the construction mode for the rig", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ForwardsSolveEvent, "Forwards Solve", "Run the forwards solve graph", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BackwardsSolveEvent, "Backwards Solve", "Run the backwards solve graph", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BackwardsAndForwardsSolveEvent, "Backwards and Forwards", "Run backwards solve followed by forwards solve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetNextEventQueue, "Next Event Queue", "Sets the next event queue executed on compile, for example Backwards Solve when the current queue is Forwards Solve", EUserInterfaceActionType::Button, FInputChord(EKeys::M));
	UI_COMMAND(SetPriorEventQueue, "Prior Event Queue", "Sets the prior event queue executed on compile, for example Borwards Solve when the current queue is Backwards Solve", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::M));
	UI_COMMAND(ToggleForwardsSolveAndConstruction, "Toggle Forwards and Construction", "Toggles between forwards solve and construction", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::M));
	UI_COMMAND(RequestDirectManipulationPosition, "Request Direct Manipulation for Position", "Request per node direct manipulation on a position", EUserInterfaceActionType::Button, FInputChord(EKeys::W));
	UI_COMMAND(RequestDirectManipulationRotation, "Request Direct Manipulation for Rotation", "Request per node direct manipulation on a rotation", EUserInterfaceActionType::Button, FInputChord(EKeys::E));
	UI_COMMAND(RequestDirectManipulationScale, "Request Direct Manipulation for Scale", "Request per node direct manipulation on a scale", EUserInterfaceActionType::Button, FInputChord(EKeys::R));
	UI_COMMAND(ToggleControlVisibility, "Controls", "Toggles the visibility of the controls.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::T));
	UI_COMMAND(ToggleControlsAsOverlay, "Controls as Overlay", "If checked controls will be rendered on top of other controls.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::U));
	UI_COMMAND(ToggleDrawNulls, "Nulls", "If checked all nulls are drawn as axes.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::G));
	UI_COMMAND(ToggleDrawSockets, "Sockets", "If checked all sockets are drawn.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::H));
	UI_COMMAND(ToggleDrawAxesOnSelection, "Axes On Selection", "If checked axes will be drawn for all selected rig elements.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::J));
	UI_COMMAND(ToggleSchematicViewportVisibility, "Schematic Viewport", "Toggles the visibility of the viewport schematic", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::Y));
	UI_COMMAND(ToggleSchematicViewportShowEmptySocketsOnly, "Empty Sockets Only", "Toggles if the schematic viewport shows all sockets or empty sockets only", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SwapModuleWithinAsset, "Swap Module (Asset)", "Swaps a module for all occurrences within this asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SwapModuleAcrossProject, "Swap Module (Project)", "Swaps a module for all occurrences in the project.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleCompileResetsEventQueue, "Compile Resets Event Queue", "When enabled, the Event Queue is reset to Forward Solve after compiling.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
