// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Editor/RigVMEditorStyle.h"

class FControlRigEditorCommands : public TCommands<FControlRigEditorCommands>
{
public:
	FControlRigEditorCommands() : TCommands<FControlRigEditorCommands>
	(
		"ControlRigBlueprint",
		NSLOCTEXT("Contexts", "Animation", "Rig Blueprint"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FRigVMEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	/** Enable the construction mode for the rig */
	TSharedPtr< FUICommandInfo > ConstructionEvent;

	/** Run the forwards solve graph */
	TSharedPtr< FUICommandInfo > ForwardsSolveEvent;

	/** Run the backwards solve graph */
	TSharedPtr< FUICommandInfo > BackwardsSolveEvent;

	/** Run the backwards solve graph followed by the forwards solve graph */
	TSharedPtr< FUICommandInfo > BackwardsAndForwardsSolveEvent;

	UE_DEPRECATED(5.8, "Instead use SetNextEventQueue")
	TSharedPtr< FUICommandInfo > SetNextSolveMode;

	/** Sets the next Event Queue, for example Backwards Solve if the last entry in the Queue is ForwardsSolve */
	TSharedPtr< FUICommandInfo > SetNextEventQueue;

	/** Sets the prior Event Queue */
	TSharedPtr< FUICommandInfo > SetPriorEventQueue;

	/** Toggles between Forwards Solve and Construction */
	TSharedPtr< FUICommandInfo > ToggleForwardsSolveAndConstruction;

	/** Request per node direct manipulation on a position */
	TSharedPtr< FUICommandInfo > RequestDirectManipulationPosition;

	/** Request per node direct manipulation on a rotation */
	TSharedPtr< FUICommandInfo > RequestDirectManipulationRotation;

	/** Request per node direct manipulation on a scale */
	TSharedPtr< FUICommandInfo > RequestDirectManipulationScale;

	/** Toggle visibility of the controls */
	TSharedPtr< FUICommandInfo > ToggleControlVisibility;

	/** Toggle if controls should be rendered on top of other controls */
	TSharedPtr< FUICommandInfo > ToggleControlsAsOverlay;

	/** Toggle visibility of nulls */
	TSharedPtr< FUICommandInfo > ToggleDrawNulls;

	/** Toggle visibility of sockets */
	TSharedPtr< FUICommandInfo > ToggleDrawSockets;

	/** Toggle visibility of axes on selection */
	TSharedPtr< FUICommandInfo > ToggleDrawAxesOnSelection;

	/** Toggle visibility of the schematic */
	TSharedPtr< FUICommandInfo > ToggleSchematicViewportVisibility;

	/** Toggles if the schematic viewport shows all sockets or empty sockets only */
	TSharedPtr< FUICommandInfo > ToggleSchematicViewportShowEmptySocketsOnly;

	/** Swap Module (Asset) */
	TSharedPtr< FUICommandInfo > SwapModuleWithinAsset;

	/** Swap Module (Project) */
	TSharedPtr< FUICommandInfo > SwapModuleAcrossProject;

	/** Toggles if the Event Queue is reset to Forward Solve after compiling */
	TSharedPtr< FUICommandInfo > ToggleCompileResetsEventQueue;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
