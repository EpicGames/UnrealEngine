// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigEditorEditMode.h"
#include "AnimationEditorViewportClient.h"
#include "IPersonaPreviewScene.h"
#include "AssetEditorModeManager.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigGizmoActor.h"
#include "Preferences/PersonaOptions.h"
#include "Settings/ControlRigSettings.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "SkeletalDebugRendering.h"

FControlRigEditorEditMode::FControlRigEditorEditMode()
{
	bShowControlsAsOverlay = IsControlsAsOverlayEnabled();
}

void FControlRigEditorEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FControlRigEditMode::Render(View, Viewport, PDI);

	if (ConfigOption == nullptr)
	{
		ConfigOption = UPersonaOptions::StaticClass()->GetDefaultObject<UPersonaOptions>();
	}
	EBoneDrawMode::Type BoneDrawMode = (EBoneDrawMode::Type)ConfigOption->DefaultBoneDrawSelection;

	// is the viewport configured to draw bones AND should Control Rig draw bones at all (only if Skeletal Mesh not present)
	const bool bShouldDrawBones = bDrawHierarchyBones && BoneDrawMode != EBoneDrawMode::None;
	if (!bShouldDrawBones)
	{
		return;
	}

	// is there a control rig to edit?
	TArray<UControlRig*> ControlRigs = GetControlRigsArray(true );
	if (ControlRigs.IsEmpty())
	{
		return;
	}

	UControlRig* ControlRig = ControlRigs[0]; //just one control rig in the CR asset editor
	if (!ControlRig) 
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	URigHierarchy* HierarchyForSelection = Hierarchy;
	
	if (FControlRigAssetInterfacePtr EditorAsset = ControlRig->GetAssetReference().GetEditorAsset())
	{
		HierarchyForSelection = EditorAsset->GetHierarchy();
	}

	// maintain 3 masks for drawing bones depending on viewport settings, selected and affected states
	TBitArray<> BonesToDraw(false, Hierarchy->Num());
	TBitArray<> SelectedBones(false, Hierarchy->Num());
	TBitArray<> AffectedBones(false, Hierarchy->Num());

	// get the viewport states to filter which bones to draw at all
	const bool bDrawAllBones = BoneDrawMode == EBoneDrawMode::All;
	const bool bDrawSelected = BoneDrawMode == EBoneDrawMode::Selected;
	const bool bDrawSelectedAndParents = BoneDrawMode == EBoneDrawMode::SelectedAndParents;
	const bool bDrawSelectedAndChildren = BoneDrawMode == EBoneDrawMode::SelectedAndChildren;
	const bool bDrawSelectedAndParentsAndChildren = BoneDrawMode == EBoneDrawMode::SelectedAndParentsAndChildren;
	
	// determine which elements are bones and optionally which are selected
	for (int32 ElementIndex = 0; ElementIndex < Hierarchy->Num(); ElementIndex++)
	{
		FRigBoneElement* BoneElement = Cast<FRigBoneElement>(Hierarchy->Get(ElementIndex));
		if(!BoneElement)
		{
			continue;
		}

		// Only draw procedurally generated bones, other bones are painted by persona
		const bool bPaintedByPersona = !Hierarchy->IsProcedural(BoneElement);
		if (bPaintedByPersona)
		{
			continue;
		}

		const int32 BoneIndex = BoneElement->GetIndex();

		// optionally draw ALL bones
		if (bDrawAllBones)
		{
			BonesToDraw[BoneIndex] = true;
		}

		// record which bones are selected
		const bool bIsSelected = HierarchyForSelection->IsSelected(BoneElement->GetKey());
		if (bIsSelected)
		{
			SelectedBones[BoneIndex] = true;
		}
		
		// draw selected bones (optionally)
		const bool bFilterOnlySelected = bDrawSelected || bDrawSelectedAndParents || bDrawSelectedAndChildren || bDrawSelectedAndParentsAndChildren;
		if (bIsSelected && bFilterOnlySelected)
		{
			BonesToDraw[BoneIndex] = true;
		
			// add children of selected to list of bones to draw (optionally)
			if (bDrawSelectedAndChildren || bDrawSelectedAndParentsAndChildren)
			{
				FRigBaseElementChildrenArray AllChildren = Hierarchy->GetChildren(BoneElement, true);
				for (FRigBaseElement* Child : AllChildren)
				{
					if(FRigBoneElement* ChildBone = Cast<FRigBoneElement>(Child))
					{
						BonesToDraw[ChildBone->GetIndex()] = true;
					}
				}
			}
		}
	}

	// add parents of selected (optionally)
	if (bDrawSelectedAndParents || bDrawSelectedAndParentsAndChildren)
	{
		for (int32 ElementIndex=0; ElementIndex<Hierarchy->Num(); ++ElementIndex)
		{
			if (!SelectedBones[ElementIndex])
			{
				continue;
			}
			
			int32 ParentIndex = Hierarchy->GetFirstParent(ElementIndex);
			while (ParentIndex != INDEX_NONE)
			{
				BonesToDraw[ParentIndex] = true;
				ParentIndex = Hierarchy->GetFirstParent(ParentIndex);
			}
		}
	}

	// determine bones are "affected" (child of a selected bone, but not selected themselves)
	Hierarchy->ForEach<FRigBoneElement>([this, Hierarchy, &SelectedBones, &AffectedBones](FRigBoneElement* BoneElement) -> bool
	{
		const int32 BoneIndex = BoneElement->GetIndex();
		int32 ParentIndex = Hierarchy->GetFirstParent(BoneIndex);
		while (ParentIndex != INDEX_NONE)
		{
			if (SelectedBones[ParentIndex])
			{
				AffectedBones[BoneIndex] = true;
				return true;
			}
			ParentIndex = Hierarchy->GetFirstParent(ParentIndex);
		}

		return true;
	});

	// get size setting for bone drawing
	FBoneAxisDrawConfig AxisConfig;
	float BoneRadius = 1.0f;
	if (FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(Viewport->GetClient()))
	{
		BoneRadius = AnimViewportClient->GetBoneDrawSize();
	}

	// use colors from user preferences
	const UPersonaOptions* PersonaOptions = GetDefault<UPersonaOptions>();

	// spin through all bones and render them
	TArray<FVector> ChildPositions;
	TArray<FLinearColor> ChildColors;
	TArray<int32> ChildrenIndices;
	for ( int32 ElementIndex=0; ElementIndex<BonesToDraw.Num(); ++ElementIndex )
	{
		// skips bones that should not be drawn
		if (!BonesToDraw[ElementIndex])
		{
			continue;
		}

		// determine color of bone based on selection / affected state
		const bool bIsSelected = SelectedBones[ElementIndex];
		const bool bIsAffected = AffectedBones[ElementIndex];
		FLinearColor DefaultBoneColor = PersonaOptions->DefaultBoneColor;
		FLinearColor BoneColor = bIsAffected ? PersonaOptions->AffectedBoneColor : DefaultBoneColor;
		BoneColor = bIsSelected ? PersonaOptions->SelectedBoneColor : BoneColor;

		// draw the little coordinate frame inside the bone ONLY if selected or affected
		AxisConfig.bDraw = bIsAffected|| bIsSelected;

		// draw cone to each child
		// but use a different color if this bone is NOT selected, but the child IS selected
		ChildPositions.Reset();
		ChildColors.Reset();
		ChildrenIndices = Hierarchy->GetChildren(ElementIndex);
		for (int32 ChildIndex : ChildrenIndices)
		{
			FTransform ChildTransform = Hierarchy->GetGlobalTransform(ChildIndex);
			ChildPositions.Add(ChildTransform.GetLocation());
			FLinearColor ChildLineColor = BoneColor;
			if (!bIsSelected && SelectedBones[ChildIndex])
			{
				ChildLineColor = PersonaOptions->ParentOfSelectedBoneColor;
			}
			ChildColors.Add(ChildLineColor);
		}

		const FName BoneName = Hierarchy->GetKey(ElementIndex).Name;
		const FTransform BoneTransform = Hierarchy->GetGlobalTransform(ElementIndex);
		
		PDI->SetHitProxy(new HPersonaBoneHitProxy(ElementIndex, BoneName));
		SkeletalDebugRendering::DrawWireBoneAdvanced(
			PDI,
			BoneTransform,
			ChildPositions,
			ChildColors,
			BoneColor,
			SDPG_Foreground,
			BoneRadius,
			AxisConfig);
		if (Hierarchy->GetFirstParent(ElementIndex) == INDEX_NONE)
		{
			SkeletalDebugRendering::DrawConeConnection(PDI, BoneTransform.GetLocation(), FVector::Zero(), BoneRadius, FLinearColor::Red);
		}
		PDI->SetHitProxy(nullptr);
	}
}

bool FControlRigEditorEditMode::GetCameraTarget(FSphere& OutTarget) const
{
	FTransform ComponentToWorld = FTransform::Identity;
	if(const USceneComponent* SceneComponent = GetHostingSceneComponent())
	{
		ComponentToWorld = SceneComponent->GetComponentToWorld();
	}

	FBox Box(ForceInit);
	TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements();
	for (int32 Index = 0; Index < SelectedRigElements.Num(); ++Index)
	{
		static const float Radius = 20.f;
		if (SelectedRigElements[Index].Type == ERigElementType::Bone || SelectedRigElements[Index].Type == ERigElementType::Null)
		{
			FTransform Transform = OnGetRigElementTransformDelegate.Execute(SelectedRigElements[Index], false, true);
			Transform = Transform * ComponentToWorld;
			Box += Transform.TransformPosition(FVector::OneVector * Radius);
			Box += Transform.TransformPosition(FVector::OneVector * -Radius);
		}
		else if (SelectedRigElements[Index].Type == ERigElementType::Control)
		{
			FTransform Transform = OnGetRigElementTransformDelegate.Execute(SelectedRigElements[Index], false, true);
			Transform = Transform * ComponentToWorld;
			Box += Transform.TransformPosition(FVector::OneVector * Radius);
			Box += Transform.TransformPosition(FVector::OneVector * -Radius);
		}
	}

	if(Box.IsValid)
	{
		OutTarget.Center = Box.GetCenter();
		OutTarget.W = Box.GetExtent().GetAbsMax() * 1.25f;
		return true;
	}

	return false;
}

IPersonaPreviewScene& FControlRigEditorEditMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FControlRigEditorEditMode::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{

}

bool FControlRigEditorEditMode::HandleBeginTransform(const FEditorViewportClient* InViewportClient)
{
	const bool bResult = FControlRigEditMode::HandleBeginTransform(InViewportClient);

	for (TWeakObjectPtr<UControlRig>& RuntimeRigPtr : RuntimeControlRigs)
	{
		if (UControlRig* ControlRig = RuntimeRigPtr.Get())
		{
			TArray<FRigElementKey> SelectedRigElements = GetSelectedRigElements(ControlRig);
			SelectedRigElements = SelectedRigElements.FilterByPredicate([](FRigElementKey Key) { return Key.Type == ERigElementType::Control; });
			for (const FRigElementKey& Key : SelectedRigElements)
			{
				ModifiedRigElements.AddUnique(Key);
			}
		}
	}
	
	return bResult;
}

bool FControlRigEditorEditMode::IsDisplayControlsEnabled() const
{
	return !GetDefault<UControlRigEditorSettings>()->bHideControlShapes;
}

void FControlRigEditorEditMode::ToggleDisplayControlsEnabled()
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	Settings->bHideControlShapes = !Settings->bHideControlShapes;
	Settings->SaveConfig();
}

bool FControlRigEditorEditMode::IsControlsAsOverlayEnabled() const
{
	return GetDefault<UControlRigEditorSettings>()->bShowControlsAsOverlay;
}

void FControlRigEditorEditMode::ToggleControlsAsOverlay()
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	Settings->bShowControlsAsOverlay = !Settings->bShowControlsAsOverlay;
	Settings->SaveConfig();
	UControlRigEditModeSettings::OnSettingsChange.Broadcast(GetDefault<UControlRigEditModeSettings>());
}

bool FControlRigEditorEditMode::IsDisplayNullsEnabled() const
{
	return GetDefault<UControlRigEditorSettings>()->bDisplayNulls;
}

void FControlRigEditorEditMode::ToggleDisplayNulls()
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	Settings->bDisplayNulls = !Settings->bDisplayNulls;
	Settings->SaveConfig();
}

bool FControlRigEditorEditMode::IsDisplaySocketsEnabled() const
{
	return GetDefault<UControlRigEditorSettings>()->bDisplaySockets;
}

void FControlRigEditorEditMode::ToggleDisplaySockets()
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	Settings->bDisplaySockets = !Settings->bDisplaySockets;
	Settings->SaveConfig();
}

bool FControlRigEditorEditMode::IsDisplayAxesOnSelectionEnabled() const
{
	return GetDefault<UControlRigEditorSettings>()->bDisplayAxesOnSelection;
}

void FControlRigEditorEditMode::ToggleDisplayAxesOnSelection()
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	Settings->bDisplayAxesOnSelection = !Settings->bDisplayAxesOnSelection;
	Settings->SaveConfig();
}

float FControlRigEditorEditMode::GetAxesOnSelectionScale() const
{
	return GetDefault<UControlRigEditorSettings>()->AxisScale;
}
