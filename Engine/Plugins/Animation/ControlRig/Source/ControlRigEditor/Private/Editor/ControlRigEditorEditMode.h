// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditMode/ControlRigEditMode.h"

class UPersonaOptions;

class FControlRigEditorEditMode : public FControlRigEditMode
{
public:
	FControlRigEditorEditMode();
	static inline const FLazyName ModeName = FLazyName(TEXT("EditMode.ControlRigEditor"));
	
	virtual bool IsInLevelEditor() const override { return false; }
	virtual bool AreEditingControlRigDirectly() const override { return true; }

	// FEdMode interface
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	/* IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;

	/* FControlRigEditMode interface */
	virtual bool HandleBeginTransform(const FEditorViewportClient* InViewportClient) override;

	virtual bool IsDisplayControlsEnabled() const override;
	virtual void ToggleDisplayControlsEnabled() override;;

	virtual bool IsControlsAsOverlayEnabled() const override;;
	virtual void ToggleControlsAsOverlay() override;;

	virtual bool IsDisplayNullsEnabled() const override;;
	virtual void ToggleDisplayNulls() override;;

	virtual bool IsDisplaySocketsEnabled() const override;;
	virtual void ToggleDisplaySockets() override;

	virtual bool IsDisplayAxesOnSelectionEnabled() const override;;
	virtual void ToggleDisplayAxesOnSelection() override;;

	virtual float GetAxesOnSelectionScale() const override;;

	/** If set to true the edit mode will additionally render all bones */
	bool bDrawHierarchyBones = false;

	/** Drawing options */
	UPersonaOptions* ConfigOption = nullptr;

	// Elements which have been modified (e.g. controls moved)
	TArray<FRigElementKey> ModifiedRigElements;
};

class FModularRigEditorEditMode : public FControlRigEditorEditMode
{
public:

	static inline const FLazyName ModeName = FLazyName(TEXT("EditMode.ModularRigEditor"));
};
