// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigEditorMode.h"
#if WITH_RIGVMLEGACYEDITOR
#include "BlueprintEditorTabs.h"
#include "SBlueprintEditorToolbar.h"
#endif 
#include "PersonaModule.h"
#include "IPersonaToolkit.h"
#include "PersonaTabs.h"
#include "Editor/RigHierarchyTabSummoner.h"
#include "Editor/ModularRigModelTabSummoner.h"
#include "Editor/RigModuleAssetBrowserTabSummoner.h"
#include "Editor/RigVMExecutionStackTabSummoner.h"
#include "Editor/RigVMEditorGraphExplorerTabSummoner.h"
#include "Editor/RigCurveContainerTabSummoner.h"
#include "Editor/RigValidationTabSummoner.h"
#include "Editor/RigAnimAttributeTabSummoner.h"
#include "Editor/ModularRigEventQueueTabSummoner.h"
#include "ToolMenus.h"
#include "Editor/RigVMCompilerResultsTabSummoner.h"
#include "Editor/RigVMDetailsInspectorTabSummoner.h"
#include "Editor/RigVMFindReferencesTabSummoner.h"
#include "Editor/RigOverrideListTabSummoner.h"
#include "Editor/RigVMGraphEditorTabSummoner.h"
#include "RigDependencyGraph/RigDependencyGraphSummoner.h"

#if WITH_RIGVMLEGACYEDITOR
FControlRigLegacyEditorMode::FControlRigLegacyEditorMode(const TSharedRef<FControlRigLegacyEditor>& InControlRigEditor, bool bCreateDefaultLayout)
	: FBlueprintEditorApplicationMode(InControlRigEditor.ToSharedPtr(), FControlRigEditorModes::ControlRigEditorMode, FControlRigEditorModes::GetLocalizedMode, false, false)
{
	ControlRigBlueprintPtr = Cast<UControlRigBlueprint>(InControlRigEditor->GetControlRigAssetInterface().GetObject()); 

	TabFactories.RegisterFactory(MakeShared<FRigHierarchyTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMExecutionStackTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMEditorGraphExplorerTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigCurveContainerTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigValidationTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigAnimAttributeTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigDependencyGraphSummoner>(InControlRigEditor));

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

 	FPersonaViewportArgs ViewportArgs(InControlRigEditor->GetPersonaToolkit()->GetPreviewScene());
 	ViewportArgs.BlueprintEditor = InControlRigEditor;
 	ViewportArgs.bShowStats = false;
	ViewportArgs.bShowPlaySpeedMenu = false;
	ViewportArgs.bShowTimeline = true;
	ViewportArgs.bShowTurnTable = false;
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(InControlRigEditor, &FControlRigBaseEditor::HandleViewportCreated);

	InControlRigEditor->SetupTimelineDelegates(ViewportArgs.TimelineDelegates);

	TSharedRef<FWorkflowCentricApplication> EditorSharedRef = StaticCastSharedRef<FWorkflowCentricApplication>(InControlRigEditor->GetHostingApp().ToSharedRef());
 	TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(EditorSharedRef, ViewportArgs));
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(EditorSharedRef, InControlRigEditor->GetPersonaToolkit()->GetPreviewScene()));

	if(bCreateDefaultLayout)
	{
		TabLayout = FTabManager::NewLayout("Standalone_ControlRigEditMode_Layout_v1.9")
			->AddArea
			(
				// Main application area
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.2f)
						->Split
						(
							//	Left top - viewport
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->SetHideTabWell(true)
							->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
						
						)
						->Split
						(
							//	Left bottom - rig/hierarchy
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(FRigHierarchyTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FRigVMExecutionStackTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FRigCurveContainerTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
						)
					)
					->Split
					(
						// Middle 
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.6f)
						->Split
						(
							// Middle top - document edit area
							FTabManager::NewStack()
							->SetSizeCoefficient(0.8f)
							->AddTab("GraphEditor", ETabState::ClosedTab)
						)
						->Split
						(
							// Middle bottom - compiler results & find
							FTabManager::NewStack()
							->SetSizeCoefficient(0.2f)
							->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab)
							->AddTab(FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab)
						)
					)
					->Split
					(
						// Right side
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.2f)
						->Split
						(
							// Right top
							FTabManager::NewStack()
							->SetHideTabWell(false)
							->SetSizeCoefficient(1.f)
							->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
							->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
							->AddTab(FRigAnimAttributeTabSummoner::TabID, ETabState::OpenedTab)
							->SetForegroundTab(FBlueprintEditorTabs::DetailsID)
						)
					)
				)
			);
	}

	if (UToolMenu* Toolbar = InControlRigEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InControlRigEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InControlRigEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		InControlRigEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
	}
}

void FControlRigLegacyEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
	BP->PushTabFactories(TabFactories);
}


FModularRigLegacyEditorMode::FModularRigLegacyEditorMode(const TSharedRef<FControlRigLegacyEditor>& InControlRigEditor)
	: FControlRigLegacyEditorMode(InControlRigEditor, false)
{

	TabFactories.RegisterFactory(MakeShared<FModularRigModelTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigModuleAssetBrowserTabSummoner>(InControlRigEditor->GetHostingApp()));
	TabFactories.RegisterFactory(MakeShared<FModularRigEventQueueTabSummoner>(InControlRigEditor));

	TabLayout = FTabManager::NewLayout("Standalone_ModularRigEditMode_Layout_v1.5")
		->AddArea
		(
			// Main application area
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						// Left top - Module Library
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
						->AddTab(FRigModuleAssetBrowserTabSummoner::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FBlueprintEditorTabs::DetailsID)
					)
					->Split
					(
						//	Left bottom - rig/hierarchy/modules
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
						->SetForegroundTab(FModularRigModelTabSummoner::TabID)
						->AddTab(FModularRigModelTabSummoner::TabID, ETabState::OpenedTab)
						->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
					)
				)
				->Split
				(
					//	Center - viewport
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->SetHideTabWell(true)
					->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
				)
				->Split
				(
					// Right side
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->SetHideTabWell(false)
					->SetForegroundTab(FBlueprintEditorTabs::DetailsID)
					->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
					->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
					->AddTab(FRigHierarchyTabSummoner::TabID, ETabState::OpenedTab)
				)
			)
		);
}

void FModularRigLegacyEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	static const TArray<FName> DisallowedTabs = {
		FBlueprintEditorTabs::PaletteID,
		FBlueprintEditorTabs::ReplaceNodeReferencesID,
		FBlueprintEditorTabs::CompilerResultsID,
		FBlueprintEditorTabs::FindResultsID,
		FBlueprintEditorTabs::BookmarksID,
		FRigVMExecutionStackTabSummoner::TabID
	};

	auto PushTabFactories = [&](FWorkflowAllowedTabSet& Tabs)
	{
		for (auto FactoryIt = Tabs.CreateIterator(); FactoryIt; ++FactoryIt)
		{
			if (DisallowedTabs.Contains(FactoryIt->Key))
			{
				continue;
			}
			FactoryIt.Value()->RegisterTabSpawner(InTabManager.ToSharedRef(), BP->GetCurrentModePtr().Get());
		}
	};

	// Mode-specific setup
#if WITH_RIGVMLEGACYEDITOR
	PushTabFactories(CoreTabFactories);
	PushTabFactories(BlueprintEditorTabFactories);
#endif
	PushTabFactories(TabFactories);
}
#endif

FControlRigEditorMode::FControlRigEditorMode(const TSharedRef<FControlRigEditor>& InControlRigEditor, bool bCreateDefaultLayout)
	: FRigVMNewEditorMode(InControlRigEditor)
{
	ControlRigBlueprintPtr = InControlRigEditor->GetControlRigAssetInterface();
	
	TabFactories.RegisterFactory(MakeShared<FRigHierarchyTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigCurveContainerTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigValidationTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigAnimAttributeTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigDependencyGraphSummoner>(InControlRigEditor));

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

 	FPersonaViewportArgs ViewportArgs(InControlRigEditor->GetPersonaToolkit()->GetPreviewScene());
 	//ViewportArgs.BlueprintEditor = InControlRigEditor;
 	ViewportArgs.bShowStats = false;
	ViewportArgs.bShowPlaySpeedMenu = false;
	ViewportArgs.bShowTimeline = true;
	ViewportArgs.bShowTurnTable = false;
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(InControlRigEditor, &FControlRigBaseEditor::HandleViewportCreated);

	InControlRigEditor->SetupTimelineDelegates(ViewportArgs.TimelineDelegates);

	TSharedRef<FWorkflowCentricApplication> EditorSharedRef = StaticCastSharedRef<FWorkflowCentricApplication>(InControlRigEditor->GetHostingApp().ToSharedRef());
 	TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(EditorSharedRef, ViewportArgs));
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(EditorSharedRef, InControlRigEditor->GetPersonaToolkit()->GetPreviewScene()));

	if(bCreateDefaultLayout)
	{
		TabLayout = FTabManager::NewLayout("Standalone_ControlRigEditMode_Layout_v1.10")
			->AddArea
			(
				// Main application area
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.2f)
						->Split
						(
							//	Left top - viewport
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->SetHideTabWell(true)
							->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
						
						)
						->Split
						(
							//	Left bottom - rig/hierarchy
							FTabManager::NewStack()
							->SetSizeCoefficient(0.5f)
							->AddTab(FRigHierarchyTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FRigVMExecutionStackTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FRigCurveContainerTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FRigVMEditorGraphExplorerTabSummoner::TabID(), ETabState::OpenedTab)
						)
					)
					->Split
					(
						// Middle 
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.6f)
						->Split
						(
							// Middle top - document edit area
							FTabManager::NewStack()
							->SetSizeCoefficient(0.8f)
							->AddTab(FRigVMGraphEditorTabSummoner::TabID(), ETabState::ClosedTab)
						)
						->Split
						(
							// Middle bottom - compiler results & find
							FTabManager::NewStack()
							->SetSizeCoefficient(0.2f)
							->AddTab(FRigVMCompilerResultsTabSummoner::TabID(), ETabState::OpenedTab)
							->AddTab(FRigVMFindReferencesTabSummoner::TabID(), ETabState::OpenedTab)
						)
					)
					->Split
					(
						// Right side
						FTabManager::NewSplitter()
						->SetOrientation(Orient_Vertical)
						->SetSizeCoefficient(0.2f)
						->Split
						(
							// Right top
							FTabManager::NewStack()
							->SetHideTabWell(false)
							->SetSizeCoefficient(1.f)
							->AddTab(FRigVMDetailsInspectorTabSummoner::TabID(), ETabState::OpenedTab)
							->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
							->AddTab(FRigAnimAttributeTabSummoner::TabID, ETabState::OpenedTab)
							->SetForegroundTab(FRigVMDetailsInspectorTabSummoner::TabID())
						)
					)
				)
			);
	}
}

void FControlRigEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	FRigVMNewEditorMode::RegisterTabFactories(InTabManager);
}


FModularRigEditorMode::FModularRigEditorMode(const TSharedRef<FControlRigEditor>& InControlRigEditor)
	: FControlRigEditorMode(InControlRigEditor, false)
{

	TabFactories.RegisterFactory(MakeShared<FModularRigModelTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FRigModuleAssetBrowserTabSummoner>(InControlRigEditor));
	TabFactories.RegisterFactory(MakeShared<FModularRigEventQueueTabSummoner>(InControlRigEditor));

	TabLayout = FTabManager::NewLayout("Standalone_ModularRigEditMode_Layout_v1.3")
		->AddArea
		(
			// Main application area
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						// Left top - Module Library
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
						->AddTab(FRigModuleAssetBrowserTabSummoner::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FRigVMDetailsInspectorTabSummoner::TabID())
					)
					->Split
					(
						//	Left bottom - rig/hierarchy/modules
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
						->SetForegroundTab(FModularRigModelTabSummoner::TabID)
						->AddTab(FModularRigModelTabSummoner::TabID, ETabState::OpenedTab)
						->AddTab(FRigVMEditorGraphExplorerTabSummoner::TabID(), ETabState::OpenedTab)
					)
				)
				->Split
				(
					//	Center - viewport
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->SetHideTabWell(true)
					->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
				)
				->Split
				(
					// Right side
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->SetHideTabWell(false)
					->SetForegroundTab(FRigVMDetailsInspectorTabSummoner::TabID())
					->AddTab(FRigVMDetailsInspectorTabSummoner::TabID(), ETabState::OpenedTab)
					->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
					->AddTab(FRigHierarchyTabSummoner::TabID, ETabState::OpenedTab)
				)
			)
		);
}

void FModularRigEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FRigVMNewEditor> BP = Editor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	static const TArray<FName> DisallowedTabs = {
		FRigVMExecutionStackTabSummoner::TabID
	};

	auto PushTabFactories = [&](FWorkflowAllowedTabSet& Tabs)
	{
		for (auto FactoryIt = Tabs.CreateIterator(); FactoryIt; ++FactoryIt)
		{
			if (DisallowedTabs.Contains(FactoryIt->Key))
			{
				continue;
			}
			FactoryIt.Value()->RegisterTabSpawner(InTabManager.ToSharedRef(), BP->GetCurrentModePtr().Get());
		}
	};

	// Mode-specific setup
	PushTabFactories(TabFactories);
}