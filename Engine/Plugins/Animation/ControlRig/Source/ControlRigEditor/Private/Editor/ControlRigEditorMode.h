// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "Editor/ControlRigEditor.h"
#include "ControlRigBlueprintLegacy.h"
#if WITH_RIGVMLEGACYEDITOR
#include "BlueprintEditorModes.h"
#include "Editor/ControlRigLegacyEditor.h"
#endif
#include "Editor/RigVMNewEditorMode.h" 
#include "Editor/ControlRigNewEditor.h"

class UControlRigBlueprint;

#if WITH_RIGVMLEGACYEDITOR
class FControlRigLegacyEditorMode : public FBlueprintEditorApplicationMode
{
public:
	FControlRigLegacyEditorMode(const TSharedRef<FControlRigLegacyEditor>& InControlRigEditor, bool bCreateDefaultLayout = true);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

protected:
	// Set of spawnable tabs
	FWorkflowAllowedTabSet TabFactories;

private:
	TWeakObjectPtr<UControlRigBlueprint> ControlRigBlueprintPtr;
};

class FModularRigLegacyEditorMode : public FControlRigLegacyEditorMode
{
public:
	FModularRigLegacyEditorMode(const TSharedRef<FControlRigLegacyEditor>& InControlRigEditor);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

	// for now just don't open up the previous edited documents
	virtual void PostActivateMode() override {}
};
#endif

class FControlRigEditorMode : public FRigVMNewEditorMode
{
public:
	FControlRigEditorMode(const TSharedRef<FControlRigEditor>& InControlRigEditor, bool bCreateDefaultLayout = true);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

private:
	TWeakInterfacePtr<IControlRigEditorAssetInterface> ControlRigBlueprintPtr;
};

class FModularRigEditorMode : public FControlRigEditorMode
{
public:
	FModularRigEditorMode(const TSharedRef<FControlRigEditor>& InControlRigEditor);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

	// for now just don't open up the previous edited documents
	virtual void PostActivateMode() override {}
};

