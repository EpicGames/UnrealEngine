// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigLegacyEditor.h"

#if WITH_RIGVMLEGACYEDITOR
#include "ControlRigContextMenuContext.h"
#include "ControlRigEditorMode.h"
#include "ControlRigEditorStyle.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "ToolMenuContext.h"


FControlRigLegacyEditor::FControlRigLegacyEditor()
: IControlRigLegacyEditor()
, FControlRigBaseEditor()
{
	LastEventQueue = ConstructionEventQueue;
}

FControlRigLegacyEditor::~FControlRigLegacyEditor()
{
	UControlRigBlueprint* RigBlueprint = GetControlRigBlueprint();

	if (RigBlueprint)
	{
		UControlRigBlueprint::sCurrentlyOpenedRigBlueprints.Remove(RigBlueprint);

		RigBlueprint->OnHierarchyModified().RemoveAll(this);
		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			RigBlueprint->OnHierarchyModified().RemoveAll(EditMode);
			EditMode->OnEditorClosed();
		}

		RigBlueprint->OnRigTypeChanged().RemoveAll(this);
		if (RigBlueprint->IsModularRig())
		{
			RigBlueprint->GetModularRigController()->OnModified().RemoveAll(this);
			RigBlueprint->OnModularRigCompiled().RemoveAll(this);

			RigBlueprint->IRigVMEditorAssetInterface::OnSetObjectBeingDebugged().RemoveAll(&SchematicModel);
			RigBlueprint->OnHierarchyModified().RemoveAll(&SchematicModel);
			RigBlueprint->GetModularRigController()->OnModified().RemoveAll(&SchematicModel);
		}
	}

	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}

TSharedPtr<FApplicationMode> FControlRigLegacyEditor::CreateEditorMode()
{
	CreatePersonaToolKitIfRequired();

	if(IsModularRig())
	{
		return MakeShareable(new FModularRigLegacyEditorMode(SharedThis(this)));
	}
	return MakeShareable(new FControlRigLegacyEditorMode(SharedThis(this)));
}

bool FControlRigLegacyEditor::IsSectionVisible(NodeSectionID::Type InSectionID) const
{
	if(!IControlRigLegacyEditor::IsSectionVisible(InSectionID))
	{
		return false;
	}

	if(const UControlRigBlueprint* RigBlueprint = GetControlRigBlueprint())
	{
		if(IsModularRig())
		{
			switch (InSectionID)
			{
				case NodeSectionID::GRAPH:
				{
					return RigBlueprint->SupportsEventGraphs();
				}
				case NodeSectionID::FUNCTION:
				{
					return RigBlueprint->SupportsFunctions();
				}
				default:
				{
					break;
				}
			}
		}
	}
	return true;
}

bool FControlRigLegacyEditor::NewDocument_IsVisibleForType(FBlueprintEditor::ECreatedDocumentType GraphType) const
{
	if(!IControlRigLegacyEditor::NewDocument_IsVisibleForType(GraphType))
	{
		return false;
	}

	if(const UControlRigBlueprint* RigBlueprint = GetControlRigBlueprint())
	{
		if(IsModularRig())
		{
			switch(GraphType)
			{
				case FBlueprintEditor::CGT_NewEventGraph:
				{
					return RigBlueprint->SupportsEventGraphs();
				}
				case FBlueprintEditor::CGT_NewFunctionGraph:
				{
					return RigBlueprint->SupportsFunctions();
				}
				default:
				{
					break;
				}
			}
		}
	}

	return true;
}

void FControlRigLegacyEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FRigVMLegacyEditor::InitToolMenuContext(MenuContext);

	if (FRigVMEditorAssetInterfacePtr RigVMAsset = GetRigVMAssetInterface())
	{
		UControlRigContextMenuContext* ContextObject = NewObject<UControlRigContextMenuContext>();
		ContextObject->Init(SharedThis(this));

		MenuContext.AddObject(ContextObject);
	}
}

#endif
