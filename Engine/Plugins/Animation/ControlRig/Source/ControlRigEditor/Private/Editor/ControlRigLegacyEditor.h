// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_RIGVMLEGACYEDITOR
#include "ControlRigEditor.h"
#include "Editor/RigVMEditor.h"
#include "Types/SlateVector2.h"

struct FToolMenuContext;

class FControlRigLegacyEditor : public IControlRigLegacyEditor, public FControlRigBaseEditor
{
public:

	
	virtual void InitRigVMEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint) override { return FControlRigBaseEditor::InitRigVMEditorImpl(Mode, InitToolkitHost, InRigVMBlueprint); }
	virtual void InitRigVMEditorSuper(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint) override { return IControlRigLegacyEditor::InitRigVMEditor(Mode, InitToolkitHost, InRigVMBlueprint); }

	virtual const FName GetEditorAppName() const override { return FControlRigBaseEditor::GetEditorAppNameImpl(); }
	virtual const FName GetEditorModeName() const override { return FControlRigBaseEditor::GetEditorModeNameImpl(); }
	virtual TSharedPtr<FApplicationMode> CreateEditorMode() override;
	virtual const FSlateBrush* GetDefaultTabIcon() const override { return FControlRigBaseEditor::GetDefaultTabIconImpl(); }

public:
	FControlRigLegacyEditor();
	virtual ~FControlRigLegacyEditor();

	// FControlRigBaseEditor
	virtual TSharedPtr<FAssetEditorToolkit> GetHostingApp() override { return IControlRigLegacyEditor::GetHostingApp(); }
	virtual TSharedRef<IControlRigBaseEditor> SharedControlRigEditorRef() override { return StaticCastSharedRef<IControlRigBaseEditor>(SharedThis(this)); }
	virtual TSharedRef<IRigVMEditor> SharedRigVMEditorRef() override { return StaticCastSharedRef<IRigVMEditor>(SharedThis(this)); }
	virtual TSharedRef<const IRigVMEditor> SharedRigVMEditorRef() const override { return StaticCastSharedRef<const IRigVMEditor>(SharedThis(this)); }
	
	virtual bool IsControlRigLegacyEditor() const override { return true; };
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const override { return IControlRigLegacyEditor::GetRigVMAssetInterface(); }
	UControlRigBlueprint* GetControlRigBlueprint() { return Cast<UControlRigBlueprint>(GetRigVMAssetInterface().GetObject()); }
	UControlRigBlueprint* GetControlRigBlueprint() const { return Cast<UControlRigBlueprint>(GetRigVMAssetInterface().GetObject()); }
	virtual URigVMHost* GetRigVMHost() const { return FRigVMEditorBase::GetRigVMHost(); }
	virtual bool IsEditorInitialized() const override { return bRigVMEditorInitialized; }
	virtual TSharedRef<FUICommandList> GetToolkitCommands() { return IControlRigLegacyEditor::GetToolkitCommands(); }
	virtual FPreviewScene* GetPreviewScene() { return IControlRigLegacyEditor::GetPreviewScene(); }
	virtual bool IsDetailsPanelRefreshSuspended() const { return IControlRigLegacyEditor::IsDetailsPanelRefreshSuspended(); }
	virtual TArray<TWeakObjectPtr<UObject>> GetSelectedObjects() const { return IControlRigLegacyEditor::GetSelectedObjects(); }
	virtual void ClearDetailObject(bool bChangeUISelectionState = true) { IControlRigLegacyEditor::ClearDetailObject(bChangeUISelectionState); }
	virtual bool DetailViewShowsStruct(UScriptStruct* InStruct) const { return IControlRigLegacyEditor::DetailViewShowsStruct(InStruct); }
	virtual TSharedPtr<class SWidget> GetInspector() const { return Inspector; }
	virtual TArray<FName> GetEventQueue() const { return IControlRigLegacyEditor::GetEventQueue(); }
	virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false) { IControlRigLegacyEditor::SummonSearchUI(bSetFindWithinBlueprint, NewSearchTerms, bSelectFirstResult); }
	virtual const TArray< UObject* >* GetObjectsCurrentlyBeingEdited() const { return IControlRigLegacyEditor::GetObjectsCurrentlyBeingEdited(); }
	virtual FEditorModeTools& GetEditorModeManagerImpl() const { return GetEditorModeManager(); }
	virtual const FName GetEditorModeNameImpl() const { return GetEditorModeName(); }
	virtual URigVMController* GetFocusedController() const { return IControlRigLegacyEditor::GetFocusedController(); }
	virtual URigVMGraph* GetFocusedModel() const { return IControlRigLegacyEditor::GetFocusedModel(); }
	virtual TArray<FName> GetLastEventQueue() const override { return LastEventQueue; }

	// FRigVMEditorBase interface
	virtual UObject* GetOuterForHost() const override { return GetOuterForHostImpl();}
	virtual UObject* GetOuterForHostSuper() const override { return IControlRigLegacyEditor::GetOuterForHost(); }
	
	virtual UClass* GetDetailWrapperClass() const { return GetDetailWrapperClassImpl(); }
	virtual void Compile() override { CompileBaseImpl(); }
	virtual void CompileSuper() override { IControlRigLegacyEditor::Compile(); }

	virtual void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) override { FControlRigBaseEditor::HandleModifiedEventImpl(InNotifType, InGraph, InSubject); }
	virtual void HandleModifiedEventSuper(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) override { IControlRigLegacyEditor::HandleModifiedEvent(InNotifType, InGraph, InSubject); }

	virtual void OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList) override { FControlRigBaseEditor::OnCreateGraphEditorCommandsImpl(GraphEditorCommandsList); }
	virtual void OnCreateGraphEditorCommandsSuper(TSharedPtr<FUICommandList> GraphEditorCommandsList) override { IControlRigLegacyEditor::OnCreateGraphEditorCommands(GraphEditorCommandsList); }
	
	virtual void HandleVMCompiledEvent(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext) override { FControlRigBaseEditor::HandleVMCompiledEventImpl(InCompiledObject, InVM, InContext); }
	virtual void HandleVMCompiledEventSuper(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext) override { IControlRigLegacyEditor::HandleVMCompiledEvent(InCompiledObject, InVM, InContext); }
	
	virtual bool ShouldOpenGraphByDefault() const override { return FControlRigBaseEditor::ShouldOpenGraphByDefaultImpl(); }
	virtual FReply OnViewportDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override { return FControlRigBaseEditor::OnViewportDropImpl(MyGeometry, DragDropEvent); }
	virtual FReply OnViewportDropSuper(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override { return IControlRigLegacyEditor::OnViewportDrop(MyGeometry, DragDropEvent); }

	// allows the editor to fill an empty graph
	virtual void CreateEmptyGraphContent(URigVMController* InController) override { FControlRigBaseEditor::CreateEmptyGraphContentImpl(InController); }

public:
	
	// IToolkit Interface
	virtual FName GetToolkitFName() const override { return FControlRigBaseEditor::GetToolkitFNameImpl(); }
	virtual FText GetBaseToolkitName() const override { return FControlRigBaseEditor::GetBaseToolkitNameImpl(); }
	virtual FString GetWorldCentricTabPrefix() const override { return FControlRigBaseEditor::GetWorldCentricTabPrefixImpl(); }
	virtual FString GetDocumentationLink() const override { return FControlRigBaseEditor::GetDocumentationLinkImpl(); }

	// BlueprintEditor interface
	virtual FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph) override { return FControlRigBaseEditor::OnSpawnGraphNodeByShortcutImpl(InChord, InPosition, InGraph); }
	virtual FReply OnSpawnGraphNodeByShortcutSuper(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph) override { return IControlRigLegacyEditor::OnSpawnGraphNodeByShortcut(InChord, InPosition, InGraph); }

	// FAssetEditorToolkit interface
	virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) override { if (PreviewViewport.IsValid()) { PreviewViewport->AddOverlayWidget(InViewportOverlayWidget, ZOrder); } }
	virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override { if (PreviewViewport.IsValid()) { PreviewViewport->RemoveOverlayWidget(InViewportOverlayWidget); } }
#if WITH_RIGVMLEGACYEDITOR
	virtual bool IsSectionVisible(NodeSectionID::Type InSectionID) const override;
	virtual bool NewDocument_IsVisibleForType(FBlueprintEditor::ECreatedDocumentType GraphType) const override;
#else
	virtual bool IsSectionVisible(RigVMNodeSectionID::Type InSectionID) const override;
	virtual bool NewDocument_IsVisibleForType(FRigVMEditorBase::ECreatedDocumentType GraphType) const override;
#endif

	virtual void PostTransaction(bool bSuccess, const FTransaction* Transaction, bool bIsRedo) override { return FControlRigBaseEditor::PostTransactionImpl(bSuccess, Transaction, bIsRedo); }

	//  FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override { return FControlRigBaseEditor::TickImpl(DeltaTime); }
	virtual void TickSuper(float DeltaTime) override { return IControlRigLegacyEditor::Tick(DeltaTime); }

	virtual void SetDetailObjects(const TArray<UObject*>& InObjects) override { return FControlRigBaseEditor::SetDetailObjectsImpl(InObjects); }
	virtual void SetDetailObjectsSuper(const TArray<UObject*>& InObjects) override { return IControlRigLegacyEditor::SetDetailObjects(InObjects); }
	virtual void SetDetailObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InObjectFilter) override { return IControlRigLegacyEditor::SetDetailObjectFilter(InObjectFilter); };
	virtual void RefreshDetailView() override { return FControlRigBaseEditor::RefreshDetailViewImpl(); }
	virtual void RefreshDetailViewSuper() override { return IControlRigLegacyEditor::RefreshDetailView(); }

public:

	
	virtual void OnGraphNodeDropToPerform(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition) override  { return FControlRigBaseEditor::OnGraphNodeDropToPerformImpl(InDragDropOp, InGraph, InNodePosition, InScreenPosition); }
	virtual void OnGraphNodeDropToPerformSuper(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition) override  { return IControlRigLegacyEditor::OnGraphNodeDropToPerform(InDragDropOp, InGraph, InNodePosition, InScreenPosition); }

	
protected:
	//~ Begin IRigVMEditor interface
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	//~ End IRigVMEditor interface

	virtual void BindCommands() override { return FControlRigBaseEditor::BindCommandsImpl(); }
	virtual void BindCommandsSuper() override { return IControlRigLegacyEditor::BindCommands(); }
	virtual void UnbindCommands() override { return FControlRigBaseEditor::UnbindCommandsImpl(); }
	virtual void UnbindCommandsSuper() override { return IControlRigLegacyEditor::UnbindCommands(); }
	virtual FMenuBuilder GenerateBulkEditMenu() override { return FControlRigBaseEditor::GenerateBulkEditMenuImpl(); }
	virtual FMenuBuilder GenerateBulkEditMenuSuper() override { return IControlRigLegacyEditor::GenerateBulkEditMenu(); }

	virtual void SaveAsset_Execute() override { return FControlRigBaseEditor::SaveAsset_ExecuteImpl(); }
	virtual void SaveAsset_ExecuteSuper() override { return IControlRigLegacyEditor::SaveAsset_Execute(); }
	virtual void SaveAssetAs_Execute() override { return FControlRigBaseEditor::SaveAssetAs_ExecuteImpl(); }
	virtual void SaveAssetAs_ExecuteSuper() override { return IControlRigLegacyEditor::SaveAssetAs_Execute(); }

	virtual void HandleVMExecutedEvent(URigVMHost* InHost, const FName& InEventName) override { return FControlRigBaseEditor::HandleVMExecutedEventImpl(InHost, InEventName); }
	virtual void HandleVMExecutedEventSuper(URigVMHost* InHost, const FName& InEventName) override { return IControlRigLegacyEditor::HandleVMExecutedEvent(InHost, InEventName); }

	virtual bool CanRunGraphOnce() const override { return IsRunningForwardSolve(); }
	virtual void OnRunGraphOnceToggled(bool bRunGraphOnceEnabled) override { OnRunGraphOnceToggledImpl(bRunGraphOnceEnabled); }

	// FBaseToolKit overrides
	virtual void CreateEditorModeManager() override { return FControlRigBaseEditor::CreateEditorModeManagerImpl(); }
private:
	/** Fill the toolbar with content */
	virtual void FillToolbar(FToolBarBuilder& ToolbarBuilder, bool bEndSection = true) override { return FControlRigBaseEditor::FillToolbarImpl(ToolbarBuilder, bEndSection); }
	virtual void FillToolbarSuper(FToolBarBuilder& ToolbarBuilder, bool bEndSection = true) override { return IControlRigLegacyEditor::FillToolbar(ToolbarBuilder, bEndSection); }

	virtual TArray<FName> GetDefaultEventQueue() const override { return FControlRigBaseEditor::GetDefaultEventQueueImpl(); }
	virtual void SetEventQueue(TArray<FName> InEventQueue, bool bCompile) override { return FControlRigBaseEditor::SetEventQueueImpl(InEventQueue, bCompile); }
	virtual void SetEventQueueSuper(TArray<FName> InEventQueue, bool bCompile) override { return IControlRigLegacyEditor::SetEventQueue(InEventQueue, bCompile); }
	virtual void SetEventQueueSuper(TArray<FName> InEventQueue) override { return IControlRigLegacyEditor::SetEventQueue(InEventQueue); }
	virtual int32 GetEventQueueComboValue() const override { return FControlRigBaseEditor::GetEventQueueComboValueImpl(); }
	virtual int32 GetEventQueueComboValueSuper() const override { return IControlRigLegacyEditor::GetEventQueueComboValue(); }
	virtual FText GetEventQueueLabel() const override { return FControlRigBaseEditor::GetEventQueueLabelImpl(); }
	virtual FSlateIcon GetEventQueueIcon(const TArray<FName>& InEventQueue) const override { return FControlRigBaseEditor::GetEventQueueIconImpl(InEventQueue); }
	virtual void HandleSetObjectBeingDebugged(UObject* InObject) override { return FControlRigBaseEditor::HandleSetObjectBeingDebuggedImpl(InObject); }
	virtual void HandleSetObjectBeingDebuggedSuper(UObject* InObject) override { return IControlRigLegacyEditor::HandleSetObjectBeingDebugged(InObject); }

	/** Push a newly compiled/opened control rig to the edit mode */
	virtual void UpdateRigVMHost() override { return FControlRigBaseEditor::UpdateRigVMHostImpl(); }
	virtual void UpdateRigVMHostSuper() override { return IControlRigLegacyEditor::UpdateRigVMHost(); }
	virtual void UpdateRigVMHost_PreClearOldHost(URigVMHost* InPreviousHost) override { return FControlRigBaseEditor::UpdateRigVMHost_PreClearOldHostImpl(InPreviousHost); }

	/** Update the name lists for use in name combo boxes */
	virtual void CacheNameLists() override { return FControlRigBaseEditor::CacheNameListsImpl(); }
	virtual void CacheNameListsSuper() override { return IControlRigLegacyEditor::CacheNameLists(); }

	virtual void GenerateEventQueueMenuContent(FMenuBuilder& MenuBuilder) override { return FControlRigBaseEditor::GenerateEventQueueMenuContentImpl(MenuBuilder); }

	virtual void HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint) override { return FControlRigBaseEditor::HandleRefreshEditorFromBlueprintImpl(InBlueprint); }
	virtual void HandleRefreshEditorFromBlueprintSuper(FRigVMEditorAssetInterfacePtr InBlueprint) override { return IControlRigLegacyEditor::HandleRefreshEditorFromBlueprint(InBlueprint); }

	
	/** delegate for changing property */
	virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override { return FControlRigBaseEditor::OnFinishedChangingPropertiesImpl(PropertyChangedEvent); }
	virtual void OnFinishedChangingPropertiesSuper(const FPropertyChangedEvent& PropertyChangedEvent) override { return IControlRigLegacyEditor::OnFinishedChangingProperties(PropertyChangedEvent); }
	virtual void OnWrappedPropertyChangedChainEvent(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent) override { return FControlRigBaseEditor::OnWrappedPropertyChangedChainEventImpl(InWrapperObject, InPropertyPath, InPropertyChangedChainEvent); }
	virtual void OnWrappedPropertyChangedChainEventSuper(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent) override { return IControlRigLegacyEditor::OnWrappedPropertyChangedChainEvent(InWrapperObject, InPropertyPath, InPropertyChangedChainEvent); }

	virtual void SetEditorModeManager(TSharedPtr<FEditorModeTools> InManager) override { EditorModeManager = InManager; }

	virtual const TArray<TStrongObjectPtr<URigVMDetailsViewWrapperObject>>& GetWrapperObjects() const override { return IControlRigLegacyEditor::GetWrapperObjects(); }
	virtual bool& GetSuspendDetailsPanelRefreshFlag() override { return IControlRigLegacyEditor::GetSuspendDetailsPanelRefreshFlag(); }

	virtual TWeakPtr<class SGraphEditor> GetFocusedGraphEd() const override { return FocusedGraphEdPtr;}

	virtual void OnClose() override { FControlRigBaseEditor::OnClose(); }
	virtual void OnCloseSuper() override { IControlRigLegacyEditor::OnClose(); }
};

#endif