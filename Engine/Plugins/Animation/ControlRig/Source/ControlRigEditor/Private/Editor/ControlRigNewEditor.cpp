// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigNewEditor.h"

#include "ControlRigContextMenuContext.h"
#include "ControlRigEditorMode.h"
#include "ControlRigEditorStyle.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "ToolMenuContext.h"


FControlRigEditor::FControlRigEditor()
: IControlRigNewEditor()
, FControlRigBaseEditor()
{
	LastEventQueue = ConstructionEventQueue;
}

FControlRigEditor::~FControlRigEditor()
{
	if (FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		IControlRigEditorAssetInterface::sCurrentlyOpenedRigBlueprints.Remove(RigBlueprint);

		RigBlueprint->OnHierarchyModified().RemoveAll(this);
		if (FControlRigEditMode* EditMode = GetEditMode())
		{
			RigBlueprint->OnHierarchyModified().RemoveAll(EditMode);
			EditMode->OnEditorClosed();
		}

		RigBlueprint->OnRigTypeChanged().RemoveAll(this);
		if (RigBlueprint->IsModularRig())
		{
			RigBlueprint->GetModularRigController()->OnModified().RemoveAll(this);
			RigBlueprint->OnModularRigCompiled().RemoveAll(this);

			RigBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().RemoveAll(&SchematicModel);
			RigBlueprint->OnHierarchyModified().RemoveAll(&SchematicModel);
			RigBlueprint->GetModularRigController()->OnModified().RemoveAll(&SchematicModel);
		}
	}

	if (PersonaToolkit.IsValid())
	{
		constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}

TSharedPtr<FApplicationMode> FControlRigEditor::CreateEditorMode()
{
	CreatePersonaToolKitIfRequired();

	if(IsModularRig())
	{
		return MakeShareable(new FModularRigEditorMode(SharedThis(this)));
	}
	return MakeShareable(new FControlRigEditorMode(SharedThis(this)));
}

bool FControlRigEditor::IsSectionVisible(RigVMNodeSectionID::Type InSectionID) const
{
	if(!IControlRigNewEditor::IsSectionVisible(InSectionID))
	{
		return false;
	}

	if(const FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if(IsModularRig())
		{
			switch (InSectionID)
			{
				case RigVMNodeSectionID::GRAPH:
				{
					return RigBlueprint->SupportsEventGraphs();
				}
				case RigVMNodeSectionID::FUNCTION:
				{
					return RigBlueprint->SupportsFunctions();
				}
				default:
				{
					break;
				}
			}
		}
	}
	return true;
}

bool FControlRigEditor::NewDocument_IsVisibleForType(FRigVMEditorBase::ECreatedDocumentType GraphType) const
{
	if(!IControlRigNewEditor::NewDocument_IsVisibleForType(GraphType))
	{
		return false;
	}

	if(const FControlRigAssetInterfacePtr RigBlueprint = GetControlRigAssetInterface())
	{
		if(IsModularRig())
		{
			switch(GraphType)
			{
				case FRigVMEditorBase::CGT_NewEventGraph:
				{
					return RigBlueprint->SupportsEventGraphs();
				}
				case FRigVMEditorBase::CGT_NewFunctionGraph:
				{
					return RigBlueprint->SupportsFunctions();
				}
				default:
				{
					break;
				}
			}
		}
	}

	return true;
}

void FControlRigEditor::PostUndo(bool bSuccess)
{
	IControlRigNewEditor::PostUndo(bSuccess);
}

void FControlRigEditor::PostTransaction(bool bSuccess, const FTransaction* Transaction, bool bIsRedo)
{
	FRigVMEditorBase::PostTransaction(bSuccess, Transaction, bIsRedo);
	return FControlRigBaseEditor::PostTransactionImpl(bSuccess, Transaction, bIsRedo);
}

void FControlRigEditor::Tick(float DeltaTime)
{
	PreviewScene.UpdateCaptureContents();
	FControlRigBaseEditor::TickImpl(DeltaTime);
}

void FControlRigEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FRigVMNewEditor::InitToolMenuContext(MenuContext);

	if (FRigVMEditorAssetInterfacePtr RigVMAsset = GetRigVMAssetInterface())
	{
		UControlRigContextMenuContext* ContextObject = NewObject<UControlRigContextMenuContext>();
		ContextObject->Init(SharedThis(this));

		MenuContext.AddObject(ContextObject);
	}
}
