// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSchematicModel.h"

#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigEditor.h"
#include "ControlRigEditorStyle.h"
#include "ModularRig.h"
#include "ModularRigRuleManager.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Rigs/RigHierarchyController.h"
#include "Async/TaskGraphInterfaces.h"
#include <SchematicGraphPanel/SchematicGraphStyle.h>
#include "Blueprint/WidgetBlueprintLibrary.h"
#include "Editor/Hierarchy/DragDrop/ModularRigHierarchyElementDragDropOp.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyElementDragDropOp.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyTagDragDropOp.h"
#include "SchematicGraphPanel/SSchematicGraphPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Dialog/SCustomDialog.h"
#include "Settings/ControlRigSettings.h"

#define LOCTEXT_NAMESPACE "ControlRigSchematicModel"

FString FControlRigSchematicRigElementKeyNode::GetDragDropDecoratorLabel() const
{
	return Key.ToString();
}

bool FControlRigSchematicRigElementKeyNode::IsDragSupported() const
{
	return Key.Type == ERigElementType::Connector;
}

const FText& FControlRigSchematicRigElementKeyNode::GetLabel() const
{
	if(IsExpanded())
	{
		static const FText& EmptyText = FText();
		return EmptyText;
	}
	
	FText LabelText = FText::FromString(Key.ToString());
	if(const FControlRigSchematicModel* ControlRigModel = Cast<FControlRigSchematicModel>(Model))
	{
		if(const UModularRig* ModularRig = Cast<UModularRig>(ControlRigModel->ControlRigBeingDebuggedPtr.Get()))
		{
			if (const URigHierarchy* Hierarchy = ModularRig->GetHierarchy())
			{
				LabelText = Hierarchy->GetDisplayNameForUI(Key);

				switch(Key.Type)
				{
					case ERigElementType::Socket:
					{
						if(const FRigSocketElement* Socket = Hierarchy->Find<FRigSocketElement>(Key))
						{
							const FString Description = Socket->GetDescription(Hierarchy);
							if(!Description.IsEmpty())
							{
								static const FText NodeLabelSocketDescriptionFormat = LOCTEXT("NodeLabelSocketDescriptionFormat", "{0}\n{1}");
								LabelText = FText::Format(NodeLabelSocketDescriptionFormat, LabelText,  FText::FromString(Description));
							}
						}
						break;
					}
					default:
					{
						break;
					}
				}

				const FModularRigModel& ModularRigModel = ModularRig->GetModularRigModel();
				const TArray<FRigElementKey>& Connectors = ModularRigModel.Connections.FindConnectorsFromTarget(Key);
				for(const FRigElementKey& Connector : Connectors)
				{
					static const FText NodeLabelConnectionFormat = LOCTEXT("NodeLabelConnectionFormat", "{0}\nConnection: {1}");
					const FText ConnectorShortestPath = Hierarchy->GetDisplayNameForUI(Connector, EElementNameDisplayMode::ForceLong);
					LabelText = FText::Format(NodeLabelConnectionFormat, LabelText, ConnectorShortestPath);
				}
			}
		}
	}
	const_cast<FControlRigSchematicRigElementKeyNode*>(this)->Label = LabelText;
	return Label;
}

FControlRigSchematicWarningTag::FControlRigSchematicWarningTag()
: FSchematicGraphTag()
{
	BackgroundColor = FLinearColor::Black;
	ForegroundColor = FLinearColor::Yellow;
	
	static const FSlateBrush* WarningBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Schematic.ConnectorWarning");
	ForegroundBrush = WarningBrush;
}

FControlRigSchematicModel::~FControlRigSchematicModel()
{
	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(UControlRig* ControlRigBeingDebugged = ControlRigBeingDebuggedPtr.Get())
		{
			if(!URigVMHost::IsGarbageOrDestroyed(ControlRigBeingDebugged))
			{
				ControlRigBeingDebugged->GetHierarchy()->OnModified().RemoveAll(this);
				ControlRigBeingDebugged->OnPostConstruction_AnyThread().RemoveAll(this);
			}
		}
	}

	ControlRigBeingDebuggedPtr.Reset();
	ControlRigEditor.Reset();
	ControlRigBlueprint.Reset();
}

void FControlRigSchematicModel::SetEditor(const TSharedRef<IControlRigBaseEditor>& InEditor)
{
	ControlRigEditor = InEditor;
	ControlRigBlueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface();
	OnSetObjectBeingDebugged(ControlRigBlueprint->GetDebuggedControlRig());
}

void FControlRigSchematicModel::ApplyToPanel(SSchematicGraphPanel* InPanel)
{
	FSchematicGraphModel::ApplyToPanel(InPanel);
	
	InPanel->OnNodeClicked().BindRaw(this, &FControlRigSchematicModel::HandleSchematicNodeClicked);
	InPanel->OnBeginDrag().BindRaw(this, &FControlRigSchematicModel::HandleSchematicBeginDrag);
	InPanel->OnEndDrag().BindRaw(this, &FControlRigSchematicModel::HandleSchematicEndDrag);
	InPanel->OnEnterDrag().BindRaw(this, &FControlRigSchematicModel::HandleSchematicEnterDrag);
	InPanel->OnLeaveDrag().BindRaw(this, &FControlRigSchematicModel::HandleSchematicLeaveDrag);
	InPanel->OnCancelDrag().BindRaw(this, &FControlRigSchematicModel::HandleSchematicCancelDrag);
	InPanel->OnAcceptDrop().BindRaw(this, &FControlRigSchematicModel::HandleSchematicDrop);
}

void FControlRigSchematicModel::Reset()
{
	Super::Reset();
	RigElementKeyToGuid.Reset();
}

void FControlRigSchematicModel::Tick(float InDeltaTime)
{
	Super::Tick(InDeltaTime);

	if(ControlRigBlueprint.IsValid())
	{
		//UE_LOGF(LogControlRig, Display, "Selected nodes %d", GetSelectedNodes().Num());
		if (UControlRig* DebuggedRig = ControlRigBlueprint->GetDebuggedControlRig())
		{
			const URigHierarchy* Hierarchy = DebuggedRig->GetHierarchy();
			check(Hierarchy);
			const FModularRigConnections& Connections = ControlRigBlueprint->GetModularRigModel().Connections;

			TArray<FRigElementKey> KeysToRemove;
			for(const TPair<FRigElementKey,FGuid>& Pair : RigElementKeyToGuid)
			{
				if((Pair.Key.Type == ERigElementType::Socket) ||
					(Pair.Key.Type == ERigElementType::Connector))
				{
					continue;
				}
				if(TemporaryNodeGuids.Contains(Pair.Value))
				{
					continue;
				}

				const TArray<FRigElementKey>& Connectors = Connections.FindConnectorsFromTarget(Pair.Key);
				if(Connectors.Num() > 1)
				{
					continue;
				}
				if(Connectors.Num() == 1)
				{
					if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(Connectors[0]))
					{
						if(!Connector->IsPrimary())
						{
							continue;
						}
					}
				}
			
				KeysToRemove.Add(Pair.Key);
			}
			for(const FRigElementKey& KeyToRemove : KeysToRemove)
			{
				RemoveElementKeyNode(KeyToRemove);
			}

			TArray<FRigElementKey> ConnectorKeys;
			for(const TPair<FRigElementKey,FGuid>& Pair : RigElementKeyToGuid)
			{
				if(Pair.Key.Type == ERigElementType::Connector)
				{
					ConnectorKeys.Add(Pair.Key);
				}
			}

			for(const FRigElementKey& ConnectorKey : ConnectorKeys)
			{
				UpdateConnector(ConnectorKey);
			}
		}
	}
}

FControlRigSchematicRigElementKeyNode* FControlRigSchematicModel::AddElementKeyNode(const FRigElementKey& InKey, bool bNotify)
{
	FControlRigSchematicRigElementKeyNode* Node = AddNode<FControlRigSchematicRigElementKeyNode>(false);
	if(Node)
	{
		ConfigureElementKeyNode(Node, InKey);
		RigElementKeyToGuid.Add(Node->GetKey(), Node->GetGuid());
		UpdateConnector(InKey);

		if(InKey.Type == ERigElementType::Connector)
		{
			Node->AddTag<FControlRigSchematicWarningTag>();
		}

		if (bNotify && OnNodeAddedDelegate.IsBound())
		{
			OnNodeAddedDelegate.Broadcast(Node);
		}

		UpdateElementKeyLinks();
	}
	return Node;
}

void FControlRigSchematicModel::ConfigureElementKeyNode(FControlRigSchematicRigElementKeyNode* InNode, const FRigElementKey& InKey)
{
	InNode->Key = InKey;
}

const FControlRigSchematicRigElementKeyNode* FControlRigSchematicModel::FindElementKeyNode(const FRigElementKey& InKey) const
{
	return const_cast<FControlRigSchematicModel*>(this)->FindElementKeyNode(InKey);
}

FControlRigSchematicRigElementKeyNode* FControlRigSchematicModel::FindElementKeyNode(const FRigElementKey& InKey)
{
	if(const FGuid* FoundGuid = RigElementKeyToGuid.Find(InKey))
	{
		return FindNode< FControlRigSchematicRigElementKeyNode >(*FoundGuid);
	}
	return nullptr;
}

bool FControlRigSchematicModel::ContainsElementKeyNode(const FRigElementKey& InKey) const
{
	return FindElementKeyNode(InKey) != nullptr;
}

bool FControlRigSchematicModel::RemoveNode(const FGuid& InGuid)
{
	FRigElementKey KeyToRemove;
	if(const FControlRigSchematicRigElementKeyNode* Node = FindNode<FControlRigSchematicRigElementKeyNode>(InGuid))
	{
		KeyToRemove = Node->GetKey();
	}
	if(Super::RemoveNode(InGuid))
	{
		RigElementKeyToGuid.Remove(KeyToRemove);
		return true;
	}
	return false;
}

bool FControlRigSchematicModel::RemoveElementKeyNode(const FRigElementKey& InKey)
{
	if(const FGuid* GuidPtr = RigElementKeyToGuid.Find(InKey))
	{
		const FGuid Guid = *GuidPtr;
		const bool bResult = RemoveNode(Guid);
		if(bResult)
		{
			UpdateElementKeyLinks();
		}
		return bResult;
	}
	return false;
}

FControlRigSchematicRigElementKeyLink* FControlRigSchematicModel::AddElementKeyLink(const FRigElementKey& InSourceKey, const FRigElementKey& InTargetKey, bool bNotify)
{
	check(InSourceKey.IsValid());
	check(InTargetKey.IsValid());
	check(InSourceKey != InTargetKey);
	const FControlRigSchematicRigElementKeyNode* SourceNode = FindElementKeyNode(InSourceKey);
	const FControlRigSchematicRigElementKeyNode* TargetNode = FindElementKeyNode(InTargetKey);
	if(SourceNode && TargetNode)
	{
		FControlRigSchematicRigElementKeyLink* Link = AddLink<FControlRigSchematicRigElementKeyLink>(SourceNode->GetGuid(), TargetNode->GetGuid());
		Link->SourceKey = InSourceKey;
		Link->TargetKey = InTargetKey;
		Link->Thickness = 16.f;
		return Link;
	}
	return nullptr;
}

void FControlRigSchematicModel::UpdateElementKeyNodes()
{
	if (ControlRigBeingDebuggedPtr.IsValid())
	{
		if (const URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy())
		{
			const TArray<FRigSocketElement*> Sockets = Hierarchy->GetElementsOfType<FRigSocketElement>();
			for (const FRigSocketElement* Socket : Sockets)
			{
				if(!ContainsElementKeyNode(Socket->GetKey()))
				{
					AddElementKeyNode(Socket->GetKey());
				}
			}

			const TArray<FRigConnectorElement*> Connectors = Hierarchy->GetElementsOfType<FRigConnectorElement>();
			for (const FRigConnectorElement* Connector : Connectors)
			{
				if(!ContainsElementKeyNode(Connector->GetKey()))
				{
					AddElementKeyNode(Connector->GetKey());
				}
				UpdateConnector(Connector->GetKey());
			}

			// remove obsolete nodes
			TArray<FGuid> GuidsToRemove;
			for(const TSharedPtr<FSchematicGraphNode>& Node : Nodes)
			{
				if(const FControlRigSchematicRigElementKeyNode* ElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(Node.Get()))
				{
					if(!Hierarchy->Contains(ElementKeyNode->GetKey()))
					{
						GuidsToRemove.Add(ElementKeyNode->GetGuid());
					}
				}
			}
			for(const FGuid& Guid : GuidsToRemove)
			{
				RemoveNode(Guid);
			}
		}
	}
}

void FControlRigSchematicModel::UpdateElementKeyLinks()
{
	if(!bUpdateElementKeyLinks)
	{
		return;
	}
	
	const TSharedPtr<IControlRigBaseEditor> Editor = ControlRigEditor.Pin();
	if(!Editor.IsValid())
	{
		return;
	}

	typedef TTuple< FRigElementKey, FRigElementKey > TElementKeyPair;
	typedef TTuple< FGuid, TElementKeyPair > TElementKeyLinkPair;

	struct FLinkTraverser
	{
		const FControlRigSchematicRigElementKeyNode* VisitElement(
			const FRigBaseElement* InElement, 
			const FControlRigSchematicRigElementKeyNode* InNode, 
			TArray< TElementKeyPair >& OutExpectedLinks) const
		{
			const FControlRigSchematicRigElementKeyNode* Node = InNode;
			if(const FControlRigSchematicRigElementKeyNode* SelfNode = FindNode(InElement->GetKey()))
			{
				Node = SelfNode;
			}

			const TConstArrayView<FRigBaseElement*> Children = Hierarchy->GetChildren(InElement);
			for(const FRigBaseElement* Child : Children)
			{
				const FControlRigSchematicRigElementKeyNode* ChildNode = VisitElement(Child, Node, OutExpectedLinks);
				if(Node && ChildNode && Node != ChildNode)
				{
					const FRigElementKey ParentA = Hierarchy->GetFirstParent(Node->GetKey());
					const FRigElementKey ParentB = Hierarchy->GetFirstParent(ChildNode->GetKey());
					if(!ParentA || !ParentB.IsValid() || ParentA != ParentB)
					{
						OutExpectedLinks.Emplace(Node->GetKey(), ChildNode->GetKey());
					}
				}
			}

			return Node;
		}

		const FControlRigSchematicRigElementKeyNode* FindNode(const FRigElementKey& InKey) const
		{
			if(const FGuid* ElementGuid = RigElementKeyToGuid->Find(InKey))
			{
				return Cast<FControlRigSchematicRigElementKeyNode>(NodeByGuid->FindChecked(*ElementGuid).Get());
			}
			if(const FRigElementKey* SocketKey = SocketToParent.Find(InKey))
			{
				if(const FGuid* ElementGuid = RigElementKeyToGuid->Find(*SocketKey))
				{
					return Cast<FControlRigSchematicRigElementKeyNode>(NodeByGuid->FindChecked(*ElementGuid).Get());
				}
			}
			if(const FRigElementKey* ParentKey = ParentToSocket.Find(InKey))
			{
				if(const FGuid* ElementGuid = RigElementKeyToGuid->Find(*ParentKey))
				{
					return Cast<FControlRigSchematicRigElementKeyNode>(NodeByGuid->FindChecked(*ElementGuid).Get());
				}
			}
			return nullptr;
		}

		TArray< TElementKeyPair > ComputeExpectedLinks() const
		{
			SocketToParent.Reset();
			ParentToSocket.Reset();

			// create a map to look up sockets
			const TArray<FRigElementKey> SocketKeys = Hierarchy->GetSocketKeys();
			for(const FRigElementKey& SocketKey : SocketKeys)
			{
				const FRigElementKey ParentKey = Hierarchy->GetFirstParent(SocketKey);
				if(ParentKey.IsValid())
				{
					SocketToParent.Add(SocketKey, ParentKey);
					if(!ParentToSocket.Contains(ParentKey))
					{
						ParentToSocket.Add(ParentKey, SocketKey);
					}
				}
			}
			
			TArray< TElementKeyPair > ExpectedLinks;
			const TArray<FRigBaseElement*> RootElements = Hierarchy->GetRootElements();
			for(const FRigBaseElement* RootElement : RootElements)
			{
				if(RootElement->GetKey().Type == ERigElementType::Curve)
				{
					continue;
				}
				VisitElement(RootElement, nullptr, ExpectedLinks);
			}
			return ExpectedLinks;
		}

		const UControlRig* ControlRig; 
		const URigHierarchy* Hierarchy; 
		const TMap<FRigElementKey, FGuid>* RigElementKeyToGuid = nullptr;
		const TMap<FGuid, TSharedPtr<FSchematicGraphNode>>* NodeByGuid = nullptr;
		mutable TMap<FRigElementKey, FRigElementKey> SocketToParent;
		mutable TMap<FRigElementKey, FRigElementKey> ParentToSocket;
	};

	FLinkTraverser Traverser;
	Traverser.ControlRig = Editor->GetControlRig();
	if(Traverser.ControlRig == nullptr)
	{
		return;
	}
	Traverser.Hierarchy = Traverser.ControlRig->GetHierarchy();
	if(Traverser.Hierarchy == nullptr)
	{
		return;
	}
	Traverser.RigElementKeyToGuid = &RigElementKeyToGuid;
	Traverser.NodeByGuid = &NodeByGuid;

	const TArray< TElementKeyPair > ExpectedLinks = Traverser.ComputeExpectedLinks();

	TArray< TElementKeyLinkPair > ExistingLinks;
	for(const TSharedPtr<FSchematicGraphLink>& Link : Links)
	{
		if(const FControlRigSchematicRigElementKeyLink* ElementKeyLink = Cast<FControlRigSchematicRigElementKeyLink>(Link.Get()))
		{
			ExistingLinks.Emplace(ElementKeyLink->GetGuid(), TElementKeyPair(ElementKeyLink->GetSourceKey(), ElementKeyLink->GetTargetKey()));
		}
	}

	// remove the obsolete links
	for(const TTuple< FGuid, TTuple< FRigElementKey, FRigElementKey > >& ExistingLink : ExistingLinks)
	{
		if(!ExpectedLinks.Contains(ExistingLink.Get<1>()))
		{
			(void)RemoveLink(ExistingLink.Get<0>());
		}
	}

	// add missing links
	for(const TElementKeyPair& ExpectedLink : ExpectedLinks)
	{
		if(!ExistingLinks.ContainsByPredicate([ExpectedLink](const TElementKeyLinkPair& ExistingLink) -> bool
		{
			return ExistingLink.Get<1>() == ExpectedLink;
		}))
		{
			(void)AddElementKeyLink(ExpectedLink.Get<0>(), ExpectedLink.Get<1>());
		}
	}

	// todo: also introduce links for module relationships
}

void FControlRigSchematicModel::UpdateControlRigContent()
{
	{
		const TGuardValue<bool> DisableUpdatingLinks(bUpdateElementKeyLinks, false);
		UpdateElementKeyNodes();
	}
	UpdateElementKeyLinks();
}

void FControlRigSchematicModel::UpdateConnector(const FRigElementKey& InElementKey)
{
	if(InElementKey.Type == ERigElementType::Connector)
	{
		if(const FControlRigSchematicRigElementKeyNode* ConnectorNode = FindElementKeyNode(InElementKey))
		{
			FRigElementKey ResolvedKey;
			if(IsConnectorResolved(InElementKey, &ResolvedKey))
			{
				if(ControlRigBlueprint.IsValid() && ControlRigBeingDebuggedPtr.IsValid() && ResolvedKey.Type != ERigElementType::Socket)
				{
					if(const URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy())
					{
						if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(InElementKey))
						{
							if(Connector->IsPrimary())
							{
								// if the resolved target has only one primary connector on it, and it is not a socket
								// let's not draw the node in the schematic for now
								if(ControlRigBlueprint->GetModularRigModel().Connections.FindConnectorsFromTarget(ResolvedKey).Num() == 1)
								{
									ResolvedKey = FRigElementKey();
								}
							}
						}
					}
				}
			}

			if(ResolvedKey.IsValid())
			{
				const FControlRigSchematicRigElementKeyNode* ResolvedNode = FindElementKeyNode(ResolvedKey);
				if(ResolvedNode == nullptr)
				{
					ResolvedNode = AddElementKeyNode(ResolvedKey);
				}

				if(ConnectorNode->GetParentNode() != ResolvedNode)
				{
					SetParentNode(ConnectorNode->GetGuid(), ResolvedNode->GetGuid());
					const_cast<FControlRigSchematicRigElementKeyNode*>(ConnectorNode)->OnMouseLeave();
				}
			}
			else if(ConnectorNode->HasParentNode())
			{
				RemoveFromParentNode(ConnectorNode->GetGuid());
				const_cast<FControlRigSchematicRigElementKeyNode*>(ConnectorNode)->OnMouseLeave();
			}
		}
	}
}

void FControlRigSchematicModel::OnSetObjectBeingDebugged(UObject* InObject)
{
	if(ControlRigBeingDebuggedPtr.Get() == InObject)
	{
		return;
	}

	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(UControlRig* ControlRigBeingDebugged = ControlRigBeingDebuggedPtr.Get())
		{
			if(!URigVMHost::IsGarbageOrDestroyed(ControlRigBeingDebugged))
			{
				ControlRigBeingDebugged->GetHierarchy()->OnModified().RemoveAll(this);
				ControlRigBeingDebugged->OnPostConstruction_AnyThread().RemoveAll(this);
			}
		}
	}

	ControlRigBeingDebuggedPtr.Reset();
	
	if(UControlRig* ControlRig = Cast<UControlRig>(InObject))
	{
		ControlRigBeingDebuggedPtr = ControlRig;
		if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			ControlRig->OnPostConstruction_AnyThread().AddRaw(this, &FControlRigSchematicModel::HandlePostConstruction);

			UpdateControlRigContent();
		}
	}
}

void FControlRigSchematicModel::HandleModularRigModified(EModularRigNotification InNotification, const FRigModuleReference* InModule)
{
	switch (InNotification)
	{
		case EModularRigNotification::ConnectionChanged:
		{
			if (ControlRigBeingDebuggedPtr.IsValid() && InModule)
			{
				if (const URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy())
				{
					const TArray<FRigElementKey> Connectors = Hierarchy->GetConnectorKeys();
					for(const FRigElementKey& ConnectorKey : Connectors)
					{
						if(const FSchematicGraphNode* ConnectorNode = FindElementKeyNode(ConnectorKey))
						{
							const FString NameString = ConnectorKey.Name.ToString();
							if(NameString.StartsWith(InModule->GetElementPrefix(), ESearchCase::IgnoreCase))
							{
								const FString LocalName = NameString.Mid(InModule->GetElementPrefix().Len());
								if(!LocalName.Contains(FRigHierarchyModulePath::ModuleNameSuffix))
								{
									UpdateConnector(ConnectorKey);
								}
							}
						}
					}
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

FSchematicGraphGroupNode* FControlRigSchematicModel::AddAutoGroupNode()
{
	FSchematicGraphGroupNode* GroupNode = Super::AddAutoGroupNode();
	GroupNode->AddTag<FControlRigSchematicWarningTag>();
	return GroupNode;
}

FVector2d FControlRigSchematicModel::GetPositionForNode(const FSchematicGraphNode* InNode) const
{
	if(const FControlRigSchematicRigElementKeyNode* Node = Cast<FControlRigSchematicRigElementKeyNode>(InNode))
	{
		if(!InNode->HasParentNode())
		{
			FRigElementKey Key = Node->GetKey();
			if(Key.Type == ERigElementType::Connector)
			{
				FRigElementKey ResolvedKey;
				if(IsConnectorResolved(Key, &ResolvedKey))
				{
					Key = ResolvedKey;
				}
			}

			if(Key.IsValid())
			{
				if(const TSharedPtr<IControlRigBaseEditor> Editor = ControlRigEditor.Pin())
				{
					if(Editor.IsValid())
					{
						if(const UControlRig* ControlRig = Editor->GetControlRig())
						{
							if(const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
							{
								const FTransform Transform = Hierarchy->GetGlobalTransform(Key);
								return Editor->ComputePersonaProjectedScreenPos(Transform.GetLocation());
							}
						}
					}
				}
			}
		}
	}
	return Super::GetPositionForNode(InNode);
}

bool FControlRigSchematicModel::GetPositionAnimationEnabledForNode(const FSchematicGraphNode* InNode) const
{
	if(InNode->IsA<FControlRigSchematicRigElementKeyNode>())
	{
		return false;
	}
	return Super::GetPositionAnimationEnabledForNode(InNode);
}

int32 FControlRigSchematicModel::GetNumLayersForNode(const FSchematicGraphNode* InNode) const
{
	if(Cast<FControlRigSchematicRigElementKeyNode>(InNode))
	{
		return 3;
	}
	return Super::GetNumLayersForNode(InNode);
}

const FSlateBrush* FControlRigSchematicModel::GetBrushForKey(const FRigElementKey& InKey, const FSchematicGraphNode* InNode) const
{
	static const FSlateBrush* UnresolvedSocketBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Dot.Small");
	static const FSlateBrush* ResolvedSocketBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Dot.Large");
	static const FSlateBrush* ResolvedMultipleSocketBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Dot.Medium");
	static const FSlateBrush* ConnectorPrimaryBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.ConnectorPrimary");
	static const FSlateBrush* ConnectorOptionalBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.ConnectorOptional");
	static const FSlateBrush* ConnectorSecondaryBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.ConnectorSecondary");
	static const FSlateBrush* BoneBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.Bone");
	static const FSlateBrush* ControlBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.Control");
	static const FSlateBrush* NullBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.Null");

	switch(InKey.Type)
	{
		case ERigElementType::Socket:
		{
			if (ControlRigBlueprint.IsValid())
			{
				int32 Count = 0;
				for(const FModularRigSingleConnection& Connection : ControlRigBlueprint->GetModularRigModel().Connections)
				{
					for(const FRigElementKey& Target : Connection.Targets)
					{
						if(Target == InKey)
						{
							Count++;
							if(Count == 2)
							{
								break;
							}
						}
					}
				}

				if(Count == 1)
				{
					return ResolvedSocketBrush;
				}
				if(Count == 2)
				{
					return ResolvedMultipleSocketBrush;
				}
			}

			return UnresolvedSocketBrush;
		}
		case ERigElementType::Connector:
		{
			if (ControlRigBlueprint.IsValid())
			{
				if(const UModularRig* ControlRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig()))
				{
					if(const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
					{
						if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(InKey))
						{
							if((Connector->IsPrimary()) && !IsConnectorResolved(InKey))
							{
								if(const FRigModuleInstance* ModuleInstance = ControlRig->FindModule(Connector))
								{
									const FSoftObjectPath& IconPath = ModuleInstance->GetRig()->GetRigModuleSettings().Icon;
									const FSlateBrush* IconBrush = ModuleIcons.Find(IconPath);
									if (!IconBrush)
									{
										if(UTexture2D* Icon = Cast<UTexture2D>(IconPath.TryLoad()))
										{
											IconBrush = &ModuleIcons.Add(IconPath, UWidgetBlueprintLibrary::MakeBrushFromTexture(Icon, 16.0f, 16.0f));
										}
									}
									if(IconBrush)
									{
										return IconBrush;
									}
								}
							}

							if(Connector->IsSecondary())
							{
								return Connector->IsOptional() ? 	ConnectorOptionalBrush : ConnectorSecondaryBrush;
							}
						}
					}
				}
			}
			return ConnectorPrimaryBrush;
		}
		default:
		{
			// check if the key has a resolved connector
			if(const FSchematicGraphGroupNode* GroupNode = Cast<FSchematicGraphGroupNode>(InNode))
			{
				if(!GroupNode->IsExpanded() && !GroupNode->IsExpanding())
				{
					if (ControlRigBlueprint.IsValid())
					{
						int32 Count = 0;
						for(const FModularRigSingleConnection& Connection : ControlRigBlueprint->GetModularRigModel().Connections)
						{
							for(const FRigElementKey& Target : Connection.Targets)
							{
								if(Target == InKey)
								{
									Count++;
									if(Count == 2)
									{
										return ResolvedMultipleSocketBrush;
									}
								}
							}
						}
						if(Count > 0)
						{
							return ResolvedSocketBrush;
						}
					}
				}
			}
			
			switch(InKey.Type)
			{
				case ERigElementType::Bone:
				{
					return BoneBrush;
				}
				case ERigElementType::Control:
				{
					return ControlBrush;
				}
				case ERigElementType::Null:
				{
					return NullBrush;
				}
				default:
				{
					break;
				}
			}
			break;
		}
	}
	return nullptr;
}

const FSlateBrush* FControlRigSchematicModel::GetBrushForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const
{
	if(const FControlRigSchematicRigElementKeyNode* Node = Cast<FControlRigSchematicRigElementKeyNode>(InNode))
	{
		if(InLayerIndex == 0)
		{
			static const FSlateBrush* BackgroundBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Background");
			return BackgroundBrush;
		}
		if(InLayerIndex == 1)
		{
			static const FSlateBrush* OutlineSingleBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Outline.Single");
			static const FSlateBrush* OutlineDoubleBrush = FSchematicGraphStyle::Get().GetBrush( "Schematic.Outline.Double");

			if (ControlRigBlueprint.IsValid())
			{
				int32 Count = 0;
				for(const FModularRigSingleConnection& Connection : ControlRigBlueprint->GetModularRigModel().Connections)
				{
					for(const FRigElementKey& Target : Connection.Targets)
					{
						if(Target == Node->GetKey())
						{
							Count++;
							if(Count == 2)
							{
								return OutlineDoubleBrush;
							}
						}
					}
				}
			}
			return OutlineSingleBrush;
		}

		if(const FSlateBrush* Brush = GetBrushForKey(Node->GetKey(), Node))
		{
			return Brush;
		}
	}
	return Super::GetBrushForNode(InNode, InLayerIndex);
}

FLinearColor FControlRigSchematicModel::GetColorForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const
{
	if (!ControlRigBeingDebuggedPtr.IsValid())
	{
		return Super::GetColorForNode(InNode, InLayerIndex);
	}

	const URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy();
	if(Hierarchy == nullptr)
	{
		return Super::GetColorForNode(InNode, InLayerIndex);
	}

	static const FLinearColor SelectionColor = FLinearColor(FColor::FromHex(TEXT("#EBA30A")));
			
	if(const FControlRigSchematicRigElementKeyNode* ElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(InNode))
	{
		if(Hierarchy->IsSelected(ElementKeyNode->GetKey()))
		{
			return SelectionColor;
		}

		if(InLayerIndex == 0) // background
		{
			return FLinearColor(0, 0, 0, 0.75);
		}
		
		switch(ElementKeyNode->GetKey().Type)
		{
			case ERigElementType::Bone:
			{
				return FControlRigEditorStyle::Get().BoneUserInterfaceColor;
			}
			case ERigElementType::Null:
			{
				return FControlRigEditorStyle::Get().NullUserInterfaceColor;
			}
			case ERigElementType::Control:
			{
				if(const FRigControlElement* Control = Hierarchy->Find<FRigControlElement>(ElementKeyNode->GetKey()))
				{
					return Control->Settings.ShapeColor;
				}
				break;
			}
			case ERigElementType::Socket:
			{
				if(const FRigSocketElement* Socket = Hierarchy->Find<FRigSocketElement>(ElementKeyNode->GetKey()))
				{
					return Socket->GetColor(Hierarchy);
				}
				break;
			}
			case ERigElementType::Connector:
			{
				return FLinearColor::White;
				//return FControlRigEditorStyle::Get().ConnectorUserInterfaceColor;
			}
			default:
			{
				break;
			}
		}
	}
	else if(const FSchematicGraphAutoGroupNode* AutoGroupNode = Cast<FSchematicGraphAutoGroupNode>(InNode))
	{
		for(int32 ChildIndex = 0; ChildIndex < AutoGroupNode->GetNumChildNodes(); ChildIndex++)
		{
			if(const FControlRigSchematicRigElementKeyNode* ChildElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(AutoGroupNode->GetChildNode(ChildIndex)))
			{
				if(Hierarchy->IsSelected(ChildElementKeyNode->GetKey()))
				{
					return SelectionColor;
				}
			}
		}
	}
	return Super::GetColorForNode(InNode, InLayerIndex);
}

ESchematicGraphVisibility::Type FControlRigSchematicModel::GetVisibilityForNode(const FSchematicGraphNode* InNode) const
{
	if(ControlRigBlueprint.IsValid())
	{
		if(const FControlRigSchematicRigElementKeyNode* Node = Cast<FControlRigSchematicRigElementKeyNode>(InNode))
		{
			const UModularRig* ModularRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
			const URigHierarchy* Hierarchy = ModularRig ? ModularRig->GetHierarchy() : nullptr;

			if (ModularRig && Hierarchy)
			{
				const UControlRigEditorSettings* EditorSettings = GetDefault<UControlRigEditorSettings>();
				if (EditorSettings->bSchematicViewportShowEmptySocketsOnly)
				{
					if (Node->GetKey().Type == ERigElementType::Socket)
					{
						const FRigSocketElement* Socket = Hierarchy->Find<FRigSocketElement>(Node->GetKey());
						const FRigTransformElement* SingleParentElement = Socket ? Socket->ParentElement : nullptr;
						if (Socket &&
							SingleParentElement)
						{
							const TArray<FRigElementKey> SocketConnectors = ControlRigBlueprint->GetModularRigModel().Connections.FindConnectorsFromTarget(Socket->GetKey());
							if (!SocketConnectors.IsEmpty())
							{
								return ESchematicGraphVisibility::Hidden;
							}
						}
					}
					else 
					{
						return ESchematicGraphVisibility::Hidden;
					}
				}

				if (Node->GetKey().Type == ERigElementType::Connector)
				{
					if (const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(Node->GetKey()))
					{
						if (IsConnectorResolved(Connector->GetKey()))
						{
							if (Connector->IsPrimary())
							{
								if (const FRigModuleInstance* ModuleInstance = ModularRig->FindModule(Connector))
								{
									if (ModuleInstance->IsRootModule())
									{
										return ESchematicGraphVisibility::Hidden;
									}
								}
							}
						}
						else
						{
							return ESchematicGraphVisibility::Hidden;
						}
					}
				}
			}
		}
	}
	return Super::GetVisibilityForNode(InNode);
}

const FSlateBrush* FControlRigSchematicModel::GetBrushForLink(const FSchematicGraphLink* InLink) const
{
	if(InLink->IsA<FControlRigSchematicRigElementKeyLink>())
	{
		if(const FSchematicGraphNode* SourceNode = FindNode(InLink->GetSourceNodeGuid()))
		{
			if(const FSchematicGraphNode* TargetNode = FindNode(InLink->GetTargetNodeGuid()))
			{
				if(SourceNode->IsA<FControlRigSchematicRigElementKeyNode>() &&
					TargetNode->IsA<FControlRigSchematicRigElementKeyNode>())
				{
					static const FSlateBrush* LinkBrush = FControlRigEditorStyle::Get().GetBrush( "ControlRig.Schematic.Link");
					return LinkBrush;
				}
			}
		}
	}
	return Super::GetBrushForLink(InLink);
}

FLinearColor FControlRigSchematicModel::GetColorForLink(const FSchematicGraphLink* InLink) const
{
	if(const FSchematicGraphNode* SourceNode = FindNode(InLink->GetSourceNodeGuid()))
	{
		if(const FSchematicGraphNode* TargetNode = FindNode(InLink->GetTargetNodeGuid()))
		{
			if(SourceNode->IsA<FControlRigSchematicRigElementKeyNode>() &&
				TargetNode->IsA<FControlRigSchematicRigElementKeyNode>())
			{
				// semi transparent dark gray
				return FLinearColor(0.7f, 0.7f, 0.7f, .5f);
			}
		}
	}
	return Super::GetColorForLink(InLink);
}

ESchematicGraphVisibility::Type FControlRigSchematicModel::GetVisibilityForTag(const FSchematicGraphTag* InTag) const
{
	if(InTag->IsA<FSchematicGraphGroupTag>())
	{
		if(const FControlRigSchematicRigElementKeyNode* Node = Cast<FControlRigSchematicRigElementKeyNode>(InTag->GetNode()))
		{
			if(Node->GetNumChildNodes() < 3)
			{
				return ESchematicGraphVisibility::Hidden;
			}
		}
	}
	if(InTag->IsA<FControlRigSchematicWarningTag>())
	{
		if(const FControlRigSchematicRigElementKeyNode* ConnectorNode = Cast<FControlRigSchematicRigElementKeyNode>(InTag->GetNode()))
		{
			if(ConnectorNode->GetKey().Type == ERigElementType::Connector)
			{
				FRigElementKey ResolvedKey;
				if(!IsConnectorResolved(ConnectorNode->GetKey(), &ResolvedKey))
				{
					return ESchematicGraphVisibility::Visible;
				}
			}
		}
		else if(const FSchematicGraphAutoGroupNode* AutoGroupNode = Cast<FSchematicGraphAutoGroupNode>(InTag->GetNode()))
		{
			for(int32 Index = 0; Index < AutoGroupNode->GetNumChildNodes(); Index++)
			{
				if(const FSchematicGraphNode* ChildNode = AutoGroupNode->GetChildNode(Index))
				{
					if(const FControlRigSchematicWarningTag* ChildTag = ChildNode->FindTag<FControlRigSchematicWarningTag>())
					{
						const ESchematicGraphVisibility::Type ChildVisibility = GetVisibilityForTag(ChildTag); 
						if(ChildVisibility != ESchematicGraphVisibility::Hidden)
						{
							return ChildVisibility;
						}
					}
				}
			}
		}
		return ESchematicGraphVisibility::Hidden;
	}
	return Super::GetVisibilityForTag(InTag);
}

const FText FControlRigSchematicModel::GetToolTipForTag(const FSchematicGraphTag* InTag) const
{
	if(InTag->IsA<FControlRigSchematicWarningTag>())
	{
		if(const FControlRigSchematicRigElementKeyNode* ConnectorNode = Cast<FControlRigSchematicRigElementKeyNode>(InTag->GetNode()))
		{
			if(ConnectorNode->GetKey().Type == ERigElementType::Connector)
			{
				if(!IsConnectorResolved(ConnectorNode->GetKey()))
				{
					static const FText UnresolvedConnectorToolTip = LOCTEXT("UnresolvedConnectorToolTip", "Connector is unresolved.");
					return UnresolvedConnectorToolTip;
				}
			}
		}
		else if(const FSchematicGraphAutoGroupNode* AutoGroupNode = Cast<FSchematicGraphAutoGroupNode>(InTag->GetNode()))
		{
			for(int32 Index = 0; Index < AutoGroupNode->GetNumChildNodes(); Index++)
			{
				if(const FSchematicGraphNode* ChildNode = AutoGroupNode->GetChildNode(Index))
				{
					if(const FControlRigSchematicWarningTag* ChildTag = ChildNode->FindTag<FControlRigSchematicWarningTag>())
					{
						const FText ChildToolTip = GetToolTipForTag(ChildTag); 
						if(!ChildToolTip.IsEmpty())
						{
							return ChildToolTip;
						}
					}
				}
			}
		}
	}
	return Super::GetToolTipForTag(InTag);
}

bool FControlRigSchematicModel::GetForwardedNodeForDrag(FGuid& InOutGuid) const
{
	if(const FControlRigSchematicRigElementKeyNode* Node = FindNode<FControlRigSchematicRigElementKeyNode>(InOutGuid))
	{
		if(ControlRigBlueprint.IsValid())
		{
			const URigHierarchy* Hierarchy = ControlRigBlueprint->GetDebuggedControlRig()->GetHierarchy();
			check(Hierarchy);
			const FModularRigConnections& Connections = ControlRigBlueprint->GetModularRigModel().Connections;
			const TArray<FRigElementKey>& Connectors = Connections.FindConnectorsFromTarget(Node->GetKey());
			if(Connectors.Num() == 1)
			{
				if(const FControlRigSchematicRigElementKeyNode* ConnectorNode = FindElementKeyNode(Connectors[0]))
				{
					InOutGuid = ConnectorNode->GetGuid();
					return true;
				}
			}
		}
	}
	return Super::GetForwardedNodeForDrag(InOutGuid);
}

bool FControlRigSchematicModel::GetContextMenuForNode(const FSchematicGraphNode* InNode, FMenuBuilder& OutMenu) const
{
	bool bSuccess = false;
	if(Super::GetContextMenuForNode(InNode, OutMenu))
	{
		bSuccess = true;
	}
	
	if(ControlRigBlueprint.IsValid())
	{
		if(const FControlRigSchematicRigElementKeyNode* ElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(InNode))
		{
			const UModularRig* ModularRig = CastChecked<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
			const FModularRigConnections& Connections = ControlRigBlueprint->GetModularRigModel().Connections;
			const TArray<FRigElementKey>& Connectors = Connections.FindConnectorsFromTarget(ElementKeyNode->Key);
			if(!Connectors.IsEmpty())
			{
				OutMenu.BeginSection(TEXT("DisconnectConnectors"), LOCTEXT("DisconnectConnectors", "Disconnect"));

				// note: this is a copy on purpose since it is passed into the lambda
				for(const FRigElementKey Connector : Connectors)
				{
					FText Label = ModularRig->GetHierarchy()->GetDisplayNameForUI(Connector, EElementNameDisplayMode::ForceShort);
					const FText Description = FText::FromString(FString::Printf(TEXT("Disconnect %s"), *Label.ToString()));
					
					OutMenu.AddMenuEntry(Description, Description, FSlateIcon(), FUIAction(
						FExecuteAction::CreateLambda([this, Connector]()
						{
							if(ControlRigBlueprint.IsValid())
							{
								if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
								{
									FScopedTransaction Transaction(LOCTEXT("DisconnectConnector", "Disconnect Connector"));
									ControlRigBlueprint->Modify();
									Controller->DisconnectConnector(Connector);
								}
							}								
						})
					));
				}
				OutMenu.EndSection();
				bSuccess = true;
			}
		}
	}
	return bSuccess;
}

TArray<FRigElementKey> FControlRigSchematicModel::GetElementKeysFromDragDropEvent(const FDragDropOperation& InDragDropOperation, const UControlRig* InControlRig)
{
	TArray<FRigElementKey> DraggedKeys;

	if(InDragDropOperation.IsOfType<FRigHierarchyElementDragDropOp>())
	{
		const FRigHierarchyElementDragDropOp* RigDragDropOp = StaticCast<const FRigHierarchyElementDragDropOp*>(&InDragDropOperation);
		const TArray<FRigHierarchyKey>& HierarchyKeys = RigDragDropOp->GetElements();
		TArray<FRigElementKey> Elements;
		Elements.Reserve(HierarchyKeys.Num());
		for(const FRigHierarchyKey& HierarchyKey : HierarchyKeys)
		{
			if(HierarchyKey.IsElement())
			{
				Elements.Add(HierarchyKey.GetElement());
			}
		}
		return Elements;
	}

	if(InDragDropOperation.IsOfType<FRigHierarchyTagDragDropOp>())
	{
		const FRigHierarchyTagDragDropOp* TagDragDropOp = StaticCast<const FRigHierarchyTagDragDropOp*>(&InDragDropOperation);
		FRigElementKey DraggedKey;
		FRigElementKey::StaticStruct()->ImportText(*TagDragDropOp->GetIdentifier(), &DraggedKey, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);
		return {DraggedKey};
	}
	
	if(InDragDropOperation.IsOfType<FModularRigHierarchyElementDragDropOp>())
	{
		const FModularRigHierarchyElementDragDropOp* ModuleDropOp = StaticCast<const FModularRigHierarchyElementDragDropOp*>(&InDragDropOperation);
		const TArray<FName> ModuleNames = ModuleDropOp->GetModules();

		URigHierarchy* Hierarchy = InControlRig->GetHierarchy();
		if (!Hierarchy)
		{
			return DraggedKeys;
		}

		const TArray<FRigConnectorElement*> Connectors = Hierarchy->GetConnectors();
		for (const FName& ModulePathOrConnectorName : ModuleNames)
		{
			if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(FRigElementKey(ModulePathOrConnectorName, ERigElementType::Connector)))
			{
				DraggedKeys.Add(Connector->GetKey());
			}

			if(const UModularRig* ModularRig = Cast<UModularRig>(InControlRig))
			{
				if(const FRigModuleInstance* ModuleInstance = ModularRig->FindModule(ModulePathOrConnectorName))
				{
					for(const FRigConnectorElement* Connector : Connectors)
					{
						const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connector->GetKey());
						if(ModuleInstance->Name == ConnectorModuleName)
						{
							if(Connector->IsPrimary())
							{
								DraggedKeys.Add(Connector->GetKey());
							}
						}
					}
				}
			}
		}
	}

	if(InDragDropOperation.IsOfType<FSchematicGraphNodeDragDropOp>())
	{
		const FSchematicGraphNodeDragDropOp* SchematicGraphNodeDragDropOp = StaticCast<const FSchematicGraphNodeDragDropOp*>(&InDragDropOperation);
		const TArray<const FSchematicGraphNode*> Nodes = SchematicGraphNodeDragDropOp->GetNodes();

		for(const FSchematicGraphNode* Node : Nodes)
		{
			if(const FControlRigSchematicRigElementKeyNode* ElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(Node))
			{
				DraggedKeys.Add(ElementKeyNode->GetKey());
			}
		}
	}

	return DraggedKeys;
}

void FControlRigSchematicModel::HandleSchematicNodeClicked(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const FPointerEvent& InMouseEvent)
{
	const bool bClearSelection = !InMouseEvent.IsShiftDown() && !InMouseEvent.IsControlDown();
	for (const TSharedPtr<FSchematicGraphNode>& Node : Nodes)
	{
		if (bClearSelection)
		{
			Node->SetSelected(Node->GetGuid() == InNode->GetGuid());
		}
		else if(Node->GetGuid() == InNode->GetGuid())
		{
			Node->SetSelected();
		}
	}

	if(const FControlRigSchematicRigElementKeyNode* Node =
			Cast<FControlRigSchematicRigElementKeyNode>(InNode->GetNodeData()))
	{
		if (ControlRigBeingDebuggedPtr.IsValid())
		{
			if (URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy())
			{
				if (URigHierarchyController* Controller = Hierarchy->GetController())
				{
					bool bSelect = true;
					if(InMouseEvent.IsControlDown())
					{
						if(Hierarchy->IsSelected(Node->GetKey()))
						{
							bSelect = false;
						}
					}
					Controller->SelectElement(Node->GetKey(), bSelect, bClearSelection);
				}
			}
		}
	}
}

void FControlRigSchematicModel::HandleSchematicBeginDrag(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const TSharedPtr<FDragDropOperation>& InDragDropOperation)
{
	if (!ControlRigBlueprint.IsValid())
	{
		return;
	}

	if (!ControlRigBeingDebuggedPtr.IsValid())
	{
		return;
	}

	FRigElementKey DraggedKey;
	if(const FControlRigSchematicRigElementKeyNode* ElementKeyNode =
		Cast<FControlRigSchematicRigElementKeyNode>(InNode->GetNodeData()))
	{
		DraggedKey = ElementKeyNode->GetKey();
	}
	else if(InDragDropOperation.IsValid())
	{
		const TArray<FRigElementKey> DraggedKeys = GetElementKeysFromDragDropEvent(*InDragDropOperation.Get(), ControlRigBlueprint->GetDebuggedControlRig());
		if(!DraggedKeys.IsEmpty())
		{
			DraggedKey = DraggedKeys[0];
		}
	}

	OnShowCandidatesForConnector(DraggedKey);
}

void FControlRigSchematicModel::HandleSchematicEndDrag(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const TSharedPtr<FDragDropOperation>& InDragDropOperation)
{
	OnHideCandidatesForConnector();
}

void FControlRigSchematicModel::HandleSchematicEnterDrag(SSchematicGraphPanel* InPanel, const TSharedPtr<FDragDropOperation>& InDragDropOperation)
{
	if(!ControlRigBlueprint.IsValid())
	{
		return;
	}
	if(!InDragDropOperation.IsValid())
	{
		return;
	}
	
	const TArray<FRigElementKey> DraggedKeys = GetElementKeysFromDragDropEvent(*InDragDropOperation.Get(), ControlRigBlueprint->GetDebuggedControlRig());
	if(!DraggedKeys.IsEmpty())
	{
		OnShowCandidatesForConnector(DraggedKeys[0]);
		return;
	}

	if(InDragDropOperation->IsOfType<FAssetDragDropOp>())
	{
		const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(InDragDropOperation);
		for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
		{
			const UClass* AssetClass = AssetData.GetClass();
			if (!AssetClass->ImplementsInterface(UControlRigEditorAssetInterface::StaticClass()) &&
				!AssetClass->ImplementsInterface(UControlRigRuntimeAssetInterface::StaticClass()))
			{
				continue;
			}

			if(FControlRigAssetInterfacePtr AssetBlueprint = AssetData.GetAsset())
			{
				if(AssetBlueprint->IsControlRigModule())
				{
					if(const FRigModuleConnector* PrimaryConnector = AssetBlueprint->GetRigModuleSettings().FindPrimaryConnector())
					{
						OnShowCandidatesForConnector(PrimaryConnector);
					}
				}
			}
			else if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = AssetData.GetAsset())
			{
				if(const FRigModuleConnector* PrimaryConnector = RuntimeAsset->GetRigModuleSettings().FindPrimaryConnector())
				{
					OnShowCandidatesForConnector(PrimaryConnector);
				}
			}
		}
	}
}

void FControlRigSchematicModel::HandleSchematicLeaveDrag(SSchematicGraphPanel* InPanel, const TSharedPtr<FDragDropOperation>& InDragDropOperation)
{
	OnHideCandidatesForConnector();
}

void FControlRigSchematicModel::HandleSchematicCancelDrag(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const TSharedPtr<FDragDropOperation>& InDragDropOperation)
{
	if (!ControlRigBlueprint.IsValid())
	{
		return;
	}

	const FControlRigSchematicRigElementKeyNode* ElementKeyNode =
		Cast<FControlRigSchematicRigElementKeyNode>(InNode->GetNodeData());
	if(ElementKeyNode == nullptr)
	{
		return;
	}

	if (InDragDropOperation.IsValid() && InDragDropOperation->IsOfType<FSchematicGraphNodeDragDropOp>())
	{
		const TSharedPtr<FSchematicGraphNodeDragDropOp> SchematicDragDropOp = StaticCastSharedPtr<FSchematicGraphNodeDragDropOp>(InDragDropOperation);
		const TArray<FGuid> Sources = SchematicDragDropOp->GetElements();
		TArray<FRigElementKey> Keys;
		for(const FGuid& Source : Sources)
		{
			if(const FControlRigSchematicRigElementKeyNode* SourceNode = FindNode<FControlRigSchematicRigElementKeyNode>(Source))
			{
				Keys.Add(SourceNode->GetKey());
			}
		}

		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, Keys]()
		{
			if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
			{
				FScopedTransaction Transaction(LOCTEXT("DisconnectConnector", "Disconnect Connector"));
				ControlRigBlueprint->Modify();
				for(const FRigElementKey& Key : Keys)
				{
					Controller->DisconnectConnector(Key);
				}
			}
		});
	}
}

void FControlRigSchematicModel::HandleSchematicDrop(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const FDragDropEvent& InDragDropEvent)
{
	if (!ControlRigBlueprint.IsValid())
	{
		return;
	}

	struct Local
	{
		static void CollectTargetKeys(TArray<FRigElementKey>& OutKeys, const FSchematicGraphNode* InNode)
		{
			check(InNode);
			
			if(const FControlRigSchematicRigElementKeyNode* ElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(InNode))
			{
				OutKeys.Add(ElementKeyNode->GetKey());
			}

			if(const FSchematicGraphGroupNode* GroupNode = Cast<FSchematicGraphGroupNode>(InNode))
			{
				for(int32 Index = 0; Index < GroupNode->GetNumChildNodes(); Index++)
				{
					if(const FSchematicGraphNode* ChildNode = GroupNode->GetChildNode(Index))
					{
						CollectTargetKeys(OutKeys, ChildNode);
					}
				}
			}
		}
	};

	TArray<FRigElementKey> TargetKeys;
	TArray<TSharedPtr<FSchematicGraphNode>> SelectedNodesShared = GetSelectedNodes();
	TArray<FSchematicGraphNode*> SelectedNodes;
	SelectedNodes.Reserve(SelectedNodesShared.Num());
	bool bApplyToSelection = false;
	for (TSharedPtr<FSchematicGraphNode>& Node : SelectedNodesShared)
	{
		if (Node.IsValid())
		{
			SelectedNodes.AddUnique(Node.Get());
			if (Node.Get() == InNode->GetNodeData())
			{
				bApplyToSelection = true;
			}
		}
	}

	// If the dropped target node is not part of the selection, ignore the selection
	if (!bApplyToSelection)
	{
		SelectedNodes.Reset();
		SelectedNodes.Add(InNode->GetNodeData());
	}

	for (FSchematicGraphNode* Node : SelectedNodes)
	{
		TArray<FRigElementKey> NodeTargetKeys;
		Local::CollectTargetKeys(NodeTargetKeys, Node);
		TargetKeys.Append(NodeTargetKeys);
	}
	
	UControlRig* ControlRig = ControlRigBlueprint->GetDebuggedControlRig();
	if (!ControlRig)
	{
		return;
	}

	URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
	if (!Hierarchy)
	{
		return;
	}

	TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>();
	TSharedPtr<FSchematicGraphNodeDragDropOp> SchematicDragDropOp = InDragDropEvent.GetOperationAs<FSchematicGraphNodeDragDropOp>();
	TSharedPtr<FModularRigHierarchyElementDragDropOp> ModuleDragDropOperation = InDragDropEvent.GetOperationAs<FModularRigHierarchyElementDragDropOp>();
	if (AssetDragDropOp.IsValid())
	{
		for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
		{
			FControlRigAssetStrongReference ControlRigAssetReference = nullptr;
			FRigModuleSettings ModuleSettings;
			if (FControlRigAssetInterfacePtr AssetBlueprint = AssetData.GetAsset())
			{
				ControlRigAssetReference = AssetBlueprint->GetControlRigAssetReference();
				ModuleSettings = AssetBlueprint->GetRigModuleSettings();
			}
			else if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = AssetData.GetAsset())
			{
				ControlRigAssetReference = RuntimeAsset.GetObject();
				ModuleSettings = RuntimeAsset->GetRigModuleSettings();
			}

			if(ControlRigAssetReference.IsValid() && ModuleSettings.IsValidModule())
			{
				ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, ControlRigAssetReference, ModuleSettings, TargetKeys]()
				{
					if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
					{
						UModularRig* ControlRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
						if (!ControlRig)
						{
							return;
						}

						URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
						if (!Hierarchy)
						{
							return;
						}

						FScopedTransaction Transaction(LOCTEXT("AddAndConnectModule", "Add and Connect Module"));

						for(const FRigElementKey& TargetKey : TargetKeys)
						{
							const FName DesiredModuleName = Controller->GetSafeNewName(FRigName(ModuleSettings.Identifier.Name));
							const FName ModuleName = Controller->AddModuleFromAssetReference(DesiredModuleName, ControlRigAssetReference, NAME_None);
							if(!ModuleName.IsNone())
							{
								const FString ModuleNameString = ModuleName.ToString();
								FRigElementKey PrimaryConnectorKey;
								TArray<FRigConnectorElement*> Connectors = Hierarchy->GetElementsOfType<FRigConnectorElement>();
								for (FRigConnectorElement* Connector : Connectors)
								{
									if (Connector->IsPrimary())
									{
										const FRigHierarchyModulePath ConnectorModulePath(Connector->GetName());
										if (ConnectorModulePath.HasModuleName(ModuleNameString))
										{
											PrimaryConnectorKey = Connector->GetKey();
											break;
										}
									}
								}
								Controller->ConnectConnectorToElement(PrimaryConnectorKey, TargetKey, true, ControlRig->GetModularRigSettings().bAutoResolve);
							}
						}

						ClearSelection();
					}
				});
			}
		}
	}
	else if(SchematicDragDropOp.IsValid())
	{
		const TArray<FGuid> Sources = SchematicDragDropOp->GetElements();
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, Sources, TargetKeys]()
		{
			UModularRig* ControlRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
			if (!ControlRig)
			{
				return;
			}

			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			if (!Hierarchy)
			{
				return;
			}

			if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
			{
				for (const TPair<FRigElementKey, FGuid> Pair : RigElementKeyToGuid)
				{
					if(Sources.Contains(Pair.Value))
					{
						if (Hierarchy->Find<FRigConnectorElement>(Pair.Key))
						{
							for(const FRigElementKey& TargetKey : TargetKeys)
							{
								if(Controller->ConnectConnectorToElement(Pair.Key, TargetKey, true, ControlRig->GetModularRigSettings().bAutoResolve))
								{
									break;
								}
							}
							break;
						}
					}
				}

				ClearSelection();
			}
		});
	}
	else if(ModuleDragDropOperation.IsValid())
	{
		const TArray<FName> ModuleNames = ModuleDragDropOperation->GetModules();
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, ModuleNames, TargetKeys]()
		{
			UModularRig* ControlRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
			if (!ControlRig)
			{
				return;
			}

			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			if (!Hierarchy)
			{
				return;
			}

			if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
			{
				const TArray<FRigConnectorElement*> Connectors = Hierarchy->GetConnectors();
				for (const FName& ModulePathOrConnectorName : ModuleNames)
				{
					if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(FRigElementKey(ModulePathOrConnectorName, ERigElementType::Connector)))
					{
						for(const FRigElementKey& TargetKey : TargetKeys)
						{
							if(Controller->ConnectConnectorToElement(Connector->GetKey(), TargetKey, true, ControlRig->GetModularRigSettings().bAutoResolve))
							{
								break;
							}
						}
						ClearSelection();
						return;
					}
					if(const FRigModuleReference* Module = Controller->FindModule(ModulePathOrConnectorName))
					{
						for(const FRigConnectorElement* Connector : Connectors)
						{
							const FName ConnectorModuleName = Hierarchy->GetModuleFName(Connector->GetKey());
							if(Module->Name == ConnectorModuleName)
							{
								if(Connector->IsPrimary())
								{
									const FRigElementKey ConnectorKey = Connector->GetKey();
									for(const FRigElementKey& TargetKey : TargetKeys)
									{
										if(Controller->ConnectConnectorToElement(ConnectorKey, TargetKey, true, ControlRig->GetModularRigSettings().bAutoResolve))
										{
											break;
										}
									}
									ClearSelection();
									return;
								}
							}
						}
					}
				}
			}
		});
	}
}

void FControlRigSchematicModel::HandlePostConstruction(UControlRig* Subject, const FName& InEventName)
{
	UpdateControlRigContent();
}

bool FControlRigSchematicModel::IsConnectorResolved(const FRigElementKey& InConnectorKey, FRigElementKey* OutKey) const
{
	if(ControlRigBlueprint.IsValid() && InConnectorKey.Type == ERigElementType::Connector)
	{
		const FRigElementKey TargetKey = ControlRigBlueprint->GetModularRigModel().Connections.FindTargetFromConnector(InConnectorKey);
		if(TargetKey.IsValid())
		{
			// make sure the target exists
			if(ControlRigBeingDebuggedPtr.IsValid())
			{
				if(const URigHierarchy* DebuggedHierarchy = ControlRigBeingDebuggedPtr->GetHierarchy())
				{
					if(!DebuggedHierarchy->Contains(TargetKey))
					{
						return false;
					}
				}
			}
			
			if(OutKey)
			{
				*OutKey = TargetKey;
			}
			return true;
		}
	}
	return false;
}

void FControlRigSchematicModel::OnShowCandidatesForConnector(const FRigElementKey& InConnectorKey)
{
	if(!InConnectorKey.IsValid())
	{
		return;
	}
	
	URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy();
	if (!Hierarchy)
	{
		return;
	}

	FRigBaseElement* Element = Hierarchy->Find(InConnectorKey);
	if (!Element)
	{
		return;
	}
	
	const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(Element);
	if (!Connector)
	{
		return;
	}

	const UModularRig* ModularRig = Cast<UModularRig>(ControlRigBeingDebuggedPtr);
	if (!ModularRig)
	{
		return;
	}

	const FName ModuleName = Hierarchy->GetModuleFName(InConnectorKey);
	const FRigModuleInstance* ModuleInstance = ModularRig->FindModule(ModuleName);
	if (!ModuleInstance)
	{
		return;
	}

	if(const FControlRigSchematicRigElementKeyNode* Node = FindElementKeyNode(InConnectorKey))
	{
		// close all expanded groups
		FGuid ParentGuid = Node->GetParentNodeGuid();
		while(ParentGuid.IsValid())
		{
			if(FSchematicGraphNode* ParentNode = FindNode(ParentGuid))
			{
				if(FSchematicGraphGroupNode* GroupNode = Cast<FSchematicGraphGroupNode>(ParentNode))
				{
					GroupNode->SetExpanded(false);
				}
				ParentGuid = ParentNode->GetParentNodeGuid();
			}
			else
			{
				break;
			}
		}
	}
	
	const UModularRigRuleManager* RuleManager = Hierarchy->GetRuleManager();
	const FModularRigResolveResult Result = RuleManager->FindMatches(Connector, ModuleInstance, ControlRigBeingDebuggedPtr->GetElementKeyRedirector());
	OnShowCandidatesForMatches(Result);
}

void FControlRigSchematicModel::OnShowCandidatesForConnector(const FRigModuleConnector* InModuleConnector)
{
	check(InModuleConnector);
	
	URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy();
	if (!Hierarchy)
	{
		return;
	}
	
	const UModularRigRuleManager* RuleManager = Hierarchy->GetRuleManager();
	const FModularRigResolveResult Result = RuleManager->FindMatches(InModuleConnector);
	OnShowCandidatesForMatches(Result);
}

void FControlRigSchematicModel::OnShowCandidatesForMatches(const FModularRigResolveResult& InMatches)
{
	const TArray<FRigElementResolveResult> Matches = InMatches.GetMatches();

	for (const FRigElementResolveResult& Match : Matches)
	{
		// Create a temporary node that will be active only while this drag operation exists
		if (!ContainsElementKeyNode(Match.GetKey()))
		{
			FSchematicGraphNode* NewNode = AddElementKeyNode(Match.GetKey());
			NewNode->SetScaleOffset(0.6f);
			TemporaryNodeGuids.Add(NewNode->GetGuid());
		}
	}

	// Fade all the unmatched nodes
	for (const TSharedPtr<FSchematicGraphNode>& Node : Nodes)
	{
		if(FControlRigSchematicRigElementKeyNode* ExistingElementKeyNode = Cast<FControlRigSchematicRigElementKeyNode>(Node.Get()))
		{
			PreDragVisibilityPerNode.Add(Node->GetGuid(), ExistingElementKeyNode->Visibility);
			ExistingElementKeyNode->SetVisibility(Matches.ContainsByPredicate([ExistingElementKeyNode](const FRigElementResolveResult& Match)
			{
				return ExistingElementKeyNode->GetKey() == Match.GetKey();
			}) ? ESchematicGraphVisibility::Visible : ESchematicGraphVisibility::Hidden);
		}
	};
}

void FControlRigSchematicModel::OnHideCandidatesForConnector()
{
	if(TemporaryNodeGuids.IsEmpty() && PreDragVisibilityPerNode.IsEmpty())
	{
		return;
	}
	
	for (FGuid& TempNodeGuid : TemporaryNodeGuids)
	{
		RemoveNode(TempNodeGuid);
	}
	TemporaryNodeGuids.Reset();

	for(const TPair<FGuid, ESchematicGraphVisibility::Type>& Pair : PreDragVisibilityPerNode)
	{
		if(FControlRigSchematicRigElementKeyNode* Node = FindNode<FControlRigSchematicRigElementKeyNode>(Pair.Key))
		{
			Node->Visibility = Pair.Value;
		}
	}
	PreDragVisibilityPerNode.Reset();

	UpdateElementKeyLinks();
}

void FControlRigSchematicModel::OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;
	
	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementSelected:
		{
			if(FControlRigSchematicRigElementKeyNode* Node = FindElementKeyNode(InElement->GetKey()))
			{
				if(InHierarchy->GetSelectedKeys().Num() == 1)
				{
					if(FSchematicGraphGroupNode* GroupNode = Node->GetGroupNode())
					{
						GroupNode->SetExpanded(true);
					}
				}
				Node->SetSelected(true);
			}
			break;
		}
		case ERigHierarchyNotification::ElementDeselected:
		{
			if(FControlRigSchematicRigElementKeyNode* Node = FindElementKeyNode(InElement->GetKey()))
			{
				Node->SetExpanded(false);
				Node->SetSelected(false);
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
