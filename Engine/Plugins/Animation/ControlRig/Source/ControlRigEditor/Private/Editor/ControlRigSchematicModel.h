// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModularRigModel.h"
#include "SchematicGraphPanel/SSchematicGraphPanel.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ControlRigBlueprintLegacy.h"
#include "Input/DragAndDrop.h"

class IControlRigBaseEditor;
class UControlRig;
class URigHierarchy;
struct FRigBaseElement;

/** Node for the schematic view */
class FControlRigSchematicRigElementKeyNode : public FSchematicGraphGroupNode
{
public:

	SCHEMATICGRAPHNODE_BODY(FControlRigSchematicRigElementKeyNode, FSchematicGraphGroupNode)

	virtual ~FControlRigSchematicRigElementKeyNode() override = default;
	
	const FRigElementKey& GetKey() const { return Key; }
	virtual FString GetDragDropDecoratorLabel() const override;
	virtual bool IsAutoScaleEnabled() const override { return true; }
	virtual bool IsDragSupported() const override;
	virtual const FText& GetLabel() const override;
	
protected:

	FRigElementKey Key;

	friend class FControlRigSchematicModel;
};

/** Link between two rig element schematic nodes */
class FControlRigSchematicRigElementKeyLink : public FSchematicGraphLink
{
public:

	SCHEMATICGRAPHLINK_BODY(FControlRigSchematicRigElementKeyLink, FSchematicGraphLink)

	virtual ~FControlRigSchematicRigElementKeyLink() override = default;
	
	const FRigElementKey& GetSourceKey() const { return SourceKey; }
	const FRigElementKey& GetTargetKey() const { return TargetKey; }
	
protected:

	FRigElementKey SourceKey;
	FRigElementKey TargetKey;

	friend class FControlRigSchematicModel;
};

class FControlRigSchematicWarningTag : public FSchematicGraphTag
{
public:

	SCHEMATICGRAPHTAG_BODY(FControlRigSchematicWarningTag, FSchematicGraphTag)

	FControlRigSchematicWarningTag();
	virtual ~FControlRigSchematicWarningTag() override = default;
};


/** Model for the schematic views */
class FControlRigSchematicModel : public FSchematicGraphModel
{
public:

	SCHEMATICGRAPHNODE_BODY(FControlRigSchematicModel, FSchematicGraphModel)

	virtual ~FControlRigSchematicModel() override;
	
	void SetEditor(const TSharedRef<IControlRigBaseEditor>& InEditor);
	virtual void ApplyToPanel(SSchematicGraphPanel* InPanel) override;

	virtual void Reset() override;
	virtual void Tick(float InDeltaTime) override;
	FControlRigSchematicRigElementKeyNode* AddElementKeyNode(const FRigElementKey& InKey, bool bNotify = true);
	const FControlRigSchematicRigElementKeyNode* FindElementKeyNode(const FRigElementKey& InKey) const;
	FControlRigSchematicRigElementKeyNode* FindElementKeyNode(const FRigElementKey& InKey);
	bool ContainsElementKeyNode(const FRigElementKey& InKey) const;
	virtual bool RemoveNode(const FGuid& InGuid) override;
	bool RemoveElementKeyNode(const FRigElementKey& InKey);
	FControlRigSchematicRigElementKeyLink* AddElementKeyLink(const FRigElementKey& InSourceKey, const FRigElementKey& InTargetKey, bool bNotify = true);
	void UpdateElementKeyNodes();
	void UpdateElementKeyLinks();
	void UpdateControlRigContent();
	void UpdateConnector(const FRigElementKey& InElementKey);

	void OnSetObjectBeingDebugged(UObject* InObject);
	void HandleModularRigModified(EModularRigNotification InNotification, const FRigModuleReference* InModule);

	virtual bool IsAutoGroupingEnabled() const { return false; }
	virtual FSchematicGraphGroupNode* AddAutoGroupNode() override;
	virtual FVector2d GetPositionForNode(const FSchematicGraphNode* InNode) const override;
	virtual bool GetPositionAnimationEnabledForNode(const FSchematicGraphNode* InNode) const override;
	virtual int32 GetNumLayersForNode(const FSchematicGraphNode* InNode) const override;
	const FSlateBrush* GetBrushForKey(const FRigElementKey& InKey, const FSchematicGraphNode* InNode) const;
	virtual const FSlateBrush* GetBrushForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const override;
	virtual FLinearColor GetColorForNode(const FSchematicGraphNode* InNode, int32 InLayerIndex) const override;
	virtual ESchematicGraphVisibility::Type GetVisibilityForNode(const FSchematicGraphNode* InNode) const override;
	virtual const FSlateBrush* GetBrushForLink(const FSchematicGraphLink* InLink) const override;
	virtual FLinearColor GetColorForLink(const FSchematicGraphLink* InLink) const override;
	virtual ESchematicGraphVisibility::Type GetVisibilityForTag(const FSchematicGraphTag* InTag) const override;
	virtual const FText GetToolTipForTag(const FSchematicGraphTag* InTag) const override;
	virtual bool GetForwardedNodeForDrag(FGuid& InOutGuid) const override;
	virtual bool GetContextMenuForNode(const FSchematicGraphNode* InNode, FMenuBuilder& OutMenu) const override;

	static TArray<FRigElementKey> GetElementKeysFromDragDropEvent(const FDragDropOperation& InDragDropOperation, const UControlRig* InControlRig);

private:

	void ConfigureElementKeyNode(FControlRigSchematicRigElementKeyNode* InNode, const FRigElementKey& InKey);

	void HandleSchematicNodeClicked(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const FPointerEvent& InMouseEvent);
	void HandleSchematicBeginDrag(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const TSharedPtr<FDragDropOperation>& InDragDropOperation);
	void HandleSchematicEndDrag(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const TSharedPtr<FDragDropOperation>& InDragDropOperation);
	void HandleSchematicEnterDrag(SSchematicGraphPanel* InPanel, const TSharedPtr<FDragDropOperation>& InDragDropOperation);
	void HandleSchematicLeaveDrag(SSchematicGraphPanel* InPanel, const TSharedPtr<FDragDropOperation>& InDragDropOperation);
	void HandleSchematicCancelDrag(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const TSharedPtr<FDragDropOperation>& InDragDropOperation);
	void HandleSchematicDrop(SSchematicGraphPanel* InPanel, SSchematicGraphNode* InNode, const FDragDropEvent& InDragDropEvent);
	void HandlePostConstruction(UControlRig* Subject, const FName& InEventName);
	bool IsConnectorResolved(const FRigElementKey& InConnectorKey, FRigElementKey* OutKey = nullptr) const;
	void OnShowCandidatesForConnector(const FRigElementKey& InConnectorKey);
	void OnShowCandidatesForConnector(const FRigModuleConnector* InModuleConnector);
	void OnShowCandidatesForMatches(const FModularRigResolveResult& InMatches);
	void OnHideCandidatesForConnector();
	void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);

	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;
	TWeakInterfacePtr<IControlRigEditorAssetInterface> ControlRigBlueprint;
	TWeakObjectPtr<UControlRig> ControlRigBeingDebuggedPtr;

	TArray<FGuid> TemporaryNodeGuids;
	TMap<FGuid, ESchematicGraphVisibility::Type> PreDragVisibilityPerNode;
	TMap<FRigElementKey, FGuid> RigElementKeyToGuid;
	mutable TMap<FSoftObjectPath, FSlateBrush> ModuleIcons;
	bool bUpdateElementKeyLinks = true;

	friend class FControlRigBaseEditor;
	friend class FControlRigSchematicRigElementKeyNode;
};
