// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigShowSchematicViewportOverride.h"

#include "Framework/Application/SlateApplication.h"
#include "Settings/ControlRigSettings.h"

namespace UE::ControlRigEditor
{
	void FControlRigShowSchematicViewportOverride::OverrideDuringDragDrop(const bool bOverrideShowSchematicViewport)
	{
		const bool bDragDropping = FSlateApplication::Get().IsDragDropping();
		if (bDragDropping && !TickerHandle.IsValid())
		{
			const bool bRestoreState = UControlRigEditorSettings::Get()->bShowSchematicViewInModularRig;
			UControlRigEditorSettings::Get()->bShowSchematicViewInModularRig = bOverrideShowSchematicViewport;

			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[bRestoreState](float DeltaTime)
				{
					if (IsEngineExitRequested())
					{
						return false;
					}

					if (!FSlateApplication::Get().IsDragDropping())
					{
						UControlRigEditorSettings::Get()->bShowSchematicViewInModularRig = bRestoreState;

						return false;
					}

					return true;
				}));
		}
	}
}
