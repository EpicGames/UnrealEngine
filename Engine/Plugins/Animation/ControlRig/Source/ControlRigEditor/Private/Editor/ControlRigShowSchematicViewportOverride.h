// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Templates/SharedPointer.h"

namespace UE::ControlRigEditor
{
	/** Utility to override per project per user settings schematic viewport visibility */
	struct FControlRigShowSchematicViewportOverride
	{
		/** 
		 * Overrides the schematic viewport visibililty during a drag drop op.
		 * 
		 * Note, this function has to be called when a drag drop op is already ongoing, 
		 * it has no effect when called while FSlateApplication::IsDragDropping is false.
		 * 
		 * Note, this function can be called once or repetitively during a drag drop op.
		 */
		void OverrideDuringDragDrop(const bool bOverrideShowSchematicViewport);

	private:
		/** Handle for the ticker while overriden */
		FTSTicker::FDelegateHandle TickerHandle;
	};
}
