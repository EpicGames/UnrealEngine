// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigSkeletalMeshComponent.h"
#include "Sequencer/ControlRigLayerInstance.h" 
#include "SkeletalDebugRendering.h"
#include "ControlRig.h"
#include "AnimPreviewInstance.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSkeletalMeshComponent)

UControlRigSkeletalMeshComponent::UControlRigSkeletalMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DebugDrawSkeleton(false)
	, HierarchyInteractionBracket(0)
	, bRebuildDebugDrawSkeletonRequired(false)
	, bIsConstructionEventRunning(false)
{
	SetDisablePostProcessBlueprint(true);
}

void UControlRigSkeletalMeshComponent::InitAnim(bool bForceReinit)
{
	UAnimInstance* AnimInstance = GetAnimInstance();
	TObjectPtr<class UAnimPreviewInstance> LastPreviewInstance = PreviewInstance;

	// Super::InitAnim might trigger an evaluation, in which case the source animation instance must be set
	UControlRigLayerInstance* ControlRigInstance = Cast<UControlRigLayerInstance>(AnimInstance);
	if (ControlRigInstance)
	{
		ControlRigInstance->SetSourceAnimInstance(PreviewInstance);
	}
	
	// skip preview init entirely, just init the super class
	Super::InitAnim(bForceReinit);

	// The preview instance or anim instance might have been created in Super::InitAnim, in which case
	// the source animation instance must be set now.
	// we also must ensure that the anim instance is linked as InitAnim can reset the linked instances
	ControlRigInstance = Cast<UControlRigLayerInstance>(GetAnimInstance());
	if (ControlRigInstance)
	{
		const bool bHaveInstancesChanged = (AnimInstance != GetAnimInstance()) || (LastPreviewInstance != PreviewInstance);

		const USkeletalMeshComponent* MeshComponent = ControlRigInstance->GetOwningComponent();
		const bool bIsAnimInstanceLinked = PreviewInstance && MeshComponent && MeshComponent->GetLinkedAnimInstances().Contains(PreviewInstance);
		if (bHaveInstancesChanged || !bIsAnimInstanceLinked)
		{
			ControlRigInstance->SetSourceAnimInstance(PreviewInstance);
		}
	}

	bRebuildDebugDrawSkeletonRequired = true;
	RebuildDebugDrawSkeleton();
}

bool UControlRigSkeletalMeshComponent::IsPreviewOn() const
{
	return (PreviewInstance != nullptr);
}

void UControlRigSkeletalMeshComponent::SetCustomDefaultPose()
{
	ShowReferencePose(false);
}

void UControlRigSkeletalMeshComponent::RebuildDebugDrawSkeleton()
{
	if ((HierarchyInteractionBracket != 0) || bIsConstructionEventRunning || !bRebuildDebugDrawSkeletonRequired)
	{
		return;
	}
	
	UControlRigLayerInstance* ControlRigInstance = Cast<UControlRigLayerInstance>(GetAnimInstance());

	bool bHasValidControlRig = false;
	
	if (ControlRigInstance)
	{
		UControlRig* ControlRig = ControlRigInstance->GetFirstAvailableControlRig();
		if (ControlRig)
		{
			bHasValidControlRig = true;
			
			// we are trying to poke into running instances of Control Rigs
			// on the anim thread and query data, using a lock here to make sure
			// we don't get an inconsistent view of the rig at some
			// intermediate stage of evaluation, for example, during evaluate, we can have a call
			// to copy hierarchy, which empties the hierarchy for a short period of time.
			// if we did not have this lock and try to grab it, we could get an empty bone array
			UE::TScopeLock EvaluateLock(ControlRig->GetEvaluateMutex());
			
			// just copy it because this is not thread safe
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

			DebugDrawSkeleton.Empty();
			DebugDrawBones.Reset();
			DebugDrawBoneIndexInHierarchy.Reset();

			// create ref modifier
			FReferenceSkeletonModifier RefSkelModifier(DebugDrawSkeleton, nullptr);

			TMap<FName, int32> AddedBoneMap;
			TArray<FRigBoneElement*> BoneElements = Hierarchy->GetBones(true);
			for(FRigBoneElement* BoneElement : BoneElements)
			{
				AddedBoneMap.FindOrAdd(BoneElement->GetFName(), DebugDrawBones.Num());
				DebugDrawBones.Add(DebugDrawBones.Num());
				DebugDrawBoneIndexInHierarchy.Add(BoneElement->GetIndex());
			}

			for(FRigBoneElement* BoneElement : BoneElements)
			{
				const int32 Index = BoneElement->GetIndex();

				FName ParentName = NAME_None;

				// find the first parent that is a bone
				Hierarchy->Traverse(BoneElement, false, [&ParentName, BoneElement](FRigBaseElement* InElement, bool& bContinue)
				{
					bContinue = true;
					
					if(InElement == BoneElement)
					{
						return;
					}

					if(FRigBoneElement* InBoneElement = Cast<FRigBoneElement>(InElement))
					{
						if(ParentName.IsNone())
						{
							ParentName = InBoneElement->GetFName();
						}
						bContinue = false;
					}
				});

				int32 ParentIndex = INDEX_NONE; 
				if(!ParentName.IsNone())
				{
					ParentIndex = AddedBoneMap.FindChecked(ParentName);
				}
					
				FMeshBoneInfo NewMeshBoneInfo;
				NewMeshBoneInfo.Name = BoneElement->GetFName();
				NewMeshBoneInfo.ParentIndex = ParentIndex; 
				// give ref pose here
				RefSkelModifier.Add(NewMeshBoneInfo, Hierarchy->GetInitialGlobalTransform(Index), true);
			}
		}
	}
	
	if (!bHasValidControlRig)
	{
		DebugDrawSkeleton.Empty();
		DebugDrawBones.Reset();
		DebugDrawBoneIndexInHierarchy.Reset();
		return;
	}
	
	bRebuildDebugDrawSkeletonRequired = false;
}

FTransform UControlRigSkeletalMeshComponent::GetDrawTransform(int32 BoneIndex) const
{
	UControlRigLayerInstance* ControlRigInstance = Cast<UControlRigLayerInstance>(GetAnimInstance());

	if (ControlRigInstance)
	{
		UControlRig* ControlRig = ControlRigInstance->GetFirstAvailableControlRig();
		if (ControlRig && DebugDrawBoneIndexInHierarchy.IsValidIndex(BoneIndex))
		{
			// just copy it because this is not thread safe
			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
			return Hierarchy->GetGlobalTransform(DebugDrawBoneIndexInHierarchy[BoneIndex]);
		}
	}

	return FTransform::Identity;
}


void UControlRigSkeletalMeshComponent::EnablePreview(bool bEnable, UAnimationAsset* PreviewAsset)
{
	if (PreviewInstance)
	{
		PreviewInstance->SetAnimationAsset(PreviewAsset);
	}
}

void UControlRigSkeletalMeshComponent::SetControlRigBeingDebugged(UControlRig* InControlRig)
{
	if(ControlRigBeingDebuggedPtr.Get() == InControlRig)
	{
		return;
	}

	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(UControlRig* ControlRigBeingDebugged = ControlRigBeingDebuggedPtr.Get())
		{
			if(!URigVMHost::IsGarbageOrDestroyed(ControlRigBeingDebugged))
			{
				ControlRigBeingDebugged->GetHierarchy()->OnModified().RemoveAll(this);
			}

#if WITH_EDITOR
			ControlRigBeingDebugged->OnPreConstructionForUI_AnyThread().RemoveAll(this);
			ControlRigBeingDebugged->OnPostConstruction_AnyThread().RemoveAll(this);
#endif
		}
	}

	ControlRigBeingDebuggedPtr.Reset();
	bRebuildDebugDrawSkeletonRequired = true;
	
	if(InControlRig)
	{
		ControlRigBeingDebuggedPtr = InControlRig;

		InControlRig->GetHierarchy()->OnModified().RemoveAll(this);
 		InControlRig->GetHierarchy()->OnModified().AddUObject(this, &UControlRigSkeletalMeshComponent::OnHierarchyModified_AnyThread);

#if WITH_EDITOR
		InControlRig->OnPreConstructionForUI_AnyThread().RemoveAll(this);
		InControlRig->OnPreConstructionForUI_AnyThread().AddUObject(this, &UControlRigSkeletalMeshComponent::OnPreConstruction_AnyThread);
		
		InControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
		InControlRig->OnPostConstruction_AnyThread().AddUObject(this, &UControlRigSkeletalMeshComponent::OnPostConstruction_AnyThread);
#endif
	}

	RebuildDebugDrawSkeleton();
}

void UControlRigSkeletalMeshComponent::OnHierarchyModified(ERigHierarchyNotification InNotif,
    URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementAdded:
		case ERigHierarchyNotification::ElementRemoved:
		case ERigHierarchyNotification::ElementRenamed:
		case ERigHierarchyNotification::ElementReordered:
		case ERigHierarchyNotification::ParentChanged:
		case ERigHierarchyNotification::HierarchyReset:
		{
			bRebuildDebugDrawSkeletonRequired = true;
			if(HierarchyInteractionBracket == 0)
			{
				RebuildDebugDrawSkeleton();
			}
			break;
		}
		case ERigHierarchyNotification::InteractionBracketOpened:
		{
			HierarchyInteractionBracket++;
			break;
		}
		case ERigHierarchyNotification::InteractionBracketClosed:
		{
			HierarchyInteractionBracket = FMath::Max(HierarchyInteractionBracket - 1, 0);
			if(HierarchyInteractionBracket == 0)
			{
				RebuildDebugDrawSkeleton();
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

void UControlRigSkeletalMeshComponent::OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif,
    URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if (bIsConstructionEventRunning)
	{
		return;
	}
	
	FRigElementKey Key;
	if(InSubject.Element)
	{
		Key = InSubject.Element->GetKey();
	}

	TWeakObjectPtr<URigHierarchy> WeakHierarchy = InHierarchy;
	
	auto Task = [this, InNotif, WeakHierarchy, Key]()
	{
		if(!WeakHierarchy.IsValid())
		{
			return;
		}
		if (const FRigBaseElement* Element = WeakHierarchy.Get()->Find(Key))
		{
			OnHierarchyModified(InNotif, WeakHierarchy.Get(), Element);
		}
	};

	if (IsInGameThread())
	{
		Task();
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [Task]()
		{
			Task();		
		});
	}
}

void UControlRigSkeletalMeshComponent::OnPreConstruction_AnyThread(UControlRig* InControlRig, const FName& InEventName)
{
	bIsConstructionEventRunning = true;
}

void UControlRigSkeletalMeshComponent::OnPostConstruction_AnyThread(UControlRig* InControlRig, const FName& InEventName)
{
	bIsConstructionEventRunning = false;

	if (IsInGameThread())
	{
		RebuildDebugDrawSkeleton();
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]()
		{
			RebuildDebugDrawSkeleton();
		});
	}
}