// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRig.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "ControlRigSkeletalMeshComponent.generated.h"

UCLASS(MinimalAPI)
class UControlRigSkeletalMeshComponent : public UDebugSkelMeshComponent
{
	GENERATED_UCLASS_BODY()

	// USkeletalMeshComponent interface
	virtual void InitAnim(bool bForceReinit) override;

	// BEGIN UDebugSkeletalMeshComponent interface
	virtual void SetCustomDefaultPose() override;
	virtual const FReferenceSkeleton& GetReferenceSkeleton() const override
	{
		return DebugDrawSkeleton;
	}

	virtual const TArray<FBoneIndexType>& GetDrawBoneIndices() const override
	{
		return DebugDrawBones;
	}

	virtual FTransform GetDrawTransform(int32 BoneIndex) const override;

	virtual int32 GetNumDrawTransform() const
	{
		return DebugDrawBones.Num();
	}

	virtual void EnablePreview(bool bEnable, class UAnimationAsset * PreviewAsset) override;

	// return true if preview animation is active 
	virtual bool IsPreviewOn() const override;
	// END UDebugSkeletalMeshComponent interface

public:

	/*
	 * React to a control rig being debugged
	 */
	void SetControlRigBeingDebugged(UControlRig* InControlRig);
	
	/*
	 *	Rebuild debug draw skeleton 
	 */
	void RebuildDebugDrawSkeleton();

	void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);

	void OnPreConstruction_AnyThread(UControlRig* InControlRig, const FName& InEventName);
	void OnPostConstruction_AnyThread(UControlRig* InControlRig, const FName& InEventName);

private:
	FReferenceSkeleton DebugDrawSkeleton;
	TArray<FBoneIndexType> DebugDrawBones;
	TArray<int32> DebugDrawBoneIndexInHierarchy;
	TWeakObjectPtr<UControlRig> ControlRigBeingDebuggedPtr;
	int32 HierarchyInteractionBracket;
	bool bRebuildDebugDrawSkeletonRequired;
	bool bIsConstructionEventRunning;
};
