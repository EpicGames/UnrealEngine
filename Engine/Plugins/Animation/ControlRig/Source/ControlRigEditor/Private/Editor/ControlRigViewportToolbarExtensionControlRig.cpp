// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigViewportToolbarExtensionControlRig.h"

#include "AnimationEditorViewportClient.h"
#include "BlueprintEditor.h"
#include "ControlRigContextMenuContext.h"
#include "ControlRigEditorStyle.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "Editor/ControlRigEditor.h"
#include "Editor/ControlRigEditorCommands.h"
#include "Editor/ControlRigNewEditor.h"
#include "Editor/Persona/Private/AnimViewportShowCommands.h"
#include "Editor/Persona/Private/ViewportToolbar/AnimationEditorMakeShowBonesMenuSection.h"
#include "Settings/ControlRigSettings.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UObject/ScriptInterface.h"
#include "UObject/WeakInterfacePtr.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include <limits>

#define LOCTEXT_NAMESPACE "ControlRigViewportToolbarExtensionControlRig"

namespace UE::ControlRigEditor
{
	/** Creates the conrol rig entry */
	namespace Private
	{
		/** Creates a widget to edit the axis scale in editor */
		TSharedRef<SWidget> MakeAxisScaleWidget()
		{
			const FText Tooltip = LOCTEXT("ControlRigAxesScaleToolTip", "Scale of axes drawn for selected rig elements");

			return 
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
					.WidthOverride(100.0f)
					[
						SNew(SNumericEntryBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.AllowSpin(true)
						.MinSliderValue(0.0f)
						.MaxSliderValue(100.0f)
						.Value_Lambda([]()
							{										
								return GetDefault<UControlRigEditorSettings>()->AxisScale;
							})
						.OnValueChanged_Lambda([](float Value)
						{
							UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
							Settings->AxisScale = FMath::Max(0.f, Value);
							Settings->SaveConfig();
						})
						.ToolTipText(Tooltip)
					]
				];
		}
		
		/** Creates a widget to edit hovered alpha value in editor */
		TSharedRef<SWidget> MakeHoveredAlphaWidget()
		{
			FProperty* HoveredProperty =
				UControlRigEditModeSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditModeSettings, HoveredColor));
			
			auto UpdateAlpha = [HoveredProperty](const float Value, EPropertyChangeType::Type ChangeType)
			{
				if (UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>())
				{
					Settings->HoveredColor.A = FMath::Clamp(Value, 0.f, 1.f);
					FPropertyChangedEvent Event(HoveredProperty, ChangeType);
					Settings->OnSettingChanged().Broadcast(Settings, Event);
					if (ChangeType == EPropertyChangeType::ValueSet)
					{
						Settings->SaveConfig();
					}
				}
			};
			
			return 
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
					.WidthOverride(100.0f)
					[
						SNew(SNumericEntryBox<float>)
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.AllowSpin(true)
						.MinSliderValue(0.0f)
						.MaxSliderValue(1.0f)
						.Value_Lambda([]()
						{
							const UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();		
							return Settings ? Settings->HoveredColor.A : 0.f;
						})
						.OnValueChanged_Lambda([UpdateAlpha](float Value)
						{
							UpdateAlpha(Value, EPropertyChangeType::Interactive);
						})
						.OnValueCommitted_Lambda([UpdateAlpha](float Value, ETextCommit::Type)
						{
							UpdateAlpha(Value, EPropertyChangeType::ValueSet);
						})
					]
				];
		}

		/** Adds the control rig sub menu to the toolbar */
		FToolMenuEntry CreateControlRigEntry(const IControlRigEditorAssetInterface& Asset)
		{
			const TWeakInterfacePtr<const IControlRigEditorAssetInterface> WeakAsset = &Asset;

			FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
				"ControlRig",
				LOCTEXT("ControlRigSubMenuLabel", "Control Rig"),
				LOCTEXT("ControlRigSubMenuTooltip", "Control Rig related settings"),
				FNewToolMenuDelegate::CreateLambda(
					[WeakAsset](UToolMenu* SubMenu) -> void
					{
						const FControlRigEditorCommands& ControlRigEditorCommands = FControlRigEditorCommands::Get();

						// Control Rig section
						{
							FToolMenuSection& ControlRigSection =
								SubMenu->FindOrAddSection(
									"ControlRig",
									LOCTEXT("ControlRigLabel", "Control Rig Display"));

							ControlRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleControlVisibility);
							ControlRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleControlsAsOverlay);
							ControlRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleDrawNulls);
							ControlRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleDrawSockets);
							ControlRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleDrawAxesOnSelection);

							ControlRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleDrawAxesOnSelection);

							ControlRigSection.AddSeparator("SliderSection");

							ControlRigSection.AddEntry(FToolMenuEntry::InitWidget(
								"AxesScale",
								MakeAxisScaleWidget(),
								LOCTEXT("ControlRigAxesScale", "Axes Scale")));

							constexpr bool bNoIndent = false, bSearchable = true, bNoPadding = false;
							ControlRigSection.AddEntry(FToolMenuEntry::InitWidget(
								"Hovered Alpha",
								MakeHoveredAlphaWidget(),
								LOCTEXT("ControlRigHoveredAlpha", "Hovered Alpha"),
								bNoIndent, bSearchable, bNoPadding,
								LOCTEXT("ControlRigEditorHoveredAlphaToolTip", "Alpha value for hovered color.")));
						}

						// Entries for Modular Rigs
						if (WeakAsset.IsValid() && WeakAsset.Get()->IsModularRig())
						{
							FToolMenuSection& ModularRigSection = SubMenu->FindOrAddSection(
								"ModularRig",
								LOCTEXT("ModularRig_Label", "Modular Rig Display"));

								ModularRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleSchematicViewportVisibility);
								ModularRigSection.AddMenuEntry(ControlRigEditorCommands.ToggleSchematicViewportShowEmptySocketsOnly);
						}

						// Show bones section
						{
							using namespace UE::Persona::ViewportToolbar;

							const EShowBonesMenuEntryFlags ShowFlags =
								EShowBonesMenuEntryFlags::DrawAll |
								EShowBonesMenuEntryFlags::DrawNone;

							MakeShowBonesMenuSection(SubMenu, ShowFlags);
						}
					})
				);

			Entry.InsertPosition = FToolMenuInsert("Show", EToolMenuInsertType::After);
			Entry.Icon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.Editor.TabIcon");
			Entry.ToolBarData.LabelOverride = FText::GetEmpty();

			// Set resize prio to highest as other entries in other sections dominate otherwise
			Entry.ToolBarData.ResizeParams.ClippingPriority = std::numeric_limits<int32>::max(); 

			return Entry;
		}
	}
	//~ namespace Private

	void PopulateControlRigViewportToolbarControlRigSubmenu()
	{
		FAnimViewportShowCommands::Register();

		constexpr const TCHAR* MenuName = TEXT("AnimationEditor.ViewportToolbar");

		TWeakObjectPtr<UToolMenu> Toolbar = UToolMenus::Get()->ExtendMenu(MenuName);
		FToolMenuSection* RightSection = Toolbar.IsValid() ? Toolbar->FindSection("Right") : nullptr;
		if (!Toolbar.IsValid() ||
			!RightSection)
		{
			return;
		}

		RightSection->AddDynamicEntry(
			"ControlRig",
			FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
				{		
					// Only for control rig assets
					const UControlRigContextMenuContext* Context = Section.FindContext<UControlRigContextMenuContext>();
					const TScriptInterface<IControlRigEditorAssetInterface> ControlRigAsset = Context ? Context->GetControlRigAssetInterface() : nullptr;
					if (Context &&
						ControlRigAsset)
					{
						Section.AddEntry(Private::CreateControlRigEntry(*ControlRigAsset));
					}
				})
		);	
	}
}

#undef LOCTEXT_NAMESPACE
