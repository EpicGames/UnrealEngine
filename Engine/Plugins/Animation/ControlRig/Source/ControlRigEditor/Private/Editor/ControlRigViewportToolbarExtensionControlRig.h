// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FName;

namespace UE::ControlRigEditor
{
	/** Populates the viewport toolbar submenu, unregisters when all instances are destroyed */
	void PopulateControlRigViewportToolbarControlRigSubmenu();
}
