// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigViewportToolbarExtensions.h"

#include "ControlRigEditorCommands.h"
#include "DetailCategoryBuilder.h"
#include "EditorViewportCommands.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "ToolMenus.h"
#include "Tools/MotionTrailOptions.h"
#include "Tools/MotionTrailMenuHelpers.h"
#include "LevelEditor.h"
#include "ILevelEditor.h"
#include "InteractiveToolManager.h"
#include "EditorModeManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"
#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "ControlRigViewportToolbar"

namespace UE::ControlRig::Private
{

const FName ControlRigOwnerName = "ControlRigViewportToolbar";

TSharedRef<SWidget> MakeHoveredAlphaWidget()
{
	FProperty* HoveredProperty =
		UControlRigEditModeSettings::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UControlRigEditModeSettings, HoveredColor));
		
	auto UpdateAlpha = [HoveredProperty](const float Value, EPropertyChangeType::Type ChangeType)
	{
		if (UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>())
		{
			Settings->HoveredColor.A = FMath::Clamp(Value, 0.f, 1.f);
			FPropertyChangedEvent Event(HoveredProperty, ChangeType);
			Settings->OnSettingChanged().Broadcast(Settings, Event);
			if (ChangeType == EPropertyChangeType::ValueSet)
			{
				Settings->SaveConfig();
			}
		}
	};
	
	return 
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.AllowSpin(true)
				.MinSliderValue(0.0f)
				.MaxSliderValue(1.0f)
				.Value_Lambda([]()
				{
					const UControlRigEditModeSettings* Settings = GetMutableDefault<UControlRigEditModeSettings>();		
					return Settings ? Settings->HoveredColor.A : 0.f;
				})
				.OnValueChanged_Lambda([UpdateAlpha](float Value)
				{
					UpdateAlpha(Value, EPropertyChangeType::Interactive);
				})
				.OnValueCommitted_Lambda([UpdateAlpha](float Value, ETextCommit::Type)
				{
					UpdateAlpha(Value, EPropertyChangeType::ValueSet);
				})
			]
		];
}
	
} // namespace UE::ControlRig::Private


void UE::ControlRig::PopulateControlRigViewportToolbarTransformSubmenu(const FName InMenuName)
{
	FToolMenuOwnerScoped ScopeOwner(UE::ControlRig::Private::ControlRigOwnerName);

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(InMenuName);

	{
		FToolMenuSection& GizmoSection = Menu->FindOrAddSection("Gizmo");

		UControlRigEditModeSettings* const ViewportSettings = GetMutableDefault<UControlRigEditModeSettings>();

		// Add "Local Transforms in Each Local Space" checkbox.
		{
			FUIAction Action;

			Action.ExecuteAction = FExecuteAction::CreateLambda(
				[ViewportSettings]() -> void
				{
					ViewportSettings->bLocalTransformsInEachLocalSpace = !ViewportSettings->bLocalTransformsInEachLocalSpace;
					ViewportSettings->PostEditChange();
				}
			);
			Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
				[ViewportSettings]() -> ECheckBoxState
				{
					return ViewportSettings->bLocalTransformsInEachLocalSpace ? ECheckBoxState::Checked
																			  : ECheckBoxState::Unchecked;
				}
			);

			FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
				"LocalTransformsInEachLocalSpace",
				LOCTEXT("LocalTransformsInEachLocalSpaceLabel", "Local Transforms in Each Local Space"),
				LOCTEXT(
					"LocalTransformsInEachLocalSpaceTooltip", "When multiple objects are selected, whether or not to transform each invidual object along its own local transform space."
				),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.LocalTransformsInEachLocalSpace"),
				Action,
				EUserInterfaceActionType::ToggleButton
			);
			// We want to appear early in the section.
			Entry.InsertPosition.Position = EToolMenuInsertType::First;
			GizmoSection.AddEntry(Entry);
		}

		// Add "Restore Coordinate Space on Switch" checkbox.
		{
			FUIAction Action;

			Action.ExecuteAction = FExecuteAction::CreateLambda(
				[ViewportSettings]() -> void
				{
					ViewportSettings->bCoordSystemPerWidgetMode = !ViewportSettings->bCoordSystemPerWidgetMode;
					ViewportSettings->PostEditChange();
				}
			);
			Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
				[ViewportSettings]() -> ECheckBoxState
				{
					return ViewportSettings->bCoordSystemPerWidgetMode ? ECheckBoxState::Checked
																	   : ECheckBoxState::Unchecked;
				}
			);

			FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
				"RestoreCoordinateSpaceOnSwitch",
				LOCTEXT("RestoreCoordinateSpaceOnSwitchLabel", "Restore Coordinate Space on Switch"),

				LOCTEXT(
					"RestoreCoordinateSpaceOnSwitchTooltip",
					"Whether to restore the coordinate space when changing Widget Modes in the Viewport."
				),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.RestoreCoordinateSpaceOnSwitch"),
				Action,
				EUserInterfaceActionType::ToggleButton
			);
			// We want to appear early in the section.
			Entry.InsertPosition.Position = EToolMenuInsertType::First;
			GizmoSection.AddEntry(Entry);
		}
	}
	{

		FToolMenuSection& PreviewToolsSection  = Menu->FindOrAddSection("PreviewTools", LOCTEXT("PreviewToolsLabel", "Preview Tools"));
		{
			// Add "Temporary Pivot" checkbox.
			{
				FUIAction Action;

				Action.ExecuteAction = FExecuteAction::CreateLambda(
					[]() -> void
					{
						if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
						{
							TSharedPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance().Pin();
							if (LevelEditorPtr.IsValid())
							{
								FString ActiveToolName = LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveToolName(EToolSide::Left);
								if (ActiveToolName == TEXT("SequencerPivotTool"))
								{
									LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->DeactivateTool(EToolSide::Left, EToolShutdownType::Completed);
								}
								else
								{
									LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->SelectActiveToolType(EToolSide::Left, TEXT("SequencerPivotTool"));
									LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->ActivateTool(EToolSide::Left);
								}
							}
						}
					}
				);
				Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
					[]() -> ECheckBoxState
					{
						if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
						{
							TSharedPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule->GetLevelEditorInstance().Pin();
							if (LevelEditorPtr.IsValid())
							{
								FString ActiveToolName = LevelEditorPtr->GetEditorModeManager().GetInteractiveToolsContext()->ToolManager->GetActiveToolName(EToolSide::Left);
								if (ActiveToolName == TEXT("SequencerPivotTool"))
								{
									return ECheckBoxState::Checked;
								}
							}
						}
						return  ECheckBoxState::Unchecked;
					}
				);

				FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
					"TemporaryPivot",
					LOCTEXT("TemporaryPivotLabel", "Temporary Pivot"),
					LOCTEXT(
						"TemporaryPivotTooltip",
						"Toggle Temporary Pivot Tool"
					),
					FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.TemporaryPivot")),
					Action,
					EUserInterfaceActionType::ToggleButton
				);
				Entry.SetShowInToolbarTopLevel(true);
				PreviewToolsSection.AddEntry(Entry);
			}
		}

		const ETrailCategory ControlRigCategory = ETrailCategory::Transform | ETrailCategory::ControlRig;
		FNewToolMenuDelegate MakeMenuDelegate = FNewToolMenuDelegate::CreateLambda(
			[ControlRigCategory](UToolMenu* Submenu)
			{
				UE::Sequencer::MotionTrailMenu::PopulateMotionTrailMenu(Submenu, ControlRigCategory);
			}
		);

		FToolUIAction CheckboxMenuAction = UE::Sequencer::MotionTrailMenu::MakeMotionTrailToggleAction(ControlRigCategory);

		FToolMenuEntry MotionPathsSubmenu = FToolMenuEntry::InitSubMenu(
			"ControlRigTrails",
			LOCTEXT("ControlRigTrailsLabel", "Motion Paths"),
			LOCTEXT("ControlRigTrailsTooltip", "Toggle control rig motion trails.\nHotkeys:\nUse SHIFT to add selected items to pin list\nUse CTRL to reset pin list to just the selected item\nUse ALT to remove selected item from pin list"),
			MakeMenuDelegate,
			CheckboxMenuAction,
			EUserInterfaceActionType::ToggleButton,
			false,
			FSlateIcon(TEXT("ControlRigEditorStyle"), TEXT("ControlRig.EditableMotionTrails"))
		);
		MotionPathsSubmenu.SetShowInToolbarTopLevel(true);

		PreviewToolsSection.AddEntry(MotionPathsSubmenu);
	}
	{
		FToolMenuSection& SelectionSection = Menu->FindOrAddSection("Selection");

		// Add "Select Only Control Rig Controls" entry.
		{
			UControlRigEditModeSettings* const Settings = GetMutableDefault<UControlRigEditModeSettings>();

			FUIAction Action;
			Action.ExecuteAction = FExecuteAction::CreateLambda(
				[Settings]() -> void
				{
					Settings->bOnlySelectRigControls = !Settings->bOnlySelectRigControls;
					Settings->PostEditChange();
				}
			);
			Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
				[Settings]() -> ECheckBoxState
				{
					return Settings->bOnlySelectRigControls ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
			);

			FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
				"OnlySelectRigControls",
				LOCTEXT("OnlySelectRigControlsLabel", "Select Only Control Rig Controls"),
				LOCTEXT("OnlySelectRigControlsTooltip", "Whether or not only Rig Controls can be selected."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.OnlySelectRigControls"),
				Action,
				EUserInterfaceActionType::ToggleButton
			);
			// We want to appear late in the section.
			Entry.InsertPosition.Position = EToolMenuInsertType::Last;
			Entry.SetShowInToolbarTopLevel(true);
			SelectionSection.AddEntry(Entry);
		}
	}
}

void CreateAxisOnSelectionMenu(UToolMenu* AnimationShowFlagsSubmenu, FToolMenuSection& UnnamedSection, UControlRigEditModeSettings* Settings)
{
	UUnrealEdViewportToolbarContext* Context = AnimationShowFlagsSubmenu->FindContext<UUnrealEdViewportToolbarContext>();

	FNewToolMenuDelegate AxisOnSelectionMenuDelegate = FNewToolMenuDelegate::CreateLambda(
		[Settings](UToolMenu* Submenu)
		{
			FToolMenuSection& AxisOnSelection =
				Submenu->FindOrAddSection("AxisOnSelection", LOCTEXT("AxisOnSelectionLabel", "Axis On Selection"));

			const FName DoubleProperty("AxisScale");
			const FName Label("Axis Scale");
			FToolMenuEntry NumericEntry = UE::Sequencer::MotionTrailMenu::CreateProperty<float>(Settings, DoubleProperty, Label);

			AxisOnSelection.AddEntry(NumericEntry);
		}
	);

	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
		"AxisOnSelection",
		LOCTEXT("AxisOnSelectionLabel", "Axis On Selection"),
		LOCTEXT("AxisOnSelectionTooltip", "Should we show axes for the selected elements"),
		AxisOnSelectionMenuDelegate,
		FToolUIAction(
			FToolMenuExecuteAction::CreateLambda([Settings,Context](const FToolMenuContext& InContext)
			{
				Settings->bDisplayAxesOnSelection = !Settings->bDisplayAxesOnSelection;
				Settings->SaveConfig();
				UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
				if (Context)
				{
					Context->RefreshViewport();
				}
			}),
			FToolMenuGetActionCheckState::CreateLambda([Settings](const FToolMenuContext& InContext)
			{
				return Settings->bDisplayAxesOnSelection ? ECheckBoxState::Checked
								: ECheckBoxState::Unchecked;
			})
		),
		EUserInterfaceActionType::ToggleButton
	);

	UnnamedSection.AddEntry(Entry);
	
}

void UE::ControlRig::PopulateControlRigViewportToolbarShowSubmenu(const FName InMenuName)
{
	FToolMenuOwnerScoped ScopeOwner(UE::ControlRig::Private::ControlRigOwnerName);

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(InMenuName);
	FToolMenuSection& AllShowFlagsSection = Menu->FindOrAddSection("AllShowFlags");

	FToolMenuEntry AnimationSubmenu = FToolMenuEntry::InitSubMenu(
		"Animation",
		LOCTEXT("AnimationLabel", "Animation"),
		LOCTEXT("AnimationTooltip", "Animation-related show flags"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* AnimationShowFlagsSubmenu)
			{
				FToolMenuSection& UnnamedSection = AnimationShowFlagsSubmenu->FindOrAddSection(NAME_None);
				
				UUnrealEdViewportToolbarContext* Context = AnimationShowFlagsSubmenu->FindContext<UUnrealEdViewportToolbarContext>();
				if (UControlRigEditModeSettings* const Settings = GetMutableDefault<UControlRigEditModeSettings>())
				{
					{
						CreateAxisOnSelectionMenu(AnimationShowFlagsSubmenu, UnnamedSection, Settings);
					}
					{
						FUIAction Action;
						Action.ExecuteAction = FExecuteAction::CreateLambda(
							[Settings, Context]() -> void
							{
								Settings->bDisplayHierarchy = !Settings->bDisplayHierarchy;
								Settings->SaveConfig();
								UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
								if (Context)
								{
									Context->RefreshViewport();
								}
							}
						);
						Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
							[Settings]() -> ECheckBoxState
							{
								return Settings->bDisplayHierarchy ? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						);

						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"Hierarchy",
							LOCTEXT("HierarchyLabel", "Hierarchy"),
							LOCTEXT("HierarchyTooltip", "Whether to show all bones in the hierarchy"),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
						UnnamedSection.AddEntry(Entry);
					}
					{
						FUIAction Action;
						Action.ExecuteAction = FExecuteAction::CreateLambda(
							[Settings, Context]() -> void
							{
								Settings->bShowControlsAsOverlay = !Settings->bShowControlsAsOverlay;
								Settings->SaveConfig();
								UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
								if (Context)
								{
									Context->RefreshViewport();
								}
							}
						);
						Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
							[Settings]() -> ECheckBoxState
							{
								return Settings->bShowControlsAsOverlay ? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						);

						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"ControlsAsOverlay",
							LOCTEXT("ControlsAsOverlayLabel", "Controls As Overlay"),
							LOCTEXT("ControlsAsOverlaylTooltip", "Whether to show controls as overlay"),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
						UnnamedSection.AddEntry(Entry);
					}
					{
						FUIAction Action;
						Action.ExecuteAction = FExecuteAction::CreateLambda(
							[Settings, Context]() -> void
							{
								Settings->bHideControlShapes = !Settings->bHideControlShapes;
								Settings->SaveConfig();
								UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
								if (Context)
								{
									Context->RefreshViewport();
								}
							}
						);
						Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
							[Settings]() -> ECheckBoxState
							{
								return Settings->bHideControlShapes == false ? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						);

						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"ControlShapes",
							LOCTEXT("ControlShapesLabel", "Control Shapes"),
							LOCTEXT("ControlShapesTooltip", "Should we always hide control shapes in viewport"),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
						UnnamedSection.AddEntry(Entry);
					}
					{
						FUIAction Action;
						Action.ExecuteAction = FExecuteAction::CreateLambda(
							[Settings, Context]() -> void
							{
								Settings->bDisplayNulls = !Settings->bDisplayNulls;
								Settings->SaveConfig();
								UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
								if (Context)
								{
									Context->RefreshViewport();
								}
							}
						);
						Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
							[Settings]() -> ECheckBoxState
							{
								return Settings->bDisplayNulls ? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						);

						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"Nulls",
							LOCTEXT("NullsLabel", "Nulls"),
							LOCTEXT("NullTooltip", "Whether to show all nulls in the hierarchy"),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
						UnnamedSection.AddEntry(Entry);
					}
					{
						FUIAction Action;
						Action.ExecuteAction = FExecuteAction::CreateLambda(
							[Settings, Context]() -> void
							{
								Settings->bShowAllProxyControls = !Settings->bShowAllProxyControls;
								Settings->SaveConfig();
								UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
								if (Context)
								{
									Context->RefreshViewport();
								}
							}
						);
						Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
							[Settings]() -> ECheckBoxState
							{
								return Settings->bShowAllProxyControls ? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						);

						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"ProxyControls",
							LOCTEXT("ProxyControlsLabel", "Proxy Controls"),
							LOCTEXT("ProxyControlsTooltip", "Whether to show Proxy Controls"),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
						UnnamedSection.AddEntry(Entry);
					}
					{
						FUIAction Action;
						Action.ExecuteAction = FExecuteAction::CreateLambda(
							[Settings, Context]() -> void
							{
								Settings->bDisplaySockets = !Settings->bDisplaySockets;
								Settings->SaveConfig();
								UControlRigEditModeSettings::OnSettingsChange.Broadcast(Settings);
								if (Context)
								{
									Context->RefreshViewport();
								}
							}
						);
						Action.GetActionCheckState = FGetActionCheckState::CreateLambda(
							[Settings]() -> ECheckBoxState
							{
								return Settings->bDisplaySockets ? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						);

						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"Sockets",
							LOCTEXT("SocketsLabel", "Sockets"),
							LOCTEXT("SocketsTooltip", "Whether to show Sockets"),
							FSlateIcon(),
							Action,
							EUserInterfaceActionType::ToggleButton
						);
						UnnamedSection.AddEntry(Entry);
					}
					
					{
						UnnamedSection.AddSeparator("SliderSection");
						
						constexpr bool bNoIndent = false, bSearchable = true, bNoPadding = false;
						UnnamedSection.AddEntry(FToolMenuEntry::InitWidget(
						"Hovered Alpha",
						Private::MakeHoveredAlphaWidget(),
						LOCTEXT("ControlRigHoveredAlpha", "Hovered Alpha"),
						bNoIndent, bSearchable, bNoPadding,
						LOCTEXT("ControlRigHoveredAlphaToolTip", "Alpha value for hovered color.")));
					}
				}
			}
		),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Animation_16x")
	);
	// Show this in the top-level to highlight it for Animation Mode users.
	AnimationSubmenu.SetShowInToolbarTopLevel(true);
	AnimationSubmenu.InsertPosition.Position = EToolMenuInsertType::First;
	AnimationSubmenu.ToolBarData.ResizeParams.AllowClipping = false;
	AllShowFlagsSection.AddEntry(AnimationSubmenu);
}

void UE::ControlRig::RemoveControlRigViewportToolbarExtensions()
{
	UToolMenus::Get()->UnregisterOwnerByName(UE::ControlRig::Private::ControlRigOwnerName);
}

#undef LOCTEXT_NAMESPACE
