// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"
#include "UObject/NameTypes.h"

class FUICommandList;

namespace UE::ControlRig
{

void PopulateControlRigViewportToolbarTransformSubmenu(const FName InMenuName);
void PopulateControlRigViewportToolbarShowSubmenu(const FName InMenuName);

void RemoveControlRigViewportToolbarExtensions();

} // namespace UE::ControlRig
