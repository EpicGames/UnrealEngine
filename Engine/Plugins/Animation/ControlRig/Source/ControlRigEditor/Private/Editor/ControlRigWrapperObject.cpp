// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ControlRigWrapperObject.h"

#if WITH_EDITOR
#include "ControlRigElementDetails.h"
#include "ControlRigModuleDetails.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigWrapperObject)

UClass* UControlRigWrapperObject::GetClassForStruct(UScriptStruct* InStruct, bool bCreateIfNeeded) const
{
	UClass* Class = Super::GetClassForStruct(InStruct, bCreateIfNeeded);
	if(Class == nullptr)
	{
		return nullptr;
	}
	
	const FName WrapperClassName = Class->GetFName();

#if WITH_EDITOR
	if(InStruct->IsChildOf(FRigBaseElement::StaticStruct()))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		if (!PropertyEditorModule.GetClassNameToDetailLayoutNameMap().Contains(WrapperClassName))
		{
			if(InStruct == FRigBoneElement::StaticStruct())
			{
				PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigBoneElementDetails::MakeInstance));
			}
			else if(InStruct == FRigNullElement::StaticStruct())
			{
				PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigNullElementDetails::MakeInstance));
			}
			else if(InStruct == FRigControlElement::StaticStruct())
			{
				PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigControlElementDetails::MakeInstance));
			}
			else if(InStruct == FRigConnectorElement::StaticStruct())
			{
				PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigConnectorElementDetails::MakeInstance));
			}
			else if(InStruct == FRigSocketElement::StaticStruct())
			{
				PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigSocketElementDetails::MakeInstance));
			}
		}
	}
	if(InStruct->IsChildOf(FRigBaseComponent::StaticStruct()))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		if (!PropertyEditorModule.GetClassNameToDetailLayoutNameMap().Contains(WrapperClassName))
		{
			PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigBaseComponentDetails::MakeInstance));
		}
	}
#endif

	return Class;
}

void UControlRigWrapperObject::SetContent(const uint8* InStructMemory, const UStruct* InStruct)
{
	Super::SetContent(InStructMemory, InStruct);

	if(InStruct->IsChildOf((FRigBaseElement::StaticStruct())))
	{
		const FRigBaseElement* SourceElement = reinterpret_cast<const FRigBaseElement*>(InStructMemory);
		HierarchyKey = SourceElement->GetKey();
	}
	else if(InStruct->IsChildOf((FRigBaseComponent::StaticStruct())))
	{
		const FRigBaseComponent* SourceComponent = reinterpret_cast<const FRigBaseComponent*>(InStructMemory);
		HierarchyKey = SourceComponent->GetKey();
	}
}

void UControlRigWrapperObject::GetContent(uint8* OutStructMemory, const UStruct* InStruct) const
{
	Super::GetContent(OutStructMemory, InStruct);

	if(InStruct->IsChildOf((FRigBaseElement::StaticStruct())))
	{
		FRigBaseElement* TargetElement = reinterpret_cast<FRigBaseElement*>(OutStructMemory);
		TargetElement->Key = HierarchyKey.GetElement();
	}
	else if(InStruct->IsChildOf((FRigBaseComponent::StaticStruct())))
	{
		FRigBaseComponent* TargetComponent = reinterpret_cast<FRigBaseComponent*>(OutStructMemory);
		TargetComponent->Key = HierarchyKey.GetComponent();
	}
}
