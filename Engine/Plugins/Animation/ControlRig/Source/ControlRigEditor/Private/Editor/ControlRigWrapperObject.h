// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/RigVMDetailsViewWrapperObject.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ControlRigWrapperObject.generated.h"

#define UE_API CONTROLRIGEDITOR_API

UCLASS(MinimalAPI)
class UControlRigWrapperObject : public URigVMDetailsViewWrapperObject
{
public:
	GENERATED_BODY()

	UE_API virtual UClass* GetClassForStruct(UScriptStruct* InStruct, bool bCreateIfNeeded) const override;

	UE_API virtual void SetContent(const uint8* InStructMemory, const UStruct* InStruct) override;
	UE_API virtual void GetContent(uint8* OutStructMemory, const UStruct* InStruct) const override;;

	FRigHierarchyKey HierarchyKey;
};

#undef UE_API
