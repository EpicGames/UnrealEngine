// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigHierarchyElementDragDropOp.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<FModularRigHierarchyElementDragDropOp> FModularRigHierarchyElementDragDropOp::New(const TArray<FName>& InModuleNames)
{
	TSharedRef<FModularRigHierarchyElementDragDropOp> Operation = MakeShared<FModularRigHierarchyElementDragDropOp>();
	Operation->ModuleNames = InModuleNames;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FModularRigHierarchyElementDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
			.Text(FText::FromString(GetJoinedModuleNames()))
			//.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
		];
}

FString FModularRigHierarchyElementDragDropOp::GetJoinedModuleNames() const
{
	TArray<FString> ElementNameStrings;
	for (const FName& ModuleName: ModuleNames)
	{
		ElementNameStrings.Add(ModuleName.ToString());
	}
	return FString::Join(ElementNameStrings, TEXT(","));
}
