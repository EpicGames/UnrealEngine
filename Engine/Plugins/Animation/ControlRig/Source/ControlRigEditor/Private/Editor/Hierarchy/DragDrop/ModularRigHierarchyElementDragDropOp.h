// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/DragAndDrop.h"

class FModularRigHierarchyElementDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FModularRigHierarchyElementDragDropOp, FDragDropOperation)

	static TSharedRef<FModularRigHierarchyElementDragDropOp> New(const TArray<FName>& InModuleNames);

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	/** @return true if this drag operation contains dragged modules */
	bool HasModules() const
	{
		return ModuleNames.Num() > 0;
	}

	/** @return The module names from this drag operation */
	const TArray<FName>& GetModules() const
	{
		return ModuleNames;
	}

	FString GetJoinedModuleNames() const;

private:

	/** Data for the module names this item represents */
	TArray<FName> ModuleNames;
};
