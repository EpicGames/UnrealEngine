// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyElementDragDropOp.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<FRigHierarchyElementDragDropOp> FRigHierarchyElementDragDropOp::New(const TArray<FRigHierarchyKey>& InElements)
{
	TSharedRef<FRigHierarchyElementDragDropOp> Operation = MakeShared<FRigHierarchyElementDragDropOp>();
	Operation->Elements = InElements;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FRigHierarchyElementDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
			.Text(FText::FromString(GetJoinedElementNames()))
			//.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
		];
}

FString FRigHierarchyElementDragDropOp::GetJoinedElementNames() const
{
	TArray<FString> ElementNameStrings;
	for (const FRigHierarchyKey& Element: Elements)
	{
		ElementNameStrings.Add(Element.GetName());
	}
	return FString::Join(ElementNameStrings, TEXT(","));
}

bool FRigHierarchyElementDragDropOp::IsDraggingSingleConnector() const
{
	if(Elements.Num() == 1)
	{
		if(Elements[0].IsElement())
		{
			return Elements[0].GetElement().Type == ERigElementType::Connector;
		}
	}
	return false;
}

bool FRigHierarchyElementDragDropOp::IsDraggingSingleSocket() const
{
	if(Elements.Num() == 1)
	{
		if(Elements[0].IsElement())
		{
			return Elements[0].GetElement().Type == ERigElementType::Socket;
		}
	}
	return false;
}
