// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/GraphNodeDragDropOp.h"
#include "Rigs/RigHierarchyDefines.h"

class SRigHierarchyTagWidget;

class FRigHierarchyElementDragDropOp : public FGraphNodeDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRigHierarchyElementDragDropOp, FGraphNodeDragDropOp)

	static TSharedRef<FRigHierarchyElementDragDropOp> New(const TArray<FRigHierarchyKey>& InElements);

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	/** @return true if this drag operation contains property paths */
	bool HasElements() const
	{
		return Elements.Num() > 0;
	}

	/** @return The property paths from this drag operation */
	const TArray<FRigHierarchyKey>& GetElements() const
	{
		return Elements;
	}

	FString GetJoinedElementNames() const;

	bool IsDraggingSingleConnector() const;
	bool IsDraggingSingleSocket() const;

private:

	/** Data for the property paths this item represents */
	TArray<FRigHierarchyKey> Elements;
};
