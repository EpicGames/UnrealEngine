// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyTagDragDropOp.h"

#include "Editor/Hierarchy/Widgets/SRigHierarchyTagWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<FRigHierarchyTagDragDropOp> FRigHierarchyTagDragDropOp::New(TSharedPtr<SRigHierarchyTagWidget> InTagWidget)
{
	TSharedRef<FRigHierarchyTagDragDropOp> Operation = MakeShared<FRigHierarchyTagDragDropOp>();
	Operation->Text = InTagWidget->Text.Get();
	Operation->Identifier = InTagWidget->Identifier.Get();
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FRigHierarchyTagDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
			.Text(Text)
		];
}
