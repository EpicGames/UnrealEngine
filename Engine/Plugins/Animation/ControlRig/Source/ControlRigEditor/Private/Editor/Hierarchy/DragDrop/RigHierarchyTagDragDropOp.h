// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

class SRigHierarchyTagWidget;

class FRigHierarchyTagDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRigHierarchyTagDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FRigHierarchyTagDragDropOp> New(TSharedPtr<SRigHierarchyTagWidget> InTagWidget);

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	/** @return The identifier being dragged */
	const FString& GetIdentifier() const
	{
		return Identifier;
	}

private:

	FText Text;
	FString Identifier;
};
