// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigHierarchyTreeElement.h"

#include "Blueprint/WidgetBlueprintLibrary.h"
#include "ControlRigEditorStyle.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyViewModel.h"
#include "Editor/Hierarchy/Widgets/SModularRigHierarchyTreeView.h"
#include "Editor/ModularRigHierarchyConnectorWarning.h"
#include "Engine/Texture2D.h"
#include "ModularRig.h"
#include "ModularRigController.h"

TMap<FSoftObjectPath, TSharedPtr<FSlateBrush>> FModularRigHierarchyTreeElement::IconPathToBrush;

TArray<TStrongObjectPtr<UTexture2D>> FModularRigHierarchyTreeElement::Icons;

FModularRigHierarchyTreeElement::FModularRigHierarchyTreeElement(
	const TSharedRef<FModularRigHierarchyViewModel>& InModularRigHierarchyViewModel, 
	const FString& InKey, 
	const bool bInIsPrimary)
	: WeakModularRigHierarchyViewModel(InModularRigHierarchyViewModel)
	, Key(InKey)
	, bIsPrimary(bInIsPrimary)
{
	FString ModuleNameString;
	FString ConnectorNameString = Key;
	FRigHierarchyModulePath(ConnectorNameString).Split(&ModuleNameString, &ConnectorNameString);

	const TSharedPtr<IControlRigBaseEditor> Editor = InModularRigHierarchyViewModel->GetControlRigEditor();
	const UModularRig* ModularRig = InModularRigHierarchyViewModel->GetModularRig();
	if (!Editor.IsValid() ||
		!ModularRig)
	{
		return;
	}

	if (bIsPrimary)
	{
		ModuleName = *Key;

		if (const FRigModuleInstance* Module = ModularRig->FindModule(ModuleName))
		{
			if (const UControlRig* Rig = Module->GetRig())
			{
				if (const FRigModuleConnector* PrimaryConnector = Rig->GetRigModuleSettings().FindPrimaryConnector())
				{
					ConnectorName = PrimaryConnector->Name;
				}
			}
		}
	}
	else
	{
		ConnectorName = ConnectorNameString;
		ModuleName = *ModuleNameString;
	}

	ShortName = *ConnectorNameString;

	const FRigHierarchyModulePath ConnectorModulePath(ModuleName.ToString(), ConnectorName);
	const FRigElementKey ConnectorKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);
	Warning = FModularRigHierarchyConnectorWarning::TryCreate(Editor.ToSharedRef(), ConnectorKey, ModuleName);

	if (ModularRig)
	{
		RefreshDisplaySettings(ModularRig);
	}
}

void FModularRigHierarchyTreeElement::RefreshDisplaySettings(const UModularRig* InModularRig)
{
	const TPair<const FSlateBrush*, FSlateColor> Result = GetBrushAndColor(InModularRig);

	IconBrush = Result.Key;
	IconColor = Result.Value;
	TextColor = FSlateColor::UseForeground();
}

TSharedRef<ITableRow> FModularRigHierarchyTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FModularRigHierarchyTreeElement> InRigTreeElement, TSharedPtr<SModularRigHierarchyTreeView> InTreeView, bool bPinned)
{
	return SNew(SModularRigHierarchyTreeItem, InOwnerTable, InRigTreeElement, InTreeView, bPinned);
}

void FModularRigHierarchyTreeElement::AddChild(const TSharedPtr<FModularRigHierarchyTreeElement>& Child)
{
	Children.Add(Child);
}

void FModularRigHierarchyTreeElement::RemoveChild(const TSharedPtr<FModularRigHierarchyTreeElement>& Child)
{
	Children.Remove(Child);
}

void FModularRigHierarchyTreeElement::RequestRename()
{
	OnRenameRequested.ExecuteIfBound();
}

bool FModularRigHierarchyTreeElement::VerifyModuleName(const FText& InNameText, FText& OutErrorMessage) const
{
	const UModularRigController* Controller = WeakModularRigHierarchyViewModel.IsValid() ? WeakModularRigHierarchyViewModel.Pin()->GetModularRigController() : nullptr;
	if (Controller)
	{
		const FName NewName = *InNameText.ToString();

		return Controller->CanRenameModule(ModuleName, NewName, OutErrorMessage);
	}

	OutErrorMessage = NSLOCTEXT("ModularRigHierarchyTreeElement", "VerifyModuleNameInvalidController", "Invalid controller for Module, cannot rename module");
	return false;
}

void FModularRigHierarchyTreeElement::SetModuleName(const FText& InNameText)
{
	UModularRigController* Controller = WeakModularRigHierarchyViewModel.IsValid() ? WeakModularRigHierarchyViewModel.Pin()->GetModularRigController() : nullptr;

	FText ErrorMessage;
	if (Controller &&
		VerifyModuleName(InNameText, ErrorMessage) &&
		ensure(ErrorMessage.IsEmpty()))
	{
		// No need to adopt the new name. Since a hierarchy change will occur, the tree will refresh.
		Controller->RenameModule(ModuleName, *InNameText.ToString());
	}
}

TPair<const FSlateBrush*, FSlateColor> FModularRigHierarchyTreeElement::GetBrushAndColor(const UModularRig* InModularRig)
{
	const FSlateBrush* Brush = nullptr;
	FLinearColor Color = FSlateColor(EStyleColor::Foreground).GetColor(FWidgetStyle());
	float Opacity = 1.f;

	if (const FRigModuleInstance* ConnectorModule = InModularRig->FindModule(ModuleName))
	{
		const FModularRigModel& Model = InModularRig->GetModularRigModel();
		const FRigHierarchyModulePath ConnectorPath(ModuleName.ToString(), ConnectorName);
		bool bIsConnected = Model.Connections.HasConnection(FRigElementKey(ConnectorPath.GetPathFName(), ERigElementType::Connector));
		bool bConnectionWarning = !bIsConnected;

		if (const UControlRig* ModuleRig = ConnectorModule->GetRig())
		{
			const FRigModuleConnector* Connector = ModuleRig->GetRigModuleSettings().ExposedConnectors.FindByPredicate([this](FRigModuleConnector& Connector)
				{
					return Connector.Name == ConnectorName;
				});
			if (Connector)
			{
				if (Connector->IsPrimary())
				{
					if (bIsConnected)
					{
						const FSoftObjectPath IconPath = ModuleRig->GetRigModuleSettings().Icon;
						const TSharedPtr<FSlateBrush>* ExistingBrush = IconPathToBrush.Find(IconPath);
						if (ExistingBrush && ExistingBrush->IsValid())
						{
							Brush = ExistingBrush->Get();
						}
						else
						{
							if (UTexture2D* Icon = Cast<UTexture2D>(IconPath.TryLoad()))
							{
								const TSharedPtr<FSlateBrush> NewBrush = MakeShareable(new FSlateBrush(UWidgetBlueprintLibrary::MakeBrushFromTexture(Icon, 16.0f, 16.0f)));
								IconPathToBrush.FindOrAdd(IconPath) = NewBrush;
								Icons.Add(TStrongObjectPtr<UTexture2D>(Icon));
								Brush = NewBrush.Get();
							}
						}
					}
					else
					{
						Brush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorWarning");
					}
				}
				else if (Connector->Settings.bOptional)
				{
					bConnectionWarning = false;
					if (!bIsConnected)
					{
						Opacity = 0.7f;
						Color = FSlateColor(EStyleColor::Hover2).GetColor(FWidgetStyle());
					}
					Brush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorOptional");
				}
				else
				{
					Brush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorSecondary");
				}
			}
		}

		if (bConnectionWarning)
		{
			Color = FSlateColor(EStyleColor::Warning).GetColor(FWidgetStyle());
		}
	}
	if (!Brush)
	{
		Brush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.RigidBody");
	}

	// Apply opacity
	Color = Color.CopyWithNewOpacity(Opacity);

	return TPair<const FSlateBrush*, FSlateColor>(Brush, Color);
}
