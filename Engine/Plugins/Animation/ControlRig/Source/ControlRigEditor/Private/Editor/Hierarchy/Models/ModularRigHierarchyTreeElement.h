// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigHierarchyTreeElement.h"

class SModularRigHierarchyTreeItem;
class SModularRigHierarchyTreeView;
class UModularRig;
class UTexture2D;

namespace UE::ControlRigEditor 
{ 
	class FModularRigHierarchyViewModel;
	class FModularRigHierarchyConnectorWarning;
}

/** An item in the tree */
class FModularRigHierarchyTreeElement : public TSharedFromThis<FModularRigHierarchyTreeElement>
{
	using FModularRigHierarchyConnectorWarning = UE::ControlRigEditor::FModularRigHierarchyConnectorWarning;
	using FModularRigHierarchyViewModel = UE::ControlRigEditor::FModularRigHierarchyViewModel;

	// Allow SModularRigHierarchyTreeItem to bind to OnRenameRequested
	friend class SModularRigHierarchyTreeItem;

public:
	FModularRigHierarchyTreeElement(
		const TSharedRef<FModularRigHierarchyViewModel>& InModularRigHierarchyViewModel, 
		const FString& InKey, 
		const bool bInIsPrimary);

	/** Returns true if this element is a primary */
	bool IsPrimary() const { return bIsPrimary; }

	/** Returns the a key that identifies this model */
	const FString& GetKey() const { return Key; }
	
	/** Returns the name of the connector this element handles */
	const FString& GetConnectorName() const { return ConnectorName; }

	/** Returns the name of the module this element handles */
	const FName& GetModuleName() const { return ModuleName; }

	/** Returns the short name of the module this element handles */
	const FName& GetShortModuleName() const { return ShortName; }

	/** Returns the text color to use for this element */
	const FSlateColor& GetTextColor() const { return TextColor; }

	/** Returns the icon brush for this element */
	const FSlateBrush* GetIconBrush() const { return IconBrush; }

	/** Returns the icon color to use for this element */
	const FSlateColor& GetIconColor() const { return IconColor; }

	/** Returns the child elements of this element */
	const TArray<TSharedPtr<FModularRigHierarchyTreeElement>>& GetChildren() const { return Children; };

	/** Adds a child to this element */
	void AddChild(const TSharedPtr<FModularRigHierarchyTreeElement>& Child);

	/** Removes a child to this element */
	void RemoveChild(const TSharedPtr<FModularRigHierarchyTreeElement>& Child);

	/** Requests to rename the modular rig hierarchy element */
	void RequestRename();

	/** Verifies if the module name is valid for this element */
	bool VerifyModuleName(const FText& InNameText, FText& OutErrorMessage) const;

	/** Sets the module name */
	void SetModuleName(const FText& InNameText);

	/** Returns the current warning, or nullptr if no warning exists */
	const TSharedPtr<FModularRigHierarchyConnectorWarning>& GetWarning() const { return Warning; }

	/** Creates a widget for this element */
	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FModularRigHierarchyTreeElement> InRigTreeElement, TSharedPtr<SModularRigHierarchyTreeView> InTreeView, bool bPinned);

private:
	void RefreshDisplaySettings(const UModularRig* InModularRig);

	TPair<const FSlateBrush*, FSlateColor> GetBrushAndColor(const UModularRig* InModularRig);

	/** A delegate to notify when RequestRename was called */
	DECLARE_DELEGATE(FOnRenameRequested);

	/** Delegate raised when RequestRename was called, useful for the related editing widget */
	FOnRenameRequested OnRenameRequested;

	/** View outer model for the modular rig hierarchy */
	TWeakPtr<FModularRigHierarchyViewModel> WeakModularRigHierarchyViewModel;

	/** Element Data to display */
	FString Key;
	bool bIsPrimary;
	FName ModuleName;
	FString ConnectorName;
	FName ShortName;
	TArray<TSharedPtr<FModularRigHierarchyTreeElement>> Children;

	static TMap<FSoftObjectPath, TSharedPtr<FSlateBrush>> IconPathToBrush;
	static TArray<TStrongObjectPtr<UTexture2D>> Icons;

	/** The brush to use when rendering an icon */
	const FSlateBrush* IconBrush = nullptr;

	/** The color to use when rendering an icon */
	FSlateColor IconColor;

	/** The color to use when rendering the label text */
	FSlateColor TextColor;

	/** A warning for this element, or nullptr if there is no warning */
	TSharedPtr<FModularRigHierarchyConnectorWarning> Warning;
};
