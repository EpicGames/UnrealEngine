// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigHierarchyViewModel.h"

#include "Editor/ControlRigEditor.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ModularRig.h"
#include "ControlRigAssetReference.h"

namespace UE::ControlRigEditor
{
	FModularRigHierarchyScopedSuspendDetailsPanelRefresh::FModularRigHierarchyScopedSuspendDetailsPanelRefresh(const TSharedRef<FModularRigHierarchyViewModel>& ViewModel)
		: WeakViewModel(ViewModel)
	{
		if (const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = ViewModel->GetControlRigEditor())
		{
			bStateToRestore = ControlRigEditor->GetSuspendDetailsPanelRefreshFlag();
			ControlRigEditor->GetSuspendDetailsPanelRefreshFlag() = true;
		}
	}

	FModularRigHierarchyScopedSuspendDetailsPanelRefresh::~FModularRigHierarchyScopedSuspendDetailsPanelRefresh()
	{
		// Only restore if the flag is still true, otherwise something set it to false in the interim
		const TSharedPtr<FModularRigHierarchyViewModel> ViewModel = WeakViewModel.IsValid() ? WeakViewModel.Pin() : nullptr;
		const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = ViewModel.IsValid() ? ViewModel->GetControlRigEditor() : nullptr;
		if (ControlRigEditor.IsValid() &&
			ControlRigEditor->GetSuspendDetailsPanelRefreshFlag())
		{
			ControlRigEditor->GetSuspendDetailsPanelRefreshFlag() = bStateToRestore;
		}
	}

	FModularRigHierarchyViewModel::FModularRigHierarchyViewModel(const TSharedRef<IControlRigBaseEditor>& ControlRigEditor)
		: WeakControlRigEditor(ControlRigEditor)
	{}

	void FModularRigHierarchyViewModel::RefreshDetails()
	{
		if (const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = GetControlRigEditor())
		{
			ControlRigEditor->RefreshDetailView();
		}
	}

	TArray<FName> FModularRigHierarchyViewModel::GetSelection() const
	{
		TArray<FName> SelectedModuleNames;
		if (UModularRigController* Controller = GetModularRigController())
		{
			SelectedModuleNames = Controller->GetSelectedModules();
		}

		return SelectedModuleNames;
	}

	void FModularRigHierarchyViewModel::SelectModules(const TArray<FName>& ModuleNames)
	{
		const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = GetControlRigEditor();
		UModularRigController* Controller = GetModularRigController();
		if (ControlRigEditor.IsValid() &&
			Controller)
		{
			Controller->SetModuleSelection(ModuleNames);
			ControlRigEditor->SetDetailViewForRigModules(ModuleNames);
		}
	}

	void FModularRigHierarchyViewModel::SelectModuleAndChildren(const FName& ModuleName)
	{
		const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = GetControlRigEditor();
		UModularRigController* Controller = GetModularRigController();
		const FRigModuleReference* Module = FindModule(ModuleName);
		if (ControlRigEditor.IsValid() &&
			Controller &&
			Module)
		{
			const TArray<FName> ModulesToSelect = GetModuleAndDecendantsOrdered(Module);

			Controller->SetModuleSelection(ModulesToSelect);

			ControlRigEditor->RefreshDetailView();
		}
	}

	FName FModularRigHierarchyViewModel::AddModule(UClass* Class, const FName& InParentModuleName)
	{
		if (UModularRigController* Controller = GetModularRigController())
		{
			FString ClassName = Class->GetName();
			ClassName.RemoveFromEnd(TEXT("_C"));

			const FRigName Name = Controller->GetSafeNewName(FRigName(ClassName));

			const FName NewModuleName = Controller->AddModule(Name, Class, InParentModuleName);

			return NewModuleName;
		}

		return NAME_None;
	}

	FName FModularRigHierarchyViewModel::AddModule(const FControlRigAssetStrongReference& InSource, const FName& InParentModuleName)
	{
		if (UModularRigController* Controller = GetModularRigController())
		{
			FString ClassName = InSource.GetName();
			ClassName.RemoveFromEnd(TEXT("_C"));

			const FRigName Name = Controller->GetSafeNewName(FRigName(ClassName));

			const FName NewModuleName = Controller->AddModuleFromAssetReference(Name, InSource, InParentModuleName);

			return NewModuleName;
		}

		return NAME_None;
	}

	void FModularRigHierarchyViewModel::DeleteModules(const TArray<FName>& ModuleNames)
	{
		UModularRigController* Controller = GetModularRigController();
		const FModularRigModel* ModularRigModel = GetModularRigModel();
		if (Controller &&
			ModularRigModel)
		{
			// Make sure we delete the modules from children to root
			TArray<FName> SortedModuleNames = ModularRigModel->SortModuleNames(ModuleNames);
			Algo::Reverse(SortedModuleNames);			

			for (const FName& ModuleName : SortedModuleNames)
			{
				Controller->DeleteModule(ModuleName);
			}
		}
	}

	void FModularRigHierarchyViewModel::ReparentModules(const TArray<FName>& ModuleNames, const FName& ParentModuleName, const int32 NewModuleIndex)
	{
		UModularRigController* Controller = GetModularRigController();
		const FModularRigModel* ModularRigModel = GetModularRigModel();
		if (Controller &&
			ModularRigModel)
		{
			for (const FName& ModuleName : ModuleNames)
			{
				Controller->ReparentModule(ModuleName, ParentModuleName);
				if (NewModuleIndex != INDEX_NONE)
				{
					Controller->ReorderModule(ModuleName, NewModuleIndex);
				}
			}			
		}
	}

	void FModularRigHierarchyViewModel::ReresolveModules(const TArray<FName>& ModuleAndConnectorNames)
	{
		const UModularRig* ModularRig = GetModularRig();
		const URigHierarchy* Hierarchy = GetHierarchy();
		UModularRigController* Controller = GetModularRigController();		
		if (!ModularRig ||
			!Hierarchy ||
			!Controller)
		{
			return;
		}

		TArray<FRigElementKey> ConnectorKeys;
		for (const FName& PathAndConnector : ModuleAndConnectorNames)
		{
			FString ModuleNameString = PathAndConnector.ToString();
			FString ConnectorName;
			FRigHierarchyModulePath(PathAndConnector).Split(&ModuleNameString, &ConnectorName);

			const FRigModuleReference* Module = FindModule(*ModuleNameString);
			if (!Module)
			{
				UE_LOGF(LogControlRig, Error, "Could not find module %ls", *ModuleNameString);
				return;
			}

			if (!ConnectorName.IsEmpty())
			{
				// if we are executing this on a primary connector we want to re-resolve all secondaries
				const FRigConnectorElement* PrimaryConnector = Module->FindPrimaryConnector(Hierarchy);
				const FName DesiredName = Hierarchy->GetNameMetadata(PrimaryConnector->GetKey(), URigHierarchy::DesiredNameMetadataName, NAME_None);
				if (!DesiredName.IsNone() && DesiredName.ToString().Equals(ConnectorName, ESearchCase::IgnoreCase))
				{
					ConnectorName.Reset();
				}
			}

			const TArray<const FRigConnectorElement*> Connectors = Module->FindConnectors(Hierarchy);
			for (const FRigConnectorElement* Connector : Connectors)
			{
				if (Connector->IsSecondary())
				{
					if (ConnectorName.IsEmpty())
					{
						ConnectorKeys.AddUnique(Connector->GetKey());
					}
					else
					{
						const FName DesiredName = Hierarchy->GetNameMetadata(Connector->GetKey(), URigHierarchy::DesiredNameMetadataName, NAME_None);
						if (!DesiredName.IsNone() && DesiredName.ToString().Equals(ConnectorName, ESearchCase::IgnoreCase))
						{
							ConnectorKeys.AddUnique(Connector->GetKey());
							break;
						}
					}
				}
			}
		}

		constexpr bool bReplaceExistingConnections = true;
		constexpr bool bSetupUndoRedo = true;
		Controller->AutoConnectSecondaryConnectors(ConnectorKeys, bReplaceExistingConnections, bSetupUndoRedo);
	}

	void FModularRigHierarchyViewModel::DuplicateModules(const TArray<FName>& ModuleNames)
	{
		// Can conveniently use MirrorModules with adequate settings
		FRigVMMirrorSettings MirrorSettings;
		MirrorSettings.MirrorAxis = EAxis::None;
		MirrorSettings.AxisToFlip = EAxis::None;

		MirrorModules(ModuleNames, MirrorSettings);
	}

	void FModularRigHierarchyViewModel::MirrorModules(const TArray<FName>& ModuleNames, const FRigVMMirrorSettings& MirrorSettings)
	{
		UModularRigController* Controller = GetModularRigController();
		const FModularRigModel* ModularRigModel = GetModularRigModel();
		if (Controller &&
			ModularRigModel)
		{
			// Mirror the whole hierarchy within module names
			const TArray<FName> SortedModuleNames = GetModuleAndDecendantsOrdered(ModuleNames);

			TMap<FName, FName> ModuleToMirroredModuleMap;
			for (const FName& ModuleName : SortedModuleNames)
			{				
				const FName MirroredModuleName = Controller->MirrorModule(ModuleName, MirrorSettings);

				if (MirroredModuleName != NAME_None)
				{
					ModuleToMirroredModuleMap.Add(ModuleName, MirroredModuleName);
				}
			}

			// Adopt the hierarchical structure
			for (const TTuple<FName, FName>& ModuleToMirroredModulePair : ModuleToMirroredModuleMap)
			{
				const FRigModuleReference* OldModule = FindModule(ModuleToMirroredModulePair.Key);
				const FRigModuleReference* OldParentModule = OldModule ? FindModule(OldModule->ParentModuleName) : nullptr;
				const FName* NewParentNamePtr = OldParentModule ? ModuleToMirroredModuleMap.Find(OldParentModule->Name) : nullptr;
				
				constexpr int32 ReparentToAnyIndex = INDEX_NONE;
				if (NewParentNamePtr)
				{
					ReparentModules({ ModuleToMirroredModulePair.Value }, *NewParentNamePtr, ReparentToAnyIndex);
				}
				else if (OldParentModule)
				{
					ReparentModules({ ModuleToMirroredModulePair.Value }, OldParentModule->Name, ReparentToAnyIndex);
				}
			}
		}
	}

	void FModularRigHierarchyViewModel::ClipboardCopyModuleSettings(const TArray<FName>& ModuleNames) const
	{
		if (const UModularRigController* Controller = GetModularRigController())
		{
			const FString ContentAsString = Controller->ExportModuleSettingsToString(ModuleNames);
			if (!ContentAsString.IsEmpty())
			{
				FPlatformApplicationMisc::ClipboardCopy(*ContentAsString);
			}
		}
	}

	bool FModularRigHierarchyViewModel::CanClipboardPasteModuleSettings(const int32 ExpectedNumElements) const
	{
		FString ContentAsString;
		FPlatformApplicationMisc::ClipboardPaste(ContentAsString);
		if (ContentAsString.IsEmpty())
		{
			return false;
		}

		FControlRigOverrideValueErrorPipe ErrorPipe;
		FModularRigModuleSettingsSetForClipboard Content;
		FModularRigModuleSettingsSetForClipboard::StaticStruct()->ImportText(*ContentAsString, &Content, nullptr, PPF_None, &ErrorPipe, FModularRigModuleSettingsForClipboard::StaticStruct()->GetName(), true);
		if (ErrorPipe.GetNumErrors() > 0)
		{
			return false;
		}

		return Content.Settings.Num() == ExpectedNumElements;
	}

	void FModularRigHierarchyViewModel::ClipboardPasteModuleSettings(const TArray<FName>& ModuleNames) const
	{
		const int32 ExpectedNumElements = ModuleNames.Num();
		if (!CanClipboardPasteModuleSettings(ExpectedNumElements))
		{
			return;
		}

		FString ContentAsString;
		FPlatformApplicationMisc::ClipboardPaste(ContentAsString);
		if (ContentAsString.IsEmpty())
		{
			return;
		}

		if (UModularRigController* Controller = GetModularRigController())
		{
			Controller->ImportModuleSettingsFromString(ContentAsString, ModuleNames);
		}
	}

	TSharedPtr<IControlRigBaseEditor> FModularRigHierarchyViewModel::GetControlRigEditor() const
	{
		const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakControlRigEditor.IsValid() ? WeakControlRigEditor.Pin() : nullptr;

		return ControlRigEditor;
	}

	FControlRigAssetInterfacePtr FModularRigHierarchyViewModel::GetControlRigAssetInterface() const
	{
		const TSharedPtr<IControlRigBaseEditor>& ControlRigEditor = GetControlRigEditor();
		const FControlRigAssetInterfacePtr ControlRigAssetInterface = ControlRigEditor.IsValid() ? ControlRigEditor->GetControlRigAssetInterface() : nullptr;

		return ControlRigAssetInterface;
	}

	FRigVMEditorAssetInterfacePtr FModularRigHierarchyViewModel::GetRigVMAssetInterface() const
	{
		const FControlRigAssetInterfacePtr ControlRigAssetInterface = GetControlRigAssetInterface();
		const FRigVMEditorAssetInterfacePtr RigVMEditorAssetInterface = ControlRigAssetInterface ? ControlRigAssetInterface->GetRigVMAssetInterface() : nullptr;

		return RigVMEditorAssetInterface;
	}

	UModularRig* FModularRigHierarchyViewModel::GetModularRig() const
	{
		if (const FControlRigAssetInterfacePtr ControlRigAssetInterface = GetControlRigAssetInterface())
		{
			if (UControlRig* DebuggedRig = ControlRigAssetInterface->GetDebuggedControlRig())
			{
				return Cast<UModularRig>(DebuggedRig);
			}
		}
		
		if (WeakControlRigEditor.IsValid())
		{
			if (UControlRig* CurrentRig = WeakControlRigEditor.Pin()->GetControlRig())
			{
				return Cast<UModularRig>(CurrentRig);
			}
		}

		return nullptr;
	}

	URigHierarchy* FModularRigHierarchyViewModel::GetHierarchy() const
	{
		const TSharedPtr<IControlRigBaseEditor>& ControlRigEditor = WeakControlRigEditor.IsValid() ? WeakControlRigEditor.Pin() : nullptr;
		URigHierarchy* Hierarchy = ControlRigEditor.IsValid() ? ControlRigEditor->GetHierarchyBeingDebugged() : nullptr;

		return Hierarchy;
	}

	UModularRigController* FModularRigHierarchyViewModel::GetModularRigController() const
	{
		const FControlRigAssetInterfacePtr ControlRigAssetInterface = GetControlRigAssetInterface();
		UModularRigController* ModularRigController = ControlRigAssetInterface ? ControlRigAssetInterface->GetModularRigController() : nullptr;

		return ModularRigController;
	}

	FModularRigModel* FModularRigHierarchyViewModel::GetModularRigModel() const
	{
		UModularRigController* ModularRigController = GetModularRigController();
		FModularRigModel* ModularRigModel = ModularRigController ? ModularRigController->Model : nullptr;

		return ModularRigModel;
	}

	TArray<FRigModuleReference*> FModularRigHierarchyViewModel::GetRootModules() const
	{
		const FModularRigModel* ModularRigModel = GetModularRigModel();
		const TArray<FRigModuleReference*> RootModules = ModularRigModel ? ModularRigModel->RootModules : TArray<FRigModuleReference*>{};

		return RootModules;
	}

	const FRigModuleReference* FModularRigHierarchyViewModel::FindModule(const FName& ModuleName) const
	{
		const FControlRigAssetInterfacePtr ControlRigAssetInterface = GetControlRigAssetInterface();
		const FRigModuleReference* Module = ControlRigAssetInterface ? ControlRigAssetInterface->GetModularRigModel().FindModule(ModuleName) : nullptr;

		return Module;
	}

	FControlRigAssetSoftReference FModularRigHierarchyViewModel::GetModuleAssetReference(const FName& ModuleName) const
	{
		FControlRigAssetSoftReference ModuleAssetReference;
		if (const FRigModuleReference* Module = FindModule(ModuleName))
		{
			ModuleAssetReference = Module->ControlRigAssetReference;
		}

		return ModuleAssetReference;
	}

	TArray<FName> FModularRigHierarchyViewModel::GetModuleAndDecendants(const FRigModuleReference* ModuleRef) const
	{
		TArray<FName> Result;
		if (ModuleRef)
		{
			Result.AddUnique(ModuleRef->Name);

			for (const FRigModuleReference* ChildModule : ModuleRef->CachedChildren)
			{
				// Recursive
				Result.Append(GetModuleAndDecendantsOrdered(ChildModule));
			}
		}

		return Result;
	}

	TArray<FName> FModularRigHierarchyViewModel::GetModuleAndDecendantsOrdered(const FRigModuleReference* ModuleRef) const
	{
		TArray<FName> Result = GetModuleAndDecendants(ModuleRef);
		if (const FModularRigModel* ModularRigModel = GetModularRigModel())
		{
			Result = ModularRigModel->SortModuleNames(Result);
		}

		return Result;
	}

	TArray<FName> FModularRigHierarchyViewModel::GetModuleAndDecendantsOrdered(const TArray<FName>& ModuleNames) const
	{
		TSet<FName> ModulesAndDecendants;
		if (UModularRigController* Controller = GetModularRigController())
		{
			for (const FName& ModuleName : ModuleNames)
			{
				if (const FRigModuleReference* ModuleRef = FindModule(ModuleName))
				{
					ModulesAndDecendants.Append(GetModuleAndDecendants(ModuleRef));
				}
			}
		}

		TArray<FName> Result = ModulesAndDecendants.Array();
		if (const FModularRigModel* ModularRigModel = GetModularRigModel())
		{
			Result = ModularRigModel->SortModuleNames(Result);
		}

		return Result;
	}
}
