// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigAssetReference.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ScriptInterface.h"
#include "UObject/SoftObjectPtr.h"

class FModularRigHierarchyTreeElement;
class IControlRigBaseEditor; 
class IControlRigEditorAssetInterface;
class IRigVMEditorAssetInterface;
class UClass;
class UControlRig;
class UModularRig;
class UModularRigController;
class URigHierarchy;
struct FModularRigModel;
struct FRigModuleReference;
struct FRigVMMirrorSettings;
template <typename InInterfaceType> class TScriptInterface;

using FControlRigAssetInterfacePtr = TScriptInterface<IControlRigEditorAssetInterface>;
using FRigVMEditorAssetInterfacePtr = TScriptInterface<IRigVMEditorAssetInterface>;

namespace UE::ControlRigEditor
{
	class FModularRigHierarchyViewModel;

	/** Struct to suspend details panel refresh over the scope of this function */
	struct FModularRigHierarchyScopedSuspendDetailsPanelRefresh
	{
		FModularRigHierarchyScopedSuspendDetailsPanelRefresh(const TSharedRef<FModularRigHierarchyViewModel>& ViewModel);

		~FModularRigHierarchyScopedSuspendDetailsPanelRefresh();

	private:
		bool bStateToRestore = false;

		TWeakPtr<FModularRigHierarchyViewModel> WeakViewModel;
	};

	/** A view model for the modular rig hierarchy */
	class FModularRigHierarchyViewModel
		: FNoncopyable
	{
	public:
		FModularRigHierarchyViewModel(const TSharedRef<IControlRigBaseEditor>& ControlRigEditor);

		/** Refreshes the details view */
		void RefreshDetails();

		/** Returns the elements selected in data */
		TArray<FName> GetSelection() const;

		/** Selects the module in data */
		void SelectModules(const TArray<FName>& ModuleNames);

		/** Selects the module and its children in data */
		void SelectModuleAndChildren(const FName& ModuleName);

		/** Adds a new module to the rig. Returns the name of the new module or NAME_None if the module could not be added */
		FName AddModule(UClass* InClass, const FName& InParentModuleName);
		FName AddModule(const FControlRigAssetStrongReference& InSource, const FName& InParentModuleName);

		/** Deletes modules given an array of module names */
		void DeleteModules(const TArray<FName>& ModuleNames);

		/** Reparents the modules under their new parent at given index */
		void ReparentModules(const TArray<FName>& ModuleNames, const FName& ParentModuleName, const int32 NewModuleIndex);

		/** Reresolves the modules by auto-conneting secondary connectors */
		void ReresolveModules(const TArray<FName>& ModuleNames);

		/** Duplicates the modules  */
		void DuplicateModules(const TArray<FName>& ModuleNames);

		/** Mirrors the modules according to mirror settings */
		void MirrorModules(const TArray<FName>& ModuleNames, const FRigVMMirrorSettings& MirrorSettings);

		/** Copies the module settings of specified modules to clipboard */
		void ClipboardCopyModuleSettings(const TArray<FName>& ModuleNames) const;

		/** Returns true if clipboard paste module settings is possible and results in expected num elements */
		bool CanClipboardPasteModuleSettings(const int32 ExpectedNumElements) const;

		/** Pastes the module settings from clipboard to the specified modules */
		void ClipboardPasteModuleSettings(const TArray<FName>& ModuleNames) const;

		/** Returns the control rig editor */
		TSharedPtr<IControlRigBaseEditor> GetControlRigEditor() const;

		/** Returns the control rig asset interface */
		FControlRigAssetInterfacePtr GetControlRigAssetInterface() const;

		/** Returns the rig vm asset interface */
		FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const;

		/** Returns the modular rig being edited */
		UModularRig* GetModularRig() const;

		/** Returns the hierarchy */
		URigHierarchy* GetHierarchy() const;

		/** Returns the Modular Rig Controller */
		UModularRigController* GetModularRigController() const;

		/** Returns the modular rig model of this hierarchy */
		FModularRigModel* GetModularRigModel() const;

		/** Returns the root modules of the hierarchy */
		TArray<FRigModuleReference*> GetRootModules() const;

		/** Finds the module with given name */
		const FRigModuleReference* FindModule(const FName& ModuleName) const;

		/** Returns the module class of the module */
		FControlRigAssetSoftReference GetModuleAssetReference(const FName& ModuleName) const;

	private:
		/** Gets the name of the module and its children recursively */
		TArray<FName> GetModuleAndDecendants(const FRigModuleReference* ModuleRef) const;

		/** Gets the name of the module and its children recursively, ordered from root to children */
		TArray<FName> GetModuleAndDecendantsOrdered(const FRigModuleReference* ModuleRef) const;

		/** Gets the name of the modules and their children recursively, ordered from root to children */
		TArray<FName> GetModuleAndDecendantsOrdered(const TArray<FName>& ModuleNames) const;

		/** The editor that relates to this model */
		TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;
	};
}
