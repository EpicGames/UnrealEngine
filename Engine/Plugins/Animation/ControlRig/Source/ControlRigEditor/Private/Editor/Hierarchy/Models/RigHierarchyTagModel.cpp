// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyTagModel.h"

#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigEditorStyle.h"
#include "Editor/Hierarchy/RigHierarchyTreeDisplaySettings.h"

#define LOCTEXT_NAMESPACE "RigHierarchyTagModel"

namespace UE::ControlRigEditor
{
	bool FRigHierarchyTagModelBase::MakeTagWidgetArgs(SRigHierarchyTagWidget::FArguments& OutArgs) const
	{
		if (!bIsValid)
		{
			return false;
		}

		SRigHierarchyTagWidget::FArguments WidgetArgs;
		{
			OutArgs.AllowDragDrop(bAllowDragDrop);
			OutArgs.Visibility(Visibility);
			OutArgs.Icon(IconBrush);
			OutArgs.IconSize(FVector2d(16.f, 16.f));
			OutArgs.Text(Label);
			OutArgs.TooltipText(Tooltip);
			OutArgs.Identifier(Identifier);

			constexpr FColor DefaultIconColor = FColor(26, 26, 26, 255); // 1A1A1A
			const FLinearColor OverridenIconColor = IconColor.IsSet() ? IconColor.GetValue() : DefaultIconColor;
			OutArgs.IconColor(OverridenIconColor);

			constexpr FColor DefaultBackgroundColor = FColor(38, 187, 255, 255); // 26BBFF
			const FLinearColor OverridenBackgroundColor = BackgroundColor.IsSet() ? BackgroundColor.GetValue() : DefaultBackgroundColor;
			OutArgs.Color(OverridenBackgroundColor);

			constexpr FColor DefaultTextColor = FColor(15, 15, 15, 255); // 0F0F0F
			OutArgs.TextColor(DefaultTextColor);

			if (OnClicked.IsBound())
			{
				OutArgs.OnClicked(OnClicked);
			}

			if (OnRenamed.IsBound())
			{
				OutArgs.OnRenamed(OnRenamed);
			}

			if (OnVerifyRename.IsBound())
			{
				OutArgs.OnVerifyRename(OnVerifyRename);
			}
		}

		return true;
	}

	FRigHierarchyValidConnectorTag::FRigHierarchyValidConnectorTag(const ERigHierarchyConnectorTagDisplayMode InTagDisplayMode)
		: TagDisplayMode(InTagDisplayMode)
	{
		bAllowDragDrop = TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Individual;
	}

	FRigHierarchyValidConnectorTag FRigHierarchyValidConnectorTag::BuildTag(const ERigHierarchyConnectorTagDisplayMode InTagDisplayMode)
	{
		return FRigHierarchyValidConnectorTag(InTagDisplayMode);
	}

	FRigHierarchyValidConnectorTag& FRigHierarchyValidConnectorTag::AddConnector(
		const UControlRig& ControlRig,
		const FRigHierarchyTreeDisplaySettings& DisplaySettings,
		const FRigConnectorElement& ConnectorElement)
	{
		URigHierarchy* Hierarchy = ControlRig.GetHierarchy();
		if (!Hierarchy)
		{
			bIsValid = false;
			return *this;
		}

		const bool bCanAdd = TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Single || ConnectorElementKeys.IsEmpty();
		if (!ensureMsgf(bCanAdd, TEXT("Unexpected trying to combine tags for a rig hierarchy tree view that uses ERigHierarchyConnectorTagDisplayMode::Individual")))
		{
			bIsValid = false;
			return *this;
		}

		const FRigElementKey ConnectorElementKey = ConnectorElement.GetKey();
		ConnectorElementKeys.Add(ConnectorElementKey);

		const EElementNameDisplayMode NameDisplayMode = DisplaySettings.NameDisplayMode;
		const FText DisplayName = Hierarchy->GetDisplayNameForUI(ConnectorElementKey, NameDisplayMode);

		if (TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Individual)
		{
			static const FSlateBrush* PrimaryBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorPrimary");
			static const FSlateBrush* SecondaryBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorSecondary");
			static const FSlateBrush* OptionalBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorOptional");

			Label = DisplayName;
			Tooltip = FText::FromName(ConnectorElementKey.Name);

			Identifier = [&ConnectorElementKey]()
				{
					FString Result;
					constexpr const void* Defaults = nullptr;
					constexpr UObject* OwnerObject = nullptr;
					constexpr EPropertyPortFlags PortFlags = PPF_None;
					constexpr UObject* ExportRootScope = nullptr;

					FRigElementKey::StaticStruct()->ExportText(Result, &ConnectorElementKey, Defaults, OwnerObject, PortFlags, ExportRootScope);
					return Result;
				}();

			IconBrush = [&ConnectorElement]()
				{

					if (ConnectorElement.Settings.Type == EConnectorType::Secondary)
					{
						return ConnectorElement.Settings.bOptional ? OptionalBrush : SecondaryBrush;
					}

					return PrimaryBrush;
				}();
		}
		else if (TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Single)
		{
			static const FSlateBrush* PlainBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorPlain.Small");

			Tooltip = !Tooltip.IsSet() ?
				FText::FromString(ConnectorElementKey.Name.ToString()) :
				FText::FromString(FString::Printf(TEXT("%s\n%s"), *Tooltip.Get().ToString(), *ConnectorElementKey.Name.ToString()));

			IconBrush = PlainBrush;
		}

		if (!ensureMsgf(Identifier.IsSet() || !bAllowDragDrop, TEXT("Unexpected tag has no identifier set, but allows drag drop. This leaves drag-drop without unfunctional.")))
		{
			bIsValid = false;
			return *this;
		}

		// Only now that all data is properly initialized this tag becomes valid
		bIsValid = true;

		return *this;
	}

	FRigHierarchyValidConnectorTag& FRigHierarchyValidConnectorTag::SetOnClicked(const FSimpleDelegate& InOnClicked)
	{
		if (!ensureMsgf(!OnClicked.IsBound(), TEXT("Unexpected trying to bind on clicked when it is already bound, ignoring new binding")))
		{
			bIsValid = false;
			return *this;
		}

		OnClicked = InOnClicked;

		return *this;
	}

	FRigHierarchyValidConnectorTag& FRigHierarchyValidConnectorTag::SetOnRenamed(const FOnTextCommitted& InOnRenamed)
	{
		if (!ensureMsgf(TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Individual, TEXT("Cannot handle renaming tags in a rig hierarchy tree view that uses ERigHierarchyConnectorTagDisplayMode::Single, it can contain multiple connectors.")))
		{
			bIsValid = false;
			return *this;
		}

		if (!ensureMsgf(!OnRenamed.IsBound(), TEXT("Unexpected trying to bind on renamed when it is already bound, ignoring new binding")))
		{
			bIsValid = false;
			return *this;
		}

		OnRenamed = InOnRenamed;

		return *this;
	}

	FRigHierarchyValidConnectorTag& FRigHierarchyValidConnectorTag::SetOnVerifyRename(const FOnVerifyTextChanged& InOnVerifyRename)
	{
		if (!ensureMsgf(TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Individual, TEXT("Cannot handle renaming tags in a rig hierarchy tree view that uses ERigHierarchyConnectorTagDisplayMode::Single, it can contain multiple connectors.")))
		{
			bIsValid = false;
			return *this;
		}

		if (!ensureMsgf(!OnVerifyRename.IsBound(), TEXT("Unexpected trying to bind on verify rename when it is already bound, ignoring new binding")))
		{
			bIsValid = false;
			return *this;
		}

		OnVerifyRename = InOnVerifyRename;

		return *this;
	}

	FRigHierarchyConnectorUnresolvedWarningTag::FRigHierarchyConnectorUnresolvedWarningTag(
		const UControlRig& ControlRig,
		const FRigHierarchyTreeDisplaySettings& DisplaySettings,
		const FRigConnectorElement& ConnectorElement)
	{
		URigHierarchy* Hierarchy = ControlRig.GetHierarchy();
		if (!Hierarchy)
		{
			bIsValid = false;
			return;
		}

		Label = LOCTEXT("ConnectorWarningTagLabel", "Warning");
	
		BackgroundColor = FColor::FromHex(TEXT("#FFB800"));
		IconColor = FColor::FromHex(TEXT("#1A1A1A"));
		IconBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorWarning");

		static const FText NotResolvedWarning = LOCTEXT("ConnectorWarningConnectorNotResolved", "Connector is not resolved.");

		const TWeakObjectPtr<const UControlRig> WeakControlRig(&ControlRig);
		const FRigElementKey ConnectorKey = ConnectorElement.GetKey();

		const auto GetTooltipLambda = TAttribute<FText>::CreateLambda([WeakControlRig, ConnectorKey]()
			{					
				const URigHierarchy* Hierarchy = WeakControlRig.IsValid() ? WeakControlRig->GetHierarchy() : nullptr;
				FControlRigAssetInterfacePtr EditorAsset = WeakControlRig.IsValid() ? WeakControlRig->GetAssetReference().GetEditorAsset() : nullptr;
				if (!Hierarchy ||
					!EditorAsset)
				{
					return FText::GetEmpty();
				}

				const FRigElementKey TargetKey = EditorAsset->GetModularRigModel().Connections.FindTargetFromConnector(ConnectorKey);

				return TargetKey.IsValid() && Hierarchy->Contains(TargetKey) ?
					FText::GetEmpty() :
					NotResolvedWarning;
			});

		Tooltip = GetTooltipLambda;

		Visibility = TAttribute<EVisibility>::CreateLambda([GetTooltipLambda]() -> EVisibility
			{
				return GetTooltipLambda.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
			});
	}
}

#undef LOCTEXT_NAMESPACE
