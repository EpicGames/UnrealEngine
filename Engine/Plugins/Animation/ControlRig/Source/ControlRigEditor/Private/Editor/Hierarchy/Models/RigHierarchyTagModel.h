// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/Widgets/SRigHierarchyTagWidget.h"
#include "Framework/SlateDelegates.h"

class UControlRig;
struct FRigConnectorElement;
struct FRigHierarchyTreeDisplaySettings;

namespace UE::ControlRigEditor
{
	/** A tag in a rig hierarchy */
	struct FRigHierarchyTagModelBase 
	{
		/** Creates arguments to display this tag in an SRigHierarchyTagWidget */
		bool MakeTagWidgetArgs(SRigHierarchyTagWidget::FArguments& OutArgs) const;

	protected:
		/** If true, this tag can be drag dropped */
		bool bAllowDragDrop = true;

		/** Visiblity attribute */
		TAttribute<EVisibility> Visibility;

		/** The icon to display */
		const FSlateBrush* IconBrush = nullptr;

		/** Optional icon color */
		TOptional<FLinearColor> IconColor;

		/** Optional backgrouond color */
		TOptional<FLinearColor> BackgroundColor;

		/** Label to display */
		TAttribute<FText> Label;

		/** The tooltip for the tag */
		TAttribute<FText> Tooltip;

		/** ID, should be set when bAllowDragDrop is true */
		TAttribute<FString> Identifier;

		/** Delegate executed when the tag is clicked */
		FSimpleDelegate OnClicked;

		/** Delegate executed when the tag was renamed */
		FOnTextCommitted OnRenamed;

		/** Delegate executed when the while the tag is being renamed, allowing to verify the currrent name */
		FOnVerifyTextChanged OnVerifyRename;

		/** Set to true when all data in this tag is valid */
		bool bIsValid = false;
	};

	/** Defines how Tags are displayed in the Hierarchy */
	enum class ERigHierarchyConnectorTagDisplayMode : uint8
	{
		Single,
		Individual
	};

	/** A tag for a valid connector in a rig hierarchy */
	struct FRigHierarchyValidConnectorTag final
		: public FRigHierarchyTagModelBase
	{
	private:
		/** Constructs this tag, private on purpose, use BuildTag instead */
		FRigHierarchyValidConnectorTag(const ERigHierarchyConnectorTagDisplayMode InTagDisplayMode);

	public:
		/** Builds a new tag */
		static FRigHierarchyValidConnectorTag BuildTag(const ERigHierarchyConnectorTagDisplayMode InTagDisplayMode);

		/**
		 * Sets the tag given a connector element, combines it if a connector element is already set.
		 * Combining should only occur when the tag display mode is ERigHierarchyConnectorTagDisplayMode::Single (ensured).
		 */
		FRigHierarchyValidConnectorTag& AddConnector(
			const UControlRig& ControlRig,
			const FRigHierarchyTreeDisplaySettings& DisplaySettings,
			const FRigConnectorElement& ConnectorElement);

		/** Sets a delegate executed when the tag is clicked */
		FRigHierarchyValidConnectorTag& SetOnClicked(const FSimpleDelegate& InOnClicked);

		/** Sets a delegate executed when the tag was renamed */
		FRigHierarchyValidConnectorTag& SetOnRenamed(const FOnTextCommitted& InOnRenamed);

		/** Sets a delegate executed when the while the tag is being renamed, allowing to verify the currrent name */
		FRigHierarchyValidConnectorTag& SetOnVerifyRename(const FOnVerifyTextChanged& InOnVerifyRename);

		/** Returns how the tag is displayed in the rig hierarchy tree */
		const ERigHierarchyConnectorTagDisplayMode GetTagDisplayMode() const { return TagDisplayMode; }

	private:
		/** The connector element keys added to this tag */
		TSet<FRigElementKey> ConnectorElementKeys;

		/** Defines how tags are displayed in the specific rig hierarchy tree */
		const ERigHierarchyConnectorTagDisplayMode TagDisplayMode;
	};


	/** A warning tag for a connector that cannot be resolved in a rig hierarchy */
	struct FRigHierarchyConnectorUnresolvedWarningTag final
		: public FRigHierarchyTagModelBase
	{
		FRigHierarchyConnectorUnresolvedWarningTag(
			const UControlRig& ControlRig,
			const FRigHierarchyTreeDisplaySettings& DisplaySettings,
			const FRigConnectorElement& ConnectorElement);
	};
}
