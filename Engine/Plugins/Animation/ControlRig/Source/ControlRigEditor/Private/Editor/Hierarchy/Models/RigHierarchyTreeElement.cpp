// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyTreeElement.h"

#include "Editor/Hierarchy/Widgets/SRigHierarchyItem.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Framework/Application/SlateApplication.h"
#include "Rigs/RigHierarchy.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Widgets/Views/ITableRow.h"

FRigHierarchyTreeElement::FRigHierarchyTreeElement(const FRigHierarchyKey& InKey, TWeakPtr<SRigHierarchyTreeView> InTreeView, bool InSupportsRename, ERigTreeFilterResult InFilterResult)
{
	Key = InKey;
	ShortName = LongName = FText::FromName(InKey.GetFName());
	ChannelName = NAME_None;
	bIsTransient = false;
	bIsAnimationChannel = false;
	bIsProcedural = false;
	bSupportsRename = InSupportsRename;
	FilterResult = InFilterResult;
	bFadedOutDuringDragDrop = false;

	if (InTreeView.IsValid())
	{
		if (const URigHierarchy* Hierarchy = InTreeView.Pin()->GetRigTreeDelegates().GetHierarchy())
		{
			if (InKey.IsElement())
			{
				LongName = Hierarchy->GetDisplayNameForUI(InKey.GetElement(), EElementNameDisplayMode::ForceLong);
				ShortName = Hierarchy->GetDisplayNameForUI(InKey.GetElement(), EElementNameDisplayMode::ForceShort);
			}

			const FRigHierarchyTreeDisplaySettings& Settings = InTreeView.Pin()->GetRigTreeDelegates().GetDisplaySettings();
			RefreshDisplaySettings(Hierarchy, Settings);
		}
	}
}


TSharedRef<ITableRow> FRigHierarchyTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigHierarchyTreeElement> InRigTreeElement, TSharedPtr<SRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned)
{
	return SNew(SRigHierarchyItem, InOwnerTable, InRigTreeElement, InTreeView, InSettings, bPinned);
}

void FRigHierarchyTreeElement::RequestRename()
{
	if (bSupportsRename)
	{
		OnRenameRequested.ExecuteIfBound();
	}
}

void FRigHierarchyTreeElement::RefreshDisplaySettings(const URigHierarchy* InHierarchy, const FRigHierarchyTreeDisplaySettings& InSettings)
{
	const TPair<const FSlateBrush*, FSlateColor> Result = SRigHierarchyItem::GetBrushForElementType(InHierarchy, Key);

	bIsProcedural = false;
	if (Key.IsElement())
	{
		if (const FRigBaseElement* Element = InHierarchy->Find(Key.GetElement()))
		{
			bIsProcedural = Element->IsProcedural();

			if (const FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
			{
				bIsTransient = ControlElement->Settings.bIsTransientControl;
				bIsAnimationChannel = ControlElement->IsAnimationChannel();
				if (bIsAnimationChannel)
				{
					ChannelName = ControlElement->GetDisplayName();
				}
			}
		}
	}
	else
	{
		bIsProcedural = InHierarchy->IsProcedural(Key.GetComponent());
	}

	IconBrush = Result.Key;
	IconColor = Result.Value;
	if (IconColor.IsColorSpecified() && InSettings.bShowIconColors)
	{
		IconColor = FilterResult == ERigTreeFilterResult::Shown ? Result.Value : FSlateColor(Result.Value.GetSpecifiedColor() * 0.5f);
	}
	else
	{
		IconColor = FilterResult == ERigTreeFilterResult::Shown ? FSlateColor::UseForeground() : FSlateColor(FLinearColor::Gray * 0.5f);
	}

	TextColor = FilterResult == ERigTreeFilterResult::Shown ?
		(bIsProcedural ? FSlateColor(FLinearColor(0.9f, 0.8f, 0.4f)) : FSlateColor::UseForeground()) :
		(bIsProcedural ? FSlateColor(FLinearColor(0.9f, 0.8f, 0.4f) * 0.5f) : FSlateColor(FLinearColor::Gray * 0.5f));
}

FSlateColor FRigHierarchyTreeElement::GetIconColor() const
{
	if (bFadedOutDuringDragDrop)
	{
		if (FSlateApplication::Get().IsDragDropping())
		{
			return IconColor.GetColor(FWidgetStyle()) * 0.3f;
		}
	}
	return IconColor;
}

FSlateColor FRigHierarchyTreeElement::GetTextColor() const
{
	if (bFadedOutDuringDragDrop)
	{
		if (FSlateApplication::Get().IsDragDropping())
		{
			return TextColor.GetColor(FWidgetStyle()) * 0.3f;
		}
	}
	return TextColor;
}
