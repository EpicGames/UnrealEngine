// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "RigHierarchyTagModel.h"
#include "Styling/SlateColor.h"

class ITableRow;
class SRigHierarchyTreeView;
class STableViewBase;
class URigHierarchy;
struct FRigHierarchyKey;
struct FSlateBrush;

/**
 * Order is important here!
 * This enum is used internally to the filtering logic and represents an ordering of most filtered (hidden) to
 * least filtered (highlighted).
 */
enum class ERigTreeFilterResult : int32
{
	/** Hide the item */
	Hidden,

	/** Show the item because child items were shown */
	ShownDescendant,

	/** Show the item */
	Shown,
};

/** An item in the tree */
class FRigHierarchyTreeElement : public TSharedFromThis<FRigHierarchyTreeElement>
{
	using FRigHierarchyValidConnectorTag = UE::ControlRigEditor::FRigHierarchyValidConnectorTag;
	using FRigHierarchyConnectorUnresolvedWarningTag = UE::ControlRigEditor::FRigHierarchyConnectorUnresolvedWarningTag;

public:
	FRigHierarchyTreeElement(const FRigHierarchyKey& InKey, TWeakPtr<SRigHierarchyTreeView> InTreeView, bool InSupportsRename, ERigTreeFilterResult InFilterResult);

	/** Returns the child elements of this element */
	const TArray<TSharedPtr<FRigHierarchyTreeElement>>& GetChildren() const { return Children; }


public:
	/** Element Data to display */
	FRigHierarchyKey Key;
	FText LongName;
	FText ShortName;
	FName ChannelName;
	bool bIsTransient;
	bool bIsAnimationChannel;
	bool bIsProcedural;
	bool bSupportsRename;
	TArray<TSharedPtr<FRigHierarchyTreeElement>> Children;

	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigHierarchyTreeElement> InRigTreeElement, TSharedPtr<SRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned);

	void RequestRename();

	void RefreshDisplaySettings(const URigHierarchy* InHierarchy, const FRigHierarchyTreeDisplaySettings& InSettings);

	FSlateColor GetIconColor() const;
	FSlateColor GetTextColor() const;

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

	/** The current filter result */
	ERigTreeFilterResult FilterResult;

	/** The brush to use when rendering an icon */
	const FSlateBrush* IconBrush;;

	/** The color to use when rendering an icon */
	FSlateColor IconColor;

	/** The color to use when rendering the label text */
	FSlateColor TextColor;

	/** If true the item is filtered out during a drag */
	bool bFadedOutDuringDragDrop;

	/** Tags for valid connectors for this element */
	TArray<FRigHierarchyValidConnectorTag> ConnectorTags;

	/** Tags for invalid connectors for this element */
	TArray<FRigHierarchyConnectorUnresolvedWarningTag> ConnectorResolveWarningTags;
};
