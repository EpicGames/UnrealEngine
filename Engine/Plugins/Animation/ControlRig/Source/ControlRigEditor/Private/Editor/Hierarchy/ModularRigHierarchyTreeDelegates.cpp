// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigHierarchyTreeDelegates.h"

const UModularRig* FModularRigHierarchyTreeDelegates::GetModularRig() const
{
	if (OnGetModularRig.IsBound())
	{
		return OnGetModularRig.Execute();
	}
	return nullptr;
}

bool FModularRigHierarchyTreeDelegates::HandleResolveConnector(const FRigElementKey& InConnector, const TArray<FRigElementKey>& InTargets)
{
	if (OnResolveConnector.IsBound())
	{
		OnResolveConnector.Execute(InConnector, InTargets);
		return true;
	}
	return false;
}

bool FModularRigHierarchyTreeDelegates::HandleDisconnectConnector(const FRigElementKey& InConnector)
{
	if (OnDisconnectConnector.IsBound())
	{
		OnDisconnectConnector.Execute(InConnector);
		return true;
	}
	return false;
}

void FModularRigHierarchyTreeDelegates::HandleSelectionChanged(TSharedPtr<FModularRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	if (bSuspendSelectionDelegate)
	{
		return;
	}
	TGuardValue<bool> Guard(bSuspendSelectionDelegate, true);
	(void)OnSelectionChanged.ExecuteIfBound(Selection, SelectInfo);
}

bool FModularRigHierarchyTreeDelegates::ShouldAlwaysShowConnector(const FName& InConnectorName) const
{
	if (OnAlwaysShowConnector.IsBound())
	{
		return OnAlwaysShowConnector.Execute(InConnectorName);
	}
	return false;
}
