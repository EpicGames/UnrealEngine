// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

DECLARE_DELEGATE_RetVal(const UModularRig*, FOnGetModularRigTreeRig);
DECLARE_DELEGATE_TwoParams(FOnModularRigTreeResolveConnector, const FRigElementKey& /*Connector*/, const TArray<FRigElementKey>& /*Targets*/);
DECLARE_DELEGATE_OneParam(FOnModularRigTreeDisconnectConnector, const FRigElementKey& /*Connector*/);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnModularRigTreeAlwaysShowConnector, const FName& /*ConnectorName*/);

typedef STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>::FOnMouseButtonClick FOnModularRigTreeMouseButtonClick;
typedef STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>::FOnMouseButtonDoubleClick FOnModularRigTreeMouseButtonDoubleClick;
typedef STableRow<TSharedPtr<FModularRigHierarchyTreeElement>>::FOnCanAcceptDrop FOnModularRigTreeCanAcceptDrop;
typedef STableRow<TSharedPtr<FModularRigHierarchyTreeElement>>::FOnAcceptDrop FOnModularRigTreeAcceptDrop;
typedef STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>::FOnSelectionChanged FOnModularRigTreeSelectionChanged;

struct FModularRigHierarchyTreeDelegates
{
public:

	FOnGetModularRigTreeRig OnGetModularRig;
	FOnModularRigTreeMouseButtonClick OnMouseButtonClick;
	FOnModularRigTreeMouseButtonDoubleClick OnMouseButtonDoubleClick;
	FOnDragDetected OnDragDetected;
	FOnModularRigTreeCanAcceptDrop OnCanAcceptDrop;
	FOnModularRigTreeAcceptDrop OnAcceptDrop;
	FOnContextMenuOpening OnContextMenuOpening;
	FOnModularRigTreeResolveConnector OnResolveConnector;
	FOnModularRigTreeDisconnectConnector OnDisconnectConnector;
	FOnModularRigTreeSelectionChanged OnSelectionChanged;
	FOnModularRigTreeAlwaysShowConnector OnAlwaysShowConnector;

	const UModularRig* GetModularRig() const;
	bool HandleResolveConnector(const FRigElementKey& InConnector, const TArray<FRigElementKey>& InTargets);
	bool HandleDisconnectConnector(const FRigElementKey& InConnector);
	void HandleSelectionChanged(TSharedPtr<FModularRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo);
	bool ShouldAlwaysShowConnector(const FName& InConnectorName) const;

private:

	bool bSuspendSelectionDelegate = false;

	friend class SModularRigHierarchyTreeView;
};
