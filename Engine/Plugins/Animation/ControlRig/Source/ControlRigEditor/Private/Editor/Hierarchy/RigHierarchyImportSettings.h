// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigHierarchyImportSettings.generated.h"

class USkeletalMesh;
template<typename T> struct TObjectPtr;

USTRUCT()
struct FRigHierarchyImportSettings
{
	GENERATED_BODY()

	FRigHierarchyImportSettings()
		: Mesh(nullptr)
	{
	}

	UPROPERTY(EditAnywhere, Category = "Hierarchy Import")
	TObjectPtr<USkeletalMesh> Mesh;
};
