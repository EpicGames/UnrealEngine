// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyTreeDelegates.h"

#include "Editor/Hierarchy/Models/RigHierarchyTagModel.h"

FRigHierarchyTreeDisplaySettings FRigHierarchyTreeDelegates::DefaultDisplaySettings;

URigHierarchy* FRigHierarchyTreeDelegates::GetHierarchy()
{
	if (OnGetHierarchy.IsBound())
	{
		return const_cast<URigHierarchy*>(OnGetHierarchy.Execute());
	}
	return nullptr;
}

const URigHierarchy* FRigHierarchyTreeDelegates::GetHierarchy() const
{
	if (OnGetHierarchy.IsBound())
	{
		return OnGetHierarchy.Execute();
	}
	return nullptr;
}

const FRigHierarchyTreeDisplaySettings& FRigHierarchyTreeDelegates::GetDisplaySettings() const
{
	if (OnGetDisplaySettings.IsBound())
	{
		return OnGetDisplaySettings.Execute();
	}
	return DefaultDisplaySettings;
}

TArray<FRigHierarchyKey> FRigHierarchyTreeDelegates::GetSelection() const
{
	if (OnGetSelection.IsBound())
	{
		return OnGetSelection.Execute();
	}
	if (const URigHierarchy* Hierarchy = GetHierarchy())
	{
		return Hierarchy->GetSelectedHierarchyKeys();
	}
	return {};
}

FName FRigHierarchyTreeDelegates::HandleRenameElement(const FRigHierarchyKey& OldKey, const FString& NewName) const
{
	if (OnRenameElement.IsBound())
	{
		return OnRenameElement.Execute(OldKey, NewName);
	}
	return OldKey.GetFName();
}

bool FRigHierarchyTreeDelegates::HandleVerifyElementNameChanged(const FRigHierarchyKey& OldKey, const FString& NewName, FText& OutErrorMessage) const
{
	if (OnVerifyElementNameChanged.IsBound())
	{
		return OnVerifyElementNameChanged.Execute(OldKey, NewName, OutErrorMessage);
	}
	return false;
}

void FRigHierarchyTreeDelegates::HandleSelectionChanged(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}
	TGuardValue<bool> Guard(bIsChangingRigHierarchy, true);
	(void)OnSelectionChanged.ExecuteIfBound(Selection, SelectInfo);
}

FRigHierarchyKey FRigHierarchyTreeDelegates::GetResolvedKey(const FRigHierarchyKey& InKey)
{
	if (OnGetResolvedKey.IsBound())
	{
		return OnGetResolvedKey.Execute(InKey);
	}
	return InKey;
}

void FRigHierarchyTreeDelegates::RequestDetailsInspection(const FRigHierarchyKey& InKey)
{
	if (OnRequestDetailsInspection.IsBound())
	{
		return OnRequestDetailsInspection.Execute(InKey);
	}
}

UE::ControlRigEditor::ERigHierarchyConnectorTagDisplayMode FRigHierarchyTreeDelegates::GetTagDisplayMode() const
{
	if (OnGetTagDisplayMode.IsBound())
	{
		return OnGetTagDisplayMode.Execute();
	}

	return UE::ControlRigEditor::ERigHierarchyConnectorTagDisplayMode::Individual;
}
