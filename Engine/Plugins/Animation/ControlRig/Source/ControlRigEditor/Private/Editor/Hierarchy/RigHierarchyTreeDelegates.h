// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Editor/Hierarchy/RigHierarchyTreeDisplaySettings.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTagWidget.h"
#include "Framework/SlateDelegates.h"
#include "Rigs/RigHierarchy.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

class FRigHierarchyTreeElement;
class URigHierarchy;
namespace UE::ControlRigEditor { enum class ERigHierarchyConnectorTagDisplayMode : uint8; }

typedef STableRow<TSharedPtr<FRigHierarchyTreeElement>>::FOnCanAcceptDrop FOnRigTreeCanAcceptDrop;
typedef STableRow<TSharedPtr<FRigHierarchyTreeElement>>::FOnAcceptDrop FOnRigTreeAcceptDrop;
typedef STreeView<TSharedPtr<FRigHierarchyTreeElement>>::FOnSelectionChanged FOnRigTreeSelectionChanged;
typedef STreeView<TSharedPtr<FRigHierarchyTreeElement>>::FOnMouseButtonClick FOnRigTreeMouseButtonClick;
typedef STreeView<TSharedPtr<FRigHierarchyTreeElement>>::FOnMouseButtonDoubleClick FOnRigTreeMouseButtonDoubleClick;
typedef STreeView<TSharedPtr<FRigHierarchyTreeElement>>::FOnSetExpansionRecursive FOnRigTreeSetExpansionRecursive;

DECLARE_DELEGATE_RetVal(const URigHierarchy*, FOnGetRigTreeHierarchy);
DECLARE_DELEGATE_RetVal(const FRigHierarchyTreeDisplaySettings&, FOnGetRigTreeDisplaySettings);
DECLARE_DELEGATE_RetVal(const TArray<FRigHierarchyKey>, FOnRigTreeGetSelection);
DECLARE_DELEGATE_RetVal_TwoParams(FName, FOnRigTreeRenameElement, const FRigHierarchyKey& /*OldKey*/, const FString& /*NewName*/);
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnRigTreeVerifyElementNameChanged, const FRigHierarchyKey& /*OldKey*/, const FString& /*NewName*/, FText& /*OutErrorMessage*/);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnRigTreeCompareKeys, const FRigHierarchyKey& /*A*/, const FRigHierarchyKey& /*B*/);
DECLARE_DELEGATE_RetVal_OneParam(FRigHierarchyKey, FOnRigTreeGetResolvedKey, const FRigHierarchyKey&);
DECLARE_DELEGATE_OneParam(FOnRigTreeRequestDetailsInspection, const FRigHierarchyKey&);
DECLARE_DELEGATE_RetVal_OneParam(TOptional<FText>, FOnRigTreeItemGetToolTip, const FRigHierarchyKey&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnRigTreeIsItemVisible, const FRigHierarchyKey&);
DECLARE_DELEGATE_RetVal(UE::ControlRigEditor::ERigHierarchyConnectorTagDisplayMode, FOnRigTreeGetTagDisplayMode);

#define UE_API CONTROLRIGEDITOR_API

struct FRigHierarchyTreeDelegates
{
	FOnGetRigTreeHierarchy OnGetHierarchy;
	FOnGetRigTreeDisplaySettings OnGetDisplaySettings;
	FOnRigTreeRenameElement OnRenameElement;
	FOnRigTreeVerifyElementNameChanged OnVerifyElementNameChanged;
	FOnDragDetected OnDragDetected;
	FOnRigTreeCanAcceptDrop OnCanAcceptDrop;
	FOnRigTreeAcceptDrop OnAcceptDrop;
	FOnRigTreeGetSelection OnGetSelection;
	FOnRigTreeSelectionChanged OnSelectionChanged;
	FOnContextMenuOpening OnContextMenuOpening;
	FOnRigTreeMouseButtonClick OnMouseButtonClick;
	FOnRigTreeMouseButtonDoubleClick OnMouseButtonDoubleClick;
	FOnRigTreeSetExpansionRecursive OnSetExpansionRecursive;
	FOnRigTreeCompareKeys OnCompareKeys;
	FOnRigTreeGetResolvedKey OnGetResolvedKey;
	FOnRigTreeRequestDetailsInspection OnRequestDetailsInspection;
	FOnRigTreeElementKeyTagDragDetected OnRigTreeElementKeyTagDragDetected;
	FOnRigTreeItemGetToolTip OnRigTreeGetItemToolTip;
	FOnRigTreeIsItemVisible OnRigTreeIsItemVisible;
	FOnRigTreeGetTagDisplayMode OnGetTagDisplayMode;

	URigHierarchy* GetHierarchy();
	const URigHierarchy* GetHierarchy() const;
	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings() const;
	TArray<FRigHierarchyKey> GetSelection() const;
	FName HandleRenameElement(const FRigHierarchyKey& OldKey, const FString& NewName) const;
	bool HandleVerifyElementNameChanged(const FRigHierarchyKey& OldKey, const FString& NewName, FText& OutErrorMessage) const;
	void HandleSelectionChanged(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo);
	FRigHierarchyKey GetResolvedKey(const FRigHierarchyKey& InKey);
	void RequestDetailsInspection(const FRigHierarchyKey& InKey);
	UE::ControlRigEditor::ERigHierarchyConnectorTagDisplayMode GetTagDisplayMode() const;

	static UE_API FRigHierarchyTreeDisplaySettings DefaultDisplaySettings;

private:
	bool bIsChangingRigHierarchy = false;
};

#undef UE_API
