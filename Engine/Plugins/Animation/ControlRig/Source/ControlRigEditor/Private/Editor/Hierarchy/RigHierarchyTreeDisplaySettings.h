// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyDefines.h"
#include "Settings/ControlRigSettings.h"

struct FRigHierarchyTreeDisplaySettings
{
	FRigHierarchyTreeDisplaySettings()
	{
		FilterText = FText();

		bFlattenHierarchyOnFilter = false;
		bHideParentsOnFilter = false;
		NameDisplayMode = EElementNameDisplayMode::AssetDefault;
		bShowImportedBones = true;
		bShowBones = true;
		bShowControls = true;
		bShowNulls = true;
		bShowReferences = true;
		bShowSockets = true;
		bShowConnectors = true;
		bShowComponents = false;
		bShowIconColors = true;
		bArrangeByModules = false;
		bFlattenModules = false;
		OutlinerDisplayMode = EMultiRigTreeDisplayMode::All;
	}

	FText FilterText;

	/** Flatten when text filtering is active */
	bool bFlattenHierarchyOnFilter;

	/** Hide parents when text filtering is active */
	bool bHideParentsOnFilter;

	/** The mode used to determine how names are displayed */
	EElementNameDisplayMode NameDisplayMode;

	/** Whether or not to show imported bones in the hierarchy */
	bool bShowImportedBones;

	/** Whether or not to show bones in the hierarchy */
	bool bShowBones;

	/** Whether or not to show controls in the hierarchy */
	bool bShowControls;

	/** Whether or not to show spaces in the hierarchy */
	bool bShowNulls;

	/** Whether or not to show references in the hierarchy */
	bool bShowReferences;

	/** Whether or not to show sockets in the hierarchy */
	bool bShowSockets;

	/** Whether or not to show connectors in the hierarchy */
	bool bShowConnectors;

	/** Whether or not to show components in the hierarchy */
	bool bShowComponents;

	/** Whether to tint the icons with the element color */
	bool bShowIconColors;

	/** Whether or not to arrange the controls into modules (only for modular rigs) */
	bool bArrangeByModules;

	/** Whether or not to arrange the modules in a flat list (only for modular rigs) */
	bool bFlattenModules;

	/** Whether or not to expand and focus the selection when this changes */
	bool bFocusOnSelection;

	/** Identify which rigs to display based on selection */
	EMultiRigTreeDisplayMode OutlinerDisplayMode;
};
