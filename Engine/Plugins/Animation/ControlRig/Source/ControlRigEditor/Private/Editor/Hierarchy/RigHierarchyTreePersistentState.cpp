// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyTreePersistentState.h"

#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"

namespace UE::ControlRigEditor::RigHierarchyTreePersistentStatePrivate
{
	FRigHierarchElementPersistentStateKey::FRigHierarchElementPersistentStateKey(const TSharedRef<FRigHierarchyTreeElement>& Element)
	{
		VariantKey.Set<FRigHierarchyKey>(Element->Key);
	}

	FRigHierarchElementPersistentStateKey::FRigHierarchElementPersistentStateKey(const TSharedRef<FModularRigHierarchyTreeElement>& Element)
	{
		VariantKey.Set<FString>(Element->GetKey());
	}

	bool FRigHierarchElementPersistentStateKey::operator==(const FRigHierarchElementPersistentStateKey& Other) const
	{
		if (VariantKey.GetIndex() != Other.VariantKey.GetIndex())
		{
			return false;
		}

		if (VariantKey.IsType<FRigHierarchyKey>())
		{
			return VariantKey.Get<FRigHierarchyKey>() == Other.VariantKey.Get<FRigHierarchyKey>();
		}
		else if (VariantKey.IsType<FString>())
		{
			return VariantKey.Get<FString>() == Other.VariantKey.Get<FString>();
		}
		else
		{
			ensureMsgf(0, TEXT("Unhandled variant"));
			return false;
		}
	}

	bool FRigHierarchElementPersistentStateKey::operator!=(const FRigHierarchElementPersistentStateKey& Other) const
	{
		return !(*this == Other);
	}

	uint32 GetTypeHash(const FRigHierarchElementPersistentStateKey& Key)
	{
		if (const FRigHierarchyKey* KeyPtr = Key.VariantKey.TryGet<FRigHierarchyKey>())
		{
			return GetTypeHash(*KeyPtr);
		}
		else if (const FString* StringKeyPtr = Key.VariantKey.TryGet<FString>())
		{
			return GetTypeHash(*StringKeyPtr);
		}
		else
		{
			ensureMsgf(0, TEXT("Unhandled variant"));
			return 0;
		}
	}
}
