// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"
#include "RigHierarchyTreePersistentStateDelegates.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/STreeView.h"

// Only use in FRigHierarchyTreePersistentStateStore
namespace UE::ControlRigEditor
{
	/** Delegate raised when items can be rebuilt during a tree refresh */
	DECLARE_DELEGATE(FRigHierarchyTreePersistentStateRebuildItemsRequested);

	/** Concept that requires TElementType to be a rig hierarchy tree element type */
	template <typename TElementType>
	concept ConceptIsHierarchyElementType = requires
	{
		std::is_same_v<TElementType, TSharedPtr<FRigHierarchyTreeElement>> ||
		std::is_same_v<TElementType, TSharedPtr<FModularRigHierarchyTreeElement>>;
	};

	namespace RigHierarchyTreePersistentStatePrivate
	{
		/** Returns all elements in the tree */
		template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
		TArray<const TElementType> GetAllElementsInTree(const STreeView<TElementType>& TreeView)
		{
			TArray<const TElementType> Result;

			const TFunction<void(TElementType)> AddElementAndChildrenRecursive = [&](const TElementType& Element)
				{
					if (!Element.IsValid())
					{
						return;
					}

					Result.Add(Element);

					for (const TElementType& ChildElement : Element->GetChildren())
					{
						AddElementAndChildrenRecursive(ChildElement);
					}
				};

			const TArrayView<const TElementType> RootElements = TreeView.GetRootItems();
			for (const TElementType& RootElement : RootElements)
			{
				AddElementAndChildrenRecursive(RootElement);
			}

			return Result;
		}

		/** Key for the persistent state of a single element in the hierarchy tree */
		struct FRigHierarchElementPersistentStateKey
		{
			FRigHierarchElementPersistentStateKey(const TSharedRef<FRigHierarchyTreeElement>& Element);
			FRigHierarchElementPersistentStateKey(const TSharedRef<FModularRigHierarchyTreeElement>& Element);

			bool operator==(const FRigHierarchElementPersistentStateKey& Other) const;
			bool operator!=(const FRigHierarchElementPersistentStateKey& Other) const;

			friend uint32 GetTypeHash(const FRigHierarchElementPersistentStateKey& Key);

		private:
			/** Variant key for which to store the persistent state */
			TVariant<FRigHierarchyKey, FString> VariantKey;
		};

		/** A persistent state of a specific hierarchy tree */
		struct FRigHierarchyTreePersistentState
			: public TSharedFromThis<FRigHierarchyTreePersistentState>
		{
		public:
			/** Returns true while this state was stored for a new instance of the tree */
			bool IsPersistent() const { return bIsPersistent; }

			/** Returns true while this state was stored but not restored yet  */
			bool IsPendingRestore() const { return bIsPendingRestore; }

			/** Initializes the state while a tree is being destroyed, so it can be restored in a new instance */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			void Store(
				const UControlRig& ControlRig,
				const STreeView<TElementType>& TreeView,
				const FName& ViewName)
			{
				bIsPersistent = true;

				StoreStateInternal(ControlRig, TreeView, ViewName);
			}

			/** Restores the state for the specified tree view. This can be a new instance of the tree view. */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			void RestoreState(
				STreeView<TElementType>& TreeView,
				const FRigHierarchyTreePersistentStateRebuildItemsRequested& OnRebuildItems,
				const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored) const
			{
				// Let the user rebuild items before restoring
				OnRebuildItems.ExecuteIfBound();

				const TArray<const TElementType> AllElements = GetAllElementsInTree(TreeView);

				for (const TElementType& Element : AllElements)
				{
					if (const bool* bExpandedPtr = GetElementExpansionState(Element))
					{
						TreeView.SetItemExpansion(Element, *bExpandedPtr);
					}
					else
					{
						// By default expand new items
						TreeView.SetItemExpansion(Element, true);
					}

					if (const bool* bSelectedPtr = GetElementSelectionState(Element))
					{
						// Restore selection 
						TreeView.SetItemSelection(Element, *bSelectedPtr, ESelectInfo::Direct);
					}
				}
				
				TreeView.RequestTreeRefresh();

				TreeView.SetScrollOffset(ScrollOffset);

				OnStateRestored.ExecuteIfBound();
			}

			/** Retains the state of the tree while it is being refreshed. Restoration can be deferred by calling DeferRestoreState */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			void RetainDuringRefresh(
				const UControlRig& ControlRig,
				const TSharedRef<STreeView<TElementType>>& TreeView,
				const FName& ViewName,
				const FRigHierarchyTreePersistentStateRebuildItemsRequested& OnRebuildItems,
				const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored)
			{
				bIsPendingRestore = true;

				StoreStateInternal(ControlRig, *TreeView, ViewName);

				const TWeakPtr<STreeView<TElementType>> WeakTreeView = TreeView;
				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
					[OnRebuildItems, OnStateRestored, WeakTreeView, WeakThis = AsWeak(), this](float DelatTime)
					{
						if (!WeakThis.IsValid())
						{
							return false;
						}

						if (!WeakTreeView.IsValid())
						{
							bIsPendingRestore = false;
							return false;
						}

						if (RestoreOnFrame < GFrameNumber)
						{
							RestoreState(*WeakTreeView.Pin(), OnRebuildItems, OnStateRestored);
							bIsPendingRestore = false;

							return false;
						}

						return true;
					})
				);

				DeferRestoreState();
			}

			/** Defers restoring the state to the first frame number where it wasn't deffered */
			void DeferRestoreState()
			{
				// Only restore two frames after, that's at least one tick after the last update, given ticker order is undefined
				RestoreOnFrame = GFrameNumber + 2;
			}

		private:
			/** Effectively stores the state */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			void StoreStateInternal(
				const UControlRig& ControlRig,
				const STreeView<TElementType>& TreeView,
				const FName& ViewName)
			{
				ScrollOffset = TreeView.GetScrollOffset();

				const TArray<const TElementType> AllElements = GetAllElementsInTree(TreeView);
				for (const TElementType& Element : AllElements)
				{
					StoreElementExpansionState(Element, TreeView.IsItemExpanded(Element));
					StoreElementSelectionState(Element, TreeView.IsItemSelected(Element));
				}
			}

			/** Stores the expansion state of the element in this persistent state */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			void StoreElementExpansionState(const TElementType& Element, bool bExpanded)
			{
				if (Element.IsValid())
				{
					const FRigHierarchElementPersistentStateKey Key(Element.ToSharedRef());
					KeyToExpansionStateMap.Add(Key, bExpanded);
				}
			}

			/** Stores the selection state of the element in this persistent state */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			void StoreElementSelectionState(const TElementType& Element, bool bExpanded)
			{
				if (Element.IsValid())
				{
					const FRigHierarchElementPersistentStateKey Key(Element.ToSharedRef());
					KeyToSelectionStateMap.Add(Key, bExpanded);
				}
			}

			/** Gets the expansion state for the element. Note, the element is compared by key, not by ptr to ensure persistency. */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			const bool* GetElementExpansionState(const TElementType& Element) const
			{
				if (Element.IsValid())
				{
					const FRigHierarchElementPersistentStateKey Key(Element.ToSharedRef());
					return KeyToExpansionStateMap.Find(Key);
				}

				return nullptr;
			}

			/** Gets the selection state for the element. Note, the element is compared by key, not by ptr to ensure persistency. */
			template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
			const bool* GetElementSelectionState(const TElementType& Element) const
			{
				if (Element.IsValid())
				{
					const FRigHierarchElementPersistentStateKey Key(Element.ToSharedRef());
					return KeyToSelectionStateMap.Find(Key);
				}

				return nullptr;
			}

			/** The scroll offset */
			float ScrollOffset = 0.f;

			/** The frame on which this state can be restored */
			uint32 RestoreOnFrame = 0;

			/** True while this state was persists for a new instance of the tree */
			bool bIsPersistent = false;

			/** True while this state was stored but not restored yet  */
			bool bIsPendingRestore = false;

			/** A map of hierarchy keys and their expansion state in the tree */
			TMap<FRigHierarchElementPersistentStateKey, bool> KeyToExpansionStateMap;

			/** A map of hierarchy keys and their selection state in the tree */
			TMap<FRigHierarchElementPersistentStateKey, bool> KeyToSelectionStateMap;
		};
	}
}
