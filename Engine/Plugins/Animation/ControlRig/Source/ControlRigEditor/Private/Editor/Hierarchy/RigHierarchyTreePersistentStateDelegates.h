// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::ControlRigEditor
{
	/** Delegate raised when the persistent state of a tree view was restored */
	DECLARE_DELEGATE(FOnRigHierarchyTreePersistentStateRestored);
}
