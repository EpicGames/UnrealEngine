// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigHierarchyTreePersistentStateStore.h"

#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"


namespace UE::ControlRigEditor
{
	TAutoConsoleVariable<bool> CVarControlRigHierarchyTreeUsePersistentStateStore(
		TEXT("ControlRig.Editor.EnableRigHierarchyTreePersistentState"),
		true,
		TEXT("When set to true the hierarchy stores and recalls its state during the lifetime of a control rig asset editor whenever it's reconstructed."));

	TSharedPtr<FRigHierarchyTreePersistentStateStore> FRigHierarchyTreePersistentStateStore::Instance;

	FRigHierarchyTreePersistentStateStore& FRigHierarchyTreePersistentStateStore::Get()
	{
		if (!Instance.IsValid())
		{
			Instance = MakeShared<FRigHierarchyTreePersistentStateStore>();
		}

		return *Instance;
	}

	bool FRigHierarchyTreePersistentStateStore::IsFeatureEnabled()
	{
		return CVarControlRigHierarchyTreeUsePersistentStateStore.GetValueOnGameThread();
	}
}
