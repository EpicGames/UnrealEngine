// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRig.h"
#include "Editor/ControlRigEditor.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentState.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateDelegates.h"

class FModularRigHierarchyTreeElement;
class FName;
class FRigHierarchyTreeElement;

namespace UE::ControlRigEditor
{	
	/** 
	 * Stores the state of a rig hierarchy tree view persistently, so it can be restored from a possibly destructed instance.
	 * 
	 * During their lifetime hierarchy tree views can call FRigHierarchyTreePersistentStateStore::RequestTreeRefresh to
	 * retain their state. Refresh will occur one tick after no RequestTreeRefresh call occured.
	 * 
	 * Hierarchy tree views can call FRigHierarchyTreePersistentStateStore::StorePersistentState anytime to store their state 
	 * persistently without refreshing.
	 */
	class FRigHierarchyTreePersistentStateStore
		: public TSharedFromThis<FRigHierarchyTreePersistentStateStore>
	{
		// Allow make shared with private constructor
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;

		using FRigHierarchyTreePersistentState = UE::ControlRigEditor::RigHierarchyTreePersistentStatePrivate::FRigHierarchyTreePersistentState;

	public:
		/** Gets the store instance */
		static FRigHierarchyTreePersistentStateStore& Get(); 

		/** 
		 * Retains the of a rig hierarchy tree while it is being destroyed,
		 * so it can be restored when it was reinstantiated and refreshed.
		 */
		template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
		void StorePersistentState(
			const UControlRig& ControlRig,
			STreeView<TElementType>& TreeView,
			const FName& ViewName)
		{
			if (!IsFeatureEnabled())
			{
				return;
			}

			const TOptional<FName> OptionalId = MakeID(ControlRig, ViewName);
			if (!OptionalId.IsSet())
			{
				return;
			}
			const FName& Id = OptionalId.GetValue();

			// Always store this state to restore it when a new instance is refreshed
			const TSharedRef<FRigHierarchyTreePersistentState> PersistentState = MakeShared<FRigHierarchyTreePersistentState>();
			PersistentState->Store(ControlRig, TreeView, ViewName);

			IdToPersistentStateMap.Add(Id, PersistentState);
		}

		/** 
		 * Refreshes the tree on the first tick after this function was last called, 
		 * possibly skipping over many ticks, e.g. during compiling where the data is reconstructed.
		 * 
		 * If this is a new tree and a previous state was stored persistently, the persistent state is recalled.
		 */
		template <typename TElementType> requires ConceptIsHierarchyElementType<TElementType>
		void RequestTreeRefresh(
			const UControlRig& ControlRig,
			const TSharedRef<STreeView<TElementType>>& TreeView,
			const FName& ViewName, 
			const TSharedPtr<IControlRigBaseEditor>& ControlRigEditor = nullptr,
			const FRigHierarchyTreePersistentStateRebuildItemsRequested& OnRebuildItems = FRigHierarchyTreePersistentStateRebuildItemsRequested(),
			const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored = FOnRigHierarchyTreePersistentStateRestored())
		{
			if (!ensureMsgf(IsFeatureEnabled(), TEXT("FRigHierarchyTreePersistentStateStore::RequestTreeRefresh should not be called when its related cvar is disabled")))
			{
				return;
			}

			const TOptional<FName> OptionalId = MakeID(ControlRig, ViewName);
			if (!OptionalId.IsSet())
			{
				return;
			}
			const FName& Id = OptionalId.GetValue();

			const TSharedRef<FRigHierarchyTreePersistentState>* PersistentStatePtr = IdToPersistentStateMap.Find(Id);
			if (PersistentStatePtr && (*PersistentStatePtr)->IsPersistent())
			{
				TSharedPtr<TGuardValue<bool>> SuspendDetailsPanelRefreshGuard;
				if (ControlRigEditor.IsValid())
				{
					SuspendDetailsPanelRefreshGuard = MakeShared<TGuardValue<bool>>(ControlRigEditor->GetSuspendDetailsPanelRefreshFlag(), true);
				}

				// Always restore persistent states
				(*PersistentStatePtr)->RestoreState(*TreeView, OnRebuildItems, OnStateRestored);

				IdToPersistentStateMap.Remove(Id);
			}
			else if (PersistentStatePtr && (*PersistentStatePtr)->IsPendingRestore())
			{
				// Defer the frame on which this state can be restored
				(*PersistentStatePtr)->DeferRestoreState();
			}
			else
			{
				// There is no persistent or pending state, retain the current one
				const TSharedRef<FRigHierarchyTreePersistentState> PersistentState = MakeShared<FRigHierarchyTreePersistentState>();
				PersistentState->RetainDuringRefresh(ControlRig, TreeView, ViewName, OnRebuildItems, OnStateRestored);

				IdToPersistentStateMap.Add(Id, PersistentState);
			}
		}

		/** Returns true if the CVar to enable this store is set */
		static bool IsFeatureEnabled();

	private:
		/** Private on purpose, instead use FRigHierarchyTreePersistentStateStore::Get */
		FRigHierarchyTreePersistentStateStore() = default;

		/** Creates a persistent ID for a specific view, consisting in the form of AssetPath.ViewName */
		TOptional<FName> MakeID(const UControlRig& ControlRig, const FName& ViewName) const
		{
			const UObject* AssetObject = ControlRig.GetAssetReference().GetEditorAsset();
			if (AssetObject)
			{
				return FName(*(FString::Printf(TEXT("%s.%s"), *AssetObject->GetPathName(), *ViewName.ToString())));
			}

			return TOptional<FName>();
		}

		/** Map of IDs in the form of AssetPath.ViewName to the persistent state */
		TMap<FName, TSharedRef<FRigHierarchyTreePersistentState>> IdToPersistentStateMap;

		/** Static instance */
		static TSharedPtr<FRigHierarchyTreePersistentStateStore> Instance;
	};
}
