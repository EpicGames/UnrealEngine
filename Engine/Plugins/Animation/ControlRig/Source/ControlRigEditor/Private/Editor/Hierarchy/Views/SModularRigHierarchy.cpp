// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModularRigHierarchy.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "ClassViewerFilter.h"
#include "ContentBrowserModule.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigModularRigCommands.h"
#include "IContentBrowserSingleton.h"
#include "Dialog/SCustomDialog.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor/ControlRigContextMenuContext.h"
#include "Editor/ControlRigEditor.h"
#include "Editor/EditorEngine.h"
#include "Editor/Hierarchy/DragDrop/ModularRigHierarchyElementDragDropOp.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateStore.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyViewModel.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Kismet2/SClassPickerDialog.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Editor/SRigModuleAssetBrowser.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SRigVMBulkEditDialog.h"
#include "Widgets/SRigVMSwapAssetReferencesWidget.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#define LOCTEXT_NAMESPACE "SModularRigHierarchy"

///////////////////////////////////////////////////////////

const FName SModularRigHierarchy::ContextMenuName = TEXT("ControlRigEditor.ModularRigModel.ContextMenu");

void SModularRigHierarchy::Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor, const FName& InViewName)
{
	ViewName = InViewName;
	ViewModel = MakeShared<FModularRigHierarchyViewModel>(InControlRigEditor);

	ControlRigBlueprint = ViewModel->GetControlRigAssetInterface();

	// for deleting, renaming, dragging
	BindCommands();

	CreateContextMenu();

	// setup all delegates for the modular rig model widget
	FModularRigHierarchyTreeDelegates Delegates;
	Delegates.OnGetModularRig = FOnGetModularRigTreeRig::CreateLambda([WeakViewModel = ViewModel.ToWeakPtr()]
		{
			const UModularRig* ModularRig = WeakViewModel.IsValid() ? WeakViewModel.Pin()->GetModularRig() : nullptr;
			return ModularRig;
		});
	Delegates.OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SModularRigHierarchy::CreateContextMenuWidget);
	Delegates.OnDragDetected = FOnDragDetected::CreateSP(this, &SModularRigHierarchy::OnDragDetected);
	Delegates.OnCanAcceptDrop = FOnModularRigTreeCanAcceptDrop::CreateSP(this, &SModularRigHierarchy::OnCanAcceptDrop);
	Delegates.OnAcceptDrop = FOnModularRigTreeAcceptDrop::CreateSP(this, &SModularRigHierarchy::OnAcceptDrop);
	Delegates.OnResolveConnector = FOnModularRigTreeResolveConnector::CreateSP(this, &SModularRigHierarchy::HandleConnectorResolved);
	Delegates.OnDisconnectConnector = FOnModularRigTreeDisconnectConnector::CreateSP(this, &SModularRigHierarchy::HandleConnectorDisconnect);
	Delegates.OnAlwaysShowConnector = FOnModularRigTreeAlwaysShowConnector::CreateSP(this, &SModularRigHierarchy::ShouldAlwaysShowConnector);
	
	const UControlRigEditorSettings* Settings = GetDefault<UControlRigEditorSettings>();

	HeaderRowWidget = SNew(SHeaderRow)
		.CanSelectGeneratedColumn(true)
		.HiddenColumnsList(Settings->ModularRigHierarchyHiddenColums)
		.OnHiddenColumnsListChanged(this, &SModularRigHierarchy::OnHiddenColumnsListChanged)

	+ SHeaderRow::Column(SModularRigHierarchyTreeView::Column_Warnings)
		.ShouldGenerateWidget(true)
		.DefaultLabel(LOCTEXT("WarningsColumnLabel", "Warnings"))
		.DefaultTooltip(LOCTEXT("WarningsColumnTooltip", "Warnings"))
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.FixedWidth(16.f)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(10.0, 10.0))
			.Image(FAppStyle::GetBrush("Icons.Warning"))
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]

	+ SHeaderRow::Column(SModularRigHierarchyTreeView::Column_ModuleName)
		.ShouldGenerateWidget(true)
		.DefaultLabel(LOCTEXT("ModuleNameColumnLabel", "Module"))
		.HAlignCell(HAlign_Left)
		.VAlignCell(VAlign_Center)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.FillWidth(2.f)
		.MinSize(60.f)

	+ SHeaderRow::Column(SModularRigHierarchyTreeView::Column_Connector)
		.DefaultLabel(LOCTEXT("ConnectorColumnLabel", "Connector"))
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.FillWidth(2.f)
		.MinSize(16.f)

	+ SHeaderRow::Column(SModularRigHierarchyTreeView::Column_ModuleClass)
		.DefaultLabel(LOCTEXT("ModuleClassColumnLabel", "Module Class"))
		.HAlignCell(HAlign_Left)
		.VAlignCell(VAlign_Center)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.FillWidth(1.f)
		.MinSize(16.f)

	+ SHeaderRow::Column(SModularRigHierarchyTreeView::Column_ModuleTags)
		.DefaultLabel(LOCTEXT("ModuleTagsColumnLabel", "Module Tags"))
		.HAlignCell(HAlign_Left)
		.VAlignCell(VAlign_Center)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.FillWidth(1.f)
		.MinSize(16.f);	
	
	// Still apply the visibility manually, the header row only handles it once setup here
	const bool bShowConnectorColumn = !Settings->ModularRigHierarchyHiddenColums.Contains(SModularRigHierarchyTreeView::Column_Connector);
	HeaderRowWidget->SetShowGeneratedColumn(SModularRigHierarchyTreeView::Column_Connector, bShowConnectorColumn);
	
	const bool bShowModuleClassColumn = !Settings->ModularRigHierarchyHiddenColums.Contains(SModularRigHierarchyTreeView::Column_ModuleClass);
	HeaderRowWidget->SetShowGeneratedColumn(SModularRigHierarchyTreeView::Column_ModuleClass, bShowModuleClassColumn);
	
	const bool bShowModuleTagsColumn = !Settings->ModularRigHierarchyHiddenColums.Contains(SModularRigHierarchyTreeView::Column_ModuleTags);
	HeaderRowWidget->SetShowGeneratedColumn(SModularRigHierarchyTreeView::Column_ModuleTags, bShowModuleTagsColumn);

	ChildSlot
	[
		SNew(SVerticalBox)
		
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(SComboButton)
			   .ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
			   .ForegroundColor(FSlateColor::UseStyle())
			   .ToolTipText(LOCTEXT("OptionsToolTip", "Open the Options Menu ."))
			   .OnGetMenuContent(this, &SModularRigHierarchy::OnGetOptionsMenu)
			   .ContentPadding(FMargin(1, 0))
			   .ButtonContent()
			   [
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
					.ColorAndOpacity(FSlateColor::UseForeground())
			   ]
			]

			+SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(FilterBox, SSearchBox)
				.OnTextChanged(this, &SModularRigHierarchy::OnFilterTextChanged)
			]
		]
		
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.ShowEffectWhenDisabled(false)
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
				[
					SAssignNew(TreeView, SModularRigHierarchyTreeView, ViewModel.ToSharedRef(), ViewName)
					.HeaderRow(HeaderRowWidget)
					.RigTreeDelegates(Delegates)
					.AutoScrollEnabled(true)
					.FilterText_Lambda([this]() { return FilterText; })
				]
			]
		]
	];

	RefreshTreeView();

	InControlRigEditor->GetKeyDownDelegate().BindLambda(
		[WeakThis = AsWeak(), this](const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
		{
			if (WeakThis.IsValid())
			{
				return OnKeyDown(MyGeometry, InKeyEvent);
			}
			return FReply::Unhandled();
		});
	InControlRigEditor->OnGetViewportContextMenu().BindSP(this, &SModularRigHierarchy::GetContextMenu);
	InControlRigEditor->OnViewportContextMenuCommands().BindSP(this, &SModularRigHierarchy::GetContextMenuCommands);
	InControlRigEditor->OnEditorClosed().AddSP(this, &SModularRigHierarchy::OnEditorClose);

	const FControlRigAssetInterfacePtr ControlRigAssetInterface = InControlRigEditor->GetControlRigAssetInterface();
	if (ControlRigAssetInterface)
	{
		ControlRigAssetInterface->GetRigVMAssetInterface()->OnRefreshEditor().AddSP(this, &SModularRigHierarchy::HandleRefreshEditorFromBlueprint);
		ControlRigAssetInterface->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().AddSP(this, &SModularRigHierarchy::HandleSetObjectBeingDebugged);
		ControlRigAssetInterface->OnModularRigCompiled().AddSP(this, &SModularRigHierarchy::HandlePostCompileModularRigs);
	}

	if(URigHierarchy* Hierarchy = ViewModel->GetHierarchy())
	{
		Hierarchy->OnModified().AddSP(this, &SModularRigHierarchy::OnHierarchyModified);
	}
}

void SModularRigHierarchy::OnEditorClose(IControlRigBaseEditor* InEditor, FControlRigAssetInterfacePtr InBlueprint)
{
	using namespace UE::ControlRigEditor;

	if (InEditor)
	{
		IControlRigBaseEditor* Editor = (IControlRigBaseEditor*)InEditor;  
		Editor->OnGetViewportContextMenu().Unbind();
		Editor->OnViewportContextMenuCommands().Unbind();
		Editor->OnEditorClosed().RemoveAll(this);
	}

	if (InBlueprint)
	{
		InBlueprint->GetRigVMAssetInterface()->OnRefreshEditor().RemoveAll(this);
		InBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().RemoveAll(this);

		InBlueprint->OnModularRigPreCompiled().RemoveAll(this);
		InBlueprint->OnModularRigCompiled().RemoveAll(this);

		if(UModularRigController* ModularRigController = InBlueprint->GetModularRigController())
		{
			ModularRigController->OnModified().RemoveAll(this);
		}
	}

	if (ViewModel.IsValid())
	{
		if(URigHierarchy* Hierarchy = ViewModel->GetHierarchy())
		{
			Hierarchy->OnModified().RemoveAll(this);
		}
	}
}

void SModularRigHierarchy::BindCommands()
{
	// Only once
	if (CommandList.IsValid())
	{
		return;
	}

	CommandList = MakeShared<FUICommandList>();

	const FControlRigModularRigCommands& Commands = FControlRigModularRigCommands::Get();

	CommandList->MapAction(Commands.AddModuleItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleNewItem),
		FCanExecuteAction());

	CommandList->MapAction(Commands.RenameModuleItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleRenameModule),
		FCanExecuteAction());

	CommandList->MapAction(Commands.DeleteModuleItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleDeleteModules),
		FCanExecuteAction());

	CommandList->MapAction(Commands.DuplicateModuleItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleDuplicateModules),
		FCanExecuteAction());

	CommandList->MapAction(Commands.MirrorModuleItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleMirrorModules),
		FCanExecuteAction());

	CommandList->MapAction(Commands.ReresolveModuleItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleReresolveModules),
		FCanExecuteAction());

	CommandList->MapAction(Commands.SwapModuleClassItem,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleSwapClassForModules),
		FCanExecuteAction::CreateSP(this, &SModularRigHierarchy::CanSwapModules));

	CommandList->MapAction(Commands.CopyModuleSettings,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandleCopyModuleSettings),
		FCanExecuteAction::CreateSP(this, &SModularRigHierarchy::CanCopyModuleSettings));

	CommandList->MapAction(Commands.PasteModuleSettings,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::HandlePasteModuleSettings),
		FCanExecuteAction::CreateSP(this, &SModularRigHierarchy::CanPasteModuleSettings));

	CommandList->MapAction(Commands.ToggleShowSecondaryContectors,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::ToggleConnectorVisibilityFlags, EModularRigHierarchyEditorConnectorVisibilityFlags::ShowSecondaryConnectors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SModularRigHierarchy::AreAnyConnectorVisibilityFlagsSet, EModularRigHierarchyEditorConnectorVisibilityFlags::ShowSecondaryConnectors)
	);

	CommandList->MapAction(Commands.ToggleShowOptionalContectors,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::ToggleConnectorVisibilityFlags, EModularRigHierarchyEditorConnectorVisibilityFlags::ShowOptionalConnectors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SModularRigHierarchy::AreAnyConnectorVisibilityFlagsSet, EModularRigHierarchyEditorConnectorVisibilityFlags::ShowOptionalConnectors)
	);

	CommandList->MapAction(Commands.ToggleShowUnresolvedContectors,
		FExecuteAction::CreateSP(this, &SModularRigHierarchy::ToggleConnectorVisibilityFlags, EModularRigHierarchyEditorConnectorVisibilityFlags::ShowUnresolvedConnectors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SModularRigHierarchy::AreAnyConnectorVisibilityFlagsSet, EModularRigHierarchyEditorConnectorVisibilityFlags::ShowUnresolvedConnectors)
	);
}

FReply SModularRigHierarchy::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SModularRigHierarchy::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FReply Reply = SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
	if(Reply.IsEventHandled())
	{
		return Reply;
	}

	if(MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		const TSharedPtr<FModularRigHierarchyTreeElement>* ItemPtr = TreeView->FindItemAtPosition(MouseEvent.GetScreenSpacePosition());
		const TSharedPtr<FModularRigHierarchyTreeElement>& Item = ItemPtr ? *ItemPtr : nullptr;

		if (ViewModel.IsValid() &&
			Item.IsValid())
		{
			ViewModel->SelectModuleAndChildren(Item->GetModuleName());
		}
	}

	return FReply::Unhandled();
}

void SModularRigHierarchy::RefreshTreeView(bool bRebuildContent)
{
	if (TreeView.IsValid())
	{
		TreeView->RefreshTreeView(bRebuildContent);
	}
}

void SModularRigHierarchy::OnHiddenColumnsListChanged()
{
	const TSharedPtr<SHeaderRow> HeaderRow = TreeView.IsValid() ? TreeView->GetHeaderRow() : nullptr;
	if (TreeView.IsValid() && HeaderRow.IsValid())
	{
		UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
		Settings->ModularRigHierarchyHiddenColums = HeaderRow->GetHiddenColumnIds();

		Settings->SaveConfig();

		TreeView->RequestTreeRefresh();
	}
}

TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SModularRigHierarchy::GetSelectedItems() const
{
	TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	SelectedItems.Remove(TSharedPtr<FModularRigHierarchyTreeElement>(nullptr));
	return SelectedItems;
}

TArray<FString> SModularRigHierarchy::GetSelectedKeys() const
{
	const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
	
	TArray<FString> SelectedKeys;
	for (const TSharedPtr<FModularRigHierarchyTreeElement>& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsValid())
		{
			if(!SelectedItem->GetKey().IsEmpty())
			{
				SelectedKeys.AddUnique(SelectedItem->GetKey());
			}
		}
	}

	return SelectedKeys;
}

void SModularRigHierarchy::HandlePostCompileModularRigs(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	if(!bKeepCurrentEditedConnectors)
	{
		CurrentlyEditedConnectors.Reset();
	}
}

void SModularRigHierarchy::HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	RefreshTreeView();
}

void SModularRigHierarchy::HandleSetObjectBeingDebugged(UObject* InObject)
{
	if (ControlRigBeingDebuggedPtr.Get() == InObject)
	{
		return;
	}

	if (ControlRigBeingDebuggedPtr.IsValid())
	{
		if (URigHierarchy* Hierarchy = ControlRigBeingDebuggedPtr->GetHierarchy())
		{
			Hierarchy->OnModified().RemoveAll(this);
		}
	}

	ControlRigBeingDebuggedPtr.Reset();

	if (UModularRig* ControlRig = Cast<UModularRig>(InObject))
	{
		ControlRigBeingDebuggedPtr = ControlRig;

		if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->OnModified().AddSP(this, &SModularRigHierarchy::OnHierarchyModified);
		}
	}

	RefreshTreeView();
}

TSharedRef<SWidget> SModularRigHierarchy::OnGetOptionsMenu()
{
	constexpr bool bShouldCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, CommandList);

	const FCanExecuteAction CanExecuteAction = FCanExecuteAction::CreateLambda([]() { return true; });

	MenuBuilder.BeginSection("FilterOptions", LOCTEXT("FilterOptions", "Filter Options"));
	{
		const FControlRigModularRigCommands& Commands = FControlRigModularRigCommands::Get();

		const FName NoExtensionHook = NAME_None;
		const TAttribute<FText> NoLabelOverride;
		const TAttribute<FText> NoTooltipOverride;

		MenuBuilder.AddMenuEntry(
			Commands.ToggleShowSecondaryContectors,
			NoExtensionHook,
			NoLabelOverride,
			NoTooltipOverride,
			FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.ConnectorSecondary"))
		);

		MenuBuilder.AddMenuEntry(
			Commands.ToggleShowOptionalContectors,
			NoExtensionHook,
			NoLabelOverride,
			NoTooltipOverride,
			FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.ConnectorOptional"))
		);

		MenuBuilder.AddMenuEntry(
			Commands.ToggleShowUnresolvedContectors,
			NoExtensionHook,
			NoLabelOverride,
			NoTooltipOverride,
			FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.ConnectorWarning"))
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SModularRigHierarchy::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;
	RefreshTreeView(true);
}

TSharedPtr<SWidget> SModularRigHierarchy::CreateContextMenuWidget()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (UToolMenu* Menu = GetContextMenu())
	{
		return ToolMenus->GenerateWidget(Menu);
	}
	
	return SNullWidget::NullWidget;
}

void SModularRigHierarchy::CreateContextMenu()
{
	static bool bCreatedMenu = false;
	if(bCreatedMenu)
	{
		return;
	}
	bCreatedMenu = true;
	
	const FName MenuName = ContextMenuName;

	UToolMenus* ToolMenus = UToolMenus::Get();
	
	if (!ensure(ToolMenus))
	{
		return;
	}

	if (UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName))
	{
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UControlRigContextMenuContext* MainContext = InMenu->FindContext<UControlRigContextMenuContext>();
				
				if (SModularRigHierarchy* ModelPanel = MainContext->GetModularRigModelPanel())
				{
					const FControlRigModularRigCommands& Commands = FControlRigModularRigCommands::Get(); 
				
					FToolMenuSection& ModulesSection = InMenu->AddSection(TEXT("Modules"), LOCTEXT("ModulesHeader", "Modules"));
					ModulesSection.AddSubMenu(TEXT("New"), LOCTEXT("New", "New"), LOCTEXT("New_ToolTip", "Create New Modules"),
						FNewToolMenuDelegate::CreateLambda([Commands, ModelPanel](UToolMenu* InSubMenu)
						{
							FToolMenuSection& DefaultSection = InSubMenu->AddSection(NAME_None);
							DefaultSection.AddMenuEntry(Commands.AddModuleItem);
						})
					);
					ModulesSection.AddMenuEntry(Commands.RenameModuleItem);
					ModulesSection.AddMenuEntry(Commands.DeleteModuleItem);
					ModulesSection.AddMenuEntry(Commands.DuplicateModuleItem);
					ModulesSection.AddMenuEntry(Commands.MirrorModuleItem);
					ModulesSection.AddMenuEntry(Commands.ReresolveModuleItem);
					ModulesSection.AddMenuEntry(Commands.SwapModuleClassItem);
					ModulesSection.AddMenuEntry(Commands.CopyModuleSettings);
					ModulesSection.AddMenuEntry(Commands.PasteModuleSettings);
				}
			})
		);
	}
}

UToolMenu* SModularRigHierarchy::GetContextMenu()
{
	const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = ViewModel.IsValid() ? ViewModel->GetControlRigEditor() : nullptr;

	const FName MenuName = ContextMenuName;
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ControlRigEditor.IsValid() ||
		!ensure(ToolMenus))
	{
		return nullptr;
	}

	// individual entries in this menu can access members of this context, particularly useful for editor scripting
	UControlRigContextMenuContext* ContextMenuContext = NewObject<UControlRigContextMenuContext>();
	FControlRigMenuSpecificContext MenuSpecificContext;
	MenuSpecificContext.ModularRigModelPanel = SharedThis(this);
	ContextMenuContext->Init(ControlRigEditor, MenuSpecificContext);

	FToolMenuContext MenuContext(CommandList);
	MenuContext.AddObject(ContextMenuContext);

	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, MenuContext);

	return Menu;
}

TSharedPtr<FUICommandList> SModularRigHierarchy::GetContextMenuCommands() const
{
	return CommandList;
}

/** Filter class to show only RigModules. */
class FClassViewerRigModulesFilter : public IClassViewerFilter
{
public:
	FClassViewerRigModulesFilter()
		: AssetRegistry(FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get())
	{}
	
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		if(InClass)
		{
			const bool bChildOfObjectClass = InClass->IsChildOf(UControlRig::StaticClass());
			const bool bMatchesFlags = !InClass->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated | CLASS_Abstract);
			const bool bNotNative = !InClass->IsNative();

			// Allow any class contained in the extra picker common classes array
			if (InInitOptions.ExtraPickerCommonClasses.Contains(InClass))
			{
				return true;
			}
			
			if (bChildOfObjectClass && bMatchesFlags && bNotNative)
			{
				const FAssetData AssetData(InClass);
				return MatchesFilter(AssetData);
			}
		}
		return false;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
	{
		const bool bChildOfObjectClass = InUnloadedClassData->IsChildOf(UControlRig::StaticClass());
		const bool bMatchesFlags = !InUnloadedClassData->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated | CLASS_Abstract);
		if (bChildOfObjectClass && bMatchesFlags)
		{
			const FString GeneratedClassPathString = InUnloadedClassData->GetClassPathName().ToString();
			const FString BlueprintPath = GeneratedClassPathString.LeftChop(2); // Chop off _C
			const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(BlueprintPath));
			return MatchesFilter(AssetData);

		}
		return false;
	}

private:
	bool MatchesFilter(const FAssetData& AssetData)
	{
		static const UEnum* ControlTypeEnum = StaticEnum<EControlRigType>();
		const FString ControlRigTypeStr = AssetData.GetTagValueRef<FString>(TEXT("ControlRigType"));
		if (ControlRigTypeStr.IsEmpty())
		{
			return false;
		}

		const EControlRigType ControlRigType = (EControlRigType)(ControlTypeEnum->GetValueByName(*ControlRigTypeStr));
		return ControlRigType == EControlRigType::RigModule;
	}

	const IAssetRegistry& AssetRegistry;
};

void SModularRigHierarchy::HandleNewItem()
{
	if (!ViewModel.IsValid() ||
		!TreeView.IsValid())
	{
		return;
	}

	FName ParentModuleName = NAME_None;

	const TArray<FString> SelectedKeys = GetSelectedKeys();
	if (SelectedKeys.Num() == 1)
	{
		const TSharedPtr<FModularRigHierarchyTreeElement> ParentElement = TreeView->FindElement(SelectedKeys[0]);
		if (ParentElement.IsValid())
		{
			ParentModuleName = ParentElement->GetModuleName();
		}
	}
	
	FAssetPickerConfig AssetPickerConfig;
	
	{
		// setup filtering
		AssetPickerConfig.Filter.ClassPaths.Add(UControlRigEditorAssetInterface::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.ClassPaths.Add(UControlRigRuntimeAssetInterface::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowPathInColumnView = true;
		AssetPickerConfig.bShowTypeInColumnView = true;
		AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Blueprint;
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
		AssetPickerConfig.bAllowDragging = true;
		AssetPickerConfig.bAllowRename = false;
		AssetPickerConfig.bForceShowPluginContent = true;
		AssetPickerConfig.bForceShowEngineContent = true;
		AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Small;

		// hide all asset registry columns by default (we only really want the name and path)
		UObject* DefaultControlRigBlueprint = UControlRigBlueprint::StaticClass()->GetDefaultObject();
		FAssetRegistryTagsContextData Context(DefaultControlRigBlueprint, EAssetRegistryTagsCaller::Uncategorized);
		DefaultControlRigBlueprint->GetAssetRegistryTags(Context);
		for (TPair<FName, UObject::FAssetRegistryTag>& AssetRegistryTagPair : Context.Tags)
		{
			AssetPickerConfig.HiddenColumnNames.Add(AssetRegistryTagPair.Value.Name.ToString());
		}

		// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
		AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
		AssetPickerConfig.HiddenColumnNames.Add(TEXT("Has Virtualized Data"));
	}
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	
	const FText TitleText = NSLOCTEXT("ControlRig", "PickRigModule", "Pick Rig Module Asset");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600.0f, 600.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedPtr<SRigModuleAssetBrowser> AssetBrowser;
	bool bPressedOk = false;
	TArray<FAssetData> SelectedAssets;
	Window->SetContent(
		SNew(SVerticalBox)
		
		+SVerticalBox::Slot()
		[
			SAssignNew(AssetBrowser, SRigModuleAssetBrowser)
			.bAllowDragging(false)
			.AssetViewType(EAssetViewType::Column)
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(5)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("OK", "OK"))
				.OnClicked_Lambda([&bPressedOk, &SelectedAssets, AssetBrowser, Window]
					{
						bPressedOk = true;
						SelectedAssets = AssetBrowser->GetSelectedAssets();
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
			]
			+SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked_Lambda([&bPressedOk, Window] { bPressedOk = false; Window->RequestDestroyWindow(); return FReply::Handled(); })
			]
		]
	);
	
	FSlateApplication::Get().AddModalWindow(Window, FSlateApplication::Get().GetActiveTopLevelWindow());
	
	
	if (bPressedOk)
	{
		TArray<FControlRigAssetStrongReference> SelectedModules;
		Algo::Transform(SelectedAssets, SelectedModules, [](FAssetData AssetData)
			{
				FControlRigAssetStrongReference Reference;
				if (FControlRigAssetInterfacePtr Blueprint = AssetData.GetAsset())
				{
					Reference = Blueprint->GetControlRigAssetReference();
				}
				else if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = AssetData.GetAsset())
				{
					Reference = RuntimeAsset->GetControlRigAssetReference();
				}
				return Reference;
			});
		
		const FScopedTransaction AddModuleTransaction(LOCTEXT("AddModuleTransaction", "Add Module"));
	
		for (FControlRigAssetStrongReference& Asset : SelectedModules)
		{
			const FName NewModuleName = ViewModel->AddModule(Asset, ParentModuleName);
	
			const TSharedPtr<FModularRigHierarchyTreeElement> Element = TreeView->FindElement(NewModuleName.ToString());
			if (Element.IsValid())
			{
				TreeView->SetSelection({ Element }, ESelectInfo::OnNavigation);

				const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = TreeView->GetSelectedItems();
				if (!SelectedItems.IsEmpty() &&
					SelectedItems[0].IsValid())
				{
					TreeView->RequestScrollIntoView(SelectedItems[0]);
				}

				TreeView->bRequestRenameSelected = true;
			}
		}
	}
}

void SModularRigHierarchy::HandleRenameModule()
{
	const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
	if (SelectedItems.Num() == 1 &&
		SelectedItems[0].IsValid())
	{
		SelectedItems[0]->RequestRename();
	}
}

void SModularRigHierarchy::HandleDeleteModules()
{
	if (ViewModel.IsValid())
	{
		const TArray<FName> SelectedModuleNames = GetSelectedModuleNames();

		const FText TransactionText = FText::Format(LOCTEXT("DeleteModulesTransaction", "Delete selected {0}|plural(one=Module,other=Modules)"), SelectedModuleNames.Num());
		const FScopedTransaction DeleteModulesTransaction(TransactionText);

		ViewModel->DeleteModules(SelectedModuleNames);
	}
}

void SModularRigHierarchy::HandleDuplicateModules()
{
	if (ViewModel.IsValid())
	{
		const TArray<FName> SelectedModuleNames = GetSelectedModuleNames();

		const FScopedTransaction DuplicateModulesTransaction(LOCTEXT("DuplicateModulesTransaction", "Duplicate Modules"));
		ViewModel->DuplicateModules(SelectedModuleNames);
	}
}

void SModularRigHierarchy::HandleMirrorModules()
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	const TSharedPtr<FStructOnScope> SettingsStructToDisplay = MakeShareable(new FStructOnScope(FRigVMMirrorSettings::StaticStruct(), (uint8*)&Settings->MirrorSettings));

#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<SKismetInspector> DetailsInspector = SNew(SKismetInspector);
#else
	TSharedRef<SRigVMDetailsInspector> DetailsInspector = SNew(SRigVMDetailsInspector);
#endif
	DetailsInspector->ShowSingleStruct(SettingsStructToDisplay);

	const TSharedRef<SCustomDialog> MirrorDialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("ControlModularModelMirror", "Mirror Selected Modules")))
		.Content()
		[
			DetailsInspector
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("OK", "OK")),
			SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
			});

	if (ViewModel.IsValid() &&
		MirrorDialog->ShowModal() == 0)
	{
		// Save the settings
		Settings->SaveConfig();

		const TArray<FName> SelectedModuleNames = GetSelectedModuleNames();

		const FScopedTransaction MirrorModulesTransaction(LOCTEXT("MirrorModulesTransaction", "Mirror Modules"));
		ViewModel->MirrorModules(SelectedModuleNames, Settings->MirrorSettings);
	}
}

void SModularRigHierarchy::HandleReresolveModules()
{
	if (ViewModel.IsValid())
	{
		const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
		TArray<FName> SelectedModuleAndConnectorNames;
		Algo::Transform(SelectedItems, SelectedModuleAndConnectorNames, [](const TSharedPtr<FModularRigHierarchyTreeElement>& Element)
			{
				if (Element.IsValid())
				{
					if (Element->GetConnectorName().IsEmpty())
					{
						return Element->GetModuleName();
					}
					return FRigHierarchyModulePath(Element->GetModuleName().ToString(), Element->GetConnectorName()).GetPathFName();
				}
				return FName(NAME_None);
			});

		ViewModel->ReresolveModules(SelectedModuleAndConnectorNames);
	}
}

bool SModularRigHierarchy::CanSwapModules() const
{
	// Only if all modules selected have the same module class
	UModularRig* Rig = ViewModel->GetModularRig();
	if (Rig)
	{
		FControlRigAssetSoftReference CommonClass = nullptr;
		const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
		for (const TSharedPtr<FModularRigHierarchyTreeElement>& SelectedItem : SelectedItems)
		{
			if(!SelectedItem.IsValid())
			{
				continue;
			}
			FControlRigAssetSoftReference ModuleClass;
			if (const FRigModuleReference* Module = ControlRigBlueprint->GetModularRigModel().FindModule(SelectedItem->GetModuleName()))
			{
				if (Module->ControlRigAssetReference.IsValid())
				{
					ModuleClass = Module->ControlRigAssetReference;
				}
			}
			if(!ModuleClass.IsValid())
			{
				return false;
			}
			if(!CommonClass.IsValid())
			{
				CommonClass = ModuleClass;
			}
			if(ModuleClass.Get() != CommonClass.Get())
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

void SModularRigHierarchy::HandleSwapClassForModules()
{
	const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
	const TArray<FName> SelectedModuleNames = GetSelectedModuleNames();

	if (SelectedModuleNames.IsEmpty())
	{
		return;
	}
		
	HandleSwapClassForModules(SelectedModuleNames);
}

void SModularRigHierarchy::HandleSwapClassForModules(const TArray<FName>& InModuleNames)
{
	TArray<FSoftObjectPath> ModulePaths;
	Algo::Transform(InModuleNames, ModulePaths, [this](const FName& InModuleName)
	{
		FSoftObjectPath ModulePath(ControlRigBlueprint.GetObject()->GetPathName());
		ModulePath.SetSubPathString(InModuleName.ToString());
		return ModulePath;
	});

	FControlRigAssetSoftReference SourceClass;
	if (FRigModuleReference* Module = ControlRigBlueprint->GetModularRigModel().FindModule(InModuleNames[0]))
	{
		SourceClass = Module->ControlRigAssetReference;
	}

	if (!SourceClass.IsValid())
	{
		return;
	}

	FAssetData SourceAsset(SourceClass.Get());
	
	SRigVMSwapAssetReferencesWidget::FArguments WidgetArgs;
	FRigVMAssetDataFilter FilterModules = FRigVMAssetDataFilter::CreateLambda([](const FAssetData& AssetData)
		{
			return UControlRigBlueprint::GetRigType(AssetData) == EControlRigType::RigModule;
		});
	TArray<FRigVMAssetDataFilter> SourceFilters = {FilterModules};
	TArray<FRigVMAssetDataFilter> TargetFilters = {FilterModules};
	
	WidgetArgs
		.EnableUndo(true)
		.CloseOnSuccess(true)
		.Source(SourceAsset)
		.ReferencePaths(ModulePaths)
		.SkipPickingRefs(true)
		.OnSwapReference_Lambda([](const FSoftObjectPath& ModulePath, const FAssetData& NewModuleAsset) -> bool
		{
			FControlRigAssetStrongReference NewModuleClass;
			if (const FControlRigAssetInterfacePtr ModuleBlueprint = NewModuleAsset.GetAsset())
			{
				NewModuleClass = ModuleBlueprint->GetControlRigAssetReference();
			}
			else if (TScriptInterface<IControlRigRuntimeAssetInterface> GeneratedClass = NewModuleAsset.GetAsset())
			{
				NewModuleClass = GeneratedClass->GetControlRigAssetReference();
			}
			if (NewModuleClass.IsValid())
			{
				FControlRigAssetInterfacePtr RigBlueprint = ModulePath.GetWithoutSubPath().ResolveObject();
				if (!RigBlueprint)
				{
					if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = ModulePath.GetWithoutSubPath().ResolveObject())
					{
						RigBlueprint = RuntimeAsset->GetEditorOnlyData();
					}
				}
				if (RigBlueprint)
				{
					return RigBlueprint->GetModularRigController()->SwapModuleSource(*ModulePath.GetSubPathString(), NewModuleClass);
				}
				
			}
			return false;
		})
		.SourceAssetFilters(SourceFilters)
		.TargetAssetFilters(TargetFilters);

	const TSharedRef<SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>> SwapModulesDialog =
		SNew(SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>)
		.WindowSize(FVector2D(800.0f, 640.0f))
		.WidgetArgs(WidgetArgs);
	
	SwapModulesDialog->ShowNormal();
}

bool SModularRigHierarchy::CanCopyModuleSettings() const
{
	return !GetSelectedItems().IsEmpty();
}

void SModularRigHierarchy::HandleCopyModuleSettings()
{
	if (ViewModel.IsValid())
	{
		TArray<FName> SelectedModuleNames;
		const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
		for (const TSharedPtr<FModularRigHierarchyTreeElement>& SelectedItem : SelectedItems)
		{
			if (SelectedItem.IsValid())
			{
				SelectedModuleNames.AddUnique(SelectedItem->GetModuleName());
			}
		}

		ViewModel->ClipboardCopyModuleSettings(SelectedModuleNames);
	}
}

bool SModularRigHierarchy::CanPasteModuleSettings() const
{
	if (ViewModel.IsValid())
	{
		const int32 ExpectedNumElements = GetSelectedItems().Num();

		return ViewModel->CanClipboardPasteModuleSettings(ExpectedNumElements);
	}

	return false;
}

void SModularRigHierarchy::HandlePasteModuleSettings()
{
	if (ViewModel.IsValid())
	{
		TArray<FName> SelectedModuleNames;
		const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
		for (const TSharedPtr<FModularRigHierarchyTreeElement>& SelectedItem : SelectedItems)
		{
			if (SelectedItem.IsValid())
			{
				SelectedModuleNames.AddUnique(SelectedItem->GetModuleName());
			}
		}

		ViewModel->ClipboardPasteModuleSettings(SelectedModuleNames);
	}
}

void SModularRigHierarchy::ToggleConnectorVisibilityFlags(const EModularRigHierarchyEditorConnectorVisibilityFlags ConnectorVisibilityFlags)
{
	UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();
	if (Settings->HasAnyModularRigHierarchyConnectorVisibilityFlags(ConnectorVisibilityFlags))
	{
		EModularRigHierarchyEditorConnectorVisibilityFlags Flags = Settings->GetModularRigHierarchyConnectorVisibilityFlags();
		EnumRemoveFlags(Flags, ConnectorVisibilityFlags);

		Settings->SetModularRigHierarchyConnectorVisibilityFlags(Flags);
	}
	else
	{
		EModularRigHierarchyEditorConnectorVisibilityFlags Flags = Settings->GetModularRigHierarchyConnectorVisibilityFlags();
		EnumAddFlags(Flags, ConnectorVisibilityFlags);

		Settings->SetModularRigHierarchyConnectorVisibilityFlags(Flags);
	}

	Settings->SaveConfig();

	if (TreeView.IsValid())
	{
		TreeView->RefreshTreeView();
	}
}

bool SModularRigHierarchy::AreAnyConnectorVisibilityFlagsSet(const EModularRigHierarchyEditorConnectorVisibilityFlags ConnectorVisibilityFlags) const
{
	const UControlRigEditorSettings* Settings = GetDefault<UControlRigEditorSettings>();
	return EnumHasAnyFlags(Settings->GetModularRigHierarchyConnectorVisibilityFlags(), ConnectorVisibilityFlags);
}

void SModularRigHierarchy::HandleConnectorResolved(const FRigElementKey& InConnector, const TArray<FRigElementKey>& InTargets)
{
	UModularRig* ModularRig = ViewModel.IsValid() ? ViewModel->GetModularRig() : nullptr;
	UModularRigController* Controller = ViewModel.IsValid() ? ViewModel->GetModularRigController() : nullptr;
	if (ModularRig &&
		Controller)
	{
		const FScopedTransaction Transaction(LOCTEXT("ModularRigModelResolveConnector", "Resolve Connector"), !GIsTransacting);

		if(!bKeepCurrentEditedConnectors)
		{
			CurrentlyEditedConnectors.Reset();
		}
		
		if (InTargets.IsEmpty())
		{
			HandleConnectorDisconnect(InConnector);
		}
		else
		{
			const TGuardValue<bool> KeepCurrentEditedConnectorsGuard(bKeepCurrentEditedConnectors, true);

			CurrentlyEditedConnectors.Add(InConnector.Name);
			Controller->ConnectConnectorToElements(InConnector, InTargets, true, ModularRig->GetModularRigSettings().bAutoResolve);
		}

		if (ViewModel.IsValid())
		{
			ViewModel->RefreshDetails();
		}
	}
}

void SModularRigHierarchy::HandleConnectorDisconnect(const FRigElementKey& InConnector)
{
	UModularRigController* Controller = ViewModel.IsValid() ? ViewModel->GetModularRigController() : nullptr;
	if (Controller)
	{
		const FScopedTransaction Transaction(LOCTEXT("ModularRigModelDisconnectConnector", "Disconnect Connector"), !GIsTransacting);

		if(!bKeepCurrentEditedConnectors)
		{
			CurrentlyEditedConnectors.Reset();
		}
		const TGuardValue<bool> KeepCurrentEditedConnectorsGuard(bKeepCurrentEditedConnectors, true); 
		CurrentlyEditedConnectors.Add(InConnector.Name);
		Controller->DisconnectConnector(InConnector, false, true);
	}
}

bool SModularRigHierarchy::ShouldAlwaysShowConnector(const FName& InConnectorName) const
{
	return CurrentlyEditedConnectors.Contains(InConnectorName);
}

void SModularRigHierarchy::OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	using namespace UE::ControlRigEditor;

	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;
	
	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementSelected:
		case ERigHierarchyNotification::ElementDeselected:
		{
			FString ModuleOrConnectorName = InHierarchy->GetModuleName(InElement->GetKey());

			if(const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(InElement))
			{
				if(Connector->IsPrimary())
            	{
					ModuleOrConnectorName = Connector->GetName();
            	}
			}

			if (ModuleOrConnectorName.IsEmpty())
			{
				TreeView->ClearSelection();
			}
			else
			{
				if(TSharedPtr<FModularRigHierarchyTreeElement> Item = TreeView->FindElement(ModuleOrConnectorName))
				{
					const bool bSelected = InNotif == ERigHierarchyNotification::ElementSelected;
					TreeView->SetItemHighlighted(Item, bSelected);
					
					if (!FRigHierarchyTreePersistentStateStore::IsFeatureEnabled())
					{
						TreeView->RequestScrollIntoView(Item);
					}
				}
			}
		}
		default:
		{
			break;
		}
	}
}

class SModularRigModelPasteTransformsErrorPipe : public FOutputDevice
{
public:

	int32 NumErrors;

	SModularRigModelPasteTransformsErrorPipe()
		: FOutputDevice()
		, NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		UE_LOGF(LogControlRig, Error, "Error importing transforms to Model: %ls", V);
		NumErrors++;
	}
};

void SModularRigHierarchy::PostRedo(bool bSuccess) 
{
	if (bSuccess)
	{
		RefreshTreeView();
	}
}

void SModularRigHierarchy::PostUndo(bool bSuccess) 
{
	if (bSuccess)
	{
		RefreshTreeView();
	}
}

FReply SModularRigHierarchy::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TArray<FString> DraggedKeys = GetSelectedKeys();
	TArray<FName> ModuleNames;
	for(const FString& DraggedKey : DraggedKeys)
	{
		ModuleNames.Add(*DraggedKey);
	}
	
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && ModuleNames.Num() > 0)
	{
		const TSharedRef<FModularRigHierarchyElementDragDropOp> DragDropOp = FModularRigHierarchyElementDragDropOp::New(MoveTemp(ModuleNames));
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SModularRigHierarchy::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FModularRigHierarchyTreeElement> TargetItem)
{
	const TOptional<EItemDropZone> InvalidDropZone;
	TOptional<EItemDropZone> ReturnDropZone = DropZone;
	int32 NewModuleIndex = INDEX_NONE;

	if(DropZone == EItemDropZone::BelowItem || DropZone == EItemDropZone::AboveItem)
	{
		DropZone = EItemDropZone::OntoItem;
		
		if (ViewModel.IsValid() &&
			TargetItem.IsValid())
		{
			TSharedPtr<FModularRigHierarchyTreeElement> ChildTargetItem;
			Swap(TargetItem, ChildTargetItem);
			
			if(const FRigModuleReference* ChildModule = ViewModel->FindModule(ChildTargetItem->GetModuleName()))
			{
				if(const FRigModuleReference* ParentModule = ViewModel->FindModule(ChildModule->ParentModuleName))
				{
					if(const TSharedPtr<FModularRigHierarchyTreeElement> ParentItem = TreeView->FindElement(ParentModule->Name.ToString()))
					{
						TargetItem = ParentItem;
					}
				}
			}
		}
	}
	
	if(DropZone != EItemDropZone::OntoItem)
	{
		return InvalidDropZone;
	}

	TSharedPtr<FAssetDragDropOp> AssetDragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	TSharedPtr<FModularRigHierarchyElementDragDropOp> ModuleDragDropOperation = DragDropEvent.GetOperationAs<FModularRigHierarchyElementDragDropOp>();
	if (AssetDragDropOperation)
	{
		for (const FAssetData& AssetData : AssetDragDropOperation->GetAssets())
		{
			static const UEnum* ControlTypeEnum = StaticEnum<EControlRigType>();
			const FString ControlRigTypeStr = AssetData.GetTagValueRef<FString>(TEXT("ControlRigType"));
			if (ControlRigTypeStr.IsEmpty())
			{
				ReturnDropZone.Reset();
				break;
			}

			const EControlRigType ControlRigType = (EControlRigType)(ControlTypeEnum->GetValueByName(*ControlRigTypeStr));
			if (ControlRigType != EControlRigType::RigModule)
			{
				ReturnDropZone.Reset();
				break;
			}
		}
	}
	else if(ModuleDragDropOperation)
	{
		if(TargetItem.IsValid())
		{
			// we cannot drag a module onto itself
			if(ModuleDragDropOperation->GetModules().Contains(TargetItem->GetModuleName()))
			{
				return InvalidDropZone;
			}
		}
	}
	else
	{
		ReturnDropZone.Reset();
	}

	return ReturnDropZone;
}

FReply SModularRigHierarchy::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FModularRigHierarchyTreeElement> TargetItem)
{
	if (!ViewModel.IsValid())
	{
		return FReply::Unhandled();
	}

	int32 NewModuleIndex = INDEX_NONE;
	
	if(DropZone == EItemDropZone::BelowItem || DropZone == EItemDropZone::AboveItem)
	{
		if (TargetItem.IsValid())
		{
			TSharedPtr<FModularRigHierarchyTreeElement> ChildTargetItem;
			Swap(TargetItem, ChildTargetItem);
			
			if(const FRigModuleReference* ChildModule = ViewModel->FindModule(ChildTargetItem->GetModuleName()))
			{
				TArray<FRigModuleReference*> Children;
				if(const FRigModuleReference* ParentModule = ViewModel->FindModule(ChildModule->ParentModuleName))
				{
					Children = ParentModule->CachedChildren;
						
					if(const TSharedPtr<FModularRigHierarchyTreeElement> ParentItem = TreeView->FindElement(ParentModule->Name.ToString()))
					{
						TargetItem = ParentItem;
					}
				}
				else
				{
					Children = ViewModel->GetRootModules();
				}

				for(int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
				{
					if(Children[ChildIndex] == ChildModule)
					{						
						NewModuleIndex = ChildIndex;
						break;
					}
				}
			}
		}

		DropZone = EItemDropZone::OntoItem;
	}

	FName ParentModuleName = NAME_None;
	if (TargetItem.IsValid())
	{
		ParentModuleName = TargetItem->GetModuleName();
	}

	TSharedPtr<FAssetDragDropOp> AssetDragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	TSharedPtr<FModularRigHierarchyElementDragDropOp> ModuleDragDropOperation = DragDropEvent.GetOperationAs<FModularRigHierarchyElementDragDropOp>();
	if (AssetDragDropOperation)
	{
		const int32 NumAssets = AssetDragDropOperation->GetAssets().Num();
		const FText TransactionText = FText::Format(LOCTEXT("DropNewModulesTransaction", "Add {0}|plural(one=Module,other=Modules)"), NumAssets);
		const FScopedTransaction DropNewModulesTransaction(TransactionText);

		for (const FAssetData& AssetData : AssetDragDropOperation->GetAssets())
		{
			static const UEnum* ControlTypeEnum = StaticEnum<EControlRigType>();
			const FString ControlRigTypeStr = AssetData.GetTagValueRef<FString>(TEXT("ControlRigType"));
			if (ControlRigTypeStr.IsEmpty())
			{
				continue;
			}

			const EControlRigType ControlRigType = (EControlRigType)(ControlTypeEnum->GetValueByName(*ControlRigTypeStr));
			if (ControlRigType != EControlRigType::RigModule)
			{
				continue;
			}

			if(FControlRigAssetInterfacePtr AssetBlueprint = AssetData.GetAsset())
			{
				FControlRigAssetStrongReference Source = AssetBlueprint->GetControlRigAssetReference();
				ViewModel->AddModule(Source, ParentModuleName);
			}
			else if (TScriptInterface<IControlRigRuntimeAssetInterface> GeneratedClass = AssetData.GetAsset())
			{
				FControlRigAssetStrongReference Source(GeneratedClass.GetObject());
				ViewModel->AddModule(Source, ParentModuleName);
			}
			else
			{
				continue;
			}
		}

		return FReply::Handled();
	}
	else if (ModuleDragDropOperation)
	{
		const TArray<FName> ModuleNames = ModuleDragDropOperation->GetModules();

		const FText TransactionText = FText::Format(LOCTEXT("ReparentModulesTransaction", "Reparent {0}|plural(one=Module,other=Modules)"), ModuleNames.Num());
		const FScopedTransaction ReparentModulesTransaction(TransactionText);

		ViewModel->ReparentModules(ModuleNames, ParentModuleName, NewModuleIndex);
	}
	
	return FReply::Unhandled();
}

FReply SModularRigHierarchy::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// only allow drops onto empty space of the widget (when there's no target item under the mouse)
	// when dropped onto an item SModularRigHierarchy::OnAcceptDrop will deal with the event
	const TSharedPtr<FModularRigHierarchyTreeElement>* ItemAtMouse = TreeView->FindItemAtPosition(DragDropEvent.GetScreenSpacePosition());
	FString ParentPath;
	if (ItemAtMouse && ItemAtMouse->IsValid())
	{
		return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
	}
	
	if (OnCanAcceptDrop(DragDropEvent, EItemDropZone::BelowItem, nullptr))
	{
		if (OnAcceptDrop(DragDropEvent, EItemDropZone::BelowItem, nullptr).IsEventHandled())
		{
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

TArray<FName> SModularRigHierarchy::GetSelectedModuleNames() const
{
	const TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
	TArray<FName> SelectedModuleNames;
	Algo::TransformIf(SelectedItems, SelectedModuleNames, 
		[](const TSharedPtr<FModularRigHierarchyTreeElement>& Element)
		{
			return 
				Element.IsValid() &&
				!Element->GetModuleName().IsNone();
		},
		[](const TSharedPtr<FModularRigHierarchyTreeElement>& Element)
		{
			return Element->GetModuleName();
		});

	return SelectedModuleNames;
}

#undef LOCTEXT_NAMESPACE

