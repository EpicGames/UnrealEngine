// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigBlueprintLegacy.h"
#include "Editor/Hierarchy/Widgets/SModularRigHierarchyTreeView.h"
#include "Editor/RigVMNewEditor.h"
#include "EditorUndoClient.h"

class FMenuBuilder;
class FUICommandList;
class IControlRigBaseEditor;
class SModularRigHierarchy;
class SSearchBox;
class UControlRig;
class URigVMBlueprint;
class UToolMenu;
struct FAssetData;
struct FToolMenuContext;

namespace UE::ControlRigEditor { class FModularRigHierarchyViewModel; }

/** Widget allowing editing of a control rig's structure */
class SModularRigHierarchy : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
	using FModularRigHierarchyViewModel = UE::ControlRigEditor::FModularRigHierarchyViewModel;

public:
	SLATE_BEGIN_ARGS(SModularRigHierarchy) 
		{}

	SLATE_END_ARGS()

	/** 
	 * Constructs this widget 
	 * 
	 * @param InArgs				Slate Arguments
	 * @param InControlRigEditor	The control rig editor that displays this view
	 * @param InViewName			The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor
	 */
	void Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor, const FName& InViewName);

private:
	void OnEditorClose(IControlRigBaseEditor* InEditor, FControlRigAssetInterfacePtr InBlueprint);

	/** Bind commands that this widget handles */
	void BindCommands();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Rebuild the tree view */
	void RefreshTreeView(bool bRebuildContent = true);

	/** Called when the hidden columns list changed */
	void OnHiddenColumnsListChanged();

	/** Returns all selected items */
	TArray<TSharedPtr<FModularRigHierarchyTreeElement>> GetSelectedItems() const;

	/** Return all selected keys */
	TArray<FString> GetSelectedKeys() const;

	/** Create a new item */
	void HandleNewItem();

	/** Rename item */
	void HandleRenameModule();
	FName HandleRenameModule(const FName& InOldModuleName, const FName& InNewName);
	bool HandleVerifyNameChanged(const FName& InOldModuleName, const FName& InNewName, FText& OutErrorMessage);

	/** Delete items */
	void HandleDeleteModules();

	/** Duplicate items */
	void HandleDuplicateModules();

	/** Mirror items */
	void HandleMirrorModules();

	/** Reresolve items */
	void HandleReresolveModules();

	/** Swap module class for items */
	bool CanSwapModules() const;
	void HandleSwapClassForModules();
	void HandleSwapClassForModules(const TArray<FName>& InModuleNames);

	/** Copy & paste module settings */
	bool CanCopyModuleSettings() const;
	void HandleCopyModuleSettings();
	bool CanPasteModuleSettings() const;
	void HandlePasteModuleSettings();

	/** Toggles visibility of specified connector visiblity flags */
	void ToggleConnectorVisibilityFlags(const EModularRigHierarchyEditorConnectorVisibilityFlags ConnectorVisibilityFlags);

	/** Returns true if any of the provided connector visiblity flags are set */
	bool AreAnyConnectorVisibilityFlagsSet(const EModularRigHierarchyEditorConnectorVisibilityFlags ConnectorVisibilityFlags) const;

	/** Resolve connector */
	void HandleConnectorResolved(const FRigElementKey& InConnector, const TArray<FRigElementKey>& InTargets);

	/** UnResolve connector */
	void HandleConnectorDisconnect(const FRigElementKey& InConnector);

	/** Set Selection Changed */
	void HandleSelectionChanged(TSharedPtr<FModularRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo);

	/** Returns true if a given connector should always be shown */
	bool ShouldAlwaysShowConnector(const FName& InConnectorName) const;

	TSharedPtr< SWidget > CreateContextMenuWidget();

	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface
	
	// reply to a drag operation
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	// reply to a drop operation on item
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FModularRigHierarchyTreeElement> TargetItem);
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FModularRigHierarchyTreeElement> TargetItem);

	// SWidget Overrides
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	static const FName ContextMenuName;
	static void CreateContextMenu();
	UToolMenu* GetContextMenu();
	TSharedPtr<FUICommandList> GetContextMenuCommands() const;

	/** Tree view widget */
	TSharedPtr<SModularRigHierarchyTreeView> TreeView;
	TSharedPtr<SHeaderRow> HeaderRowWidget;

	TWeakInterfacePtr<IControlRigEditorAssetInterface> ControlRigBlueprint;
	TWeakObjectPtr<UModularRig> ControlRigBeingDebuggedPtr;

	/** Command list we bind to */
	TSharedPtr<FUICommandList> CommandList;
	
	void HandlePostCompileModularRigs(FRigVMEditorAssetInterfacePtr InBlueprint);
	void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);

	void HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint);
	void HandleSetObjectBeingDebugged(UObject* InObject);

	TSharedRef<SWidget> OnGetOptionsMenu();
	void OnFilterTextChanged(const FText& SearchText);

	/** Gets the currently selected module names */
	TArray<FName> GetSelectedModuleNames() const;

	FText FilterText;

	TSharedPtr<SSearchBox> FilterBox;
	bool bKeepCurrentEditedConnectors = false;
	TSet<FName> CurrentlyEditedConnectors;

	/** The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor */
	FName ViewName;

	/** A model for this view */
	TSharedPtr<FModularRigHierarchyViewModel> ViewModel;

public:

	friend class FModularRigHierarchyTreeElement;
	friend class SModularRigHierarchyTreeItem;
	friend class UControlRigBlueprintEditorLibrary;
};

