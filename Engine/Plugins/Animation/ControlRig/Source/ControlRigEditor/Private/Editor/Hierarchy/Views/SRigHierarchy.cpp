// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigHierarchy.h"

#include "ControlRig.h"
#include "ControlRigHierarchyCommands.h"
#include "Dialog/SCustomDialog.h"
#include "Dialogs/Dialogs.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor/ControlRigContextMenuContext.h"
#include "Editor/ControlRigEditor.h"
#include "Editor/ControlRigSkeletalMeshComponent.h"
#include "Editor/Hierarchy/DragDrop/ModularRigHierarchyElementDragDropOp.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyElementDragDropOp.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyTagDragDropOp.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateStore.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IPersonaToolkit.h"
#include "ModularRig.h"
#include "ModularRigRuleManager.h"
#include "Preferences/PersonaOptions.h"
#include "PropertyCustomizationHelpers.h"
#include "Rigs/RigHierarchyController.h"
#include "ScopedTransaction.h"
#include "SEnumCombo.h"
#include "Sequencer/ControlRigLayerInstance.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#define LOCTEXT_NAMESPACE "SRigHierarchy"

///////////////////////////////////////////////////////////

const FName SRigHierarchy::ContextMenuName = TEXT("ControlRigEditor.RigHierarchy.ContextMenu");
const FName SRigHierarchy::DragDropMenuName = TEXT("ControlRigEditor.RigHierarchy.DragDropMenu");

SRigHierarchy::~SRigHierarchy()
{
	IControlRigBaseEditor* Editor = ControlRigEditor.IsValid() ? ControlRigEditor.Pin().Get() : nullptr;
	OnEditorClose(Editor, ControlRigBlueprint.GetObject());
}

void SRigHierarchy::Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor, const FName& InViewName)
{
	ViewName = InViewName;

	ControlRigEditor = InControlRigEditor;

	ControlRigBlueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface();

	ControlRigBlueprint->GetHierarchy()->OnModified().AddRaw(this, &SRigHierarchy::OnHierarchyModified);
	ControlRigBlueprint->GetRigVMAssetInterface()->OnRefreshEditor().AddRaw(this, &SRigHierarchy::HandleRefreshEditorFromBlueprint);
	ControlRigBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().AddRaw(this, &SRigHierarchy::HandleSetObjectBeingDebugged);

	if(UModularRigController* ModularRigController = ControlRigBlueprint->GetModularRigController())
	{
		ModularRigController->OnModified().AddSP(this, &SRigHierarchy::OnModularRigModified);
	}

	// for deleting, renaming, dragging
	CommandList = MakeShared<FUICommandList>();

	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}

	BindCommands();

	// setup all delegates for the rig hierarchy widget
	FRigHierarchyTreeDelegates Delegates;
	Delegates.OnGetHierarchy = FOnGetRigTreeHierarchy::CreateSP(this, &SRigHierarchy::GetHierarchyForTreeView);
	Delegates.OnGetDisplaySettings = FOnGetRigTreeDisplaySettings::CreateSP(this, &SRigHierarchy::GetDisplaySettings);
	Delegates.OnRenameElement = FOnRigTreeRenameElement::CreateSP(this, &SRigHierarchy::HandleRenameElement);
	Delegates.OnVerifyElementNameChanged = FOnRigTreeVerifyElementNameChanged::CreateSP(this, &SRigHierarchy::HandleVerifyNameChanged);
	Delegates.OnSelectionChanged = FOnRigTreeSelectionChanged::CreateSP(this, &SRigHierarchy::OnSelectionChanged);
	Delegates.OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SRigHierarchy::CreateContextMenuWidget);
	Delegates.OnMouseButtonClick = FOnRigTreeMouseButtonClick::CreateSP(this, &SRigHierarchy::OnItemClicked);
	Delegates.OnMouseButtonDoubleClick = FOnRigTreeMouseButtonClick::CreateSP(this, &SRigHierarchy::OnItemDoubleClicked);
	Delegates.OnSetExpansionRecursive = FOnRigTreeSetExpansionRecursive::CreateSP(this, &SRigHierarchy::OnSetExpansionRecursive);
	Delegates.OnCanAcceptDrop = FOnRigTreeCanAcceptDrop::CreateSP(this, &SRigHierarchy::OnCanAcceptDrop);
	Delegates.OnAcceptDrop = FOnRigTreeAcceptDrop::CreateSP(this, &SRigHierarchy::OnAcceptDrop);
	Delegates.OnDragDetected = FOnDragDetected::CreateSP(this, &SRigHierarchy::OnDragDetected);
	Delegates.OnGetResolvedKey = FOnRigTreeGetResolvedKey::CreateSP(this, &SRigHierarchy::OnGetResolvedKey);
	Delegates.OnRequestDetailsInspection = FOnRigTreeRequestDetailsInspection::CreateSP(this, &SRigHierarchy::OnRequestDetailsInspection);
	Delegates.OnRigTreeElementKeyTagDragDetected = FOnRigTreeElementKeyTagDragDetected::CreateSP(this, &SRigHierarchy::OnElementKeyTagDragDetected);
	Delegates.OnRigTreeGetItemToolTip = FOnRigTreeItemGetToolTip::CreateSP(this, &SRigHierarchy::OnGetItemTooltip);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SRigHierarchy::IsToolbarVisible)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.MaxWidth(180.0f)
					.Padding(3.0f, 1.0f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
						.ForegroundColor(FLinearColor::White)
						.OnClicked(FOnClicked::CreateSP(this, &SRigHierarchy::OnImportSkeletonClicked))
						.Text(this, &SRigHierarchy::GetImportHierarchyText)
						.IsEnabled(this, &SRigHierarchy::IsImportHierarchyEnabled)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)
					.Visibility(this, &SRigHierarchy::IsSearchbarVisible)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 2.0f, 0.0f)
					[
						SNew(SComboButton)
						.Visibility(EVisibility::Visible)
						.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
						.ForegroundColor(FSlateColor::UseStyle())
						.ContentPadding(0.0f)
						.OnGetMenuContent(this, &SRigHierarchy::CreateFilterMenu)
						.ButtonContent()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						 ]
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(3.0f, 1.0f)
					[
						SAssignNew(FilterBox, SSearchBox)
						.OnTextChanged(this, &SRigHierarchy::OnFilterTextChanged)
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.ShowEffectWhenDisabled(false)
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
				[
					SAssignNew(TreeView, SRigHierarchyTreeView, ViewName)
					.RigTreeDelegates(Delegates)
					.AutoScrollEnabled(true)
					.ForModalWindow(InArgs._ForModalWindow)
				]
			]
		]
	];

	bIsChangingRigHierarchy = false;
	LastHierarchyHash = INDEX_NONE;
	bIsConstructionEventRunning = false;
	
	RefreshTreeView();

	if (ControlRigEditor.IsValid())
	{
		ControlRigEditor.Pin()->GetKeyDownDelegate().BindLambda([&](const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)->FReply {
			return OnKeyDown(MyGeometry, InKeyEvent);
		});
		ControlRigEditor.Pin()->OnGetViewportContextMenu().BindSP(this, &SRigHierarchy::GetContextMenu);
		ControlRigEditor.Pin()->OnViewportContextMenuCommands().BindSP(this, &SRigHierarchy::GetContextMenuCommands);
		ControlRigEditor.Pin()->OnEditorClosed().AddSP(this, &SRigHierarchy::OnEditorClose);
		ControlRigEditor.Pin()->OnRequestNavigateToConnectorWarning().AddSP(this, &SRigHierarchy::OnNavigateToFirstConnectorWarning);
	}
	
	CreateContextMenu();
	CreateDragDropMenu();

	// after opening the editor the debugged rig won't exist yet. we'll have to wait for a tick so
	// that we have a valid rig to listen to.
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, float) {
		if(ControlRigBlueprint.IsValid())
		{
			(void)HandleSetObjectBeingDebugged(ControlRigBlueprint->GetDebuggedControlRig());
		}
		return EActiveTimerReturnType::Stop;
	}));
}

void SRigHierarchy::OnEditorClose(IControlRigBaseEditor* InEditor, FControlRigAssetInterfacePtr InBlueprint)
{
	using namespace UE::ControlRigEditor;

	if (InEditor)
	{
		UControlRig* ControlRig = InEditor->GetControlRig();
		if (ControlRig && TreeView.IsValid())
		{
			// Store the tree state so it gets restored when the editor is opened anew
			FRigHierarchyTreePersistentStateStore::Get().StorePersistentState(*ControlRig, *TreeView, ViewName);
		}

		InEditor->GetKeyDownDelegate().Unbind();
		InEditor->OnGetViewportContextMenu().Unbind();
		InEditor->OnViewportContextMenuCommands().Unbind();
		InEditor->OnEditorClosed().RemoveAll(this);
	}

	if (InBlueprint)
	{
		InBlueprint->GetHierarchy()->OnModified().RemoveAll(this);
		InBlueprint->GetRigVMAssetInterface()->OnRefreshEditor().RemoveAll(this);
		InBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().RemoveAll(this);

		if(UModularRigController* ModularRigController = InBlueprint->GetModularRigController())
		{
			ModularRigController->OnModified().RemoveAll(this);
		}

		using namespace UE::ControlRigEditor;
	}

	ControlRigEditor.Reset();
	ControlRigBlueprint.Reset();
}

void SRigHierarchy::BindCommands()
{
	// create new command
	const FControlRigHierarchyCommands& Commands = FControlRigHierarchyCommands::Get();

	CommandList->MapAction(Commands.AddBoneItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleNewItem, ERigElementType::Bone, false),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanAddElement, ERigElementType::Bone));

	CommandList->MapAction(Commands.AddControlItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleNewItem, ERigElementType::Control, false),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanAddElement, ERigElementType::Control));

	CommandList->MapAction(Commands.AddAnimationChannelItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleNewItem, ERigElementType::Control, true),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanAddAnimationChannel));

	CommandList->MapAction(Commands.AddNullItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleNewItem, ERigElementType::Null, false),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanAddElement, ERigElementType::Null));

	CommandList->MapAction(Commands.AddConnectorItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleNewItem, ERigElementType::Connector, false),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanAddElement, ERigElementType::Connector));

	CommandList->MapAction(Commands.AddSocketItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleNewItem, ERigElementType::Socket, false),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanAddElement, ERigElementType::Socket));

	CommandList->MapAction(Commands.FindReferencesOfItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleFindReferencesOfItem),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanFindReferencesOfItem));

	CommandList->MapAction(Commands.DuplicateItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleDuplicateItem),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanDuplicateItem));

	CommandList->MapAction(Commands.MirrorItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleMirrorItem),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanDuplicateItem));

	CommandList->MapAction(Commands.DeleteItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleDeleteItem),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanDeleteItem));

	CommandList->MapAction(Commands.RenameItem,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleRenameItem),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanRenameItem));

	CommandList->MapAction(Commands.CopyItems,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleCopyItems),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanCopyOrPasteItems));

	CommandList->MapAction(Commands.PasteItems,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandlePasteItems),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanPasteItems));

	CommandList->MapAction(Commands.PasteLocalTransforms,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandlePasteLocalTransforms),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanCopyOrPasteItems));

	CommandList->MapAction(Commands.PasteGlobalTransforms,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandlePasteGlobalTransforms),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanCopyOrPasteItems));

	CommandList->MapAction(
		Commands.ResetTransform,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleResetTransform, true),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::IsMultiSelected, true));

	CommandList->MapAction(
		Commands.ResetAllTransforms,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleResetTransform, false));

	CommandList->MapAction(
		Commands.ZeroControls,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleZeroControls),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanHandleZeroControls));

	CommandList->MapAction(
		Commands.ZeroControlsFromClosestBone,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleZeroControlsFromClosestBone),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanHandleZeroControlsFromClosestBone));

	CommandList->MapAction(
		Commands.ZeroControlShape,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleZeroControlShape),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanHandleZeroControlShape));

	CommandList->MapAction(
		Commands.SetInitialTransformFromCurrent,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleSetInitialTransformsAsCurrent),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::CanHandleSetInitialTransformsAsCurrent));

	constexpr bool bFrameInViewport = true;
	CommandList->MapAction(
		Commands.FrameSelection,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleFrameSelection, bFrameInViewport),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::IsMultiSelected, true));

	CommandList->MapAction(
		Commands.ControlBoneTransform,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleControlBoneOrSpaceTransform),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::IsSingleBoneSelected, false),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &SRigHierarchy::IsSingleBoneSelected, false)
		);

	CommandList->MapAction(
		Commands.Unparent,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleUnparent),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::IsMultiSelected, false));

	CommandList->MapAction(
		Commands.FilteringFlattensHierarchy,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bFlattenHierarchyOnFilter = !DisplaySettings.bFlattenHierarchyOnFilter; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bFlattenHierarchyOnFilter; }));

	CommandList->MapAction(
		Commands.HideParentsWhenFiltering,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bHideParentsOnFilter = !DisplaySettings.bHideParentsOnFilter; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bHideParentsOnFilter; }));

	CommandList->MapAction(
		Commands.ShowImportedBones,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowImportedBones = !DisplaySettings.bShowImportedBones; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowImportedBones; }));
	
	CommandList->MapAction(
		Commands.ShowBones,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowBones = !DisplaySettings.bShowBones; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowBones; }));

	CommandList->MapAction(
		Commands.ShowControls,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowControls = !DisplaySettings.bShowControls; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowControls; }));
	
	CommandList->MapAction(
		Commands.ShowNulls,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowNulls = !DisplaySettings.bShowNulls; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowNulls; }));

	CommandList->MapAction(
		Commands.ShowReferences,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowReferences = !DisplaySettings.bShowReferences; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowReferences; }));

	CommandList->MapAction(
		Commands.ShowSockets,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowSockets = !DisplaySettings.bShowSockets; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowSockets; }));

	CommandList->MapAction(
		Commands.ToggleControlShapeTransformEdit,
		FExecuteAction::CreateLambda([this]()
		{
			ControlRigEditor.Pin()->GetEditMode()->ToggleControlShapeTransformEdit();
		}));

	CommandList->MapAction(
		Commands.SpaceSwitching,
		FExecuteAction::CreateSP(this, &SRigHierarchy::HandleTestSpaceSwitching),
		FCanExecuteAction::CreateSP(this, &SRigHierarchy::IsControlSelected, true));

	CommandList->MapAction(
		Commands.ShowComponents,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowComponents = !DisplaySettings.bShowComponents; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowComponents; }));

	CommandList->MapAction(
		Commands.ShowIconColors,
		FExecuteAction::CreateLambda([this]() { DisplaySettings.bShowIconColors = !DisplaySettings.bShowIconColors; RefreshTreeView(); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() { return DisplaySettings.bShowIconColors; }));
}

FReply SRigHierarchy::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SRigHierarchy::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
	if(Reply.IsEventHandled())
	{
		return Reply;
	}

	if(MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		if(const TSharedPtr<FRigHierarchyTreeElement>* ItemPtr = TreeView->FindItemAtPosition(MouseEvent.GetScreenSpacePosition()))
		{
			if(const TSharedPtr<FRigHierarchyTreeElement>& Item = *ItemPtr)
			{
				if (URigHierarchy* Hierarchy = GetHierarchy())
				{
					if(Item->Key.IsElement())
					{
						TArray<FRigElementKey> KeysToSelect = {Item->Key.GetElement()};
						KeysToSelect.Append(Hierarchy->GetChildren(Item->Key.GetElement(), true));

						URigHierarchyController* Controller = Hierarchy->GetController(true);
						check(Controller);
			
						Controller->SetSelection(KeysToSelect);
					}
					else
					{
						// todo: component selection
					}
				}
			}
		}
	}

	return FReply::Unhandled();
}

EVisibility SRigHierarchy::IsToolbarVisible() const
{
	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		if (Hierarchy->Num(ERigElementType::Bone) > 0)
		{
			return EVisibility::Collapsed;
		}
	}
	return EVisibility::Visible;
}

EVisibility SRigHierarchy::IsSearchbarVisible() const
{
	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		if ((Hierarchy->Num(ERigElementType::Bone) +
			Hierarchy->Num(ERigElementType::Null) +
			Hierarchy->Num(ERigElementType::Control) +
			Hierarchy->Num(ERigElementType::Connector) +
			Hierarchy->Num(ERigElementType::Socket)) > 0)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

FReply SRigHierarchy::OnImportSkeletonClicked()
{
	TSharedPtr<FStructOnScope> StructToDisplay = MakeShareable(new FStructOnScope(FRigHierarchyImportSettings::StaticStruct(), (uint8*)&ImportSettings));
#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<SKismetInspector> KismetInspector = SNew(SKismetInspector);
#else
	TSharedRef<SRigVMDetailsInspector> KismetInspector = SNew(SRigVMDetailsInspector);
#endif
	KismetInspector->ShowSingleStruct(StructToDisplay);

	SGenericDialogWidget::FArguments DialogArguments;
	DialogArguments.OnOkPressed_Lambda([this] ()
	{
		if (ImportSettings.Mesh != nullptr)
		{
			if(ControlRigBlueprint->IsControlRigModule())
			{
				UpdateMesh(ImportSettings.Mesh, true);
			}
			else
			{
				ImportHierarchy(FAssetData(ImportSettings.Mesh));
			}
		}
	});

	constexpr bool bAsModalDialog = false;
	SGenericDialogWidget::OpenDialog(LOCTEXT("ControlRigHierarchyImport", "Import Hierarchy"), KismetInspector, DialogArguments, bAsModalDialog);

	return FReply::Handled();
}

FText SRigHierarchy::GetImportHierarchyText() const
{
	if (ControlRigBlueprint.IsValid())
	{
		if(ControlRigBlueprint->IsControlRigModule())
		{
			return LOCTEXT("SetPreviewMesh", "Set Preview Mesh");
		}
	}
	return LOCTEXT("ImportHierarchy", "Import Hierarchy");
}

bool SRigHierarchy::IsImportHierarchyEnabled() const
{
	// for now we'll enable this always
	return true;
}

void SRigHierarchy::OnFilterTextChanged(const FText& SearchText)
{
	DisplaySettings.FilterText = SearchText;
	RefreshTreeView();
}

void SRigHierarchy::RefreshTreeView(bool bRebuildContent)
{
	using namespace UE::ControlRigEditor;

	const URigHierarchy* Hierarchy = GetHierarchy();
	if(Hierarchy)
	{
		// is the rig currently running
		if(Hierarchy->HasExecuteContext())
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, bRebuildContent]()
			{
				RefreshTreeView(bRebuildContent);
			});
		}
	}

	bIsChangingRigHierarchy = true;
	
	TreeView->RefreshTreeView(bRebuildContent,
		FOnRigHierarchyTreePersistentStateRestored::CreateLambda([WeakThis = AsWeak(), this]()
			{
				if (WeakThis.IsValid())
				{
					bIsChangingRigHierarchy = false;
				}
			})
	);
}

TArray<FRigHierarchyKey> SRigHierarchy::GetSelectedKeys() const
{
	TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	
	TArray<FRigHierarchyKey> SelectedKeys;
	for (const TSharedPtr<FRigHierarchyTreeElement>& SelectedItem : SelectedItems)
	{
		if(SelectedItem->Key.IsValid())
		{
			SelectedKeys.AddUnique(SelectedItem->Key);
		}
	}

	return SelectedKeys;
}

TArray<FRigElementKey> SRigHierarchy::GetSelectedElementKeys() const
{
	TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();
	TArray<FRigElementKey> ElementKeys;
	ElementKeys.Reserve(SelectedKeys.Num());
	for(const FRigHierarchyKey& SelectedKey : SelectedKeys)
	{
		if(SelectedKey.IsElement())
		{
			ElementKeys.Add(SelectedKey.GetElement());
		}
	}
	return ElementKeys;
}

void SRigHierarchy::OnSelectionChanged(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	if (bIsChangingRigHierarchy ||
		SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	TreeView->ClearHighlightedItems();

	// an element to use for the control rig editor's detail panel
	FRigHierarchyKey LastSelectedElement;

	URigHierarchy* Hierarchy = GetHierarchy();
	if (Hierarchy)
	{
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);
		
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);

		// flag to guard during selection changes.
		// in case there's no editor we'll use the local variable.
		bool bDummySuspensionFlag = false;
		bool* SuspensionFlagPtr = &bDummySuspensionFlag;
		if (ControlRigEditor.IsValid())
		{
			SuspensionFlagPtr = &ControlRigEditor.Pin()->GetSuspendDetailsPanelRefreshFlag();
		}

		TGuardValue<bool> SuspendDetailsPanelRefreshGuard(*SuspensionFlagPtr, true);
		
		const TArray<FRigHierarchyKey> NewSelection = GetSelectedKeys();
		if(!Controller->SetHierarchySelection(NewSelection, true))
		{
			return;
		}

		if (NewSelection.Num() > 0)
		{
			if (ControlRigEditor.IsValid())
			{
				if (ControlRigEditor.Pin()->GetEventQueueComboValue() == 1)
				{
					HandleControlBoneOrSpaceTransform();
				}
			}

			LastSelectedElement = NewSelection.Last();
		}
	}

	if (ControlRigEditor.IsValid())
	{
		if(LastSelectedElement.IsValid())
		{
			ControlRigEditor.Pin()->SetDetailViewForRigElements();
		}
		else
		{
			ControlRigEditor.Pin()->ClearDetailObject();
		}
	}
}

void SRigHierarchy::OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if(!ControlRigBlueprint.IsValid())
	{
		return;
	}
	
	if (ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications)
	{
		return;
	}

	if (bIsChangingRigHierarchy || bIsConstructionEventRunning)
	{
		return;
	}

	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;

	if(InElement)
	{
		if(InElement->IsTypeOf(ERigElementType::Curve))
		{
			return;
		}
	}

	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementAdded:
		{
			if(InElement)
			{
				if(TreeView->AddElement(InElement))
				{
					RefreshTreeView(false);
				}
			}
			break;
		}
		case ERigHierarchyNotification::ElementRemoved:
		{
			if(InElement)
			{
				if(TreeView->RemoveElement(InElement->GetKey()))
				{
					RefreshTreeView(false);
				}
			}
			break;
		}
		case ERigHierarchyNotification::ComponentAdded:
		{
			if(InComponent)
			{
				if(TreeView->AddComponent(InComponent))
				{
					RefreshTreeView(false);
				}
			}
			break;
		}
		case ERigHierarchyNotification::ComponentRemoved:
		{
			if(InComponent)
			{
				if(TreeView->RemoveElement(InComponent->GetKey()))
				{
					RefreshTreeView(false);
				}
			}
			break;
		}
		case ERigHierarchyNotification::ParentChanged:
		{
			check(InHierarchy);
			if(InElement)
			{
				const FRigElementKey ParentKey = InHierarchy->GetFirstParent(InElement->GetKey());
				if(TreeView->ReparentElement(InElement->GetKey(), ParentKey))
				{
					RefreshTreeView(false);
				}
			}
			break;
		}
		case ERigHierarchyNotification::ParentWeightsChanged:
		{
			if(URigHierarchy* Hierarchy = GetHierarchy())
			{
				if(InElement)
				{
					TArray<FRigElementWeight> ParentWeights = Hierarchy->GetParentWeightArray(InElement->GetKey());
					if(ParentWeights.Num() > 0)
					{
						TArray<FRigElementKey> ParentKeys = Hierarchy->GetParents(InElement->GetKey());
						check(ParentKeys.Num() == ParentWeights.Num());
						for(int32 ParentIndex=0;ParentIndex<ParentKeys.Num();ParentIndex++)
						{
							if(ParentWeights[ParentIndex].IsAlmostZero())
							{
								continue;
							}

							if(TreeView->ReparentElement(InElement->GetKey(), ParentKeys[ParentIndex]))
							{
								RefreshTreeView(false);
							}
							break;
						}
					}
				}
			}
			break;
		}
		case ERigHierarchyNotification::ElementRenamed:
		case ERigHierarchyNotification::ElementReordered:
		case ERigHierarchyNotification::HierarchyReset:
		case ERigHierarchyNotification::ComponentRenamed:
		case ERigHierarchyNotification::ComponentReparented:
		{
			RefreshTreeView();
			break;
		}
		case ERigHierarchyNotification::ElementSelected:
		case ERigHierarchyNotification::ElementDeselected:
		case ERigHierarchyNotification::ComponentSelected:
		case ERigHierarchyNotification::ComponentDeselected:
		{
			if(InElement || InComponent)
			{
				const bool bSelected =
					(InNotif == ERigHierarchyNotification::ElementSelected) || 
					(InNotif == ERigHierarchyNotification::ComponentSelected);;

				const FRigHierarchyKey Key =
					InElement ?
					FRigHierarchyKey(InElement->GetKey()) :
					FRigHierarchyKey(InComponent->GetKey()) ;
					
				for (int32 RootIndex = 0; RootIndex < TreeView->RootElements.Num(); ++RootIndex)
				{
					TSharedPtr<FRigHierarchyTreeElement> Found = TreeView->FindElement(Key, TreeView->RootElements[RootIndex]);
					if (Found.IsValid())
					{
						TreeView->SetItemSelection(Found, bSelected, ESelectInfo::OnNavigation);

						if(GetDefault<UPersonaOptions>()->bExpandTreeOnSelection && bSelected)
						{
							constexpr bool bFrameInViewport = false;
							HandleFrameSelection(bFrameInViewport);
						}

						if (ControlRigEditor.IsValid() && !GIsTransacting)
						{
							if (ControlRigEditor.Pin()->GetEventQueueComboValue() == 1)
							{
								TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
								HandleControlBoneOrSpaceTransform();
							}
						}
					}
				}
			}
			break;
		}
		case ERigHierarchyNotification::ControlSettingChanged:
		case ERigHierarchyNotification::ConnectorSettingChanged:
		case ERigHierarchyNotification::SocketColorChanged:
		{
			// update color and other settings of the item
			if(InElement && (
				(InElement->GetType() == ERigElementType::Control) ||
				(InElement->GetType() == ERigElementType::Connector) ||
				(InElement->GetType() == ERigElementType::Socket)))
			{
				for (int32 RootIndex = 0; RootIndex < TreeView->RootElements.Num(); ++RootIndex)
				{
					TSharedPtr<FRigHierarchyTreeElement> TreeElement = TreeView->FindElement(InElement->GetKey(), TreeView->RootElements[RootIndex]);
					if (TreeElement.IsValid())
					{
						const FRigHierarchyTreeDisplaySettings& Settings = TreeView->GetRigTreeDelegates().GetDisplaySettings();
						TreeElement->RefreshDisplaySettings(InHierarchy, Settings);
					}
				}
			}
			break;
		}
		case ERigHierarchyNotification::ComponentContentChanged:
		{
			if(InComponent)
			{
				for (int32 RootIndex = 0; RootIndex < TreeView->RootElements.Num(); ++RootIndex)
				{
					TSharedPtr<FRigHierarchyTreeElement> TreeElement = TreeView->FindElement(InComponent->GetKey(), TreeView->RootElements[RootIndex]);
					if (TreeElement.IsValid())
					{
						const FRigHierarchyTreeDisplaySettings& Settings = TreeView->GetRigTreeDelegates().GetDisplaySettings();
						TreeElement->RefreshDisplaySettings(InHierarchy, Settings);
					}
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

void SRigHierarchy::OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if(bIsChangingRigHierarchy)
	{
		return;
	}
	
	if(!ControlRigBeingDebuggedPtr.IsValid())
	{
		return;
	}

	if(InHierarchy != ControlRigBeingDebuggedPtr->GetHierarchy())
	{
		return;
	}

	if(bIsConstructionEventRunning)
	{
		return;
	}

	const FRigBaseElement* InElement = InSubject.Element;
	const FRigBaseComponent* InComponent = InSubject.Component;

	if(IsInGameThread())
	{
		OnHierarchyModified(InNotif, InHierarchy, InSubject);
	}
	else
	{
		FRigElementKey Key;
		if(InElement)
		{
			Key = InElement->GetKey();
		}

		TWeakObjectPtr<URigHierarchy> WeakHierarchy = InHierarchy;

		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, InNotif, WeakHierarchy, Key]()
        {
            if(!WeakHierarchy.IsValid())
            {
                return;
            }
            if (const FRigBaseElement* Element = WeakHierarchy.Get()->Find(Key))
            {
	            OnHierarchyModified(InNotif, WeakHierarchy.Get(), Element);
            }
			
        });
	}
}

void SRigHierarchy::OnModularRigModified(EModularRigNotification InNotif, const FRigModuleReference* InModule)
{
	using namespace UE::ControlRigEditor;

	if(!ControlRigBlueprint.IsValid())
	{
		return;
	}

	switch(InNotif)
	{
		case EModularRigNotification::ModuleSelected:
		case EModularRigNotification::ModuleDeselected:
		{
			const bool bSelected = InNotif == EModularRigNotification::ModuleSelected;
			if(ControlRigEditor.IsValid())
			{
				if(UControlRig* ControlRig = ControlRigEditor.Pin()->GetControlRig())
				{
					if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
					{
						TArray<FRigElementKey> Keys = Hierarchy->GetAllKeys();
						Keys = Keys.FilterByPredicate([Hierarchy, InModule](const FRigElementKey& InKey)
						{
							return InModule->Name == Hierarchy->GetModuleFName(InKey);
						});

						bool bScrollIntoView = !FRigHierarchyTreePersistentStateStore::IsFeatureEnabled();

						for(const FRigElementKey& Key : Keys)
						{
							if(const TSharedPtr<FRigHierarchyTreeElement> TreeElement = TreeView->FindElement(Key))
							{
								TreeView->SetItemHighlighted(TreeElement, bSelected);
								if(bScrollIntoView)
								{
									TreeView->RequestScrollIntoView(TreeElement);
									bScrollIntoView = false;
								}
							}
						}
					}
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

void SRigHierarchy::HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}
	RefreshTreeView();
}

void SRigHierarchy::HandleSetObjectBeingDebugged(UObject* InObject)
{
	if(ControlRigBeingDebuggedPtr.Get() == InObject)
	{
		return;
	}

	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(UControlRig* ControlRigBeingDebugged = ControlRigBeingDebuggedPtr.Get())
		{
			if(!URigVMHost::IsGarbageOrDestroyed(ControlRigBeingDebugged))
			{
				ControlRigBeingDebugged->GetHierarchy()->OnModified().RemoveAll(this);
			}
		}
	}

	ControlRigBeingDebuggedPtr.Reset();
	
	if(UControlRig* ControlRig = Cast<UControlRig>(InObject))
	{
		ControlRigBeingDebuggedPtr = ControlRig;
		if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->OnModified().RemoveAll(this);
			Hierarchy->OnModified().AddSP(this, &SRigHierarchy::OnHierarchyModified_AnyThread);
		}
		ControlRig->OnPreConstructionForUI_AnyThread().RemoveAll(this);
		ControlRig->OnPreConstructionForUI_AnyThread().AddSP(this, &SRigHierarchy::OnPreConstruction_AnyThread);
		ControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
		ControlRig->OnPostConstruction_AnyThread().AddSP(this, &SRigHierarchy::OnPostConstruction_AnyThread);
		LastHierarchyHash = INDEX_NONE;
	}

	RefreshTreeView();
}

void SRigHierarchy::OnPreConstruction_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	if(InRig != ControlRigBeingDebuggedPtr.Get())
	{
		return;
	}
	bIsConstructionEventRunning = true;
	SelectionBeforeConstruction = InRig->GetHierarchy()->GetSelectedHierarchyKeys();
}

void SRigHierarchy::OnPostConstruction_AnyThread(UControlRig* InRig, const FName& InEventName)
{
	if(InRig != ControlRigBeingDebuggedPtr.Get())
	{
		return;
	}

	bIsConstructionEventRunning = false;

	const URigHierarchy* Hierarchy = InRig->GetHierarchy();
	const int32 HierarchyHash = HashCombine(
		Hierarchy->GetTopologyHash(false),
		InRig->ElementKeyRedirector.GetHash());

	if(LastHierarchyHash != HierarchyHash)
	{
		LastHierarchyHash = HierarchyHash;
		
		auto Task = [this]()
		{
			TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);

			RefreshTreeView(true);

			TreeView->ClearSelection();
			if(!SelectionBeforeConstruction.IsEmpty())
			{
				for (int32 RootIndex = 0; RootIndex < TreeView->RootElements.Num(); ++RootIndex)
				{
					for(const FRigHierarchyKey& Key : SelectionBeforeConstruction)
					{
						TSharedPtr<FRigHierarchyTreeElement> Found = TreeView->FindElement(Key, TreeView->RootElements[RootIndex]);
						if (Found.IsValid())
						{
							TreeView->SetItemSelection(Found, true, ESelectInfo::OnNavigation);
						}
					}
				}
			}
		};
				
		if(IsInGameThread())
		{
			Task();
		}
		else
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [Task]()
			{
				Task();
			});
		}
	}
}

void SRigHierarchy::OnNavigateToFirstConnectorWarning()
{
	if(ControlRigEditor.IsValid())
	{
		if(UControlRig* ControlRig = ControlRigEditor.Pin()->GetControlRig())
		{
			FRigElementKey ConnectorKey;
			if(!ControlRig->AllConnectorsAreResolved(nullptr, &ConnectorKey))
			{
				if(ConnectorKey.IsValid())
				{
					if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
					{
						if(URigHierarchyController* HierarchyController = Hierarchy->GetController())
						{
							{
								const FRigHierarchyRedirectorGuard RedirectorGuard(ControlRig);
								HierarchyController->SetSelection({ConnectorKey}, false);
							}

							constexpr bool bFrameInViewport = false;
							HandleFrameSelection(bFrameInViewport);
						}
					}
				}
			}
		}
	}
}

void SRigHierarchy::ClearDetailPanel() const
{
	if(ControlRigEditor.IsValid())
	{
		ControlRigEditor.Pin()->ClearDetailObject();
	}
}

TSharedRef< SWidget > SRigHierarchy::CreateFilterMenu()
{
	const FControlRigHierarchyCommands& Actions = FControlRigHierarchyCommands::Get();

	const bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

	MenuBuilder.BeginSection("FilterOptions", LOCTEXT("OptionsMenuHeading", "Options"));
	{
		MenuBuilder.AddMenuEntry(Actions.FilteringFlattensHierarchy);
		MenuBuilder.AddMenuEntry(Actions.HideParentsWhenFiltering);

		const UEnum* NameModeEnum = StaticEnum<EElementNameDisplayMode>();
		TArray<int32> EnumValueSubset;
		EnumValueSubset.Reserve(NameModeEnum->NumEnums() - 1);
		for (int32 i = 0; i < NameModeEnum->NumEnums() - 1; i++)
		{
			const int32 Value = static_cast<int32>(NameModeEnum->GetValueByIndex(i));
			if(Value != static_cast<int32>(EElementNameDisplayMode::AssetDefault))
			{
				EnumValueSubset.Add(Value);
			}
		}

		MenuBuilder.AddWidget(
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(3)
			[
				SNew(SEnumComboBox, NameModeEnum)
				.EnumValueSubset(EnumValueSubset)
				.CurrentValue_Lambda([this]() -> int32
				{
					const IControlRigBaseEditor* StrongEditor = ControlRigEditor.Pin().Get();
					check(StrongEditor);
					
					if (FControlRigAssetInterfacePtr Blueprint = StrongEditor->GetControlRigAssetInterface())
					{
						return static_cast<int32>(Blueprint->GetHierarchySettings().ElementNameDisplayMode);
					}
					
					return static_cast<int32>(EElementNameDisplayMode::AssetDefault);
				})
				.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type)
				{
					IControlRigBaseEditor* StrongEditor = ControlRigEditor.Pin().Get();
					check(StrongEditor);
										
					if (FControlRigAssetInterfacePtr Blueprint = StrongEditor->GetControlRigAssetInterface())
					{
						FScopedTransaction Transaction(LOCTEXT("HierarchySetElementNameDisplayMode", "Change Name Mode"));
						Blueprint->Modify();
						Blueprint->GetHierarchySettings().ElementNameDisplayMode = static_cast<EElementNameDisplayMode>(InEnumValue);
						StrongEditor->Compile();
					}
				})
			],
			LOCTEXT("ElementNameDisplayMode", "Name Mode"),
			false,
			true,
			LOCTEXT("NameModeToolTip", "Defines how the names of the elements will be shown in the tree view")
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("FilterBones", LOCTEXT("BonesMenuHeading", "Bones"));
	{
		MenuBuilder.AddMenuEntry(Actions.ShowImportedBones);
		MenuBuilder.AddMenuEntry(Actions.ShowBones);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("FilterControls", LOCTEXT("ControlsMenuHeading", "Controls"));
	{
		MenuBuilder.AddMenuEntry(Actions.ShowControls);
		MenuBuilder.AddMenuEntry(Actions.ShowNulls);
		MenuBuilder.AddMenuEntry(Actions.ShowComponents);
		MenuBuilder.AddMenuEntry(Actions.ShowIconColors);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedPtr< SWidget > SRigHierarchy::CreateContextMenuWidget()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (UToolMenu* Menu = GetContextMenu())
	{
		return ToolMenus->GenerateWidget(Menu);
	}
	
	return SNullWidget::NullWidget;
}

void SRigHierarchy::OnItemClicked(TSharedPtr<FRigHierarchyTreeElement> InItem)
{
	URigHierarchy* Hierarchy = GetHierarchy();
	check(Hierarchy);

	if (Hierarchy->IsHierarchyKeySelected(InItem->Key))
	{
		if (ControlRigEditor.IsValid())
		{
			ControlRigEditor.Pin()->SetDetailViewForRigElements();
		}

		if(InItem->Key.IsElement())
		{
			if (InItem->Key.GetElement().Type == ERigElementType::Bone)
			{
				if(FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(InItem->Key.GetElement()))
				{
					if (BoneElement->BoneType == ERigBoneType::Imported)
					{
						return;
					}
				}
			}
		}

		uint32 CurrentCycles = FPlatformTime::Cycles();
		double SecondsPassed = double(CurrentCycles - TreeView->LastClickCycles) * FPlatformTime::GetSecondsPerCycle();
		if (SecondsPassed > 0.5f)
		{
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, float) {
				HandleRenameItem();
				return EActiveTimerReturnType::Stop;
			}));
		}

		TreeView->LastClickCycles = CurrentCycles;
	}
}

void SRigHierarchy::OnItemDoubleClicked(TSharedPtr<FRigHierarchyTreeElement> InItem)
{
	if (TreeView->IsItemExpanded(InItem))
	{
		TreeView->SetExpansionRecursive(InItem, false, false);
	}
	else
	{
		TreeView->SetExpansionRecursive(InItem, false, true);
	}
}

void SRigHierarchy::OnSetExpansionRecursive(TSharedPtr<FRigHierarchyTreeElement> InItem, bool bShouldBeExpanded)
{
	TreeView->SetExpansionRecursive(InItem, false, bShouldBeExpanded);
}

TOptional<FText> SRigHierarchy::OnGetItemTooltip(const FRigHierarchyKey& InKey) const
{
	if(!DragRigResolveResults.IsEmpty() && FSlateApplication::Get().IsDragDropping())
	{
		FString Message;
		if(InKey.IsElement())
		{
			for(const TPair<FRigElementKey, FModularRigResolveResult>& DragRigResolveResult : DragRigResolveResults)
			{
				if(DragRigResolveResult.Key != InKey.GetElement())
				{
					if(!DragRigResolveResult.Value.ContainsMatch(InKey.GetElement(), &Message))
					{
						if(!Message.IsEmpty())
						{
							return FText::FromString(Message);
						}
					}
				}
			}
		}
	}
	return TOptional<FText>();
}

void SRigHierarchy::CreateDragDropMenu()
{
	static bool bCreatedMenu = false;
	if(bCreatedMenu)
	{
		return;
	}
	bCreatedMenu = true;

	const FName MenuName = DragDropMenuName;
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ensure(ToolMenus))
	{
		return;
	}

	if (UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName))
	{
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UToolMenus* ToolMenus = UToolMenus::Get();
				UControlRigContextMenuContext* MainContext = InMenu->FindContext<UControlRigContextMenuContext>();
				
				if (SRigHierarchy* RigHierarchyPanel = MainContext->GetRigHierarchyPanel())
				{
					FToolMenuEntry ParentEntry = FToolMenuEntry::InitMenuEntry(
                        TEXT("Parent"),
                        LOCTEXT("DragDropMenu_Parent", "Parent"),
                        LOCTEXT("DragDropMenu_Parent_ToolTip", "Parent Selected Items to the Target Item"),
                        FSlateIcon(),
                        FToolMenuExecuteAction::CreateSP(RigHierarchyPanel, &SRigHierarchy::HandleParent)
                    );
		
                    ParentEntry.InsertPosition.Position = EToolMenuInsertType::First;
                    InMenu->AddMenuEntry(NAME_None, ParentEntry);
		
                    UToolMenu* AlignMenu = InMenu->AddSubMenu(
                        ToolMenus->CurrentOwner(),
                        NAME_None,
                        TEXT("Align"),
                        LOCTEXT("DragDropMenu_Align", "Align"),
                        LOCTEXT("DragDropMenu_Align_ToolTip", "Align Selected Items' Transforms to Target Item's Transform")
                    );
		
                    if (FToolMenuSection* DefaultSection = InMenu->FindSection(NAME_None))
                    {
                        if (FToolMenuEntry* AlignMenuEntry = DefaultSection->FindEntry(TEXT("Align")))
                        {
                            AlignMenuEntry->InsertPosition.Name = ParentEntry.Name;
                            AlignMenuEntry->InsertPosition.Position = EToolMenuInsertType::After;
                        }
                    }
		
                    FToolMenuEntry AlignAllEntry = FToolMenuEntry::InitMenuEntry(
                        TEXT("All"),
                        LOCTEXT("DragDropMenu_Align_All", "All"),
                        LOCTEXT("DragDropMenu_Align_All_ToolTip", "Align Selected Items' Transforms to Target Item's Transform"),
                        FSlateIcon(),
                        FToolMenuExecuteAction::CreateSP(RigHierarchyPanel, &SRigHierarchy::HandleAlign)
                    );
                    AlignAllEntry.InsertPosition.Position = EToolMenuInsertType::First;
		
                    AlignMenu->AddMenuEntry(NAME_None, AlignAllEntry);	
				}
			})
		);
	}
}

UToolMenu* SRigHierarchy::GetDragDropMenu(const TArray<FRigHierarchyKey>& DraggedKeys, FRigElementKey TargetKey)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ensure(ToolMenus))
	{
		return nullptr;
	}
	
	const FName MenuName = DragDropMenuName;
	UControlRigContextMenuContext* MenuContext = NewObject<UControlRigContextMenuContext>();
	FControlRigMenuSpecificContext MenuSpecificContext;
	MenuSpecificContext.RigHierarchyDragAndDropContext = FControlRigRigHierarchyDragAndDropContext(DraggedKeys, TargetKey);
	MenuSpecificContext.RigHierarchyPanel = SharedThis(this);
	MenuContext->Init(ControlRigEditor, MenuSpecificContext);
	
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, FToolMenuContext(MenuContext));

	return Menu;
}

void SRigHierarchy::CreateContextMenu()
{
	static bool bCreatedMenu = false;
	if(bCreatedMenu)
	{
		return;
	}
	bCreatedMenu = true;
	
	const FName MenuName = ContextMenuName;

	UToolMenus* ToolMenus = UToolMenus::Get();
	
	if (!ensure(ToolMenus))
	{
		return;
	}

	if (UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName))
	{
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				UControlRigContextMenuContext* MainContext = InMenu->FindContext<UControlRigContextMenuContext>();
				
				if (SRigHierarchy* RigHierarchyPanel = MainContext->GetRigHierarchyPanel())
				{
					const FControlRigHierarchyCommands& Commands = FControlRigHierarchyCommands::Get(); 
				
					FToolMenuSection& ElementsSection = InMenu->AddSection(TEXT("Elements"), LOCTEXT("ElementsHeader", "Elements"));
					ElementsSection.AddSubMenu(TEXT("NewElement"), LOCTEXT("NewElement", "New Element"), LOCTEXT("NewElement_ToolTip", "Create New Elements"),
						FNewToolMenuDelegate::CreateLambda([Commands, RigHierarchyPanel](UToolMenu* InSubMenu)
						{
							FToolMenuSection& DefaultSection = InSubMenu->AddSection(NAME_None);
							FRigHierarchyKey SelectedKey;
							TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = RigHierarchyPanel->TreeView->GetSelectedItems();
							if (SelectedItems.Num() > 0)
							{
								SelectedKey = SelectedItems[0]->Key;
							}
							else
							{
								// we use an invalid key in case the user has clicked into the view but not on an element
								SelectedKey = FRigHierarchyKey(FRigElementKey(), true);
							}

							if(SelectedKey.IsElement())
							{
								static const FSlateIcon ControlIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.Control"));
								static const FSlateIcon NullIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.Null"));
								static const FSlateIcon BoneIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.BoneImported"));
								static const FSlateIcon SocketIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.Socket_Open"));
								static const FSlateIcon ConnectorIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.ConnectorPrimary"));
								static const FSlateIcon AnimationChannelIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Kismet.VariableList.TypeIcon"));

								const FRigElementKey& SelectedElementKey = SelectedKey.GetElement();
								if (!SelectedElementKey || SelectedElementKey.Type == ERigElementType::Bone || SelectedElementKey.Type == ERigElementType::Connector)
								{
									DefaultSection.AddMenuEntry(Commands.AddBoneItem, {}, {}, BoneIcon);
								}
								DefaultSection.AddMenuEntry(Commands.AddControlItem, {}, {}, ControlIcon);
								if(SelectedElementKey.Type == ERigElementType::Control)
								{
									DefaultSection.AddMenuEntry(Commands.AddAnimationChannelItem, {}, {}, AnimationChannelIcon);
								}
								DefaultSection.AddMenuEntry(Commands.AddNullItem, {}, {}, NullIcon);

								if(CVarControlRigHierarchyEnableModules.GetValueOnAnyThread())
								{
									DefaultSection.AddMenuEntry(Commands.AddConnectorItem, {}, {}, ConnectorIcon);
									DefaultSection.AddMenuEntry(Commands.AddSocketItem, {}, {}, SocketIcon);
								}
							}
						})
					);

					const TArray<UScriptStruct*> RigComponentStructs = FRigBaseComponent::GetAllComponentScriptStructs();
					if(!RigComponentStructs.IsEmpty())
					{
						ElementsSection.AddSubMenu(TEXT("NewComponent"), LOCTEXT("NewComponent", "New Component"), LOCTEXT("NewComponent_ToolTip", "Create New Component"),
							FNewToolMenuDelegate::CreateLambda([Commands, RigHierarchyPanel, RigComponentStructs](UToolMenu* InSubMenu)
							{
								FToolMenuSection& DefaultSection = InSubMenu->AddSection(NAME_None);
								TArray<FRigElementKey> SelectedElements;
								TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = RigHierarchyPanel->TreeView->GetSelectedItems();
								for(const TSharedPtr<FRigHierarchyTreeElement>& SelectedItem : SelectedItems)
								{
									if(SelectedItem->Key.IsElement())
									{
										SelectedElements.Add(SelectedItem->Key.GetElement());
									}
								}

								for(UScriptStruct* RigComponentStruct : RigComponentStructs)
								{
									const FString ComponentName = RigComponentStruct->GetName();
									const FText ComponentLabel = FText::Format(LOCTEXT("AddComponentLabelFormat", "Add {0}"), RigComponentStruct->GetDisplayNameText());

									const FStructOnScope StructOnScope(RigComponentStruct);
									const FRigBaseComponent* DefaultComponent = reinterpret_cast<const FRigBaseComponent*>(StructOnScope.GetStructMemory());
									const FSlateIcon& ComponentIcon = DefaultComponent->GetIconForUI();

									TSharedPtr<FString> FailureReason = MakeShared<FString>();
									FCanExecuteAction CanExecute = FCanExecuteAction::CreateLambda([RigHierarchyPanel, RigComponentStruct, SelectedElements, FailureReason]() {
										const URigHierarchy* CurrentHierarchy = RigHierarchyPanel->GetHierarchy();
										if(CurrentHierarchy == nullptr)
										{
											return false;
										}

										for(const FRigElementKey& SelectedElement : SelectedElements)
										{
											if(!CurrentHierarchy->CanAddComponent(SelectedElement, RigComponentStruct, FailureReason.Get()))
											{
												return false;
											}
										}
										return true;
									});

									DefaultSection.AddMenuEntry(
										*FString::Printf(TEXT("Add%s"), *ComponentName),
										ComponentLabel,
										TAttribute<FText>::CreateLambda([RigComponentStruct, FailureReason, CanExecute]() -> FText
										{
											FText TooltipText = RigComponentStruct->GetToolTipText();
											if(!CanExecute.Execute())
											{
												if(!FailureReason->IsEmpty())
												{
													TooltipText = FText::Format(LOCTEXT("AddComponentTooltipTextFormat", "{0}\n\n{1}"), TooltipText, FText::FromString(*FailureReason.Get()));
												}
											}
											return TooltipText;
										}),
										ComponentIcon,
										FUIAction(
											FExecuteAction::CreateLambda([RigHierarchyPanel, RigComponentStruct, SelectedElements, ComponentLabel]() {
												if(!RigHierarchyPanel->ControlRigBlueprint.IsValid())
												{
													return;
												}

												RigHierarchyPanel->DisplaySettings.bShowComponents = true;

												FScopedTransaction Transaction(ComponentLabel);
												FControlRigAssetInterfacePtr CurrentControlRigBlueprint = RigHierarchyPanel->ControlRigBlueprint.GetObject();
												URigHierarchyController* Controller = CurrentControlRigBlueprint->GetHierarchyController();
												TArray<FRigComponentKey> ComponentKeys;
												for(const FRigElementKey& SelectedElementKey : SelectedElements)
												{
													ComponentKeys.Add(Controller->AddComponent(RigComponentStruct, NAME_None, SelectedElementKey, FString(), true, true));
												}
												Controller->SetComponentSelection(ComponentKeys);
											}),
											CanExecute
										)
									);
								}
							})
						);
					}

					ElementsSection.AddMenuEntry(Commands.DeleteItem);
					ElementsSection.AddMenuEntry(Commands.DuplicateItem);
					ElementsSection.AddMenuEntry(Commands.FindReferencesOfItem);
					ElementsSection.AddMenuEntry(Commands.RenameItem);
					ElementsSection.AddMenuEntry(Commands.MirrorItem);

					if(RigHierarchyPanel->IsProceduralSelected() && RigHierarchyPanel->ControlRigBlueprint.IsValid())
					{
						ElementsSection.AddMenuEntry(
							"SelectSpawnerNode",
							LOCTEXT("SelectSpawnerNode", "Select Spawner Node"),
							LOCTEXT("SelectSpawnerNode_Tooltip", "Selects the node that spawn / added this element."),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateLambda([RigHierarchyPanel]() {
								if(!RigHierarchyPanel->ControlRigBlueprint.IsValid())
								{
									return;
								}
								const FControlRigAssetInterfacePtr CurrentControlRigBlueprint = RigHierarchyPanel->ControlRigBlueprint.GetObject();
								const TArray<const FRigBaseElement*> Elements = RigHierarchyPanel->GetHierarchy()->GetSelectedElements();
								for(const FRigBaseElement* Element : Elements)
								{
									if(Element->IsProcedural())
									{
										const int32 InstructionIndex = Element->GetCreatedAtInstructionIndex();
										if(const UControlRig* ControlRig = Cast<UControlRig>(CurrentControlRigBlueprint->GetObjectBeingDebugged()))
										{
											if(ControlRig->VM)
											{
												if(URigVMNode* Node = Cast<URigVMNode>(ControlRig->VM->GetByteCode().GetSubjectForInstruction(InstructionIndex)))
												{
													if(URigVMController* Controller = CurrentControlRigBlueprint->GetRigVMAssetInterface()->GetController(Node->GetGraph()))
													{
														Controller->SelectNode(Node);
														(void)Controller->RequestJumpToHyperlinkDelegate.ExecuteIfBound(Node);
													}
												}
											}
										}
									}
								}
							})
						));
					}

					if (RigHierarchyPanel->IsSingleBoneSelected(false) || RigHierarchyPanel->IsControlSelected(false))
					{
						FToolMenuSection& InteractionSection = InMenu->AddSection(TEXT("Interaction"), LOCTEXT("InteractionHeader", "Interaction"));
						if(RigHierarchyPanel->IsSingleBoneSelected(false))
						{
							InteractionSection.AddMenuEntry(Commands.ControlBoneTransform);
						}
						else if(RigHierarchyPanel->IsControlSelected(false))
						{
							InteractionSection.AddMenuEntry(Commands.SpaceSwitching);
						}
					}

					FToolMenuSection& CopyPasteSection = InMenu->AddSection(TEXT("Copy&Paste"), LOCTEXT("Copy&PasteHeader", "Copy & Paste"));
					CopyPasteSection.AddMenuEntry(Commands.CopyItems);
					CopyPasteSection.AddMenuEntry(Commands.PasteItems);
					CopyPasteSection.AddMenuEntry(Commands.PasteLocalTransforms);
					CopyPasteSection.AddMenuEntry(Commands.PasteGlobalTransforms);
					
					FToolMenuSection& TransformsSection = InMenu->AddSection(TEXT("Transforms"), LOCTEXT("TransformsHeader", "Transforms"));
					TransformsSection.AddMenuEntry(Commands.ResetTransform);
					TransformsSection.AddMenuEntry(Commands.ResetAllTransforms);

					{
						static const FString InitialKeyword = TEXT("Initial");
						static const FString OffsetKeyword = TEXT("Offset");
						static const FString InitialOffsetKeyword = TEXT("Initial / Offset");
						
						const FString* Keyword = &InitialKeyword;
						TArray<ERigElementType> SelectedTypes;;
						TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = RigHierarchyPanel->TreeView->GetSelectedItems();
						for(const TSharedPtr<FRigHierarchyTreeElement>& SelectedItem : SelectedItems)
						{
							if(SelectedItem.IsValid())
							{
								if(SelectedItem->Key.IsElement())
								{
									SelectedTypes.AddUnique(SelectedItem->Key.GetElement().Type);
								}
							}
						}
						if(SelectedTypes.Contains(ERigElementType::Control))
						{
							// since it is unique this means it is only controls
							if(SelectedTypes.Num() == 1)
							{
								Keyword = &OffsetKeyword;
							}
							else
							{
								Keyword = &InitialOffsetKeyword;
							}
						}

						const FText FromCurrentLabel = FText::Format(LOCTEXT("SetTransformFromCurrentTransform", "Zero {0}"), FText::FromString(*Keyword));
						const FText FromClosestBoneLabel = FText::Format(LOCTEXT("SetTransformFromClosestBone", "Zero {0} from Closest Bone"), FText::FromString(*Keyword));
						TransformsSection.AddMenuEntry(Commands.ZeroControls, FromCurrentLabel);
						TransformsSection.AddMenuEntry(Commands.ZeroControlsFromClosestBone, FromClosestBoneLabel);
					}
					
					TransformsSection.AddMenuEntry(Commands.ZeroControlShape);
					TransformsSection.AddMenuEntry(Commands.SetInitialTransformFromCurrent);

					TransformsSection.AddMenuEntry(Commands.Unparent);

					FToolMenuSection& AssetsSection = InMenu->AddSection(TEXT("Assets"), LOCTEXT("AssetsHeader", "Assets"));
					AssetsSection.AddSubMenu(TEXT("Import"), LOCTEXT("ImportSubMenu", "Import"),
						LOCTEXT("ImportSubMenu_ToolTip", "Import hierarchy to the current rig. This only imports non-existing node. For example, if there is hand_r, it won't import hand_r. If you want to reimport whole new hiearchy, delete all nodes, and use import hierarchy."),
						FNewMenuDelegate::CreateSP(RigHierarchyPanel, &SRigHierarchy::CreateImportMenu)
					);
					
					AssetsSection.AddSubMenu(TEXT("Refresh"), LOCTEXT("RefreshSubMenu", "Refresh"),
						LOCTEXT("RefreshSubMenu_ToolTip", "Refresh the existing initial transform from the selected mesh. This only updates if the node is found."),
						FNewMenuDelegate::CreateSP(RigHierarchyPanel, &SRigHierarchy::CreateRefreshMenu)
					);	

					AssetsSection.AddSubMenu(TEXT("ResetCurves"), LOCTEXT("ResetCurvesSubMenu", "Reset Curves"),
						LOCTEXT("ResetCurvesSubMenu_ToolTip", "Reset all curves in this rig asset to the selected mesh, Useful when if you add more morphs to the mesh but control rig does not update."),
						FNewMenuDelegate::CreateSP(RigHierarchyPanel, &SRigHierarchy::CreateResetCurvesMenu)
					);				
				}
			})
		);
	}
}

UToolMenu* SRigHierarchy::GetContextMenu()
{
	const FName MenuName = ContextMenuName;
	UToolMenus* ToolMenus = UToolMenus::Get();

	if(!ensure(ToolMenus))
	{
		return nullptr;
	}

	// individual entries in this menu can access members of this context, particularly useful for editor scripting
	UControlRigContextMenuContext* ContextMenuContext = NewObject<UControlRigContextMenuContext>();
	FControlRigMenuSpecificContext MenuSpecificContext;
	MenuSpecificContext.RigHierarchyPanel = SharedThis(this);
	ContextMenuContext->Init(ControlRigEditor, MenuSpecificContext);

	FToolMenuContext MenuContext(CommandList);
	MenuContext.AddObject(ContextMenuContext);

	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, MenuContext);

	return Menu;
}

TSharedPtr<FUICommandList> SRigHierarchy::GetContextMenuCommands() const
{
	return CommandList;
}

void SRigHierarchy::CreateRefreshMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddWidget(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("ControlRig.Hierarchy.Menu"))
			.Text(LOCTEXT("RefreshMesh_Title", "Select Mesh"))
			.ToolTipText(LOCTEXT("RefreshMesh_Tooltip", "Select Mesh to refresh transform from... It will refresh init transform from selected mesh. This doesn't change hierarchy. If you want to reimport hierarchy, please delete all nodes, and use import hierarchy."))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(USkeletalMesh::StaticClass())
			.OnObjectChanged(this, &SRigHierarchy::RefreshHierarchy, false, bRestrictRefreshToMeshBones)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 5.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RefreshMesh_RestrictToMeshBones", "Restrict to Mesh Bones"))
				.ToolTipText(LOCTEXT("RefreshMesh_RestrictToMeshBones_Tooltip", "It will remove any bones that does not exist in the Mesh. It might break compatibility if the Rig is used across different meshes that share a skeleton with a slightly different set of bones on each mesh."))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					return bRestrictRefreshToMeshBones ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState CheckBoxState)
				{
					bRestrictRefreshToMeshBones = CheckBoxState == ECheckBoxState::Checked;
				})
			]
		]
		,
		FText()
	);
}

void SRigHierarchy::CreateResetCurvesMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddWidget(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("ControlRig.Hierarchy.Menu"))
			.Text(LOCTEXT("ResetMesh_Title", "Select Mesh"))
			.ToolTipText(LOCTEXT("ResetMesh_Tooltip", "Select mesh to reset curves to."))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(USkeletalMesh::StaticClass())
			.OnObjectChanged(this, &SRigHierarchy::RefreshHierarchy, true, false)
		]
		,
		FText()
	);
}

void SRigHierarchy::UpdateMesh(USkeletalMesh* InMesh, const bool bImport) const
{
	if (!InMesh || !ControlRigBlueprint.IsValid() || !ControlRigEditor.IsValid())
	{
		return;
	}

	const bool bUpdateMesh = bImport ? ControlRigBlueprint->GetPreviewMesh() == nullptr : true;
	if (!bUpdateMesh)
	{
		return;
	}

	const TSharedPtr<IControlRigBaseEditor> EditorSharedPtr = ControlRigEditor.Pin();
	EditorSharedPtr->GetPersonaToolkit()->SetPreviewMesh(InMesh, true);

	UControlRigSkeletalMeshComponent* Component = Cast<UControlRigSkeletalMeshComponent>(EditorSharedPtr->GetPersonaToolkit()->GetPreviewMeshComponent());
	if (bImport)
	{
		Component->InitAnim(true);
	}

	UAnimInstance* AnimInstance = Component->GetAnimInstance();
	if (UControlRigLayerInstance* ControlRigLayerInstance = Cast<UControlRigLayerInstance>(AnimInstance))
	{
		EditorSharedPtr->SetPreviewInstance(Cast<UAnimPreviewInstance>(ControlRigLayerInstance->GetSourceAnimInstance()));
	}
	else
	{
		EditorSharedPtr->SetPreviewInstance(Cast<UAnimPreviewInstance>(AnimInstance));
	}

	EditorSharedPtr->Compile();
}

void SRigHierarchy::RefreshHierarchy(const FAssetData& InAssetData, bool bOnlyResetCurves, bool bRestrictToMeshBones)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}
	TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);

	if(!ControlRigEditor.IsValid())
	{
		return;
	}

	IControlRigBaseEditor* StrongEditor = ControlRigEditor.Pin().Get();
	StrongEditor->ClearDetailObject();

	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	USkeletalMesh* Mesh = Cast<USkeletalMesh>(InAssetData.GetAsset());
	if (Mesh && Hierarchy)
	{
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

		FScopedTransaction Transaction(LOCTEXT("HierarchyRefresh", "Refresh Transform"));

		// don't select bone if we are in construction mode.
		// we do this to avoid the editmode / viewport shapes to refresh recursively,
		// which can add an extreme slowdown depending on the number of bones (n^(n-1))
		bool bSelectBones = true;
		if (UControlRig* CurrentRig = StrongEditor->GetControlRig())
		{
			bSelectBones = !CurrentRig->IsConstructionModeEnabled();
		}

		const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();

		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		if(bOnlyResetCurves)
		{
			TArray<FRigElementKey> CurveKeys = Hierarchy->GetAllKeys(false, ERigElementType::Curve);
			for(const FRigElementKey& CurveKey : CurveKeys)
			{
				Controller->RemoveElement(CurveKey, true, true);
			}			
			Controller->ImportCurvesFromSkeletalMesh(Mesh, NAME_None, false, true, true);
		}
		else
		{
			if (bRestrictRefreshToMeshBones)
			{
				Controller->ImportBonesFromSkeletalMesh(Mesh, NAME_None, true, true, bSelectBones, true, true);
			}
			else
			{
				Controller->ImportBones(Mesh->GetSkeleton(), NAME_None, true, true, bSelectBones, true, true);
			}
			Controller->ImportCurvesFromSkeletalMesh(Mesh, NAME_None, false, true, true);
			Controller->ImportSocketsFromSkeletalMesh(Mesh, NAME_None, true, true, false, true, true);
		}
	}

	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	StrongEditor->OnHierarchyChanged();
	ControlRigBlueprint->GetRigVMAssetInterface()->BroadcastRefreshEditor();
	RefreshTreeView();
	FSlateApplication::Get().DismissAllMenus();

	static constexpr bool bImport = false;
	UpdateMesh(Mesh, bImport);
}

void SRigHierarchy::CreateImportMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddWidget(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("ControlRig.Hierarchy.Menu"))
			.Text(LOCTEXT("ImportMesh_Title", "Select Mesh"))
			.ToolTipText(LOCTEXT("ImportMesh_Tooltip", "Select Mesh to import hierarchy from... It will only import if the node doesn't exist in the current hierarchy."))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(USkeletalMesh::StaticClass())
			.OnObjectChanged(this, &SRigHierarchy::ImportHierarchy)
		]
		,
		FText()
	);
}

FRigHierarchyKey SRigHierarchy::OnGetResolvedKey(const FRigHierarchyKey& InKey)
{
	if (const FControlRigAssetInterfacePtr Blueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface())
	{
		const FRigElementKey ResolvedKey = Blueprint->GetModularRigModel().Connections.FindTargetFromConnector(InKey.GetElement());
		if(ResolvedKey.IsValid())
		{
			if(InKey.IsElement())
			{
				return ResolvedKey;
			}
			return FRigComponentKey(ResolvedKey, InKey.GetFName());
		}
	}
	return InKey;
}

void SRigHierarchy::OnRequestDetailsInspection(const FRigHierarchyKey& InKey)
{
	if(!ControlRigEditor.IsValid())
	{
		return;
	}
	ControlRigEditor.Pin()->SetDetailViewForRigElements({InKey});
}

void SRigHierarchy::ImportHierarchy(const FAssetData& InAssetData)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}

	USkeletalMesh* Mesh = Cast<USkeletalMesh>(InAssetData.GetAsset());
	if (!Mesh)
	{
		return;
	}
	
	TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);

	if (!ControlRigEditor.IsValid())
	{
		return;
	}
	
	const TSharedPtr<IControlRigBaseEditor> EditorSharedPtr = ControlRigEditor.Pin();
	if (URigHierarchy* Hierarchy = GetDefaultHierarchy())
	{
		// filter out meshes that don't contain a skeleton
		if(Mesh->GetSkeleton() == nullptr)
		{
			FNotificationInfo Info(LOCTEXT("SkeletalMeshHasNoSkeleton", "Chosen Skeletal Mesh has no assigned skeleton. This needs to fixed before the mesh can be used for a Control Rig."));
			Info.bUseSuccessFailIcons = true;
			Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
			Info.bFireAndForget = true;
			Info.bUseThrobber = true;
			Info.FadeOutDuration = 2.f;
			Info.ExpireDuration = 8.f;;
			TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
			if (NotificationPtr)
			{
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
			}
			return;
		}
		
		EditorSharedPtr->ClearDetailObject();
		
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

		FScopedTransaction Transaction(LOCTEXT("HierarchyImport", "Import Hierarchy"));

		// don't select bone if we are in construction mode.
		// we do this to avoid the editmode / viewport shapes to refresh recursively,
		// which can add an extreme slowdown depending on the number of bones (n^(n-1))
		bool bSelectBones = true;
		bool bIsModularRig = false;
		if (const UControlRig* CurrentRig = EditorSharedPtr->GetControlRig())
		{
			bSelectBones = !CurrentRig->IsConstructionModeEnabled();
			bIsModularRig = CurrentRig->IsModularRig();
		}

		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		const TArray<FRigElementKey> ImportedBones = Controller->ImportBones(Mesh->GetSkeleton(), NAME_None, false, false, bSelectBones, true, true);
		Controller->ImportCurvesFromSkeletalMesh(Mesh, NAME_None, false, true, true);
		Controller->ImportSocketsFromSkeletalMesh(Mesh, NAME_None, false, false, false, true, true);

		ControlRigBlueprint->GetSourceHierarchyImport() = Mesh->GetSkeleton();
		ControlRigBlueprint->GetSourceCurveImport() = Mesh->GetSkeleton();

		if (ImportedBones.Num() > 0)
		{
			EditorSharedPtr->GetEditMode()->FrameItems(ImportedBones);
		}
	}

	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	EditorSharedPtr->OnHierarchyChanged();
	ControlRigBlueprint->GetRigVMAssetInterface()->BroadcastRefreshEditor();
	RefreshTreeView();
	FSlateApplication::Get().DismissAllMenus();

	static constexpr bool bImport = true;
	UpdateMesh(Mesh, bImport);
}

bool SRigHierarchy::IsMultiSelected(bool bIncludeProcedural) const
{
	if(GetSelectedKeys().Num() > 0)
	{
		if(!bIncludeProcedural && IsProceduralSelected())
		{
			return false;
		}
		return true;
	}
	return false;
}

bool SRigHierarchy::IsSingleSelected(bool bIncludeProcedural) const
{
	if(GetSelectedKeys().Num() == 1)
	{
		if(!bIncludeProcedural && IsProceduralSelected())
		{
			return false;
		}
		return true;
	}
	return false;
}

bool SRigHierarchy::IsSingleBoneSelected(bool bIncludeProcedural) const
{
	if(!IsSingleSelected(bIncludeProcedural))
	{
		return false;
	}
	if(!GetSelectedKeys()[0].IsElement())
	{
		return false;
	}
	return GetSelectedKeys()[0].GetElement().Type == ERigElementType::Bone;
}

bool SRigHierarchy::IsSingleNullSelected(bool bIncludeProcedural) const
{
	if(!IsSingleSelected(bIncludeProcedural))
	{
		return false;
	}
	if(!GetSelectedKeys()[0].IsElement())
	{
		return false;
	}
	return GetSelectedKeys()[0].GetElement().Type == ERigElementType::Null;
}

bool SRigHierarchy::IsControlSelected(bool bIncludeProcedural) const
{
	if(!bIncludeProcedural && IsProceduralSelected())
	{
		return false;
	}
	
	TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();
	for (const FRigHierarchyKey& SelectedKey : SelectedKeys)
	{
		if(SelectedKey.IsElement())
		{
			if (SelectedKey.GetElement().Type == ERigElementType::Control)
			{
				return true;
			}
		}
	}
	return false;
}

bool SRigHierarchy::IsControlOrNullSelected(bool bIncludeProcedural) const
{
	if(!bIncludeProcedural && IsProceduralSelected())
	{
		return false;
	}

	TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();
	for (const FRigHierarchyKey& SelectedKey : SelectedKeys)
	{
		if(SelectedKey.IsElement())
		{
			if (SelectedKey.GetElement().Type == ERigElementType::Control)
			{
				return true;
			}
			if (SelectedKey.GetElement().Type == ERigElementType::Null)
			{
				return true;
			}
		}
	}
	return false;
}

bool SRigHierarchy::IsProceduralSelected() const
{
	TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();
	if (SelectedKeys.IsEmpty())
	{
		return false;
	}
	
	for (const FRigHierarchyKey& SelectedKey : SelectedKeys)
	{
		if(SelectedKey.IsElement() && !GetHierarchy()->IsProcedural(SelectedKey.GetElement()))
		{
			return false;
		}
		if(SelectedKey.IsComponent() && !GetHierarchy()->IsProcedural(SelectedKey.GetComponent()))
		{
			return false;
		}
	}
	return true;
}

bool SRigHierarchy::IsNonProceduralSelected() const
{
	TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();
	if (SelectedKeys.IsEmpty())
	{
		return false;
	}
	
	for (const FRigHierarchyKey& SelectedKey : SelectedKeys)
	{
		if(SelectedKey.IsElement() && GetHierarchy()->IsProcedural(SelectedKey.GetElement()))
		{
			return false;
		}
		if(SelectedKey.IsComponent() && GetHierarchy()->IsProcedural(SelectedKey.GetComponent()))
		{
			return false;
		}
	}
	return true;
}

bool SRigHierarchy::CanAddElement(const ERigElementType ElementType) const
{
	if (ElementType == ERigElementType::Connector)
	{
		return ControlRigBlueprint->IsControlRigModule();
	}
	if (ElementType == ERigElementType::Socket)
	{
		return ControlRigBlueprint->IsControlRigModule() ||
			ControlRigBlueprint->IsModularRig();
	}
	return !ControlRigBlueprint->IsControlRigModule();
}

bool SRigHierarchy::CanAddAnimationChannel() const
{
	if (!IsControlSelected(false))
	{
		return false;
	}

	return !ControlRigBlueprint->IsControlRigModule();
}

void SRigHierarchy::HandleDeleteItem()
{
	if(!ControlRigEditor.IsValid())
	{
		return;
	}

	if(!CanDeleteItem())
	{
		return;
	}

	URigHierarchy* Hierarchy = GetDefaultHierarchy();
 	if (Hierarchy)
 	{
		ClearDetailPanel();
		FScopedTransaction Transaction(LOCTEXT("HierarchyTreeDeleteSelected", "Delete selected items from hierarchy"));

		// clear detail view display
		ControlRigEditor.Pin()->ClearDetailObject();

		bool bConfirmedByUser = false;
		bool bDeleteImportedBones = false;

 		URigHierarchyController* Controller = Hierarchy->GetController(true);
 		check(Controller);

 		TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();

 		if (ControlRigBlueprint.IsValid() && ControlRigBlueprint->IsControlRigModule())
 		{
 			SelectedKeys.RemoveAll([Hierarchy, Controller](const FRigHierarchyKey& Selected)
			{
 				if(Selected.IsElement())
 				{
 					if (const FRigBaseElement* Element = Hierarchy->Find(Selected.GetElement()))
 					{
						if (const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(Element))
						{
							if (Connector->IsPrimary())
							{
								static constexpr TCHAR Format[] = TEXT("Cannot delete primary connector: %s");
								Controller->ReportAndNotifyErrorf(Format, *Connector->GetName());
								return true;
							}
						}
					}
 				}
				return false;
			});
 		}

 		// clear selection early here to make sure ControlRigEditMode can react to this deletion
 		// it cannot react to it during Controller->RemoveElement() later because bSuspendAllNotifications is true
 		Controller->ClearSelection();

 		FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);
 		
		for (const FRigHierarchyKey& SelectedKey : SelectedKeys)
		{
			TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
			TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

			if(SelectedKey.IsElement())
			{
				if (SelectedKey.GetElement().Type == ERigElementType::Bone)
				{
					if (FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(SelectedKey.GetElement()))
					{
						if (BoneElement->BoneType == ERigBoneType::Imported && BoneElement->ParentElement != nullptr)
						{
							if (!bConfirmedByUser)
							{
								FText ConfirmDelete = LOCTEXT("ConfirmDeleteBoneHierarchy", "Deleting imported(white) bones can cause issues with animation - are you sure ?");

								FSuppressableWarningDialog::FSetupInfo Info(ConfirmDelete, LOCTEXT("DeleteImportedBone", "Delete Imported Bone"), "DeleteImportedBoneHierarchy_Warning");
								Info.ConfirmText = LOCTEXT("DeleteImportedBoneHierarchy_Yes", "Yes");
								Info.CancelText = LOCTEXT("DeleteImportedBoneHierarchy_No", "No");

								FSuppressableWarningDialog DeleteImportedBonesInHierarchy(Info);
								bDeleteImportedBones = DeleteImportedBonesInHierarchy.ShowModal() != FSuppressableWarningDialog::Cancel;
								bConfirmedByUser = true;
							}

							if (!bDeleteImportedBones)
							{
								break;
							}
						}
					}
				}

				Controller->RemoveElement(SelectedKey.GetElement(), true, true);
			}
			else if(SelectedKey.IsComponent())
			{
				Controller->RemoveComponent(SelectedKey.GetComponent(), true, true);
			}
		}
	}

	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	ControlRigEditor.Pin()->OnHierarchyChanged();
	RefreshTreeView();
	FSlateApplication::Get().DismissAllMenus();
}

bool SRigHierarchy::CanDeleteItem() const
{
	return IsMultiSelected(false);
}

/** Create Item */
void SRigHierarchy::HandleNewItem(ERigElementType InElementType, bool bIsAnimationChannel)
{
	if(!ControlRigEditor.IsValid())
	{
		return;
	}

	FRigElementKey NewItemKey;
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	URigHierarchy* DebugHierarchy = GetHierarchy();
	if (Hierarchy)
	{
		// unselect current selected item
		ClearDetailPanel();

		const bool bAllowMultipleItems =
			InElementType == ERigElementType::Socket ||
			InElementType == ERigElementType::Null ||
			(InElementType == ERigElementType::Control && !bIsAnimationChannel);

		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		FScopedTransaction Transaction(LOCTEXT("HierarchyTreeAdded", "Add new item to hierarchy"));

		// It's important to also modify the rig to restore the hierarchy after undo/redo
		if (UControlRig* ControlRig = ControlRigEditor.Pin()->GetControlRig())
		{
			ControlRig->Modify();
		}

		TArray<FRigHierarchyKey> SelectedKeys = GetSelectedKeys();
		TArray<FRigElementKey> SelectedElementKeys;
		SelectedElementKeys.Reserve(SelectedKeys.Num());
		for(const FRigHierarchyKey& SelectedKey : SelectedKeys)
		{
			if(SelectedKey.IsElement())
			{
				SelectedElementKeys.Add(SelectedKey.GetElement());
			}
		}
		if (SelectedElementKeys.Num() > 1 && !bAllowMultipleItems)
		{
			SelectedElementKeys = {SelectedElementKeys[0]};
		}
		else if(SelectedElementKeys.IsEmpty())
		{
			SelectedElementKeys = {FRigElementKey()};
		}

		TMap<FRigElementKey, FRigElementKey> SelectedToCreated;
		for(const FRigElementKey& SelectedKey : SelectedElementKeys)
		{
			FRigElementKey ParentKey;
			FTransform ParentTransform = FTransform::Identity;

			if(SelectedKey.IsValid())
			{
				ParentKey = SelectedKey;
				// Use the transform of the debugged hierarchy rather than the default hierarchy
				ParentTransform = DebugHierarchy->GetGlobalTransform(ParentKey);
			}

			// use bone's name as prefix if creating a control
			FString NewNameTemplate;
			if(ParentKey.IsValid() && ParentKey.Type == ERigElementType::Bone)
			{
				NewNameTemplate = ParentKey.Name.ToString();

				if(InElementType == ERigElementType::Control)
				{
					static const FString CtrlSuffix(TEXT("_ctrl"));
					NewNameTemplate += CtrlSuffix;
				}
				else if(InElementType == ERigElementType::Null)
				{
					static const FString NullSuffix(TEXT("_null"));
					NewNameTemplate += NullSuffix;
				}
				else if(InElementType == ERigElementType::Socket)
				{
					static const FString SocketSuffix(TEXT("_socket"));
					NewNameTemplate += SocketSuffix;
				}
				else
				{
					NewNameTemplate.Reset();
				}
			}

			if(NewNameTemplate.IsEmpty())
			{
				NewNameTemplate = FString::Printf(TEXT("New%s"), *StaticEnum<ERigElementType>()->GetNameStringByValue((int64)InElementType));

				if(bIsAnimationChannel)
				{
					static const FString NewAnimationChannel = TEXT("Channel");
					NewNameTemplate = NewAnimationChannel;
				}
			}
			
			const FName NewElementName = CreateUniqueName(*NewNameTemplate, InElementType);
			{
				TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
				switch (InElementType)
				{
					case ERigElementType::Bone:
					{
						NewItemKey = Controller->AddBone(NewElementName, ParentKey, ParentTransform, true, ERigBoneType::User, true, true);
						break;
					}
					case ERigElementType::Control:
					{
						FRigControlSettings Settings;

						if(bIsAnimationChannel)
						{
							Settings.AnimationType = ERigControlAnimationType::AnimationChannel;
							Settings.ControlType = ERigControlType::Float;
							Settings.MinimumValue = FRigControlValue::Make<float>(0.f);
							Settings.MaximumValue = FRigControlValue::Make<float>(1.f);
							Settings.DisplayName = Hierarchy->GetSafeNewDisplayName(ParentKey, FRigName(NewNameTemplate));

							NewItemKey = Controller->AddAnimationChannel(NewElementName, ParentKey, Settings, true, true);
						}
						else
						{
							Settings.ControlType = ERigControlType::EulerTransform;
							FEulerTransform Identity = FEulerTransform::Identity;
							FRigControlValue ValueToSet = FRigControlValue::Make<FEulerTransform>(Identity);
							Settings.MinimumValue = ValueToSet;
							Settings.MaximumValue = ValueToSet;

							FRigElementKey NewParentKey;
							FTransform OffsetTransform = ParentTransform;
							if (FRigElementKey* CreatedParentKey = SelectedToCreated.Find(Hierarchy->GetDefaultParent(ParentKey)))
							{
								NewParentKey = *CreatedParentKey;
								OffsetTransform = ParentTransform.GetRelativeTransform(Hierarchy->GetGlobalTransform(NewParentKey, true));
							}

							NewItemKey = Controller->AddControl(NewElementName, NewParentKey, Settings, Settings.GetIdentityValue(), OffsetTransform, FTransform::Identity, true, true);

							SelectedToCreated.Add(SelectedKey, NewItemKey);
						}
						break;
					}
					case ERigElementType::Null:
					{
						NewItemKey = Controller->AddNull(NewElementName, ParentKey, ParentTransform, true, true, true);
						break;
					}
					case ERigElementType::Connector:
					{
						FString FailureReason;
						if(!ControlRigBlueprint->CanTurnIntoControlRigModule(false, &FailureReason))
						{
							if(ControlRigBlueprint->GetHierarchy()->Num(ERigElementType::Connector) == 0)
							{
								if(!ControlRigBlueprint->IsControlRigModule())
								{
									static constexpr TCHAR Format[] = TEXT("Connector cannot be created: %s");
									UE_LOG(LogControlRig, Warning, Format, *FailureReason);
									FNotificationInfo Info(FText::FromString(FString::Printf(Format, *FailureReason)));
									Info.bUseSuccessFailIcons = true;
									Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
									Info.bFireAndForget = true;
									Info.bUseThrobber = true;
									Info.FadeOutDuration = 2.f;
									Info.ExpireDuration = 8.f;;
									TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
									if (NotificationPtr)
									{
										NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
									}
									return;
								}
							}
						}

						const TArray<FRigConnectorElement*> Connectors = Hierarchy->GetConnectors(false);
						const bool bIsPrimary = !Connectors.ContainsByPredicate([](const FRigConnectorElement* Connector) { return Connector->IsPrimary(); });
						
						FRigConnectorSettings Settings;
						Settings.Type = bIsPrimary ? EConnectorType::Primary : EConnectorType::Secondary;
						if(!bIsPrimary)
						{
							Settings.Rules.Reset();
							Settings.AddRule(FRigChildOfPrimaryConnectionRule());
							Settings.bOptional = true;
						}
						NewItemKey = Controller->AddConnector(NewElementName, Settings, true);
						(void)ResolveConnector(NewItemKey, ParentKey);
						break;
					}
					case ERigElementType::Socket:
					{
						NewItemKey = Controller->AddSocket(NewElementName, ParentKey, ParentTransform, true, FRigSocketElement::SocketDefaultColor, FString(), true, true);
						break;
					}
					default:
					{
						return;
					}
				}
			}
		}
	}

	if (ControlRigBlueprint.IsValid())
	{
		ControlRigBlueprint->GetRigVMAssetInterface()->BroadcastRefreshEditor();
	}
	
	if (Hierarchy && NewItemKey.IsValid())
	{
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		Controller->ClearSelection();
		Controller->SelectElement(NewItemKey);
	}

	FSlateApplication::Get().DismissAllMenus();
	RefreshTreeView();
}

bool SRigHierarchy::CanFindReferencesOfItem() const
{
	return !GetSelectedKeys().IsEmpty();
}

void SRigHierarchy::HandleFindReferencesOfItem()
{
	if(!ControlRigEditor.IsValid() || GetSelectedKeys().IsEmpty())
	{
		return;
	}

	ControlRigEditor.Pin()->FindReferencesOfItem(GetSelectedKeys()[0]);
}

/** Check whether we can deleting the selected item(s) */
bool SRigHierarchy::CanDuplicateItem() const
{
	if (!IsMultiSelected(false))
	{
		return false;
	}

	// don't allow duplication on components
	if(GetSelectedKeys().ContainsByPredicate([](const FRigHierarchyKey& Key)
	{
		return Key.IsComponent();
	}))
	{
		return false;
	}

	if (ControlRigBlueprint->IsControlRigModule())
	{
		bool bAnyNonConnector = GetSelectedElementKeys().ContainsByPredicate([](const FRigElementKey& Key)
		{
			return Key.Type != ERigElementType::Connector;
		});
		
		return !bAnyNonConnector;
	}

	return true;
}

/** Duplicate Item */
void SRigHierarchy::HandleDuplicateItem()
{
	if(!ControlRigEditor.IsValid())
	{
		return;
	}

	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		ClearDetailPanel();
		{
			TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
			TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

			FScopedTransaction Transaction(LOCTEXT("HierarchyTreeDuplicateSelected", "Duplicate selected items from hierarchy"));

			URigHierarchyController* Controller = Hierarchy->GetController(true);
			check(Controller);

			const TArray<FRigElementKey> KeysToDuplicate = GetSelectedElementKeys();
			Controller->DuplicateElements(KeysToDuplicate, true, true, true);
		}

		ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	}

	FSlateApplication::Get().DismissAllMenus();
	ControlRigEditor.Pin()->OnHierarchyChanged();
	{
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		ControlRigBlueprint->GetRigVMAssetInterface()->BroadcastRefreshEditor();
	}
	RefreshTreeView();
}

/** Mirror Item */
void SRigHierarchy::HandleMirrorItem()
{
	if(!ControlRigEditor.IsValid())
	{
		return;
	}
	
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);
		
		UControlRigEditorSettings* Settings = GetMutableDefault<UControlRigEditorSettings>();

		TSharedPtr<FStructOnScope> StructToDisplay = MakeShared<FStructOnScope>(FRigVMMirrorSettings::StaticStruct(), (uint8*)&Settings->MirrorSettings);
#if WITH_RIGVMLEGACYEDITOR
		TSharedRef<SKismetInspector> KismetInspector = SNew(SKismetInspector);
#else
		TSharedRef<SRigVMDetailsInspector> KismetInspector = SNew(SRigVMDetailsInspector);
#endif
		KismetInspector->ShowSingleStruct(StructToDisplay);

		TSharedRef<SCustomDialog> MirrorDialog = SNew(SCustomDialog)
			.Title(FText(LOCTEXT("ControlRigHierarchyMirror", "Mirror Selected Rig Elements")))
			.Content()
			[
				KismetInspector
			]
			.Buttons({
				SCustomDialog::FButton(LOCTEXT("OK", "OK")),
				SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
		});

		if (MirrorDialog->ShowModal() == 0)
		{
			ClearDetailPanel();
			{
				TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
				TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

				FScopedTransaction Transaction(LOCTEXT("HierarchyTreeMirrorSelected", "Mirror selected items from hierarchy"));

				const TArray<FRigElementKey> KeysToMirror = GetSelectedElementKeys();
				Controller->MirrorElements(KeysToMirror, Settings->MirrorSettings, true, true, true);
			}
			ControlRigBlueprint->PropagateHierarchyFromBPToInstances();

			Settings->SaveConfig();
		}
	}

	FSlateApplication::Get().DismissAllMenus();
	ControlRigEditor.Pin()->OnHierarchyChanged();
	RefreshTreeView();
}

/** Check whether we can deleting the selected item(s) */
bool SRigHierarchy::CanRenameItem() const
{
	if(IsSingleSelected(false))
	{
		const FRigHierarchyKey Key = GetSelectedKeys()[0];
		if(Key.IsElement())
		{
			if(Key.GetElement().Type == ERigElementType::Physics ||
				Key.GetElement().Type == ERigElementType::Reference)
			{
				return false;
			}
			if(Key.GetElement().Type == ERigElementType::Control)
			{
				if(URigHierarchy* DebuggedHierarchy = GetHierarchy())
				{
					if(FRigControlElement* ControlElement = DebuggedHierarchy->Find<FRigControlElement>(Key.GetElement()))
					{
						if(ControlElement->Settings.bIsTransientControl)
						{
							return false;
						}
					}
				}
			}
			return true;
		}
		if(Key.IsComponent())
		{
			if(URigHierarchy* DebuggedHierarchy = GetHierarchy())
			{
				if(const FRigBaseComponent* Component = DebuggedHierarchy->FindComponent(Key.GetComponent()))
				{
					return Component->CanBeRenamed();
				}
			}
		}
	}
	return false;
}

/** Delete Item */
void SRigHierarchy::HandleRenameItem()
{
	if(!ControlRigEditor.IsValid())
	{
		return;
	}

	if (!CanRenameItem())
	{
		return;
	}

	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		FScopedTransaction Transaction(LOCTEXT("HierarchyTreeRenameSelected", "Rename selected item from hierarchy"));

		TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = TreeView->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			if(SelectedItems[0]->Key.IsElement())
			{
				if (SelectedItems[0]->Key.GetElement().Type == ERigElementType::Bone)
				{
					if(FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(SelectedItems[0]->Key.GetElement()))
					{
						if (BoneElement->BoneType == ERigBoneType::Imported)
						{
							FText ConfirmRename = LOCTEXT("RenameDeleteBoneHierarchy", "Renaming imported(white) bones can cause issues with animation - are you sure ?");

							FSuppressableWarningDialog::FSetupInfo Info(ConfirmRename, LOCTEXT("RenameImportedBone", "Rename Imported Bone"), "RenameImportedBoneHierarchy_Warning");
							Info.ConfirmText = LOCTEXT("RenameImportedBoneHierarchy_Yes", "Yes");
							Info.CancelText = LOCTEXT("RenameImportedBoneHierarchy_No", "No");

							FSuppressableWarningDialog RenameImportedBonesInHierarchy(Info);
							if (RenameImportedBonesInHierarchy.ShowModal() == FSuppressableWarningDialog::Cancel)
							{
								return;
							}
						}
					}
				}
			}
			SelectedItems[0]->RequestRename();
		}
	}
}

bool SRigHierarchy::CanPasteItems() const
{
	return true;
}	

bool SRigHierarchy::CanCopyOrPasteItems() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

void SRigHierarchy::HandleCopyItems()
{
	if (URigHierarchy* Hierarchy = GetHierarchy())
	{
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		const TArray<FRigElementKey> Selection = GetHierarchy()->GetSelectedKeys();
		const FString Content = Controller->ExportToText(Selection);
		FPlatformApplicationMisc::ClipboardCopy(*Content);
	}
}

void SRigHierarchy::HandlePasteItems()
{
	if (URigHierarchy* Hierarchy = GetDefaultHierarchy())
	{
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

		FString Content;
		FPlatformApplicationMisc::ClipboardPaste(Content);

		FScopedTransaction Transaction(LOCTEXT("HierarchyTreePastedRigElements", "Pasted rig elements."));

		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		ERigElementType AllowedTypes = ControlRigBlueprint->IsControlRigModule() ? ERigElementType::Connector : ERigElementType::All;
		Controller->ImportFromText(Content, AllowedTypes, false, true, true, true);
	}

	//ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	ControlRigEditor.Pin()->OnHierarchyChanged();
	{
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		ControlRigBlueprint->GetRigVMAssetInterface()->BroadcastRefreshEditor();
	}
	RefreshTreeView();
}

class SRigHierarchyPasteTransformsErrorPipe : public FOutputDevice
{
public:

	int32 NumErrors;

	SRigHierarchyPasteTransformsErrorPipe()
		: FOutputDevice()
		, NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		UE_LOGF(LogControlRig, Error, "Error importing transforms to Hierarchy: %ls", V);
		NumErrors++;
	}
};

void SRigHierarchy::HandlePasteLocalTransforms()
{
	HandlePasteTransforms(ERigTransformType::CurrentLocal, true);
}

void SRigHierarchy::HandlePasteGlobalTransforms()
{
	HandlePasteTransforms(ERigTransformType::CurrentGlobal, false);
}

void SRigHierarchy::HandlePasteTransforms(ERigTransformType::Type InTransformType, bool bAffectChildren)
{
	if (URigHierarchy* Hierarchy = GetDefaultHierarchy())
	{
		FString Content;
		FPlatformApplicationMisc::ClipboardPaste(Content);

		FScopedTransaction Transaction(LOCTEXT("HierarchyTreePastedTransform", "Pasted transforms."));

		SRigHierarchyPasteTransformsErrorPipe ErrorPipe;
		FRigHierarchyCopyPasteContent Data;
		FRigHierarchyCopyPasteContent::StaticStruct()->ImportText(*Content, &Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe, FRigHierarchyCopyPasteContent::StaticStruct()->GetName(), true);
		if(ErrorPipe.NumErrors > 0)
		{
			return;
		}
		
		URigHierarchy* DebuggedHierarchy = GetHierarchy();

		const TArray<FRigElementKey> CurrentSelection = Hierarchy->GetSelectedKeys();
		const int32 Count = FMath::Min<int32>(CurrentSelection.Num(), Data.Elements.Num());
		for(int32 Index = 0; Index < Count; Index++)
		{
			const FRigHierarchyCopyPasteContentPerElement& PerElementData = Data.Elements[Index];
			const FTransform Transform =  PerElementData.Poses[(int32)InTransformType];

			if(FRigTransformElement* TransformElement = Hierarchy->Find<FRigTransformElement>(CurrentSelection[Index]))
			{
				Hierarchy->SetTransform(TransformElement, Transform, InTransformType, bAffectChildren, true, false, true);
			}
			if(FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(CurrentSelection[Index]))
			{
				Hierarchy->SetTransform(BoneElement, Transform, ERigTransformType::MakeInitial(InTransformType), bAffectChildren, true, false, true);
			}
			
            if(DebuggedHierarchy && DebuggedHierarchy != Hierarchy)
            {
            	if(FRigTransformElement* TransformElement = DebuggedHierarchy->Find<FRigTransformElement>(CurrentSelection[Index]))
            	{
            		DebuggedHierarchy->SetTransform(TransformElement, Transform, InTransformType, bAffectChildren, true);
            	}
            	if(FRigBoneElement* BoneElement = DebuggedHierarchy->Find<FRigBoneElement>(CurrentSelection[Index]))
            	{
            		DebuggedHierarchy->SetTransform(BoneElement, Transform, ERigTransformType::MakeInitial(InTransformType), bAffectChildren, true);
            	}
            }
		}
	}
}

URigHierarchy* SRigHierarchy::GetHierarchy() const
{
	if (ControlRigBlueprint.IsValid())
	{
		if (UControlRig* DebuggedRig = ControlRigBeingDebuggedPtr.Get())
		{
			return DebuggedRig->GetHierarchy();
		}
	}
	if (ControlRigEditor.IsValid())
	{
		if (UControlRig* CurrentRig = ControlRigEditor.Pin()->GetControlRig())
		{
			return CurrentRig->GetHierarchy();
		}
	}
	return GetDefaultHierarchy();
}

URigHierarchy* SRigHierarchy::GetDefaultHierarchy() const
{
	if (ControlRigBlueprint.IsValid())
	{
		return ControlRigBlueprint->GetHierarchy();
	}
	return nullptr;
}


FName SRigHierarchy::CreateUniqueName(const FName& InBaseName, ERigElementType InElementType) const
{
	return GetHierarchy()->GetSafeNewName(InBaseName, InElementType);
}

void SRigHierarchy::PostRedo(bool bSuccess) 
{
	if (bSuccess)
	{
		RefreshTreeView();
	}
}

void SRigHierarchy::PostUndo(bool bSuccess) 
{
	if (bSuccess)
	{
		RefreshTreeView();
	}
}

FReply SRigHierarchy::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TArray<FRigHierarchyKey> DraggedElements = GetSelectedKeys();
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && DraggedElements.Num() > 0)
	{
		if (TSharedPtr<IControlRigBaseEditor> EditorPtr = ControlRigEditor.Pin())
		{
			UpdateConnectorMatchesOnDrag(DraggedElements);
			
			TSharedRef<FRigHierarchyElementDragDropOp> DragDropOp = FRigHierarchyElementDragDropOp::New(MoveTemp(DraggedElements));
			DragDropOp->OnPerformDropToGraphAtLocation.BindSP(EditorPtr.ToSharedRef(), &FControlRigBaseEditor::OnGraphNodeDropToPerform);
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SRigHierarchy::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigHierarchyTreeElement> TargetItem)
{
	const TOptional<EItemDropZone> InvalidDropZone;
	TOptional<EItemDropZone> ReturnDropZone;

	const TSharedPtr<FRigHierarchyElementDragDropOp> RigDragDropOp = DragDropEvent.GetOperationAs<FRigHierarchyElementDragDropOp>();
	if (RigDragDropOp.IsValid())
	{
		FRigHierarchyKey TargetKey;
		if (const URigHierarchy* Hierarchy = GetHierarchy())
		{
			switch(DropZone)
			{
				case EItemDropZone::AboveItem:
				case EItemDropZone::BelowItem:
				{
					if(TargetItem->Key.IsElement())
					{
						TargetKey = Hierarchy->GetFirstParent(TargetItem->Key.GetElement());
					}
					else if(TargetItem->Key.IsComponent())
					{
						return InvalidDropZone;
					}
					break;
				}
				case EItemDropZone::OntoItem:
				{
					TargetKey = TargetItem->Key.GetElement();
					break;
				}
			}

			if(TargetKey.IsComponent())
			{
				return InvalidDropZone;
			}

			for (const FRigHierarchyKey& DraggedKey : RigDragDropOp->GetElements())
			{
				if(Hierarchy->IsProcedural(DraggedKey) && !RigDragDropOp->IsDraggingSingleConnector())
				{
					return InvalidDropZone;
				}

				// re-parenting directly onto an item
				if (DraggedKey == TargetKey)
				{
					return InvalidDropZone;
				}

				if(RigDragDropOp->IsDraggingSingleConnector() || RigDragDropOp->IsDraggingSingleSocket())
				{
					if(const FModularRigResolveResult* ResolveResult = DragRigResolveResults.Find(DraggedKey.GetElement()))
					{
						if(!ResolveResult->ContainsMatch(TargetKey.GetElement()))
						{
							return InvalidDropZone;
						}
					}
					if(DropZone != EItemDropZone::OntoItem)
					{
						return InvalidDropZone;
					}
				}
				else if(DropZone == EItemDropZone::OntoItem)
				{
					if(DraggedKey.IsElement())
					{
						if(Hierarchy->IsParentedTo(TargetKey.GetElement(), DraggedKey.GetElement()))
						{
							return InvalidDropZone;
						}
					}
					if(DraggedKey.IsComponent())
					{
						const FRigBaseComponent* Component = Hierarchy->FindComponent(DraggedKey.GetComponent());
						if(Component == nullptr)
						{
							return InvalidDropZone;
						}
						if(!Hierarchy->CanAddComponent(TargetKey.GetElement(), Component))
						{
							return InvalidDropZone;
						}
					}
				}
			}
		}

		// don't allow dragging onto procedural items (except for connectors + sockets)
		if(TargetKey.IsValid() && !GetDefaultHierarchy()->Contains(TargetKey.GetElement()) &&
			!(RigDragDropOp->IsDraggingSingleConnector() || RigDragDropOp->IsDraggingSingleSocket()))
		{
			return InvalidDropZone;
		}

		switch (TargetKey.GetElement().Type)
		{
			case ERigElementType::Bone:
			{
				// bones can parent anything
				ReturnDropZone = DropZone;
				break;
			}
			case ERigElementType::Control:
			case ERigElementType::Null:
			case ERigElementType::Physics:
			case ERigElementType::Reference:
			{
				for (const FRigHierarchyKey& DraggedKey : RigDragDropOp->GetElements())
				{
					if(DraggedKey.IsElement())
					{
						switch (DraggedKey.GetElement().Type)
						{
							case ERigElementType::Control:
							case ERigElementType::Null:
							case ERigElementType::Physics:
							case ERigElementType::Reference:
							case ERigElementType::Connector:
							case ERigElementType::Socket:
							{
								break;
							}
							default:
							{
								return InvalidDropZone;
							}
						}
					}
				}
				ReturnDropZone = DropZone;
				break;
			}
			case ERigElementType::Connector:
			{
				// anything can be parented under a connector
				ReturnDropZone = DropZone;
				break;
			}
			case ERigElementType::Socket:
			{
				// Only connectors can be parented under a socket
				if (RigDragDropOp->IsDraggingSingleConnector())
				{
					ReturnDropZone = DropZone;
				}
				else
				{
					return InvalidDropZone;
				}
				break;
			}
			default:
			{
				ReturnDropZone = DropZone;
				break;
			}
		}
	}

	const TSharedPtr<FRigHierarchyTagDragDropOp> TagDragDropOp = DragDropEvent.GetOperationAs<FRigHierarchyTagDragDropOp>();
	if(TagDragDropOp.IsValid())
	{
		if(DropZone != EItemDropZone::OntoItem)
		{
			return InvalidDropZone;
		}

		if (const URigHierarchy* Hierarchy = GetHierarchy())
		{
			FRigElementKey DraggedKey;
			FRigElementKey::StaticStruct()->ImportText(*TagDragDropOp->GetIdentifier(), &DraggedKey, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);

			if(Hierarchy->Contains(DraggedKey) && TargetItem.IsValid())
			{
				if(DraggedKey.Type == ERigElementType::Connector)
				{
					if(const FModularRigResolveResult* ResolveResult = DragRigResolveResults.Find(DraggedKey))
					{
						if(!ResolveResult->ContainsMatch(TargetItem->Key.GetElement()))
						{
							return InvalidDropZone;
						}
					}
				}
				ReturnDropZone = DropZone;
			}
			else if(!TargetItem.IsValid())
			{
				ReturnDropZone = DropZone;
			}
		}
	}
	
	const TSharedPtr<FModularRigHierarchyElementDragDropOp> ModuleDropOp = DragDropEvent.GetOperationAs<FModularRigHierarchyElementDragDropOp>();
	if (ModuleDropOp.IsValid() && TargetItem.IsValid())
	{
		if(DropZone != EItemDropZone::OntoItem)
		{
			return InvalidDropZone;
		}

		const UModularRig* ControlRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
		if (!ControlRig)
		{
			return InvalidDropZone;
		}

		const FRigElementKey TargetKey = TargetItem->Key.GetElement();
		const TArray<FRigElementKey> DraggedKeys = FControlRigSchematicModel::GetElementKeysFromDragDropEvent(*ModuleDropOp.Get(), ControlRig);
		for(const FRigElementKey& DraggedKey : DraggedKeys)
		{
			if(DraggedKey.Type != ERigElementType::Connector)
			{
				continue;
			}
			
			if(!DragRigResolveResults.Contains(DraggedKey))
			{
				UpdateConnectorMatchesOnDrag({DraggedKey});
			}
			
			const FModularRigResolveResult& ResolveResult = DragRigResolveResults.FindChecked(DraggedKey);
			if(ResolveResult.ContainsMatch(TargetKey))
			{
				return DropZone;
			}
		}

		return InvalidDropZone;
	}

	TSharedPtr<FAssetDragDropOp> AssetDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetDragDropOp.IsValid())
	{
		for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
		{
			static const UEnum* ControlTypeEnum = StaticEnum<EControlRigType>();
			const FString ControlRigTypeStr = AssetData.GetTagValueRef<FString>(TEXT("ControlRigType"));
			if (ControlRigTypeStr.IsEmpty())
			{
				return InvalidDropZone;
			}

			const EControlRigType ControlRigType = (EControlRigType)(ControlTypeEnum->GetValueByName(*ControlRigTypeStr));
			if (ControlRigType != EControlRigType::RigModule)
			{
				return InvalidDropZone;
			}

			if(FControlRigAssetInterfacePtr AssetBlueprint = AssetData.GetAsset())
			{
				if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
				{
					FRigModuleConnector* PrimaryConnector = nullptr;
					for (FRigModuleConnector& Connector : AssetBlueprint->GetRigModuleSettings().ExposedConnectors)
					{
						if (Connector.IsPrimary())
						{
							PrimaryConnector = &Connector;
							break;
						}
					}
					if (!PrimaryConnector)
					{
						return InvalidDropZone;
					}

					FRigElementKey TargetKey;
					if (TargetItem.IsValid())
					{
						TargetKey = TargetItem->Key.GetElement();
					}

					/*
					FText ErrorMessage;
					if (Controller->CanConnectConnectorToElement(*PrimaryConnector, TargetKey, ErrorMessage))
					{
						ReturnDropZone = DropZone;
					}
					else
					{
						return InvalidDropZone;
					}
					*/
					ReturnDropZone = DropZone;
				}
			}
		}
	}

	return ReturnDropZone;
}

FReply SRigHierarchy::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigHierarchyTreeElement> TargetItem)
{
	bool bSummonDragDropMenu = DragDropEvent.GetModifierKeys().IsAltDown() && DragDropEvent.GetModifierKeys().IsShiftDown(); 
	bool bMatchTransforms = DragDropEvent.GetModifierKeys().IsAltDown();
	bool bReparentItems = !bMatchTransforms;
	UpdateConnectorMatchesOnDrag({});

	TSharedPtr<FRigHierarchyElementDragDropOp> RigDragDropOp = DragDropEvent.GetOperationAs<FRigHierarchyElementDragDropOp>();
	if (RigDragDropOp.IsValid())
	{
		if (bSummonDragDropMenu)
		{
			const FVector2D& SummonLocation = DragDropEvent.GetScreenSpacePosition();

			// Get the context menu content. If NULL, don't open a menu.
			UToolMenu* DragDropMenu = GetDragDropMenu(RigDragDropOp->GetElements(), TargetItem->Key.GetElement());
			const TSharedPtr<SWidget> MenuContent = UToolMenus::Get()->GenerateWidget(DragDropMenu);

			if (MenuContent.IsValid())
			{
				const FWidgetPath WidgetPath = DragDropEvent.GetEventPath() != nullptr ? *DragDropEvent.GetEventPath() : FWidgetPath();
				FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			}
			
			return FReply::Handled();
		}
		else
		{
			const URigHierarchy* Hierarchy = GetDefaultHierarchy();

			FRigElementKey TargetKey;
			int32 LocalIndex = INDEX_NONE;


			if (TargetItem.IsValid())
			{
				switch(DropZone)
				{
					case EItemDropZone::AboveItem:
					{
						TargetKey = Hierarchy->GetFirstParent(TargetItem->Key.GetElement());
						LocalIndex = Hierarchy->GetLocalIndex(TargetItem->Key.GetElement());
						break;
					}
					case EItemDropZone::BelowItem:
					{
						TargetKey = Hierarchy->GetFirstParent(TargetItem->Key.GetElement());
						LocalIndex = Hierarchy->GetLocalIndex(TargetItem->Key.GetElement()) + 1;
						break;
					}
					case EItemDropZone::OntoItem:
					{
						TargetKey = TargetItem->Key.GetElement();
						break;
					}
				}
			}

			if(RigDragDropOp->IsDraggingSingleConnector())
			{
				return ResolveConnector(RigDragDropOp->GetElements()[0].GetElement(), TargetKey);
			}

			return ReparentOrMatchTransform(RigDragDropOp->GetElements(), TargetKey, bReparentItems, LocalIndex);			
		}

	}

	const TSharedPtr<FRigHierarchyTagDragDropOp> TagDragDropOp = DragDropEvent.GetOperationAs<FRigHierarchyTagDragDropOp>();
	if(TagDragDropOp.IsValid())
	{
		FRigElementKey DraggedKey;
		FRigElementKey::StaticStruct()->ImportText(*TagDragDropOp->GetIdentifier(), &DraggedKey, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);
		if(TargetItem.IsValid())
		{
			TArray<FRigElementKey> TargetsForConnector = {TargetItem->Key.GetElement()};
			
			// do we want to add this target to an array connector?
			if(DragDropEvent.GetModifierKeys().IsShiftDown())
			{
				if(const UControlRig* ControlRig = Cast<UControlRig>(ControlRigBeingDebuggedPtr.Get()))
				{
					if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
					{
						if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(DraggedKey))
						{
							if(Connector->IsArrayConnector())
							{
								if(const FRigElementKeyRedirector::FCachedKeyArray* Cache = ControlRigBeingDebuggedPtr->GetElementKeyRedirector().Find(Connector->GetKey()))
								{
									TargetsForConnector.Reset();
									TargetsForConnector.Append(FRigElementKeyRedirector::Convert(*Cache));
									TargetsForConnector.AddUnique(TargetItem->Key.GetElement());
								}
							}
						}
					}
				}
			}
			return ResolveConnectorToArray(DraggedKey, TargetsForConnector);
		}
		return ResolveConnector(DraggedKey, FRigElementKey());
	}

	const TSharedPtr<FModularRigHierarchyElementDragDropOp> ModuleDropOp = DragDropEvent.GetOperationAs<FModularRigHierarchyElementDragDropOp>();
	if (ModuleDropOp.IsValid() && TargetItem.IsValid())
	{
		UModularRig* ControlRig = Cast<UModularRig>(ControlRigBlueprint->GetDebuggedControlRig());
		if (!ControlRig)
		{
			return FReply::Handled();
		}

		const FRigElementKey TargetKey = TargetItem->Key.GetElement();
		const TArray<FRigElementKey> DraggedKeys = FControlRigSchematicModel::GetElementKeysFromDragDropEvent(*ModuleDropOp.Get(), ControlRig);

		bool bSuccess = false;
		for(const FRigElementKey& DraggedKey : DraggedKeys)
		{
			const FReply Reply = ResolveConnector(DraggedKey, TargetKey);
			if(Reply.IsEventHandled())
			{
				bSuccess = true;
			}
		}
		return bSuccess ? FReply::Handled() : FReply::Unhandled();
	}
	
	TSharedPtr<FAssetDragDropOp> AssetDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (AssetDragDropOp.IsValid())
	{
		for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
		{
			FControlRigAssetStrongReference ControlRigAssetReference;
			FRigModuleSettings ModuleSettings;
			if(FControlRigAssetInterfacePtr AssetBlueprint = AssetData.GetAsset())
			{
				ControlRigAssetReference = AssetBlueprint->GetControlRigAssetReference();
				ModuleSettings = AssetBlueprint->GetRigModuleSettings();
			}
			else if (TScriptInterface<IControlRigRuntimeAssetInterface> GeneratedClass = AssetData.GetAsset())
			{
				ControlRigAssetReference = GeneratedClass->GetControlRigAssetReference();
				ModuleSettings = GeneratedClass->GetRigModuleSettings();
			}

			if(ControlRigAssetReference.IsValid())
			{
				if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
				{
					const FRigName DesiredModuleName = Controller->GetSafeNewName(FRigName(ModuleSettings.Identifier.Name));
					const FName ModuleName = Controller->AddModuleFromAssetReference(DesiredModuleName.GetFName(), ControlRigAssetReference, NAME_None);
					if(TargetItem.IsValid() && !ModuleName.IsNone())
					{
						FRigElementKey PrimaryConnectorKey;
						TArray<FRigConnectorElement*> Connectors = GetHierarchy()->GetElementsOfType<FRigConnectorElement>();

						const FString ModuleNameString = ModuleName.ToString();
						for (FRigConnectorElement* Connector : Connectors)
						{
							if (Connector->IsPrimary())
							{
								const FRigHierarchyModulePath ConnectorModulePath(Connector->GetName());
								if (ConnectorModulePath.HasModuleName(ModuleNameString))
								{
									PrimaryConnectorKey = Connector->GetKey();
									break;
								}
							}
						}
						return ResolveConnector(PrimaryConnectorKey, TargetItem->Key.GetElement());
					}
				}
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

void SRigHierarchy::OnElementKeyTagDragDetected(const FRigElementKey& InDraggedTag)
{
	UpdateConnectorMatchesOnDrag({InDraggedTag});
}

void SRigHierarchy::UpdateConnectorMatchesOnDrag(const TArray<FRigHierarchyKey>& InDraggedKeys)
{
	DragRigResolveResults.Reset();

	// fade in all items
	for(const TPair<FRigHierarchyKey, TSharedPtr<FRigHierarchyTreeElement>>& Pair : TreeView->ElementMap)
	{
		Pair.Value->bFadedOutDuringDragDrop = false;
	}
	
	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(const UModularRig* ControlRig = Cast<UModularRig>(ControlRigBeingDebuggedPtr.Get()))
		{
			if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
			{
				if(const UModularRigRuleManager* RuleManager = Hierarchy->GetRuleManager())
				{
					for(const FRigHierarchyKey& DraggedElement : InDraggedKeys)
					{
						if(DraggedElement.IsElement())
						{
							if(DraggedElement.GetElement().Type == ERigElementType::Connector)
							{
								if(const FRigConnectorElement* Connector = Hierarchy->Find<FRigConnectorElement>(DraggedElement.GetElement()))
								{
									const FName ModuleName = Hierarchy->GetModuleFName(Connector->GetKey());
									if(!ModuleName.IsNone())
									{
										if(const FRigModuleInstance* Module = ControlRig->FindModule(ModuleName))
										{
											const FModularRigResolveResult ResolveResult = RuleManager->FindMatches(Connector, Module, ControlRig->ElementKeyRedirector); 
											DragRigResolveResults.Add(DraggedElement.GetElement(), ResolveResult);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// fade out anything that's on an excluded list
	for(const TPair<FRigElementKey, FModularRigResolveResult>& Pair: DragRigResolveResults)
	{
		for(const FRigElementResolveResult& ExcludedElement : Pair.Value.GetExcluded())
		{
			if(const TSharedPtr<FRigHierarchyTreeElement>* TreeElementPtr = TreeView->ElementMap.Find(ExcludedElement.GetKey()))
			{
				TreeElementPtr->Get()->bFadedOutDuringDragDrop = true;
			}
		}
	}
}

FName SRigHierarchy::HandleRenameElement(const FRigHierarchyKey& OldKey, const FString& NewName)
{
	ClearDetailPanel();

	// make sure there is no duplicate
	if (ControlRigBlueprint.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("HierarchyRename", "Rename Hierarchy Element"));

		URigHierarchy* Hierarchy = GetDefaultHierarchy();
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		FRigName SanitizedName(NewName);
		Hierarchy->SanitizeName(SanitizedName);

		FName ResultingName = NAME_None;
		if(OldKey.IsElement())
		{
			bool bUseDisplayName = false;
			if(OldKey.IsElement())
			{
				if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(OldKey.GetElement()))
				{
					if(ControlElement->IsAnimationChannel())
					{
						bUseDisplayName = true;
					}
				}

				if(bUseDisplayName)
				{
					ResultingName = Controller->SetDisplayName(OldKey.GetElement(), SanitizedName, true, true, true);
				}
				else
				{
					ResultingName = Controller->RenameElement(OldKey.GetElement(), SanitizedName, true, true, false).Name;
				}
			}
		}
		else if(OldKey.IsComponent())
		{
			ResultingName = Controller->RenameComponent(OldKey.GetComponent(), SanitizedName, true, true, false).Name;
		}
		ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
		return ResultingName;
	}

	return NAME_None;
}

bool SRigHierarchy::HandleVerifyNameChanged(const FRigHierarchyKey& OldKey, const FString& NewName, FText& OutErrorMessage)
{
	bool bIsAnimationChannel = false;
	if (ControlRigBlueprint.IsValid())
	{
		URigHierarchy* Hierarchy = GetHierarchy();
		if(OldKey.IsElement())
		{
			if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(OldKey.GetElement()))
			{
				if(ControlElement->IsAnimationChannel())
				{
					bIsAnimationChannel = true;

					if(ControlElement->GetDisplayName().ToString() == NewName)
					{
						return true;
					}
				}
			}
		}
	}

	if(!bIsAnimationChannel)
	{
		if (OldKey.GetName() == NewName)
		{
			return true;
		}
	}

	if (NewName.IsEmpty())
	{
		OutErrorMessage = FText::FromString(TEXT("Name is empty."));
		return false;
	}

	// make sure there is no duplicate
	if (ControlRigBlueprint.IsValid())
	{
		URigHierarchy* Hierarchy = GetHierarchy();

		if(bIsAnimationChannel)
		{
			if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(OldKey.GetElement()))
			{
				if(const FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(ControlElement))
				{
					FString OutErrorString;
					if (!Hierarchy->IsDisplayNameAvailable(ParentElement->GetKey(), FRigName(NewName), &OutErrorString))
					{
						OutErrorMessage = FText::FromString(OutErrorString);
						return false;
					}
				}
			}
		}
		else if(OldKey.IsElement())
		{
			FString OutErrorString;
			if (!Hierarchy->IsNameAvailable(FRigName(NewName), OldKey.GetElement().Type, &OutErrorString))
			{
				OutErrorMessage = FText::FromString(OutErrorString);
				return false;
			}
		}
		else if(OldKey.IsComponent())
		{
			FString OutErrorString;
			if (!Hierarchy->IsComponentNameAvailable(OldKey.GetElement(), FRigName(NewName), &OutErrorString))
			{
				OutErrorMessage = FText::FromString(OutErrorString);
				return false;
			}
		}
	}
	return true;
}

FReply SRigHierarchy::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// only allow drops onto empty space of the widget (when there's no target item under the mouse)
	const TSharedPtr<FRigHierarchyTreeElement>* ItemAtMouse = TreeView->FindItemAtPosition(DragDropEvent.GetScreenSpacePosition());
	if (ItemAtMouse && ItemAtMouse->IsValid())
	{
		return FReply::Unhandled();
	}
	
	return OnAcceptDrop(DragDropEvent, EItemDropZone::OntoItem, nullptr);
}

void SRigHierarchy::HandleResetTransform(bool bSelectionOnly)
{
	if ((IsMultiSelected(true) || !bSelectionOnly) && ControlRigEditor.IsValid())
	{
		if (FControlRigAssetInterfacePtr Blueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface())
		{
			if (URigHierarchy* DebuggedHierarchy = GetHierarchy())
			{
				FScopedTransaction Transaction(LOCTEXT("HierarchyResetTransforms", "Reset Transforms"));

				TArray<FRigElementKey> KeysToReset = GetSelectedElementKeys();
				if (!bSelectionOnly)
				{
					KeysToReset = DebuggedHierarchy->GetAllKeys(true, ERigElementType::Control);

					// Bone Transforms can also be modified, support reset for them as well
					KeysToReset.Append(DebuggedHierarchy->GetAllKeys(true, ERigElementType::Bone));
				}

				for (FRigElementKey Key : KeysToReset)
				{
					const FTransform InitialTransform = GetHierarchy()->GetInitialLocalTransform(Key);
					GetHierarchy()->SetLocalTransform(Key, InitialTransform, false, true, true, true);
					DebuggedHierarchy->SetLocalTransform(Key, InitialTransform, false, true, true);

					if (Key.Type == ERigElementType::Bone)
					{
						Blueprint->RemoveTransientControl(Key);
						ControlRigEditor.Pin()->RemoveBoneModification(Key.Name); 
					}
				}
			}
		}
	}
}

void SRigHierarchy::HandleZeroControls()
{
	const TSharedPtr<IControlRigBaseEditor> PinnedControlRigEditor = ControlRigEditor.IsValid() ? ControlRigEditor.Pin() : nullptr;
	const TScriptInterface<IControlRigEditorAssetInterface> Blueprint = PinnedControlRigEditor.IsValid() ? PinnedControlRigEditor->GetControlRigAssetInterface() : nullptr;
	URigHierarchy* DefaultHierarchy = GetDefaultHierarchy();
	URigHierarchy* DebuggedHierarchy = GetHierarchy();

	if (CanHandleZeroControls() &&
		PinnedControlRigEditor.IsValid() &&
		Blueprint &&
		DefaultHierarchy &&
		DebuggedHierarchy)
	{
		const FScopedTransaction Transaction(LOCTEXT("HandleZeroControlsTransaction", "Zero Controls"));

		TArray<FRigElementKey> SelectedKeys = GetSelectedElementKeys();

		TMap<FRigElementKey, FTransform> GlobalTransforms;
		TMap<FRigElementKey, FTransform> ParentGlobalTransforms;
		for (const FRigElementKey& SelectedKey : SelectedKeys)
		{
			if (SelectedKey.Type != ERigElementType::Control)
			{
				continue;
			}

			GlobalTransforms.FindOrAdd(SelectedKey) = DebuggedHierarchy->GetGlobalTransform(SelectedKey);
			ParentGlobalTransforms.FindOrAdd(SelectedKey) = DebuggedHierarchy->GetParentTransform(SelectedKey);
		}

		for (const FRigElementKey& SelectedKey : SelectedKeys)
		{
			if (SelectedKey.Type != ERigElementType::Control)
			{
				continue;
			}

			const FTransform GlobalTransform = GlobalTransforms[SelectedKey];
			const FTransform LocalTransform = GlobalTransform.GetRelativeTransform(ParentGlobalTransforms[SelectedKey]);

			constexpr bool bAffectChildren = true;
			constexpr bool bSetupUndo = true;
			constexpr bool bForce = false;
			constexpr bool bPrintPythonCommand = true;

			if (FRigControlElement* DebuggedControlElement = DebuggedHierarchy->Find<FRigControlElement>(SelectedKey))
			{
				DebuggedHierarchy->SetControlOffsetTransform(DebuggedControlElement, LocalTransform, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
				DebuggedHierarchy->SetControlOffsetTransform(DebuggedControlElement, LocalTransform, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
				DebuggedHierarchy->SetTransform(DebuggedControlElement, FTransform::Identity, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
				DebuggedHierarchy->SetTransform(DebuggedControlElement, FTransform::Identity, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
			}

			if (FRigControlElement* DefaultControlElement = DefaultHierarchy->Find<FRigControlElement>(SelectedKey))
			{
				DefaultHierarchy->SetControlOffsetTransform(DefaultControlElement, LocalTransform, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo);
				DefaultHierarchy->SetControlOffsetTransform(DefaultControlElement, LocalTransform, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo);
				DefaultHierarchy->SetTransform(DefaultControlElement, FTransform::Identity, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo);
				DefaultHierarchy->SetTransform(DefaultControlElement, FTransform::Identity, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo);
			}
		}
	}
}

bool SRigHierarchy::CanHandleZeroControls() const
{
	if (IsProceduralSelected())
	{
		return false;
	}

	return GetSelectedElementKeys().ContainsByPredicate(
		[](const FRigElementKey& SelectedKey)
		{
			return SelectedKey.Type == ERigElementType::Control;
		});
}

void SRigHierarchy::HandleZeroControlsFromClosestBone()
{
	const TSharedPtr<IControlRigBaseEditor> PinnedControlRigEditor = ControlRigEditor.IsValid() ? ControlRigEditor.Pin() : nullptr;
	const TScriptInterface<IControlRigEditorAssetInterface> Blueprint = PinnedControlRigEditor.IsValid() ? PinnedControlRigEditor->GetControlRigAssetInterface() : nullptr;
	URigHierarchy* DefaultHierarchy = GetDefaultHierarchy();
	URigHierarchy* DebuggedHierarchy = GetHierarchy();

	if (CanHandleZeroControlsFromClosestBone() &&
		PinnedControlRigEditor.IsValid() &&
		Blueprint &&
		DefaultHierarchy &&
		DebuggedHierarchy)
	{
		const FScopedTransaction Transaction(LOCTEXT("ZeroControlsFromClosestBoneTransaction", "Zero Controls from Closest Bone"));

		const TArray<FRigElementKey> SelectedKeys = GetSelectedElementKeys();

		TMap<FRigElementKey, FTransform> ClosestTransforms;
		TMap<FRigElementKey, FTransform> ParentGlobalTransforms;
		for (const FRigElementKey& SelectedKey : GetSelectedElementKeys())
		{
			if (SelectedKey.Type == ERigElementType::Control || SelectedKey.Type == ERigElementType::Null)
			{
				const FTransform GlobalTransform = DebuggedHierarchy->GetGlobalTransform(SelectedKey);
				FTransform ClosestTransform;
				FName ClosestRigElement;

				if (!FindClosestBone(GlobalTransform.GetLocation(), ClosestRigElement, ClosestTransform))
				{
					continue;
				}

				ClosestTransforms.FindOrAdd(SelectedKey) = ClosestTransform;
				ParentGlobalTransforms.FindOrAdd(SelectedKey) = DebuggedHierarchy->GetParentTransform(SelectedKey);
			}
		}

		for (const FRigElementKey& SelectedKey : SelectedKeys)
		{
			if (!ClosestTransforms.Contains(SelectedKey))
			{
				continue;
			}
			const FTransform GlobalTransform = ClosestTransforms[SelectedKey];
			const FTransform LocalTransform = GlobalTransform.GetRelativeTransform(ParentGlobalTransforms[SelectedKey]);

			if (SelectedKey.Type == ERigElementType::Control)
			{
				constexpr bool bAffectChildren = true;
				constexpr bool bSetupUndo = true;
				constexpr bool bForce = false;
				constexpr bool bPrintPythonCommand = true;

				if (FRigControlElement* DebuggedControlElement = DebuggedHierarchy->Find<FRigControlElement>(SelectedKey))
				{
					DebuggedHierarchy->SetControlOffsetTransform(DebuggedControlElement, LocalTransform, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
					DebuggedHierarchy->SetControlOffsetTransform(DebuggedControlElement, LocalTransform, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
					DebuggedHierarchy->SetTransform(DebuggedControlElement, FTransform::Identity, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
					DebuggedHierarchy->SetTransform(DebuggedControlElement, FTransform::Identity, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
				}

				if (FRigControlElement* DefaultControlElement = DefaultHierarchy->Find<FRigControlElement>(SelectedKey))
				{
					DefaultHierarchy->SetControlOffsetTransform(DefaultControlElement, LocalTransform, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo);
					DefaultHierarchy->SetControlOffsetTransform(DefaultControlElement, LocalTransform, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo);
					DefaultHierarchy->SetTransform(DefaultControlElement, FTransform::Identity, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo);
					DefaultHierarchy->SetTransform(DefaultControlElement, FTransform::Identity, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo);
				}
			}
		}
	}
}

bool SRigHierarchy::CanHandleZeroControlsFromClosestBone() const
{
	if (IsProceduralSelected())
	{
		return false;
	}

	return GetSelectedElementKeys().ContainsByPredicate(
		[](const FRigElementKey& SelectedKey)
		{
			return SelectedKey.Type == ERigElementType::Control;
		});
}

void SRigHierarchy::HandleZeroControlShape()
{
	const TSharedPtr<IControlRigBaseEditor> PinnedControlRigEditor = ControlRigEditor.IsValid() ? ControlRigEditor.Pin() : nullptr;
	const TScriptInterface<IControlRigEditorAssetInterface> Blueprint = PinnedControlRigEditor.IsValid() ? PinnedControlRigEditor->GetControlRigAssetInterface() : nullptr;
	URigHierarchy* DebuggedHierarchy = GetHierarchy();

	if (CanHandleZeroControlShape() &&
		PinnedControlRigEditor.IsValid() &&
		Blueprint &&
		DebuggedHierarchy)
	{
		const FScopedTransaction Transaction(LOCTEXT("ZeroControlShapeTransaction", "Zero Control Shape"));

		const FRigHierarchyInteractionBracket DebuggedInteractionBracket(DebuggedHierarchy);

		TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = TreeView->GetSelectedItems();
		for (const TSharedPtr<FRigHierarchyTreeElement>& SelectedItem : SelectedItems)
		{
			if (!SelectedItem->Key.IsElement())
			{
				continue;
			}
			if (FRigControlElement* ControlElement = DebuggedHierarchy->Find<FRigControlElement>(SelectedItem->Key.GetElement()))
			{
				if (!ControlElement->Settings.SupportsShape())
				{
					continue;
				}

				const FRigElementKey Key = ControlElement->GetKey();

				const FTransform OffsetGlobalTransform = DebuggedHierarchy->GetGlobalControlOffsetTransform(Key);
				const FTransform ShapeGlobalTransform = DebuggedHierarchy->GetGlobalControlShapeTransform(Key);
				const FTransform ShapeLocalTransform = ShapeGlobalTransform.GetRelativeTransform(OffsetGlobalTransform);

				constexpr bool bInitial = true;
				constexpr bool bNotInitial = false;
				constexpr bool bSetupUndo = true;
				constexpr bool bAffectChildren = true;
				constexpr bool bPrintPythonCommand = true;

				DebuggedHierarchy->SetControlShapeTransform(Key, ShapeLocalTransform, bInitial, bSetupUndo);
				DebuggedHierarchy->SetControlShapeTransform(Key, ShapeLocalTransform, bNotInitial, bSetupUndo);
				GetHierarchy()->SetControlShapeTransform(Key, ShapeLocalTransform, bInitial, bSetupUndo);
				GetHierarchy()->SetControlShapeTransform(Key, ShapeLocalTransform, bNotInitial, bSetupUndo);

				DebuggedHierarchy->SetLocalTransform(Key, FTransform::Identity, bNotInitial, bAffectChildren, bSetupUndo);
				DebuggedHierarchy->SetLocalTransform(Key, FTransform::Identity, bInitial, bAffectChildren, bSetupUndo);
				GetHierarchy()->SetLocalTransform(Key, FTransform::Identity, bNotInitial, bAffectChildren, bSetupUndo, bPrintPythonCommand);
				GetHierarchy()->SetLocalTransform(Key, FTransform::Identity, bInitial, bAffectChildren, bSetupUndo, bPrintPythonCommand);

				if (FControlRigEditorEditMode* EditMode = PinnedControlRigEditor->GetEditMode())
				{
					EditMode->RequestToRecreateControlShapeActors();
				}
			}
		}
	}
}

bool SRigHierarchy::CanHandleZeroControlShape() const
{
	if (IsProceduralSelected())
	{
		return false;
	}

	return GetSelectedElementKeys().ContainsByPredicate(
		[](const FRigElementKey& SelectedKey)
		{
			return SelectedKey.Type == ERigElementType::Control;
		});
}

void SRigHierarchy::HandleSetInitialTransformsAsCurrent()
{
	const TScriptInterface<IControlRigEditorAssetInterface> Blueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface();
	URigHierarchy* DefaultHierarchy = GetDefaultHierarchy();
	URigHierarchy* DebuggedHierarchy = GetHierarchy();

	if (CanHandleSetInitialTransformsAsCurrent() &&
		Blueprint &&
		DefaultHierarchy &&
		DebuggedHierarchy)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetInitialTransformsAsCurrentTransaction", "Set Initial Transforms as Current"));

		const TArray<FRigElementKey> SelectedKeys = GetSelectedElementKeys();

		TMap<FRigElementKey, FTransform> GlobalTransforms;
		TMap<FRigElementKey, FTransform> ParentGlobalTransforms;
		for (const FRigElementKey& SelectedKey : SelectedKeys)
		{
			GlobalTransforms.FindOrAdd(SelectedKey) = DebuggedHierarchy->GetGlobalTransform(SelectedKey);
			ParentGlobalTransforms.FindOrAdd(SelectedKey) = DebuggedHierarchy->GetParentTransform(SelectedKey);
		}

		for (const FRigElementKey& SelectedKey : SelectedKeys)
		{
			if (SelectedKey.Type == ERigElementType::Null ||
				SelectedKey.Type == ERigElementType::Bone ||
				SelectedKey.Type == ERigElementType::Connector)
			{
				const FTransform GlobalTransform = GlobalTransforms[SelectedKey];
				const FTransform LocalTransform = GlobalTransform.GetRelativeTransform(ParentGlobalTransforms[SelectedKey]);

				constexpr bool bAffectChildren = true;
				constexpr bool bSetupUndo = true;
				constexpr bool bForce = false;
				constexpr bool bPrintPythonCommand = true;

				if (FRigTransformElement* DebuggedTransformElement = DebuggedHierarchy->Find<FRigTransformElement>(SelectedKey))
				{
					DebuggedHierarchy->SetTransform(DebuggedTransformElement, LocalTransform, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
					DebuggedHierarchy->SetTransform(DebuggedTransformElement, LocalTransform, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo, bForce, bPrintPythonCommand);
				}
				
				if (FRigTransformElement* DefaultTransformElement = DefaultHierarchy->Find<FRigTransformElement>(SelectedKey))
				{
					DefaultHierarchy->SetTransform(DefaultTransformElement, LocalTransform, ERigTransformType::InitialLocal, bAffectChildren, bSetupUndo);
					DefaultHierarchy->SetTransform(DefaultTransformElement, LocalTransform, ERigTransformType::CurrentLocal, bAffectChildren, bSetupUndo);
				}
			}
		}
	}
}

bool SRigHierarchy::CanHandleSetInitialTransformsAsCurrent() const
{
	if (IsProceduralSelected())
	{
		return false;
	}

	return GetSelectedElementKeys().ContainsByPredicate(
		[](const FRigElementKey& SelectedKey)
		{
			return
				SelectedKey.Type == ERigElementType::Null ||
				SelectedKey.Type == ERigElementType::Bone ||
				SelectedKey.Type == ERigElementType::Connector;
		});
}

void SRigHierarchy::HandleFrameSelection(const bool bFrameInViewport)
{
	TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	for (TSharedPtr<FRigHierarchyTreeElement> SelectedItem : SelectedItems)
	{
		TreeView->SetExpansionRecursive(SelectedItem, true, true);
	}

	if (SelectedItems.Num() > 0)
	{
		TreeView->RequestScrollIntoView(SelectedItems.Last());

		if (ControlRigEditor.IsValid() &&
			bFrameInViewport)
		{
			if (FControlRigEditorEditMode* EditMode = ControlRigEditor.Pin()->GetEditMode())
			{
				EditMode->FrameSelection();
			}
		}
	}
}

void SRigHierarchy::HandleControlBoneOrSpaceTransform()
{
	FControlRigAssetInterfacePtr Blueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface();
	if (Blueprint == nullptr)
	{
		return;
	}

	UControlRig* DebuggedControlRig = Cast<UControlRig>(Blueprint->GetRigVMAssetInterface()->GetObjectBeingDebugged());
	if(DebuggedControlRig == nullptr)
	{
		return;
	}

	TArray<FRigElementKey> SelectedKeys = GetSelectedElementKeys();
	if (SelectedKeys.Num() == 1)
	{
		if (SelectedKeys[0].Type == ERigElementType::Bone ||
			SelectedKeys[0].Type == ERigElementType::Null ||
			SelectedKeys[0].Type == ERigElementType::Connector)
		{
			if(!DebuggedControlRig->GetHierarchy()->IsProcedural(SelectedKeys[0]))
			{
				Blueprint->AddTransientControl(SelectedKeys[0]);
			}
		}
	}
}

void SRigHierarchy::HandleUnparent()
{
	FControlRigAssetInterfacePtr Blueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface();
	if (Blueprint == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("HierarchyTreeUnparentSelected", "Unparent selected items from hierarchy"));

	bool bUnparentImportedBones = false;
	bool bConfirmedByUser = false;

	TArray<FRigElementKey> SelectedKeys = GetSelectedElementKeys();
	TMap<FRigElementKey, FTransform> InitialTransforms;
	TMap<FRigElementKey, FTransform> GlobalTransforms;

	for (const FRigElementKey& SelectedKey : SelectedKeys)
	{
		URigHierarchy* Hierarchy = GetHierarchy();
		InitialTransforms.Add(SelectedKey, Hierarchy->GetInitialGlobalTransform(SelectedKey));
		GlobalTransforms.Add(SelectedKey, Hierarchy->GetGlobalTransform(SelectedKey));
	}

	for (const FRigElementKey& SelectedKey : SelectedKeys)
	{
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

		URigHierarchy* Hierarchy = GetDefaultHierarchy();
		check(Hierarchy);

		URigHierarchyController* Controller = Hierarchy->GetController(true);
		check(Controller);

		const FTransform& InitialTransform = InitialTransforms.FindChecked(SelectedKey);
		const FTransform& GlobalTransform = GlobalTransforms.FindChecked(SelectedKey);

		switch (SelectedKey.Type)
		{
			case ERigElementType::Bone:
			{
				
				bool bIsImportedBone = false;
				if(FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(SelectedKey))
				{
					bIsImportedBone = BoneElement->BoneType == ERigBoneType::Imported;
				}
					
				if (bIsImportedBone && !bConfirmedByUser)
				{
					FText ConfirmUnparent = LOCTEXT("ConfirmUnparentBoneHierarchy", "Unparenting imported(white) bones can cause issues with animation - are you sure ?");

					FSuppressableWarningDialog::FSetupInfo Info(ConfirmUnparent, LOCTEXT("UnparentImportedBone", "Unparent Imported Bone"), "UnparentImportedBoneHierarchy_Warning");
					Info.ConfirmText = LOCTEXT("UnparentImportedBoneHierarchy_Yes", "Yes");
					Info.CancelText = LOCTEXT("UnparentImportedBoneHierarchy_No", "No");

					FSuppressableWarningDialog UnparentImportedBonesInHierarchy(Info);
					bUnparentImportedBones = UnparentImportedBonesInHierarchy.ShowModal() != FSuppressableWarningDialog::Cancel;
					bConfirmedByUser = true;
				}

				if (bUnparentImportedBones || !bIsImportedBone)
				{
					Controller->RemoveAllParents(SelectedKey, true, true, true);
				}
				break;
			}
			case ERigElementType::Null:
			case ERigElementType::Control:
			case ERigElementType::Connector:
			{
				Controller->RemoveAllParents(SelectedKey, true, true, true);
				break;
			}
			default:
			{
				break;
			}
		}

		Hierarchy->SetInitialGlobalTransform(SelectedKey, InitialTransform, true, true);
		Hierarchy->SetGlobalTransform(SelectedKey, GlobalTransform, false, true, true);
	}

	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	ControlRigEditor.Pin()->OnHierarchyChanged();
	RefreshTreeView();

	if (URigHierarchy* Hierarchy = GetDefaultHierarchy())
	{
		Hierarchy->GetController()->SetSelection(SelectedKeys);
	}
	
	FSlateApplication::Get().DismissAllMenus();
}

bool SRigHierarchy::FindClosestBone(const FVector& Point, FName& OutRigElementName, FTransform& OutGlobalTransform) const
{
	if (URigHierarchy* DebuggedHierarchy = GetHierarchy())
	{
		float NearestDistance = BIG_NUMBER;

		DebuggedHierarchy->ForEach<FRigBoneElement>([&] (FRigBoneElement* Element) -> bool
		{
			const FTransform CurTransform = DebuggedHierarchy->GetTransform(Element, ERigTransformType::CurrentGlobal);
            const float CurDistance = FVector::Distance(CurTransform.GetLocation(), Point);
            if (CurDistance < NearestDistance)
            {
                NearestDistance = CurDistance;
                OutGlobalTransform = CurTransform;
                OutRigElementName = Element->GetFName();
            }
            return true;
		});

		return (OutRigElementName != NAME_None);
	}
	return false;
}

void SRigHierarchy::HandleTestSpaceSwitching()
{
	if (FControlRigEditorEditMode* EditMode = ControlRigEditor.Pin()->GetEditMode())
	{
		/// toooo centralize code here
		EditMode->OpenSpacePickerWidget();
	}
}

void SRigHierarchy::HandleParent(const FToolMenuContext& Context)
{
	if (UControlRigContextMenuContext* MenuContext = Cast<UControlRigContextMenuContext>(Context.FindByClass(UControlRigContextMenuContext::StaticClass())))
	{
		const FControlRigRigHierarchyDragAndDropContext DragAndDropContext = MenuContext->GetRigHierarchyDragAndDropContext();
		ReparentOrMatchTransform(DragAndDropContext.DraggedHierarchyKeys, DragAndDropContext.TargetHierarchyKey, true);
	}
}

void SRigHierarchy::HandleAlign(const FToolMenuContext& Context)
{
	if (UControlRigContextMenuContext* MenuContext = Cast<UControlRigContextMenuContext>(Context.FindByClass(UControlRigContextMenuContext::StaticClass())))
	{
		const FControlRigRigHierarchyDragAndDropContext DragAndDropContext = MenuContext->GetRigHierarchyDragAndDropContext();
		ReparentOrMatchTransform(DragAndDropContext.DraggedHierarchyKeys, DragAndDropContext.TargetHierarchyKey, false);
	}
}

FReply SRigHierarchy::ReparentOrMatchTransform(const TArray<FRigHierarchyKey>& DraggedKeys, FRigHierarchyKey TargetKey, bool bReparentItems, int32 LocalIndex)
{
	bool bMatchTransforms = !bReparentItems;

	URigHierarchy* DebuggedHierarchy = GetHierarchy();
	URigHierarchy* Hierarchy = GetDefaultHierarchy();

	const TArray<FRigElementKey> SelectedKeys = (Hierarchy) ? Hierarchy->GetSelectedKeys() : TArray<FRigElementKey>();

	if (Hierarchy && ControlRigBlueprint.IsValid())
	{
		URigHierarchyController* Controller = Hierarchy->GetController(true);
		if(Controller == nullptr)
		{
			return FReply::Unhandled();
		}
		if(!TargetKey.IsElement())
		{
			return FReply::Unhandled();
		}

		// only suspend blueprint notifs if we are dragging non-components
		bool bBlueprintSuspensionFlag = ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications;
		for (const FRigHierarchyKey& DraggedKey : DraggedKeys)
		{
			if (!DraggedKey.IsComponent())
			{
				bBlueprintSuspensionFlag = true;
			}
		}

		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, bBlueprintSuspensionFlag);
		FScopedTransaction Transaction(LOCTEXT("HierarchyDragAndDrop", "Drag & Drop"));
		FRigHierarchyInteractionBracket InteractionBracket(Hierarchy);

		FTransform TargetGlobalTransform = DebuggedHierarchy->GetGlobalTransform(TargetKey.GetElement());

		for (const FRigHierarchyKey& DraggedKey : DraggedKeys)
		{
			if (DraggedKey == TargetKey)
			{
				return FReply::Unhandled();
			}

			if(DraggedKey.IsElement())
			{
				if (bReparentItems)
				{
					if(Hierarchy->IsParentedTo(TargetKey.GetElement(), DraggedKey.GetElement()))
					{
						if(LocalIndex == INDEX_NONE)
						{
							return FReply::Unhandled();
						}
					}
				}

				if (DraggedKey.GetElement().Type == ERigElementType::Bone)
				{
					if(FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(DraggedKey.GetElement()))
					{
						if (BoneElement->BoneType == ERigBoneType::Imported && BoneElement->ParentElement != nullptr)
						{
							FText ConfirmText = bMatchTransforms ?
								LOCTEXT("ConfirmMatchTransform", "Matching transforms of imported(white) bones can cause issues with animation - are you sure ?") :
								LOCTEXT("ConfirmReparentBoneHierarchy", "Reparenting imported(white) bones can cause issues with animation - are you sure ?");

							FText TitleText = bMatchTransforms ?
								LOCTEXT("MatchTransformImportedBone", "Match Transform on Imported Bone") :
								LOCTEXT("ReparentImportedBone", "Reparent Imported Bone");

							FSuppressableWarningDialog::FSetupInfo Info(ConfirmText, TitleText, "SRigHierarchy_Warning");
							Info.ConfirmText = LOCTEXT("SRigHierarchy_Warning_Yes", "Yes");
							Info.CancelText = LOCTEXT("SRigHierarchy_Warning_No", "No");

							FSuppressableWarningDialog ChangeImportedBonesInHierarchy(Info);
							if (ChangeImportedBonesInHierarchy.ShowModal() == FSuppressableWarningDialog::Cancel)
							{
								return FReply::Unhandled();
							}
						}
					}
				}
			}

			if(DraggedKey.IsComponent())
			{
				if(DraggedKey.GetElement() == TargetKey.GetElement())
				{
					return FReply::Unhandled();
				}
			}
		}

		for (const FRigHierarchyKey& DraggedKey : DraggedKeys)
		{
			if (bMatchTransforms && DraggedKey.IsElement())
			{
				if (DraggedKey.GetElement().Type == ERigElementType::Control)
				{
					int32 ControlIndex = DebuggedHierarchy->GetIndex(DraggedKey.GetElement());
					if (ControlIndex == INDEX_NONE)
					{
						continue;
					}

					FTransform ParentTransform = DebuggedHierarchy->GetParentTransformByIndex(ControlIndex, false);
					FTransform OffsetTransform = TargetGlobalTransform.GetRelativeTransform(ParentTransform);

					Hierarchy->SetControlOffsetTransformByIndex(ControlIndex, OffsetTransform, ERigTransformType::InitialLocal, true, true, true);
					Hierarchy->SetControlOffsetTransformByIndex(ControlIndex, OffsetTransform, ERigTransformType::CurrentLocal, true, true, true);
					Hierarchy->SetLocalTransform(DraggedKey.GetElement(), FTransform::Identity, true, true, true, true);
					Hierarchy->SetInitialLocalTransform(DraggedKey.GetElement(), FTransform::Identity, true, true, true);
					DebuggedHierarchy->SetControlOffsetTransformByIndex(ControlIndex, OffsetTransform, ERigTransformType::InitialLocal, true, true);
					DebuggedHierarchy->SetControlOffsetTransformByIndex(ControlIndex, OffsetTransform, ERigTransformType::CurrentLocal, true, true);
					DebuggedHierarchy->SetLocalTransform(DraggedKey.GetElement(), FTransform::Identity, true, true, true);
					DebuggedHierarchy->SetInitialLocalTransform(DraggedKey.GetElement(), FTransform::Identity, true, true);
				}
				else
				{
					Hierarchy->SetInitialGlobalTransform(DraggedKey.GetElement(), TargetGlobalTransform, true, true);
					Hierarchy->SetGlobalTransform(DraggedKey.GetElement(), TargetGlobalTransform, false, true, true);
					DebuggedHierarchy->SetInitialGlobalTransform(DraggedKey.GetElement(), TargetGlobalTransform, true, true);
					DebuggedHierarchy->SetGlobalTransform(DraggedKey.GetElement(), TargetGlobalTransform, false, true, true);
				}
				continue;
			}

			FRigElementKey ParentKey = TargetKey.GetElement();

			if(DraggedKey.IsComponent())
			{
				Controller->ReparentComponent(DraggedKey.GetComponent(), ParentKey, true, true);
			}

			if(DraggedKey.IsElement())
			{
				const FTransform InitialGlobalTransform = DebuggedHierarchy->GetInitialGlobalTransform(DraggedKey.GetElement());
				const FTransform CurrentGlobalTransform = DebuggedHierarchy->GetGlobalTransform(DraggedKey.GetElement());
				const FTransform InitialLocalTransform = DebuggedHierarchy->GetInitialLocalTransform(DraggedKey.GetElement());
				const FTransform CurrentLocalTransform = DebuggedHierarchy->GetLocalTransform(DraggedKey.GetElement());
				const FTransform CurrentGlobalOffsetTransform = DebuggedHierarchy->GetGlobalControlOffsetTransform(DraggedKey.GetElement(), false);

				bool bElementWasReparented = false;
				if(ParentKey.IsValid() && (Hierarchy->GetFirstParent(DraggedKey.GetElement()) != ParentKey))
				{
					bElementWasReparented = Controller->SetParent(DraggedKey.GetElement(), ParentKey, true, true, true);
				}
				else if(!ParentKey.IsValid() && (Hierarchy->GetNumberOfParents(DraggedKey.GetElement()) > 0))
				{
					bElementWasReparented = Controller->RemoveAllParents(DraggedKey.GetElement(), true, true, true);
				}

				if(LocalIndex != INDEX_NONE)
				{
					Controller->ReorderElement(DraggedKey.GetElement(), LocalIndex, true, true);
				}

				if(bElementWasReparented)
				{
					if (DraggedKey.GetElement().Type == ERigElementType::Control)
					{
						int32 ControlIndex = DebuggedHierarchy->GetIndex(DraggedKey.GetElement());
						if (ControlIndex == INDEX_NONE)
						{
							continue;
						}

						const FTransform GlobalParentTransform = DebuggedHierarchy->GetGlobalTransform(ParentKey, false);
						const FTransform LocalOffsetTransform = CurrentGlobalOffsetTransform.GetRelativeTransform(GlobalParentTransform);

						Hierarchy->SetControlOffsetTransformByIndex(ControlIndex, LocalOffsetTransform, ERigTransformType::InitialLocal, true, true, true);
						Hierarchy->SetControlOffsetTransformByIndex(ControlIndex, LocalOffsetTransform, ERigTransformType::CurrentLocal, true, true, true);
						Hierarchy->SetLocalTransform(DraggedKey.GetElement(), CurrentLocalTransform, true, true, true, true);
						Hierarchy->SetInitialLocalTransform(DraggedKey.GetElement(), InitialLocalTransform, true, true, true);
						DebuggedHierarchy->SetControlOffsetTransformByIndex(ControlIndex, LocalOffsetTransform, ERigTransformType::InitialLocal, true, true);
						DebuggedHierarchy->SetControlOffsetTransformByIndex(ControlIndex, LocalOffsetTransform, ERigTransformType::CurrentLocal, true, true);
						DebuggedHierarchy->SetLocalTransform(DraggedKey.GetElement(), CurrentLocalTransform, true, true, true);
						DebuggedHierarchy->SetInitialLocalTransform(DraggedKey.GetElement(), InitialLocalTransform, true, true);
					}
					else
					{
						DebuggedHierarchy->SetInitialGlobalTransform(DraggedKey.GetElement(), InitialGlobalTransform, true, true);
						DebuggedHierarchy->SetGlobalTransform(DraggedKey.GetElement(), CurrentGlobalTransform, false, true, true);
						Hierarchy->SetInitialGlobalTransform(DraggedKey.GetElement(), InitialGlobalTransform, true, true);
						Hierarchy->SetGlobalTransform(DraggedKey.GetElement(), CurrentGlobalTransform, false, true, true);
					}
				}
			}
		}
	}

	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();

	if(bReparentItems)
	{
		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);
		ControlRigBlueprint->GetRigVMAssetInterface()->BroadcastRefreshEditor();
		RefreshTreeView();
	}

	if (Hierarchy)
	{
		Hierarchy->GetController()->SetSelection(SelectedKeys);
	}
	
	return FReply::Handled();

}

FReply SRigHierarchy::ResolveConnector(const FRigElementKey& DraggedKey, const FRigElementKey& TargetKey)
{
	return ResolveConnectorToArray(DraggedKey, {TargetKey});
}

FReply SRigHierarchy::ResolveConnectorToArray(const FRigElementKey& DraggedKey, const TArray<FRigElementKey>& TargetKeys)
{
	if (FControlRigAssetInterfacePtr Blueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface())
	{
		if (const URigHierarchy* DebuggedHierarchy = GetHierarchy())
		{
			if(DebuggedHierarchy->Contains(DraggedKey))
			{
				if(Blueprint->ResolveConnectorToArray(DraggedKey, TargetKeys))
				{
					RefreshTreeView();
					return FReply::Handled();
				}
			}
		}
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE

