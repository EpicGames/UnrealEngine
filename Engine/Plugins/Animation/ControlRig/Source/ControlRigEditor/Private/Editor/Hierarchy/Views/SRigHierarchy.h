// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigEditorAsset.h"
#include "Editor/Hierarchy/RigHierarchyImportSettings.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "EditorUndoClient.h"
#include "Engine/SkeletalMesh.h"

class FMenuBuilder;
class FUICommandList;
class IControlRigBaseEditor;
class SRigHierarchy;
class SSearchBox;
class UControlRig;
class URigVMBlueprint;
class UToolMenu;
struct FAssetData;
struct FToolMenuContext;

/** Widget allowing editing of a control rig's structure */
class SRigHierarchy : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SRigHierarchy) 
		: _ForModalWindow(false)
		{}

		/** Disables features that don't work in modal windows due to the absence of the game ticker and frame numbers */
		SLATE_ARGUMENT(bool, ForModalWindow)

	SLATE_END_ARGS()

	~SRigHierarchy();

	/**
	 * Constructs this widget
	 *
	 * @param InArgs				Slate Arguments
	 * @param InControlRigEditor	The control rig editor that displays this view
	 * @param InViewName			The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor
	 */
	void Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor, const FName& InViewName);

	IControlRigBaseEditor* GetControlRigEditor() const
	{
		if(ControlRigEditor.IsValid())
		{
			return ControlRigEditor.Pin().Get();
		}
		return nullptr;
	}

private:

	void OnEditorClose(IControlRigBaseEditor* InEditor, FControlRigAssetInterfacePtr InBlueprint);

	/** Bind commands that this widget handles */
	void BindCommands();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Rebuild the tree view */
	void RefreshTreeView(bool bRebuildContent = true);

	/** Return all selected keys */
	TArray<FRigHierarchyKey> GetSelectedKeys() const;

	/** Return all selected element keys */
	TArray<FRigElementKey> GetSelectedElementKeys() const;

	/** Check whether we can deleting the selected item(s) */
	bool CanDeleteItem() const;

	/** Delete Item */
	void HandleDeleteItem();

	/** Create a new item */
	void HandleNewItem(ERigElementType InElementType, bool bIsAnimationChannel);

	/** Check we can find the references of an item */
	bool CanFindReferencesOfItem() const;

	/** Find all references of an item */
	void HandleFindReferencesOfItem();

	/** Check whether we can deleting the selected item(s) */
	bool CanDuplicateItem() const;

	/** Duplicate Item */
	void HandleDuplicateItem();

	/** Mirror Item */
	void HandleMirrorItem();

	/** Check whether we can deleting the selected item(s) */
	bool CanRenameItem() const;

	/** Rename Item */
	void HandleRenameItem();

	bool CanPasteItems() const;
	bool CanCopyOrPasteItems() const;
	void HandleCopyItems();
	void HandlePasteItems();
	void HandlePasteLocalTransforms();
	void HandlePasteGlobalTransforms();
	void HandlePasteTransforms(ERigTransformType::Type InTransformType, bool bAffectChildren);

	/** Set Selection Changed */
	void OnSelectionChanged(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo);

	TSharedRef< SWidget > CreateFilterMenu();
	TSharedPtr< SWidget > CreateContextMenuWidget();
	void OnItemClicked(TSharedPtr<FRigHierarchyTreeElement> InItem);
	void OnItemDoubleClicked(TSharedPtr<FRigHierarchyTreeElement> InItem);
	void OnSetExpansionRecursive(TSharedPtr<FRigHierarchyTreeElement> InItem, bool bShouldBeExpanded);
	TOptional<FText> OnGetItemTooltip(const FRigHierarchyKey& InKey) const;

	// FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	// reply to a drag operation
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigHierarchyTreeElement> TargetItem);
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigHierarchyTreeElement> TargetItem);
	void OnElementKeyTagDragDetected(const FRigElementKey& InDraggedTag);
	void UpdateConnectorMatchesOnDrag(const TArray<FRigHierarchyKey>& InDraggedKeys);

	static const FName ContextMenuName;
	static void CreateContextMenu();
	UToolMenu* GetContextMenu();
	TSharedPtr<FUICommandList> GetContextMenuCommands() const;
	
	static const FName DragDropMenuName;
	static void CreateDragDropMenu();
	UToolMenu* GetDragDropMenu(const TArray<FRigHierarchyKey>& DraggedKeys, FRigElementKey TargetKey);

	/** Our owning control rig editor */
	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;

	FRigHierarchyTreeDisplaySettings DisplaySettings;
	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings() const { return DisplaySettings; }

	// struct for import hierarchy dialog (not on stack to allow non-modal dialog)
	FRigHierarchyImportSettings ImportSettings;

	/** Search box widget */
	TSharedPtr<SSearchBox> FilterBox;

	EVisibility IsToolbarVisible() const;
	EVisibility IsSearchbarVisible() const;
	FReply OnImportSkeletonClicked();
	FText GetImportHierarchyText() const;
	bool IsImportHierarchyEnabled() const;
	void OnFilterTextChanged(const FText& SearchText);

	/** Tree view widget */
	TSharedPtr<SRigHierarchyTreeView> TreeView;

	TWeakInterfacePtr<IControlRigEditorAssetInterface> ControlRigBlueprint;
	TWeakObjectPtr<UControlRig> ControlRigBeingDebuggedPtr;
	
	/** Command list we bind to */
	TSharedPtr<FUICommandList> CommandList;

	bool IsMultiSelected(bool bIncludeProcedural) const;
	bool IsSingleSelected(bool bIncludeProcedural) const;
	bool IsSingleBoneSelected(bool bIncludeProcedural) const;
	bool IsSingleNullSelected(bool bIncludeProcedural) const;
	bool IsControlSelected(bool bIncludeProcedural) const;
	bool IsControlOrNullSelected(bool bIncludeProcedural) const;
	bool IsProceduralSelected() const;
	bool IsNonProceduralSelected() const;
	bool CanAddElement(const ERigElementType ElementType) const;
	bool CanAddAnimationChannel() const;

	URigHierarchy* GetHierarchy() const;
	URigHierarchy* GetDefaultHierarchy() const;
	const URigHierarchy* GetHierarchyForTreeView() const { return GetHierarchy(); }
	FRigHierarchyKey OnGetResolvedKey(const FRigHierarchyKey& InKey);
	void OnRequestDetailsInspection(const FRigHierarchyKey& InKey);
	
	void ImportHierarchy(const FAssetData& InAssetData);
	void CreateImportMenu(FMenuBuilder& MenuBuilder);
	void CreateRefreshMenu(FMenuBuilder& MenuBuilder);
	void CreateResetCurvesMenu(FMenuBuilder& MenuBuilder);
	bool ShouldFilterOnImport(const FAssetData& AssetData) const;
	void RefreshHierarchy(const FAssetData& InAssetData, bool bOnlyResetCurves, bool bRestrictToMeshBones);
	void UpdateMesh(USkeletalMesh* InMesh, const bool bImport) const;

	void HandleResetTransform(bool bSelectionOnly);

	/** Sets the transform of the control as offset, zeroes the initial and current transform */
	void HandleZeroControls();
	bool CanHandleZeroControls() const;

	/** Sets the offset to the transform of the closest bone, zeroes the initial and current transform */
	void HandleZeroControlsFromClosestBone();
	bool CanHandleZeroControlsFromClosestBone() const;

	/** Sets the current shape relative to the control's offset, zeroes the control's initial and current transform */
	void HandleZeroControlShape();
	bool CanHandleZeroControlShape() const;

	/** Set the current transform of selected bone, null or connector as its initial and current transform */
	void HandleSetInitialTransformsAsCurrent();
	bool CanHandleSetInitialTransformsAsCurrent() const;

	void HandleFrameSelection(const bool bFrameInViewport);
	void HandleControlBoneOrSpaceTransform();
	void HandleUnparent();
	bool FindClosestBone(const FVector& Point, FName& OutRigElementName, FTransform& OutGlobalTransform) const;
	void HandleTestSpaceSwitching();

	void HandleParent(const FToolMenuContext& Context);
	void HandleAlign(const FToolMenuContext& Context);
	FReply ReparentOrMatchTransform(const TArray<FRigHierarchyKey>& DraggedKeys, FRigHierarchyKey TargetKey, bool bReparentItems, int32 LocalIndex = INDEX_NONE);
	FReply ResolveConnector(const FRigElementKey& DraggedKey, const FRigElementKey& TargetKey);
	FReply ResolveConnectorToArray(const FRigElementKey& DraggedKey, const TArray<FRigElementKey>& TargetKeys);

	FName CreateUniqueName(const FName& InBaseName, ERigElementType InElementType) const;

	void ClearDetailPanel() const;

	bool bIsChangingRigHierarchy;
	void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void OnModularRigModified(EModularRigNotification InNotif, const FRigModuleReference* InModule);
	void HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint);
	void HandleSetObjectBeingDebugged(UObject* InObject);
	void OnPreConstruction_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnPostConstruction_AnyThread(UControlRig* InRig, const FName& InEventName);
	void OnNavigateToFirstConnectorWarning();

	bool bIsConstructionEventRunning;
	bool bRestrictRefreshToMeshBones = false;
	uint32 LastHierarchyHash;
	TArray<FRigHierarchyKey> SelectionBeforeConstruction;
	TMap<FRigElementKey, FModularRigResolveResult> DragRigResolveResults;

public:
	FName HandleRenameElement(const FRigHierarchyKey& OldKey, const FString& NewName);
	bool HandleVerifyNameChanged(const FRigHierarchyKey& OldKey, const FString& NewName, FText& OutErrorMessage);

	// SWidget Overrides
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor */
	FName ViewName;

	friend class FRigHierarchyTreeElement;
	friend class SRigHierarchyItem;
	friend class UControlRigBlueprintEditorLibrary;
};

