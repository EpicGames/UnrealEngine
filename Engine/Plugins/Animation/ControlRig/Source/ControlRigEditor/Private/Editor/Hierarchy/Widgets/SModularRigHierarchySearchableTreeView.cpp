// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModularRigHierarchySearchableTreeView.h"

#include "SModularRigHierarchyTreeView.h"
#include "Widgets/Layout/SScrollBox.h"

void SModularRigHierarchySearchableTreeView::Construct(const FArguments& InArgs, const TSharedRef<FModularRigHierarchyViewModel>& InViewModel, const FName& InViewName)
{
	FModularRigHierarchyTreeDelegates TreeDelegates = InArgs._RigTreeDelegates;
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)
		.Padding(0.0f, 0.0f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
				[
					SAssignNew(TreeView, SModularRigHierarchyTreeView, InViewModel, InViewName)
					.RigTreeDelegates(TreeDelegates)
				]
			]
		]
	];
}
