// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/ModularRigHierarchyTreeDelegates.h"
#include "Widgets/SCompoundWidget.h"

class SModularRigHierarchyTreeView;
namespace UE::ControlRigEditor { class FModularRigHierarchyViewModel; }

class SModularRigHierarchySearchableTreeView : public SCompoundWidget
{
	using FModularRigHierarchyViewModel = UE::ControlRigEditor::FModularRigHierarchyViewModel;

public:

	SLATE_BEGIN_ARGS(SModularRigHierarchySearchableTreeView) {}
		SLATE_ARGUMENT(FModularRigHierarchyTreeDelegates, RigTreeDelegates)
		SLATE_ARGUMENT(FText, InitialFilterText)
	SLATE_END_ARGS()

	/** 
	 * Constructs this widget 
	 * 
	 * @param InArgs				Slate Arguments
	 * @param InViewModel			The view model for this widget
	 * @param InViewName			The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FModularRigHierarchyViewModel>& InViewModel, const FName& InViewName);

	virtual ~SModularRigHierarchySearchableTreeView() = default;
	TSharedRef<SModularRigHierarchyTreeView> GetTreeView() const { return TreeView.ToSharedRef(); }

private:

	TSharedPtr<SModularRigHierarchyTreeView> TreeView;
};
