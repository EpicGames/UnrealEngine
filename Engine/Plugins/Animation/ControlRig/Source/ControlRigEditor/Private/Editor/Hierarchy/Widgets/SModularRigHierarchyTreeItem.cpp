// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModularRigHierarchyTreeItem.h"

#include "ControlRigEditorStyle.h"
#include "Editor/Hierarchy/Widgets/SModularRigHierarchyTreeView.h"
#include "Editor/ModularRigHierarchyConnectorWarning.h"
#include "Editor/SRigConnectorTargetWidget.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "Rigs/RigHierarchyController.h"
#include "RigVMCore/RigVMVariant.h"
#include "RigVMEditorAsset.h"
#include "ScopedTransaction.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SModularRigHierarchyTreeItem"

void SModularRigHierarchyTreeItem::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& OwnerTable, 
	const TSharedRef<FModularRigHierarchyTreeElement>& InRigTreeElement, 
	const TSharedPtr<SModularRigHierarchyTreeView>& InTreeView, 
	bool bPinned)
{
	WeakTreeView = InTreeView;
	WeakRigTreeElement = InRigTreeElement;
	Delegates = InTreeView->GetRigTreeDelegates();

	if (InRigTreeElement->GetKey().IsEmpty())
	{
		SMultiColumnTableRow<TSharedPtr<FModularRigHierarchyTreeElement>>::Construct(
			SMultiColumnTableRow<TSharedPtr<FModularRigHierarchyTreeElement>>::FArguments()
			.ShowSelection(false)
			.OnCanAcceptDrop(Delegates.OnCanAcceptDrop)
			.OnAcceptDrop(Delegates.OnAcceptDrop)
			, OwnerTable);
		return;
	}

	const FString& ModuleName = InRigTreeElement->GetModuleName().ToString();
	const FRigHierarchyModulePath ConnectorModulePath(ModuleName, InRigTreeElement->GetConnectorName());
	ConnectorKey = FRigElementKey(ConnectorModulePath.GetPathFName(), ERigElementType::Connector);

	SMultiColumnTableRow<TSharedPtr<FModularRigHierarchyTreeElement>>::Construct(
		SMultiColumnTableRow<TSharedPtr<FModularRigHierarchyTreeElement>>::FArguments()
		.OnDragDetected(Delegates.OnDragDetected)
		.OnCanAcceptDrop(Delegates.OnCanAcceptDrop)
		.OnAcceptDrop(Delegates.OnAcceptDrop)
		.ShowWires(true), OwnerTable);
}

bool SModularRigHierarchyTreeItem::OnConnectorTargetChanged(TArray<FRigElementKey> InTargets, const FRigElementKey InConnectorKey)
{
	FScopedTransaction Transaction(LOCTEXT("ModuleHierarchyResolveConnector", "Resolve Connector"));
	Delegates.HandleResolveConnector(InConnectorKey, InTargets);
	return false;
}

void SModularRigHierarchyTreeItem::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	// for now only allow enter
	// because it is important to keep the unique names per pose
	if (WeakRigTreeElement.IsValid() &&
		InCommitType == ETextCommit::OnEnter)
	{
		const FScopedTransaction RenameModuleTransaction(LOCTEXT("RenameModuleTransaction", "Rename Module"));

		WeakRigTreeElement.Pin()->SetModuleName(InText);
	}
}

bool SModularRigHierarchyTreeItem::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	if (WeakRigTreeElement.IsValid())
	{
		return WeakRigTreeElement.Pin()->VerifyModuleName(InText, OutErrorMessage);
	}
	
	return false;
}

FReply SModularRigHierarchyTreeItem::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// SMultiColumnTableRow doesn't support the bAllowPreselectedItemActivation slate arg,
	// hence handle reselecting selected elements here 
	const TSharedPtr<SModularRigHierarchyTreeView> TreeView = WeakTreeView.Pin();
	const TSharedPtr<FModularRigHierarchyTreeElement> Element = WeakRigTreeElement.Pin();
	if (TreeView.IsValid() &&
		Element.IsValid() &&
		MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton &&
		!MouseEvent.GetModifierKeys().AnyModifiersDown() &&
		IsSelected())
	{
		TreeView->ClearSelection();
		TreeView->SetItemSelection(Element, true, ESelectInfo::OnMouseClick);
		
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return Super::OnMouseButtonDown(MyGeometry, MouseEvent);
}

TSharedRef<SWidget> SModularRigHierarchyTreeItem::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SModularRigHierarchyTreeView::Column_ModuleName)
	{
		return CreateModuleNameWidget();
	}
	if (ColumnName == SModularRigHierarchyTreeView::Column_Warnings)
	{
		return CreateWarningsWidget();
	}
	if (ColumnName == SModularRigHierarchyTreeView::Column_Connector)
	{
		return CreateConnectorWidget();
	}
	if (ColumnName == SModularRigHierarchyTreeView::Column_ModuleClass)
	{
		return CreateModuleClassWidget();
	}
	if (ColumnName == SModularRigHierarchyTreeView::Column_ModuleTags)
	{
		return CreateModuleTagsWidget();
	}
	
	ensureMsgf(0, TEXT("Unhandled collumn name, creating null widget for Column %s"), *ColumnName.ToString());
	return SNullWidget::NullWidget;	
}

TSharedRef<SWidget> SModularRigHierarchyTreeItem::CreateModuleNameWidget()
{
	constexpr float TopPadding = 2.f;
		
	TSharedPtr<SInlineEditableTextBlock> InlineWidget;
		
	const TSharedRef<SWidget> Widget = 
		SNew(SHorizontalBox)
		.ToolTipText(GetModuleNameTooltip())

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(6, TopPadding, 0, 0)
		.VAlign(VAlign_Fill)
		[
			SNew(SExpanderArrow, SharedThis(this))
			.IndentAmount(12)
			.ShouldDrawWires(true)
		]

		+SHorizontalBox::Slot()
		.MaxWidth(25)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, TopPadding, 3.f, 0.f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MaxHeight(25)
			[
				SNew(SImage)
				.Image_Lambda([this]() -> const FSlateBrush*
				{
					if (WeakRigTreeElement.IsValid())
					{
						return WeakRigTreeElement.Pin()->GetIconBrush();
					}

					return nullptr;
				})
				.ColorAndOpacity_Lambda([this]()
				{
					if (WeakRigTreeElement.IsValid())
					{
						return WeakRigTreeElement.Pin()->GetIconColor();
					}

					return FSlateColor::UseForeground();
				})
				.DesiredSizeOverride(FVector2D(16.0, 16.0))
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0, TopPadding, 0, 0)
		[
			SAssignNew(InlineWidget, SInlineEditableTextBlock)
			.Text(this, &SModularRigHierarchyTreeItem::GetModuleName, true)
			.MaximumLength(NAME_SIZE-1)
			.OnVerifyTextChanged(this, &SModularRigHierarchyTreeItem::OnVerifyNameChanged)
			.OnTextCommitted(this, &SModularRigHierarchyTreeItem::OnNameCommitted)
			.MultiLine(false)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
			.ColorAndOpacity_Lambda([this]()
			{
				if(WeakRigTreeElement.IsValid())
				{
					return WeakRigTreeElement.Pin()->GetTextColor();
				}
				return FSlateColor::UseForeground();
			})
		];

	if (WeakRigTreeElement.IsValid())
	{
		WeakRigTreeElement.Pin()->OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	}

	return Widget;
}

TSharedRef<SWidget> SModularRigHierarchyTreeItem::CreateWarningsWidget()
{
	using namespace UE::ControlRigEditor;

	const TSharedRef<SWidget> Widget = SNew(SImage)
		.ToolTipText(GetModuleNameTooltip())
		.Visibility_Lambda([this]()
			{
				const TSharedPtr<FModularRigHierarchyTreeElement> TreeElement = WeakRigTreeElement.IsValid() ? WeakRigTreeElement.Pin() : nullptr;
				const TSharedPtr<FModularRigHierarchyConnectorWarning>& Warning = TreeElement.IsValid() ? TreeElement->GetWarning() : nullptr;

				const EVisibility Visiblity = Warning.IsValid() ? EVisibility::Visible : EVisibility::Hidden;

				return Visiblity;
			})
		.ToolTipText_Lambda([this]()
			{
				const TSharedPtr<FModularRigHierarchyTreeElement> TreeElement = WeakRigTreeElement.IsValid() ? WeakRigTreeElement.Pin() : nullptr;
				const TSharedPtr<FModularRigHierarchyConnectorWarning>& Warning = TreeElement.IsValid() ? TreeElement->GetWarning() : nullptr;

				const FText& WarningText = Warning.IsValid() ? Warning->GetTooltip() : FText::GetEmpty();

				return WarningText;
			})
		.Image_Lambda([this]()
			{
				const TSharedPtr<FModularRigHierarchyTreeElement> TreeElement = WeakRigTreeElement.IsValid() ? WeakRigTreeElement.Pin() : nullptr;
				const TSharedPtr<FModularRigHierarchyConnectorWarning>& Warning = TreeElement.IsValid() ? TreeElement->GetWarning() : nullptr;

				const FSlateBrush* Brush = Warning.IsValid() ? Warning->GetBrush() : nullptr;

				return Brush;
			})
		.DesiredSizeOverride(FVector2D(16, 16));

	return Widget;
}

TSharedRef<SWidget> SModularRigHierarchyTreeItem::CreateConnectorWidget()
{
	const bool bIsArrayConnector = [this]()
		{
			const UModularRig* ModularRig = Delegates.GetModularRig();
			const URigHierarchy* Hierarchy = ModularRig ? ModularRig->GetHierarchy() : nullptr;
			const FRigBaseElement* Element = Hierarchy ? Hierarchy->Find(ConnectorKey) : nullptr;

			if (const FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Element))
			{
				return ConnectorElement->IsArrayConnector();
			}

			return false;
		}();
		
	FRigHierarchyTreeDelegates RigTreeDelegates;
	RigTreeDelegates.OnGetHierarchy = FOnGetRigTreeHierarchy::CreateLambda(
		[WeakThis = AsWeak(), this]() -> const URigHierarchy*
		{
			if (WeakThis.IsValid())
			{
				const UModularRig* ModularRig = Delegates.GetModularRig();
				if (ModularRig)
				{
					return ModularRig->GetHierarchy();
				}
			}

			return nullptr;
		});

	const TSharedRef<SWidget> Widget = 
		SNew(SHorizontalBox)
		.ToolTipText(GetModuleNameTooltip())

		// Connector 
		+ SHorizontalBox::Slot()
		.FillContentWidth(0.f, 1.f)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SRigConnectorTargetWidget)
			.Clipping(EWidgetClipping::ClipToBoundsAlways)
			.ToolTipText(GetModuleNameTooltip())
			.Outer(const_cast<UModularRig*>(Delegates.GetModularRig()))
			.ConnectorKey(ConnectorKey)
			.IsArray(bIsArrayConnector)
			.Targets(GetTargetKeys())
			.OnSetTargetArray(FRigConnectorTargetWidget_SetTargetArray::CreateSP(this, &SModularRigHierarchyTreeItem::OnConnectorTargetChanged, ConnectorKey))
			.RigTreeDelegates(RigTreeDelegates)
		]

		// Reset button
		+SHorizontalBox::Slot()
		.Padding(4.f, 0.f)
		.FillContentWidth(1.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ResetConnectorButton, SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity_Lambda([this]()
				{
					return ResetConnectorButton.IsValid() && ResetConnectorButton->IsHovered()
						? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
						: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
				})
				.OnClicked_Lambda([this]()
				{
					Delegates.HandleDisconnectConnector(ConnectorKey);
					return FReply::Handled();
				})
				.ContentPadding(1.f)
				.ToolTipText(NSLOCTEXT("ControlRigModuleDetails", "Reset_Connector", "Reset Connector"))
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda( [this]()
					{
						return ResetConnectorButton.IsValid() && ResetConnectorButton->IsHovered()
						? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
						: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
					})
					.Image(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault").GetIcon())
				]
			]

			// Use button
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(UseSelectedButton, SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity_Lambda([this]()
				{
					return UseSelectedButton.IsValid() && UseSelectedButton->IsHovered()
						? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
						: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
				})
				.OnClicked_Lambda([this]()
				{
					if (const UModularRig* ModularRig = Delegates.GetModularRig())
					{
						const TArray<FRigElementKey>& Selected = ModularRig->GetHierarchy()->GetSelectedKeys();
						if (Selected.Num() > 0)
						{
							Delegates.HandleResolveConnector(ConnectorKey, Selected);
						}
					}
					return FReply::Handled();
				})
				.ContentPadding(1.f)
				.ToolTipText(NSLOCTEXT("ControlRigModuleDetails", "Use_Selected", "Use Selected"))
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda( [this]()
					{
						return UseSelectedButton.IsValid() && UseSelectedButton->IsHovered()
						? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
						: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
					})
					.Image(FAppStyle::GetBrush("Icons.CircleArrowLeft"))
				]
			]
		
			// Select in hierarchy button
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(SelectElementButton, SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity_Lambda([this]()
				{
					return SelectElementButton.IsValid() && SelectElementButton->IsHovered()
						? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
						: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
				})
				.OnClicked_Lambda([this]()
				{
					if (const UModularRig* ModularRig = Delegates.GetModularRig())
					{
						const FRigElementKeyRedirector& Redirector = ModularRig->GetElementKeyRedirector();
						if (const FRigElementKeyRedirector::FKeyArray* TargetKeys = Redirector.FindExternalKey(ConnectorKey))
						{
							bool bClearSelection = true;
							for(const FRigElementKey& TargetKey : *TargetKeys)
							{
								ModularRig->GetHierarchy()->GetController()->SelectElement(TargetKey, true, bClearSelection);
								bClearSelection = false;
							}
						}
					}
					return FReply::Handled();
				})
				.ContentPadding(1.f)
				.ToolTipText(NSLOCTEXT("ControlRigModuleDetails", "Select_Element", "Select Element"))
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda( [this]()
					{
						return SelectElementButton.IsValid() && SelectElementButton->IsHovered()
						? FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.8f))
						: FSlateColor(FLinearColor(1.f, 1.f, 1.f, 0.4f));
					})
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
		];

	return Widget;
}

TSharedRef<SWidget> SModularRigHierarchyTreeItem::CreateModuleClassWidget()
{
	return
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f, 0.f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &SModularRigHierarchyTreeItem::GetModuleAssetName)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		];
}

TSharedRef<SWidget> SModularRigHierarchyTreeItem::CreateModuleTagsWidget()
{
	const TScriptInterface<const IRigVMAssetInterface> AssetInterface = GetModuleRigVMAsset();
	if (AssetInterface &&
		!AssetInterface->GetAssetVariant().Tags.IsEmpty())
	{
		const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);
		for (const FRigVMTag& AssetTag : AssetInterface->GetAssetVariant().Tags)
		{
			HorizontalBox->AddSlot()
			.AutoWidth()
			.Padding(2.f, 0.f)
			[
				SNew(SBorder)
				.Padding(2.f)
				.BorderImage(FControlRigEditorStyle::Get().GetBrush(TEXT("ModularRig.Tree.RoundedBox")))
				.BorderBackgroundColor(AssetTag.Color.CopyWithNewOpacity(0.45f))
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.Padding(FMargin(0.f, 1.f))
					[
						SNew(STextBlock)
						.Text(FText::FromString(AssetTag.GetLabel()))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
						.ToolTipText(AssetTag.ToolTip)
					]
				]
			];
		}

		return HorizontalBox;
	}

	return SNullWidget::NullWidget;
}

FText SModularRigHierarchyTreeItem::GetModuleNameTooltip() const
{
	const FText Tooltip = [this]()
		{	
			constexpr bool bShortName = false;
			const FText FullName = GetModuleName(bShortName);

			const UModularRig* ModularRig = Delegates.GetModularRig();
			URigHierarchy* Hierarchy = ModularRig ? ModularRig->GetHierarchy() : nullptr;
			const FRigConnectorElement* ConnectorElement = Hierarchy ? Cast<FRigConnectorElement>(Hierarchy->Find(ConnectorKey)) : nullptr;
			if (ConnectorElement &&
				!ConnectorElement->Settings.Description.IsEmpty())
			{
				const FText& ConnectorDescription = FText::FromString(ConnectorElement->Settings.Description);
				return FText::Format(FText::FromString("{0} - {1}"), FullName, ConnectorDescription);
			}
			else
			{
				return FullName;
			}
		}();

	constexpr bool bShortName = true;
	const FText ShortName = GetModuleName(bShortName);
	if (Tooltip.EqualTo(ShortName))
	{
		return FText();
	}
	else
	{
		return Tooltip;
	}
}

TArray<FRigElementKey> SModularRigHierarchyTreeItem::GetTargetKeys() const
{
	TArray<FRigElementKey> Result;
	if (const UModularRig* ModularRig = Delegates.GetModularRig())
	{
		Result = ModularRig->GetModularRigModel().Connections.FindTargetsFromConnector(ConnectorKey);
	}
	return Result;
}

FText SModularRigHierarchyTreeItem::GetModuleName(bool bUseShortName) const
{
	if (WeakRigTreeElement.IsValid())
	{
		if (bUseShortName)
		{
			return (FText::FromName(WeakRigTreeElement.Pin()->GetShortModuleName()));
		}
		return (FText::FromName(WeakRigTreeElement.Pin()->GetModuleName()));
	}

	return FText::GetEmpty();
}

FText SModularRigHierarchyTreeItem::GetModuleAssetName() const
{
	const TScriptInterface<const IRigVMAssetInterface> AssetInterface = GetModuleRigVMAsset();
	const UObject* AssetObject = AssetInterface ? AssetInterface.GetObject() : nullptr;
	if (AssetObject)
	{
		return FText::FromString(AssetObject->GetName());
	}

	return FText::GetEmpty();
}

const TScriptInterface<const IRigVMEditorAssetInterface> SModularRigHierarchyTreeItem::GetModuleRigVMAsset() const
{
	const UModularRig* ModularRig = Delegates.GetModularRig();

	const FRigModuleInstance* Module = ModularRig && WeakRigTreeElement.IsValid() ? 
		ModularRig->FindModule(WeakRigTreeElement.Pin()->GetModuleName()) :
		nullptr;

	const UControlRig* ModuleRig = Module ? Module->GetRig() : nullptr;
	
	if (ModuleRig &&
		ModuleRig->GetClass()->ClassGeneratedBy &&
		ModuleRig->GetClass()->ClassGeneratedBy->Implements<URigVMEditorAssetInterface>())
	{
		return TScriptInterface<const IRigVMEditorAssetInterface>(ModuleRig->GetClass()->ClassGeneratedBy);
	}

	if (ModuleRig)
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> GeneratedByAsset = ModuleRig->GetGeneratedByAsset())
		{
			return TScriptInterface<const IRigVMEditorAssetInterface>(GeneratedByAsset->GetEditorOnlyData());
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
