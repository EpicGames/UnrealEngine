// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Editor/Hierarchy/ModularRigHierarchyTreeDelegates.h"
#include "Internationalization/Text.h"
#include "Rigs/RigHierarchyDefines.h"
#include "UObject/ScriptInterface.h"
#include "Widgets/Views/STableRow.h"

class IRigVMEditorAssetInterface;
class SRigHierarchySearchableTreeView;
struct FRigVMTag;

class SModularRigHierarchyTreeItem : 
	public SMultiColumnTableRow<TSharedPtr<FModularRigHierarchyTreeElement>>
{
	using Super = SMultiColumnTableRow<TSharedPtr<FModularRigHierarchyTreeElement>>;

	friend class SModularRigHierarchyTreeView;

public:

	void Construct(
		const FArguments& InArgs, 
		const TSharedRef<STableViewBase>& OwnerTable, 
		const TSharedRef<FModularRigHierarchyTreeElement>& InRigTreeElement, 
		const TSharedPtr<SModularRigHierarchyTreeView>& InTreeView, 
		bool bPinned);

	bool OnConnectorTargetChanged(TArray<FRigElementKey> InTargets, const FRigElementKey InConnectorKey);

	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:
	//~ Begin SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget interface

private:
	/** Creates the widget for the module column */
	TSharedRef<SWidget> CreateModuleNameWidget();

	/** Creates the widget for the warnings column */
	TSharedRef<SWidget> CreateWarningsWidget();

	/** Creates the widget for the connector column */
	TSharedRef<SWidget> CreateConnectorWidget();

	/** Creates the widget for the module class column */
	TSharedRef<SWidget> CreateModuleClassWidget();

	/** Creates the widget for the module tags column */
	TSharedRef<SWidget> CreateModuleTagsWidget();

	TArray<FRigElementKey> GetTargetKeys() const;

	/** Returns the name of the module displayed in this row */
	FText GetModuleName(bool bUseShortName) const;

	/** Returns the tooltip displayed when the module name is hovered */
	FText GetModuleNameTooltip() const;

	/** Returns the name of the module asset displayed in this row */
	FText GetModuleAssetName() const;

	/** Returns the rig vm asset of the module displayed in this row, or nullptr if no module blueprint can be found */
	const TScriptInterface<const IRigVMEditorAssetInterface> GetModuleRigVMAsset() const;

	/** The tree element being displayed in this widget */
	TWeakPtr<FModularRigHierarchyTreeElement> WeakRigTreeElement;
	
	/** The tree view that displays this widget */
	TWeakPtr<SModularRigHierarchyTreeView> WeakTreeView;
	
	FModularRigHierarchyTreeDelegates Delegates;
	TSharedPtr<SRigHierarchySearchableTreeView> ConnectorComboBox;
	TSharedPtr<SButton> ResetConnectorButton;
	TSharedPtr<SButton> UseSelectedButton;
	TSharedPtr<SButton> SelectElementButton;
	FRigElementKey ConnectorKey;
	TOptional<FModularRigResolveResult> ConnectorMatches;
};
