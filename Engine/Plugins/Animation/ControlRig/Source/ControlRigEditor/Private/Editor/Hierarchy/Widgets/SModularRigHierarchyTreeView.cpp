// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModularRigHierarchyTreeView.h"

#include "ControlRig.h"
#include "Editor/Hierarchy/DragDrop/ModularRigHierarchyElementDragDropOp.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyViewModel.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentState.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateStore.h"
#include "Editor/ControlRigEditor.h"
#include "Framework/Application/SlateApplication.h"
#include "ModularRig.h"
#include "ModularRigController.h"
#include "Settings/ControlRigSettings.h"

#define LOCTEXT_NAMESPACE "SModularRigHierarchyTreeView"

const FName SModularRigHierarchyTreeView::Column_Warnings = TEXT("Warnings");
const FName SModularRigHierarchyTreeView::Column_Connector = TEXT("Connector");
const FName SModularRigHierarchyTreeView::Column_ModuleName = TEXT("ModuleName");
const FName SModularRigHierarchyTreeView::Column_ModuleClass = TEXT("ModuleClass");
const FName SModularRigHierarchyTreeView::Column_ModuleTags = TEXT("ModuleTags");

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow to destruct deprecated OldSparseItemInfos
SModularRigHierarchyTreeView::~SModularRigHierarchyTreeView()
{
	using namespace UE::ControlRigEditor;

	// Retain the state so it gets restored when the tree is constructed anew
	if (FRigHierarchyTreePersistentStateStore::IsFeatureEnabled())
	{
		const UModularRig* ModularRig = WeakViewModel.IsValid() ? WeakViewModel.Pin()->GetModularRig() : nullptr;
		if (ModularRig)
		{
			FRigHierarchyTreePersistentStateStore::Get().StorePersistentState(*ModularRig, *this, ViewName);
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SModularRigHierarchyTreeView::Construct(const FArguments& InArgs, const TSharedRef<FModularRigHierarchyViewModel>& InViewModel, const FName& InViewName)
{
	WeakViewModel = InViewModel;
	ViewName = InViewName;

	Delegates = InArgs._RigTreeDelegates;
	bAutoScrollEnabled = InArgs._AutoScrollEnabled;
	FilterText = InArgs._FilterText;
	
	STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>::FArguments SuperArgs;
	SuperArgs.HeaderRow(InArgs._HeaderRow);
	SuperArgs.TreeItemsSource(&RootElements);
	SuperArgs.SelectionMode(ESelectionMode::Multi);
	SuperArgs.OnGenerateRow(this, &SModularRigHierarchyTreeView::MakeTableRowWidget, false);
	SuperArgs.OnGetChildren(this, &SModularRigHierarchyTreeView::HandleGetChildrenForTree);
	SuperArgs.OnSelectionChanged(this, &SModularRigHierarchyTreeView::OnSelectionChanged);
	SuperArgs.OnContextMenuOpening(Delegates.OnContextMenuOpening);
	SuperArgs.HighlightParentNodesForSelection(true);
	SuperArgs.AllowInvisibleItemSelection(true);  //without this we deselect everything when we filter or we collapse
	SuperArgs.OnMouseButtonClick(Delegates.OnMouseButtonClick);
	SuperArgs.OnMouseButtonDoubleClick(Delegates.OnMouseButtonDoubleClick);
	SuperArgs.OnSetExpansionRecursive(this, &SModularRigHierarchyTreeView::SetExpansionRecursive);

	SuperArgs.ShouldStackHierarchyHeaders_Lambda([]() -> bool {
		return UControlRigEditorSettings::Get()->bShowStackedHierarchy;
	});
	SuperArgs.OnGeneratePinnedRow(this, &SModularRigHierarchyTreeView::MakeTableRowWidget, true);
	SuperArgs.MaxPinnedItems_Lambda([]() -> int32
	{
		return FMath::Max<int32>(1, UControlRigEditorSettings::Get()->MaxStackSize);
	});

	STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>::Construct(SuperArgs);

	if (UModularRigController* ModularRigController = InViewModel->GetModularRigController())
	{
		ModularRigController->OnModified().AddSP(this, &SModularRigHierarchyTreeView::OnModularRigControllerModified);
	}

	if (const FControlRigAssetInterfacePtr ControlRigEditorAssetInterface = InViewModel->GetControlRigAssetInterface())
	{
		ControlRigEditorAssetInterface->OnModularRigCompiled().AddSP(this, &SModularRigHierarchyTreeView::OnPostCompileModularRigs);
	}
}

void SModularRigHierarchyTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STreeView<TSharedPtr<FModularRigHierarchyTreeElement, ESPMode::ThreadSafe>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FGeometry PaintGeometry = GetPaintSpaceGeometry();
	const FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();

	if(PaintGeometry.IsUnderLocation(MousePosition))
	{
		const FVector2D WidgetPosition = PaintGeometry.AbsoluteToLocal(MousePosition);

		static constexpr float SteadyMousePositionTolerance = 5.f;

		if(LastMousePosition.Equals(MousePosition, SteadyMousePositionTolerance))
		{
			TimeAtMousePosition += InDeltaTime;
		}
		else
		{
			LastMousePosition = MousePosition;
			TimeAtMousePosition = 0.0;
		}

		static constexpr float AutoScrollStartDuration = 0.5f; // in seconds
		static constexpr float AutoScrollDistance = 24.f; // in pixels
		static constexpr float AutoScrollSpeed = 150.f;

		if(TimeAtMousePosition > AutoScrollStartDuration && FSlateApplication::Get().IsDragDropping())
		{
			if((WidgetPosition.Y < AutoScrollDistance) || (WidgetPosition.Y > PaintGeometry.Size.Y - AutoScrollDistance))
			{
				if(bAutoScrollEnabled)
				{
					const bool bScrollUp = (WidgetPosition.Y < AutoScrollDistance);

					const float DeltaInSlateUnits = (bScrollUp ? -InDeltaTime : InDeltaTime) * AutoScrollSpeed; 
					ScrollBy(GetCachedGeometry(), DeltaInSlateUnits, EAllowOverscroll::No);
				}
			}
			else
			{
				const TSharedPtr<FModularRigHierarchyTreeElement>* Item = FindItemAtPosition(MousePosition);
				if(Item && Item->IsValid())
				{
					if(!IsItemExpanded(*Item))
					{
						SetItemExpansion(*Item, true);
					}
				}
			}
		}
	}

	if (bRequestRenameSelected)
	{
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, float) {
			TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedItems = GetSelectedItems();
			if (SelectedItems.Num() == 1)
			{
				SelectedItems[0]->RequestRename();
			}
			return EActiveTimerReturnType::Stop;
		}));
		bRequestRenameSelected = false;
	}

	// Show the schematic viewport while drag-dropping
	const TSharedPtr<FDragDropOperation> DragDropOp = FSlateApplication::Get().GetDragDroppingContent();
	const bool bIsDragDropping = DragDropOp.IsValid() && DragDropOp->IsOfType<FModularRigHierarchyElementDragDropOp>();
	if (bIsDragDropping)
	{
		constexpr bool bForceShowSchematicViewport = true;
		ShowSchematicViewportOverride.OverrideDuringDragDrop(bForceShowSchematicViewport);
	}
}

TSharedPtr<FModularRigHierarchyTreeElement> SModularRigHierarchyTreeView::FindElement(const FString& InElementKey)
{
	for (TSharedPtr<FModularRigHierarchyTreeElement> Root : RootElements)
	{
		if (TSharedPtr<FModularRigHierarchyTreeElement> Found = FindElement(InElementKey, Root))
		{
			return Found;
		}
	}

	return TSharedPtr<FModularRigHierarchyTreeElement>();
}

TSharedPtr<FModularRigHierarchyTreeElement> SModularRigHierarchyTreeView::FindElement(const FString& InElementKey, TSharedPtr<FModularRigHierarchyTreeElement> CurrentItem)
{
	if (CurrentItem->GetKey() == InElementKey)
	{
		return CurrentItem;
	}

	for (int32 ChildIndex = 0; ChildIndex < CurrentItem->GetChildren().Num(); ++ChildIndex)
	{
		TSharedPtr<FModularRigHierarchyTreeElement> Found = FindElement(InElementKey, CurrentItem->GetChildren()[ChildIndex]);
		if (Found.IsValid())
		{
			return Found;
		}
	}

	return TSharedPtr<FModularRigHierarchyTreeElement>();
}

bool SModularRigHierarchyTreeView::AddElement(FString InKey, FString InParentKey, bool bApplyFilterText)
{
	if (!WeakViewModel.IsValid())
	{
		return false;
	}
	const TSharedRef<FModularRigHierarchyViewModel> ViewModel = WeakViewModel.Pin().ToSharedRef();

	using namespace UE::ControlRigEditor;

	if(ElementMap.Contains(InKey))
	{
		return false;
	}

	if (!InKey.IsEmpty())
	{
		const FString ModulePath = InKey;

		const UControlRigEditorSettings* Settings = GetDefault<UControlRigEditorSettings>();
		const bool bModuleClassColumnVisible = !Settings->ModularRigHierarchyHiddenColums.Contains(Column_ModuleClass);

		// Don't filter by module class if the related column isn't visible
		bool bFilteredOutModuleClass = !bModuleClassColumnVisible;
		bool bFilteredOutConnector = false;

		const FString FilterTextString = FilterText.Get().ToString();
		if(!FilterTextString.IsEmpty())
		{
			FString KeyStringToSearch = InKey;
			FRigHierarchyModulePath(KeyStringToSearch).Split(nullptr, &KeyStringToSearch);
			
			if(!KeyStringToSearch.Contains(FilterTextString, ESearchCase::IgnoreCase))
			{
				bFilteredOutConnector |= true;
			}

			if (bModuleClassColumnVisible)
			{
				const FControlRigAssetSoftReference SoftModuleSource = ViewModel->GetModuleAssetReference(*ModulePath);
				if (!SoftModuleSource.IsValid() &&
					!SoftModuleSource.GetPathName().Contains(FilterTextString, ESearchCase::IgnoreCase))
				{
					bFilteredOutModuleClass |= true;
				}
			}
		}

		TArray<FRigHierarchyModulePath> FilteredConnectors;
		if (const UModularRig* ModularRig = Delegates.GetModularRig())
		{
			const FModularRigModel& Model = ModularRig->GetModularRigModel();
			
			if (const FRigModuleInstance* Module = ModularRig->FindModule(*ModulePath))
			{
				if (const UControlRig* ModuleRig = Module->GetRig())
				{
					const TArray<FRigModuleConnector>& Connectors = ModuleRig->GetAssetReference().GetRigModuleSettings().ExposedConnectors;

					for (const FRigModuleConnector& Connector : Connectors)
					{
						if (Connector.IsPrimary())
						{
							continue;
						}

						const FRigHierarchyModulePath Key(ModulePath, Connector.Name);
						bool bShouldFilterByConnectorType = true;

						if(!FilterTextString.IsEmpty())
						{
							const bool bMatchesFilter = Connector.Name.Contains(FilterTextString, ESearchCase::IgnoreCase);
							if(bFilteredOutConnector)
							{
								if(!bMatchesFilter)
								{
									continue;
								}
							}
							bShouldFilterByConnectorType = !bMatchesFilter;
						}

						if(bShouldFilterByConnectorType)
						{
							if(Delegates.ShouldAlwaysShowConnector(Key.GetPathFName()))
							{
								bShouldFilterByConnectorType = false;
							}
						}

						if(bShouldFilterByConnectorType)
						{
							const bool bIsConnected = Model.Connections.HasConnection(FRigElementKey(Key.GetPathFName(), ERigElementType::Connector));

							const bool bShowUnresolvedConnectors = Settings->HasAnyModularRigHierarchyConnectorVisibilityFlags(EModularRigHierarchyEditorConnectorVisibilityFlags::ShowUnresolvedConnectors);
							const bool bShowOptionalConnectors = Settings->HasAnyModularRigHierarchyConnectorVisibilityFlags(EModularRigHierarchyEditorConnectorVisibilityFlags::ShowOptionalConnectors);
							const bool bShowSecondaryConnectors = Settings->HasAnyModularRigHierarchyConnectorVisibilityFlags(EModularRigHierarchyEditorConnectorVisibilityFlags::ShowSecondaryConnectors);

							if (bIsConnected || !bShowUnresolvedConnectors)
							{
								if (Connector.IsOptional())
								{
									if (!bShowOptionalConnectors)
									{
										continue;
									}
								}
								else if (Connector.IsSecondary())
								{
									if (!bShowSecondaryConnectors)
									{
										continue;
									}
								}
							}
						}

						FilteredConnectors.Add(Key);
					}
				}
			}
		}
		
		if (bFilteredOutConnector && 
			bFilteredOutModuleClass &&
			bApplyFilterText && 
			FilteredConnectors.IsEmpty())
		{
			return false;
		}
		
		constexpr bool bIsParentPrimary = true;
		const TSharedRef<FModularRigHierarchyTreeElement> NewItem = MakeShared<FModularRigHierarchyTreeElement>(ViewModel, InKey, bIsParentPrimary);

		ElementMap.Add(InKey, NewItem);
		if (!InParentKey.IsEmpty())
		{
			ParentMap.Add(InKey, InParentKey);

			TSharedPtr<FModularRigHierarchyTreeElement>* FoundItem = ElementMap.Find(InParentKey);
			check(FoundItem);
			FoundItem->Get()->AddChild(NewItem);
		}
		else
		{
			RootElements.Add(NewItem);
		}

		if (!FRigHierarchyTreePersistentStateStore::IsFeatureEnabled())
		{
			SetItemExpansion(NewItem, true);
		}

		for (const FString& Key : FilteredConnectors)
		{
			constexpr bool bIsChildPrimary = false;

			TSharedPtr<FModularRigHierarchyTreeElement> ConnectorItem = MakeShared<FModularRigHierarchyTreeElement>(ViewModel, Key, bIsChildPrimary);
			NewItem->AddChild(ConnectorItem);
			ElementMap.Add(Key, ConnectorItem);
			ParentMap.Add(Key, InKey);
		}
	}

	return true;
}

bool SModularRigHierarchyTreeView::AddElement(const FRigModuleInstance* InElement, bool bApplyFilterText)
{
	check(InElement);
	
	if (ElementMap.Contains(InElement->Name.ToString()))
	{
		return false;
	}

	const UModularRig* ModularRig = Delegates.GetModularRig();

	if(!AddElement(InElement->Name.ToString(), FString(), bApplyFilterText))
	{
		return false;
	}

	if (ElementMap.Contains(InElement->Name.ToString()))
	{
		if(ModularRig)
		{
			const FName ParentModuleName = ModularRig->GetParentModuleName(InElement->Name);
			if (!ParentModuleName.IsNone())
			{
				if(const FRigModuleInstance* ParentElement = ModularRig->FindModule(ParentModuleName))
				{
					AddElement(ParentElement, false);

					if(ElementMap.Contains(ParentModuleName.ToString()))
					{
						ReparentElement(InElement->Name.ToString(), ParentModuleName.ToString());
					}
				}
			}
		}
	}

	return true;
}

void SModularRigHierarchyTreeView::AddSpacerElement()
{
	AddElement(FString(), FString());
}

bool SModularRigHierarchyTreeView::ReparentElement(const FString InKey, const FString InParentKey)
{
	if (InKey.IsEmpty() || InKey == InParentKey)
	{
		return false;
	}

	TSharedPtr<FModularRigHierarchyTreeElement>* FoundItem = ElementMap.Find(InKey);
	if (FoundItem == nullptr)
	{
		return false;
	}

	if (const FString* ExistingParentKey = ParentMap.Find(InKey))
	{
		if (*ExistingParentKey == InParentKey)
		{
			return false;
		}

		if (TSharedPtr<FModularRigHierarchyTreeElement>* ExistingParent = ElementMap.Find(*ExistingParentKey))
		{
			(*ExistingParent)->RemoveChild(*FoundItem);
		}

		ParentMap.Remove(InKey);
	}
	else
	{
		if (InParentKey.IsEmpty())
		{
			return false;
		}

		RootElements.Remove(*FoundItem);
	}

	if (!InParentKey.IsEmpty())
	{
		ParentMap.Add(InKey, InParentKey);

		TSharedPtr<FModularRigHierarchyTreeElement>* FoundParent = ElementMap.Find(InParentKey);
		if(FoundParent)
		{
			(*FoundParent)->AddChild(*FoundItem);
		}
	}
	else
	{
		RootElements.Add(*FoundItem);
	}

	return true;
}

void SModularRigHierarchyTreeView::RefreshTreeView(const bool bRebuildContent, const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored)
{
	using namespace UE::ControlRigEditor;

	if (!WeakViewModel.IsValid())
	{
		OnStateRestored.ExecuteIfBound();
		return;
	}
	const FModularRigHierarchyScopedSuspendDetailsPanelRefresh SuspendDetailsPanelRefresh(WeakViewModel.Pin().ToSharedRef());

	if (FRigHierarchyTreePersistentStateStore::IsFeatureEnabled())
	{
		// New with persistent state store
		const UControlRig* ControlRig = Delegates.GetModularRig();
		if (!ControlRig)
		{
			OnStateRestored.ExecuteIfBound();
			return;
		}

		bRebuildContentRequested |= bRebuildContent;

		const TSharedRef<STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>> ThisAsTreeView =
			StaticCastSharedRef<STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>>(SharedThis(this));

		const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakViewModel.IsValid() ? WeakViewModel.Pin()->GetControlRigEditor() : nullptr;

		FRigHierarchyTreePersistentStateStore::Get().RequestTreeRefresh(
			*ControlRig,
			ThisAsTreeView,
			ViewName,
			ControlRigEditor,
			FRigHierarchyTreePersistentStateRebuildItemsRequested::CreateLambda(
				[WeakThis = AsWeak(), this]()
				{
					if (!WeakThis.IsValid())
					{
						return;
					}
	
					if (bRebuildContentRequested)
					{
						RootElements.Reset();
						ElementMap.Reset();
						ParentMap.Reset();

						const UModularRig* ModularRig = Delegates.GetModularRig();
						if (ModularRig)
						{
							ModularRig->ForEachModule([&](const FRigModuleInstance* Element)
								{
									constexpr bool bApplyFilterText = true;
									AddElement(Element, bApplyFilterText);
									return true;
								});

							if (RootElements.Num() > 0)
							{
								AddSpacerElement();
							}
						}

						bRebuildContentRequested = false;
					}
				}),
			OnStateRestored
		);

		return;
	}

	// Old without persistent state store
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	TMap<FString, bool> ExpansionState;
	TArray<FName> Selection;

	if(bRebuildContent)
	{
		for (TPair<FString, TSharedPtr<FModularRigHierarchyTreeElement>> Pair : ElementMap)
		{
			ExpansionState.FindOrAdd(Pair.Key) = IsItemExpanded(Pair.Value);
		}

		// internally save expansion states before rebuilding the tree, so the states can be restored later
		SaveAndClearSparseItemInfos();

		RootElements.Reset();
		ElementMap.Reset();
		ParentMap.Reset();

		Selection = GetSelectedModuleNames();
	}

	if(bRebuildContent)
	{
		const UModularRig* ModularRig = Delegates.GetModularRig();
		if(ModularRig)
		{
			ModularRig->ForEachModule([&](const FRigModuleInstance* Element)
			{
				AddElement(Element, true);
				return true;
			});

			ModularRig->GetModularRigModel().ForEachModule([&](const FRigModuleReference* Reference)
			{
				if (!ModularRig->FindModule(Reference->Name))
				{
					AddElement(Reference->Name.ToString());
				}
				return true;
			});

			// expand all elements upon the initial construction of the tree
			if (ExpansionState.Num() < ElementMap.Num())
			{
				for (const TPair<FString, TSharedPtr<FModularRigHierarchyTreeElement>>& Element : ElementMap)
				{
					if (!ExpansionState.Contains(Element.Key))
					{
						SetItemExpansion(Element.Value, true);
					}
				}
			}

			for (const auto& Pair : ElementMap)
			{
				RestoreSparseItemInfos(Pair.Value);
			}

			if (RootElements.Num() > 0)
			{
				AddSpacerElement();
			}
		}
	}
	else
	{
		if (RootElements.Num()> 0)
		{
			// elements may be added at the end of the list after a spacer element
			// we need to remove the spacer element and re-add it at the end
			RootElements.RemoveAll([](TSharedPtr<FModularRigHierarchyTreeElement> InElement)
			{
				return InElement->GetKey() == FString();
			});
			AddSpacerElement();
		}
	}

	RequestTreeRefresh();
	{
		TGuardValue<bool> Guard(Delegates.bSuspendSelectionDelegate, true);
		ClearSelection();

		if(!Selection.IsEmpty())
		{
			TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedElements;
			for(const FName& SelectedModuleName : Selection)
			{
				if(const TSharedPtr<FModularRigHierarchyTreeElement> ElementToSelect = FindElement(SelectedModuleName.ToString()))
				{
					SelectedElements.Add(ElementToSelect);
				}
			}
			if(!SelectedElements.IsEmpty())
			{
				SetSelection(SelectedElements, ESelectInfo::OnNavigation);
			}
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedRef<ITableRow> SModularRigHierarchyTreeView::MakeTableRowWidget(TSharedPtr<FModularRigHierarchyTreeElement> InItem,
	const TSharedRef<STableViewBase>& OwnerTable, bool bPinned)
{
	return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this), bPinned);
}

void SModularRigHierarchyTreeView::HandleGetChildrenForTree(TSharedPtr<FModularRigHierarchyTreeElement> InItem,
	TArray<TSharedPtr<FModularRigHierarchyTreeElement>>& OutChildren)
{
	OutChildren = InItem.Get()->GetChildren();
}

TArray<FName> SModularRigHierarchyTreeView::GetSelectedModuleNames() const
{
	TArray<FName> ModuleNames;
	TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedElements = GetSelectedItems();
	for(const TSharedPtr<FModularRigHierarchyTreeElement>& SelectedElement : SelectedElements)
	{
		if (SelectedElement.IsValid())
		{
			ModuleNames.AddUnique(SelectedElement->GetModuleName());
		}
	}
	return ModuleNames;
}

void SModularRigHierarchyTreeView::SetSelection(const TArray<TSharedPtr<FModularRigHierarchyTreeElement>>& InSelection, const ESelectInfo::Type SelectInfo)
{
	ClearSelection();
	SetItemSelection(InSelection, true, SelectInfo);
}

const TSharedPtr<FModularRigHierarchyTreeElement>* SModularRigHierarchyTreeView::FindItemAtPosition(FVector2D InScreenSpacePosition) const
{
	if (ItemsPanel.IsValid() && SListView<TSharedPtr<FModularRigHierarchyTreeElement>>::HasValidItemsSource())
	{
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		const int32 Index = FindChildUnderPosition(ArrangedChildren, InScreenSpacePosition);
		if (ArrangedChildren.IsValidIndex(Index))
		{
			TSharedRef<SModularRigHierarchyTreeItem> ItemWidget = StaticCastSharedRef<SModularRigHierarchyTreeItem>(ArrangedChildren[Index].Widget);
			if (ItemWidget->WeakRigTreeElement.IsValid())
			{
				const FString& Key = ItemWidget->WeakRigTreeElement.Pin()->GetKey();
				const TSharedPtr<FModularRigHierarchyTreeElement>* ResultPtr = SListView<TSharedPtr<FModularRigHierarchyTreeElement>>::GetItems().FindByPredicate([Key](const TSharedPtr<FModularRigHierarchyTreeElement>& Item) -> bool
					{
						return Item->GetKey() == Key;
					});

				if (ResultPtr)
				{
					return ResultPtr;
				}
			}
		}
	}
	return nullptr;
}

void SModularRigHierarchyTreeView::OnSelectionChanged(TSharedPtr<FModularRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct ||
		bIsPerformingSelection ||
		!WeakViewModel.IsValid())
	{
		return;
	}

	const TGuardValue<bool> GuardSelection(bIsPerformingSelection, true);

	ClearHighlightedItems();

	const TArray<FName> NewSelection = GetSelectedModuleNames();
	WeakViewModel.Pin()->SelectModules(NewSelection);
}

void SModularRigHierarchyTreeView::OnModularRigControllerModified(EModularRigNotification Notif, const FRigModuleReference* Module)
{
	switch (Notif)
	{
	case EModularRigNotification::ModuleSelected:
	case EModularRigNotification::ModuleDeselected:
	{
		if (!bIsPerformingSelection &&
			WeakViewModel.IsValid())
		{
			const TGuardValue<bool> GuardSelection(bIsPerformingSelection, true);

			const TArray<FName> SelectedModuleNames = WeakViewModel.Pin()->GetSelection();
			TArray<TSharedPtr<FModularRigHierarchyTreeElement>> NewSelection;
			for (const FName& SelectedModuleName : SelectedModuleNames)
			{
				if (const TSharedPtr<FModularRigHierarchyTreeElement> ModuleTreeElement = FindElement(SelectedModuleName.ToString()))
				{
					NewSelection.Add(ModuleTreeElement);
				}
			}

			SetSelection(NewSelection, ESelectInfo::OnNavigation);
		}
		break;
	}
	case EModularRigNotification::ModuleAdded:
	case EModularRigNotification::ModuleRenamed:
	case EModularRigNotification::ModuleRemoved:
	case EModularRigNotification::ModuleReparented:
	case EModularRigNotification::ModuleReordered:
	case EModularRigNotification::ConnectionChanged:
	case EModularRigNotification::ModuleConfigValueChanged:
	case EModularRigNotification::ModuleShortNameChanged:
	case EModularRigNotification::ModuleClassChanged:
	{
		RefreshTreeView();

		break;
	}
	default:
	{
		break;
	}
	}
}

void SModularRigHierarchyTreeView::OnPostCompileModularRigs(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	RefreshTreeView();

	const TSharedPtr<FModularRigHierarchyViewModel> ViewModel = WeakViewModel.IsValid() ? WeakViewModel.Pin() : nullptr;
	if (ViewModel.IsValid())
	{
		TArray<TSharedPtr<FModularRigHierarchyTreeElement>> SelectedElements;
		Algo::Transform(ViewModel->GetSelection(), SelectedElements, [this](const FName& ModuleName)
			{
				return FindElement(ModuleName.ToString());
			});

		SetSelection(SelectedElements, ESelectInfo::OnNavigation);
	}
}

void SModularRigHierarchyTreeView::SetExpansionRecursive(TSharedPtr<FModularRigHierarchyTreeElement> InElement, bool bShouldBeExpanded)
{
	SetItemExpansion(InElement, bShouldBeExpanded);

	for (int32 ChildIndex = 0; ChildIndex < InElement->GetChildren().Num(); ++ChildIndex)
	{
		SetExpansionRecursive(InElement->GetChildren()[ChildIndex], bShouldBeExpanded);
	}
}

#undef LOCTEXT_NAMESPACE
