// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/ControlRigShowSchematicViewportOverride.h"
#include "Editor/Hierarchy/ModularRigHierarchyTreeDelegates.h"
#include "Editor/Hierarchy/Widgets/SModularRigHierarchyTreeItem.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchySearchableTreeView.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateDelegates.h"
#include "ModularRig.h"
#include "Widgets/Views/STreeView.h"

class IRigVMEditorAssetInterface;
class SRigHierarchyTreeView;
class SRigHierarchyItem;
class FRigHierarchyTreeElement;
template <typename InInterfaceType> class TScriptInterface;

using FRigVMEditorAssetInterfacePtr = TScriptInterface<IRigVMEditorAssetInterface>;


namespace UE::ControlRigEditor { class FModularRigHierarchyViewModel; }

class SModularRigHierarchyTreeView : public STreeView<TSharedPtr<FModularRigHierarchyTreeElement>>
{
	using FControlRigShowSchematicViewportOverride = UE::ControlRigEditor::FControlRigShowSchematicViewportOverride;
	using FModularRigHierarchyViewModel = UE::ControlRigEditor::FModularRigHierarchyViewModel;
	using FOnRigHierarchyTreePersistentStateRestored = UE::ControlRigEditor::FOnRigHierarchyTreePersistentStateRestored;

public:

	static const FName Column_ModuleName;
	static const FName Column_Warnings;
	static const FName Column_Connector;
	static const FName Column_ModuleClass;
	static const FName Column_ModuleTags;

	SLATE_BEGIN_ARGS(SModularRigHierarchyTreeView)
		: _AutoScrollEnabled(false)
		, _FilterText()
	{}
		SLATE_ARGUMENT( TSharedPtr<SHeaderRow>, HeaderRow )
		SLATE_ARGUMENT(FModularRigHierarchyTreeDelegates, RigTreeDelegates)
		SLATE_ARGUMENT(bool, AutoScrollEnabled)
		SLATE_ATTRIBUTE(FText, FilterText)
	SLATE_END_ARGS()

	virtual ~SModularRigHierarchyTreeView();

	/** 
	 * Constructs this widget 
	 * 
	 * @param InArgs			Slate Arguments
	 * @param ViewModel			The model for the Modular Rig Hierarchy View this tree resides in 
	 * @param InViewName		The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FModularRigHierarchyViewModel>& ViewModel, const FName& InViewName);

	/** Performs auto scroll */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	bool bRequestRenameSelected = false;

	/** Save a snapshot of the internal map that tracks item expansion before tree reconstruction */
	UE_DEPRECATED(5.8, "Should not be used anymore. The tree uses the RigHierarchyTreePersistentStateStore to retain its state")
	void SaveAndClearSparseItemInfos()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Only save the info if there is something to save (do not overwrite info with an empty map)
		if (!SparseItemInfos.IsEmpty())
		{
			OldSparseItemInfos = SparseItemInfos;
		}
		ClearExpandedItems();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Restore the expansion infos map from the saved snapshot after tree reconstruction */
	UE_DEPRECATED(5.8, "Should not be used anymore. The tree uses the RigHierarchyTreePersistentStateStore to retain its state")
	void RestoreSparseItemInfos(TSharedPtr<FModularRigHierarchyTreeElement> ItemPtr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (const auto& Pair : OldSparseItemInfos)
		{
			if (Pair.Key->GetKey() == ItemPtr->GetKey())
			{
				// the SparseItemInfos now reference the new element, but keep the same expansion state
				SparseItemInfos.Add(ItemPtr, Pair.Value);
				break;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TSharedPtr<FModularRigHierarchyTreeElement> FindElement(const FString& InElementKey);
	static TSharedPtr<FModularRigHierarchyTreeElement> FindElement(const FString& InElementKey, TSharedPtr<FModularRigHierarchyTreeElement> CurrentItem);
	bool AddElement(FString InKey, FString InParentKey = FString(), bool bApplyFilterText = true);
	bool AddElement(const FRigModuleInstance* InElement, bool bApplyFilterText);
	void AddSpacerElement();
	bool ReparentElement(const FString InKey, const FString InParentKey);
	void RefreshTreeView(const bool bRebuildContent = true, const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored = FOnRigHierarchyTreePersistentStateRestored());
	TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FModularRigHierarchyTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bPinned);
	void HandleGetChildrenForTree(TSharedPtr<FModularRigHierarchyTreeElement> InItem, TArray<TSharedPtr<FModularRigHierarchyTreeElement>>& OutChildren);

	TArray<FName> GetSelectedModuleNames() const;
	void SetSelection(const TArray<TSharedPtr<FModularRigHierarchyTreeElement>>& InSelection, const ESelectInfo::Type SelectInfo);
	const TArray<TSharedPtr<FModularRigHierarchyTreeElement>>& GetRootElements() const { return RootElements; }
	FModularRigHierarchyTreeDelegates& GetRigTreeDelegates() { return Delegates; }

	/** Given a position, return the item under that position. If nothing is there, return null. */
	const TSharedPtr<FModularRigHierarchyTreeElement>* FindItemAtPosition(FVector2D InScreenSpacePosition) const;

private:
	/** Called when the tree selection changed */
	void OnSelectionChanged(TSharedPtr<FModularRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo);

	/** Called when the modular rig controller was modified */
	void OnModularRigControllerModified(EModularRigNotification Notif, const FRigModuleReference* Module);

	/** Called after modular rigs were compiled */
	void OnPostCompileModularRigs(FRigVMEditorAssetInterfacePtr InBlueprint);

	/** Called when the user shift clicks the expand button, so children are expanded/collapsed along with their parent */
	void SetExpansionRecursive(TSharedPtr<FModularRigHierarchyTreeElement> InItem, bool bShouldBeExpanded);

	/** A temporary snapshot of the SparseItemInfos in STreeView, used during RefreshTreeView() */
	UE_DEPRECATED(5.8, "Use RigHierarchyTreePersistentStateStore to retain the tree state")
	TSparseItemMap OldSparseItemInfos;

	/** View model for the Modular Rig Hierchy view that draws this tree */
	TWeakPtr<FModularRigHierarchyViewModel> WeakViewModel;

	/** Backing array for tree view */
	TArray<TSharedPtr<FModularRigHierarchyTreeElement>> RootElements;
	
	/** A map for looking up items based on their key */
	TMap<FString, TSharedPtr<FModularRigHierarchyTreeElement>> ElementMap;

	/** A map for looking up a parent based on their key */
	TMap<FString, FString> ParentMap;

	FModularRigHierarchyTreeDelegates Delegates;

	bool bAutoScrollEnabled = false;
	FVector2D LastMousePosition = FVector2D::ZeroVector;
	double TimeAtMousePosition = 0.0;

	TAttribute<FText> FilterText;

	/** Overrides the per project per user schematic viewport visibility */
	FControlRigShowSchematicViewportOverride ShowSchematicViewportOverride;

	/** True while this treeview is selecting */
	bool bIsPerformingSelection = false;

	/** True when rebuild content is requested but not yet carried out */
	bool bRebuildContentRequested = false;

	/** The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor */
	FName ViewName;

	friend class SModularRigHierarchy;
};

