// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigHierarchyItem.h"

#include "ControlRig.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTagWidget.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SRigHierarchyItem"

void SRigHierarchyItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FRigHierarchyTreeElement> InRigTreeElement, TSharedPtr<SRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned)
{
	using namespace UE::ControlRigEditor;

	WeakRigTreeElement = InRigTreeElement;
	Delegates = InTreeView->GetRigTreeDelegates();
	FRigHierarchyTreeDisplaySettings DisplaySettings = Delegates.GetDisplaySettings();

	if (!InRigTreeElement->Key.IsValid())
	{
		STableRow<TSharedPtr<FRigHierarchyTreeElement>>::Construct(
			STableRow<TSharedPtr<FRigHierarchyTreeElement>>::FArguments()
			.ShowSelection(false)
			.OnCanAcceptDrop(Delegates.OnCanAcceptDrop)
			.OnAcceptDrop(Delegates.OnAcceptDrop)
			.Content()
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(200.f)
				[
					SNew(SSpacer)
				]
			], OwnerTable);

		return;
	}

	TSharedPtr< SInlineEditableTextBlock > InlineWidget;
	TSharedPtr< SHorizontalBox > HorizontalBox;

	STableRow<TSharedPtr<FRigHierarchyTreeElement>>::Construct(
		STableRow<TSharedPtr<FRigHierarchyTreeElement>>::FArguments()
		.Padding(FMargin(0, 1, 1, 1))
		.OnDragDetected(Delegates.OnDragDetected)
		.OnCanAcceptDrop(Delegates.OnCanAcceptDrop)
		.OnAcceptDrop(Delegates.OnAcceptDrop)
		.ShowWires(true)
		.Content()
		[
			SAssignNew(HorizontalBox, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.MaxWidth(18)
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this]() -> const FSlateBrush*
					{
						if (WeakRigTreeElement.IsValid())
						{
							return WeakRigTreeElement.Pin()->IconBrush;
						}
						return nullptr;
					})
				.ColorAndOpacity_Lambda([this]()
					{
						if (WeakRigTreeElement.IsValid())
						{
							return WeakRigTreeElement.Pin()->GetIconColor();
						}
						return FSlateColor::UseForeground();
					})
				.DesiredSizeOverride(FVector2D(16, 16))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
				.Text(this, &SRigHierarchyItem::GetNameForUI)
				.ToolTipText(this, &SRigHierarchyItem::GetItemTooltip)
				.OnVerifyTextChanged(this, &SRigHierarchyItem::OnVerifyNameChanged)
				.OnTextCommitted(this, &SRigHierarchyItem::OnNameCommitted)
				.MultiLine(false)
				.ColorAndOpacity_Lambda([this]()
					{
						if (WeakRigTreeElement.IsValid())
						{
							return WeakRigTreeElement.Pin()->GetTextColor();
						}
						return FSlateColor::UseForeground();
					})
			]
		], OwnerTable);

	const bool bHasAnyTags = !InRigTreeElement->ConnectorTags.IsEmpty() || !InRigTreeElement->ConnectorResolveWarningTags.IsEmpty();
	if (bHasAnyTags)
	{
		HorizontalBox->AddSlot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			];

		// Show invalid connenctors first so they're less likely to get hidden due to a lack of space
		for (const FRigHierarchyConnectorUnresolvedWarningTag& ConnectorResolveWarningTag : InRigTreeElement->ConnectorResolveWarningTags)
		{
			SRigHierarchyTagWidget::FArguments Args;
			if (!ConnectorResolveWarningTag.MakeTagWidgetArgs(Args))
			{
				continue;
			}

			const TSharedRef<SRigHierarchyTagWidget> TagWidget = SArgumentNew(Args, SRigHierarchyTagWidget);

			HorizontalBox->AddSlot()
				.AutoWidth()
				[
					TagWidget
				];
		}

		for (const FRigHierarchyValidConnectorTag& ConnectorTag : InRigTreeElement->ConnectorTags)
		{
			SRigHierarchyTagWidget::FArguments Args;
			if (!ConnectorTag.MakeTagWidgetArgs(Args))
			{
				continue;
			}

			const TSharedRef<SRigHierarchyTagWidget> TagWidget = SArgumentNew(Args, SRigHierarchyTagWidget);

			const bool bCanDragDropTag = ConnectorTag.GetTagDisplayMode() == ERigHierarchyConnectorTagDisplayMode::Individual;
			if (bCanDragDropTag)
			{
				TagWidget->OnElementKeyDragDetected().BindSP(InTreeView.Get(), &SRigHierarchyTreeView::OnElementKeyTagDragDetected);
			}

			HorizontalBox->AddSlot()
				.Padding(4.f, 0.f, 0.f, 0.f)
				.AutoWidth()
				[
					TagWidget
				];
		}
	}

	InRigTreeElement->OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
}

FText SRigHierarchyItem::GetNameForUI() const
{
	return GetName(Delegates.GetDisplaySettings().NameDisplayMode);
}

FText SRigHierarchyItem::GetName(EElementNameDisplayMode InNameDisplayMode) const
{
	const TSharedPtr<FRigHierarchyTreeElement> RigTreeElement = WeakRigTreeElement.IsValid() ? WeakRigTreeElement.Pin() : nullptr;
	if (!ensureMsgf(RigTreeElement.IsValid(), TEXT("Unexpected view without model")))
	{
		return LOCTEXT("DestroyedControlLabel", "Destroyed Control");
	}

	if (RigTreeElement->bIsTransient)
	{
		return LOCTEXT("TemporaryControlLabel", "Temporary Control");
	}

	if (RigTreeElement->bIsAnimationChannel)
	{
		return FText::FromName(RigTreeElement->ChannelName);
	}

	if (InNameDisplayMode == EElementNameDisplayMode::AssetDefault)
	{
		if (const URigHierarchy* Hierarchy = Delegates.GetHierarchy())
		{
			if (const UControlRig* ControlRig = Cast<UControlRig>(Hierarchy->GetOuter()))
			{
				InNameDisplayMode = ControlRig->HierarchySettings.ElementNameDisplayMode;
			}
		}
	}

	if (InNameDisplayMode == EElementNameDisplayMode::Auto)
	{
		if (const URigHierarchy* Hierarchy = Delegates.GetHierarchy())
		{
			const FRigElementKey& Key = WeakRigTreeElement.Pin()->Key.GetElement();
			InNameDisplayMode = Hierarchy->HasUniqueShortName(Key.Type, Key.Name) ? EElementNameDisplayMode::ForceShort : EElementNameDisplayMode::ForceLong;
		}
	}

	if (InNameDisplayMode == EElementNameDisplayMode::ForceShort)
	{
		return WeakRigTreeElement.Pin()->ShortName;
	}

	return WeakRigTreeElement.Pin()->LongName;
}

FText SRigHierarchyItem::GetItemTooltip() const
{
	if (Delegates.OnRigTreeGetItemToolTip.IsBound())
	{
		const TOptional<FText> ToolTip = Delegates.OnRigTreeGetItemToolTip.Execute(WeakRigTreeElement.Pin()->Key);
		if (ToolTip.IsSet())
		{
			return ToolTip.GetValue();
		}
	}
	
	const FText FullName = GetName(EElementNameDisplayMode::ForceLong);
	const FText ShortName = GetName(EElementNameDisplayMode::ForceShort);
	if (FullName.EqualTo(ShortName))
	{
		return FText();
	}

	return FullName;
}

#undef LOCTEXT_NAMESPACE
