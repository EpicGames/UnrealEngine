// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"
#include "Editor/Hierarchy/RigHierarchyTreeDelegates.h"
#include "Internationalization/Text.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Widgets/Views/STableRow.h"

class SRigHierarchyTreeView;
enum class EElementNameDisplayMode : uint8;

class SRigHierarchyItem : public STableRow<TSharedPtr<FRigHierarchyTreeElement>>
{
public:

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FRigHierarchyTreeElement> InRigTreeElement, TSharedPtr<SRigHierarchyTreeView> InTreeView, const FRigHierarchyTreeDisplaySettings& InSettings, bool bPinned);
	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);
	static TPair<const FSlateBrush*, FSlateColor> GetBrushForElementType(const URigHierarchy* InHierarchy, const FRigHierarchyKey& InKey);
	static FLinearColor GetColorForControlType(ERigControlType InControlType, UEnum* InControlEnum);

private:
	TWeakPtr<FRigHierarchyTreeElement> WeakRigTreeElement;
	FRigHierarchyTreeDelegates Delegates;

	FText GetNameForUI() const;
	FText GetName(EElementNameDisplayMode InNameDisplayMode) const;
	FText GetItemTooltip() const;

	friend class SRigHierarchyTreeView;
};
