// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigHierarchySearchableTreeView.h"

#include "SRigHierarchyTreeView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBox.h"

void SRigHierarchySearchableTreeView::Construct(
	const FArguments& InArgs, 
	const FName& InViewName, 
	const TSharedPtr<IControlRigBaseEditor>& InControlRigEditor)
{
	FRigHierarchyTreeDelegates TreeDelegates = InArgs._RigTreeDelegates;
	SuperGetRigTreeDisplaySettings = TreeDelegates.OnGetDisplaySettings;

	TreeDelegates.OnGetDisplaySettings.BindSP(this, &SRigHierarchySearchableTreeView::GetDisplaySettings);

	TSharedPtr<SVerticalBox> VerticalBox;
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
		[
			SAssignNew(VerticalBox, SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(SearchBox, SSearchBox)
				.InitialText(InArgs._InitialFilterText)
				.OnTextChanged(this, &SRigHierarchySearchableTreeView::OnFilterTextChanged)
			]

			+SVerticalBox::Slot()
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					SAssignNew(TreeView, SRigHierarchyTreeView, InViewName, InControlRigEditor)
					.RigTreeDelegates(TreeDelegates)
					.ForModalWindow(InArgs._ForModalWindow)
				]
			]
		]
	];
}

const FRigHierarchyTreeDisplaySettings& SRigHierarchySearchableTreeView::GetDisplaySettings()
{
	if(SuperGetRigTreeDisplaySettings.IsBound())
	{
		Settings = SuperGetRigTreeDisplaySettings.Execute();
	}
	Settings.FilterText = FilterText;
	return Settings;
}

void SRigHierarchySearchableTreeView::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;
	GetTreeView()->RefreshTreeView();
}
