// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/RigHierarchyTreeDelegates.h"
#include "Editor/Hierarchy/RigHierarchyTreeDisplaySettings.h"
#include "Internationalization/Text.h"
#include "Widgets/SCompoundWidget.h"

class IControlRigBaseEditor;
class SRigHierarchyTreeView;
class SSearchBox;

class SRigHierarchySearchableTreeView : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRigHierarchySearchableTreeView)
		{
		}
		SLATE_ARGUMENT(FRigHierarchyTreeDelegates, RigTreeDelegates)
		SLATE_ARGUMENT(FText, InitialFilterText)

		/** Disables features that don't work in modal windows due to the absence of the game ticker and frame numbers */
		SLATE_ARGUMENT(bool, ForModalWindow)

	SLATE_END_ARGS()

	/**
	 * Constructs this widget
	 *
	 * @param InArgs				Slate Arguments
	 * @param InViewName			The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor
	 * @param InControlRigEditor	(optional) The editor that displays this widget. When set, supends details updates during tree refresh.
	 */
	void Construct(
		const FArguments& InArgs, 
		const FName& InViewName,
		const TSharedPtr<IControlRigBaseEditor>& InControlRigEditor = nullptr);

	TSharedRef<SSearchBox> GetSearchBox() const { return SearchBox.ToSharedRef(); }
	TSharedRef<SRigHierarchyTreeView> GetTreeView() const { return TreeView.ToSharedRef(); }
	const FRigHierarchyTreeDisplaySettings& GetDisplaySettings();

private:

	void OnFilterTextChanged(const FText& SearchText);

	FOnGetRigTreeDisplaySettings SuperGetRigTreeDisplaySettings;
	FText FilterText;
	FRigHierarchyTreeDisplaySettings Settings;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SRigHierarchyTreeView> TreeView;
};
