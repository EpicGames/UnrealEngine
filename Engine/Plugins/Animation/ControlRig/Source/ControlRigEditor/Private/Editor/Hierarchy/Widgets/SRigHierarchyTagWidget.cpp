// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigHierarchyTagWidget.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "DetailLayoutBuilder.h"
#include "Editor/Hierarchy/DragDrop/RigHierarchyTagDragDropOp.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SRigHierarchyTagWidget"

//////////////////////////////////////////////////////////////
/// SRigHierarchyTagWidget
///////////////////////////////////////////////////////////

void SRigHierarchyTagWidget::Construct(const FArguments& InArgs)
{
	Text = InArgs._Text;
	Icon = InArgs._Icon;
	IconColor = InArgs._IconColor;
	Color = InArgs._Color;
	Radius = InArgs._Radius;
	Identifier = InArgs._Identifier;
	bAllowDragDrop = InArgs._AllowDragDrop;
	OnClicked = InArgs._OnClicked;
	OnRenamed = InArgs._OnRenamed;
	OnVerifyRename = InArgs._OnVerifyRename;

	SetVisibility(InArgs._Visibility);

	const bool bWithIcon = Icon.IsSet() && Icon.Get() != nullptr;
	const bool bWithText = Text.IsSet() && !Text.Get().IsEmpty();
	if (!ensureMsgf(bWithIcon || bWithText, TEXT("Constructing a SRigHierarchyTagWidget that has neither an icon nor a text. This widget will never be visible.")))
	{
		return;
	}

	CombinedPadding = bWithIcon && bWithText ? FMargin(2.f, 1.f, 2.f, 1.f) : FMargin(1.f);

	const TSharedRef<SHorizontalBox> HorizontalBox =
		SNew(SHorizontalBox)
		.ToolTipText(InArgs._TooltipText);

	if (bWithIcon)
	{
		const EHorizontalAlignment HorizontalAlignment = bWithText ? HAlign_Left : HAlign_Center;
		const FMargin IconPadding = bWithText ? FMargin(2.f, 1.f, 1.f, 1.f) : FMargin(1.f);

		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HorizontalAlignment)
			.VAlign(VAlign_Center)
			.Padding(IconPadding)
			[
				SNew(SImage)
				.Visibility_Lambda([this]()
					{
						return Icon.Get() ? EVisibility::Visible : EVisibility::Collapsed;
					})
				.Image(Icon)
				.ColorAndOpacity(IconColor)
				.DesiredSizeOverride(InArgs._IconSize)
			];
	}

	if (bWithText)
	{
		const EHorizontalAlignment HorizontalAlignment = bWithIcon ? HAlign_Right : HAlign_Center;
		const FMargin TextPadding = bWithIcon ? FMargin(1.f, 1.f, 2.f, 1.f) : FMargin(1.f);
		
		HorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HorizontalAlignment)
			.VAlign(VAlign_Center)
			.Padding(TextPadding)
			[
				SNew(SInlineEditableTextBlock)
				.Text(Text)
				.ColorAndOpacity(InArgs._TextColor)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsReadOnly_Lambda([this](){ return !OnRenamed.IsBound(); })
				.OnTextCommitted(this, &SRigHierarchyTagWidget::HandleElementRenamed)
				.OnVerifyTextChanged(this, &SRigHierarchyTagWidget::HandleVerifyRename)
			];
	}

	ChildSlot
	[
		HorizontalBox
	];
}

int32 SRigHierarchyTagWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId,
	const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FSlateRoundedBoxBrush RoundedBoxBrush(FLinearColor::White, Radius);
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(
			AllottedGeometry.GetLocalSize() - CombinedPadding.GetDesiredSize(),
			FSlateLayoutTransform(FVector2d(CombinedPadding.Left, CombinedPadding.Top))
		),
		&RoundedBoxBrush,
		ESlateDrawEffect::NoPixelSnapping,
		Color.Get()
	);
	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply SRigHierarchyTagWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if(OnClicked.IsBound())
		{
			OnClicked.Execute();
		}
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}
	return FReply::Unhandled();
}

FReply SRigHierarchyTagWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(bAllowDragDrop && (Identifier.IsSet() || Identifier.IsBound())) 
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && !Identifier.Get().IsEmpty())
		{
			TSharedRef<FRigHierarchyTagDragDropOp> DragDropOp = FRigHierarchyTagDragDropOp::New(SharedThis(this));

			FRigElementKey DraggedKey;
			FRigElementKey::StaticStruct()->ImportText(*Identifier.Get(), &DraggedKey, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);

			if(DraggedKey.IsValid())
			{
				(void)OnElementKeyDragDetectedDelegate.ExecuteIfBound(DraggedKey);
			}

			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}
	return FReply::Unhandled();
}

void SRigHierarchyTagWidget::HandleElementRenamed(const FText& InNewName, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		if (OnRenamed.IsBound())
		{
			OnRenamed.Execute(InNewName, InCommitType);
		}
	}
}

bool SRigHierarchyTagWidget::HandleVerifyRename(const FText& InText, FText& OutError)
{
	if (OnVerifyRename.IsBound())
	{
		return OnVerifyRename.Execute(InText, OutError);
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
