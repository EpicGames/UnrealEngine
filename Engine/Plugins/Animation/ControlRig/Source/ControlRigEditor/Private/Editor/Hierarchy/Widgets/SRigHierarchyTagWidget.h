// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Widgets/SCompoundWidget.h"

class SRigHierarchyTagWidget;

DECLARE_DELEGATE_OneParam(FOnRigTreeElementKeyTagDragDetected, const FRigElementKey&);

class SRigHierarchyTagWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigHierarchyTagWidget)
		: _Text()
		, _TooltipText()
		, _Icon(nullptr)
		, _Color(FLinearColor(.2f, .2f, .2f))
		, _IconColor(FLinearColor::White)
		, _TextColor(FLinearColor(.8f, .8f, .8f))
		, _Radius(5)
		, _Identifier()
		, _AllowDragDrop(false)
		, _OnClicked()
		, _OnRenamed()
		, _OnVerifyRename()
	{}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ATTRIBUTE(FText, TooltipText)
		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)
		SLATE_ATTRIBUTE(TOptional<FVector2d>, IconSize)
		SLATE_ATTRIBUTE(FLinearColor, Color)
		SLATE_ATTRIBUTE(FSlateColor, IconColor)
		SLATE_ATTRIBUTE(FSlateColor, TextColor)
		SLATE_ARGUMENT(float, Radius)
		SLATE_ATTRIBUTE(FString, Identifier)
		SLATE_ARGUMENT(bool, AllowDragDrop)
		SLATE_EVENT(FSimpleDelegate, OnClicked)
		SLATE_EVENT(FOnTextCommitted, OnRenamed)
		SLATE_EVENT(FOnVerifyTextChanged, OnVerifyRename)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	FOnRigTreeElementKeyTagDragDetected& OnElementKeyDragDetected() { return OnElementKeyDragDetectedDelegate; }

private:
	void HandleElementRenamed(const FText& InNewName, ETextCommit::Type InCommitType);
	bool HandleVerifyRename(const FText& InText, FText& OutError);

	TAttribute<FText> Text;
	TAttribute<const FSlateBrush*> Icon;
	TAttribute<FLinearColor> Color;
	TAttribute<FSlateColor> IconColor;
	float Radius = 5.f;
	FMargin CombinedPadding;
	TAttribute<FString> Identifier;
	bool bAllowDragDrop = false;
	FSimpleDelegate OnClicked;
	FOnRigTreeElementKeyTagDragDetected OnElementKeyDragDetectedDelegate;
	FOnTextCommitted OnRenamed;
	FOnVerifyTextChanged OnVerifyRename;

	friend class FRigHierarchyTagDragDropOp;
};
