// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigHierarchyTreeView.h"

#include "Algo/Sort.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigEditorStyle.h"
#include "Editor/Hierarchy/Models/RigHierarchyTagModel.h"
#include "Editor/Hierarchy/RigHierarchyTreeDelegates.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateStore.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyItem.h"
#include "Framework/Application/SlateApplication.h"
#include "Graph/ControlRigGraphSchema.h"
#include "Settings/ControlRigSettings.h"
#include "Styling/AppStyle.h"
#include "RigVMDeveloperTypeUtils.h"

#define LOCTEXT_NAMESPACE "SRigHierarchyTreeView"

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Allow to destruct deprecated OldSparseItemInfos
SRigHierarchyTreeView::~SRigHierarchyTreeView() 
{
	using namespace UE::ControlRigEditor;

	// @todo, it's not safe to call Delegates.GetHierarchy() here.
	//if (FRigHierarchyTreePersistentStateStore::IsFeatureEnabled())
	//{
	//	// Retain the state so it gets restored when the tree is constructed anew
	//	const URigHierarchy* Hierarchy = Delegates.GetHierarchy();
	//	const UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
	//	if (ControlRig)
	//	{
	//		FRigHierarchyTreePersistentStateStore::Get().StorePersistentState(*ControlRig, *this, ViewName);
	//	}
	//}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SRigHierarchyTreeView::Construct(const FArguments& InArgs, const FName& InViewName, const TSharedPtr<IControlRigBaseEditor>& InControlRigEditor)
{
	WeakControlRigEditor = InControlRigEditor;
	ViewName = InViewName;

	Delegates = InArgs._RigTreeDelegates;
	bAutoScrollEnabled = InArgs._AutoScrollEnabled;
	bForModalWindow = InArgs._ForModalWindow;

	STreeView<TSharedPtr<FRigHierarchyTreeElement>>::FArguments SuperArgs;
	SuperArgs.TreeItemsSource(&RootElements);
	SuperArgs.SelectionMode(ESelectionMode::Multi);
	SuperArgs.OnGenerateRow(this, &SRigHierarchyTreeView::MakeTableRowWidget, false);
	SuperArgs.OnGetChildren(this, &SRigHierarchyTreeView::HandleGetChildrenForTree);
	SuperArgs.OnSelectionChanged(FOnRigTreeSelectionChanged::CreateRaw(&Delegates, &FRigHierarchyTreeDelegates::HandleSelectionChanged));
	SuperArgs.OnContextMenuOpening(Delegates.OnContextMenuOpening);
	SuperArgs.OnMouseButtonClick(Delegates.OnMouseButtonClick);
	SuperArgs.OnMouseButtonDoubleClick(Delegates.OnMouseButtonDoubleClick);
	SuperArgs.OnSetExpansionRecursive(Delegates.OnSetExpansionRecursive);
	SuperArgs.HighlightParentNodesForSelection(true);
	SuperArgs.AllowInvisibleItemSelection(true);  //without this we deselect everything when we filter or we collapse
	
	SuperArgs.ShouldStackHierarchyHeaders_Lambda([]() -> bool {
		return UControlRigEditorSettings::Get()->bShowStackedHierarchy;
	});
	SuperArgs.OnGeneratePinnedRow(this, &SRigHierarchyTreeView::MakeTableRowWidget, true);
	SuperArgs.MaxPinnedItems_Lambda([]() -> int32
	{
		return FMath::Max<int32>(1, UControlRigEditorSettings::Get()->MaxStackSize);
	});

	STreeView<TSharedPtr<FRigHierarchyTreeElement>>::Construct(SuperArgs);

	LastMousePosition = FVector2D::ZeroVector;
	TimeAtMousePosition = 0.0;

	if(InArgs._PopulateOnConstruct)
	{
		RefreshTreeView();
	}
}

void SRigHierarchyTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STreeView<TSharedPtr<FRigHierarchyTreeElement, ESPMode::ThreadSafe>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FGeometry PaintGeometry = GetPaintSpaceGeometry();
	const FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();

	if(PaintGeometry.IsUnderLocation(MousePosition))
	{
		const FVector2D WidgetPosition = PaintGeometry.AbsoluteToLocal(MousePosition);

		static constexpr float SteadyMousePositionTolerance = 5.f;

		if(LastMousePosition.Equals(MousePosition, SteadyMousePositionTolerance))
		{
			TimeAtMousePosition += InDeltaTime;
		}
		else
		{
			LastMousePosition = MousePosition;
			TimeAtMousePosition = 0.0;
		}

		static constexpr float AutoScrollStartDuration = 0.5f; // in seconds
		static constexpr float AutoScrollDistance = 24.f; // in pixels
		static constexpr float AutoScrollSpeed = 150.f;

		if(TimeAtMousePosition > AutoScrollStartDuration && FSlateApplication::Get().IsDragDropping())
		{
			if((WidgetPosition.Y < AutoScrollDistance) || (WidgetPosition.Y > PaintGeometry.Size.Y - AutoScrollDistance))
			{
				if(bAutoScrollEnabled)
				{
					const bool bScrollUp = (WidgetPosition.Y < AutoScrollDistance);

					const float DeltaInSlateUnits = (bScrollUp ? -InDeltaTime : InDeltaTime) * AutoScrollSpeed; 
					ScrollBy(GetCachedGeometry(), DeltaInSlateUnits, EAllowOverscroll::No);
				}
			}
			else
			{
				const TSharedPtr<FRigHierarchyTreeElement>* Item = FindItemAtPosition(MousePosition);
				if(Item && Item->IsValid())
				{
					if(!IsItemExpanded(*Item))
					{
						SetItemExpansion(*Item, true);
					}
				}
			}
		}
	}
}

TSharedPtr<FRigHierarchyTreeElement> SRigHierarchyTreeView::FindElement(const FRigHierarchyKey& InElementKey) const
{
	for (int32 RootIndex = 0; RootIndex < RootElements.Num(); ++RootIndex)
	{
		TSharedPtr<FRigHierarchyTreeElement> Found = FindElement(InElementKey, RootElements[RootIndex]);
		if (Found.IsValid())
		{
			return Found;
		}
	}
	return TSharedPtr<FRigHierarchyTreeElement>();
}

TSharedPtr<FRigHierarchyTreeElement> SRigHierarchyTreeView::FindElement(const FRigHierarchyKey& InElementKey, TSharedPtr<FRigHierarchyTreeElement> CurrentItem)
{
	if (CurrentItem->Key == InElementKey)
	{
		return CurrentItem;
	}

	for (int32 ChildIndex = 0; ChildIndex < CurrentItem->Children.Num(); ++ChildIndex)
	{
		TSharedPtr<FRigHierarchyTreeElement> Found = FindElement(InElementKey, CurrentItem->Children[ChildIndex]);
		if (Found.IsValid())
		{
			return Found;
		}
	}

	return TSharedPtr<FRigHierarchyTreeElement>();
}

bool SRigHierarchyTreeView::AddElement(FRigHierarchyKey InKey, FRigHierarchyKey InParentKey)
{
	if(ElementMap.Contains(InKey))
	{
		return false;
	}

	// skip transient controls
	if(InKey.IsElement())
	{
		if(const URigHierarchy* Hierarchy = Delegates.GetHierarchy())
		{
			if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(InKey.GetElement()))
			{
				if(ControlElement->Settings.bIsTransientControl)
				{
					return false;
				}
			}
		}
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();
	const bool bSupportsRename = Delegates.OnRenameElement.IsBound();

	const FString FilteredString = Settings.FilterText.ToString();
	bool bAnyFilteredOut = Delegates.OnRigTreeIsItemVisible.IsBound();
	if (!bAnyFilteredOut)
	{
		bAnyFilteredOut = !FilteredString.IsEmpty() && InKey.IsValid();
	}

	if (!bAnyFilteredOut)
	{
		TSharedPtr<FRigHierarchyTreeElement> NewItem = MakeShared<FRigHierarchyTreeElement>(InKey, SharedThis(this), bSupportsRename, ERigTreeFilterResult::Shown);

		if (InKey.IsValid())
		{
			ElementMap.Add(InKey, NewItem);
			if (InParentKey.IsValid())
			{
				ParentMap.Add(InKey, InParentKey);
			}

			if (InParentKey.IsValid())
			{
				TSharedPtr<FRigHierarchyTreeElement>* FoundItem = ElementMap.Find(InParentKey);
				check(FoundItem);
				FoundItem->Get()->Children.Add(NewItem);
			}
			else
			{
				RootElements.Add(NewItem);
			}
		}
		else
		{
			RootElements.Add(NewItem);
		}
	}
	else
	{
		bool bIsFilteredOut = false;
		if (Delegates.OnRigTreeIsItemVisible.IsBound())
		{
			bIsFilteredOut = !Delegates.OnRigTreeIsItemVisible.Execute(InKey);
		}

		const URigHierarchy* Hierarchy = Delegates.GetHierarchy();
		auto GetFirstParent = [Hierarchy](const FRigHierarchyKey& InKey) -> FRigHierarchyKey
		{
			if(InKey.IsElement())
			{
				return Hierarchy->GetFirstParent(InKey.GetElement());;
			}
			if(InKey.IsComponent())
			{
				return InKey.GetComponent().ElementKey;
			}
			return FRigHierarchyKey();
		};
		
		FString FilteredStringUnderScores = FilteredString.Replace(TEXT(" "), TEXT("_"));
		if (!bIsFilteredOut && (InKey.GetName().Contains(FilteredString) || InKey.GetName().Contains(FilteredStringUnderScores)))
		{
			TSharedPtr<FRigHierarchyTreeElement> NewItem = MakeShared<FRigHierarchyTreeElement>(InKey, SharedThis(this), bSupportsRename, ERigTreeFilterResult::Shown);
			ElementMap.Add(InKey, NewItem);
			RootElements.Add(NewItem);

			if (!Settings.bFlattenHierarchyOnFilter && !Settings.bHideParentsOnFilter)
			{
				if(Hierarchy)
				{
					TSharedPtr<FRigHierarchyTreeElement> ChildItem = NewItem;
					FRigHierarchyKey ParentKey = GetFirstParent(InKey);
					while (ParentKey.IsValid())
					{
						if (!ElementMap.Contains(ParentKey))
						{
							TSharedPtr<FRigHierarchyTreeElement> ParentItem = MakeShared<FRigHierarchyTreeElement>(ParentKey, SharedThis(this), bSupportsRename, ERigTreeFilterResult::ShownDescendant);							
							ElementMap.Add(ParentKey, ParentItem);
							RootElements.Add(ParentItem);

							ReparentElement(ChildItem->Key, ParentKey);

							ChildItem = ParentItem;
							ParentKey = GetFirstParent(ParentKey);
						}
						else
						{
							ReparentElement(ChildItem->Key, ParentKey);
							break;
						}						
					}
				}
			}
		}
	}

	return true;
}

bool SRigHierarchyTreeView::AddElement(const FRigBaseElement* InElement)
{
	using namespace UE::ControlRigEditor;

	const URigHierarchy* Hierarchy = Delegates.GetHierarchy();
	UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
	if (!InElement ||
		!Hierarchy ||
		!ControlRig)
	{
		return false;
	}

	const bool bAlreadyAdded = ElementMap.Contains(InElement->GetKey());
	if (bAlreadyAdded)
	{
		return false;
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();

	if (InElement->GetType() == ERigElementType::Physics ||
		InElement->GetType() == ERigElementType::Curve)
	{
		return false;
	}
	else if (InElement->GetType() == ERigElementType::Bone)
	{
		if (!Settings.bShowBones)
		{
			return false;
		}

		const FRigBoneElement* BoneElement = CastChecked<FRigBoneElement>(InElement);
		if (!Settings.bShowImportedBones && BoneElement->BoneType == ERigBoneType::Imported)
		{
			return false;
		}
	}
	else if (InElement->GetType() == ERigElementType::Null)
	{
		if (!Settings.bShowNulls)
		{
			return false;
		}
	}
	else if (InElement->GetType() == ERigElementType::Control)
	{
		if (!Settings.bShowControls)
		{
			return false;
		}
	}
	else if (InElement->GetType() == ERigElementType::Reference)
	{
		if (!Settings.bShowReferences)
		{
			return false;
		}
	}
	else if (InElement->GetType() == ERigElementType::Socket)
	{
		if (!Settings.bShowSockets)
		{
			return false;
		}
	}
	else if (InElement->GetType() == ERigElementType::Connector)
	{
		// Add the connector as a tag
		const FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(InElement);
		if (!ConnectorElement)
		{
			return false;
		}

		FRigElementKeyRedirector& Redirector = ControlRig->GetElementKeyRedirector();
		FRigElementKeyRedirector::FCachedKeyArray* Cache = Redirector.Find(InElement->GetKey());
		if (Cache)
		{
			// Find the elements this connector is connected to
			bool bValid = true;
			TArray<TSharedRef<FRigHierarchyTreeElement>> TargetElements;

			for (FCachedRigElement& CachedRigElement : *Cache)
			{
				if (!CachedRigElement.UpdateCache(Hierarchy))
				{
					bValid = false;
				}

				if (const TSharedPtr<FRigHierarchyTreeElement>* TargetElementPtr = ElementMap.Find(CachedRigElement.GetKey()))
				{
					if (TargetElementPtr->IsValid())
					{
						TargetElements.Add((*TargetElementPtr).ToSharedRef());
					}
				}
			}

			if (bValid)
			{
				AddConnectorTagToElements(TargetElements, *ConnectorElement, *ControlRig);
				return true;
			}
		}
	}

	if (!AddElement(InElement->GetKey(), FRigHierarchyKey()))
	{
		return false;
	}

	if (ElementMap.Contains(InElement->GetKey()))
	{
		if (Hierarchy)
		{
			if (InElement->GetType() == ERigElementType::Connector)
			{
				if (const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(InElement))
				{
					AddConnectorResolveWarningTagToElement(ElementMap.FindChecked(InElement->GetKey()).ToSharedRef(), *Connector, *ControlRig);
				}
			}

			auto FindParentKey = [&](const FRigBaseElement* InElement)
				{
					FRigElementKey ParentKey = Hierarchy->GetFirstParent(InElement->GetKey());
					if (InElement->GetType() == ERigElementType::Connector)
					{
						ParentKey = Delegates.GetResolvedKey(InElement->GetKey()).GetElement();
						if (ParentKey == InElement->GetKey())
						{
							ParentKey.Reset();
						}
					}

					if (FRigBaseElement* Parent = Hierarchy->GetActiveParent(InElement))
					{
						ParentKey = Parent->GetKey();
					}
					
					return ParentKey;
				};
			FRigElementKey ParentKey = FindParentKey(InElement);
			if (ParentKey.IsValid())
			{
				bool bReparented = false;
				do
				{
					if (const FRigBaseElement* ParentElement = Hierarchy->Find(ParentKey))
					{
						AddElement(ParentElement);

						if (ElementMap.Contains(ParentKey))
						{
							ReparentElement(InElement->GetKey(), ParentKey);
							bReparented = true;
						}
					
						if (!bReparented)
						{
							ParentKey = FindParentKey(ParentElement); 
						}
					}
					else
					{
						ParentKey.Reset();
					}
				}
				while (!bReparented && ParentKey.IsValid());
			}
		}
	}

	for (int32 ComponentIndex = 0; ComponentIndex < InElement->NumComponents(); ComponentIndex++)
	{
		AddComponent(InElement->GetComponent(ComponentIndex));
	}

	return true;
}

bool SRigHierarchyTreeView::AddComponent(const FRigBaseComponent* InComponent)
{
	check(InComponent);
	
	if (ElementMap.Contains(InComponent->GetKey()))
	{
		return false;
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();
	if(!Settings.bShowComponents)
	{
		return false;
	}

	if(!AddElement(InComponent->GetKey(), InComponent->GetElementKey()))
	{
		return false;
	}

	return true;
}

void SRigHierarchyTreeView::AddSpacerElement()
{
	AddElement(FRigElementKey(), FRigHierarchyKey());
}

bool SRigHierarchyTreeView::ReparentElement(FRigHierarchyKey InKey, FRigHierarchyKey InParentKey)
{
	if (!InKey.IsValid() || InKey == InParentKey)
	{
		return false;
	}

	if(InKey.IsElement())
	{
		if(InKey.GetElement().Type == ERigElementType::Connector)
		{
			return false;
		}
	}

	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();

	TSharedPtr<FRigHierarchyTreeElement>* FoundItem = ElementMap.Find(InKey);
	if (FoundItem == nullptr)
	{
		return false;
	}

	if (!Settings.FilterText.IsEmpty() && Settings.bFlattenHierarchyOnFilter)
	{
		return false;
	}

	if (const FRigHierarchyKey* ExistingParentKey = ParentMap.Find(InKey))
	{
		if (*ExistingParentKey == InParentKey)
		{
			return false;
		}

		if (TSharedPtr<FRigHierarchyTreeElement>* ExistingParent = ElementMap.Find(*ExistingParentKey))
		{
			(*ExistingParent)->Children.Remove(*FoundItem);
		}

		ParentMap.Remove(InKey);
	}
	else
	{
		if (!InParentKey.IsValid())
		{
			return false;
		}

		RootElements.Remove(*FoundItem);
	}

	if (InParentKey.IsValid())
	{
		ParentMap.Add(InKey, InParentKey);

		TSharedPtr<FRigHierarchyTreeElement>* FoundParent = ElementMap.Find(InParentKey);
		if(FoundParent)
		{
			FoundParent->Get()->Children.Add(*FoundItem);
		}
	}
	else
	{
		RootElements.Add(*FoundItem);
	}

	return true;
}

bool SRigHierarchyTreeView::RemoveElement(FRigHierarchyKey InKey)
{
	TSharedPtr<FRigHierarchyTreeElement>* FoundItem = ElementMap.Find(InKey);
	if (FoundItem == nullptr)
	{
		return false;
	}

	ReparentElement(InKey, FRigHierarchyKey());

	RootElements.Remove(*FoundItem);
	return ElementMap.Remove(InKey) > 0;
}

void SRigHierarchyTreeView::RefreshTreeView(const bool bRebuildContent, const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored)
{
	using namespace UE::ControlRigEditor;

	const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakControlRigEditor.IsValid() ? WeakControlRigEditor.Pin() : nullptr;

	const bool bCanUsePersistentStateStore = !bForModalWindow;
	if (bCanUsePersistentStateStore &&
		FRigHierarchyTreePersistentStateStore::IsFeatureEnabled())
	{
		// New with persistent state store
		const URigHierarchy* Hierarchy = Delegates.GetHierarchy();
		const UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
		if (!Hierarchy || !ControlRig)
		{
			OnStateRestored.ExecuteIfBound();
			return;
		}

		bRebuildContentRequested |= bRebuildContent;

		const TSharedRef<STreeView<TSharedPtr<FRigHierarchyTreeElement>>> ThisAsTreeView =
			StaticCastSharedRef<STreeView<TSharedPtr<FRigHierarchyTreeElement>>>(SharedThis(this));

		FRigHierarchyTreePersistentStateStore::Get().RequestTreeRefresh(
			*ControlRig,
			ThisAsTreeView,
			ViewName,
			ControlRigEditor,
			FRigHierarchyTreePersistentStateRebuildItemsRequested::CreateLambda(
				[WeakThis = AsWeak(), this]()
				{
					if (!WeakThis.IsValid())
					{
						return;
					}

					if (bRebuildContentRequested)
					{
						RootElements.Reset();
						ElementMap.Reset();
						ParentMap.Reset();
						ClearSelection();

						const URigHierarchy* Hierarchy = Delegates.GetHierarchy();
						if (!Hierarchy)
						{
							return;
						}

						TArray<const FRigSocketElement*> Sockets;
						TArray<const FRigConnectorElement*> Connectors;
						TArray<const FRigBaseElement*> EverythingElse;
						TMap<const FRigBaseElement*, int32> ElementDepth;
						Sockets.Reserve(Hierarchy->Num(ERigElementType::Socket));
						Connectors.Reserve(Hierarchy->Num(ERigElementType::Connector));
						EverythingElse.Reserve(Hierarchy->Num() - Hierarchy->Num(ERigElementType::Socket) - Hierarchy->Num(ERigElementType::Connector));

						Hierarchy->Traverse([&](FRigBaseElement* Element, bool& bContinue)
							{
								int32& Depth = ElementDepth.Add(Element, 0);
								if (const FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(Element))
								{
									if (int32* ParentDepth = ElementDepth.Find(ParentElement))
									{
										Depth = *ParentDepth + 1;
									}
								}

								if (const FRigSocketElement* Socket = Cast<FRigSocketElement>(Element))
								{
									Sockets.AddUnique(Socket);
								}
								else if (const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(Element))
								{
									Connectors.AddUnique(Connector);
								}
								else
								{
									EverythingElse.AddUnique(Element);
								}
								bContinue = true;
							}, true, true);

						// sort the sockets by depth
						Algo::SortBy(Sockets, [ElementDepth](const FRigSocketElement* Socket) -> int32
							{
								return ElementDepth.FindChecked(Socket);
							});
						for (const FRigSocketElement* Socket : Sockets)
						{
							AddElement(Socket);
						}

						// add everything but connectors and sockets
						for (const FRigBaseElement* Element : EverythingElse)
						{
							AddElement(Element);
						}

						// add all of the connectors. their parent relationship in the tree represents resolve
						for (const FRigConnectorElement* Connector : Connectors)
						{
							AddElement(Connector);
						}

						if (Delegates.OnCompareKeys.IsBound())
						{
							Algo::Sort(RootElements, [&](const TSharedPtr<FRigHierarchyTreeElement>& A, const TSharedPtr<FRigHierarchyTreeElement>& B)
								{
									return Delegates.OnCompareKeys.Execute(A->Key, B->Key);
								});
						}

						if (RootElements.Num() > 0)
						{
							AddSpacerElement();
						}
						
						{
							ClearSelection();

							TArray<FRigHierarchyKey> Selection = Delegates.GetSelection();
							for (const FRigHierarchyKey& Key : Selection)
							{
								for (int32 RootIndex = 0; RootIndex < RootElements.Num(); ++RootIndex)
								{
									TSharedPtr<FRigHierarchyTreeElement> Found = FindElement(Key, RootElements[RootIndex]);
									if (Found.IsValid())
									{
										SetItemSelection(Found, true, ESelectInfo::OnNavigation);
									}
								}
							}
						}

						bRebuildContentRequested = false;
					}
				}),
			OnStateRestored
		);

		return;
	}
	
	// Old without persistent state store
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	bool bDummySuspensionFlag = false;
	bool* SuspensionFlagPtr = &bDummySuspensionFlag;
	if (ControlRigEditor.IsValid())
	{
		SuspensionFlagPtr = &ControlRigEditor->GetSuspendDetailsPanelRefreshFlag();
	}
	TGuardValue<bool> SuspendDetailsPanelRefreshGuard(*SuspensionFlagPtr, true);

	TMap<FRigHierarchyKey, bool> ExpansionState;

	if(bRebuildContent)
	{
		for (TPair<FRigHierarchyKey, TSharedPtr<FRigHierarchyTreeElement>> Pair : ElementMap)
		{
			ExpansionState.FindOrAdd(Pair.Key) = IsItemExpanded(Pair.Value);
		}

		// internally save expansion states before rebuilding the tree, so the states can be restored later
		SaveAndClearSparseItemInfos();

		RootElements.Reset();
		ElementMap.Reset();
		ParentMap.Reset();
	}

	if(bRebuildContent)
	{
		const URigHierarchy* Hierarchy = Delegates.GetHierarchy();
		if (ensure(Hierarchy))
		{
			TArray<const FRigSocketElement*> Sockets;
			TArray<const FRigConnectorElement*> Connectors;
			TArray<const FRigBaseElement*> EverythingElse;
			TMap<const FRigBaseElement*, int32> ElementDepth;
			Sockets.Reserve(Hierarchy->Num(ERigElementType::Socket));
			Connectors.Reserve(Hierarchy->Num(ERigElementType::Connector));
			EverythingElse.Reserve(Hierarchy->Num() - Hierarchy->Num(ERigElementType::Socket) - Hierarchy->Num(ERigElementType::Connector));
			
			Hierarchy->Traverse([&](FRigBaseElement* Element, bool& bContinue)
			{
				int32& Depth = ElementDepth.Add(Element, 0);
				if(const FRigBaseElement* ParentElement = Hierarchy->GetFirstParent(Element))
				{
					if (int32* ParentDepth = ElementDepth.Find(ParentElement))
					{
						Depth = *ParentDepth + 1;
					}
				}
				
				if(const FRigSocketElement* Socket = Cast<FRigSocketElement>(Element))
				{
					Sockets.AddUnique(Socket);
				}
				else if(const FRigConnectorElement* Connector = Cast<FRigConnectorElement>(Element))
				{
					Connectors.AddUnique(Connector);
				}
				else
				{
					EverythingElse.AddUnique(Element);
				}
				bContinue = true;
			}, true, true);

			// sort the sockets by depth
			Algo::SortBy(Sockets, [ElementDepth](const FRigSocketElement* Socket) -> int32
			{
				return ElementDepth.FindChecked(Socket);
			});
			for(const FRigSocketElement* Socket : Sockets)
			{
				AddElement(Socket);
			}

			// add everything but connectors and sockets
			for(const FRigBaseElement* Element : EverythingElse)
			{
				AddElement(Element);
			}

			// add all of the connectors. their parent relationship in the tree represents resolve
			for(const FRigConnectorElement* Connector : Connectors)
			{
				AddElement(Connector);
			}

			// expand all elements upon the initial construction of the tree
			if (ExpansionState.Num() == 0)
			{
				for (TSharedPtr<FRigHierarchyTreeElement> RootElement : RootElements)
				{
					SetExpansionRecursive(RootElement, false, true);
				}
			}
			else if (ExpansionState.Num() < ElementMap.Num())
			{
				for (const TPair<FRigHierarchyKey, TSharedPtr<FRigHierarchyTreeElement>>& Element : ElementMap)
				{
					if (!ExpansionState.Contains(Element.Key))
					{
						SetItemExpansion(Element.Value, true);
					}
				}
			}

			for (const auto& Pair : ElementMap)
			{
				RestoreSparseItemInfos(Pair.Value);
			}

			if(Delegates.OnCompareKeys.IsBound())
			{
				Algo::Sort(RootElements, [&](const TSharedPtr<FRigHierarchyTreeElement>& A, const TSharedPtr<FRigHierarchyTreeElement>& B)
				{
					return Delegates.OnCompareKeys.Execute(A->Key, B->Key);
				});
			}

			if (RootElements.Num() > 0)
			{
				AddSpacerElement();
			}
		}
	}
	else
	{
		if (RootElements.Num()> 0)
		{
			// elements may be added at the end of the list after a spacer element
			// we need to remove the spacer element and re-add it at the end
			RootElements.RemoveAll([](TSharedPtr<FRigHierarchyTreeElement> InElement)
			{
				return InElement.Get()->Key == FRigElementKey();
			});
			AddSpacerElement();
		}
	}

	RequestTreeRefresh();
	{
		ClearSelection();

		TArray<FRigHierarchyKey> Selection = Delegates.GetSelection();
		for (const FRigHierarchyKey& Key : Selection)
		{
			for (int32 RootIndex = 0; RootIndex < RootElements.Num(); ++RootIndex)
			{
				TSharedPtr<FRigHierarchyTreeElement> Found = FindElement(Key, RootElements[RootIndex]);
				if (Found.IsValid())
				{
					SetItemSelection(Found, true, ESelectInfo::OnNavigation);
				}
			}
		}
	}

	OnStateRestored.ExecuteIfBound();

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void SRigHierarchyTreeView::SetExpansionRecursive(TSharedPtr<FRigHierarchyTreeElement> InElement, bool bTowardsParent,
	bool bShouldBeExpanded)
{
	SetItemExpansion(InElement, bShouldBeExpanded);

	if (bTowardsParent)
	{
		if (const FRigHierarchyKey* ParentKey = ParentMap.Find(InElement->Key))
		{
			if (TSharedPtr<FRigHierarchyTreeElement>* ParentItem = ElementMap.Find(*ParentKey))
			{
				SetExpansionRecursive(*ParentItem, bTowardsParent, bShouldBeExpanded);
			}
		}
	}
	else
	{
		for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
		{
			SetExpansionRecursive(InElement->Children[ChildIndex], bTowardsParent, bShouldBeExpanded);
		}
	}
}

TSharedRef<ITableRow> SRigHierarchyTreeView::MakeTableRowWidget(TSharedPtr<FRigHierarchyTreeElement> InItem,
	const TSharedRef<STableViewBase>& OwnerTable, bool bPinned)
{
	const FRigHierarchyTreeDisplaySettings& Settings = Delegates.GetDisplaySettings();
	return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this), Settings, bPinned);
}

void SRigHierarchyTreeView::HandleGetChildrenForTree(TSharedPtr<FRigHierarchyTreeElement> InItem,
	TArray<TSharedPtr<FRigHierarchyTreeElement>>& OutChildren)
{
	OutChildren = InItem.Get()->Children;
}

void SRigHierarchyTreeView::OnElementKeyTagDragDetected(const FRigElementKey& InDraggedTag)
{
	(void)Delegates.OnRigTreeElementKeyTagDragDetected.ExecuteIfBound(InDraggedTag);
}

TArray<FRigHierarchyKey> SRigHierarchyTreeView::GetSelectedKeys() const
{
	TArray<FRigHierarchyKey> Keys;
	TArray<TSharedPtr<FRigHierarchyTreeElement>> SelectedElements = GetSelectedItems();
	for(const TSharedPtr<FRigHierarchyTreeElement>& SelectedElement : SelectedElements)
	{
		Keys.Add(SelectedElement->Key);
	}
	return Keys;
}

const TSharedPtr<FRigHierarchyTreeElement>* SRigHierarchyTreeView::FindItemAtPosition(FVector2D InScreenSpacePosition) const
{
	if (ItemsPanel.IsValid() && SListView<TSharedPtr<FRigHierarchyTreeElement>>::HasValidItemsSource())
	{
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		const int32 Index = FindChildUnderPosition(ArrangedChildren, InScreenSpacePosition);
		if (ArrangedChildren.IsValidIndex(Index))
		{
			TSharedRef<SRigHierarchyItem> ItemWidget = StaticCastSharedRef<SRigHierarchyItem>(ArrangedChildren[Index].Widget);
			if (ItemWidget->WeakRigTreeElement.IsValid())
			{
				const FRigHierarchyKey Key = ItemWidget->WeakRigTreeElement.Pin()->Key;
				const TSharedPtr<FRigHierarchyTreeElement>* ResultPtr = SListView<TSharedPtr<FRigHierarchyTreeElement>>::GetItems().FindByPredicate([Key](const TSharedPtr<FRigHierarchyTreeElement>& Item) -> bool
					{
						return Item->Key == Key;
					});

				if (ResultPtr)
				{
					return ResultPtr;
				}
			}
		}
	}
	return nullptr;
}

void SRigHierarchyTreeView::AddConnectorTagToElements(
	const TArray<TSharedRef<FRigHierarchyTreeElement>> TargetElements,
	const FRigConnectorElement& ConnectorElement,
	const UControlRig& ControlRig)
{
	using namespace UE::ControlRigEditor;

	const FRigElementKey ConnectorKey = ConnectorElement.GetKey();

	const ERigHierarchyConnectorTagDisplayMode TagDisplayMode = Delegates.GetTagDisplayMode();

	if (TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Single)
	{
		// Get or create a single tag for each element the connector targets
		for (const TSharedRef<FRigHierarchyTreeElement>& TargetElement : TargetElements)
		{
			if (TargetElement->ConnectorTags.IsEmpty())
			{
				const FRigHierarchyValidConnectorTag NewTag = FRigHierarchyValidConnectorTag::BuildTag(TagDisplayMode);

				TargetElement->ConnectorTags.Add(NewTag);
			}
			check(!TargetElement->ConnectorTags.IsEmpty());

			TargetElement->ConnectorTags[0].AddConnector(ControlRig, Delegates.GetDisplaySettings(), ConnectorElement);
		}
	}
	else if (TagDisplayMode == ERigHierarchyConnectorTagDisplayMode::Individual)
	{
		// Add a tag too each element the connector targets
		const FRigHierarchyValidConnectorTag ElementTag =
			FRigHierarchyValidConnectorTag::BuildTag(TagDisplayMode)
			.AddConnector(ControlRig, Delegates.GetDisplaySettings(), ConnectorElement)
			.SetOnClicked(FSimpleDelegate::CreateLambda([ConnectorKey, this]()
				{
					Delegates.RequestDetailsInspection(ConnectorKey);
				}))
			.SetOnRenamed(FOnTextCommitted::CreateLambda([ConnectorKey, this](const FText& InNewName, ETextCommit::Type InCommitType)
				{
					(void)Delegates.HandleRenameElement(ConnectorKey, InNewName.ToString());
				}))
			.SetOnVerifyRename(FOnVerifyTextChanged::CreateLambda([ConnectorKey, this](const FText& InText, FText& OutError)
				{
					return Delegates.HandleVerifyElementNameChanged(ConnectorKey, InText.ToString(), OutError);
				}));

		for (const TSharedRef<FRigHierarchyTreeElement>& TargetElement : TargetElements)
		{
			TargetElement->ConnectorTags.Add(ElementTag);
		}
	}
	else
	{
		ensureMsgf(0, TEXT("Unhandled enum value"));
	}
}

void SRigHierarchyTreeView::AddConnectorResolveWarningTagToElement(
	const TSharedRef<FRigHierarchyTreeElement> TreeElement,
	const FRigConnectorElement& ConnectorElement,
	const UControlRig& ControlRig)
{
	using namespace UE::ControlRigEditor;

	if (ConnectorElement.IsOptional())
	{
		return;
	}

	const FRigHierarchyConnectorUnresolvedWarningTag NewTag(ControlRig, Delegates.GetDisplaySettings(), ConnectorElement);

	TreeElement->ConnectorResolveWarningTags.Add(NewTag);
}

bool SRigHierarchyItem::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	const FString NewName = InText.ToString();
	const FRigHierarchyKey OldKey = WeakRigTreeElement.Pin()->Key;
	return Delegates.HandleVerifyElementNameChanged(OldKey, NewName, OutErrorMessage);
}

TPair<const FSlateBrush*, FSlateColor> SRigHierarchyItem::GetBrushForElementType(const URigHierarchy* InHierarchy, const FRigHierarchyKey& InKey)
{
	static const FSlateBrush* ProxyControlBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.ProxyControl"); 
	static const FSlateBrush* ControlBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.Control");
	static const FSlateBrush* NullBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.Null");
	static const FSlateBrush* BoneImportedBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.BoneImported");
	static const FSlateBrush* BoneUserBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.BoneUser");
	static const FSlateBrush* PhysicsBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.RigidBody");
	static const FSlateBrush* SocketOpenBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.Socket_Open");
	static const FSlateBrush* SocketClosedBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.Tree.Socket_Closed");
	static const FSlateBrush* PrimaryConnectorBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorPrimary");
	static const FSlateBrush* SecondaryConnectorBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorSecondary");
	static const FSlateBrush* OptionalConnectorBrush = FControlRigEditorStyle::Get().GetBrush("ControlRig.ConnectorOptional");

	const FSlateBrush* Brush = nullptr;
	FSlateColor Color = FSlateColor::UseForeground();

	if (!InHierarchy)
	{
		Brush = FAppStyle::Get().GetBrush("Icons.Warning");
		Color = FAppStyle::Get().GetSlateColor("Colors.AccentYellow");
	}
	else if (InKey.IsElement())
	{
		switch (InKey.GetElement().Type)
		{
			case ERigElementType::Control:
			{
				if(const FRigControlElement* Control = InHierarchy->Find<FRigControlElement>(InKey.GetElement()))
				{
					FLinearColor ShapeColor = FLinearColor::White;
					
					if(Control->Settings.SupportsShape())
					{
						if(Control->Settings.AnimationType == ERigControlAnimationType::ProxyControl)
						{
							Brush = ProxyControlBrush;
						}
						else
						{
							Brush = ControlBrush;
						}
						ShapeColor = Control->Settings.ShapeColor;
					}
					else
					{
						static const FLazyName TypeIcon(TEXT("Kismet.VariableList.TypeIcon"));
						Brush = FAppStyle::GetBrush(TypeIcon);
						ShapeColor = GetColorForControlType(Control->Settings.ControlType, Control->Settings.ControlEnum);
					}
					
					// ensure the alpha is always visible
					ShapeColor.A = 1.f;
					Color = FSlateColor(ShapeColor);
				}
				else
				{
					Brush = ControlBrush;
				}
				break;
			}
			case ERigElementType::Null:
			{
				Brush = NullBrush;
				break;
			}
			case ERigElementType::Bone:
			{
				ERigBoneType BoneType = ERigBoneType::User;

				if(InHierarchy)
				{
					const FRigBoneElement* BoneElement = InHierarchy->Find<FRigBoneElement>(InKey.GetElement());
					if(BoneElement)
					{
						BoneType = BoneElement->BoneType;
					}
				}

				switch (BoneType)
				{
					case ERigBoneType::Imported:
					{
						Brush = BoneImportedBrush;
						break;
					}
					case ERigBoneType::User:
					default:
					{
						Brush = BoneUserBrush;
						break;
					}
				}

				break;
			}
			case ERigElementType::Physics:
			{
				Brush = PhysicsBrush;
				break;
			}
			case ERigElementType::Reference:
			case ERigElementType::Socket:
			{
				Brush = SocketOpenBrush;

				if(UControlRig* ControlRig = Cast<UControlRig>(InHierarchy->GetOuter()))
				{
					if(const FRigElementKey* ConnectorKey = ControlRig->GetElementKeyRedirector().FindReverse(InKey.GetElement()))
					{
						if(ConnectorKey->Type == ERigElementType::Connector)
						{
							Brush = SocketClosedBrush;
						}
					}
				}

				if(const FRigSocketElement* Socket = InHierarchy->Find<FRigSocketElement>(InKey.GetElement()))
				{
					Color = Socket->GetColor(InHierarchy);
				}
				break;
			}
			case ERigElementType::Connector:
			{
				Brush = PrimaryConnectorBrush;
				if(const FRigConnectorElement* Connector = InHierarchy->Find<FRigConnectorElement>(InKey.GetElement()))
				{
					if(!Connector->IsPrimary())
					{
						Brush = Connector->IsOptional() ? OptionalConnectorBrush : SecondaryConnectorBrush;
					}
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}
	else if(InKey.IsComponent())
	{
		if(const FRigBaseComponent* Component = InHierarchy->FindComponent(InKey.GetComponent()))
		{
			Brush = Component->GetIconForUI().GetIcon();
		}
	}

	return TPair<const FSlateBrush*, FSlateColor>(Brush, Color);
}

FLinearColor SRigHierarchyItem::GetColorForControlType(ERigControlType InControlType, UEnum* InControlEnum)
{
	FEdGraphPinType PinType;
	switch(InControlType)
	{
		case ERigControlType::Bool:
		{
			PinType = RigVMTypeUtils::PinTypeFromCPPType(RigVMTypeUtils::BoolTypeName, nullptr);
			break;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			PinType = RigVMTypeUtils::PinTypeFromCPPType(RigVMTypeUtils::FloatTypeName, nullptr);
			break;
		}
		case ERigControlType::Integer:
		{
			if(InControlEnum)
			{
				PinType = RigVMTypeUtils::PinTypeFromCPPType(NAME_None, InControlEnum);
			}
			else
			{
				PinType = RigVMTypeUtils::PinTypeFromCPPType(RigVMTypeUtils::Int32TypeName, nullptr);
			}
			break;
		}
		case ERigControlType::Vector2D:
		{
			UScriptStruct* Struct = TBaseStructure<FVector2D>::Get(); 
			PinType = RigVMTypeUtils::PinTypeFromCPPType(*RigVMTypeUtils::GetUniqueStructTypeName(Struct), Struct);
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		{
			UScriptStruct* Struct = TBaseStructure<FVector>::Get(); 
			PinType = RigVMTypeUtils::PinTypeFromCPPType(*RigVMTypeUtils::GetUniqueStructTypeName(Struct), Struct);
			break;
		}
		case ERigControlType::Rotator:
		{
			UScriptStruct* Struct = TBaseStructure<FRotator>::Get(); 
			PinType = RigVMTypeUtils::PinTypeFromCPPType(*RigVMTypeUtils::GetUniqueStructTypeName(Struct), Struct);
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		default:
		{
			UScriptStruct* Struct = TBaseStructure<FTransform>::Get(); 
			PinType = RigVMTypeUtils::PinTypeFromCPPType(*RigVMTypeUtils::GetUniqueStructTypeName(Struct), Struct);
			break;
		}
	}
	const UControlRigGraphSchema* Schema = GetDefault<UControlRigGraphSchema>();
	return Schema->GetPinTypeColor(PinType);
}

void SRigHierarchyItem::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	// for now only allow enter
	// because it is important to keep the unique names per pose
	if (InCommitType == ETextCommit::OnEnter)
	{
		FString NewName = InText.ToString();
		const FRigHierarchyKey OldKey = WeakRigTreeElement.Pin()->Key;

		const FName NewSanitizedName = Delegates.HandleRenameElement(OldKey, NewName);
		if (NewSanitizedName.IsNone())
		{
			return;
		}
		NewName = NewSanitizedName.ToString();

		if (WeakRigTreeElement.IsValid())
		{
			WeakRigTreeElement.Pin()->Key.SetName(*NewName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
