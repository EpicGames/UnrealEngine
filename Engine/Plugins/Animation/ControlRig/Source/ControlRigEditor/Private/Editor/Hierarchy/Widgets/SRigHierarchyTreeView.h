// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"
#include "Editor/Hierarchy/RigHierarchyTreeDelegates.h"
#include "Editor/Hierarchy/RigHierarchyTreePersistentStateDelegates.h"
#include "Layout/Geometry.h"
#include "Widgets/Views/STreeView.h"

class FRigHierarchyTreeElement;
class IControlRigBaseEditor;
class SRigHierarchyItem;
class SRigHierarchyTreeView;

#define UE_API CONTROLRIGEDITOR_API

class SRigHierarchyTreeView : public STreeView<TSharedPtr<FRigHierarchyTreeElement>>
{
	using FOnRigHierarchyTreePersistentStateRestored = UE::ControlRigEditor::FOnRigHierarchyTreePersistentStateRestored;

public:

	SLATE_BEGIN_ARGS(SRigHierarchyTreeView)
		: _AutoScrollEnabled(false)
		, _PopulateOnConstruct(false)
		, _ForModalWindow(false)
	{}
		SLATE_ARGUMENT(FRigHierarchyTreeDelegates, RigTreeDelegates)
		SLATE_ARGUMENT(bool, AutoScrollEnabled)
		SLATE_ARGUMENT(bool, PopulateOnConstruct)

		/** Disables features that don't work in modal windows due to the absence of the game ticker and frame numbers */
		SLATE_ARGUMENT(bool, ForModalWindow)

	SLATE_END_ARGS()

	virtual ~SRigHierarchyTreeView();

	/** 
	 * Constructs this widget 
	 * 
	 * @param InArgs				Slate Arguments
	 * @param InViewName			The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor
	 * @param InControlRigEditor	(optional) The editor that displays this widget. When set, supends details updates during tree refresh.
	 */
	void Construct(
		const FArguments& InArgs, 		
		const FName& InViewName,
		const TSharedPtr<IControlRigBaseEditor>& InControlRigEditor = nullptr);

	/** Performs auto scroll */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		FReply Reply = STreeView<TSharedPtr<FRigHierarchyTreeElement>>::OnFocusReceived(MyGeometry, InFocusEvent);

		LastClickCycles = FPlatformTime::Cycles();

		return Reply;
	}

	uint32 LastClickCycles = 0;

	/** Save a snapshot of the internal map that tracks item expansion before tree reconstruction */
	UE_DEPRECATED(5.8, "Should not be used anymore. The tree uses the RigHierarchyTreePersistentStateStore to retain its state")
	void SaveAndClearSparseItemInfos()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Only save the info if there is something to save (do not overwrite info with an empty map)
		if (!SparseItemInfos.IsEmpty())
		{
			OldSparseItemInfos = SparseItemInfos;
		}
		ClearExpandedItems();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Restore the expansion infos map from the saved snapshot after tree reconstruction */
	UE_DEPRECATED(5.8, "Should not be used anymore. The tree uses the RigHierarchyTreePersistentStateStore to retain its state")
	void RestoreSparseItemInfos(TSharedPtr<FRigHierarchyTreeElement> ItemPtr)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (const auto& Pair : OldSparseItemInfos)
		{
			if (Pair.Key->Key == ItemPtr->Key)
			{
				// the SparseItemInfos now reference the new element, but keep the same expansion state
				SparseItemInfos.Add(ItemPtr, Pair.Value);
				break;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	TSharedPtr<FRigHierarchyTreeElement> FindElement(const FRigHierarchyKey& InElementKey) const;
	static TSharedPtr<FRigHierarchyTreeElement> FindElement(const FRigHierarchyKey& InElementKey, TSharedPtr<FRigHierarchyTreeElement> CurrentItem);
	bool AddElement(FRigHierarchyKey InKey, FRigHierarchyKey InParentKey = FRigHierarchyKey());
	bool AddElement(const FRigBaseElement* InElement);
	bool AddComponent(const FRigBaseComponent* InComponent);
	void AddSpacerElement();
	bool ReparentElement(FRigHierarchyKey InKey, FRigHierarchyKey InParentKey);
	bool RemoveElement(FRigHierarchyKey InKey);
	void RefreshTreeView(const bool bRebuildContent = true, const FOnRigHierarchyTreePersistentStateRestored& OnStateRestored = FOnRigHierarchyTreePersistentStateRestored());
	void SetExpansionRecursive(TSharedPtr<FRigHierarchyTreeElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);
	TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FRigHierarchyTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable, bool bPinned);
	void HandleGetChildrenForTree(TSharedPtr<FRigHierarchyTreeElement> InItem, TArray<TSharedPtr<FRigHierarchyTreeElement>>& OutChildren);
	void OnElementKeyTagDragDetected(const FRigElementKey& InDraggedTag);

	TArray<FRigHierarchyKey> GetSelectedKeys() const;
	const TArray<TSharedPtr<FRigHierarchyTreeElement>>& GetRootElements() const { return RootElements; }
	FRigHierarchyTreeDelegates& GetRigTreeDelegates() { return Delegates; }

	/** Given a position, return the item under that position. If nothing is there, return null. */
	const TSharedPtr<FRigHierarchyTreeElement>* FindItemAtPosition(FVector2D InScreenSpacePosition) const;

private:
	/** Addds a connector tag to the tree elements */
	void AddConnectorTagToElements(
		const TArray<TSharedRef<FRigHierarchyTreeElement>> TargetElements,
		const FRigConnectorElement& ConnectorElement,
		const UControlRig& ControlRig);
	
	/** Adds a warning tag to the tree element indicating the connector could not be resolved */
	void AddConnectorResolveWarningTagToElement(
		const TSharedRef<FRigHierarchyTreeElement> TreeElement,
		const FRigConnectorElement& ConnectorElement,
		const UControlRig& ControlRig);

	/** A temporary snapshot of the SparseItemInfos in STreeView, used during RefreshTreeView() */
	UE_DEPRECATED(5.8, "Use RigHierarchyTreePersistentStateStore to retain the tree state")
	TSparseItemMap OldSparseItemInfos;

	/** Backing array for tree view */
	TArray<TSharedPtr<FRigHierarchyTreeElement>> RootElements;
	
	/** A map for looking up items based on their key */
	TMap<FRigHierarchyKey, TSharedPtr<FRigHierarchyTreeElement>> ElementMap;

	/** A map for looking up a parent based on their key */
	TMap<FRigHierarchyKey, FRigHierarchyKey> ParentMap;

	FRigHierarchyTreeDelegates Delegates;

	bool bAutoScrollEnabled;
	FVector2D LastMousePosition;
	double TimeAtMousePosition;

	/** True when rebuild content is requested but not yet carried out */
	bool bRebuildContentRequested = false;

	/** The editor that displays this widget */
	TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;

	/** The name of this view to clearly identify it amongst other rig hierarchies in the control rig editor */
	FName ViewName;

	/** When true the tree is displayed in a modal window and cannot use FTSTicker anywhere */
	bool bForModalWindow = false;

	friend class SRigHierarchy;
};

#undef UE_API
