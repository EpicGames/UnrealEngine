// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ModularRigEventQueueCommands.h"

#define LOCTEXT_NAMESPACE "ModularRigEventQueueCommands"

void FModularRigEventQueueCommands::RegisterCommands()
{
	UI_COMMAND(FocusOnSelection, "Focus On Selection", "Finds the selected event's module in the modular rig tree.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
