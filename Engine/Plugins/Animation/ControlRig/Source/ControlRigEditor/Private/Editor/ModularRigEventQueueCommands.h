// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class CONTROLRIGEDITOR_API FModularRigEventQueueCommands : public TCommands<FModularRigEventQueueCommands>
{
public:
	FModularRigEventQueueCommands() : TCommands<FModularRigEventQueueCommands>
	(
		"FModularRigEventQueue",
		NSLOCTEXT("Contexts", "ModularRig", "Event Queue"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		"ControlRigEditorStyle" // Icon Style Set
	)
	{}
	
	/** Focuses on a selected event's module */
	TSharedPtr< FUICommandInfo > FocusOnSelection;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
