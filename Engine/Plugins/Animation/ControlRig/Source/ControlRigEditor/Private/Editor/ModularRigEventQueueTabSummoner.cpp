// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ModularRigEventQueueTabSummoner.h"
#include "Editor/SModularRigEventQueueView.h"
#include "Editor/ControlRigEditor.h"

#define LOCTEXT_NAMESPACE "ModularRigEventQueueTabSummoner"

FModularRigEventQueueTabSummoner::FModularRigEventQueueTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("ModularRigEventQueueTabLabel", "Event Queue");
	TabIcon = FSlateIcon(TEXT("RigVMEditorStyle"), "ExecutionStack.TabIcon"); // use the same icon as for the execution stack

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ModularRigEventQueue_ViewMenu_Desc", "Event Queue");
	ViewMenuTooltip = LOCTEXT("ModularRigEventQueue_ViewMenu_ToolTip", "Show the Event Queue tab");
}

TSharedRef<SWidget> FModularRigEventQueueTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SModularRigEventQueueView, ControlRigEditor.Pin().ToSharedRef());
}

#undef LOCTEXT_NAMESPACE 
