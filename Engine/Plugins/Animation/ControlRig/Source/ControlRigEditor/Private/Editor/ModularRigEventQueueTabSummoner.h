// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define UE_API CONTROLRIGEDITOR_API

class IControlRigBaseEditor;

struct FModularRigEventQueueTabSummoner : public FWorkflowTabFactory
{
public:

	static inline const FLazyName TabID = FLazyName(TEXT("Event Queue"));
	
public:
	UE_API FModularRigEventQueueTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor);
	
	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;
};

#undef UE_API
