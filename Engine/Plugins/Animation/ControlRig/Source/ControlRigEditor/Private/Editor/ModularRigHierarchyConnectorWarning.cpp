// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularRigHierarchyConnectorWarning.h"

#include "Editor/ControlRigEditor.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyTreeElement.h"
#include "Editor/Hierarchy/Models/ModularRigHierarchyViewModel.h"
#include "Editor/RigVMEditor.h"
#include "Rigs/RigConnectionRules.h"
#include "ModularRig.h"

#define LOCTEXT_NAMESPACE "SModularRigHierarchyTreeView"

namespace UE::ControlRigEditor
{
	TSharedPtr<FModularRigHierarchyConnectorWarning> FModularRigHierarchyConnectorWarning::TryCreate(
		const TSharedRef<IControlRigBaseEditor> InControlRigEditor,
		const FRigElementKey& InConnectorKey,
		const FName& InTargetModuleName)
	{
		const UModularRig* ModularRig = Cast<UModularRig>(InControlRigEditor->GetControlRig());
		const URigHierarchy* Hierarchy = InControlRigEditor->GetHierarchyBeingDebugged();
		if (ModularRig &&
			Hierarchy)
		{
			const FRigModuleInstance* Module = ModularRig->FindModule(InTargetModuleName);
			const FRigConnectorElement* ConnectorElement = Cast<FRigConnectorElement>(Hierarchy->Find(InConnectorKey));
			
			if (Module &&
				ConnectorElement)
			{
				const TSharedRef<FModularRigHierarchyConnectorWarning> Instance =
					MakeShared<FModularRigHierarchyConnectorWarning>(
						FModularRigHierarchyConnectorWarning::FPrivateToken{},
						*ModularRig,
						*Hierarchy,
						*ConnectorElement,
						*Module);

				if (Instance->Flags != EModularRigHierarchyConnectorWarningFlags::NoWarning)
				{
					return Instance;
				}
			}
		}

		return nullptr;
	}

	FModularRigHierarchyConnectorWarning::FModularRigHierarchyConnectorWarning(
		FModularRigHierarchyConnectorWarning::FPrivateToken,
		const UModularRig& ModularRig,
		const URigHierarchy& Hierarchy,
		const FRigConnectorElement& Connector,
		const FRigModuleInstance& Module)
	{
		Initialize(ModularRig, Hierarchy, Connector, Module);

		UpdateTooltipText(ModularRig, Hierarchy, Connector, Module);
		UpdateBrush();
	}

	void FModularRigHierarchyConnectorWarning::Initialize(
		const UModularRig& ModularRig,
		const URigHierarchy& Hierarchy,
		const FRigConnectorElement& Connector,
		const FRigModuleInstance& Module)
	{
		CurrentTargets = ModularRig.GetModularRigModel().Connections.FindTargetsFromConnector(Connector.GetKey());

		CurrentTargets.RemoveAll([](const FRigElementKey& Target)
			{
				return !Target.IsValid();
			});

		if (Connector.IsPrimary())
		{			
			// Following warnings should only show for the primary connector
			if (!ModularRig.GetModularRigModel().FindModule(Module.Name))
			{
				Flags |= EModularRigHierarchyConnectorWarningFlags::NotInModularRigModel;
			}

			if (!ModularRig.FindModule(Module.Name))
			{
				Flags |= EModularRigHierarchyConnectorWarningFlags::NotInModularRig;
			}

			if (const TScriptInterface<const IRigVMEditorAssetInterface> ModuleAsset = GetModuleRigVMAsset(Module))
			{
				const bool bMarkSubjectAsInvalid = ModuleAsset->GetAssetVariant().Tags.ContainsByPredicate(
					[](const FRigVMTag& AssetTag)
					{
						return AssetTag.bMarksSubjectAsInvalid;
					});

				if (bMarkSubjectAsInvalid)
				{
					Flags |= EModularRigHierarchyConnectorWarningFlags::MarkedAsInvalid;
				}
			}
		}

		const bool bConnected =	ModularRig.GetModularRigModel().Connections.HasConnection(Connector.GetKey());
		if (!Connector.IsOptional() &&
			!bConnected)
		{
			Flags |= EModularRigHierarchyConnectorWarningFlags::NotConnected;
		}

		AddFlagsFromArraySizeRule(ModularRig, Hierarchy, Connector, Module);
	}

	void FModularRigHierarchyConnectorWarning::AddFlagsFromArraySizeRule(
		const UModularRig& ModularRig,
		const URigHierarchy& Hierarchy,
		const FRigConnectorElement& Connector,
		const FRigModuleInstance& Module)
	{
		if (!Connector.Settings.bIsArray)
		{
			return;
		}

		for (const FRigConnectionRuleStash& Stash : Connector.Settings.Rules)
		{
			if (!Stash.IsValid())
			{
				continue;
			}
			TSharedPtr<FStructOnScope> Storage;

			const FRigConnectionRule* Rule = Stash.Get(Storage);
			if (Rule &&
				Rule->GetScriptStruct() == FRigArraySizeConnectionRule::StaticStruct())
			{
				const FRigArraySizeConnectionRule* ArraySizeRule = static_cast<const FRigArraySizeConnectionRule*>(Rule);
				if (const TOptional<int32> OtherMinArraySize = ArraySizeRule->GetMinNumConnections())
				{
					if (MinArraySize.IsSet())
					{
						MinArraySize = FMath::Min(MinArraySize.GetValue(), OtherMinArraySize.GetValue());
						Flags |= EModularRigHierarchyConnectorWarningFlags::MultipleArraySizeRules;
					}
					else if (OtherMinArraySize.IsSet())
					{
						MinArraySize = OtherMinArraySize;
					}
				}

				if (const TOptional<int32> OtherMaxArraySize = ArraySizeRule->GetMaxNumConnections())
				{
					if (MaxArraySize.IsSet())
					{
						MaxArraySize = FMath::Max(MaxArraySize.GetValue(), OtherMaxArraySize.GetValue());
						Flags |= EModularRigHierarchyConnectorWarningFlags::MultipleArraySizeRules;
					}
					else if (OtherMaxArraySize.IsSet())
					{
						MaxArraySize = OtherMaxArraySize;
					}
				}
			}
		}

		if (MinArraySize.IsSet() &&
			CurrentTargets.Num() < MinArraySize.GetValue())
		{
			Flags |= EModularRigHierarchyConnectorWarningFlags::ExeedsMinArraySizeRule;
		}

		if (MaxArraySize.IsSet() &&
			CurrentTargets.Num() > MaxArraySize.GetValue())
		{
			Flags |= EModularRigHierarchyConnectorWarningFlags::ExeedsMaxArraySizeRule;
		}
	}

	void FModularRigHierarchyConnectorWarning::UpdateTooltipText(
		const UModularRig& ModularRig,
		const URigHierarchy& Hierarchy,
		const FRigConnectorElement& Connector,
		const FRigModuleInstance& Module)
	{
		const TScriptInterface<const IRigVMEditorAssetInterface> ModuleAsset = GetModuleRigVMAsset(Module);
		if (!ModuleAsset)
		{
			return;
		}

		TArray<FString> ToolTipArray;

		// Marked as invalid
		if (EnumHasAnyFlags(Flags, EModularRigHierarchyConnectorWarningFlags::MarkedAsInvalid))
		{
			for (const FRigVMTag& AssetTag : ModuleAsset->GetAssetVariant().Tags)
			{
				if (AssetTag.bMarksSubjectAsInvalid)
				{
					ToolTipArray.Add(FString::Printf(TEXT("%s: %s"), *AssetTag.Label, *AssetTag.ToolTip.ToString()));
					ToolTipArray.Add(LOCTEXT("MarkedAsInvalidTooltip", "Right click on the module to swap it to a newer variant.").ToString());
				}
			}
		}

		// Missing module 
		if (EnumHasAnyFlags(Flags, 
			EModularRigHierarchyConnectorWarningFlags::NotInModularRig | 
			EModularRigHierarchyConnectorWarningFlags::NotInModularRigModel))
		{
			const FName& ModuleName = Module.Name;
			if (!ModularRig.FindModule(ModuleName))
			{
				if (const FRigModuleReference* Reference = ModularRig.GetModularRigModel().FindModule(ModuleName))
				{
					const FText FormatText = LOCTEXT("MissingModuleTooltip", "Could not find RigModule class {0}");
					const FText TooltipText = FText::Format(FormatText, FText::FromString(*Reference->ControlRigAssetReference.GetPathName()));

					ToolTipArray.Add(TooltipText.ToString());
				}
			}
		}

		// Not connected
		if (EnumHasAnyFlags(Flags, EModularRigHierarchyConnectorWarningFlags::NotConnected))
		{
			const FText TooltipText = LOCTEXT("NotConnectedTooltip", "Missing Connector");

			ToolTipArray.Add(TooltipText.ToString());
		}

		// Ambigous Array Size Rules
		if (EnumHasAnyFlags(Flags, EModularRigHierarchyConnectorWarningFlags::MultipleArraySizeRules))
		{
			const FText TooltipText = LOCTEXT("MultipleArraySizeRulesTooltip", "Found more than one Array Size Rule, using the largest extents.");

			ToolTipArray.Add(TooltipText.ToString());
		}

		// Array Size Rule
		if (EnumHasAnyFlags(Flags, EModularRigHierarchyConnectorWarningFlags::ExeedsMinArraySizeRule) &&
			MinArraySize.IsSet())
		{
			const FText TooltipText = FText::Format(
				LOCTEXT("ExceedsArrayMinTooltip", "Expects min {0} {1}|plural(one=Connection,other=Connections) but got {2}"),
				MinArraySize.GetValue(), 
				MinArraySize.GetValue(),
				CurrentTargets.Num());

			ToolTipArray.Add(TooltipText.ToString());
		}
		else if (EnumHasAnyFlags(Flags, EModularRigHierarchyConnectorWarningFlags::ExeedsMaxArraySizeRule) &&
			MaxArraySize.IsSet())
		{
			const FText TooltipText = FText::Format(
				LOCTEXT("ExceedsArrayMaxTooltip", "Expects max {0} {1}|plural(one=Connection,other=Connections) but got {2}"), 
				MaxArraySize.GetValue(),
				MaxArraySize.GetValue(),
				CurrentTargets.Num());

			ToolTipArray.Add(TooltipText.ToString());
		}

		Tooltip = FText::FromString(FString::Join(ToolTipArray, TEXT("\n")));
	}

	void FModularRigHierarchyConnectorWarning::UpdateBrush()
	{
		const bool bShowError = EnumHasAnyFlags(Flags,
			EModularRigHierarchyConnectorWarningFlags::MarkedAsInvalid | 
			EModularRigHierarchyConnectorWarningFlags::NotInModularRigModel);

		const bool bShowWarning = EnumHasAnyFlags(Flags,
			EModularRigHierarchyConnectorWarningFlags::NotInModularRig |
			EModularRigHierarchyConnectorWarningFlags::NotConnected |
			EModularRigHierarchyConnectorWarningFlags::ExeedsMinArraySizeRule |
			EModularRigHierarchyConnectorWarningFlags::ExeedsMaxArraySizeRule |
			EModularRigHierarchyConnectorWarningFlags::MultipleArraySizeRules);

		if (bShowError)
		{
			Brush = FAppStyle::Get().GetBrush("Icons.ErrorWithColor");
		}
		else if (bShowWarning)
		{
			Brush = FAppStyle::Get().GetBrush("Icons.WarningWithColor");
		}
	}

	const TScriptInterface<const IRigVMEditorAssetInterface> FModularRigHierarchyConnectorWarning::GetModuleRigVMAsset(const FRigModuleInstance& Module)
	{
		const UControlRig* ModuleRig = Module.GetRig();
		if (ModuleRig &&
			ModuleRig->GetClass()->ClassGeneratedBy &&
			ModuleRig->GetClass()->ClassGeneratedBy->Implements<URigVMEditorAssetInterface>())
		{
			return TScriptInterface<const IRigVMEditorAssetInterface>(ModuleRig->GetClass()->ClassGeneratedBy);
		}

		if (ModuleRig)
		{
			if (TScriptInterface<IRigVMRuntimeAssetInterface> GeneratedByAsset = ModuleRig->GetGeneratedByAsset())
			{
				return TScriptInterface<const IRigVMEditorAssetInterface>(GeneratedByAsset->GetEditorOnlyData());
			}
		}

		return nullptr;
	}
}

#undef LOCTEXT_NAMESPACE
