// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatform.h"
#include "Templates/SharedPointer.h"
#include "UObject/ScriptInterface.h"

class IControlRigBaseEditor;
class IRigVMEditorAssetInterface;
class UModularRig;
class URigHierarchy;
struct FRigConnectorElement;
struct FRigElementKey;
struct FRigModuleInstance;
struct FSlateBrush;

namespace UE::ControlRigEditor
{
	class FModularRigHierarchyViewModel;

	/** Flags for different warnings */
	enum class EModularRigHierarchyConnectorWarningFlags : uint8
	{
		/** There is no warning for this element */
		NoWarning = 0x00,

		/** The module is flagged as invalid in asset variant tags */
		MarkedAsInvalid = 0x01,

		/** The module cannot be found in the modular rig model */
		NotInModularRigModel = 0x01 << 1,

		/** The module cannot be found in the modular rig */
		NotInModularRig = 0x01 << 2,

		/** The module is not connected */
		NotConnected = 0x01 << 3,

		/** Not enough connections given the array size rule */
		ExeedsMinArraySizeRule = 0x01 << 4,

		/** Too many connections given the array size rule */
		ExeedsMaxArraySizeRule = 0x01 << 5,

		/** Ambigous array size rules */
		MultipleArraySizeRules = 0x01 << 6
	};
	ENUM_CLASS_FLAGS(EModularRigHierarchyConnectorWarningFlags);

	/** Component to create consistent warnings for a Modular Rig Hierarchy Tree Element */
	class FModularRigHierarchyConnectorWarning
		: FNoncopyable
	{	
		/** Prevents from public construction, use TryCreate instead */
		struct FPrivateToken { explicit FPrivateToken() = default; };

	public:
		/** Tries to create a warning, returns nullptr if there is no warning for this data */
		static TSharedPtr<FModularRigHierarchyConnectorWarning> TryCreate(
			const TSharedRef<IControlRigBaseEditor> InControlRigEditor,
			const FRigElementKey& InConnectorKey,
			const FName& InTargetModuleName);

		const FText& GetTooltip() const { return Tooltip; }

		const FSlateBrush* GetBrush() const { return Brush; }

		FModularRigHierarchyConnectorWarning(	
			FModularRigHierarchyConnectorWarning::FPrivateToken,
			const UModularRig& ModularRig,
			const URigHierarchy& Hierarchy,
			const FRigConnectorElement& Connector,
			const FRigModuleInstance& Module);

	private:
		/** Initializes all members */
		void Initialize(
			const UModularRig& ModularRig,
			const URigHierarchy& Hierarchy,
			const FRigConnectorElement& Connector,
			const FRigModuleInstance& Module);

		/** Adds flags resulting from the array size rule if any is present */
		void AddFlagsFromArraySizeRule(
			const UModularRig& ModularRig,
			const URigHierarchy& Hierarchy,
			const FRigConnectorElement& Connector,
			const FRigModuleInstance& Module);

		/** Updates the tooltip text member */
		void UpdateTooltipText(
			const UModularRig& ModularRig,
			const URigHierarchy& Hierarchy,
			const FRigConnectorElement& Connector,
			const FRigModuleInstance& Module);

		/** Inits the brush member */
		void UpdateBrush();

		/** Returns the Rig VM Asset of the Module the warning relates to */
		const TScriptInterface<const IRigVMEditorAssetInterface> GetModuleRigVMAsset(const FRigModuleInstance& Module);

		/** Warnings flags */
		EModularRigHierarchyConnectorWarningFlags Flags = EModularRigHierarchyConnectorWarningFlags::NoWarning;

		/** If set, the min allowed array size */
		TOptional<int32> MinArraySize;

		/** If set, the max allowed array size */
		TOptional<int32> MaxArraySize;

		/** The tooltip text for this warning */
		FText Tooltip;

		/** The actual icon brush, can be the CustomIconBrush or a common slate brush */
		const FSlateBrush* Brush = nullptr;		

		/** The current targets of the connector of this element */
		TArray<FRigElementKey> CurrentTargets;
	};
}
