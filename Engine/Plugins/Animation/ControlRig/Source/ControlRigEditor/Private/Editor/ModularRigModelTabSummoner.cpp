// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/ModularRigModelTabSummoner.h"
#include "Editor/Hierarchy/Views/SModularRigHierarchy.h"
#include "ControlRigEditorStyle.h"
#include "Editor/ControlRigEditor.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "ModularRigHierarchyTabSummoner"

const FName FModularRigModelTabSummoner::TabID(TEXT("ModularRigModel"));

FModularRigModelTabSummoner::FModularRigModelTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, WeakControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("ModularRigHierarchyTabLabel", "Module Hierarchy");
	TabIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ModularRigHierarchy.TabIcon");

	ViewMenuDescription = LOCTEXT("ModularRigHierarchy_ViewMenu_Desc", "Module Hierarchy");
	ViewMenuTooltip = LOCTEXT("ModularRigHierarchy_ViewMenu_ToolTip", "Show the Module Hierarchy tab");
}

FTabSpawnerEntry& FModularRigModelTabSummoner::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* CurrentApplicationMode) const
{
	FTabSpawnerEntry& SpawnerEntry = FWorkflowTabFactory::RegisterTabSpawner(InTabManager, CurrentApplicationMode);

	SpawnerEntry.SetReuseTabMethod(FOnFindTabToReuse::CreateLambda([](const FTabId& InTabId) ->TSharedPtr<SDockTab> {
	
		return TSharedPtr<SDockTab>();

	}));

	return SpawnerEntry;
}

TSharedRef<SWidget> FModularRigModelTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakControlRigEditor.IsValid() ? WeakControlRigEditor.Pin() : nullptr;
	if (ControlRigEditor.IsValid())
	{
		constexpr const TCHAR* ViewName = TEXT("ModularRigHierarchy");

		ControlRigEditor->IncreaseModularRigHierarchyTabCount();
		return SNew(SModularRigHierarchy, ControlRigEditor.ToSharedRef(), ViewName);
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SDockTab> FModularRigModelTabSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakControlRigEditor.IsValid() ? WeakControlRigEditor.Pin() : nullptr;

	TSharedRef<SDockTab>  DockTab = FWorkflowTabFactory::SpawnTab(Info);
	TWeakPtr<SDockTab> WeakDockTab = DockTab;
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateLambda([WeakDockTab, CapturedWeakControlRigEditor = WeakControlRigEditor]()
    {
		int32 HierarchyTabCount = 0;
		if (TSharedPtr<SDockTab> SharedDocTab = WeakDockTab.Pin())
		{
			if(SWidget* Content = &SharedDocTab->GetContent().Get())
			{
				SModularRigHierarchy* RigHierarchy = (SModularRigHierarchy*)Content;
				const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = CapturedWeakControlRigEditor.IsValid() ? CapturedWeakControlRigEditor.Pin() : nullptr;
				if (ControlRigEditor.IsValid())
				{
					HierarchyTabCount = ControlRigEditor->GetModularRigHierarchyTabCount();
				}
				else
				{
					return true; // if the editor has been already destroyed, allow closing the tab, so it does not stay alive and crash next frame
				}
			}
		}
		return HierarchyTabCount > 0;
    }));
	DockTab->SetOnTabClosed( SDockTab::FOnTabClosedCallback::CreateLambda([CapturedWeakControlRigEditor = WeakControlRigEditor](TSharedRef<SDockTab> DockTab)
	{
		if(SWidget* Content = &DockTab->GetContent().Get())
		{
			SModularRigHierarchy* RigHierarchy = (SModularRigHierarchy*)Content;
			const TSharedPtr<IControlRigBaseEditor> ControlRigEditor = CapturedWeakControlRigEditor.IsValid() ? CapturedWeakControlRigEditor.Pin() : nullptr;
			if (ControlRigEditor.IsValid())
			{
				ControlRigEditor->DecreaseModularRigHierarchyTabCount();
			}
		}
	}));
	return DockTab;
}

#undef LOCTEXT_NAMESPACE 
