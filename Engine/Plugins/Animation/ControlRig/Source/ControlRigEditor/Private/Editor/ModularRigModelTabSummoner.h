// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class IControlRigBaseEditor;

struct FModularRigModelTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FModularRigModelTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor);
	
	virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const;
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;
};
