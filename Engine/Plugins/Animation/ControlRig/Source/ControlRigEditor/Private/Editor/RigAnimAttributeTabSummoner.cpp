// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigAnimAttributeTabSummoner.h"
#include "Editor/SControlRigAnimAttributeView.h"

#include "Editor/ControlRigEditor.h"

#define LOCTEXT_NAMESPACE "RigAnimAttributeTabSummoner"

const FName FRigAnimAttributeTabSummoner::TabID(TEXT("RigAnimAttribute"));

FRigAnimAttributeTabSummoner::FRigAnimAttributeTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("RigAnimAttributeTabLabel", "Animation Attributes");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimGraph.Attribute.Attributes.Icon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigAnimAttribute_ViewMenu_Desc", "Animation Attribute");
	ViewMenuTooltip = LOCTEXT("RigAnimAttribute_ViewMenu_ToolTip", "Show the Animation Attribute tab");
}

TSharedRef<SWidget> FRigAnimAttributeTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SControlRigAnimAttributeView, ControlRigEditor.Pin().ToSharedRef());
}

#undef LOCTEXT_NAMESPACE 
