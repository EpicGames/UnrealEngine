// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigCurveContainerCommands.h"

#define LOCTEXT_NAMESPACE "CurveContainerCommands"

void FCurveContainerCommands::RegisterCommands()
{
	UI_COMMAND( AddCurve, "Add Curve", "Add a curve to the ControlRig", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
