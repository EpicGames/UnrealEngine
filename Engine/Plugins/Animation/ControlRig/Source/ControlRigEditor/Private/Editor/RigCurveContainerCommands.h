// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

/**
* Class containing commands for curve viewer actions
*/
class FCurveContainerCommands : public TCommands<FCurveContainerCommands>
{
public:
	FCurveContainerCommands()
	: TCommands<FCurveContainerCommands>
		(
			TEXT("CurveContainer"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "CurveContainer", "Curve Viewer"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Add curve */
	TSharedPtr< FUICommandInfo > AddCurve;
};


