// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigCurveContainerTabSummoner.h"
#include "Editor/SRigCurveContainer.h"
#include "ControlRigEditorStyle.h"
#include "Editor/ControlRigEditor.h"

#define LOCTEXT_NAMESPACE "RigCurveContainerTabSummoner"

const FName FRigCurveContainerTabSummoner::TabID(TEXT("RigCurveContainer"));

FRigCurveContainerTabSummoner::FRigCurveContainerTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("RigCurveContainerTabLabel", "Curve Container");
	TabIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "CurveContainer.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigCurveContainer_ViewMenu_Desc", "Curve Container");
	ViewMenuTooltip = LOCTEXT("RigCurveContainer_ViewMenu_ToolTip", "Show the Rig Curve Container tab");
}

TSharedRef<SWidget> FRigCurveContainerTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRigCurveContainer, ControlRigEditor.Pin().ToSharedRef());
}

#undef LOCTEXT_NAMESPACE 
