// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigHierarchyTabSummoner.h"

#include "ControlRigEditorStyle.h"
#include "Editor/ControlRigEditor.h"
#include "Editor/Hierarchy/Views/SRigHierarchy.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "RigHierarchyTabSummoner"

const FName FRigHierarchyTabSummoner::TabID(TEXT("RigHierarchy"));

FRigHierarchyTabSummoner::FRigHierarchyTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("RigHierarchyTabLabel", "Rig Hierarchy");
	TabIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "RigHierarchy.TabIcon");

	ViewMenuDescription = LOCTEXT("RigHierarchy_ViewMenu_Desc", "Rig Hierarchy");
	ViewMenuTooltip = LOCTEXT("RigHierarchy_ViewMenu_ToolTip", "Show the Rig Hierarchy tab");
}

FTabSpawnerEntry& FRigHierarchyTabSummoner::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* CurrentApplicationMode) const
{
	FTabSpawnerEntry& SpawnerEntry = FWorkflowTabFactory::RegisterTabSpawner(InTabManager, CurrentApplicationMode);

	SpawnerEntry.SetReuseTabMethod(FOnFindTabToReuse::CreateLambda([](const FTabId& InTabId) ->TSharedPtr<SDockTab> {
	
		return TSharedPtr<SDockTab>();

	}));

	return SpawnerEntry;
}

TSharedRef<SWidget> FRigHierarchyTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	constexpr const TCHAR* ViewName = TEXT("RigHierarchy");

	ControlRigEditor.Pin()->IncreaseRigHierarchyTabCount();
	return SNew(SRigHierarchy, ControlRigEditor.Pin().ToSharedRef(), ViewName);
}

TSharedRef<SDockTab> FRigHierarchyTabSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab>  DockTab = FWorkflowTabFactory::SpawnTab(Info);
	TWeakPtr<SDockTab> WeakDockTab = DockTab;
	DockTab->SetCanCloseTab(SDockTab::FCanCloseTab::CreateLambda([WeakDockTab]()
    {
		int32 HierarchyTabCount = 0;
		if (TSharedPtr<SDockTab> SharedDocTab = WeakDockTab.Pin())
		{
			if(SWidget* Content = &SharedDocTab->GetContent().Get())
			{
				SRigHierarchy* RigHierarchy = (SRigHierarchy*)Content;
				if(IControlRigBaseEditor* ControlRigEditorForTab = RigHierarchy->GetControlRigEditor())
				{
					HierarchyTabCount = ControlRigEditorForTab->GetRigHierarchyTabCount();
				}
				else
				{
					return true; // if the editor has been already destroyed, allow closing the tab, so it does not stay alive and crash next frame
				}
			}
		}
		return HierarchyTabCount > 0;
    }));
	DockTab->SetOnTabClosed( SDockTab::FOnTabClosedCallback::CreateLambda([](TSharedRef<SDockTab> DockTab)
	{
		if(SWidget* Content = &DockTab->GetContent().Get())
		{
			SRigHierarchy* RigHierarchy = (SRigHierarchy*)Content;
			if(IControlRigBaseEditor* ControlRigEditorForTab = RigHierarchy->GetControlRigEditor())
			{
				ControlRigEditorForTab->DecreaseRigHierarchyTabCount();
			}
		}
	}));
	return DockTab;
}

#undef LOCTEXT_NAMESPACE 
