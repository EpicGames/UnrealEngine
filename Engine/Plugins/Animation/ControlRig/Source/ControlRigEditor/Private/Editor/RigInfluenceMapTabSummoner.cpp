// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigInfluenceMapTabSummoner.h"
#include "Editor/Hierarchy/Views/SRigHierarchy.h"
#include "Editor/RigVMEditorStyle.h"
#include "Editor/ControlRigEditor.h"
#include "ControlRigBlueprintLegacy.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#define LOCTEXT_NAMESPACE "RigInfluenceMapTabSummoner"

const FName FRigInfluenceMapTabSummoner::TabID(TEXT("RigInfluenceMap"));

FRigInfluenceMapTabSummoner::FRigInfluenceMapTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("RigInfluenceMapTabLabel", "Rig Influence Map");
	TabIcon = FSlateIcon(FRigVMEditorStyle::Get().GetStyleSetName(), "RigVM.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigInfluenceMap_ViewMenu_Desc", "Rig Influence Map");
	ViewMenuTooltip = LOCTEXT("RigInfluenceMap_ViewMenu_ToolTip", "Show the Rig Influence Map tab");
}

TSharedRef<SWidget> FRigInfluenceMapTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<SKismetInspector> KismetInspector = SNew(SKismetInspector);
#else
	TSharedRef<SRigVMDetailsInspector> KismetInspector = SNew(SRigVMDetailsInspector);
#endif

	if (TSharedPtr<IControlRigBaseEditor> Editor = ControlRigEditor.Pin())
	{
		if (FControlRigAssetInterfacePtr RigBlueprint = Editor->GetControlRigAssetInterface())
		{
			TSharedPtr<FStructOnScope> StructToDisplay = MakeShareable(new FStructOnScope(
				FRigInfluenceMapPerEvent::StaticStruct(), (uint8*)&RigBlueprint->GetInfluences()));
			StructToDisplay->SetPackage(RigBlueprint.GetObject()->GetOutermost());
			KismetInspector->ShowSingleStruct(StructToDisplay);
		}
	}

	return KismetInspector;
}

#undef LOCTEXT_NAMESPACE 
