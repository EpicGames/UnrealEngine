// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigModuleAssetBrowserTabSummoner.h"

#include "IDocumentation.h"
#include "ControlRigEditor.h"
#include "SRigModuleAssetBrowser.h"

#define LOCTEXT_NAMESPACE "RigModuleAssetBrowserTabSummoner"

const FName FRigModuleAssetBrowserTabSummoner::TabID(TEXT("RigModuleAssetBrowser"));

FRigModuleAssetBrowserTabSummoner::FRigModuleAssetBrowserTabSummoner(const TSharedPtr<FAssetEditorToolkit>& InEditorToolkit)
	: FWorkflowTabFactory(TabID, InEditorToolkit)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("RigModuleAssetBrowserTabLabel", "Module Assets");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon");

	ViewMenuDescription = LOCTEXT("RigModuleAssetBrowser_ViewMenu_Desc", "Module Assets");
	ViewMenuTooltip = LOCTEXT("RigModuleAssetBrowser_ViewMenu_ToolTip", "Show the Module Assets Tab");
}

TSharedPtr<SToolTip> FRigModuleAssetBrowserTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("RigModuleAssetBrowserTooltip", "Drag and drop the rig modules into the Modular Rig Hierarchy tab."), NULL, TEXT("Shared/Editors/Persona"), TEXT("RigModuleAssetBrowser_Window"));
}

TSharedRef<SWidget> FRigModuleAssetBrowserTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRigModuleAssetBrowser);
}

TSharedRef<SDockTab> FRigModuleAssetBrowserTabSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	return FWorkflowTabFactory::SpawnTab(Info);
}

#undef LOCTEXT_NAMESPACE 
