// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class IControlRigBaseEditor;

struct FRigModuleAssetBrowserTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
	FRigModuleAssetBrowserTabSummoner(const TSharedPtr<FAssetEditorToolkit>& InEditorToolkit);
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

};
