// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigOverrideListTabSummoner.h"
#include "Editor/RigVMEditorStyle.h"
#include "Editor/ControlRigEditor.h"
#include "ControlRigBlueprintLegacy.h"
#include "Overrides/SOverrideListWidget.h"

#define LOCTEXT_NAMESPACE "RigOverrideListTabSummoner"

const FName FRigOverrideListTabSummoner::TabID(TEXT("OverrideList"));

FRigOverrideListTabSummoner::FRigOverrideListTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("OverrideListTabLabel", "Active Overrides");
	TabIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "DetailsView.OverrideInside.Hovered");
	
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("OverrideList_ViewMenu_Desc", "Active Overrides");
	ViewMenuTooltip = LOCTEXT("OverrideList_ViewMenu_ToolTip", "Show the Active Overrides tab");
}

TSharedRef<SWidget> FRigOverrideListTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	if (TSharedPtr<IControlRigBaseEditor> Editor = ControlRigEditor.Pin())
	{
		if (FControlRigAssetInterfacePtr RigBlueprint = Editor->GetControlRigAssetInterface())
		{
			check(RigBlueprint->IsModularRig());

			return SNew(SOverrideListWidget)
				.SubjectsHash_Raw(RigBlueprint.GetInterface(), &IControlRigEditorAssetInterface::GetOverrideSubjectsHash)
				.Subjects_Raw(RigBlueprint.GetInterface(), &IControlRigEditorAssetInterface::GetOverrideSubjects);
		}
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE 
