// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class IControlRigBaseEditor;

struct FRigOverrideListTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FRigOverrideListTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;
};
