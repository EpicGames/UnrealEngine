// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigValidationTabSummoner.h"
#include "Editor/Hierarchy/Views/SRigHierarchy.h"
#include "ControlRigEditorStyle.h"
#include "Editor/ControlRigEditor.h"
#include "ControlRigBlueprintLegacy.h"
#include "Editor/SControlRigValidationWidget.h"

#define LOCTEXT_NAMESPACE "RigValidationTabSummoner"

const FName FRigValidationTabSummoner::TabID(TEXT("RigValidation"));

FRigValidationTabSummoner::FRigValidationTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory(TabID, InControlRigEditor->GetHostingApp())
	, WeakControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("RigValidationTabLabel", "Rig Validation");
	TabIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "RigValidation.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigValidation_ViewMenu_Desc", "Rig Validation");
	ViewMenuTooltip = LOCTEXT("RigValidation_ViewMenu_ToolTip", "Show the Rig Validation tab");
}

TSharedRef<SWidget> FRigValidationTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	ensure(WeakControlRigEditor.IsValid());
	TSharedRef<IRigVMEditor> EditorRef = WeakControlRigEditor.Pin()->SharedRigVMEditorRef();
	TSharedPtr<FRigVMEditorBase> EditorPtr = StaticCastSharedPtr<FRigVMEditorBase>(EditorRef.ToSharedPtr());
	
	if (TSharedPtr<FRigVMEditorBase> Editor = EditorPtr)
	{
		FControlRigAssetInterfacePtr RigBlueprint(Editor->GetRigVMAssetInterface()->GetObject());
		check(RigBlueprint);
	
		UControlRigValidator* Validator = RigBlueprint->GetValidator();
		check(Validator);

		TSharedRef<SControlRigValidationWidget> ValidationWidget = SNew(SControlRigValidationWidget, Validator);
		Validator->SetControlRig(Cast<UControlRig>(RigBlueprint->GetObjectBeingDebugged()));
		return ValidationWidget;
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE 
