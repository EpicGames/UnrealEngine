// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class IControlRigBaseEditor;

struct FRigValidationTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
public:
	FRigValidationTabSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;
};
