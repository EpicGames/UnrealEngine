// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/SControlRigAnimAttributeView.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor/ControlRigEditor.h"
#include "ControlRigBlueprintLegacy.h"
#include "Engine/Console.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "SAnimAttributeView.h"

#define LOCTEXT_NAMESPACE "SControlRigAnimAttributeView"

static const FName IncomingSnapshotName(TEXT("Incoming Anim Attributes"));
static const FName OutgoingSnapshotName(TEXT("Outgoing Anim Attributes"));
static const TArray<FName> SnapshotNames = {IncomingSnapshotName, OutgoingSnapshotName};

SControlRigAnimAttributeView::~SControlRigAnimAttributeView()
{
	StopObservingCurrentControlRig();
	
	ControlRigBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().RemoveAll(this);
}

void SControlRigAnimAttributeView::Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor)
{
	SAssignNew(AttributeView, SAnimAttributeView);
	
	ChildSlot
	[
		AttributeView.ToSharedRef()
	];

	ControlRigEditor = InControlRigEditor;
	ControlRigBlueprint = ControlRigEditor.Pin()->GetControlRigAssetInterface();
	ControlRigBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().AddRaw(this, &SControlRigAnimAttributeView::HandleSetObjectBeingDebugged);

	if (UControlRig* ControlRig = Cast<UControlRig>(ControlRigBlueprint->GetObjectBeingDebugged()))
	{
		StartObservingNewControlRig(ControlRig);
	}
}

void SControlRigAnimAttributeView::HandleSetObjectBeingDebugged(UObject* InObject)
{
	if(ControlRigBeingDebuggedPtr.Get() == InObject)
	{
		return;
	}

	StopObservingCurrentControlRig();
	
	if(UControlRig* ControlRig = Cast<UControlRig>(InObject))
	{
		StartObservingNewControlRig(ControlRig);
	}
}


void SControlRigAnimAttributeView::StartObservingNewControlRig(UControlRig* InControlRig) 
{
	if (InControlRig)
	{
		InControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
		InControlRig->OnPostForwardsSolve_AnyThread().AddRaw(this, &SControlRigAnimAttributeView::HandleControlRigPostForwardSolve);
		InControlRig->SetEnableAnimAttributeTrace(true);
	}
	
	ControlRigBeingDebuggedPtr = InControlRig;
}

void SControlRigAnimAttributeView::StopObservingCurrentControlRig()
{
	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(UControlRig* ControlRig = ControlRigBeingDebuggedPtr.Get())
		{
			if(!URigVMHost::IsGarbageOrDestroyed(ControlRig))
			{
				ControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
				ControlRig->SetEnableAnimAttributeTrace(false);
			}
		}
	}
	
	ControlRigBeingDebuggedPtr.Reset();
}

void SControlRigAnimAttributeView::HandleControlRigPostForwardSolve(
	UControlRig* InControlRig,
	const FName& InEventName) const
{
	if (IsValid(InControlRig) && ensure(InControlRig == ControlRigBeingDebuggedPtr))
	{
		if (InEventName == FRigUnit_BeginExecution::EventName)
		{
			if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InControlRig->OuterSceneComponent.Get()))
			{
				const UE::Anim::FMeshAttributeContainer& InputSnapshot = InControlRig->InputAnimAttributeSnapshot;
				const UE::Anim::FMeshAttributeContainer& OutputSnapshot = InControlRig->OutputAnimAttributeSnapshot;

				const TArray<TTuple<FName, const UE::Anim::FMeshAttributeContainer&>> Snapshots =
				{
					{TEXT("Input") , InputSnapshot  },
					{TEXT("Output"), OutputSnapshot }
				};
				
				AttributeView->DisplayNewAttributeContainerSnapshots(Snapshots, SkeletalMeshComponent);
				return;
			}
		}
	}

	AttributeView->ClearListView();
}

#undef LOCTEXT_NAMESPACE
