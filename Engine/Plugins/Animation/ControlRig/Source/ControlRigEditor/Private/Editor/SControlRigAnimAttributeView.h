// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SCompoundWidget.h"

class IControlRigEditorAssetInterface;
class IControlRigBaseEditor;
class UControlRig;

class SAnimAttributeView;


class SControlRigAnimAttributeView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SControlRigAnimAttributeView )
	{}
	
	SLATE_END_ARGS()

	~SControlRigAnimAttributeView();
	
	void Construct( const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor);

	void HandleSetObjectBeingDebugged(UObject* InObject);
	
	void StartObservingNewControlRig(UControlRig* InControlRig);
	void StopObservingCurrentControlRig();

	void HandleControlRigPostForwardSolve(UControlRig* InControlRig, const FName& InEventName) const;
	
	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;
	TWeakInterfacePtr<IControlRigEditorAssetInterface> ControlRigBlueprint;
	TWeakObjectPtr<UControlRig> ControlRigBeingDebuggedPtr;
	TSharedPtr<SAnimAttributeView> AttributeView;
};
