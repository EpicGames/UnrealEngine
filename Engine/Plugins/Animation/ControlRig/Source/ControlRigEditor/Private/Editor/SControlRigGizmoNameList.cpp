// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/SControlRigGizmoNameList.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "EdGraphSchema_K2.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "ControlRigBlueprintLegacy.h"

void SControlRigShapeNameList::Construct(const FArguments& InArgs, FRigControlElement* ControlElement, FControlRigAssetInterfacePtr InBlueprint)
{
	TArray<FRigControlElement*> ControlElements;
	ControlElements.Add(ControlElement);
	return Construct(InArgs, ControlElements, InBlueprint);
}

void SControlRigShapeNameList::Construct(const FArguments& InArgs, TArray<FRigControlElement*> ControlElements, FControlRigAssetInterfacePtr InBlueprint)
{
	this->OnGetNameListContent = InArgs._OnGetNameListContent;
	this->ControlKeys.Reset();

	for(FRigControlElement* ControlElement : ControlElements)
	{
		this->ControlKeys.Add(ControlElement->GetKey());
	}
	this->Blueprint = InBlueprint;
	ensure(Blueprint);

	ConstructCommon();
}

void SControlRigShapeNameList::Construct(const FArguments& InArgs, TArray<FRigControlElement> ControlElements, FControlRigAssetInterfacePtr InBlueprint)
{
	this->OnGetNameListContent = InArgs._OnGetNameListContent;
	this->ControlKeys.Reset();

	for(const FRigControlElement& ControlElement : ControlElements)
	{
		this->ControlKeys.Add(ControlElement.GetKey());
	}
	this->Blueprint = InBlueprint;
	ensure(Blueprint);

	ConstructCommon();
}

void SControlRigShapeNameList::BeginDestroy()
{
	if(NameListComboBox.IsValid())
	{
		NameListComboBox->SetOptionsSource(&GetEmptyList());
	}
}

void SControlRigShapeNameList::ConstructCommon()
{
	SBox::Construct(SBox::FArguments());

	TSharedPtr<FRigVMStringWithTag> InitialSelected;
	for (TSharedPtr<FRigVMStringWithTag> Item : GetNameList())
	{
		if (Item->Equals(GetNameListText().ToString()))
		{
			InitialSelected = Item;
		}
	}

	SetContent(
		SNew(SBox)
		.MinDesiredWidth(150)
		.MaxDesiredWidth(400)
		[
			SAssignNew(NameListComboBox, SRigVMGraphPinNameListValueWidget)
			.OptionsSource(&GetNameList())
			.OnGenerateWidget(this, &SControlRigShapeNameList::MakeNameListItemWidget)
			.OnSelectionChanged(this, &SControlRigShapeNameList::OnNameListChanged)
			.OnComboBoxOpening(this, &SControlRigShapeNameList::OnNameListComboBox)
			.InitiallySelectedItem(InitialSelected)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SControlRigShapeNameList::GetNameListText)
			]
		]
	);
}

const TArray<TSharedPtr<FRigVMStringWithTag>>& SControlRigShapeNameList::GetNameList() const
{
	if (OnGetNameListContent.IsBound())
	{
		return OnGetNameListContent.Execute();
	}
	return GetEmptyList();
}

FText SControlRigShapeNameList::GetNameListText() const
{
	FName FirstName = NAME_None;
	FText Text;
	for(int32 KeyIndex = 0; KeyIndex < ControlKeys.Num(); KeyIndex++)
	{
		const int32 ControlIndex = Blueprint->GetHierarchy()->GetIndex(ControlKeys[KeyIndex]);
		if (ControlIndex != INDEX_NONE)
		{
			const FName ShapeName = Blueprint->GetHierarchy()->GetChecked<FRigControlElement>(ControlIndex)->Settings.ShapeName; 
			if(KeyIndex == 0)
			{
				Text = FText::FromName(ShapeName);
				FirstName = ShapeName;
			}
			else if(FirstName != ShapeName)
			{
				static const FString MultipleValues = TEXT("Multiple Values");
				Text = FText::FromString(MultipleValues);
				break;
			}
		}
	}
	return Text;
}

void SControlRigShapeNameList::SetNameListText(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	const FScopedTransaction Transaction(NSLOCTEXT("ControlRigEditor", "ChangeShapeName", "Change Shape Name"));

	for(int32 KeyIndex = 0; KeyIndex < ControlKeys.Num(); KeyIndex++)
	{
		const int32 ControlIndex = Blueprint->GetHierarchy()->GetIndex(ControlKeys[KeyIndex]);
		if (ControlIndex != INDEX_NONE)
		{
			const FName NewName = *NewTypeInValue.ToString();
			URigHierarchy* Hierarchy = Blueprint->GetHierarchy();

			FRigControlElement* ControlElement = Hierarchy->Get<FRigControlElement>(ControlIndex);
			if ((ControlElement != nullptr) && (ControlElement->Settings.ShapeName != NewName))
			{
				Hierarchy->Modify();

				FRigControlSettings Settings = ControlElement->Settings;
				Settings.ShapeName = NewName;
				Hierarchy->SetControlSettings(ControlElement, Settings, true, true, true);
			}
		}
	}
}

TSharedRef<SWidget> SControlRigShapeNameList::MakeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem)
{
	return 	SNew(STextBlock).Text(FText::FromString(InItem->GetString()));// .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SControlRigShapeNameList::OnNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		if (NewSelection.IsValid())
		{
			const FString& NewValue = NewSelection->GetString();
			SetNameListText(FText::FromString(NewValue), ETextCommit::OnEnter);
		}
		else
		{
			SetNameListText(FText(), ETextCommit::OnEnter);
		}
	}
}

void SControlRigShapeNameList::OnNameListComboBox()
{
	TSharedPtr<FRigVMStringWithTag> CurrentlySelected;
	for (TSharedPtr<FRigVMStringWithTag> Item : GetNameList())
	{
		if (Item->Equals(GetNameListText().ToString()))
		{
			CurrentlySelected = Item;
		}
	}
	NameListComboBox->SetSelectedItem(CurrentlySelected);
}
