// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Rigs/RigHierarchyContainer.h"
#include "ControlRigBlueprintLegacy.h"

class SControlRigShapeNameList : public SBox
{
public:

	DECLARE_DELEGATE_RetVal( const TArray<TSharedPtr<FRigVMStringWithTag>>&, FOnGetNameListContent );

	SLATE_BEGIN_ARGS(SControlRigShapeNameList){}

		SLATE_EVENT(FOnGetNameListContent, OnGetNameListContent)

	SLATE_END_ARGS()

	UE_DEPRECATED(5.7, "Use void Construct(const FArguments& InArgs, FRigControlElement* ControlElement, FControlRigAssetInterfacePtr InBlueprint); instead")
	void Construct(const FArguments& InArgs, FRigControlElement* ControlElement, UControlRigBlueprint* InBlueprint){}
	UE_DEPRECATED(5.7, "Use void Construct(const FArguments& InArgs, TArray<FRigControlElement*> ControlElements, FControlRigAssetInterfacePtr InBlueprint); instead")
	void Construct(const FArguments& InArgs, TArray<FRigControlElement*> ControlElements, UControlRigBlueprint* InBlueprint){}
	UE_DEPRECATED(5.7, "Use void Construct(const FArguments& InArgs, TArray<FRigControlElement> ControlElements, FControlRigAssetInterfacePtr InBlueprint); instead")
	void Construct(const FArguments& InArgs, TArray<FRigControlElement> ControlElements, UControlRigBlueprint* InBlueprint){}

	void Construct(const FArguments& InArgs, FRigControlElement* ControlElement, FControlRigAssetInterfacePtr InBlueprint);
	void Construct(const FArguments& InArgs, TArray<FRigControlElement*> ControlElements, FControlRigAssetInterfacePtr InBlueprint);
	void Construct(const FArguments& InArgs, TArray<FRigControlElement> ControlElements, FControlRigAssetInterfacePtr InBlueprint);
	void BeginDestroy();

protected:

	void ConstructCommon();

	const TArray<TSharedPtr<FRigVMStringWithTag>>& GetNameList() const;
	FText GetNameListText() const;
	virtual void SetNameListText(const FText& NewTypeInValue, ETextCommit::Type CommitInfo);

	TSharedRef<SWidget> MakeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem);
	void OnNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo);
	void OnNameListComboBox();

	static const TArray< TSharedPtr<FRigVMStringWithTag> >& GetEmptyList()
	{
		static const TArray< TSharedPtr<FRigVMStringWithTag> > EmptyList;
		return EmptyList;
	}

	FOnGetNameListContent OnGetNameListContent;
	TSharedPtr<SRigVMGraphPinNameListValueWidget> NameListComboBox;

	TArray<FRigElementKey> ControlKeys;
	FControlRigAssetInterfacePtr Blueprint;
};
