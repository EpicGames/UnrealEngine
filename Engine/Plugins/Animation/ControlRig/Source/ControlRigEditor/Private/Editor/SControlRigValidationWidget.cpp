// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/SControlRigValidationWidget.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Styling/AppStyle.h"
#include "SlateOptMacros.h"
#include "HAL/ConsoleManager.h"
#include "UserInterface/SMessageLogListing.h"
#include "Framework/Application/SlateApplication.h"
#include "ControlRigBlueprintLegacy.h"
#include "Editor/RigVMNewEditor.h"
#include "UObject/UObjectIterator.h"
#include "Rigs/RigHierarchyController.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#define LOCTEXT_NAMESPACE "SControlRigValidationWidget"

//////////////////////////////////////////////////////////////
/// SControlRigValidationPassTableRow
///////////////////////////////////////////////////////////

void SControlRigValidationPassTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, SControlRigValidationWidget* InValidationWidget, TSharedRef<FControlRigValidationPassItem> InPassItem)
{
	STableRow<TSharedPtr<FControlRigValidationPassItem>>::Construct(
		STableRow<TSharedPtr<FControlRigValidationPassItem>>::FArguments()
		.Content()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.0f, 4.0f, 0.f, 0.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 12.f, 0.f)
				[
					SNew(STextBlock)
					.Text(InPassItem->DisplayText)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(InValidationWidget, &SControlRigValidationWidget::IsClassEnabled, InPassItem->Class)
					.OnCheckStateChanged(InValidationWidget, &SControlRigValidationWidget::SetClassEnabled, InPassItem->Class)
				]
			]
	
			+ SVerticalBox::Slot()
			.Padding(24.0f, 4.0f, 0.f, 0.f)
			.AutoHeight()
			[
#if WITH_RIGVMLEGACYEDITOR
				SAssignNew(KismetInspector, SKismetInspector)
#else
				SAssignNew(KismetInspector, SRigVMDetailsInspector)
#endif
				.Visibility(InValidationWidget, &SControlRigValidationWidget::IsClassVisible, InPassItem->Class)
				.ShowTitleArea(false)
			]
		]
		, OwnerTable
	);

	RefreshDetails(InValidationWidget->Validator, InPassItem->Class);
}

void SControlRigValidationPassTableRow::RefreshDetails(UControlRigValidator* InValidator, UClass* InClass)
{
	check(InValidator);

	if (UControlRigValidationPass* Pass = InValidator->FindPass(InClass))
	{
		KismetInspector->ShowDetailsForSingleObject(Pass);
	}
	else
	{
		KismetInspector->ShowDetailsForSingleObject(nullptr);
	}
}

//////////////////////////////////////////////////////////////
/// SControlRigValidationWidget
///////////////////////////////////////////////////////////

SControlRigValidationWidget::SControlRigValidationWidget()
	: ListingModel(FMessageLogListingModel::Create(TEXT("ValidationLog")))
	, ListingView(FMessageLogListingViewModel::Create(ListingModel, LOCTEXT("ValidationLog", "Validation Log")))
{
	Validator = nullptr;
	ListingView->OnMessageTokenClicked().AddRaw(this, &SControlRigValidationWidget::HandleMessageTokenClicked);
}

SControlRigValidationWidget::~SControlRigValidationWidget()
{
	if(Validator)
	{
		Validator->SetControlRig(nullptr);
		Validator->OnClear().Unbind();
		Validator->OnReport().Unbind();
	}
}

void SControlRigValidationWidget::Construct(const FArguments& InArgs, UControlRigValidator* InValidator)
{
	Validator = InValidator;

	ClassItems.Reset();
	for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
	{
		const bool bIsValidationPassChild = ClassIterator->IsChildOf(UControlRigValidationPass::StaticClass());
		if (bIsValidationPassChild && !ClassIterator->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			ClassItems.Add(MakeShared<FControlRigValidationPassItem>(*ClassIterator));
		}
	}

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.0f, 4.0f, 0.f, 0.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SListView<TSharedPtr<FControlRigValidationPassItem>>)
					.ListItemsSource(&ClassItems)
					.OnGenerateRow(this, &SControlRigValidationWidget::GenerateClassListRow)
					.SelectionMode(ESelectionMode::None)
				]
			]

			+ SVerticalBox::Slot()
			.Padding(4.0f, 4.0f, 0.f, 0.f)
			.AutoHeight()
			[
				SNew(SMessageLogListing, ListingView)
			]
		];

	Validator->OnClear().BindRaw(this, &SControlRigValidationWidget::HandleClearMessages);
	Validator->OnReport().BindRaw(this, &SControlRigValidationWidget::HandleMessageReported);
}

TSharedRef<ITableRow> SControlRigValidationWidget::GenerateClassListRow(TSharedPtr<FControlRigValidationPassItem> InItem, const TSharedRef<STableViewBase>& InOwningTable)
{
	TSharedRef<SControlRigValidationPassTableRow> TableRow = SNew(SControlRigValidationPassTableRow, InOwningTable, this, InItem.ToSharedRef());
	TableRows.Add(InItem->Class, TableRow);
	return TableRow;
}

ECheckBoxState SControlRigValidationWidget::IsClassEnabled(UClass* InClass) const
{
	if (Validator)
	{
		return Validator->FindPass(InClass) != nullptr ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

EVisibility SControlRigValidationWidget::IsClassVisible(UClass* InClass) const
{
	if (IsClassEnabled(InClass) == ECheckBoxState::Checked)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

void SControlRigValidationWidget::SetClassEnabled(ECheckBoxState NewState, UClass* InClass)
{
	if (Validator)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			Validator->AddPass(InClass);
		}
		else
		{
			Validator->RemovePass(InClass);
		}

		if (TSharedRef<SControlRigValidationPassTableRow>* TableRowPtr = TableRows.Find(InClass))
		{
			TSharedRef<SControlRigValidationPassTableRow>& TableRow = *TableRowPtr;
			TableRow->RefreshDetails(Validator, InClass);
		}
	}
}

void SControlRigValidationWidget::HandleClearMessages()
{
	ListingModel->ClearMessages();
}

void SControlRigValidationWidget::HandleMessageReported(EMessageSeverity::Type InSeverity, const FRigElementKey& InKey, float InQuality, const FString& InMessage)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity);

	if (InKey.IsValid())
	{
		FString TypeString = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)InKey.Type).ToString();
		FString KeyString = FString::Printf(TEXT("%s %s"), *TypeString, *InKey.Name.ToString());
		Message->AddToken(FAssetNameToken::Create(KeyString));
	}

	/*
	if (InQuality != FLT_MAX)
	{
		FString QualityString = FString::Printf(TEXT("%.02f"), InQuality);
		Message->AddToken(FTextToken::Create(FText::FromString(QualityString)));
	}
	*/

	if (!InMessage.IsEmpty())
	{
		Message->AddToken(FTextToken::Create(FText::FromString(InMessage)));
	}

	ListingModel->AddMessage(Message, false, true);
}

void SControlRigValidationWidget::HandleMessageTokenClicked(const TSharedRef<class IMessageToken>& InToken)
{
	if (UControlRig* ControlRig = Validator->GetControlRig())
	{
		if (InToken->GetType() == EMessageToken::AssetName)
		{
			FString Content = StaticCastSharedRef<FAssetNameToken>(InToken)->GetAssetName();

			FString Left, Right;
			if (Content.Split(TEXT(" "), &Left, &Right))
			{
				int32 TypeIndex = StaticEnum<ERigElementType>()->GetIndexByNameString(Left);
				if (TypeIndex != INDEX_NONE)
				{
					FRigElementKey Key(*Right, (ERigElementType)StaticEnum<ERigElementType>()->GetValueByIndex(TypeIndex));

					if (FControlRigAssetInterfacePtr EditorAsset = ControlRig->GetAssetReference().GetEditorAsset())
					{
						EditorAsset->GetHierarchyController()->ClearSelection();
						EditorAsset->GetHierarchyController()->SelectElement(Key, true);
						
						if(URigHierarchyController* Controller = ControlRig->GetHierarchy()->GetController())
						{
							Controller->ClearSelection();
							Controller->SelectElement(Key, true);
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
