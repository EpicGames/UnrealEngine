// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"
#include "Model/MessageLogListingModel.h"
#include "Presentation/MessageLogListingViewModel.h"
#include "ControlRigValidationPass.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif

#if !WITH_RIGVMLEGACYEDITOR
class SRigVMDetailsInspector;
#endif
class SControlRigValidationWidget;

class FControlRigValidationPassItem : public TSharedFromThis<FControlRigValidationPassItem>
{
public:
	FControlRigValidationPassItem(UClass* InClass)
		: Class(InClass)
	{
		static const FName DisplayName(TEXT("DisplayName"));
		DisplayText = FText::FromString(Class->GetMetaData(DisplayName));
	}

	/** The name to display in the UI */
	FText DisplayText;

	/** The struct of the rig unit */
	UClass* Class;
};

class SControlRigValidationPassTableRow : public STableRow<TSharedPtr<FControlRigValidationPassItem>>
{
	SLATE_BEGIN_ARGS(SControlRigValidationPassTableRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, SControlRigValidationWidget* InValidationWidget, TSharedRef<FControlRigValidationPassItem> InPassItem);
	void RefreshDetails(UControlRigValidator* InValidator, UClass* InClass);

private:
#if WITH_RIGVMLEGACYEDITOR
	TSharedPtr<SKismetInspector> KismetInspector;
#else
	TSharedPtr<SRigVMDetailsInspector> KismetInspector;
#endif
};

class SControlRigValidationWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SControlRigValidationWidget) {}
	SLATE_END_ARGS()

	SControlRigValidationWidget();
	~SControlRigValidationWidget();

	void Construct(const FArguments& InArgs, UControlRigValidator* InValidator);

	TSharedRef<ITableRow> GenerateClassListRow(TSharedPtr<FControlRigValidationPassItem> InItem, const TSharedRef<STableViewBase>& InOwningTable);
	ECheckBoxState IsClassEnabled(UClass* InClass) const;
	EVisibility IsClassVisible(UClass* InClass) const;
	void SetClassEnabled(ECheckBoxState NewState, UClass* InClass);

private:

	void HandleClearMessages();
	void HandleMessageReported(EMessageSeverity::Type InSeverity, const FRigElementKey& InKey, float InQuality, const FString& InMessage);
	void HandleMessageTokenClicked(const TSharedRef<class IMessageToken>& InToken);

	UControlRigValidator* Validator;
	TArray<TSharedPtr<FControlRigValidationPassItem>> ClassItems;
	TMap<UClass*, TSharedRef<SControlRigValidationPassTableRow>> TableRows;
	TSharedRef<FMessageLogListingModel> ListingModel;
	TSharedRef<FMessageLogListingViewModel> ListingView;

	friend struct FRigValidationTabSummoner;
	friend class SControlRigValidationPassTableRow;
};
