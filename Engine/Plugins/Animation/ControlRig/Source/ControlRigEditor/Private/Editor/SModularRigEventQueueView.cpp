// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/SModularRigEventQueueView.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Editor/ControlRigNewEditor.h"
#include "Editor/ModularRigEventQueueCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SSearchBox.h"
#include "Dialog/SCustomDialog.h"
#include "ControlRigEditor.h"
#include "MVVM/ViewModels/OutlinerColumns/OutlinerColumnTypes.h"

#define LOCTEXT_NAMESPACE "SModularRigEventQueueView"

//////////////////////////////////////////////////////////////
/// FModularRigEventEntry
///////////////////////////////////////////////////////////
FModularRigEventEntry::FModularRigEventEntry(int32 InEventIndex, const FName& InEventName, const FModuleInstanceHandle& InModule, bool InExecuted, double InDurationMicroSeconds)
	: EventIndex(InEventIndex)
	, EventName(InEventName)
	, Module(InModule)
	, bExecuted(InExecuted)
{
	UpdateDurationMicroSeconds(InDurationMicroSeconds);
}

TSharedRef<ITableRow> FModularRigEventEntry::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FModularRigEventEntry> InEntry, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SModularRigEventQueueView> InEventQueueView)
{
	return SNew(SModularRigEventItem, InOwnerTable, InEntry, InCommandList);
}

void FModularRigEventEntry::UpdateDurationMicroSeconds(double InDurationMicroSeconds)
{
	MicroSeconds = 0;
	if (!bExecuted)
	{
		return;
	}
		
	if (MicroSecondsFrames.Num() == 100)
	{
		MicroSecondsFrames.RemoveAt(0, EAllowShrinking::No);
	}
	MicroSecondsFrames.Add(InDurationMicroSeconds);

	for (const double& Frame : MicroSecondsFrames)
	{
		MicroSeconds += Frame;
	}
	MicroSeconds /= static_cast<double>(MicroSecondsFrames.Num());
}

//////////////////////////////////////////////////////////////
/// SModularRigEventItem
///////////////////////////////////////////////////////////
void SModularRigEventItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FModularRigEventEntry> InEventEntry, TSharedRef<FUICommandList> InCommandList)
{
	WeakEventEntry = InEventEntry;
	WeakCommandList = InCommandList;

	TSharedPtr< STextBlock > NumberWidget;
	TSharedPtr< STextBlock > TextWidget;

	const FSlateBrush* Icon = FSlateIcon(TEXT("RigVMEditor"), "RigVM.Unit").GetIcon();

	STableRow<TSharedPtr<FModularRigEventEntry>>::Construct(
		STableRow<TSharedPtr<FModularRigEventEntry>>::FArguments()
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(35.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SAssignNew(NumberWidget, STextBlock)
					.Text(this, &SModularRigEventItem::GetIndexText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(22.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(Icon)
				]
				
				+ SHorizontalBox::Slot()
				.Padding(4.f, 1, 1, 1)
				.FillWidth(1.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(TextWidget, STextBlock)
					.Text(this, &SModularRigEventItem::GetEventLabelText)
					.Font(this, &SModularRigEventItem::GetEventLabelFont)
					.ToolTipText(this, &SModularRigEventItem::GetTooltip)
					.Justification(ETextJustify::Left)
				]

				+ SHorizontalBox::Slot()
				.Padding(8.f, 1, 1, 1)
				.FillWidth(1.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(TextWidget, STextBlock)
					.Text(this, &SModularRigEventItem::GetModuleLabelText)
					.Font(this, &SModularRigEventItem::GetModuleLabelFont)
					.ToolTipText(this, &SModularRigEventItem::GetTooltip)
					.Justification(ETextJustify::Left)
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
	            
	            + SHorizontalBox::Slot()
	            .Padding(20, 0, 0, 0)
				.AutoWidth()
	            .VAlign(VAlign_Center)
	            .HAlign(HAlign_Left)
	            [
	                SNew(STextBlock)
	                .Text(this, &SModularRigEventItem::GetDurationText)
	                .Font(IDetailLayoutBuilder::GetDetailFont())
	            ]
	        ]
        ], OwnerTable);
}

FText SModularRigEventItem::GetIndexText() const
{
	const FString IndexStr = FString::FromInt(WeakEventEntry.Pin()->EventIndex) + TEXT(".");
	return FText::FromString(IndexStr);
}

FText SModularRigEventItem::GetModuleLabelText() const
{
	return (FText::FromName(WeakEventEntry.Pin()->Module.GetModuleName()));
}

FSlateFontInfo SModularRigEventItem::GetModuleLabelFont() const
{
	return IDetailLayoutBuilder::GetDetailFontBold();
}

FText SModularRigEventItem::GetEventLabelText() const
{
	return (FText::FromName(WeakEventEntry.Pin()->EventName));
}

FSlateFontInfo SModularRigEventItem::GetEventLabelFont() const
{
	return IDetailLayoutBuilder::GetDetailFont();
}

FText SModularRigEventItem::GetTooltip() const
{
	// empty for now
	return FText(); 
}

FText SModularRigEventItem::GetDurationText() const
{
	if(WeakEventEntry.IsValid())
	{
		if (TSharedPtr<FModularRigEventEntry> Entry = WeakEventEntry.Pin())
		{
			if (!Entry->MicroSecondsFrames.IsEmpty())
			{
				return FText::FromString(FString::Printf(TEXT("%.02f µs"), (float)Entry->MicroSeconds));
			}
		}
	}
	return FText();
}

//////////////////////////////////////////////////////////////
/// SModularRigEventQueueView
///////////////////////////////////////////////////////////

SModularRigEventQueueView::~SModularRigEventQueueView()
{
	if (TSharedPtr<IControlRigBaseEditor> Editor = WeakEditor.Pin())
	{
		if (OnControlRigExecutedHandle.IsValid())
		{
			if(URigVMHost* ModularRig = Editor->GetRigVMHost())
			{
				ModularRig->OnExecuted_AnyThread().Remove(OnControlRigExecutedHandle);
			}
		}
		if (OnPreviewHostUpdatedHandle.IsValid())
		{
			Editor->SharedRigVMEditorRef()->OnPreviewHostUpdated().Remove(OnPreviewHostUpdatedHandle);
		}
	}
}

void SModularRigEventQueueView::Construct( const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor)
{
	WeakEditor = InControlRigEditor;
	CommandList = MakeShared<FUICommandList>();

	BindCommands();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)
					//.Visibility(this, &SRigHierarchy::IsSearchbarVisible)
					+SHorizontalBox::Slot()
					.AutoWidth()
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(3.0f, 1.0f)
					[
						SAssignNew(FilterBox, SSearchBox)
						.OnTextChanged(this, &SModularRigEventQueueView::OnFilterTextChanged)
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
			[
				SAssignNew(TreeView, STreeView<TSharedPtr<FModularRigEventEntry>>)
				.TreeItemsSource(&Events)
				.SelectionMode(ESelectionMode::Multi)
				.OnGenerateRow(this, &SModularRigEventQueueView::MakeTableRowWidget)
				.OnGetChildren(this, &SModularRigEventQueueView::HandleGetChildrenForTree)
				.OnSelectionChanged(this, &SModularRigEventQueueView::OnSelectionChanged)
			]
		]
	];

	if (TSharedPtr<IControlRigBaseEditor> Editor = WeakEditor.Pin())
	{
		OnPreviewHostUpdatedHandle = Editor->SharedRigVMEditorRef()->OnPreviewHostUpdated().AddSP(this, &SModularRigEventQueueView::HandlePreviewHostUpdated);
		HandlePreviewHostUpdated(&Editor->SharedRigVMEditorRef().Get());
	}
}

void SModularRigEventQueueView::OnSelectionChanged(TSharedPtr<FModularRigEventEntry> Selection, ESelectInfo::Type SelectInfo)
{
	if (bIsSelecting)
	{
		return;
	}
	TGuardValue<bool> SelectionGuard(bIsSelecting, true);

	if (!WeakModularRigController.IsValid())
	{
		return;
	}

	UModularRigController* ModularRigController = WeakModularRigController.Get();
	if (ModularRigController == nullptr)
	{
		return;
	}
	
	TArray<TSharedPtr<FModularRigEventEntry>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		if (TSharedPtr<IControlRigBaseEditor> Editor = WeakEditor.Pin())
		{
			TArray<FName> SelectedModules;
			for (const TSharedPtr<FModularRigEventEntry>& Item : SelectedItems)
			{
				SelectedModules.AddUnique(Item->Module.GetModuleName());
			}
			
			const TArray<FName> PreviouslySelectedModules = Editor->GetSelectedModules();

			if (SelectedModules.Num() == PreviouslySelectedModules.Num())
			{
				bool bFoundDifference = false;
				for (const FName& SelectedModule : SelectedModules)
				{
					if (!PreviouslySelectedModules.Contains(SelectedModule))
					{
						bFoundDifference = true;
						break;
					}
				}

				if (!bFoundDifference)
				{
					return;
				}
			}

			ModularRigController->SetModuleSelection(SelectedModules);
		}
	}
}

void SModularRigEventQueueView::BindCommands()
{
	// create new command
	const FModularRigEventQueueCommands& Commands = FModularRigEventQueueCommands::Get();
	CommandList->MapAction(Commands.FocusOnSelection, FExecuteAction::CreateSP(this, &SModularRigEventQueueView::HandleFocusOnSelectedModule));
}

TSharedRef<ITableRow> SModularRigEventQueueView::MakeTableRowWidget(TSharedPtr<FModularRigEventEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), CommandList.ToSharedRef(), SharedThis(this));
}

void SModularRigEventQueueView::HandleGetChildrenForTree(TSharedPtr<FModularRigEventEntry> InItem,
	TArray<TSharedPtr<FModularRigEventEntry>>& OutChildren)
{
	// nothing to do here
}

bool SModularRigEventQueueView::PopulateEventQueueView(const UModularRig* InModularRig)
{
	if (InModularRig)
	{
		const FString FilterString = FilterText.ToString();
		
		// copy the queue
		const FRigModuleExecutionQueue Queue = InModularRig->GetLastExecutionQueue();
		const uint32 Hash = HashCombine(GetTypeHash(Queue), GetTypeHash(FilterString));
		if (Hash != LastHash)
		{
			Events.Reset();

			for (int32 Index = 0; Index < Queue.Elements.Num(); Index++)
			{
				const FRigModuleExecutionElement& Element = Queue.Elements[Index];
				if (!FilterString.IsEmpty())
				{
					if (!Element.EventName.ToString().Contains(FilterString, ESearchCase::IgnoreCase) &&
						!Element.ModuleName.ToString().Contains(FilterString, ESearchCase::IgnoreCase))
					{
						continue;
					}
				}
				TSharedPtr<FModularRigEventEntry> NewEntry = MakeShared<FModularRigEventEntry>(Index, Element.EventName, FModuleInstanceHandle(InModularRig, Element.ModuleInstance), Element.bExecuted, Element.DurationInMicroSeconds);
				Events.Add(NewEntry);
			}

			LastHash = Hash;
			return true;
		}
		else
		{
			for (int32 Index = 0; Index < Events.Num(); Index++)
			{
				const int32 QueueIndex = Events[Index]->EventIndex;
				Events[Index]->UpdateDurationMicroSeconds(Queue.Elements[QueueIndex].DurationInMicroSeconds);
			}
		}
	}

	return false;
}

void SModularRigEventQueueView::RefreshTreeView(const UModularRig* InModularRig)
{
	if (PopulateEventQueueView(InModularRig))
	{
		TreeView->RequestTreeRefresh();
	}
}

void SModularRigEventQueueView::HandleFocusOnSelectedModule()
{
	OnSelectionChanged(TSharedPtr<FModularRigEventEntry>(), ESelectInfo::Direct);
}

void SModularRigEventQueueView::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;
	if (const UModularRig* ModularRig = WeakModularRig.Get())
	{
		RefreshTreeView(ModularRig);
	}
}

void SModularRigEventQueueView::HandleModularRigExecuted(class URigVMHost* InModularRig, const FName& InEventName)
{
	if (!WeakModularRig.IsValid())
	{
		return;
	}

	if (WeakModularRig.Get() != InModularRig)
	{
		return;
	}

	RefreshTreeView(Cast<UModularRig>(InModularRig));
}

void SModularRigEventQueueView::HandleModularRigModelModified(EModularRigNotification InNotification, const FRigModuleReference* InModuleReference)
{
	if (bIsSelecting)
	{
		return;
	}
	TGuardValue<bool> SelectionGuard(bIsSelecting, true);
	
	if (InNotification == EModularRigNotification::ModuleSelected)
	{
		for (const TSharedPtr<FModularRigEventEntry>& Entry : Events)
		{
			if (Entry->Module.GetModuleName() == InModuleReference->GetName())
			{
				TreeView->SetItemSelection(Entry, true, ESelectInfo::Type::Direct);
			}
		}
	}
	else if (InNotification == EModularRigNotification::ModuleDeselected)
	{
		for (const TSharedPtr<FModularRigEventEntry>& Entry : Events)
		{
			if (Entry->Module.GetModuleName() == InModuleReference->GetName())
			{
				TreeView->SetItemSelection(Entry, false, ESelectInfo::Type::Direct);
			}
		}
	}
}

void SModularRigEventQueueView::HandlePreviewHostUpdated(IRigVMEditor* InEditor)
{
	if (WeakModularRig.IsValid() && OnControlRigExecutedHandle.IsValid())
	{
		if (UModularRig* ModularRig = WeakModularRig.Get())
		{
			ModularRig->OnExecuted_AnyThread().Remove(OnControlRigExecutedHandle);
		}
	}

	if (WeakModularRigController.IsValid())
	{
		if (UModularRigController* Controller = WeakModularRigController.Get())
		{
			Controller->OnModified().Remove(OnModularRigModelModifiedHandle);
		}
	}

	WeakModularRig.Reset();
	WeakModularRigController.Reset();
	
	if(UModularRig* ModularRig = Cast<UModularRig>(InEditor->GetRigVMHost()))
	{
		WeakModularRig = ModularRig;
		
		OnControlRigExecutedHandle = ModularRig->OnExecuted_AnyThread().AddSP(this, &SModularRigEventQueueView::HandleModularRigExecuted);
		RefreshTreeView(ModularRig);

		if (UControlRigBlueprint* ControlRigBlueprint = Cast<UControlRigBlueprint>(ModularRig->GetClass()->ClassGeneratedBy))
		{
			if (UModularRigController* Controller = ControlRigBlueprint->GetModularRigController())
			{
				WeakModularRigController = Controller;
				Controller->OnModified().AddSP(this, &SModularRigEventQueueView::HandleModularRigModelModified);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
