// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModularRig.h"
#include "Editor/RigVMEditor.h"
#include "Widgets/Views/STreeView.h"

#define UE_API CONTROLRIGEDITOR_API

class SModularRigEventQueueView;
class FUICommandList;
class SSearchBox;
class IControlRigBaseEditor;

/** An event in the queue */
class FModularRigEventEntry : public TSharedFromThis<FModularRigEventEntry>
{
public:
	UE_API FModularRigEventEntry(int32 InEventIndex, const FName& InEventName, const FModuleInstanceHandle& InModule, bool InExecuted, double InDurationMicroSeconds);

	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FModularRigEventEntry> InEntry, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SModularRigEventQueueView> InEventQueueView);

	void UpdateDurationMicroSeconds(double InDurationMicroSeconds);

	int32 EventIndex;
	FName EventName;
	FModuleInstanceHandle Module;
	bool bExecuted = false;
	double MicroSeconds = 0.0;
	TArray<double> MicroSecondsFrames;
};

class SModularRigEventItem : public STableRow<TSharedPtr<FModularRigEventEntry>>
{
	SLATE_BEGIN_ARGS(SModularRigEventItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FModularRigEventEntry> InEventEntry, TSharedRef<FUICommandList> InCommandList);

private:
	TWeakPtr<FModularRigEventEntry> WeakEventEntry;
	TWeakPtr<FUICommandList> WeakCommandList;

	FText GetIndexText() const;
	FText GetModuleLabelText() const;
	FSlateFontInfo GetModuleLabelFont() const;
	FText GetEventLabelText() const;
	FSlateFontInfo GetEventLabelFont() const;
	FText GetTooltip() const;
	FText GetDurationText() const;
};

class SModularRigEventQueueView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SModularRigEventQueueView) {}
	SLATE_END_ARGS()

	UE_API ~SModularRigEventQueueView();

	UE_API void Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor);

	/** Set Selection Changed */
	UE_API void OnSelectionChanged(TSharedPtr<FModularRigEventEntry> Selection, ESelectInfo::Type SelectInfo);

protected:

	/** Rebuild the tree view */
	UE_API void RefreshTreeView(const UModularRig* InModularRig);

private:

	/** Bind commands that this widget handles */
	UE_API void BindCommands();

	/** Make a row widget for the table */
	UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FModularRigEventEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get children for the tree */
	UE_API void HandleGetChildrenForTree(TSharedPtr<FModularRigEventEntry> InItem, TArray<TSharedPtr<FModularRigEventEntry>>& OutChildren);

	/** Focus on the selected module in the rig */
	UE_API void HandleFocusOnSelectedModule();

	/** Search box widget */
	TSharedPtr<SSearchBox> FilterBox;
	FText FilterText;
	UE_API void OnFilterTextChanged(const FText& SearchText);

	UE_API void HandlePreviewHostUpdated(IRigVMEditor* InEditor);
	UE_API void HandleModularRigExecuted(class URigVMHost* InModularRig, const FName& InEventName);
	UE_API void HandleModularRigModelModified(EModularRigNotification InNotification, const FRigModuleReference* InModuleReference);

	/**
	 * Populate the event queue with descriptive names for each event
	 * Returns true if the treeview has to refresh.
	 */
	UE_API bool PopulateEventQueueView(const UModularRig* InModularRig);

	TSharedPtr<STreeView<TSharedPtr<FModularRigEventEntry>>> TreeView;

	/** Command list we bind to */
	TSharedPtr<FUICommandList> CommandList;

	TWeakPtr<IControlRigBaseEditor> WeakEditor;
	TWeakObjectPtr<UModularRig> WeakModularRig;
	TWeakObjectPtr<UModularRigController> WeakModularRigController;

	TArray<TSharedPtr<FModularRigEventEntry>> Events;
	uint32 LastHash = 0;
	bool bIsSelecting = false;

	FDelegateHandle OnPreviewHostUpdatedHandle;
	FDelegateHandle OnControlRigExecutedHandle;
	FDelegateHandle OnModularRigModelModifiedHandle;
};

#undef UE_API
