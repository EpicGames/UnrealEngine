// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigConnectorTargetWidget.h"

#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Editor/Hierarchy/Models/RigHierarchyTreeElement.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchySearchableTreeView.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyItem.h"
#include "IDetailChildrenBuilder.h"
#include "ISinglePropertyView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "ControlRigEditorStyle.h"
#include "ModularRig.h"
#include "ModularRigRuleManager.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SRigConnectorTargetWidget)

#define LOCTEXT_NAMESPACE "SRigConnectorTargetWidget"

SRigConnectorTargetComboButton::~SRigConnectorTargetComboButton()
{
}

void SRigConnectorTargetComboButton::Construct(const FArguments& InArgs)
{
	using namespace UE::ControlRigEditor;

	ConnectorKey = InArgs._ConnectorKey;
	TargetKey = InArgs._TargetKey;
	ArrayIndex = InArgs._ArrayIndex;
	RigTreeDelegates = InArgs._RigTreeDelegates;
	OnSetTarget = InArgs._OnSetTarget;

	RigTreeDelegates.OnGetSelection.BindLambda([WeakThis = AsWeak(), this]() -> TArray<FRigHierarchyKey>
	{
		if (WeakThis.IsValid())
		{
			return { TargetKey.Get() };
		}

		return { };
	});
	RigTreeDelegates.OnSelectionChanged.BindSP(this, &SRigConnectorTargetComboButton::OnConnectorTargetPicked);

	RigTreeDelegates.OnGetTagDisplayMode = FOnRigTreeGetTagDisplayMode::CreateLambda([]() 
		{
			return ERigHierarchyConnectorTagDisplayMode::Single;
		}
	);

	constexpr const TCHAR* ConnecterTargetRigHierarchyView = TEXT("ConnecterTargetRigHierarchyView");

	SearchableTreeView = SNew(SRigHierarchySearchableTreeView, ConnecterTargetRigHierarchyView)
		.RigTreeDelegates(RigTreeDelegates);

	SComboButton::FArguments ComboButtonArgs;
	ComboButtonArgs.ContentPadding(InArgs._ContentPadding);
	ComboButtonArgs.MenuPlacement(InArgs._MenuPlacement);
	ComboButtonArgs.OnComboBoxOpened(this, &SRigConnectorTargetComboButton::OnComboBoxOpened);
	ComboButtonArgs.ButtonContent()
	[
		// Wrap in configurable box to restrain height/width of menu
		SNew(SBox)
		.MinDesiredWidth(InArgs._ButtonMinDesiredWith)
		.Padding(FMargin(0, 2, 0, 2))
		[
			SAssignNew(VerticalButtonBox, SVerticalBox)
		]
	];
	ComboButtonArgs.MenuContent()
	[
		SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0)
			.MaxWidth(900)
			[
				SearchableTreeView.ToSharedRef()
			]
		]
	];

	SComboButton::Construct(ComboButtonArgs);
	PopulateButtonBox();
}

void SRigConnectorTargetComboButton::OnComboBoxOpened()
{
	SearchableTreeView->GetTreeView()->RefreshTreeView();

	// set the focus to the search box so you can start typing right away
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double,float)
	{
		FSlateApplication::Get().ForEachUser([this](FSlateUser& User)
		{
			User.SetFocus(SearchableTreeView->GetSearchBox());
		});
		return EActiveTimerReturnType::Stop;
	}));
}

void SRigConnectorTargetComboButton::PopulateButtonBox()
{
	const URigHierarchy* Hierarchy = RigTreeDelegates.GetHierarchy();
	if (!Hierarchy)
	{
		return;
	}
	
	static const FSlateBrush* RoundedBoxBrush = FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.SpacePicker.RoundedRect"));

	const TWeakObjectPtr<const URigHierarchy> WeakHierarchy = Hierarchy;
	auto GetIconAndColor = [WeakHierarchy, this]() -> TPair<const FSlateBrush*, FSlateColor>
		{
			if (WeakHierarchy.IsValid())
			{
				FRigElementKey CurrentTargetKey(NAME_None, ERigElementType::Bone);
				if (TargetKey.IsSet() || TargetKey.IsBound())
				{
					CurrentTargetKey = TargetKey.Get();
				}
				return SRigHierarchyItem::GetBrushForElementType(WeakHierarchy.Get(), CurrentTargetKey);
			}

			return TPair<const FSlateBrush*, FSlateColor>(nullptr, FSlateColor::UseForeground());
		};

	TSharedPtr<SHorizontalBox> RowBox, ButtonBox;
	VerticalButtonBox->AddSlot()
	.AutoHeight()
	.VAlign(VAlign_Top)
	.HAlign(HAlign_Fill)
	.Padding(0.0, 0.0, 4.0, 0.0)
	[
		SNew(SBorder)
		.Padding(FMargin(2.0, 2.0, 5.0, 2.0))
		.BorderImage(RoundedBoxBrush)
		.BorderBackgroundColor(FSlateColor(FLinearColor::Transparent))
		.Content()
		[
			SAssignNew(ButtonBox, SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([GetIconAndColor]()
				{
					return GetIconAndColor().Key;
				})
				.ColorAndOpacity_Lambda([GetIconAndColor]()
				{
					return GetIconAndColor().Value;
				})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(0)
			[
				SNew( STextBlock )
				.OverflowPolicy(ETextOverflowPolicy::Clip)
				.Text_Lambda([WeakHierarchy, this]()
					{
						if (WeakHierarchy.IsValid())
						{
							const EElementNameDisplayMode NameDisplayMode = RigTreeDelegates.GetDisplaySettings().NameDisplayMode;
							return WeakHierarchy->GetDisplayNameForUI(TargetKey.Get(), NameDisplayMode);
						}

						return FText::GetEmpty();
					})
				.Font( IDetailLayoutBuilder::GetDetailFont() )
			]
		]
	];
}

void SRigConnectorTargetComboButton::OnConnectorTargetPicked(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	if(SelectInfo == ESelectInfo::OnNavigation)
	{
		return;
	}
	if(OnSetTarget.IsBound() && Selection.IsValid())
	{
		if(Selection->Key != TargetKey.Get())
		{
			(void)OnSetTarget.Execute(Selection->Key.GetElement());
		}
	}
}

SRigConnectorTargetWidget::~SRigConnectorTargetWidget()
{
}

void SRigConnectorTargetWidget::Construct(const FArguments& InArgs)
{
	check(InArgs._Outer);

	SetPadding(InArgs._Padding);
	bIsArray = InArgs._IsArray;
	Connector = InArgs._ConnectorKey;
	RigTreeDelegates = InArgs._RigTreeDelegates;

	if(!RigTreeDelegates.OnRigTreeIsItemVisible.IsBound())
	{
		TArray<FRigElementKey> PotentialTargets;
		if(URigHierarchy* Hierarchy = RigTreeDelegates.GetHierarchy())
		{
			if(const UControlRig* ControlRig = Hierarchy->GetTypedOuter<UControlRig>())
			{
				if(const FRigConnectorElement* ConnectorElement = Hierarchy->Find<FRigConnectorElement>(Connector))
				{
					const UModularRig* ModularRig = Cast<UModularRig>(ControlRig);

					// Only filter by module if this is a modular rig, otherwise 
					// use a nullptr module for it to be ignored when finding matches
					const FRigModuleInstance* Module = ModularRig ? ModularRig->FindModule(ConnectorElement) : nullptr;

					if (ModularRig == nullptr || Module != nullptr)
					{
						const UModularRigRuleManager* RuleManager = Hierarchy->GetRuleManager();
						TArray<FRigElementResolveResult> Matches = RuleManager->FindMatches(ConnectorElement, Module, ControlRig->GetElementKeyRedirector()).GetMatches();
						PotentialTargets.Reserve(Matches.Num());
						for(const FRigElementResolveResult& SingleMatch : Matches)
						{
							PotentialTargets.Add(SingleMatch.GetKey());
						}
					}
				}
			}
		}

		RigTreeDelegates.OnRigTreeIsItemVisible.BindLambda([PotentialTargets](const FRigHierarchyKey& InItem)
		{
			return InItem.IsValid() && InItem.IsElement() && (PotentialTargets.IsEmpty() || PotentialTargets.Contains(InItem.GetElement())); 
		});
	}

	OnSetTargetArray = InArgs._OnSetTargetArray;

	if(bIsArray)
	{
		TargetsDetailWrapper = TStrongObjectPtr<URigConnectorTargetsDetailWrapper>(NewObject<URigConnectorTargetsDetailWrapper>(InArgs._Outer, NAME_None, RF_Public | RF_Transient | RF_TextExportTransient | RF_DuplicateTransient));
		TargetsDetailWrapper->Connector = Connector;
		TargetsDetailWrapper->RigTreeDelegates = &RigTreeDelegates;
	}
	HandleTargetsChangedInClient(InArgs._Targets);

	if(bIsArray)
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowSectionSelector = false;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bShowScrollBar = false;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.ColumnWidth = .93f;

		const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

		// Define a min height. The details view is painted one tick after it was constructed
		// and resizes the widget. This causes the outer widget to resize, e.g. when shown in 
		// the modular rig hierarchy tree view, it leaves the tree with a borked scrollbar 
		// towards the bottom unless the height is defined ahead of time.
		constexpr float MinHeight = 28.f;

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(0)
			.FillWidth(1.f)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.Padding(0)
				.FillHeight(1.f)
				.MinHeight(MinHeight)
				[
					DetailsView
				]
			]
		];

		DetailsView->OnFinishedChangingProperties().AddSP(this, &SRigConnectorTargetWidget::OnFinishedChangingProperties);

		DetailsView->RegisterInstancedCustomPropertyTypeLayout(
			FRigElementKey::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateSP(this, &SRigConnectorTargetWidget::GetRigElementKeyCustomization)
		);

		TArray<UObject*> Objects = {TargetsDetailWrapper.Get()}; 
		DetailsView->SetObjects(Objects, true);
	}
	else
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0)
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(0)
				.FillHeight(1)
				[
					SNew(SRigConnectorTargetComboButton)
					.ConnectorKey(Connector)
					.TargetKey(this, &SRigConnectorTargetWidget::GetSingleTargetKey)
					.ContentPadding(FMargin(0, 3, 3, 0))
					.OnSetTarget_Lambda([this](FRigElementKey InTarget) -> bool
					{
						SingleTarget = InTarget;
						OnPropertyChanged();
						return true;
					})
					.RigTreeDelegates(RigTreeDelegates)
				]
			]
		];
	}
}

void SRigConnectorTargetWidget::HandleTargetsChangedInClient(TArray<FRigElementKey> InTargets)
{
	SingleTarget = FRigElementKey(NAME_None, ERigElementType::Bone);
	if(bIsArray)
	{
		TargetsDetailWrapper->TargetArray = InTargets;
	}
	else
	{
		if(InTargets.Num() == 1)
		{
			SingleTarget = InTargets[0];
		}
	}
}

void SRigConnectorTargetWidget::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	OnPropertyChanged();
}

void SRigConnectorTargetWidget::OnPropertyChanged()
{
	if(!OnSetTargetArray.IsBound())
	{
		return;
	}
	if(bIsArray)
	{
		(void)OnSetTargetArray.Execute(TargetsDetailWrapper->TargetArray);
	}
	else
	{
		(void)OnSetTargetArray.Execute({SingleTarget});
	}
}

TSharedRef<IPropertyTypeCustomization> SRigConnectorTargetWidget::GetRigElementKeyCustomization() const
{
	return MakeShareable(new FRigConnectorTargetWidgetCustomization);
}

FRigElementKey SRigConnectorTargetWidget::GetSingleTargetKey() const
{
	return SingleTarget;
}

void FRigConnectorTargetWidgetCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow,
                                                             IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle.ToSharedPtr();

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	for (UObject* Object : Objects)
	{
		if (Object->IsA<URigConnectorTargetsDetailWrapper>())
		{
			TargetsDetailWrapper = Cast<URigConnectorTargetsDetailWrapper>(Object);
			break;
		}
	}

	HeaderRow.NameContent()
	.MaxDesiredWidth(30)
	[
		SNullWidget::NullWidget
	];

	if(TargetsDetailWrapper.IsValid())
	{
		const TWeakPtr<IPropertyHandle> WeakStructPropertyHandle = InStructPropertyHandle;

		const int32 ArrayIndex = InStructPropertyHandle->GetArrayIndex();
		TSharedPtr<SRigConnectorTargetComboButton> ComboButton;
		if(TargetsDetailWrapper->TargetArray.IsValidIndex(ArrayIndex))
		{
			HeaderRow.ValueContent()
			.HAlign(HAlign_Fill)
			[
				SAssignNew(ComboButton, SRigConnectorTargetComboButton)
				.Clipping(EWidgetClipping::ClipToBounds)
				.ConnectorKey(TargetsDetailWrapper->Connector)
				.TargetKey(this, &FRigConnectorTargetWidgetCustomization::GetElementKey)
				.ContentPadding(FMargin(0, 3, 3, 0))
				.OnSetTarget_Lambda([WeakStructPropertyHandle, this, ArrayIndex](FRigElementKey InTarget) -> bool
					{
						if (WeakStructPropertyHandle.IsValid() &&
							WeakStructPropertyHandle.Pin()->IsValidHandle())
						{
							TargetsDetailWrapper->TargetArray[ArrayIndex] = InTarget;
							WeakStructPropertyHandle.Pin()->NotifyFinishedChangingProperties();
							return true;
						}

						return false;
					})
				.RigTreeDelegates(*TargetsDetailWrapper->GetRigTreeDelegates())
			];
		}
	}
}

void FRigConnectorTargetWidgetCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle,
	IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// nothing to do here
}

FRigElementKey FRigConnectorTargetWidgetCustomization::GetElementKey() const
{
	void* Data = nullptr;
	if(FPropertyAccess::Success == StructPropertyHandle->GetValueData(Data))
	{
		return *reinterpret_cast<FRigElementKey*>(Data);
	}
	return FRigElementKey();
}

#undef LOCTEXT_NAMESPACE
