// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/Hierarchy/RigHierarchyTreeDelegates.h"
#include "Rigs/RigHierarchy.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "PropertyEditorDelegates.h"
#include "UObject/StrongObjectPtrTemplates.h"
#include "IPropertyTypeCustomization.h"
#include "SRigConnectorTargetWidget.generated.h"
  
class SRigHierarchySearchableTreeView;
struct FRigHierarchyTreeDelegates;

DECLARE_DELEGATE_RetVal_OneParam(bool, FRigConnectorTargetWidget_SetTarget, FRigElementKey);
DECLARE_DELEGATE_RetVal_OneParam(bool, FRigConnectorTargetWidget_SetTargetArray, TArray<FRigElementKey>);
DECLARE_DELEGATE_OneParam(FRigConnectorTargetWidget_HandleTargetsChangedInClient, TArray<FRigElementKey>);

UCLASS()
class URigConnectorTargetsDetailWrapper : public UObject
{
	GENERATED_BODY()

public:
	
	URigConnectorTargetsDetailWrapper()
	: Connector(NAME_None, ERigElementType::Connector)
	{
	}

	UPROPERTY()
	FRigElementKey Connector;

	UPROPERTY(EditAnywhere, Category=NoCategory, meta=(FullyExpand=false))
	TArray<FRigElementKey> TargetArray;

	FRigHierarchyTreeDelegates* GetRigTreeDelegates() const
	{
		return RigTreeDelegates;
	}

private:
	
	FRigHierarchyTreeDelegates* RigTreeDelegates = nullptr;

	friend class SRigConnectorTargetWidget;
	friend class FRigConnectorTargetWidgetCustomization;
};

/** Widget allowing editing the targets of a connector */
class SRigConnectorTargetComboButton : public SComboButton
{
public:
	SLATE_BEGIN_ARGS(SRigConnectorTargetComboButton)
		: _Padding(0)
		, _ContentPadding(3)
		, _MenuPlacement(EMenuPlacement::MenuPlacement_BelowAnchor)
		, _ConnectorKey(NAME_None, ERigElementType::Connector)
		, _ArrayIndex(INDEX_NONE)
		, _ButtonMinDesiredWith(150.f)
	{}
	SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_ATTRIBUTE( FMargin, ContentPadding )
	SLATE_ATTRIBUTE( EMenuPlacement, MenuPlacement )
	SLATE_ARGUMENT(FRigElementKey, ConnectorKey)
	SLATE_ATTRIBUTE(FRigElementKey, TargetKey)
	SLATE_ATTRIBUTE(int32, ArrayIndex)
	SLATE_ARGUMENT(FRigHierarchyTreeDelegates, RigTreeDelegates)
	SLATE_ARGUMENT(float, ButtonMinDesiredWith)
	SLATE_EVENT(FRigConnectorTargetWidget_SetTarget, OnSetTarget)
	SLATE_END_ARGS()

	virtual ~SRigConnectorTargetComboButton() override;

	void Construct(const FArguments& InArgs);

private:

	void OnComboBoxOpened();
	void PopulateButtonBox();
	void OnConnectorTargetPicked(TSharedPtr<FRigHierarchyTreeElement> Selection, ESelectInfo::Type SelectInfo);
	
	FRigElementKey ConnectorKey;
	TAttribute<FRigElementKey> TargetKey;
	TAttribute<int32> ArrayIndex;
	FRigConnectorTargetWidget_SetTarget OnSetTarget;
	TSharedPtr<SVerticalBox> VerticalButtonBox;
	FRigHierarchyTreeDelegates RigTreeDelegates;
	TSharedPtr<SRigHierarchySearchableTreeView> SearchableTreeView;
};

/** Widget allowing editing the targets of a connector */
class SRigConnectorTargetWidget : public SBox
{
public:
	SLATE_BEGIN_ARGS(SRigConnectorTargetWidget)
		: _Outer(nullptr)
		, _ConnectorKey(NAME_None, ERigElementType::Connector)
		, _IsArray(false)
		, _ExpandArrayByDefault(false)
		, _RespectConnectorRules(true)
		, _HandleTargetsChangedInClient(nullptr)
		, _Padding(FMargin(0))
	{}
	SLATE_ARGUMENT(UObject*, Outer)
	SLATE_ARGUMENT(FRigElementKey, ConnectorKey)
	SLATE_ARGUMENT(TArray<FRigElementKey>, Targets)
	SLATE_ARGUMENT(bool, IsArray)
	SLATE_ARGUMENT(bool, ExpandArrayByDefault)
	SLATE_ARGUMENT(bool, RespectConnectorRules)
	SLATE_EVENT(FRigConnectorTargetWidget_SetTargetArray, OnSetTargetArray)
	SLATE_ARGUMENT(FRigConnectorTargetWidget_HandleTargetsChangedInClient*, HandleTargetsChangedInClient)
	SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_ARGUMENT(FRigHierarchyTreeDelegates, RigTreeDelegates)
	SLATE_END_ARGS()

	virtual ~SRigConnectorTargetWidget() override;

	void Construct(const FArguments& InArgs);

private:

	void HandleTargetsChangedInClient(TArray<FRigElementKey> InTargets);
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);
	void OnPropertyChanged();
	TSharedRef<IPropertyTypeCustomization> GetRigElementKeyCustomization() const;

	FRigElementKey GetSingleTargetKey() const;

	FRigElementKey Connector;
	FRigElementKey SingleTarget;
	TStrongObjectPtr<URigConnectorTargetsDetailWrapper> TargetsDetailWrapper;
	FRigConnectorTargetWidget_SetTargetArray OnSetTargetArray;
	bool bIsArray = false;
	FRigHierarchyTreeDelegates RigTreeDelegates;
};

class FRigConnectorTargetWidgetCustomization : public IPropertyTypeCustomization
{
public:

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	FRigElementKey GetElementKey() const;

	mutable TSharedPtr<IPropertyHandle> StructPropertyHandle;
	TWeakObjectPtr<URigConnectorTargetsDetailWrapper> TargetsDetailWrapper;

	friend class SRigConnectorTargetWidget;
};
