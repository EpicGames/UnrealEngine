// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/SRigCurveContainer.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Graph/ControlRigGraphNode.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "PropertyCustomizationHelpers.h"
#include "Framework/Commands/GenericCommands.h"
#include "Editor/RigCurveContainerCommands.h"
#include "Editor/RigVMEdGraphNodeRegistry.h"
#include "Editor/ControlRigEditor.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "ControlRigBlueprintLegacy.h"
#include "ScopedTransaction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/Hierarchy/RigUnit_CurveExists.h"
#include "Units/Hierarchy/RigUnit_GetCurveValue.h"
#include "Units/Hierarchy/RigUnit_SetCurveValue.h"
#include "Units/Hierarchy/RigUnit_UnsetCurveValue.h"

#define LOCTEXT_NAMESPACE "SRigCurveContainer"

static const FName ColumnId_RigCurveNameLabel( "Curve" );
static const FName ColumnID_RigCurveValueLabel( "Value" );

//////////////////////////////////////////////////////////////////////////
// SRigCurveListRow

void SRigCurveListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	OnTextCommitted = InArgs._OnTextCommitted;
	OnSetRigCurveValue = InArgs._OnSetRigCurveValue;
	OnGetRigCurveValue = InArgs._OnGetRigCurveValue;
	OnGetFilterText = InArgs._OnGetFilterText;

	check( Item.IsValid() );

	SMultiColumnTableRow< FDisplayedRigCurveInfoPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SRigCurveListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == ColumnId_RigCurveNameLabel )
	{
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SAssignNew(Item->EditableText, SInlineEditableTextBlock)
				.OnTextCommitted(OnTextCommitted)
				.ColorAndOpacity(Item->TextColor)
				.IsSelected(this, &SRigCurveListRow::IsSelected)
				.Text(this, &SRigCurveListRow::GetItemName)
				.HighlightText(this, &SRigCurveListRow::GetFilterText)
			];
	}
	else if ( ColumnName == ColumnID_RigCurveValueLabel )
	{
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 1.0f )
			.VAlign( VAlign_Center )
			[
				SNew( SSpinBox<float> )
				.Value( this, &SRigCurveListRow::GetValue )
				.OnValueChanged( this, &SRigCurveListRow::OnRigCurveValueChanged )
				.OnValueCommitted( this, &SRigCurveListRow::OnRigCurveValueValueCommitted )
				.IsEnabled(false)
			];
	}

	return SNullWidget::NullWidget;
}

void SRigCurveListRow::OnRigCurveValueChanged( float NewValue )
{
	Item->Value = NewValue;

	OnSetRigCurveValue.ExecuteIfBound(Item->CurveName, NewValue);
}

void SRigCurveListRow::OnRigCurveValueValueCommitted( float NewValue, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		OnRigCurveValueChanged(NewValue);
	}
}


FText SRigCurveListRow::GetItemName() const
{
	return FText::FromName(Item->CurveName);
}

FText SRigCurveListRow::GetFilterText() const
{
	if (OnGetFilterText.IsBound())
	{
		return OnGetFilterText.Execute();
	}
	else
	{
		return FText::GetEmpty();
	}
}


float SRigCurveListRow::GetValue() const 
{ 
	if (OnGetRigCurveValue.IsBound())
	{
		return OnGetRigCurveValue.Execute(Item->CurveName);
	}

	return 0.f;
}

//////////////////////////////////////////////////////////////////////////
// SRigCurveContainer

void SRigCurveContainer::Construct(const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor)
{
	ControlRigEditor = InControlRigEditor;
	ControlRigBlueprint = InControlRigEditor.Get().GetControlRigAssetInterface();
	bIsChangingRigHierarchy = false;
	
	const TSharedRef<IRigVMEditor> RigVMEditor = InControlRigEditor->SharedRigVMEditorRef();
	const TScriptInterface<IRigVMEditorAssetInterface> AssetInterface = RigVMEditor->GetRigVMAssetInterface();
	if (AssetInterface)
	{
		EdGraphNodeFunctionRegistry = FRigVMEdGraphNodeRegistry::GetOrCreateRegistry(AssetInterface, URigVMUnitNode::StaticClass());
	}

	ControlRigBlueprint->GetHierarchy()->OnModified().AddRaw(this, &SRigCurveContainer::OnHierarchyModified_AnyThread);
	ControlRigBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().AddRaw(this, &SRigCurveContainer::HandleSetObjectBeingDebugged);
	ControlRigBlueprint->GetRigVMAssetInterface()->OnRefreshEditor().AddRaw(this, &SRigCurveContainer::HandleRefreshEditorFromBlueprint);

	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}


	// Register and bind all our menu commands
	FCurveContainerCommands::Register();
	BindCommands();

	ChildSlot
	[
		SNew( SVerticalBox )
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth( 1 )
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SRigCurveContainer::OnFilterTextChanged )
				.OnTextCommitted( this, &SRigCurveContainer::OnFilterTextCommitted )
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( RigCurveListView, SRigCurveListType )
			.ListItemsSource( &RigCurveList )
			.OnGenerateRow( this, &SRigCurveContainer::GenerateRigCurveRow )
			.OnContextMenuOpening( this, &SRigCurveContainer::OnGetContextMenuContent )
			.SelectionMode(ESelectionMode::Multi)
			.OnSelectionChanged( this, &SRigCurveContainer::OnSelectionChanged )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_RigCurveNameLabel )
				.FillWidth(1.f)
				.DefaultLabel( LOCTEXT( "RigCurveNameLabel", "Curve" ) )

				+ SHeaderRow::Column( ColumnID_RigCurveValueLabel )
				.FillWidth(1.f)
				.DefaultLabel( LOCTEXT( "RigCurveValueLabel", "Value" ) )
			)
		]
	];

	CreateRigCurveList();

	if(ControlRigEditor.IsValid())
	{
		ControlRigEditor.Pin()->OnEditorClosed().AddSP(this, &SRigCurveContainer::OnEditorClose);
	}

	// after opening the editor the debugged rig won't exist yet. we'll have to wait for a tick so
	// that we have a valid rig to listen to.
	RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double, float) {
		if(ControlRigBlueprint.IsValid())
		{
			(void)HandleSetObjectBeingDebugged(ControlRigBlueprint->GetDebuggedControlRig());
		}
		return EActiveTimerReturnType::Stop;
	}));
}

SRigCurveContainer::~SRigCurveContainer()
{
	IControlRigBaseEditor* Editor = ControlRigEditor.IsValid() ? ControlRigEditor.Pin().Get() : nullptr;
	OnEditorClose(Editor, ControlRigBlueprint.ToScriptInterface());
}

FReply SRigCurveContainer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SRigCurveContainer::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());

	UICommandList = MakeShareable(new FUICommandList);

	FUICommandList& CommandList = *UICommandList;

	// Grab the list of menu commands to bind...
	const FCurveContainerCommands& MenuActions = FCurveContainerCommands::Get();

	// ...and bind them all

	CommandList.MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SRigCurveContainer::OnRenameClicked),
		FCanExecuteAction::CreateSP(this, &SRigCurveContainer::CanRename));

	CommandList.MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SRigCurveContainer::OnDeleteNameClicked),
		FCanExecuteAction::CreateSP(this, &SRigCurveContainer::CanDelete));

	CommandList.MapAction(
		MenuActions.AddCurve,
		FExecuteAction::CreateSP(this, &SRigCurveContainer::OnAddClicked),
		FCanExecuteAction());
}

void SRigCurveContainer::OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh)
{
	RefreshCurveList_AnyThread();
}

void SRigCurveContainer::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	RefreshCurveList_AnyThread();
}

void SRigCurveContainer::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SRigCurveContainer::GenerateRigCurveRow(FDisplayedRigCurveInfoPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew( SRigCurveListRow, OwnerTable)
		.Item( InInfo )
		.OnTextCommitted(this, &SRigCurveContainer::OnNameCommitted, InInfo)
		.OnSetRigCurveValue(this, &SRigCurveContainer::SetCurveValue)
		.OnGetRigCurveValue(this, &SRigCurveContainer::GetCurveValue)
		.OnGetFilterText(this, &SRigCurveContainer::GetFilterText);
}

TSharedPtr<SWidget> SRigCurveContainer::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, UICommandList);

	const FCurveContainerCommands& Actions = FCurveContainerCommands::Get();

	MenuBuilder.BeginSection("RigCurveAction", LOCTEXT( "CurveAction", "Curve Actions" ) );

	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameSmartNameLabel", "Rename Curve"), LOCTEXT("RenameSmartNameToolTip", "Rename the selected curve"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("DeleteSmartNameLabel", "Delete Curve"), LOCTEXT("DeleteSmartNameToolTip", "Delete the selected curve"));
	MenuBuilder.AddMenuEntry(Actions.AddCurve);
	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddSubMenu(
		LOCTEXT("ImportSubMenu", "Import"),
		LOCTEXT("ImportSubMenu_ToolTip", "Import curves to the current rig. This only imports non-existing curve."),
		FNewMenuDelegate::CreateSP(const_cast<SRigCurveContainer*>(this), &SRigCurveContainer::CreateImportMenu)
	);


	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRigCurveContainer::OnEditorClose(IControlRigBaseEditor* InEditor, FControlRigAssetInterfacePtr InBlueprint)
{
	if (InBlueprint)
	{
		InBlueprint->GetHierarchy()->OnModified().RemoveAll(this);
		InBlueprint->GetRigVMAssetInterface()->OnRefreshEditor().RemoveAll(this);
		InBlueprint->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().RemoveAll(this);
	}

	if (InEditor)
	{
		InEditor->OnEditorClosed().RemoveAll(this);
	}

	ControlRigBlueprint.Reset();
	ControlRigEditor.Reset();
}

void SRigCurveContainer::OnRenameClicked()
{
	TArray< FDisplayedRigCurveInfoPtr > SelectedItems = RigCurveListView->GetSelectedItems();
	
	if (!SelectedItems.IsEmpty())
	{
		SelectedItems[0]->EditableText->EnterEditingMode();
	}
}

bool SRigCurveContainer::CanRename()
{
	return RigCurveListView->GetNumItemsSelected() == 1;
}

void SRigCurveContainer::OnAddClicked()
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewSmartnameLabel", "New Name"))
		.OnTextCommitted(this, &SRigCurveContainer::CreateNewNameEntry);

	FSlateApplication& SlateApp = FSlateApplication::Get();
	SlateApp.PushMenu(
		AsShared(),
		FWidgetPath(),
		TextEntry,
		SlateApp.GetCursorPos(),
		FPopupTransitionEffect::TypeInPopup
		);
}


void SRigCurveContainer::CreateNewNameEntry(const FText& CommittedText, ETextCommit::Type CommitType)
{
	FSlateApplication::Get().DismissAllMenus();
	if (!CommittedText.IsEmpty() && CommitType == ETextCommit::OnEnter)
	{
		URigHierarchy* Hierarchy = GetDefaultHierarchy();
		if (Hierarchy)
		{
			TGuardValue<bool> GuardReentry(bIsChangingRigHierarchy, true);

			const FName NewName = FName(*CommittedText.ToString());

			if(URigHierarchyController* Controller = Hierarchy->GetController())
			{
				const FRigElementKey CurveKey = Controller->AddCurve(NewName, 0.f, true);
				Controller->ClearSelection();
				Controller->SelectElement(CurveKey);
			}
		}

		FSlateApplication::Get().DismissAllMenus();
		RefreshCurveList();
	}
}

void SRigCurveContainer::CreateRigCurveList( const FString& SearchText )
{
	if(bIsChangingRigHierarchy)
	{
		return;
	}
	TGuardValue<bool> GuardReentry(bIsChangingRigHierarchy, true);

	if (!ControlRigBlueprint.IsValid())
	{
		return;
	}
	
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		RigCurveList.Reset();

		// Iterate through all curves..
		Hierarchy->ForEach<FRigCurveElement>([this](FRigCurveElement* CurveElement) -> bool
		{
			const FString CurveString = CurveElement->GetName();
			
			// See if we pass the search filter
            if (!FilterText.IsEmpty() && !CurveString.Contains(*FilterText.ToString()))
            {
                return true;
            }

			const FSlateColor ItemTextColor = IsCurveElementUsedInGraph(*CurveElement) ?
				FSlateColor::UseForeground() :
				FSlateColor::UseSubduedForeground();

            const TSharedRef<FDisplayedRigCurveInfo> NewItem = FDisplayedRigCurveInfo::Make(CurveElement->GetFName(), ItemTextColor);
            RigCurveList.Add(NewItem);

			return true;
		});

		// Sort final list
		struct FSortNamesAlphabetically
		{
			bool operator()(const FDisplayedRigCurveInfoPtr& A, const FDisplayedRigCurveInfoPtr& B) const
			{
				return (A.Get()->CurveName.Compare(B.Get()->CurveName) < 0);
			}
		};

		RigCurveList.Sort(FSortNamesAlphabetically());
	}
	RigCurveListView->RequestListRefresh();

	if (Hierarchy)
	{
		TArray<FRigElementKey> SelectedCurveKeys = Hierarchy->GetSelectedKeys(ERigElementType::Curve);
		for (const FRigElementKey& SelectedCurveKey : SelectedCurveKeys)
		{
			if(FRigCurveElement* CurveElement = Hierarchy->Find<FRigCurveElement>(SelectedCurveKey))
			{
				OnHierarchyModified(ERigHierarchyNotification::ElementSelected, Hierarchy, CurveElement);
			}
		}
	}
}

bool SRigCurveContainer::IsCurveElementUsedInGraph(const FRigCurveElement& CurveElement) const
{
	if (!ensure(EdGraphNodeFunctionRegistry.IsValid()))
	{
		return false;
	}

	const TArray<TWeakObjectPtr<URigVMEdGraphNode>>& WeakConnectedEdGraphNodes = EdGraphNodeFunctionRegistry->GetConnectedEdGrapNodes();
	
	return 
		WeakConnectedEdGraphNodes.ContainsByPredicate(
			[&CurveElement](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
				{
					const URigVMUnitNode* UnitNode = WeakEdGraphNode.IsValid() ? Cast<URigVMUnitNode>(WeakEdGraphNode->GetModelNode()) : nullptr;
					const UScriptStruct* ScriptStruct = UnitNode ? UnitNode->GetScriptStruct() : nullptr;
            					
					if (UnitNode &&	ScriptStruct)
					{
						if (ScriptStruct == FRigUnit_CurveExists::StaticStruct())
						{
							const URigVMPin* CurvePin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_CurveExists, Curve));
							return CurvePin && CurvePin->GetDefaultValue() == CurveElement.GetName();
						}    
						else if (ScriptStruct == FRigUnit_GetCurveValue::StaticStruct())
						{
							const URigVMPin* CurvePin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_GetCurveValue, Curve));
							return CurvePin && CurvePin->GetDefaultValue() == CurveElement.GetName();
						}
						else if (ScriptStruct == FRigUnit_SetCurveValue::StaticStruct())
						{
							const URigVMPin* CurvePin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_SetCurveValue, Curve));
							return CurvePin && CurvePin->GetDefaultValue() == CurveElement.GetName();
						}            		
						else if (ScriptStruct == FRigUnit_UnsetCurveValue::StaticStruct())
						{
							const URigVMPin* CurvePin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_UnsetCurveValue, Curve));
							return CurvePin && CurvePin->GetDefaultValue() == CurveElement.GetName();
						}
					}
					
					return false;
				});
}

void SRigCurveContainer::RefreshCurveList_AnyThread()
{
	if(IsInGameThread())
	{
		RefreshCurveList();
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]()
		{
			RefreshCurveList();
		});
	}
}

void SRigCurveContainer::RefreshCurveList()
{
	CreateRigCurveList(FilterText.ToString());
}

void SRigCurveContainer::OnNameCommitted(const FText& InNewName, ETextCommit::Type CommitType, FDisplayedRigCurveInfoPtr Item)
{
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		if (CommitType == ETextCommit::OnEnter)
		{
			if(URigHierarchyController* Controller = Hierarchy->GetController())
			{
				FName NewName = FName(*InNewName.ToString());
				FName OldName = Item->CurveName;
				Controller->RenameElement(FRigElementKey(OldName, ERigElementType::Curve), NewName, true, true);
			}
		}
	}
}

void SRigCurveContainer::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		RefreshCurveList();
	}
}

void SRigCurveContainer::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		RefreshCurveList();
	}
}

void SRigCurveContainer::OnDeleteNameClicked()
{
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

		TArray< FDisplayedRigCurveInfoPtr > SelectedItems = RigCurveListView->GetSelectedItems();
		for (auto Item : SelectedItems)
		{
			if(URigHierarchyController* Controller = Hierarchy->GetController())
			{
				Controller->RemoveElement(FRigElementKey(Item->CurveName, ERigElementType::Curve), true, true);
			}
		}
	}

	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
	RefreshCurveList();
}

bool SRigCurveContainer::CanDelete()
{
	return RigCurveListView->GetNumItemsSelected() > 0;
}

void SRigCurveContainer::SetCurveValue(const FName& CurveName, float CurveValue)
{
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		Hierarchy->SetCurveValue(FRigElementKey(CurveName, ERigElementType::Curve), CurveValue, true);
	}
}

float SRigCurveContainer::GetCurveValue(const FName& CurveName)
{
	// For actions that involve topological changes, the code uses the default hierarchy
	// But here, we use the instance, in order to be able to see the curve values while they change in the debugged instance
	URigHierarchy* Hierarchy = GetHierarchy();
	if (Hierarchy)
	{
		return Hierarchy->GetCurveValue(FRigElementKey(CurveName, ERigElementType::Curve));
	}
	return 0.f;
}

void SRigCurveContainer::ChangeCurveName(const FName& OldName, const FName& NewName)
{
	TGuardValue<bool> GuardReentry(bIsChangingRigHierarchy, true);

	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		if(URigHierarchyController* Controller = Hierarchy->GetController())
		{
			Controller->RenameElement(FRigElementKey(OldName, ERigElementType::Curve), NewName, true, true);
		}
	}
}

void SRigCurveContainer::OnSelectionChanged(FDisplayedRigCurveInfoPtr Selection, ESelectInfo::Type SelectInfo)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}

	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		URigHierarchyController* Controller = Hierarchy->GetController();
		if(Controller == nullptr)
		{
			return;
		}

		FScopedTransaction ScopedTransaction(LOCTEXT("SelectCurveTransaction", "Select Curve"), !GIsTransacting);

		TGuardValue<bool> GuardRigHierarchyChanges(bIsChangingRigHierarchy, true);

		TArray<FRigElementKey> OldSelection = Hierarchy->GetSelectedKeys(ERigElementType::Curve);
		TArray<FRigElementKey> NewSelection;

		TArray<FDisplayedRigCurveInfoPtr> SelectedItems = RigCurveListView->GetSelectedItems();
		for (const FDisplayedRigCurveInfoPtr& SelectedItem : SelectedItems)
		{
			NewSelection.Add(FRigElementKey(SelectedItem->CurveName, ERigElementType::Curve));
		}

		for (const FRigElementKey& PreviouslySelected : OldSelection)
		{
			if (NewSelection.Contains(PreviouslySelected))
			{
				continue;
			}
			Controller->DeselectElement(PreviouslySelected);
		}

		for (const FRigElementKey& NewlySelected : NewSelection)
		{
			Controller->SelectElement(NewlySelected);
		}
	}
}

void SRigCurveContainer::OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if(bIsChangingRigHierarchy)
	{
		return;
	}
	
	if(!ControlRigBeingDebuggedPtr.IsValid())
	{
		return;
	}

	if(InHierarchy != ControlRigBeingDebuggedPtr->GetHierarchy())
	{
		return;
	}

	if(IsInGameThread())
	{
		OnHierarchyModified(InNotif, InHierarchy, InSubject);
	}
	else
	{
		FRigElementKey Key;
		if(const FRigBaseElement* InElement = InSubject.Element)
		{
			Key = InElement->GetKey();
		}
		
		TWeakObjectPtr<URigHierarchy> WeakHierarchy = InHierarchy;
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, InNotif, WeakHierarchy,Key]()
		{
			if(!WeakHierarchy.IsValid())
			{
				return;
			}

			if (const FRigBaseElement* Element = WeakHierarchy.Get()->Find(Key))
			{
				OnHierarchyModified(InNotif, WeakHierarchy.Get(), Element);
			}
		});
	}
}

void SRigCurveContainer::OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if (bIsChangingRigHierarchy)
	{
		return;
	}

	const FRigBaseElement* InElement = InSubject.Element;
	if(InElement)
	{
		if(!InElement->IsTypeOf(ERigElementType::Curve))
		{
			return;
		}
	}

	if (ControlRigBlueprint.IsValid())
	{
		if (ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications)
		{
			return;
		}
	}

	switch(InNotif)
	{
		case ERigHierarchyNotification::ElementAdded:
		case ERigHierarchyNotification::ElementRemoved:
		case ERigHierarchyNotification::ElementRenamed:
		case ERigHierarchyNotification::ElementReordered:
		case ERigHierarchyNotification::HierarchyReset:
		{
			RefreshCurveList_AnyThread();
			break;
		}
		case ERigHierarchyNotification::ElementSelected:
    	case ERigHierarchyNotification::ElementDeselected:
		{
			if(InElement)
			{
				const bool bSelected = InNotif == ERigHierarchyNotification::ElementSelected;
				for(const FDisplayedRigCurveInfoPtr& Item : RigCurveList)
				{
					if (Item->CurveName == InElement->GetFName())
					{
						RigCurveListView->SetItemSelection(Item, bSelected);
						break;
					}
				}
			}
			break;
		}
		default:
		{
			break;
		}
    }
}

void SRigCurveContainer::OnPostConstruction(UControlRig* InControlRig, const FName& InName)
{
	RefreshCurveList_AnyThread();
}

void SRigCurveContainer::HandleSetObjectBeingDebugged(UObject* InObject)
{
	if(ControlRigBeingDebuggedPtr.Get() == InObject)
	{
		return;
	}

	if(ControlRigBeingDebuggedPtr.IsValid())
	{
		if(UControlRig* ControlRigBeingDebugged = ControlRigBeingDebuggedPtr.Get())
		{
			if(!URigVMHost::IsGarbageOrDestroyed(ControlRigBeingDebugged))
			{
				ControlRigBeingDebugged->GetHierarchy()->OnModified().RemoveAll(this);
			}
		}
	}

	ControlRigBeingDebuggedPtr.Reset();
	
	if(UControlRig* ControlRig = Cast<UControlRig>(InObject))
	{
		ControlRigBeingDebuggedPtr = ControlRig;
		if(URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			Hierarchy->OnModified().RemoveAll(this);
			Hierarchy->OnModified().AddSP(this, &SRigCurveContainer::OnHierarchyModified_AnyThread);
		}
		ControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
		ControlRig->OnPostConstruction_AnyThread().AddSP(this, &SRigCurveContainer::OnPostConstruction);
	}

	RefreshCurveList_AnyThread();
}

void SRigCurveContainer::HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	RefreshCurveList_AnyThread();
}

void SRigCurveContainer::CreateImportMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddWidget(
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("ControlRig.Curve.Menu"))
			.Text(LOCTEXT("ImportMesh_Title", "Select Mesh"))
			.ToolTipText(LOCTEXT("ImportMesh_Tooltip", "Select Mesh to import Curve from... It will only import if the node doesn't exist in the current Curve."))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(3)
		[
			SNew(SObjectPropertyEntryBox)
			//.ObjectPath_UObject(this, &SRigCurveContainer::GetCurrentHierarchy)
			.OnShouldFilterAsset(this, &SRigCurveContainer::ShouldFilterOnImport)
			.OnObjectChanged(this, &SRigCurveContainer::ImportCurve)
		]
		,
		FText()
		);
}

bool SRigCurveContainer::ShouldFilterOnImport(const FAssetData& AssetData) const
{
	return (AssetData.AssetClassPath != USkeletalMesh::StaticClass()->GetClassPathName() &&
		AssetData.AssetClassPath != USkeleton::StaticClass()->GetClassPathName());
}

void SRigCurveContainer::ImportCurve(const FAssetData& InAssetData)
{
	URigHierarchy* Hierarchy = GetDefaultHierarchy();
	if (Hierarchy)
	{
		TGuardValue<bool> SuspendBlueprintNotifs(ControlRigBlueprint->GetRigVMAssetInterface()->bSuspendAllNotifications, true);

		USkeleton* Skeleton = nullptr;
		USkeletalMesh* Mesh = Cast<USkeletalMesh>(InAssetData.GetAsset());
		if (Mesh)
		{
			Skeleton = Mesh->GetSkeleton();
			ControlRigBlueprint->GetSourceCurveImport() = Skeleton;
		}
		else 
		{
			Skeleton = Cast<USkeleton>(InAssetData.GetAsset());
			ControlRigBlueprint->GetSourceCurveImport() = Skeleton;
		}

		if (Skeleton)
		{
			TGuardValue<bool> GuardReentry(bIsChangingRigHierarchy, true);

			FScopedTransaction Transaction(LOCTEXT("CurveImport", "Import Curve"));
			ControlRigBlueprint->Modify();
			
			if(URigHierarchyController* Controller = Hierarchy->GetController())
			{
				Controller->ClearSelection();
				if(Mesh)
				{
					Controller->ImportCurvesFromSkeletalMesh(Mesh, NAME_None, false, true, true);
				}
				else
				{
					Controller->ImportCurves(Skeleton, NAME_None, false, true, true);
				}
			}

			FSlateApplication::Get().DismissAllMenus();
		}
	}

	RefreshCurveList();
	ControlRigBlueprint->PropagateHierarchyFromBPToInstances();
}

URigHierarchy* SRigCurveContainer::GetHierarchy() const
{
	if (ControlRigBlueprint.IsValid())
	{
		if (UControlRig* DebuggedRig = ControlRigBeingDebuggedPtr.Get())
		{
			return DebuggedRig->GetHierarchy();
		}
	}
	if (ControlRigEditor.IsValid())
	{
		if (UControlRig* CurrentRig = ControlRigEditor.Pin()->GetControlRig())
		{
			return CurrentRig->GetHierarchy();
		}
	}
	return GetDefaultHierarchy();
}

URigHierarchy* SRigCurveContainer::GetDefaultHierarchy() const
{
	if (ControlRigBlueprint.IsValid())
	{
		return ControlRigBlueprint->GetHierarchy();
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
