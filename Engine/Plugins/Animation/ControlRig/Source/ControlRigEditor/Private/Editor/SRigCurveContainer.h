// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Rigs/RigHierarchy.h"
#include "ControlRigBlueprintLegacy.h"
#include "ControlRigEditor.h"
#include "EditorUndoClient.h"

//////////////////////////////////////////////////////////////////////////
// FDisplayedRigCurveInfo

class SInlineEditableTextBlock;
class IControlRigBaseEditor;
struct FAssetData;
class URigVMBlueprint;
class FRigVMEditorBase;
namespace UE::RigVMEditor { class FRigVMEdGraphNodeRegistry; }

class FDisplayedRigCurveInfo
{
public:
	const FName CurveName;
	const FSlateColor TextColor;
	float Value;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedRigCurveInfo> Make(const FName& InCurveName, const FSlateColor& InTextColor)
	{
		return MakeShareable(new FDisplayedRigCurveInfo(InCurveName, InTextColor));
	}

	// editable text
	TSharedPtr<SInlineEditableTextBlock> EditableText;

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedRigCurveInfo(const FName& InCurveName, const FSlateColor& InTextColor)
		: CurveName(InCurveName)
		, TextColor(InTextColor)
		, Value(0.f)
	{}
};

typedef TSharedPtr< FDisplayedRigCurveInfo > FDisplayedRigCurveInfoPtr;
typedef SListView< FDisplayedRigCurveInfoPtr > SRigCurveListType;

//////////////////////////////////////////////////////////////////////////
// SRigCurveListRow

DECLARE_DELEGATE_TwoParams(FSetRigCurveValue, const FName&, float);
DECLARE_DELEGATE_RetVal_OneParam(float, FGetRigCurveValue, const FName&);
DECLARE_DELEGATE_RetVal(FText&, FGetFilterText);

class SRigCurveListRow : public SMultiColumnTableRow< FDisplayedRigCurveInfoPtr >
{
public:

	SLATE_BEGIN_ARGS(SRigCurveListRow) {}

		/** The item for this row **/
		SLATE_ARGUMENT(FDisplayedRigCurveInfoPtr, Item)

		/** Callback when the text is committed. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** set the value delegate */
		SLATE_EVENT(FSetRigCurveValue, OnSetRigCurveValue)

		/** set the value delegate */
		SLATE_EVENT(FGetRigCurveValue, OnGetRigCurveValue)

		/** get filter text */
		SLATE_EVENT(FGetFilterText, OnGetFilterText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/**
	* Called when the user changes the value of the SSpinBox
	*
	* @param NewValue - The new number the SSpinBox is set to
	* this may not work right if the rig is being evaluated
	* this just overwrite the data when it's written
	*
	*/
	void OnRigCurveValueChanged(float NewValue);

	/**
	* Called when the user types the value and enters
	*
	* @param NewValue - The new number the SSpinBox is set to
	*
	*/
	void OnRigCurveValueValueCommitted(float NewValue, ETextCommit::Type CommitType);

	/** Returns the Value of this curve */
	float GetValue() const;
	/** Returns name of this curve */
	FText GetItemName() const;
	/** Get text we are filtering for */
	FText GetFilterText() const;

	/** The name and Value of the morph target */
	FDisplayedRigCurveInfoPtr	Item;

	FOnTextCommitted OnTextCommitted;
	FSetRigCurveValue OnSetRigCurveValue;
	FGetRigCurveValue OnGetRigCurveValue;
	FGetFilterText OnGetFilterText;
};

//////////////////////////////////////////////////////////////////////////
// SRigCurveContainer

class SRigCurveContainer : public SCompoundWidget, public FEditorUndoClient
{
	using FRigVMEdGraphNodeRegistry = UE::RigVMEditor::FRigVMEdGraphNodeRegistry;

public:
	SLATE_BEGIN_ARGS( SRigCurveContainer )
	{}
	
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct( const FArguments& InArgs, TSharedRef<IControlRigBaseEditor> InControlRigEditor);

	/**
	* Destructor - resets the animation curve
	*
	*/
	virtual ~SRigCurveContainer();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/**
	* Is registered with ControlRig Editor to handle when its preview mesh is changed.
	*
	* @param NewPreviewMesh - The new preview mesh being used by ControlRig Editor
	*
	*/
	void OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh);

	/**
	* Is registered with ControlRig Editor to handle when its preview asset is changed.
	*
	* Pose Asset will have to add curve manually
	*/
	void OnPreviewAssetChanged(class UAnimationAsset* NewPreviewAsset);

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged( const FText& SearchText );


	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GenerateRigCurveRow(FDisplayedRigCurveInfoPtr InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	void RefreshCurveList_AnyThread();
	void RefreshCurveList();

	// When a name is committed after being edited in the list
	virtual void OnNameCommitted(const FText& NewName, ETextCommit::Type CommitType, FDisplayedRigCurveInfoPtr Item);

	// FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

private:

	void BindCommands();

	/** Handler for context menus */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;

	void OnEditorClose(IControlRigBaseEditor* InEditor, FControlRigAssetInterfacePtr InBlueprint);

	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreateRigCurveList( const FString& SearchText = FString() );
	
	/** Returns true if the curve is used in the graph */
	bool IsCurveElementUsedInGraph(const FRigCurveElement& CurveElement) const;

	void OnDeleteNameClicked();
	bool CanDelete();

	void OnRenameClicked();
	bool CanRename();

	void OnAddClicked();

	// Adds a new smartname entry to the skeleton in the container we are managing
	void CreateNewNameEntry(const FText& CommittedText, ETextCommit::Type CommitType);

	// delegate for changing info
	void SetCurveValue(const FName& CurveName, float CurveValue);
	float GetCurveValue(const FName& CurveName);
	void ChangeCurveName(const FName& OldName, const FName& NewName);

	void OnSelectionChanged(FDisplayedRigCurveInfoPtr Selection, ESelectInfo::Type SelectInfo);

	bool bIsChangingRigHierarchy;
	void OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	void OnPostConstruction(UControlRig* InControlRig, const FName& InName);
	void HandleSetObjectBeingDebugged(UObject* InObject);
	void HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint);

	// import curve part
	void ImportCurve(const FAssetData& InAssetData);
	void CreateImportMenu(FMenuBuilder& MenuBuilder);
	bool ShouldFilterOnImport(const FAssetData& AssetData) const;

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** A list of animation curve. Used by the RigCurveListView. */
	TArray< FDisplayedRigCurveInfoPtr > RigCurveList;

	/** Widget used to display the list of animation curve */
	TSharedPtr<SRigCurveListType> RigCurveListView;

	/** Control Rig Blueprint */
	TWeakInterfacePtr<IControlRigEditorAssetInterface> ControlRigBlueprint;
	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;
	TWeakObjectPtr<UControlRig> ControlRigBeingDebuggedPtr;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Function graph node registry to find coneected curves */
	TSharedPtr<FRigVMEdGraphNodeRegistry> EdGraphNodeFunctionRegistry;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;

	URigHierarchy* GetHierarchy() const;
	URigHierarchy* GetDefaultHierarchy() const;

	friend class SRigCurveListRow;
	friend class SRigCurveTypeList;
};