// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigModuleAssetBrowser.h"

#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserModule.h"
#include "ControlRigBlueprintLegacy.h"
#include "IContentBrowserSingleton.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "ControlRigEditor.h"
#include "ControlRigShowSchematicViewportOverride.h"
#include "FrontendFilterBase.h"
#include "Editor/RigVMEditorTools.h"
#include "Styling/SlateIconFinder.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SRigVMVariantWidget.h"

#define LOCTEXT_NAMESPACE "RigModuleAssetBrowser"

namespace UE::Editor::ContentBrowser
{
	static bool IsNewStyleEnabled()
	{
		static bool bIsNewStyleEnabled = [&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle")))
			{
				ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline, @see: UnrealEdGlobals.cpp, UE::Editor::ContentBrowser::EnableContentBrowserNewStyleCVarRegistration."));
				return CVar->GetBool();
			}
			return false;
		}();

		return bIsNewStyleEnabled;
	}
}

void SRigModuleAssetBrowser::Construct(
	const FArguments& InArgs)
{
	bAllowDragging = InArgs._bAllowDragging;
	AssetViewType = InArgs._AssetViewType;
	CommandList = MakeShared<FUICommandList>();

	const FGlobalEditorCommonCommands& GlobalCommands = FGlobalEditorCommonCommands::Get();
	CommandList->MapAction(
		GlobalCommands.FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SRigModuleAssetBrowser::ExecuteFindInContentBrowserAction),
		FCanExecuteAction::CreateSP(this, &SRigModuleAssetBrowser::CanExecuteFindInContentBrowserAction)
	);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SAssignNew(AssetBrowserBox, SBox)
		]
	];

	RefreshView();

	// Register for Asset Registry events to auto-refresh
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	OnAssetAddedHandle = AssetRegistry.OnAssetAdded().AddSP(this, &SRigModuleAssetBrowser::OnAssetAdded);
	OnAssetRemovedHandle = AssetRegistry.OnAssetRemoved().AddSP(this, &SRigModuleAssetBrowser::OnAssetRemoved);
	OnAssetRenamedHandle = AssetRegistry.OnAssetRenamed().AddSP(this, &SRigModuleAssetBrowser::OnAssetRenamed);
}

SRigModuleAssetBrowser::~SRigModuleAssetBrowser()
{
	// Unregister Asset Registry delegates
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().Remove(OnAssetAddedHandle);
			AssetRegistry->OnAssetRemoved().Remove(OnAssetRemovedHandle);
			AssetRegistry->OnAssetRenamed().Remove(OnAssetRenamedHandle);
		}
	}
	
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(DeferredRefreshHandle);
	}
}

void SRigModuleAssetBrowser::RefreshView()
{
	// Save the current filter states before recreating the picker
	TMap<FString, bool> SavedFilterStates;
	for (const TSharedRef<FRigVMFilterTag>& Filter : CurrentFilters)
	{
		SavedFilterStates.Add(Filter->GetName(), Filter->IsActive());
	}

	FAssetPickerConfig AssetPickerConfig;
	
	// setup filtering
	AssetPickerConfig.Filter.ClassPaths.Add(UControlRigEditorAssetInterface::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UControlRigRuntimeAssetInterface::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.InitialAssetViewType = AssetViewType;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SRigModuleAssetBrowser::OnShouldFilterOutAsset);
	AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Blueprint;
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SRigModuleAssetBrowser::OnGetAssetContextMenu);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
	AssetPickerConfig.bAllowDragging = bAllowDragging;
	if (bAllowDragging)
	{
		AssetPickerConfig.OnAssetsDragged.BindSP(this, &SRigModuleAssetBrowser::OnAssetsDragged);
	}
	AssetPickerConfig.bAllowRename = false;
	AssetPickerConfig.bForceShowPluginContent = true;
	AssetPickerConfig.bForceShowEngineContent = true;
	AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Small;
	AssetPickerConfig.OnGetCustomAssetToolTip = FOnGetCustomAssetToolTip::CreateSP(this, &SRigModuleAssetBrowser::CreateCustomAssetToolTip);

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* DefaultControlRigBlueprint = UControlRigBlueprint::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData Context(DefaultControlRigBlueprint, EAssetRegistryTagsCaller::Uncategorized);
	DefaultControlRigBlueprint->GetAssetRegistryTags(Context);
	for (TPair<FName, UObject::FAssetRegistryTag>& AssetRegistryTagPair : Context.Tags)
	{
		AssetPickerConfig.HiddenColumnNames.Add(AssetRegistryTagPair.Value.Name.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Has Virtualized Data"));

	// allow to open the rigs directly on double click
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(this, &SRigModuleAssetBrowser::OnAssetDoubleClicked);

	TSharedRef<FFrontendFilterCategory>	ControlRigFilterCategory = MakeShared<FFrontendFilterCategory>(LOCTEXT("ControlRigFilterCategoryName", "Control Rig Tags"), LOCTEXT("ControlRigFilterCategoryToolTip", "Filter ControlRigs by variant tags specified in ControlRig Blueprint class settings"));
	const URigVMProjectSettings* Settings = GetDefault<URigVMProjectSettings>(URigVMProjectSettings::StaticClass());
	TArray<FRigVMTag> AvailableTags = Settings->VariantTags;

	TArray<TSharedRef<FRigVMFilterTag>> Filters;
	for (const FRigVMTag& CurTag : AvailableTags)
	{
		if (CurTag.bShowInUserInterface)
		{
			Filters.Add(MakeShared<FRigVMFilterTag>(ControlRigFilterCategory, CurTag));
			AssetPickerConfig.ExtraFrontendFilters.Add(Filters.Last());
		}
	}

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));

	// Update CurrentFilters and restore saved state
	CurrentFilters.Empty();
	for (TSharedRef<FRigVMFilterTag> Filter : Filters)
	{
		CurrentFilters.Add(Filter);

		// Check if we have saved state for this filter
		const bool* SavedState = SavedFilterStates.Find(Filter->GetName());
		if (SavedState != nullptr)
		{
			// Restore the saved state
			Filter->SetActive(*SavedState);
		}
		else
		{
			// First time or no saved state, use default
			Filter->SetActive(Filter->ShouldBeMarkedAsInvalid());
		}
	}
}

FReply SRigModuleAssetBrowser::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SBox::OnKeyDown(InGeometry, InKeyEvent);
}

void SRigModuleAssetBrowser::ExecuteFindInContentBrowserAction()
{
	if (!GetCurrentSelectionDelegate.IsBound())
	{
		return;
	}

	const TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(SelectedAssets);
}

bool SRigModuleAssetBrowser::CanExecuteFindInContentBrowserAction()
{
	return GetCurrentSelectionDelegate.IsBound() && !GetCurrentSelectionDelegate.Execute().IsEmpty();
}

TSharedPtr<SWidget> SRigModuleAssetBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const
{
	if (SelectedAssets.IsEmpty())
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, CommandList);
	MenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("AssetSectionLabel", "Asset"));
	{
		const TSharedPtr<FUICommandInfo> FindInContentBrowserCommand = FGlobalEditorCommonCommands::Get().FindInContentBrowser;
		checkf(FindInContentBrowserCommand.IsValid(), TEXT("FGlobalEditorCommonCommands::FindInContentBrowser command not found!"));
		MenuBuilder.AddMenuEntry(FindInContentBrowserCommand);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SRigModuleAssetBrowser::OnShouldFilterOutAsset(const struct FAssetData& AssetData)
{
	// Return true to filter out (hide), false to keep (show)
	// We want to show rig modules, so filter out anything that's NOT a rig module
	return !IsRigModuleAsset(AssetData);
}

void SRigModuleAssetBrowser::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	if (UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		EditorSubsystem->OpenEditorForAsset(AssetData.ToSoftObjectPath());
	}
}

void SRigModuleAssetBrowser::OnAssetsDragged(const TArray<FAssetData>& Assets)
{	
	// Show the schematic viewport during drag-dropping assets, 
	// on the next tick when FSlateApplication::IsDragDropping is true
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda(
		[WeakThis = AsWeak(), this]()
		{
			if (WeakThis.IsValid())
			{
				constexpr bool bForceShowSchematicViewport = true;
				ShowSchematicViewportOverride.OverrideDuringDragDrop(bForceShowSchematicViewport);
			}
		}));
}

TSharedRef<SToolTip> SRigModuleAssetBrowser::CreateCustomAssetToolTip(FAssetData& AssetData)
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		return CreateCustomAssetToolTipNewStyle(AssetData);
	}

	// Make a list of tags to show
	TArray<UObject::FAssetRegistryTag> Tags;
	UClass* AssetClass = FindObject<UClass>(AssetData.AssetClassPath);
	check(AssetClass);
	UObject* DefaultObject = AssetClass->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(DefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	DefaultObject->GetAssetRegistryTags(TagsContext);

	TArray<FName> TagsToShow;
	static const FName ModulePath(TEXT("Path"));
	static const FName ModuleSettings(TEXT("RigModuleSettings"));
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		if(TagPair.Key == ModulePath ||
			TagPair.Key == ModuleSettings)
		{
			TagsToShow.Add(TagPair.Key);
		}
	}

	TMap<FName, FText> TagsAndValuesToShow;

	// Add asset registry tags to a text list; except skeleton as that is implied in Persona
	TSharedRef<SVerticalBox> DescriptionBox = SNew(SVerticalBox);

	static const FName AssetVariantPropertyName = TEXT("AssetVariant");
	const FProperty* AssetVariantProperty = CastField<FProperty>(AssetData.GetClass()->FindPropertyByName(AssetVariantPropertyName));
	const FString VariantStr = AssetData.GetTagValueRef<FString>(AssetVariantPropertyName);
	if(!VariantStr.IsEmpty())
	{
		FRigVMVariant AssetVariant;
		AssetVariantProperty->ImportText_Direct(*VariantStr, &AssetVariant, nullptr, EPropertyPortFlags::PPF_None);

		if(!AssetVariant.Tags.IsEmpty())
		{
			DescriptionBox->AddSlot()
			.AutoHeight()
			.Padding(0,0,5,0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AssetBrowser_RigVMTagsLabel", "Tags :"))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					SNew(SRigVMVariantTagWidget)
					.Visibility(EVisibility::Visible)
					.CanAddTags(false)
					.EnableContextMenu(false)
					.EnableTick(false)
					.Orientation(EOrientation::Orient_Horizontal)
					.OnGetTags_Lambda([AssetVariant]() { return AssetVariant.Tags; })
				]
			];
		}
	}
	
	for(TPair<FName, FAssetTagValueRef> TagPair : AssetData.TagsAndValues)
	{
		if(TagsToShow.Contains(TagPair.Key))
		{
			// Check for DisplayName metadata
			FName DisplayName;
			if (FProperty* Field = FindFProperty<FProperty>(AssetClass, TagPair.Key))
			{
				DisplayName = *Field->GetDisplayNameText().ToString();
			}
			else
			{
				DisplayName = TagPair.Key;
			}

			if (TagPair.Key == ModuleSettings)
			{
				FRigModuleSettings Settings;
				FRigVMPinDefaultValueImportErrorContext ErrorPipe;
				FRigModuleSettings::StaticStruct()->ImportText(*TagPair.Value.GetValue(), &Settings, nullptr, PPF_None, &ErrorPipe, FString());
				if (ErrorPipe.NumErrors == 0)
				{
					TagsAndValuesToShow.Add(TEXT("Default Name"), FText::FromString(Settings.Identifier.Name));
					TagsAndValuesToShow.Add(TEXT("Category"), FText::FromString(Settings.Category));
					TagsAndValuesToShow.Add(TEXT("Keywords"), FText::FromString(Settings.Keywords));
					TagsAndValuesToShow.Add(TEXT("Description"), FText::FromString(Settings.Description));
				}
			}
			else
			{
				TagsAndValuesToShow.Add(DisplayName, TagPair.Value.AsText());
			}
		}
	}

	for (const TPair<FName, FText>& TagPair : TagsAndValuesToShow)
	{
		DescriptionBox->AddSlot()
		.AutoHeight()
		.Padding(0,0,5,0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("AssetTagKey", "{0}: "), FText::FromName(TagPair.Key)))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(TagPair.Value)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	DescriptionBox->AddSlot()
		.AutoHeight()
		.Padding(0,0,5,0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AssetBrowser_FolderPathLabel", "Folder :"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromName(AssetData.PackagePath))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.WrapTextAt(300.f)
			]
		];

	TSharedPtr<SHorizontalBox> ContentBox = nullptr;
	TSharedRef<SToolTip> ToolTipWidget = SNew(SToolTip)
	.TextMargin(1.f)
	.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ToolTipBorder"))
	[
		SNew(SBorder)
		.Padding(6.f)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0,0,0,4)
			[
				SNew(SBorder)
				.Padding(6.f)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
				[
					SNew(SBox)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(FText::FromName(AssetData.AssetName))
						.Font(FAppStyle::GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
					]
				]
			]
		
			+ SVerticalBox::Slot()
			[
				SAssignNew(ContentBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBorder)
					.Padding(6.f)
					.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						DescriptionBox
					]
				]
			]
		]
	];
	return ToolTipWidget;
}

TSharedRef<SToolTip> SRigModuleAssetBrowser::CreateCustomAssetToolTipNewStyle(FAssetData& AssetData)
{
	// Make a list of tags to show
	TArray<UObject::FAssetRegistryTag> Tags;
	UClass* AssetClass = FindObject<UClass>(AssetData.AssetClassPath);
	check(AssetClass);
	UObject* DefaultObject = AssetClass->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(DefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	DefaultObject->GetAssetRegistryTags(TagsContext);

	TArray<FName> TagsToShow;
	static const FName ModulePath(TEXT("Path"));
	static const FName ModuleSettings(TEXT("RigModuleSettings"));
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		if(TagPair.Key == ModulePath ||
			TagPair.Key == ModuleSettings)
		{
			TagsToShow.Add(TagPair.Key);
		}
	}

	TSharedRef<SVerticalBox> OverallTooltipVBox = SNew(SVerticalBox);

	// Asset Name/Type Area
	{
		const FSlateBrush* ClassIcon = FAppStyle::GetDefaultBrush();
		TOptional<FLinearColor> Color;
		if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetData.GetClass()))
		{
			ClassIcon = AssetDefinition->GetIconBrush(AssetData, AssetData.AssetClassPath.GetAssetName());
			Color = AssetDefinition->GetAssetColor();
		}

		if (ClassIcon == nullptr || ClassIcon == FAppStyle::GetDefaultBrush())
		{
			ClassIcon = FSlateIconFinder::FindIconForClass(AssetData.GetClass()).GetIcon();
		}

		FText ClassNameText = LOCTEXT("ClassNameText", "Not Found");
		if (AssetClass != NULL)
		{
			ClassNameText = AssetClass->GetDisplayNameText();
		}
		else if (!AssetData.AssetClassPath.IsNull())
		{
			ClassNameText = FText::FromString(AssetData.AssetClassPath.ToString());
		}

		const FText NameText = FText::FromString(AssetData.AssetName.ToString());

		// Name/Type Slot
		OverallTooltipVBox->AddSlot()
		.AutoHeight()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(0.f, 0.f, 0.f, 6.f)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(NameText)
				.ColorAndOpacity(FStyleColors::White)
			]

			+ SVerticalBox::Slot()
			.Padding(0.f, 0.f, 0.f, 6.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(SBox)
					.WidthOverride(16.f)
					.HeightOverride(16.f)
					[
						SNew(SImage)
						.Image(ClassIcon)
						.ColorAndOpacity_Lambda([Color] () { return Color.IsSet() ? Color.GetValue() : FStyleColors::White;})
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(ClassNameText)
				]
			]
		];
	}

	// Separator
	OverallTooltipVBox->AddSlot()
	.Padding(0.f,0.f, 0.f, 6.f)
	.AutoHeight()
	[
		SNew(SSeparator)
		.Orientation(Orient_Horizontal)
		.Thickness(1.f)
		.ColorAndOpacity(COLOR("#484848FF"))
		.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
	];

	// Add asset registry tags to a text list; except skeleton as that is implied in Persona
	TMap<FName, FText> TagsAndValuesToShow;

	static const FName AssetVariantPropertyName = TEXT("AssetVariant");
	const FProperty* AssetVariantProperty = CastField<FProperty>(AssetData.GetClass()->FindPropertyByName(AssetVariantPropertyName));
	const FString VariantStr = AssetData.GetTagValueRef<FString>(AssetVariantPropertyName);
	if(!VariantStr.IsEmpty())
	{
		FRigVMVariant AssetVariant;
		AssetVariantProperty->ImportText_Direct(*VariantStr, &AssetVariant, nullptr, EPropertyPortFlags::PPF_None);

		if(!AssetVariant.Tags.IsEmpty())
		{
			OverallTooltipVBox->AddSlot()
			.AutoHeight()
			.Padding(0,0,5,0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AssetBrowser_RigVMTagsLabel_NewStyle", "Tags :"))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					SNew(SRigVMVariantTagWidget)
					.CapsuleTagBorder(FRigVMEditorStyle::Get().GetBrush("RigVM.TagCapsuleDark"))
					.Visibility(EVisibility::Visible)
					.CanAddTags(false)
					.EnableContextMenu(false)
					.EnableTick(false)
					.Orientation(EOrientation::Orient_Horizontal)
					.OnGetTags_Lambda([AssetVariant]() { return AssetVariant.Tags; })
				]
			];
		}
	}
	
	for(TPair<FName, FAssetTagValueRef> TagPair : AssetData.TagsAndValues)
	{
		if(TagsToShow.Contains(TagPair.Key))
		{
			// Check for DisplayName metadata
			FName DisplayName;
			if (FProperty* Field = FindFProperty<FProperty>(AssetClass, TagPair.Key))
			{
				DisplayName = *Field->GetDisplayNameText().ToString();
			}
			else
			{
				DisplayName = TagPair.Key;
			}

			if (TagPair.Key == ModuleSettings)
			{
				FRigModuleSettings Settings;
				FRigVMPinDefaultValueImportErrorContext ErrorPipe;
				FRigModuleSettings::StaticStruct()->ImportText(*TagPair.Value.GetValue(), &Settings, nullptr, PPF_None, &ErrorPipe, FString());
				if (ErrorPipe.NumErrors == 0)
				{
					TagsAndValuesToShow.Add(TEXT("Default Name"), FText::FromString(Settings.Identifier.Name));
					TagsAndValuesToShow.Add(TEXT("Category"), FText::FromString(Settings.Category));
					TagsAndValuesToShow.Add(TEXT("Keywords"), FText::FromString(Settings.Keywords));
					TagsAndValuesToShow.Add(TEXT("Description"), FText::FromString(Settings.Description));
				}
			}
			else
			{
				TagsAndValuesToShow.Add(DisplayName, TagPair.Value.AsText());
			}
		}
	}

	for (const TPair<FName, FText>& TagPair : TagsAndValuesToShow)
	{
		OverallTooltipVBox->AddSlot()
		.AutoHeight()
		.Padding(0,0,5,0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("AssetTagKey_NewStyle", "{0}: "), FText::FromName(TagPair.Key)))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(TagPair.Value)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

	OverallTooltipVBox->AddSlot()
		.AutoHeight()
		.Padding(0,0,5,0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AssetBrowser_FolderPathLabel_NewStyle", "Folder :"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromName(AssetData.PackagePath))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.WrapTextAt(300.f)
			]
		];

	TSharedRef<SToolTip> ToolTipWidget = SNew(SToolTip)
	.TextMargin(FMargin(12.f, 8.f, 12.f, 8.f))
	.BorderImage(FAppStyle::GetBrush("AssetThumbnail.Tooltip.Border"))
	[
		OverallTooltipVBox
	];

	return ToolTipWidget;
}

TArray<FAssetData> SRigModuleAssetBrowser::GetSelectedAssets() const
{
	if (GetCurrentSelectionDelegate.IsBound())
	{
		return GetCurrentSelectionDelegate.Execute();
	}
	return TArray<FAssetData>();
}

void SRigModuleAssetBrowser::RequestDeferredRefresh()
{
	// Defer refresh with a small delay to ensure asset metadata is fully populated
	// One tick is not always enough for the asset registry to update metadata
	GEditor->GetTimerManager()->SetTimer(
		DeferredRefreshHandle, // Reusing handle cancels previous timer
		FTimerDelegate::CreateSP(this, &SRigModuleAssetBrowser::RefreshView),
		0.1f,
		false);
}

void SRigModuleAssetBrowser::OnAssetAdded(const FAssetData& AssetData)
{
	// Check if this is a Control Rig asset (metadata about whether this asset is a rig module might not be fully populated yet)
	if (AssetData.GetClass()->ImplementsInterface(UControlRigRuntimeAssetInterface::StaticClass())
		|| AssetData.GetClass()->ImplementsInterface(UControlRigEditorAssetInterface::StaticClass()))
	{
		RequestDeferredRefresh();
	}
}

void SRigModuleAssetBrowser::OnAssetRemoved(const FAssetData& AssetData)
{
	// For removal, metadata should be available, but defer anyway for consistency
	if (IsRigModuleAsset(AssetData))
	{
		RequestDeferredRefresh();
	}
}

void SRigModuleAssetBrowser::OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath)
{
	// For rename, metadata should be available, but defer anyway for consistency
	if (IsRigModuleAsset(AssetData))
	{
		RequestDeferredRefresh();
	}
}

bool SRigModuleAssetBrowser::IsRigModuleAsset(const FAssetData& AssetData) const
{
	if (AssetData.GetClass() == nullptr)
	{
		return false;
	}
	
	// Check if this is a Control Rig asset
	if (!AssetData.GetClass()->ImplementsInterface(UControlRigRuntimeAssetInterface::StaticClass())
		&& !AssetData.GetClass()->ImplementsInterface(UControlRigEditorAssetInterface::StaticClass()))
	{
		return false;
	}

	// Check if it's a RigModule type
	static const UEnum* ControlTypeEnum = StaticEnum<EControlRigType>();
	const FString ControlRigTypeStr = AssetData.GetTagValueRef<FString>(TEXT("ControlRigType"));
	if (ControlRigTypeStr.IsEmpty())
	{
		return false;
	}

	const EControlRigType ControlRigType = (EControlRigType)(ControlTypeEnum->GetValueByName(*ControlRigTypeStr));
	return ControlRigType == EControlRigType::RigModule;
}

#undef LOCTEXT_NAMESPACE
