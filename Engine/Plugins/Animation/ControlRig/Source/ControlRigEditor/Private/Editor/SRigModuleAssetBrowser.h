// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserDelegates.h"
#include "IContentBrowserSingleton.h"
#include "Editor/ControlRigShowSchematicViewportOverride.h"
#include "Engine/TimerHandle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWindow.h"


namespace UE::RigVM::Editor::Tools
{
	class FFilterByAssetTag;
}

class IControlRigBaseEditor;

class SRigModuleAssetBrowser : public SBox
{
	using FControlRigShowSchematicViewportOverride = UE::ControlRigEditor::FControlRigShowSchematicViewportOverride;
	using FRigVMFilterTag = UE::RigVM::Editor::Tools::FFilterByAssetTag;

public:
	SLATE_BEGIN_ARGS(SRigModuleAssetBrowser)
		: _bAllowDragging(true), _AssetViewType(EAssetViewType::Tile)
		{}
		SLATE_ARGUMENT(bool, bAllowDragging)
		SLATE_ARGUMENT(EAssetViewType::Type, AssetViewType)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SRigModuleAssetBrowser();

	TArray<FAssetData> GetSelectedAssets() const;
	
private:
	void RefreshView();

	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	void ExecuteFindInContentBrowserAction();
	bool CanExecuteFindInContentBrowserAction();
	
	bool OnShouldFilterOutAsset(const struct FAssetData& AssetData);
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const;
	void OnAssetDoubleClicked(const FAssetData& AssetData);
	
	/** Called when assets where dragged */
	void OnAssetsDragged(const TArray<FAssetData>& Assets);
	
	TSharedRef<SToolTip> CreateCustomAssetToolTip(FAssetData& AssetData);
	TSharedRef<SToolTip> CreateCustomAssetToolTipNewStyle(FAssetData& AssetData);

	/** Asset Registry event handlers */
	FTimerHandle DeferredRefreshHandle;
	void RequestDeferredRefresh();
	void OnAssetAdded(const FAssetData& AssetData);
	void OnAssetRemoved(const FAssetData& AssetData);
	void OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath);
	

	/** Check if asset is a rig module */
	bool IsRigModuleAsset(const FAssetData& AssetData) const;

	/** Used to get the currently selected assets */
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
		

	/** the animation asset browser */
	TSharedPtr<SBox> AssetBrowserBox;

	/** Overrides the per project per user schematic viewport visibility */
	FControlRigShowSchematicViewportOverride ShowSchematicViewportOverride;
	
	bool bAllowDragging;
	EAssetViewType::Type AssetViewType;

	/** Commands to execute */
	TSharedPtr<FUICommandList> CommandList;

	/** Delegate handles for Asset Registry */
	FDelegateHandle OnAssetAddedHandle;
	FDelegateHandle OnAssetRemovedHandle;
	FDelegateHandle OnAssetRenamedHandle;

	/** Current filters (stored to preserve state across refreshes) */
	TArray<TSharedRef<FRigVMFilterTag>> CurrentFilters;

	friend class FControlRigBaseEditor;
};
