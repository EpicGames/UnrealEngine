// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/ControlRigGraphPanelPinFactory.h"
#include "Graph/ControlRigGraphSchema.h"
#include "Graph/ControlRigGraphNode.h"
#include "Graph/ControlRigGraph.h"
#include "Widgets/SRigVMGraphPinNameList.h"
#include "Widgets/SRigVMGraphPinCurveFloat.h"
#include "Widgets/SRigVMGraphPinVariableName.h"
#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "Widgets/SRigVMGraphPinUserDataNameSpace.h"
#include "Widgets/SRigVMGraphPinUserDataPath.h"
#include "KismetPins/SGraphPinExec.h"
#include "SGraphPinComboBox.h"
#include "ControlRig.h"
#include "NodeFactory.h"
#include "EdGraphSchema_K2.h"
#include "Curves/CurveFloat.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Units/Execution/RigUnit_DynamicHierarchy.h"
#include "Units/Hierarchy/RigUnit_Metadata.h"
#include "ControlRigElementDetails.h"
#include "IPropertyAccessEditor.h"
#include "ModularRig.h"

TSharedPtr<SGraphPin> FControlRigGraphPanelPinFactory::CreatePin(UEdGraphPin* InPin) const
{
	if(InPin == nullptr)
	{
		return nullptr;
	}

	// if the graph we are looking at is not a control rig graph - let's not do this
	if (const UEdGraphNode* OwningNode = InPin->GetOwningNode())
	{
		const UControlRigGraph* EdGraph = Cast<UControlRigGraph>(OwningNode->GetGraph());
		if(!EdGraph)
		{
			return nullptr;
		}
	}
	
	TSharedPtr<SGraphPin> InternalResult = CreatePin_Internal(InPin);
	if(InternalResult.IsValid())
	{
		return InternalResult;
	}

	TSharedPtr<SGraphPin> K2PinWidget = FNodeFactory::CreateK2PinWidget(InPin);
	if(K2PinWidget.IsValid())
	{
		if(InPin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			// if we are an enum pin - and we are inside a RigElementKey,
			// let's remove the "all" entry.
			if(InPin->PinType.PinSubCategoryObject == StaticEnum<ERigElementType>())
			{
				if(InPin->ParentPin)
				{
					if(InPin->ParentPin->PinType.PinSubCategoryObject == FRigElementKey::StaticStruct())
					{
						TSharedPtr<SWidget> ValueWidget = K2PinWidget->GetValueWidget();
						if(ValueWidget.IsValid())
						{
							if(TSharedPtr<SPinComboBox> EnumCombo = StaticCastSharedPtr<SPinComboBox>(ValueWidget))
							{
								if(EnumCombo.IsValid())
								{
									EnumCombo->RemoveItemByIndex(StaticEnum<ERigElementType>()->GetIndexByValue((int64)ERigElementType::All));
								}
							}
						}
					}
				}
			}

			const UEnum* RigControlTransformChannelEnum = StaticEnum<ERigControlTransformChannel>();
			if (InPin->PinType.PinSubCategoryObject == RigControlTransformChannelEnum)
			{
				TSharedPtr<SWidget> ValueWidget = K2PinWidget->GetValueWidget();
				if(ValueWidget.IsValid())
				{
					if(TSharedPtr<SPinComboBox> EnumCombo = StaticCastSharedPtr<SPinComboBox>(ValueWidget))
					{
						if(EnumCombo.IsValid())
						{
							if (const UControlRigGraphNode* RigNode = Cast<UControlRigGraphNode>(InPin->GetOwningNode()))
							{
								if (const URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(InPin->GetName()))
								{
									if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelPin->GetNode()))
									{
										if(UnitNode->GetScriptStruct() &&
											UnitNode->GetScriptStruct()->IsChildOf(FRigUnit_HierarchyAddControlElement::StaticStruct()))
										{
											const TSharedPtr<FStructOnScope> StructInstanceScope = UnitNode->ConstructStructInstance();
											const FRigUnit_HierarchyAddControlElement* StructInstance = 
												(const FRigUnit_HierarchyAddControlElement*)StructInstanceScope->GetStructMemory();

											if(const TArray<ERigControlTransformChannel>* VisibleChannels =
												FRigControlTransformChannelDetails::GetVisibleChannelsForControlType(StructInstance->GetControlTypeToSpawn()))
											{
												for(int32 Index = 0; Index < RigControlTransformChannelEnum->NumEnums(); Index++)
												{
													const ERigControlTransformChannel Value =
														(ERigControlTransformChannel)RigControlTransformChannelEnum->GetValueByIndex(Index);
													if(!VisibleChannels->Contains(Value))
													{
														EnumCombo->RemoveItemByIndex(Index);
													}
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
	return K2PinWidget;
}

FName FControlRigGraphPanelPinFactory::GetFactoryName() const
{
	return IControlRigEditorAssetInterface::ControlRigPanelNodeFactoryName;
}

TSharedPtr<SGraphPin> FControlRigGraphPanelPinFactory::CreatePin_Internal(UEdGraphPin* InPin) const
{
	TSharedPtr<SGraphPin> SuperResult = FRigVMEdGraphPanelPinFactory::CreatePin_Internal(InPin);
	if(SuperResult.IsValid())
	{
		return SuperResult;
	}

	if (InPin)
	{
		if (const UEdGraphNode* OwningNode = InPin->GetOwningNode())
		{
			// only create pins within control rig graphs
			if(const URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(OwningNode->GetGraph()))
			{
				if ((Cast<UControlRigGraph>(EdGraph) == nullptr) &&
					!EdGraph->IsPreviewGraph())
				{
					return nullptr;
				}
			}
		}

		if (UControlRigGraphNode* RigNode = Cast<UControlRigGraphNode>(InPin->GetOwningNode()))
		{
			UControlRigGraph* RigGraph = Cast<UControlRigGraph>(RigNode->GetGraph());

			if (URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(InPin->GetName()))
			{
				if(ModelPin->GetDirection() != ERigVMPinDirection::Input && 
					ModelPin->GetDirection() != ERigVMPinDirection::IO && 
					ModelPin->GetDirection() != ERigVMPinDirection::Visible)
				{
					return nullptr;
				}
				
				FName CustomWidgetName = ModelPin->GetCustomWidgetName();
				if (CustomWidgetName == TEXT("BoneName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetBoneNameList)
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
			}
				else if (CustomWidgetName == TEXT("ControlName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetControlNameListWithoutAnimationChannels)
						.OnGetNameListContentForValidation_UObject(RigGraph, &UControlRigGraph::GetControlNameList)
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
				}
				else if (CustomWidgetName == TEXT("SpaceName") || CustomWidgetName == TEXT("NullName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetNullNameList)
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
				}
				else if (CustomWidgetName == TEXT("CurveName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetCurveNameList);
				}
				else if (CustomWidgetName == TEXT("ElementName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetElementNameList)
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
				}
				else if (CustomWidgetName == TEXT("ConnectorName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetConnectorNameList)
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
				}
				else if (CustomWidgetName == TEXT("DrawingName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameFromSelection_UObject(RigGraph, &UControlRigGraph::GetSelectedElementsNameList)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetDrawingNameList);
				}
				else if (CustomWidgetName == TEXT("ShapeName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameListContent_UObject(RigGraph, &UControlRigGraph::GetShapeNameList);
				}
				else if (CustomWidgetName == TEXT("ComponentName"))
				{
					struct FCachedComponentNames
					{
						int32 TopologyVersion;
						TSharedPtr<TArray<TSharedPtr<FRigVMStringWithTag>>> Names;
						
						FCachedComponentNames()
						: TopologyVersion(INDEX_NONE)
						{}
					};
					
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameListContent_Lambda([RigGraph](const URigVMPin* InPin)
						{
							if (FControlRigAssetInterfacePtr Asset = IControlRigEditorAssetInterface::GetInterfaceOuter(RigGraph))
							{
								FRigElementKey ElementKey;

								// find the pin that holds the element
								if(const URigVMPin* ParentPin = InPin->GetParentPin())
								{
									if(const URigVMPin* ElementPin = ParentPin->FindSubPin(GET_MEMBER_NAME_STRING_CHECKED(FRigComponentKey, ElementKey)))
									{
										if(ElementPin->GetCPPType() == RigVMTypeUtils::GetUniqueStructTypeName(FRigElementKey::StaticStruct()))
										{
											const FString DefaultValue = ElementPin->GetDefaultValue();
											if(!DefaultValue.IsEmpty())
											{
												FRigElementKey::StaticStruct()->ImportText(*DefaultValue, &ElementKey, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);
											}
										}
									}
								}

								const URigHierarchy* Hierarchy = Asset->GetHierarchy();
								if (UControlRig* ControlRig = Cast<UControlRig>(Asset->GetObjectBeingDebugged()))
								{
									Hierarchy = ControlRig->GetHierarchy();
								}
								
								if(Hierarchy->Find(ElementKey) == nullptr)
								{
									ElementKey.Reset();
								}

								const FString MapHash = ElementKey.IsValid() ?
									Asset.GetObject()->GetPathName() + TEXT("|") + ElementKey.ToString() :
									FName(NAME_None).ToString();

								static TMap<FString, FCachedComponentNames> ComponentNameLists;
								FCachedComponentNames& ComponentNames = ComponentNameLists.FindOrAdd(MapHash);

								bool bRefreshList = !ComponentNames.Names.IsValid();

								if(!bRefreshList)
								{
									const int32 TopologyVersion = Hierarchy->GetTopologyVersion();
									if(ComponentNames.TopologyVersion != TopologyVersion)
									{
										bRefreshList = true;
										ComponentNames.TopologyVersion = TopologyVersion;
									}
								}

								if(bRefreshList)
								{
									if(!ComponentNames.Names.IsValid())
									{
										ComponentNames.Names = MakeShareable(new TArray<TSharedPtr<FRigVMStringWithTag>>());
									}
									ComponentNames.Names->Reset();
									ComponentNames.Names->Add(MakeShareable(new FRigVMStringWithTag(FName(NAME_None).ToString())));

									if(const FRigBaseElement* Element = Hierarchy->Find<FRigBaseElement>(ElementKey))
									{
										TArray<FRigComponentKey> ComponentKeys = Element->GetComponentKeys();
										ComponentKeys.Sort();
										for(const FRigComponentKey& ComponentKey : ComponentKeys)
										{
											ComponentNames.Names->Add(MakeShareable(new FRigVMStringWithTag(ComponentKey.Name.ToString())));
										}
									}
								}
								return ComponentNames.Names.Get();
							}

							static TArray<TSharedPtr<FRigVMStringWithTag>> EmptyNameList;
							return &EmptyNameList;
						})
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
				}
				else if (CustomWidgetName == TEXT("AnimationChannelName"))
				{
					struct FCachedAnimationChannelNames
					{
						int32 TopologyVersion;
						TSharedPtr<TArray<TSharedPtr<FRigVMStringWithTag>>> Names;
						
						FCachedAnimationChannelNames()
						: TopologyVersion(INDEX_NONE)
						{}
					};
					
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameListContent_Lambda([RigGraph](const URigVMPin* InPin)
						{
							if (FControlRigAssetInterfacePtr Asset = IControlRigEditorAssetInterface::GetInterfaceOuter(RigGraph))
							{
								FRigElementKey ControlKey;

								// find the pin that holds the control
								for(URigVMPin* Pin : InPin->GetRootPin()->GetNode()->GetPins())
								{
									if(Pin->GetCPPType() == RigVMTypeUtils::FNameType && Pin->GetCustomWidgetName() == TEXT("ControlName"))
									{
										const FString DefaultValue = Pin->GetDefaultValue();
										const FName ControlName = DefaultValue.IsEmpty() ? FName(NAME_None) : FName(*DefaultValue);
										ControlKey = FRigElementKey(ControlName, ERigElementType::Control);
										break;
									}

									if(Pin->GetCPPType() == RigVMTypeUtils::GetUniqueStructTypeName(FRigElementKey::StaticStruct()))
									{
										const FString DefaultValue = Pin->GetDefaultValue();
										if(!DefaultValue.IsEmpty())
										{
											FRigElementKey::StaticStruct()->ImportText(*DefaultValue, &ControlKey, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRigElementKey::StaticStruct()->GetName(), true);
										}
										break;
									}
								}

								const URigHierarchy* Hierarchy = Asset->GetHierarchy();
								if (UControlRig* ControlRig = Cast<UControlRig>(Asset->GetObjectBeingDebugged()))
								{
									Hierarchy = ControlRig->GetHierarchy();
								}
								
								if(Hierarchy->Find<FRigControlElement>(ControlKey) == nullptr)
								{
									ControlKey.Reset();
								}

								const FString MapHash = ControlKey.IsValid() ?
									Asset.GetObject()->GetPathName() + TEXT("|") + ControlKey.Name.ToString() :
									FName(NAME_None).ToString();

								static TMap<FString, FCachedAnimationChannelNames> ChannelNameLists;
								FCachedAnimationChannelNames& ChannelNames = ChannelNameLists.FindOrAdd(MapHash);

								bool bRefreshList = !ChannelNames.Names.IsValid();

								if(!bRefreshList)
								{
									const int32 TopologyVersion = Hierarchy->GetTopologyVersion();
									if(ChannelNames.TopologyVersion != TopologyVersion)
									{
										bRefreshList = true;
										ChannelNames.TopologyVersion = TopologyVersion;
									}
								}

								if(bRefreshList)
								{
									if(!ChannelNames.Names.IsValid())
									{
										ChannelNames.Names = MakeShareable(new TArray<TSharedPtr<FRigVMStringWithTag>>());
									}
									ChannelNames.Names->Reset();
									ChannelNames.Names->Add(MakeShareable(new FRigVMStringWithTag(FName(NAME_None).ToString())));

									if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(ControlKey))
									{
										for(const FRigBaseElement* Child : Hierarchy->GetChildren(ControlElement))
										{
											if(const FRigControlElement* ChildControl = Cast<FRigControlElement>(Child))
											{
												if(ChildControl->IsAnimationChannel())
												{
													ChannelNames.Names->Add(MakeShareable(new FRigVMStringWithTag(ChildControl->GetDisplayName().ToString())));
												}
											}
										}
									}
								}
								return ChannelNames.Names.Get();
							}

							static TArray<TSharedPtr<FRigVMStringWithTag>> EmptyNameList;
							return &EmptyNameList;
						})
						.OnGetSelectedClicked_UObject(RigGraph, &UControlRigGraph::HandleGetSelectedClicked)
						.OnBrowseClicked_UObject(RigGraph, &UControlRigGraph::HandleBrowseClicked);
				}
				else if (CustomWidgetName == TEXT("MetadataName"))
				{
					struct FCachedMetadataNames
					{
						int32 MetadataVersion;
						TSharedPtr<TArray<TSharedPtr<FRigVMStringWithTag>>> Names;
						
						FCachedMetadataNames()
						: MetadataVersion(INDEX_NONE)
						{}
					};

					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.SearchHintText(NSLOCTEXT("FControlRigGraphPanelPinFactory", "MetadataName", "Metadata Name"))
						.AllowUserProvidedText(true)
						.EnableNameListCache(false)
						.OnGetNameListContent_Lambda([RigGraph](const URigVMPin* InPin)
						{
							if (FControlRigAssetInterfacePtr Asset = IControlRigEditorAssetInterface::GetInterfaceOuter(RigGraph))
							{
								if(UControlRig* ControlRig = Cast<UControlRig>(Asset->GetObjectBeingDebugged()))
								{
									const FString MapHash = Asset.GetObject()->GetPathName();
									const int32 MetadataVersion = ControlRig->GetHierarchy()->GetMetadataVersion();
									
									ERigMetaDataNameSpace NameSpace = ERigMetaDataNameSpace::None;
									if (const URigVMNode* ModelNode = InPin->GetNode())
									{
										if(const URigVMPin* NameSpacePin = ModelNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_HasMetadata, NameSpace)))
										{
											NameSpace = (ERigMetaDataNameSpace)StaticEnum<ERigMetaDataNameSpace>()->GetValueByNameString(NameSpacePin->GetDefaultValue());
										}
									}

									const bool bUseShortNames = NameSpace != ERigMetaDataNameSpace::None;

									static TMap<FString, FCachedMetadataNames> MetadataNameLists;
									FCachedMetadataNames& MetadataNames = MetadataNameLists.FindOrAdd(MapHash);

									if(MetadataNames.MetadataVersion != MetadataVersion)
									{
										TArray<FName> Names;
										for(int32 ElementIndex=0; ElementIndex < ControlRig->GetHierarchy()->Num(); ElementIndex++)
										{
											const FRigBaseElement* OtherElement = ControlRig->GetHierarchy()->Get(ElementIndex);
											for(FName MetadataName: ControlRig->GetHierarchy()->GetMetadataNames(OtherElement->GetKey()))
											{
												Names.AddUnique(MetadataName);
											}
										}

										if(!MetadataNames.Names.IsValid())
										{
											MetadataNames.Names = MakeShareable(new TArray<TSharedPtr<FRigVMStringWithTag>>());
										}
										MetadataNames.Names->Reset();

										for(const FName& Name : Names)
										{
											FString NameString = Name.ToString();
											if(bUseShortNames)
											{
												int32 Index = INDEX_NONE;
												if(NameString.FindLastChar(FRigHierarchyModulePath::ModuleNameSuffixChar, Index))
												{
													NameString.MidInline(Index + 1);
												}
											}
											MetadataNames.Names->Add(MakeShareable(new FRigVMStringWithTag(NameString)));
										}

										MetadataNames.Names->Sort([](const TSharedPtr<FRigVMStringWithTag>& A, const TSharedPtr<FRigVMStringWithTag>& B)
										{
											const FRigVMStringWithTag* StringA = A.Get();
											const FRigVMStringWithTag* StringB = B.Get();
											if(StringA && StringB)
											{
												return (*StringA) < (*StringB);
											}
											return false;
										});
										MetadataNames.Names->Insert(MakeShareable(new FRigVMStringWithTag(FName(NAME_None).ToString())), 0);

										MetadataNames.MetadataVersion = MetadataVersion;
									}
									return MetadataNames.Names.Get();
								}
							}

							static TArray<TSharedPtr<FRigVMStringWithTag>> EmptyNameList;
							return &EmptyNameList;
						});
				}
				else if (CustomWidgetName == TEXT("MetadataTagName"))
				{
					struct FCachedMetadataTagNames
					{
						int32 MetadataTagVersion;
						TSharedPtr<TArray<TSharedPtr<FRigVMStringWithTag>>> Names;
						FCachedMetadataTagNames()
						: MetadataTagVersion(INDEX_NONE)
						{}
					};

					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.SearchHintText(NSLOCTEXT("FControlRigGraphPanelPinFactory", "TagName", "Tag Name"))
						.AllowUserProvidedText(true)
						.EnableNameListCache(false)
						.OnGetNameListContent_Lambda([RigGraph](const URigVMPin* InPin)
						{
							if (FControlRigAssetInterfacePtr Asset = IControlRigEditorAssetInterface::GetInterfaceOuter(RigGraph))
							{
								if(UControlRig* ControlRig = Cast<UControlRig>(Asset->GetObjectBeingDebugged()))
								{
									const FString MapHash = Asset.GetObject()->GetPathName();
									const int32 MetadataTagVersion = ControlRig->GetHierarchy()->GetMetadataTagVersion(); 

									ERigMetaDataNameSpace NameSpace = ERigMetaDataNameSpace::None;
									if (const URigVMNode* ModelNode = InPin->GetNode())
									{
										if(const URigVMPin* NameSpacePin = ModelNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_HasMetadata, NameSpace)))
										{
											NameSpace = (ERigMetaDataNameSpace)StaticEnum<ERigMetaDataNameSpace>()->GetValueByNameString(NameSpacePin->GetDefaultValue());
										}
									}
									const bool bUseShortNames = NameSpace != ERigMetaDataNameSpace::None;

									static TMap<FString, FCachedMetadataTagNames> MetadataTagNameLists;
									FCachedMetadataTagNames& MetadataTagNames = MetadataTagNameLists.FindOrAdd(MapHash);

									if(MetadataTagNames.MetadataTagVersion != MetadataTagVersion)
									{
										TArray<FName> Tags; 
										for(int32 ElementIndex=0; ElementIndex < ControlRig->GetHierarchy()->Num(); ElementIndex++)
										{
											const FRigBaseElement* Element = ControlRig->GetHierarchy()->Get(ElementIndex);
											if(const FRigNameArrayMetadata* Md = Cast<FRigNameArrayMetadata>(Element->GetMetadata(URigHierarchy::TagMetadataName, ERigMetadataType::NameArray)))
											{
												for(const FName& Tag : Md->GetValue())
												{
													Tags.AddUnique(Tag);
												}
											}
										}

										if(!MetadataTagNames.Names.IsValid())
										{
											MetadataTagNames.Names = MakeShareable(new TArray<TSharedPtr<FRigVMStringWithTag>>());
										}
										MetadataTagNames.Names->Reset();

										for(const FName& Tag : Tags)
										{
											FString TagString = Tag.ToString();
											if(bUseShortNames)
											{
												int32 Index = INDEX_NONE;
												if(TagString.FindLastChar(FRigHierarchyModulePath::ModuleNameSuffixChar, Index))
												{
													TagString.MidInline(Index + 1);
												}
											}
											MetadataTagNames.Names->Add(MakeShareable(new FRigVMStringWithTag(TagString)));
										}
										MetadataTagNames.Names->Sort([](const TSharedPtr<FRigVMStringWithTag>& A, const TSharedPtr<FRigVMStringWithTag>& B)
										{
											const FRigVMStringWithTag* StringA = A.Get();
											const FRigVMStringWithTag* StringB = B.Get();
											if(StringA && StringB)
											{
												return (*StringA) < (*StringB);
											}
											return false;
										});
										MetadataTagNames.Names->Insert(MakeShareable(new FRigVMStringWithTag(FName(NAME_None).ToString())), 0);

										MetadataTagNames.MetadataTagVersion = MetadataTagVersion;
									}

									return MetadataTagNames.Names.Get();
								}
							}

							static TArray<TSharedPtr<FRigVMStringWithTag>> EmptyNameList;
							return &EmptyNameList;
						});
				}
			}
		}
	}

	return nullptr;
}
