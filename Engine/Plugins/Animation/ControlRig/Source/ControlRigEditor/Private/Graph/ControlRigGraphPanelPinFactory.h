// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/RigVMEdGraphPanelPinFactory.h"

class FControlRigGraphPanelPinFactory : public FRigVMEdGraphPanelPinFactory
{
public:
	// FGraphPanelPinFactory interface
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;

	// FRigVMEdGraphPanelPinFactory interface
	virtual FName GetFactoryName() const override;
	virtual TSharedPtr<class SGraphPin> CreatePin_Internal(class UEdGraphPin* InPin) const override;
};
