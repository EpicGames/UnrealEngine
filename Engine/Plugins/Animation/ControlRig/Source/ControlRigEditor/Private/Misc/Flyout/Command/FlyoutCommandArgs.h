// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Templates/UnrealTemplate.h"

class FUICommandInfo;
class FUICommandList;

// This code intentionally has NO knowledge about Control Rig at all. The plan is to possiblely move it out to ToolWidgets.
// DO NOT introduce ControlRig specific code here (placing it in the ControlRigEditor namespace for now is the only exception as the code standard enforces it).

namespace UE::ControlRigEditor
{
struct FFlyoutCommandArgs
{
	explicit FFlyoutCommandArgs(const TSharedRef<FUICommandList>& InCommandList)
		: CommandList(InCommandList)
	{}
	
	FFlyoutCommandArgs& SetToggleVisibility(TSharedPtr<FUICommandInfo> InCommand) { ToggleVisibilityCommand = MoveTemp(InCommand); return *this; }
	FFlyoutCommandArgs& SetSummonToCursor(TSharedPtr<FUICommandInfo> InCommand) { SummonToCursorCommand = MoveTemp(InCommand); return *this; }

	/** Command list that the commands should be bound to. Required. */
	const TSharedRef<FUICommandList> CommandList;

	/** Shows or hides the widget. Optional.*/
	TSharedPtr<FUICommandInfo> ToggleVisibilityCommand;
	/** Summons the widget to the cursor. Once the cursor moves out of the widget bounds, it is placed back ast its original position. Optional.*/
	TSharedPtr<FUICommandInfo> SummonToCursorCommand;
};
}

