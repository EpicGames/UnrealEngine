// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlyoutCommandBinder.h"

#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Flyout/FlyoutOverlayManager.h"

namespace UE::ControlRigEditor
{

FFlyoutCommandBinder::FFlyoutCommandBinder(FFlyoutOverlayManager& InManager, FFlyoutCommandArgs InArgs)
	: Manager(InManager)
	, Args(MoveTemp(InArgs))
{
	if (Args.SummonToCursorCommand)
	{
		SummonCommand.Emplace(InManager, Args.SummonToCursorCommand.ToSharedRef());
	}
	
	BindCommands();
}

FFlyoutCommandBinder::~FFlyoutCommandBinder()
{
	UnbindCommands();
}

void FFlyoutCommandBinder::BindSummonCommand()
{
	if (Args.SummonToCursorCommand)
	{
		GetCommandList()->MapAction(
			Args.SummonToCursorCommand,
			FExecuteAction::CreateRaw(this, &FFlyoutCommandBinder::HandleSummonToCursorCommand)
		);
	}
}

void FFlyoutCommandBinder::UnbindSummonCommand()
{
	if (Args.SummonToCursorCommand)
	{
		GetCommandList()->UnmapAction(Args.SummonToCursorCommand);
	}
}

void FFlyoutCommandBinder::BindCommands()
{
	// The commands are optional... if the API user does not specify them, they don't want that feature.
	if (Args.ToggleVisibilityCommand)
	{
		GetCommandList()->MapAction(
			Args.ToggleVisibilityCommand,
			FExecuteAction::CreateRaw(this, &FFlyoutCommandBinder::HandleToggleVisibilityCommand)
		);
	}
	
	BindSummonCommand();
}

void FFlyoutCommandBinder::UnbindCommands()
{
	if (Args.ToggleVisibilityCommand)
	{
		GetCommandList()->UnmapAction(Args.ToggleVisibilityCommand);
	}
	
	UnbindSummonCommand();
}

void FFlyoutCommandBinder::HandleToggleVisibilityCommand()
{
	Manager.ToggleVisibility();
}

void FFlyoutCommandBinder::HandleSummonToCursorCommand()
{
	SummonCommand->Execute();
}
}
