// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FlyoutCommandArgs.h"
#include "FlyoutSummonCommand.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ControlRigEditor
{
class FFlyoutOverlayManager;
class FFlyoutTemporaryPositionOverride;

/** 
 * Processes UI commands and forwards them to FFlyoutOverlayManager. 
 * - Command: Toggle widget visible / hidden
 * - Command: Summon widget to cursor. Once your mouse leaves the widget bounds, the widget is positioned back to its original position.
 */
class FFlyoutCommandBinder : public FNoncopyable
{
public:
	
	explicit FFlyoutCommandBinder(FFlyoutOverlayManager& InManager UE_LIFETIMEBOUND, FFlyoutCommandArgs InArgs);
	~FFlyoutCommandBinder();

	void BindSummonCommand();
	void UnbindSummonCommand();
	
private:
	
	/** The manager that is being extended. */
	FFlyoutOverlayManager& Manager;
	
	/** The arguments this was constructed with. */
	FFlyoutCommandArgs Args;
	
	/** Implements the summon to cursor command. */
	TOptional<FFlyoutSummonCommand> SummonCommand;
	
	TSharedRef<FUICommandList> GetCommandList() const { return Args.CommandList; }

	void BindCommands();
	void UnbindCommands();
	
	void HandleToggleVisibilityCommand();
	void HandleSummonToCursorCommand();
};
}


