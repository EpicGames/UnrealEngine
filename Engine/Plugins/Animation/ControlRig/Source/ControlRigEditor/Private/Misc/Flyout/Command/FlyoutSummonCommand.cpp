// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlyoutSummonCommand.h"

#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Flyout/FlyoutOverlayManager.h"

namespace UE::ControlRigEditor
{
namespace Private
{
/** Keeps the flyout widget temporarily repositioned for as long as the user keeps SummonToCursorCommand pressed down. */
class FSummonToCursorInputProcessor : public IInputProcessor, public TSharedFromThis<FSummonToCursorInputProcessor>
{
	const TSharedPtr<FUICommandInfo> SummonToCursorCommand;
	const TSharedPtr<FFlyoutTemporaryPositionOverride> KeepAliveOverride;
public:

	explicit FSummonToCursorInputProcessor(
		TSharedPtr<FUICommandInfo> InSummonToCursorCommand, TSharedPtr<FFlyoutTemporaryPositionOverride> InKeepAliveOverride
		)
		: SummonToCursorCommand(MoveTemp(InSummonToCursorCommand))
		, KeepAliveOverride(MoveTemp(InKeepAliveOverride))
	{}

	virtual bool HandleKeyUpEvent(FSlateApplication&, const FKeyEvent& InKeyEvent) override
	{
		if (!SummonToCursorCommand)
		{
			return false;
		}

		const FKey Key = InKeyEvent.GetKey();

		bool bHasStoppedPressing = false;
		for (int32 ChordIndex = 0; ChordIndex < static_cast<int32>(EMultipleKeyBindingIndex::NumChords); ++ChordIndex)
		{
			const TSharedRef<const FInputChord> Chord = SummonToCursorCommand->GetActiveChord(static_cast<EMultipleKeyBindingIndex>(ChordIndex));
			if (!Chord->IsValidChord())
			{
				continue;
			}
			if (Chord->Key == Key
				|| (Chord->NeedsShift() && (Key == EKeys::LeftShift || Key == EKeys::RightShift))
				|| (Chord->NeedsAlt() && (Key == EKeys::LeftAlt || Key == EKeys::RightAlt))
				|| (Chord->NeedsControl() && (Key == EKeys::LeftControl || Key == EKeys::RightControl))
				|| (Chord->NeedsCommand() && (Key == EKeys::LeftCommand || Key == EKeys::RightCommand)))
			{
				bHasStoppedPressing = true;
				break;
			}
		}
		if (bHasStoppedPressing)
		{
			FSlateApplication::Get().UnregisterInputPreProcessor(SharedThis(this));
		}
		
		return false;
	}

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}
};
}

FFlyoutSummonCommand::FFlyoutSummonCommand(
	FFlyoutOverlayManager& InManager UE_LIFETIMEBOUND, 
	const TSharedRef<FUICommandInfo>& InSummonToCursorCommand
	)
	: Manager(InManager)
	, SummonToCursorCommand(InSummonToCursorCommand)
{
	Manager.OnMouseLeftContent().AddRaw(this, &FFlyoutSummonCommand::OnMouseLeftContent);
	Manager.OnMenuClosedWhileActive().AddRaw(this, &FFlyoutSummonCommand::OnMenuClosed);
}

FFlyoutSummonCommand::~FFlyoutSummonCommand()
{
	Manager.OnMouseLeftContent().RemoveAll(this);
	Manager.OnMenuClosedWhileActive().RemoveAll(this);
}

void FFlyoutSummonCommand::Execute()
{
	if (!Manager.IsTemporarilyRepositioned() && !Manager.IsShowingWidget())
	{
		const ETemporaryFlyoutPositionFlags Flags = ETemporaryFlyoutPositionFlags::HideAtEnd;
		const TSharedPtr<FFlyoutTemporaryPositionOverride> Override = Manager.TryTemporarilyPositionWidgetAtCursor(Flags);
		
		CommandTriggeredPositionOverride = Override;
		FSlateApplication::Get().RegisterInputPreProcessor(MakeShared<Private::FSummonToCursorInputProcessor>(SummonToCursorCommand, Override));
	}
}

void FFlyoutSummonCommand::OnMouseLeftContent()
{
	CommandTriggeredPositionOverride.Reset();
}

void FFlyoutSummonCommand::OnMenuClosed()
{
	CommandTriggeredPositionOverride.Reset();
}
}
