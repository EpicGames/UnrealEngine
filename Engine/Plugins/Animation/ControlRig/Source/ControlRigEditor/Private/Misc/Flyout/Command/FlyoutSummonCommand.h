// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FUICommandInfo;

namespace UE::ControlRigEditor
{
class FFlyoutOverlayManager;
class FFlyoutTemporaryPositionOverride;

/** Temporarily summons the flyout widget to the cursor. */
class FFlyoutSummonCommand : public FNoncopyable
{
public:
	
	explicit FFlyoutSummonCommand(
		FFlyoutOverlayManager& InManager UE_LIFETIMEBOUND, 
		const TSharedRef<FUICommandInfo>& InSummonToCursorCommand
		);
	~FFlyoutSummonCommand();
	
	/** Temporarily summons the flyout widget to the cursor. */
	void Execute();
	
public:
	
	/** The manager that is being extended. */
	FFlyoutOverlayManager& Manager;
	
	/** The command that FFlyoutSummonCommand is bound to. Used to look up key mapping. */
	const TSharedRef<FUICommandInfo> SummonToCursorCommand;
	
	/** Overrides the position when FFlyoutWidgetArgs::SummonToCursorCommand is invoked. Resets once the mouse leaves the widget bounds. */
	TSharedPtr<FFlyoutTemporaryPositionOverride> CommandTriggeredPositionOverride;
	
	void OnMouseLeftContent();
	void OnMenuClosed();
};
}

