// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlyoutOverlayManager.h"

#include "Editor.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Host/IWidgetHost.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Widgets/SNullWidget.h"

namespace UE::ControlRigEditor
{
namespace Private
{
/** Content placed inside the SDraggableBoxOverlay. This lets us know when the mouse cursor has left the content. */
class SFlyoutWidgetWrapper : public SCompoundWidget
{
	FSimpleDelegate OnMouseEnterDelegate;
	FSimpleDelegate OnMouseLeftDelegate;

	bool bDetectMouseLeave = false;
public:

	SLATE_BEGIN_ARGS(SFlyoutWidgetWrapper){}
		SLATE_NAMED_SLOT(FArguments, Content)
		SLATE_EVENT(FSimpleDelegate, OnMouseEnter)
		SLATE_EVENT(FSimpleDelegate, OnMouseLeft)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnMouseEnterDelegate = InArgs._OnMouseEnter;
		OnMouseLeftDelegate = InArgs._OnMouseLeft;
		
		ChildSlot
		[
			InArgs._Content.Widget
		];
		
		SetVisibility(EVisibility::Visible); // Allow clicks to go to child widget.
	}

	void ClearChildContent()
	{
		ChildSlot[ SNullWidget::NullWidget ];
	}

	void DetectMouseLeave() { bDetectMouseLeave = true; }

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
		OnMouseEnterDelegate.ExecuteIfBound();
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
		bDetectMouseLeave = false;
		OnMouseLeftDelegate.ExecuteIfBound();
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (bDetectMouseLeave && !AllottedGeometry.IsUnderLocation(FSlateApplication::Get().GetCursorPos()))
		{
			bDetectMouseLeave = false;
			OnMouseLeftDelegate.ExecuteIfBound();
		}
	}
};
}
	
FFlyoutOverlayManager::FFlyoutOverlayManager(FFlyoutWidgetArgs InArgs)
	: Args(MoveTemp(InArgs))
{
	if (InArgs.StateToRestoreFrom && InArgs.StateToRestoreFrom->bWasVisible)
	{
		ShowWidget();
	}

	if (FSlateApplication::IsInitialized())
	{
		// This is needed so we can detect when a drop-down "windows" being closed.
		FSlateApplication::Get().OnWindowBeingDestroyed().AddRaw(this, &FFlyoutOverlayManager::OnWindowDestroyed);
	}
}

FFlyoutOverlayManager::~FFlyoutOverlayManager()
{
	if (DragWidget)
	{
		Args.WidgetHost->RemoveWidgetFromHost();
		ensureAlwaysMsgf(DragWidget.IsUnique(), TEXT("Widget not properly removed from hierarchy"));
		
		// Defensive: if WidgetHost did not properly remove from the hierarchy, the widget life will be extended by our DragWidget referencing it.
		// Clear it to give it a chance to die.
		ContentWidget->ClearChildContent(); 
	}

	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnWindowBeingDestroyed().RemoveAll(this);
	}
}

void FFlyoutOverlayManager::ShowWidget()
{
	if (bShouldShowWidget)
	{
		return;
	}
	bShouldShowWidget = true;
	
	if (!DragWidget)
	{
		DragWidget = SNew(ToolWidgets::SDraggableBoxOverlay)
			.OnUserDraggedToNewPosition_Raw(this, &FFlyoutOverlayManager::SaveWidgetState)
			.Draggable_Raw(this, &FFlyoutOverlayManager::CanDragWidget)
			.Visibility_Lambda([this]
			{
				// Don't show the widget while in PIE because it prevents interaction with UMG widgets.
				const bool bIsPlaying = GEditor && (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld);
				// Use SelfHitTestInvisible (not Visible) so the overlay itself doesn't block mouse events in empty
				// areas of the viewport — only the SDraggableBox child (the actual content) is hit-testable.
				return !bIsPlaying && bShouldShowWidget ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
			})
			.Content()
			[
				SAssignNew(ContentWidget, Private::SFlyoutWidgetWrapper)
				.OnMouseEnter_Raw(this, &FFlyoutOverlayManager::HandleMouseEnterContent)
				.OnMouseLeft_Raw(this, &FFlyoutOverlayManager::HandleMouseLeftContent)
				.Content()[ Args.GetWidgetDelegate.Execute() ]
			];
		Args.WidgetHost->AddWidgetToHost(DragWidget.ToSharedRef());

		if (Args.StateToRestoreFrom)
		{
			DragWidget->SetDragBoxPosition(Args.StateToRestoreFrom->Position);
		}

		TrySubdueContent();
	}

	// Save that the widget was last visible.
	SaveWidgetState();

	OnVisibilityChangedDelegate.Broadcast(true);
}

void FFlyoutOverlayManager::HideWidget()
{
	if (DragWidget && bShouldShowWidget)
	{
		bShouldShowWidget = false;

		constexpr bool bSkipHide = true; // Avoid recursive call.
		StopTemporaryWidgetPosition(bSkipHide);
		constexpr bool bForceSubdue = true; 
		TrySubdueContent(bForceSubdue);
		
		// Save that the widget was last not visible.
		SaveWidgetState();

		OnVisibilityChangedDelegate.Broadcast(false);
	}
}

void FFlyoutOverlayManager::DestroyWidget()
{
	if (DragWidget)
	{
		if (bShouldShowWidget)
		{
			HideWidget(); //hide before destroying
		}
		Args.WidgetHost->RemoveWidgetFromHost();
		ensureAlwaysMsgf(DragWidget.IsUnique(), TEXT("Widget not properly removed from hierarchy"));

		// Defensive: if WidgetHost did not properly remove from the hierarchy, the widget life will be extended by our DragWidget referencing it.
		// Clear it to give it a chance to die.
		ContentWidget->ClearChildContent();
		DragWidget.Reset();
		ContentWidget.Reset();
	}
}
TSharedPtr<FFlyoutTemporaryPositionOverride> FFlyoutOverlayManager::TryTemporarilyPositionWidgetAtCursor(ETemporaryFlyoutPositionFlags InFlags)
{
	// No double starting the operation
	if (TemporaryMoveOperation)
	{
		return nullptr;
	}

	// If the widget was hidden, its geometry may be stale from a previous viewport (the widget moved via OnActiveViewportChanged while collapsed
	// and never repainted in the new viewport). We must defer so that ShowWidget() triggers a repaint and the geometry is updated.
	const bool bWasAlreadyVisible = IsShowingWidget();
	ShowWidget();
	if (!ensure(DragWidget))
	{
		return nullptr;
	}

	// The widget's size is required in order to center it around the mouse cursor. For that the widget needs to paint at least once.
	const FVector2f LocalSize = DragWidget->GetTickSpaceGeometry().GetLocalSize();
	const bool bWasEverTicked = !LocalSize.IsNearlyZero() && bWasAlreadyVisible;
	if (!bWasEverTicked)
	{
		const TSharedRef<FFlyoutTemporaryPositionOverride> LatentOverride = MakeShared<FFlyoutTemporaryPositionOverride>(
			FFlyoutTemporaryPositionOverride::FPrivateToken(), *this
			);
		// ShowTweenWidget call above may have just created the drag widget, in which case GetTickSpaceGeometry() is unset.
		// We need to wait for the widget to paint at least once so the geometry becomes set.
		// If that does not happen next tick (it should), disregard this request.
		ExecuteOnGameThread(TEXT("Defer PositionWidgetAtMouseCursor"), [this, InFlags, WeakWidget = DragWidget.ToWeakPtr(), LatentOverride]
		{
			// We assume that the DragWidget is destroyed when this FFlyoutOverlayManager is destroyed.
			// That may not be the case if RemoveFromOverlayDelegate does not remove the widget properly... in that case we'll get undefined behavior.
			const TSharedPtr<ToolWidgets::SDraggableBoxOverlay> ThisPin = WeakWidget.Pin();
			if (!ThisPin)
			{
				return;
			}

			if (!LatentOverride->IsCancelled() && !TemporarilyMoveWidgetToCursor(InFlags))
			{
				LatentOverride->Cancel();
			}
		});
		return LatentOverride;
	}

	if (TemporarilyMoveWidgetToCursor(InFlags))
	{
		return MakeShared<FFlyoutTemporaryPositionOverride>(FFlyoutTemporaryPositionOverride::FPrivateToken(), *this);
	}
	return nullptr;
}
	
void FFlyoutOverlayManager::TrySubdueContent(bool bForce) const
{
	if (bForce ||
		// No subduing while the widget is temporarily moved to the mouse cursor
		(!TemporaryMoveOperation && Args.CanSubdueAttr.Get())
		)
	{
		ContentWidget->SetColorAndOpacity(Args.SubduedTintAttr.Get());
	}
}

void FFlyoutOverlayManager::UnsubdueContent() const
{
	ContentWidget->SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.f));
}
	
bool FFlyoutOverlayManager::TemporarilyMoveWidgetToCursor(ETemporaryFlyoutPositionFlags InFlags)
{
	check(DragWidget);

	// InternalPositionWidgetAt can fail, so get its position before the move so we can save it if the move succeeds. 
	const FToolWidget_DragBoxPosition SizeBeforeTemporaryMove = DragWidget->GetDragBoxPosition();
	const bool bHideAtEnd = EnumHasAnyFlags(InFlags, ETemporaryFlyoutPositionFlags::HideAtEnd);
	const bool bAllowCursorOutsideOfParent = EnumHasAnyFlags(InFlags, ETemporaryFlyoutPositionFlags::AllowCursorOutsideOfParent);
	
	const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
	const FToolWidget_DragBoxPosition PositionBeforeMove = DragWidget->GetDragBoxPosition();
	if (InternalPositionWidgetAt(CursorPos + Args.AbsOffsetFromCursorAttr.Get(), bAllowCursorOutsideOfParent))
	{
		// If the user moves the mouse very quickly and summons to cursor, then OnMouseLeave may not fire. To fix this, we'll check every tick.
		ContentWidget->DetectMouseLeave();
		
		// The user wants the widget to open up at the same position if they close it now.
		SaveWidgetState(SizeBeforeTemporaryMove);

		TemporaryMoveOperation.Emplace(PositionBeforeMove, bHideAtEnd);
		UnsubdueContent();
		return true;
	}

	if (bHideAtEnd) // There was no suitable position to place the widget (e.g. cursor outside of viewport), so hide it again if it was requested.
	{
		HideWidget();
	}
	return false;
}

void FFlyoutOverlayManager::StopTemporaryWidgetPosition(bool bSkipHideEvenIfRequested)
{
	if (TemporaryMoveOperation)
	{
		DragWidget->SetDragBoxPosition(TemporaryMoveOperation->RestorePosition);

		const bool bHide = TemporaryMoveOperation->bHideWidgetAtEnd; 
		TemporaryMoveOperation.Reset();
		
		if (bHide && !bSkipHideEvenIfRequested)
		{
			HideWidget();
		}
	}
}

bool FFlyoutOverlayManager::InternalPositionWidgetAt(const FVector2f& InAbsoluteScreenPosition, bool bAllowCursorOutsideOfParent) const
{
	const FVector2f LocalSize = DragWidget->GetTickSpaceGeometry().GetLocalSize();
	if (!ensure(!LocalSize.IsNearlyZero())) // Widget was never painted, so GetTickSpaceGeometry() is unset. The caller make sure it's valid. 
	{
		return false;
	}
	const FSlateLayoutTransform AbsoluteToWidget = DragWidget->GetTickSpaceGeometry().GetAccumulatedLayoutTransform().Inverse();

	// If the cursor is outside of the parent widget (e.g. the viewport), there's no point to continue because
	// the widget can't be placed there. Use the actual cursor position for this check, not InAbsoluteScreenPosition,
	// which may already have a centering offset applied that pushes it outside the viewport (e.g. near the left border).
	const FVector2f CursorPos = FSlateApplication::Get().GetCursorPos();
	const FVector2f LocalNormalizedCursorPosition = AbsoluteToWidget.TransformPoint(CursorPos) / LocalSize;
	const bool bClickedOutsideOfParent = LocalNormalizedCursorPosition.X < 0.f || LocalNormalizedCursorPosition.X > 1.f
		|| LocalNormalizedCursorPosition.Y < 0.f || LocalNormalizedCursorPosition.Y > 1.f;
	if (bClickedOutsideOfParent && !bAllowCursorOutsideOfParent)
	{
		return false;
	}
	
	const FVector2f ContentSize = ContentWidget->GetTickSpaceGeometry().GetAbsoluteSize();
	const FVector2f AbsFinalPosition = InAbsoluteScreenPosition
		// Position center of content widget in at the mouse cursor.
		- FVector2f(0.5f, 0.5f) * ContentSize;
	const FVector2f LocalFinalPosition = AbsoluteToWidget.TransformPoint(AbsFinalPosition);

	// Normalized coord: (0,0) is top-left, and (1,1) is bottom-right.
	// bAllowCursorOutsideOfParent must be true here... so we need to make sure to snap the target location to be inside the parent widget (e.g. viewport).
	const FVector2f NormalizedLocalCoords = FVector2f::Clamp(LocalFinalPosition / LocalSize, FVector2f::ZeroVector, FVector2f(1.f, 1.f));
	const FVector2f OffsetTopLeftCorner = NormalizedLocalCoords * LocalSize;
	
	// Whatever is passed to SetBoxAlignmentOffset is relative to the current HAlign and VAlign, which may be different from the top-left atm.
	DragWidget->SetBoxHorizontalAlignment(HAlign_Left);
	DragWidget->SetBoxVerticalAlignment(VAlign_Top);
	DragWidget->SetBoxAlignmentOffset(OffsetTopLeftCorner);

	return true;
}

void FFlyoutOverlayManager::SaveWidgetState() const
{
	SaveWidgetState(DragWidget->GetDragBoxPosition());
}

void FFlyoutOverlayManager::SaveWidgetState(const FToolWidget_DragBoxPosition& InPosition) const
{
	if (Args.SaveStateDelegate.IsBound() && ensure(DragWidget)
		// Invariant: We don't expect anything to trigger calling SaveWidgetState while position is temporarily moved.
		&& ensureAlways(!TemporaryMoveOperation))
	{
		const FToolWidget_FlyoutSavedState SavedState { InPosition, bShouldShowWidget };
		Args.SaveStateDelegate.Execute(SavedState);
	}
}

bool FFlyoutOverlayManager::CanDragWidget() const
{
	return !TemporaryMoveOperation.IsSet();
}

void FFlyoutOverlayManager::HandleMouseEnterContent()
{
	UnsubdueContent();
}

void FFlyoutOverlayManager::HandleMouseLeftContent()
{
	// If a menu was opened while the mouse hovered the content widget, we'll assume that the menu belongs to our content widget.
	// We'll defer hiding finishing the operation until the menu is closed.
	if (!FSlateApplication::Get().AnyMenusVisible())
	{
		OnMouseLeftContentDelegate.Broadcast();
		TrySubdueContent();
	}
}

void FFlyoutOverlayManager::OnWindowDestroyed(const SWindow&)
{
	// If a menu was opened by the content widget and the mouse had left the content bounds (OnMouseLeftContent), then we deferred subduing the
	// content until the menu is closed...
	if (ContentWidget
		// ... the menu may be set up to not hide after you click something in it. That gives the user the chance to hover the content again and
		// e.g. click the combo button that opened the menu...
		&& !ContentWidget->IsHovered()
		// ... and here we handle that it's not a sub-menu that was just closed. 
		&& !FSlateApplication::Get().AnyMenusVisible())
	{
		OnMenuClosedWhileActiveDelegate.Broadcast();
		TrySubdueContent();
	}
}
}