// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FlyoutWidgetArgs.h"
#include "Templates/UnrealTemplate.h"
#include "TemporaryPositionOverride.h"

// This code intentionally has no knowledge about Control Rig at all.
// The plan is to move it out to ToolWidgets if other tools want to adapt a similar style widget - until then we keep it private to enjoy the
// flexibility of private code.
// Please DO NOT introduce ControlRig specific code here; instead invert dependencies by injecting Control Rig specific behaviour.

class IMenu;
namespace UE::ToolWidgets { class SDraggableBoxOverlay; }

namespace UE::ControlRigEditor
{
class FFlyoutTemporaryPositionOverride;
namespace Private { class SFlyoutWidgetWrapper; }

enum class ETemporaryFlyoutPositionFlags : uint8
{
	None = 0,

	/**
	 * If set, the mouse cursor position is snapped to the parent widget draw space.
	 * If unset, and the mouse cursor is outside the parent widget, the operation is skipped (but a set optional is still returned).
	 */
	AllowCursorOutsideOfParent = 1 << 0,
	
	/** If set, then the widget will be hidden once the flyout position is moved back to its original position. */
	HideAtEnd = 1 << 1
};
ENUM_CLASS_FLAGS(ETemporaryFlyoutPositionFlags);
	
/**
 * Manages all logic around a flyout widget.
 * 
 * A flyout widget is designed to be overlaid with another widget, usually the viewport.
 * The user can reposition it freely in the parent widget. The following functionality is provided:
 * - Drag-drop the widget to reposition it. The widget is "docked" to the corner closest to which the widget is dragged.
 *	Example: If the user drops the widget at the right-bottom corner, and moves the right side of the viewport to the left, this content widget
 *	maintains the same offset to the right side.
 * - Subdue widget while not in use:
 *	- While hovered, the content widget renders at full opacity.
 *	- While not hovered, the content widget is subdued.
 *	- This also handles the edge case of the content widget opening a menu and the user unhovering the content widget. No subduing takes place while
 *	a menu is being shown.
 */
class FFlyoutOverlayManager : public FNoncopyable
{
	friend class FFlyoutTemporaryPositionOverride;
public:
	
	explicit FFlyoutOverlayManager(FFlyoutWidgetArgs InArgs);
	~FFlyoutOverlayManager();

	bool IsShowingWidget() const { return bShouldShowWidget; }
	void ToggleVisibility() { if (IsShowingWidget()) { HideWidget(); } else { ShowWidget(); } }
	void ShowWidget();
	void HideWidget();
	void DestroyWidget();
	/** @return Whether the widget is temporarily repositioned. */
	bool IsTemporarilyRepositioned() const { return TemporaryMoveOperation.IsSet(); }
	/**
	 * Positions the widget at the mouse cursor until the returned FFlyoutTemporaryPositionOverride is destroyed.
	 *
	 * If the widget has never been painted, the repositioning happens next tick. In that case this returns a valid FFlyoutTemporaryPositionOverride
	 * but that may end up being cancelled next tick (e.g. if the cursor is outside of the viewport).
	 */
	TSharedPtr<FFlyoutTemporaryPositionOverride> TryTemporarilyPositionWidgetAtCursor(
		ETemporaryFlyoutPositionFlags InFlags = ETemporaryFlyoutPositionFlags::None
		);
	
	/** Subdues the content if allowed. */
	void TrySubdueContent(bool bForce = false) const;
	/** Shows the content normally again. */
	void UnsubdueContent() const;

	/** Broadcasts when the visibility of the widget changes. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVisibilityChanged, bool /*bIsVisible*/);
	FOnVisibilityChanged& OnVisibilityChanged() { return OnVisibilityChangedDelegate; }
	/** This delegate is invoked when the mouse cursor leaves the content of the widget. */
	FSimpleMulticastDelegate& OnMouseLeftContent() { return OnMouseLeftContentDelegate; }
	/** This delegate is invoked when a menu is closed while this widget is active (i.e. not subdued). */
	FSimpleMulticastDelegate& OnMenuClosedWhileActive() { return OnMenuClosedWhileActiveDelegate; }

private:

	/** The behaviour this widget was initialized with. */
	const FFlyoutWidgetArgs Args;

	/** Contains the actual content widget. Nullptr until shown for the first time. */
	TSharedPtr<ToolWidgets::SDraggableBoxOverlay> DragWidget;
	/** The widget displayed in DragWidget. This widget wraps the API caller's supplied widget. */
	TSharedPtr<Private::SFlyoutWidgetWrapper> ContentWidget;
	
	/** Whether the widget should be shown. */
	bool bShouldShowWidget = false;

	struct FTemporaryMoveData
	{
		/** The position the box had before it was temporarily moved. */
		const FToolWidget_DragBoxPosition RestorePosition;

		/** Whether to hide the widget after the operation concludes. */
		const bool bHideWidgetAtEnd;

		explicit FTemporaryMoveData(const FToolWidget_DragBoxPosition& InRestorePosition, bool bHideWidgetAtEnd)
			: RestorePosition(InRestorePosition), bHideWidgetAtEnd(bHideWidgetAtEnd)
		{}
	};
	/** Set while the content widget is temporarily moved. */
	TOptional<FTemporaryMoveData> TemporaryMoveOperation;

	/** Broadcasts when the visibility of the widget changes. */
	FOnVisibilityChanged OnVisibilityChangedDelegate;
	
	/** This delegate is invoked when the mouse cursor leaves the content of the widget. */
	FSimpleMulticastDelegate OnMouseLeftContentDelegate;
	/** This delegate is invoked when a menu is closed while this widget is active (i.e. not subdued). */
	FSimpleMulticastDelegate OnMenuClosedWhileActiveDelegate;

	/** Moves the widget to the cursor until the cursor leaves the widget bounds. After leaving the widget bounds, move the widget back to where it was. */
	bool TemporarilyMoveWidgetToCursor(ETemporaryFlyoutPositionFlags InFlags);
	/** Moves the widget back to its original position. */
	void StopTemporaryWidgetPosition(bool bSkipHideEvenIfRequested = false);
	
	/**
	 * Places the widget at the given position. If the position is outside the viewport bounds, nothing happens.
	 * 
	 * @param bAllowCursorOutsideOfParent If true, InAbsoluteScreenPosition is snapped to the parent widget draw space.
	 *	If false, nothing happens if the user clicks outside.
	 * @return Whether the widget was moved. 
	 */
	bool InternalPositionWidgetAt(const FVector2f& InAbsoluteScreenPosition, bool bAllowCursorOutsideOfParent = false) const;
	
	/** Saves the current widget state, if supported. */
	void SaveWidgetState() const;
	void SaveWidgetState(const FToolWidget_DragBoxPosition& InPosition) const;
	/** @return Whether to allow the content widget to be dragged. False while the widget is temporarily moved. */
	bool CanDragWidget() const;

	/** Invokes when the mouse hovers the content area. */
	void HandleMouseEnterContent();
	/** Invoked when the mouse has left the content area. */
	void HandleMouseLeftContent();

	/**
	 * Used to detect when menu window is destroyed, so we can subdue the content widget.
	 * 
	 * This is called shortly after FSlateApplication::OnMenuBeingDestroyed; during OnMenuBeingDestroyed, FSlateApplication::AnyMenusVisible still
	 * returns true broadcast timing. OnWindowDestroyed is called right after, when AnyMenusVisible returns the updated value.
	 */
	void OnWindowDestroyed(const SWindow&);
};
}

