// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Overlay/DragBoxPosition.h"
#include "FlyoutSavedState.generated.h"

/** Holds information to save the state of the flyout manager. This is useful for saving in a config file. */
USTRUCT()
struct FToolWidget_FlyoutSavedState
{
	GENERATED_BODY()
	
	/** Where the widget was docked. */
	UPROPERTY()
	FToolWidget_DragBoxPosition Position;

	/** Whether to show the widget automatically. */
	UPROPERTY()
	bool bWasVisible = false;
};

namespace UE::ControlRigEditor
{
using FFlyoutSavedState = FToolWidget_FlyoutSavedState;
DECLARE_DELEGATE_OneParam(FSaveFlyoutState, FFlyoutSavedState);
}

