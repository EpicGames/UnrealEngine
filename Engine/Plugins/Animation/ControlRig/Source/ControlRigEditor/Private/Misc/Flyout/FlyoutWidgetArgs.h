// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FlyoutSavedState.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/UnrealTemplate.h"
#include "Tools/AssetDefinition_ControlRigPose.h"

class FUICommandInfo;
class FUICommandList;

// This code intentionally has NO knowledge about Control Rig at all. The plan is to possible move it out to ToolWidgets.
// DO NOT introduce ControlRig specific code here (placing it in the ControlRigEditor namespace for now is the only exception as the code standard enforces it).

namespace UE::ControlRigEditor
{
class IWidgetHost;
using FGetFlyoutWidgetContent = FOnGetContent;
	
struct FFlyoutWidgetArgs
{
	explicit FFlyoutWidgetArgs(
		FGetFlyoutWidgetContent InGetWidgetDelegate,
		const TSharedRef<IWidgetHost>& InWidgetHost
		)
		: GetWidgetDelegate(MoveTemp(InGetWidgetDelegate))
		, WidgetHost(InWidgetHost)
	{
		check(GetWidgetDelegate.IsBound());
	}
	
	explicit FFlyoutWidgetArgs(
		const TSharedRef<SWidget>& InContentWidget,
		const TSharedRef<IWidgetHost>& InWidgetHost
		)
		: FFlyoutWidgetArgs(
			FGetFlyoutWidgetContent::CreateLambda([InContentWidget]{ return InContentWidget; }),
			InWidgetHost
			)
	{}

	FFlyoutWidgetArgs& SetStateToRestoreFrom(FFlyoutSavedState InState) { StateToRestoreFrom = InState; return *this; }
	FFlyoutWidgetArgs& SetSaveStateDelegate(FSaveFlyoutState InDelegate) { SaveStateDelegate = MoveTemp(InDelegate); return *this; }
	FFlyoutWidgetArgs& SetCanSubdue(TAttribute<bool> InCanSubdueAttr) { CanSubdueAttr = MoveTemp(InCanSubdueAttr); return *this; }
	FFlyoutWidgetArgs& SetSubduedOpacity(TAttribute<FLinearColor> InTintAttr) { SubduedTintAttr = MoveTemp(InTintAttr); return *this; }
	FFlyoutWidgetArgs& SetAbsOffsetFromCursor(TAttribute<FVector2f> InOffsetAttr) { AbsOffsetFromCursorAttr = MoveTemp(InOffsetAttr); return *this; }

	/** Creates the widget that is supposed to be displayed. Required. */
	const FGetFlyoutWidgetContent GetWidgetDelegate;
	/** Handles adding the widget to some UI hierarchy. Required. */
	const TSharedPtr<IWidgetHost> WidgetHost;

	/** The initial state that should be applied. You usually pass what you have saved in a user config to this. Optional. */
	TOptional<FFlyoutSavedState> StateToRestoreFrom;
	/**
	 * Saves the state of the flyout widget so it can be restored at a later time, e.g. after an editor restart, etc.
	 * You usually pass the saved state to RestoredFromState. Optional.
	 */
	FSaveFlyoutState SaveStateDelegate;

	/**
	 * Decides whether the widget can be subdued, i.e. render it so it is less noticeable to the user. Optional.
	 * 
	 * This is only invoked when the widget determines now is a good time be hidden, e.g. when the cursor isn't near it.
	 * In other words: simply returning true does not mean it will be subdued - it just indicates whether right now it is appropriate or not.
	 */
	TAttribute<bool> CanSubdueAttr = true;
	/** The render opacity to set when the widget is out of focus. */
	TAttribute<FLinearColor> SubduedTintAttr = FLinearColor(1.f, 1.f, 1.f, .35f);

	/**
	 * By default, the center of the widget is placed at the mouse cursor.
	 * This is an extra offset that is added to the absolute cursor position. The widget's center is positioned at this final transformed point.
	 */
	TAttribute<FVector2f> AbsOffsetFromCursorAttr = FVector2f::ZeroVector;
};
}

