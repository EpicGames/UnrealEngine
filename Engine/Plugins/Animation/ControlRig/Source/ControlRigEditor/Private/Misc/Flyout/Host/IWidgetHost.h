// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class SWidget;

namespace UE::ControlRigEditor
{
/** Hosts a widget in some UI. */
class IWidgetHost
{
public:

	/** Adds the widget to the target UI. The implementation is allowed to keep a strong reference to the widget until RemoveWidgetFromHost is called. */
	virtual void AddWidgetToHost(const TSharedRef<SWidget>& Widget) = 0;

	/** Removes the widget from the target UI and clears and strong reference to it. */
	virtual void RemoveWidgetFromHost() = 0;

	virtual ~IWidgetHost() = default;
};
}
