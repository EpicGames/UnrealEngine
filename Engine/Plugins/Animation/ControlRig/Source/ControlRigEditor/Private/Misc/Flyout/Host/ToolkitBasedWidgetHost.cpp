// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolkitBasedWidgetHost.h"

#include "Toolkits/IToolkitHost.h"

namespace UE::ControlRigEditor
{
FToolkitBasedWidgetHost::FToolkitBasedWidgetHost(const TSharedRef<IToolkitHost>& InToolkitHost)
	: ToolkitHost(InToolkitHost)
{
	ToolkitHost->OnActiveViewportChanged().AddRaw(this, &FToolkitBasedWidgetHost::OnActiveViewportChanged);
}

FToolkitBasedWidgetHost::~FToolkitBasedWidgetHost()
{
	ToolkitHost->OnActiveViewportChanged().RemoveAll(this);
}

void FToolkitBasedWidgetHost::AddWidgetToHost(const TSharedRef<SWidget>& Widget)
{
	ensure(!HostedWidget.IsValid());
	HostedWidget = Widget;
	ToolkitHost->AddViewportOverlayWidget(Widget);
}

void FToolkitBasedWidgetHost::RemoveWidgetFromHost()
{
	if (ensure(HostedWidget))
	{
		ToolkitHost->RemoveViewportOverlayWidget(HostedWidget.ToSharedRef());
	}
	HostedWidget.Reset();
}

void FToolkitBasedWidgetHost::OnActiveViewportChanged(TSharedPtr<IAssetViewport> OldViewport, TSharedPtr<IAssetViewport> NewViewport) const
{
	if (HostedWidget)
	{
		ToolkitHost->RemoveViewportOverlayWidget(HostedWidget.ToSharedRef(), OldViewport);
		ToolkitHost->AddViewportOverlayWidget(HostedWidget.ToSharedRef(), NewViewport);
	}
}
}
