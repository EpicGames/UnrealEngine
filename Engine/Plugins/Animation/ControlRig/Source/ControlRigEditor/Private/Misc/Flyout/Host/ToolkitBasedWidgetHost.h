// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWidgetHost.h"
#include "Templates/SharedPointer.h"

class IAssetViewport;
class IToolkitHost;
class SWidget;

namespace UE::ControlRigEditor
{
/** Adapter which hosts a widget in the IToolkitHost's active viewport. When the viewport changes, the widget is moved to the active viewport. */
class FToolkitBasedWidgetHost : public IWidgetHost
{
public:

	explicit FToolkitBasedWidgetHost(const TSharedRef<IToolkitHost>& InToolkitHost);
	virtual ~FToolkitBasedWidgetHost() override;

	//~ Begin IWidgetHost Interface
	virtual void AddWidgetToHost(const TSharedRef<SWidget>& Widget) override;
	virtual void RemoveWidgetFromHost() override;
	//~ End IWidgetHost Interface

private:

	/** The toolkit host in whicht the widget is hosted. */
	TSharedRef<IToolkitHost> ToolkitHost;

	/** The widget that is hosted. Set by AddWidgetToHost. */
	TSharedPtr<SWidget> HostedWidget;

	/** Moves HostedWidget to the new viewport. */
	void OnActiveViewportChanged(TSharedPtr<IAssetViewport> OldViewport, TSharedPtr<IAssetViewport> NewViewport) const;
};
}
