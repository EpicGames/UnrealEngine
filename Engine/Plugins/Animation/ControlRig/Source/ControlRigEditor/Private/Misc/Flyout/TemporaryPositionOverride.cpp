// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemporaryPositionOverride.h"

#include "FlyoutOverlayManager.h"

namespace UE::ControlRigEditor
{
FFlyoutTemporaryPositionOverride::FFlyoutTemporaryPositionOverride(FPrivateToken, FFlyoutOverlayManager& InManager)
	: ManagerToRestore(&InManager)
	{}
	
FFlyoutTemporaryPositionOverride::FFlyoutTemporaryPositionOverride(FFlyoutTemporaryPositionOverride&& Other)
	: ManagerToRestore(Other.ManagerToRestore)
{
	Other.Cancel();
}

FFlyoutTemporaryPositionOverride::~FFlyoutTemporaryPositionOverride()
{
	if (ManagerToRestore)
	{
		ManagerToRestore->StopTemporaryWidgetPosition();
	}
}
}
