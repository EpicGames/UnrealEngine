// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTemplate.h"

namespace UE::ControlRigEditor
{
class FFlyoutOverlayManager;

/** Keeps the flyout widget at the temporary override position until this object is destroyed, at which point it calls StopTemporaryWidgetPosition. */
class FFlyoutTemporaryPositionOverride : public FNoncopyable
{
	friend class FFlyoutOverlayManager;
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
	FFlyoutOverlayManager* ManagerToRestore;
	

	void Cancel() { ManagerToRestore = nullptr; }
	bool IsCancelled() const { return ManagerToRestore == nullptr; }

public:
	
	FFlyoutTemporaryPositionOverride(FPrivateToken, FFlyoutOverlayManager& InManager);
	FFlyoutTemporaryPositionOverride(FFlyoutTemporaryPositionOverride&& Other);
	FFlyoutTemporaryPositionOverride& operator=(FFlyoutTemporaryPositionOverride&& Other) = delete;

	~FFlyoutTemporaryPositionOverride();
};
}