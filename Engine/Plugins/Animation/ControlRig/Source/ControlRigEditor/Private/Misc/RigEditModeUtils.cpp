// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditModeUtils.h"

#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"

namespace UE::ControlRigEditor
{
FInitialSpacePickerSelection DetermineInitialSpacePickerSelection(FControlRigEditMode& InEditMode)
{
	TMap<UControlRig*, TArray<FRigElementKey>> RigToControlsMap;
	InEditMode.GetAllSelectedControls(RigToControlsMap);

	FInitialSpacePickerSelection Result;
	Algo::TransformIf(RigToControlsMap, Result.WeakHierarchyToControlsMap,
		[](const TTuple<UControlRig*, TArray<FRigElementKey>>& RigToControlPair)
		{
			return
				RigToControlPair.Key &&
				RigToControlPair.Key->GetHierarchy();
		},
		[](const TTuple<UControlRig*, TArray<FRigElementKey>>& RigToControlPair)
		{
			return MakeTuple(
				TWeakObjectPtr<URigHierarchy>(RigToControlPair.Key->GetHierarchy()),
				RigToControlPair.Value);
		});

	return Result;
}

}
