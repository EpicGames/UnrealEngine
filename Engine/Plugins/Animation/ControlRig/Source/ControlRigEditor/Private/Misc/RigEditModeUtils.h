// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/WeakObjectPtr.h"

class FControlRigEditMode;
class UControlRig;
class URigHierarchy;
struct FRigElementKey;

namespace UE::ControlRigEditor
{
struct FInitialSpacePickerSelection
{
	/** 
	 * All selected controls per rig, covering every selected rig. Used by the
	 * floating space picker so multi-rig selections are represented correctly. 
	 */
	TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>> WeakHierarchyToControlsMap;

	bool IsValid() const { return !WeakHierarchyToControlsMap.IsEmpty(); }
};

/** @return The initial hierarchy and controls to select when constructing SRigSpaceEditor. */
FInitialSpacePickerSelection DetermineInitialSpacePickerSelection(FControlRigEditMode& InEditMode);
}
