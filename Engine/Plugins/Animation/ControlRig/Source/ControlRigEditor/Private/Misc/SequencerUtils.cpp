// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerUtils.h"

#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

namespace UE::ControlRigEditor
{
TSharedPtr<FCurveEditor> GetCurveEditorFromSequencer(const TSharedPtr<ISequencer>& InSequencer)
{
	if (!InSequencer)
	{
		return nullptr;
	}

	const TSharedPtr<Sequencer::FSequencerEditorViewModel> SequencerViewModel = InSequencer->GetViewModel();
	const Sequencer::FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamic<Sequencer::FCurveEditorExtension>();
	return ensure(CurveEditorExtension) ? CurveEditorExtension->GetCurveEditor() : nullptr;
}
}
