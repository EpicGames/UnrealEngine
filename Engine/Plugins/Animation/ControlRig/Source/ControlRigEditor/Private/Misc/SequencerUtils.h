// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class ISequencer;
class FCurveEditor;

namespace UE::ControlRigEditor
{
/** @return The curve editor used by sequencer */
TSharedPtr<FCurveEditor> GetCurveEditorFromSequencer(const TSharedPtr<ISequencer>& InSequencer);
}
