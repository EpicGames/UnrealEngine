// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInlineTabPanel.h"

#include "STabArea.h"
#include "Components/VerticalBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::ControlRigEditor
{
namespace InlineTabPanel
{
static TArray<FTabEntry> Transform(TConstArrayView<FInlineTabArgs> InArgs, const TSharedRef<SInlineTabPanel>& This)
{
	const int32 Num = InArgs.Num();
	
	TArray<FTabEntry> Result;
	Result.Reserve(Num);
	
	for (int32 Index = 0; Index < Num; ++Index)
	{
		const FInlineTabArgs& CurrentArgs = InArgs[Index];

		Result.Emplace();
		FTabEntry& Entry = Result[Index];

		Entry.MinButtonSlotWidth = 50.f; // When user resizes the button to be quite narrow, leave enough space to show icon & text. 
		Entry.ButtonContent.Widget =
			SNew(SHorizontalBox)
			.ToolTipText(CurrentArgs.ToolTipText)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage)
				.Visibility(CurrentArgs.Icon.IsSet() ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
				.Image(CurrentArgs.Icon.GetIcon())
			]
			
			+SHorizontalBox::Slot()
			.FillContentWidth(1.f) // Reduce text desired size when user resizes button to be narrow...
			.MinWidth(20.f) // ... but force enough space to show the first letter ...
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.Text(CurrentArgs.Label)
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis) // ... and cut off the rest of the text when there's not enough space
			];
		
		Entry.OnTabSelected.BindLambda([Index, WeakOwnerPanel = This.ToWeakPtr(), Delegate = CurrentArgs.OnTabSelected]()
		{
			if (const TSharedPtr<SInlineTabPanel> OwnerPin = WeakOwnerPanel.Pin())
			{
				OwnerPin->SwitchToTab(Index);
				Delegate.ExecuteIfBound();
			}
		});
	}

	return Result;
}
}
	
void SInlineTabPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)

		// Toolbar
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				SAssignNew(TabArea, STabArea)
					.Tabs(InlineTabPanel::Transform(InArgs._Tabs, SharedThis(this)))
					.ActiveTabIndex(InArgs._ActiveTabIndex)
					.Padding(InArgs._Padding)
			]
		]

		// Separate the tabs & content
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SNew(SSeparator)
		]

		// Tabs
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(TabContentSwitcher, SWidgetSwitcher)
		]
	];

	for (int32 Index = 0; Index < InArgs._Tabs.Num(); ++Index)
	{
		const FInlineTabArgs& TabArgs = InArgs._Tabs[Index];
		TabContentSwitcher->AddSlot()
		[
			TabArgs.Content.Widget
		];
	}
}

void SInlineTabPanel::SwitchToTab(int32 InIndex) const
{
	if (ensure(InIndex >= 0 && InIndex < TabContentSwitcher->GetNumWidgets()))
	{
		TabArea->SetButtonActivated(InIndex);
		TabContentSwitcher->SetActiveWidgetIndex(InIndex);
	}
}
}
