// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidgetSwitcher;

namespace UE::ControlRigEditor
{
class STabArea;
	
struct FInlineTabArgs
{
	/** The name of the tab */
	TAttribute<FText> Label;
	
	/** The tooltip of the tab */
	TAttribute<FText> ToolTipText;

	/** Optional icon displayed in front of the label. */
	FSlateIcon Icon;

	/** Content to display as tab content */
	TAlwaysValidWidget Content;

	/** Invoked when this tab's content is shown (except for when SInlineTabPanel is constructed). */
	FSimpleDelegate OnTabSelected;

	FInlineTabArgs& SetLabel(TAttribute<FText> InLabel) { Label = MoveTemp(InLabel); return *this; }
	FInlineTabArgs& SetToolTipText(TAttribute<FText> InToolTipText) { ToolTipText = MoveTemp(InToolTipText); return *this; }
	FInlineTabArgs& SetIcon(const FSlateIcon& InIcon) { Icon = InIcon; return *this; }
	FInlineTabArgs& SetContent(TAlwaysValidWidget InWidget) { Content = MoveTemp(InWidget); return *this; }
	FInlineTabArgs& SetContent(const TSharedRef<SWidget>& InWidget) { Content.Widget = InWidget; return *this; }
	FInlineTabArgs& SetOnTabSelected(FSimpleDelegate InOnTabSelected) { OnTabSelected = MoveTemp(InOnTabSelected); return *this; }
};

/**
 * Coordinator widget. A vertical box with tab buttons on the top and the selected tab content at the bottom.
 * 
 * This is supposed to implement easy to use fence pattern.
 * If you need more flexibility for styling, considering building your widget hierarchy directly by using STabArea, SWidgetSwitcher, and STabButton
 * instead of complicating this widget's interface.
 */
class SInlineTabPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SInlineTabPanel)
		: _ActiveTabIndex(0)
		, _Padding(3.f)
		{}
		
		/** The tabs to create */
		SLATE_ARGUMENT(TArray<FInlineTabArgs>, Tabs)
		/** The tab that should be active by default. */
		SLATE_ARGUMENT(int32, ActiveTabIndex)
		
		/** Padding between buttons */
		SLATE_ARGUMENT(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Shows the content of the specified tab index. */
	void SwitchToTab(int32 InIndex) const;

private:

	/** Displays the tab buttons. */
	TSharedPtr<STabArea> TabArea;
	/** Switches the content for the tabs */
	TSharedPtr<SWidgetSwitcher> TabContentSwitcher;
};
}

