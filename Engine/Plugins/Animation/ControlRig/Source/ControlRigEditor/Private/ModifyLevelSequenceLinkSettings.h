// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "LevelSequenceAnimSequenceLink.h"
#include "ModifyLevelSequenceLinkSettings.generated.h"

//Object to hold the LinkItem so we can modify it in asset details
UCLASS(Transient)
class UModifyLevelSequenceLinkSettings : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Property, meta = (ShowOnlyInnerProperties))
	FLevelSequenceAnimSequenceLinkItem LinkItem;
};
