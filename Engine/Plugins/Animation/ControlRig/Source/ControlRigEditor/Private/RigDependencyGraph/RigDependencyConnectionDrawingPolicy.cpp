// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/RigDependencyConnectionDrawingPolicy.h"

#include "RigDependencyGraph.h"
#include "RigDependencyGraph/RigDependencyGraphNode.h"

void FRigDependencyConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	const URigDependencyGraphNode* OutputNode = Cast<URigDependencyGraphNode>(OutputPin->GetOwningNode());
	const URigDependencyGraphNode* InputNode = Cast<URigDependencyGraphNode>(InputPin->GetOwningNode());

	if (OutputNode && InputNode)
	{
		if (OutputNode->GetNodeId().IsElement() && InputNode->GetNodeId().IsElement())
		{
			static const FLinearColor ParentChildWireColor = FLinearColor(FColor::FromHex(TEXT("#AAAAAA")));
			static const FLinearColor ControlSpaceWireColor = FLinearColor(FColor::FromHex(TEXT("#22AA22")));
			Params.WireColor = ParentChildWireColor;

			if (const URigDependencyGraph* DependencyGraph = Cast<URigDependencyGraph>(OutputNode->GetGraph()))
			{
				const uint32 SpaceToControlhash = HashCombine(GetTypeHash(OutputNode->GetNodeId().Index), GetTypeHash(InputNode->GetNodeId().Index));
				if (DependencyGraph->HierarchySpaceToControlHashes.Contains(SpaceToControlhash))
				{
					Params.WireColor = ControlSpaceWireColor;
				}
			}

			Params.WireColor.A = 0.75f;
		}

		const float FadedState = FMath::Min(OutputNode->GetFadedOutState(), InputNode->GetFadedOutState());
		Params.WireColor.A *= FadedState;

		Params.bUserFlag1 = OutputNode->GetNodeId().IsInstruction() || InputNode->GetNodeId().IsInstruction();
	}
}