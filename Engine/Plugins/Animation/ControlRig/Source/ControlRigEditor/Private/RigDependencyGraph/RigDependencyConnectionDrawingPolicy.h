// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"

class FRigDependencyConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FRigDependencyConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
		: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
	{
	}

	virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override
	{
		const int32 Tension = FMath::Abs<int32>(Start.X - End.X);
		return Tension * FVector2f(1.0f, 0.0f);
	}

	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
};
