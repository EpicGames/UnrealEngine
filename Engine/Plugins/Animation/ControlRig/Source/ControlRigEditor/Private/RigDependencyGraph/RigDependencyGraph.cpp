// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/RigDependencyGraph.h"

#include "ITransportControl.h"
#include "ModularRig.h"
#include "Editor/ControlRigEditor.h"
#include "RigVMModel/RigVMClient.h"
#include "RigDependencyGraph/RigDependencyGraphNode.h"
#include "RigDependencyGraph/RigDependencyGraphSchema.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDependencyGraph)

URigDependencyGraph::URigDependencyGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultLayoutIterationsPerNode(50)
	, LayoutIterationsLeft(0)
	, LayoutIterationsTotal(0)
	, LayoutIterationsMax(DefaultLayoutIterationsPerNode * 1000)
	, bFollowParentRelationShips(true)
	, bFollowControlSpaceRelationships(true)
	, bFollowVMRelationShips(false)
	, bLockContent(false)
	, bIsPerformingGridLayout(true)
	, BlockSelectionCounter(2)
{
}

void URigDependencyGraph::Initialize(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor, const TWeakObjectPtr<UControlRig>& InControlRig)
{
	const bool bIsFirstTime = !WeakControlRigEditor.IsValid();
	
	if (WeakControlRigEditor.IsValid())
	{
		if (TSharedPtr<IControlRigBaseEditor> PreviousControlRigEditor = WeakControlRigEditor.Pin())
		{
			PreviousControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().Remove(OnSetObjectBeingDebuggedHandle);
			PreviousControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->OnVMCompiled().Remove(OnVMCompiledHandle);
			PreviousControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->OnModified().Remove(OnRigVMGraphModifiedHandle);
		}
	}
	
	if (UControlRig* PreviousControlRig = GetControlRig())
	{
		if (IsValid(PreviousControlRig))
		{
			if (URigHierarchy* Hierarchy = PreviousControlRig->GetHierarchy())
			{
				Hierarchy->OnModified().Remove(OnHierarchyModifiedHandle);
				Hierarchy->OnMetadataChanged().Remove(OnMetadataChangedHandle);
			}
		}
	}
	
	WeakControlRigEditor = InControlRigEditor;
	WeakControlRig = InControlRig;

	if(WeakControlRig.IsValid() && WeakControlRig.Get())
	{
		RigDependencyProvider = MakeShareable(new FRigDependenciesProviderForControlRig(WeakControlRig.Get(), NAME_None, true));
	}
	else
	{
		RigDependencyProvider = MakeShareable(new FEmptyRigDependenciesProvider());
	}

	OnSetObjectBeingDebuggedHandle = InControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->OnSetObjectBeingDebugged().AddUObject(this, &URigDependencyGraph::OnSetObjectBeingDebugged);
	OnVMCompiledHandle = InControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->OnVMCompiled().AddUObject(this, &URigDependencyGraph::OnVMCompiled);
	OnRigVMGraphModifiedHandle = InControlRigEditor->GetControlRigAssetInterface()->GetRigVMAssetInterface()->OnModified().AddUObject(this, &URigDependencyGraph::OnRigVMGraphModified);

	if (UControlRig* ControlRig = GetControlRig())
	{
		if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
		{
			OnHierarchyModifiedHandle = Hierarchy->OnModified().AddUObject(this, &URigDependencyGraph::OnHierarchyModified);
			OnMetadataChangedHandle = Hierarchy->OnMetadataChanged().AddUObject(this, &URigDependencyGraph::OnMetadataChanged);
		}
	}

	if (bIsFirstTime && WeakControlRigEditor.IsValid())
	{
		RebuildGraph(true);
	}
	else if (RigDependencyProvider.IsValid())
	{
		RigDependencyProvider->InvalidateCache();
	}
}

UControlRig* URigDependencyGraph::GetControlRig() const
{
	if (WeakControlRig.IsValid())
	{
		return WeakControlRig.Get();
	}
	return nullptr;
}

URigHierarchy* URigDependencyGraph::GetRigHierarchy() const
{
	if (UControlRig* ControlRig = GetControlRig())
	{
		return ControlRig->GetHierarchy();
	}
	return nullptr;
}

const FRigVMClient* URigDependencyGraph::GetRigVMClient() const
{
	TSharedRef<IControlRigBaseEditor> ControlRigEditor = GetControlRigEditor();
	return ControlRigEditor->GetControlRigAssetInterface()->GetRigVMClient();
}

void URigDependencyGraph::RequestRefreshLayout()
{
	UpdateNodeIndices();

	int32 NumSimulatedNodes = 0;
	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		if (Node->FollowLayout.Get(true))
		{
			NumSimulatedNodes++;
		}
	}
	
	LayoutIterationsLeft = LayoutIterationsTotal =
		FMath::Min(LayoutIterationsMax, DefaultLayoutIterationsPerNode * NumSimulatedNodes);

	LayoutEdges.Reset();

	UpdateFadeOutStates();
}

void URigDependencyGraph::RequestRefreshLayout(const TArray<FNodeId>& InNodes)
{
	for (const FNodeId& NodeId : InNodes)
	{
		if (URigDependencyGraphNode* const* NodePtr = NodeIdLookup.Find(NodeId))
		{
			URigDependencyGraphNode* Node = *NodePtr;
			if (Node)
			{
				Node->FollowLayout.Reset();
			}
		}
	}
	RequestRefreshLayout();
}

bool URigDependencyGraph::NeedsRefreshLayout() const
{
	return LayoutIterationsLeft > 0;
}

TSharedRef<IRigDependenciesProvider> URigDependencyGraph::GetRigDependencyProvider() const
{
	return RigDependencyProvider.ToSharedRef();
}

const URigDependencyGraphSchema* URigDependencyGraph::GetRigDependencyGraphSchema() const
{
	return CastChecked<const URigDependencyGraphSchema>(GetSchema());
}

const URigDependencyGraphNode* URigDependencyGraph::FindNode(const FNodeId& InNodeId) const
{
	if (URigDependencyGraphNode* const* NodePtr = NodeIdLookup.Find(InNodeId))
	{
		return *NodePtr;
	}
	return nullptr;
}

URigDependencyGraphNode* URigDependencyGraph::FindNode(const FNodeId& InNodeId)
{
	const URigDependencyGraph* ConstThis = this;
	return const_cast<URigDependencyGraphNode*>(ConstThis->FindNode(InNodeId));
}

bool URigDependencyGraph::Contains(const FNodeId& InNodeId) const
{
	return FindNode(InNodeId) != nullptr;
}

void URigDependencyGraph::RebuildGraph(bool bExpand)
{
	const URigHierarchy* Hierarchy = GetRigHierarchy();
	if (Hierarchy == nullptr)
	{
		return;
	}

	bool bNeedToRebuildGraph = false;
	TArray<FNodeId> PreviousNodes;
	PreviousNodes.Reserve(DependencyGraphNodes.Num());
	TMap<FNodeId, FVector2f> PreviousPositions;
	PreviousPositions.Reserve(DependencyGraphNodes.Num());
	
	for (const TObjectPtr<URigDependencyGraphNode>& PreviousNode : DependencyGraphNodes)
	{
		const FNodeId& PreviousNodeId = PreviousNode->NodeId;

		switch (PreviousNodeId.Type)
		{
			case FNodeId::EType_RigElement:
			case FNodeId::EType_Metadata:
			{
				const FRigBaseElement* Element = PreviousNode->GetRigElement();
				if (!Element)
				{
					bNeedToRebuildGraph = true;
					continue;
				}
				if (!PreviousNodeId.IsMetadata())
				{
					break;
				}
				if (!Hierarchy->FindMetadataForElement(Element, PreviousNodeId.Name, ERigMetadataType::Invalid))
				{
					bNeedToRebuildGraph = true;
					continue;
				}
				break;
			}
			case FNodeId::EType_Instruction:
			{
				if (!PreviousNode->GetRigVMNodeForInstruction())
				{
					bNeedToRebuildGraph = true;
					continue;
				}
				break;
			}
			case FNodeId::EType_Variable:
			{
				if (!PreviousNode->GetExternalVariable()->IsValid())
				{
					bNeedToRebuildGraph = true;
					continue;
				}
				break;
			}
			case FNodeId::EType_Invalid:
			default:
			{
				bNeedToRebuildGraph = true;
				continue;
			}
		}
		
		PreviousNodes.Add(PreviousNodeId);
		PreviousPositions.Add(PreviousNodeId, FVector2f(PreviousNode->NodePosX, PreviousNode->NodePosY));
	}

	if (!bNeedToRebuildGraph)
	{
		RebuildLinks();
		return;
	}
	
	RemoveAllNodes();
	ConstructNodes(PreviousNodes, bExpand);

	for (const TObjectPtr<URigDependencyGraphNode>& CurrentNode : DependencyGraphNodes)
	{
		if (const FVector2f* PreviousPosition = PreviousPositions.Find(CurrentNode->NodeId))
		{
			CurrentNode->NodePosX = PreviousPosition->X;
			CurrentNode->NodePosY = PreviousPosition->Y;
			CurrentNode->FollowLayout = false;
		}
	}
}

void URigDependencyGraph::RebuildLinks()
{
	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		Node->InputPin->BreakAllPinLinks();
		Node->OutputPin->BreakAllPinLinks();
	}

	auto MakeLink = [](URigDependencyGraphNode* InSourceNode, URigDependencyGraphNode* InTargetNode)
	{
		if (InTargetNode->NodeId == InSourceNode->NodeId)
		{
			return;
		}
		if (InTargetNode->GetNodeId().IsInstruction() &&
			InTargetNode->OutputPin->LinkedTo.Contains(InSourceNode->InputPin))
		{
			return;
		}
		if (!InSourceNode->OutputPin->LinkedTo.Contains(InTargetNode->InputPin))
		{
			InSourceNode->OutputPin->MakeLinkTo(InTargetNode->InputPin);
		}
		if (InSourceNode->GetNodeId().IsInstruction() &&
			InTargetNode->OutputPin->LinkedTo.Contains(InSourceNode->InputPin))
		{
			InTargetNode->OutputPin->BreakLinkTo(InSourceNode->InputPin);
		}
	};

	HierarchySpaceToControlHashes.Reset();
	HierarchyControlToSpace.Reset();
	HierarchySpaceToControl.Reset();

	if (bFollowParentRelationShips)
	{
		if (const URigHierarchy* Hierarchy = GetRigHierarchy())
		{
			if (bFollowControlSpaceRelationships)
			{
				ComputeControlToSpaceMap(Hierarchy, &HierarchySpaceToControlHashes, &HierarchyControlToSpace, &HierarchySpaceToControl);
			}
			
			for (TObjectPtr<URigDependencyGraphNode>& ParentElementNode : DependencyGraphNodes)
			{
				if (!ParentElementNode->NodeId.IsElement())
				{
					continue;
				}
				if (Hierarchy->IsValidIndex(ParentElementNode->NodeId.Index))
				{
					TArray<int32> Children = Hierarchy->GetChildren(ParentElementNode->NodeId.Index);
					if (bFollowControlSpaceRelationships)
					{
						if (const TArray<int32>* ControlsWithThisSpace = HierarchySpaceToControl.Find(ParentElementNode->NodeId.Index))
						{
							Children.Append(*ControlsWithThisSpace);
						}
					}
					
					for (const int32& ChildIndex : Children)
					{
						const FNodeId ChildId(FNodeId::EType_RigElement, ChildIndex, NAME_None);
						if (URigDependencyGraphNode* ChildElementNode = FindNode(ChildId))
						{
							MakeLink(ParentElementNode, ChildElementNode);
						}
					}
				}
			}

			for (TObjectPtr<URigDependencyGraphNode>& ChildElementNode : DependencyGraphNodes)
			{
				if (!ChildElementNode->NodeId.IsElement())
				{
					continue;
				}
				if (Hierarchy->IsValidIndex(ChildElementNode->NodeId.Index))
				{
					TArray<int32> Parents = Hierarchy->GetParents(ChildElementNode->NodeId.Index);
					if (bFollowControlSpaceRelationships)
					{
						if (const TArray<int32>* Spaces = HierarchyControlToSpace.Find(ChildElementNode->NodeId.Index))
						{
							Parents.Append(*Spaces);
						}
					}

					for (const int32& ParentIndex : Parents)
					{
						const FNodeId ParentId(FNodeId::EType_RigElement, ParentIndex, NAME_None);
						if (URigDependencyGraphNode* ParentElementNode = FindNode(ParentId))
						{
							MakeLink(ParentElementNode, ChildElementNode);
						}
					}
				}
			}
		}
	}

	if (bFollowVMRelationShips)
	{
		const TRigHierarchyDependencyMap& Dependencies = RigDependencyProvider->GetRigHierarchyDependencies();
		for (TObjectPtr<URigDependencyGraphNode>& TargetNode : DependencyGraphNodes)
		{
			if (const TSet<FRigHierarchyRecord>* Records = Dependencies.Find(TargetNode->NodeId))
			{
				for (const FRigHierarchyRecord& DependencyRecord : *Records)
				{
					if (URigDependencyGraphNode* SourceNode = FindNode(DependencyRecord))
					{
						MakeLink(SourceNode, TargetNode);
					}
				}
			}
		}
	}

	UpdatePinVisibility();
	UpdateFadeOutStates();
	RebuildIslands();
}

void URigDependencyGraph::RebuildIslands()
{
	NodeIslands.Reset();
	TMap<FGuid, int32> GuidToIsland;

	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		Node->IslandGuid.Reset();
	}

	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		const FGuid& Guid = Node->GetIslandGuid();
		check(Guid.IsValid());

		if (const int32* ExistingIslandIndex = GuidToIsland.Find(Guid))
		{
			NodeIslands[*ExistingIslandIndex].NodeIds.Add(Node->GetNodeId());
		}
		else
		{
			GuidToIsland.Add(Guid, NodeIslands.AddDefaulted(1));
			NodeIslands.Last().Guid = Guid;
			NodeIslands.Last().NodeIds.Add(Node->GetNodeId());
		}
	}
}

void URigDependencyGraph::UpdatePinVisibility()
{
	const URigHierarchy* Hierarchy = GetRigHierarchy();

	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		const FNodeId& NodeId = Node->GetNodeId();
		const uint32 OldHash = HashCombine(GetTypeHash(Node->InputPin->bHidden), GetTypeHash(Node->OutputPin->bHidden));
		Node->InputPin->bHidden = false;
		Node->OutputPin->bHidden = false;

		if (Node->InputPin->LinkedTo.IsEmpty())
		{
			Node->InputPin->bHidden = true;
			
			if (Node->InputPin->bHidden && NodeId.IsElement() && Hierarchy && bFollowParentRelationShips)
			{
				if (Hierarchy->GetNumberOfParents(NodeId.Index) > 0)
				{
					Node->InputPin->bHidden = false;
				}
				if (bFollowControlSpaceRelationships)
				{
					if (HierarchyControlToSpace.Contains(NodeId.Index))
					{
						Node->InputPin->bHidden = false;
					}
				}
			}

			if (Node->InputPin->bHidden && RigDependencyProvider.IsValid() && bFollowVMRelationShips)
			{
				if (const TSet<FNodeId>* Dependencies = RigDependencyProvider->GetRigHierarchyDependencies().Find(NodeId))
				{
					Node->InputPin->bHidden = false;
				}
			}
		}

		if (Node->OutputPin->LinkedTo.IsEmpty())
		{
			Node->OutputPin->bHidden = true;
			
			if (Node->OutputPin->bHidden && NodeId.IsElement() && Hierarchy && bFollowParentRelationShips)
			{
				if (Hierarchy->IsValidIndex(NodeId.Index))
				{
					if (!Hierarchy->GetChildren(NodeId.Index).IsEmpty())
					{
						Node->OutputPin->bHidden = false;
					}
					if (bFollowControlSpaceRelationships)
					{
						if (HierarchySpaceToControl.Contains(NodeId.Index))
						{
							Node->OutputPin->bHidden = false;
						}
					}
				}
			}

			if (Node->OutputPin->bHidden && RigDependencyProvider.IsValid() && bFollowVMRelationShips)
			{
				if (RigDependencyProvider->GetReverseRigHierarchyDependencies().Contains(NodeId))
				{
					Node->OutputPin->bHidden = false;
				}
			}
		}

		const uint32 NewHash = HashCombine(GetTypeHash(Node->InputPin->bHidden), GetTypeHash(Node->OutputPin->bHidden));
		if (NewHash != OldHash)
		{
			NotifyNodeChanged(Node);
		}
	}
}

void URigDependencyGraph::UpdateFadeOutStates()
{
	if (SelectedNodes.IsEmpty())
	{
		for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
		{
			Node->bIsFadedOut.Reset();
		}
		return;
	}

	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		Node->bIsFadedOut = true;
	}

	TArray<TPair<const URigDependencyGraphNode*, int32>> NodesOfInterest;
	NodesOfInterest.Reserve(DependencyGraphNodes.Num());
	for (const FNodeId& NodeId : SelectedNodes)
	{
		if (const URigDependencyGraphNode* Node = FindNode(NodeId))
		{
			NodesOfInterest.Emplace(Node, false);
		}
	}

	for (int32 Index = 0; Index < NodesOfInterest.Num(); Index++)
	{
		const URigDependencyGraphNode* Node = NodesOfInterest[Index].Key;
		if (Node->bIsFadedOut.IsSet() && Node->bIsFadedOut.GetValue() == false)
		{
			continue;
		}
		
		const int32 Direction = NodesOfInterest[Index].Value;
		Node->bIsFadedOut = false;
		
		if (Direction <= 0)
		{
			for (const UEdGraphPin* LinkedPin : Node->InputPin->LinkedTo)
			{
				if (const URigDependencyGraphNode* LinkedNode = Cast<URigDependencyGraphNode>(LinkedPin->GetOwningNode()))
				{
					NodesOfInterest.Emplace(LinkedNode, -1);
				}
			}
		}
		if (Direction >= 0)
		{
			for (const UEdGraphPin* LinkedPin : Node->OutputPin->LinkedTo)
			{
				if (const URigDependencyGraphNode* LinkedNode = Cast<URigDependencyGraphNode>(LinkedPin->GetOwningNode()))
				{
					NodesOfInterest.Emplace(LinkedNode, 1);
				}
			}
		}
	}
}

void URigDependencyGraph::SetFollowParentRelationShips(bool InFollowParentRelationShips)
{
	bFollowParentRelationShips = InFollowParentRelationShips;
	RebuildGraph(false);
}

void URigDependencyGraph::SetFollowControlSpaceRelationships(bool InFollowControlSpaceRelationships)
{
	bFollowControlSpaceRelationships = InFollowControlSpaceRelationships;
	RebuildGraph(false);
}

void URigDependencyGraph::SetFollowVMRelationShips(bool InFollowVMRelationShips)
{
	bFollowVMRelationShips = InFollowVMRelationShips;
	RebuildGraph(false);
}

void URigDependencyGraph::SetContentLocked(bool InLockContent)
{
	bLockContent = InLockContent
;}

TArray<URigDependencyGraph::FNodeId> URigDependencyGraph::GetNodesToExpand(const TArray<FNodeId>& InNodeIds, bool bExpandInputs, bool bExpandOutputs) const
{
	TArray<FNodeId> NodesToExpand;
	if (!bExpandInputs && !bExpandOutputs)
	{
		return NodesToExpand;
	}
	
	TArray<FNodeId> NodesToFollow = InNodeIds;
	
	// expand the nodes to construct one level using the dependency provider
	if (bFollowVMRelationShips && RigDependencyProvider.IsValid())
	{
		const TRigHierarchyDependencyMap& Dependencies = RigDependencyProvider->GetRigHierarchyDependencies();
		const TRigHierarchyDependencyMap& ReverseDependencies = RigDependencyProvider->GetReverseRigHierarchyDependencies();

		for (int32 NodeIndex = 0; NodeIndex < NodesToFollow.Num(); ++NodeIndex)
		{
			const FRigHierarchyRecord Record = NodesToFollow[NodeIndex];
			if (bExpandInputs)
			{
				if (const TSet<FRigHierarchyRecord>* Records = Dependencies.Find(Record))
				{
					for (const FRigHierarchyRecord& DependencyRecord : *Records)
					{
						NodesToExpand.AddUnique(DependencyRecord);
					
						if(DependencyRecord.IsInstruction() || DependencyRecord.IsVariable() || DependencyRecord.IsMetadata())
						{
							NodesToFollow.AddUnique(DependencyRecord);
						}
					}
				}
			}
			if (bExpandOutputs)
			{
				if (const TSet<FRigHierarchyRecord>* ReverseRecords = ReverseDependencies.Find(Record))
				{
					for (const FRigHierarchyRecord& DependentRecord : *ReverseRecords)
					{
						NodesToExpand.AddUnique(DependentRecord);
					}
				}
			}
		}
	}

	if (bFollowParentRelationShips)
	{
		if (const URigHierarchy* Hierarchy = GetRigHierarchy())
		{
			TMap<int32, TArray<int32>> ControlToSpace, SpaceToControl;
			if (bFollowControlSpaceRelationships)
			{
				ComputeControlToSpaceMap(Hierarchy, nullptr, &ControlToSpace, &SpaceToControl);
			}

			// also expand the nodes to construct parent and child nodes
			for (int32 NodeIndex = 0; NodeIndex < InNodeIds.Num(); ++NodeIndex)
			{
				if (InNodeIds[NodeIndex].Type != FNodeId::EType_RigElement)
				{
					continue;
				}

				if (Hierarchy->IsValidIndex(InNodeIds[NodeIndex].Index))
				{
					TArray<int32> ParentsAndChildren;
					if (bExpandInputs)
					{
						ParentsAndChildren.Append(Hierarchy->GetParents(InNodeIds[NodeIndex].Index));
						if (bFollowControlSpaceRelationships)
						{
							if (const TArray<int32>* Spaces = ControlToSpace.Find(InNodeIds[NodeIndex].Index))
							{
								ParentsAndChildren.Append(*Spaces);
							}
						}
					}
					if (bExpandOutputs)
					{
						ParentsAndChildren.Append(Hierarchy->GetChildren(InNodeIds[NodeIndex].Index));
						if (bFollowControlSpaceRelationships)
						{
							if (const TArray<int32>* ControlsUsingThisSpace = SpaceToControl.Find(InNodeIds[NodeIndex].Index))
							{
								ParentsAndChildren.Append(*ControlsUsingThisSpace);
							}
						}
					}

					for (const int32& ParentOrChildIndex : ParentsAndChildren)
					{
						const FNodeId ParentOrChildId(FNodeId::EType_RigElement, ParentOrChildIndex, NAME_None);
						NodesToExpand.AddUnique(ParentOrChildId);
					}
				}
			}
		}
	}

	// only consider nodes to expand that we don't have yet
	NodesToExpand.RemoveAll([this](const FNodeId& InNodeId) -> bool
	{
		return FindNode(InNodeId) != nullptr;
	});

	return NodesToExpand;
}

const URigVMNode* URigDependencyGraph::GetRigVMNodeForInstruction(const FNodeId& InNodeId) const
{
	if (!InNodeId.IsInstruction())
	{
		return nullptr;
	}

	UControlRig* ControlRig = GetControlRig();
	if (!ControlRig)
	{
		return nullptr;
	}

	UModularRig* ModularRig = Cast<UModularRig>(ControlRig);

	// find the node path in question based on the VM
	const URigVM* VM = nullptr;
								
	if (InNodeId.Name.IsNone() || !ModularRig)
	{
		VM = ControlRig->GetVM();
	}
	else
	{
		const FRigModuleInstance* RigModule = ModularRig->FindModule(InNodeId.Name);
		if (!RigModule)
		{
			return nullptr;
		}
		UControlRig* ModuleRig = RigModule->GetRig();
		if (!ModuleRig)
		{
			return nullptr;
		}
		VM = ModuleRig->GetVM();
	}

	check(VM);

	if (!VM->GetByteCode().GetInstructions().IsValidIndex(InNodeId.Index))
	{
		return nullptr;
	}
	
	const FRigVMInstruction Instruction = VM->GetByteCode().GetInstructions()[InNodeId.Index];
	if (Instruction.OpCode != ERigVMOpCode::Execute)
	{
		return nullptr;
	}

	return Cast<URigVMNode>(VM->GetByteCode().GetSubjectForInstruction(InNodeId.Index));
}

FRigHierarchyRecord URigDependencyGraph::GetRecordForRigVMNode(const URigVMNode* InNode) const
{
	if (!InNode)
	{
		return FRigHierarchyRecord();
	}

	UControlRig* ControlRig = GetControlRig();
	if (!ControlRig)
	{
		return FRigHierarchyRecord();
	}

	UModularRig* ModularRig = Cast<UModularRig>(ControlRig);
	
	TArray<TTuple<const URigVM*, FName>> VMs;;
								
	if (!ModularRig)
	{
		VMs.Emplace(ControlRig->GetVM(), NAME_None);
	}
	else
	{
		ModularRig->ForEachModule([&VMs](FRigModuleInstance* RigModuleInstance)
		{
			if (UControlRig* ModuleRig = RigModuleInstance->GetRig())
			{
				VMs.Emplace(ModuleRig->GetVM(), RigModuleInstance->Name);
			}
			return true;
		});
	}

	TSharedRef<IRigDependenciesProvider> Provider = GetRigDependencyProvider();
	for (const TTuple<const URigVM*, FName>& Pair : VMs)
	{
		const URigVM* VM = Pair.Get<0>();
		const TArray<int32>& InstructionIndices = VM->GetByteCode().GetAllInstructionIndicesForSubject(const_cast<URigVMNode*>(InNode));
		for (const int32& InstructionIndex : InstructionIndices)
		{
			const FRigVMInstruction Instruction = VM->GetByteCode().GetInstructions()[InstructionIndex];
			if (Instruction.OpCode != ERigVMOpCode::Execute)
			{
				continue;
			}
			
			const FRigHierarchyRecord Record(FRigHierarchyRecord::EType_Instruction, InstructionIndex, Pair.Get<1>());
			if (Provider->GetRigHierarchyDependencies().Contains(Record) ||
				Provider->GetReverseRigHierarchyDependencies().Contains(Record))
			{
				return Record;
			}
		}
	}

	return FRigHierarchyRecord();
}

const FRigVMExternalVariableDef* URigDependencyGraph::GetExternalVariable(const FNodeId& InNodeId) const
{
	if (!InNodeId.IsVariable())
	{
		return nullptr;
	}

	UControlRig* ControlRig = GetControlRig();
	if (!ControlRig)
	{
		return nullptr;
	}

	UModularRig* ModularRig = Cast<UModularRig>(ControlRig);

	// find the node path in question based on the VM
	const URigVM* VM = nullptr;
								
	if (InNodeId.Name.IsNone() || !ModularRig)
	{
		VM = ControlRig->GetVM();
	}
	else
	{
		const FRigModuleInstance* RigModule = ModularRig->FindModule(InNodeId.Name);
		if (!RigModule)
		{
			return nullptr;
		}
		UControlRig* ModuleRig = RigModule->GetRig();
		if (!ModuleRig)
		{
			return nullptr;
		}
		VM = ModuleRig->GetVM();
	}

	check(VM);

	const TArray<FRigVMExternalVariableDef>& ExternalVariableDefs = VM->GetExternalVariableDefs();

	if (ExternalVariableDefs.IsValidIndex(InNodeId.Index))
	{
		return &ExternalVariableDefs[InNodeId.Index];
	}

	return nullptr;
}

void URigDependencyGraph::Tick(float DeltaTime)
{
	if (--BlockSelectionCounter == 0)
	{
		UE::TScopeLock Lock(NodesToSelectCriticalSection);
		if (NodesToSelectDuringTick.IsSet())
		{
			const TArray<FNodeId> NodeIds = NodesToSelectDuringTick.GetValue();
			bool bRemoveNodes = RemoveAllNodesDuringTick.Get(true);

			SelectedNodes.Reset();
			NodesToSelectDuringTick.Reset();
			RemoveAllNodesDuringTick.Reset();
			ZoomAndFitDuringLayout = bRemoveNodes;

			if (bRemoveNodes)
			{
				RemoveAllNodes();
			}
			ConstructNodes(NodeIds, bRemoveNodes);
			SelectNodes(NodeIds);
		}
	}

	if (LastRecordsHash.IsSet() && !bLockContent)
	{
		if (const UControlRig* ControlRig = GetControlRig())
		{
			const FRigVMExtendedExecuteContext& ExtendedExecuteContext = ControlRig->GetRigVMExtendedExecuteContext();
			const FControlRigExecuteContext& Context = ExtendedExecuteContext.GetPublicData<FControlRigExecuteContext>();
			if (LastRecordsHash.GetValue() != Context.GetInstructionRecordsHash())
			{
				RebuildGraph(false);
				LastRecordsHash = Context.GetInstructionRecordsHash();
			}
		}
	}
}

void URigDependencyGraph::SelectAllNodes()
{
	SelectNodes(GetAllNodeIds());
}

void URigDependencyGraph::ConstructNodes(const TArray<FNodeId>& InNodesToConstruct, bool bExpand)
{
	if (!RigDependencyProvider.IsValid())
	{
		return;
	}

	const URigHierarchy* Hierarchy = GetRigHierarchy();
	if (!Hierarchy)
	{
		return;
	}

	TArray<FNodeId> NodesToConstruct = InNodesToConstruct;

	if (bExpand)
	{
		if (!NodesToConstruct.IsEmpty())
		{
			const TArray<FNodeId> NodesToExpand = GetNodesToExpand(InNodesToConstruct);
			for (const FNodeId& NodeId : NodesToExpand)
			{
				NodesToConstruct.AddUnique(NodeId);
			}
		}
	}
	
	const TArray<FNodeId> PreviousSelection = SelectedNodes;
	SelectedNodes.Reset();
	bool bCreatedAnyNodes = false;

	TSet<const UEdGraphNode*> NodesToSelect;
	for (const FNodeId& NodeId : NodesToConstruct)
	{
		URigDependencyGraphNode* Node = FindNode(NodeId);
		if (!Node)
		{
			Node = GetRigDependencyGraphSchema()->CreateGraphNode(this, NodeId);
			bCreatedAnyNodes = true;
		}
		
		if(PreviousSelection.Contains(NodeId))
		{
			SelectedNodes.Add(NodeId);
			NodesToSelect.Add(Node);
		}
	}

	SelectNodeSet(NodesToSelect);

	RebuildLinks();

	if (bCreatedAnyNodes)
	{
		RequestRefreshLayout();
		NotifyGraphChanged();
	}
	
	if (const UControlRig* ControlRig = GetControlRig())
	{
		const FRigVMExtendedExecuteContext& ExtendedExecuteContext = ControlRig->GetRigVMExtendedExecuteContext();
		const FControlRigExecuteContext& Context = ExtendedExecuteContext.GetPublicData<FControlRigExecuteContext>();
		LastRecordsHash = Context.GetInstructionRecordsHash();
	}
}

void URigDependencyGraph::RemoveAllNodes()
{
	if (Nodes.IsEmpty())
	{
		return;
	}

	TArray<URigDependencyGraphNode*> NodesToRemove = DependencyGraphNodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		RemoveDependencyGraphNode(NodesToRemove[NodeIndex]);
	}
	
	DependencyGraphNodes.Reset();
	LastRecordsHash.Reset();
	bIsPerformingGridLayout = true;

	RequestRefreshLayout();
	NotifyGraphChanged();
}

TArray<URigDependencyGraph::FNodeId> URigDependencyGraph::GetAllNodeIds() const
{
	TArray<FNodeId> NodeIds;
	NodeIds.Reserve(DependencyGraphNodes.Num());
	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		NodeIds.Add(Node->GetNodeId());
	}
	return NodeIds;
}

void URigDependencyGraph::RemoveSelectedNodes()
{
	if (DependencyGraphNodes.IsEmpty() || SelectedNodes.IsEmpty())
	{
		return;
	}

	TArray<TObjectPtr<URigDependencyGraphNode>> NodesToRemove = DependencyGraphNodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		if (!SelectedNodes.Contains(NodesToRemove[NodeIndex]->NodeId))
		{
			continue;
		}
		RemoveDependencyGraphNode(NodesToRemove[NodeIndex]);
	}

	SelectedNodes.Reset();
	RequestRefreshLayout();
	NotifyGraphChanged();
}

void URigDependencyGraph::RemoveUnselectedNodes()
{
	if (DependencyGraphNodes.IsEmpty() || SelectedNodes.IsEmpty())
	{
		return;
	}

	TArray<TObjectPtr<URigDependencyGraphNode>> NodesToRemove = DependencyGraphNodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		if (SelectedNodes.Contains(NodesToRemove[NodeIndex]->NodeId))
		{
			continue;
		}
		RemoveDependencyGraphNode(NodesToRemove[NodeIndex]);
	}

	RequestRefreshLayout();
	NotifyGraphChanged();
}

void URigDependencyGraph::RemoveUnrelatedNodes()
{
	if (DependencyGraphNodes.IsEmpty() || SelectedNodes.IsEmpty())
	{
		return;
	}

	TArray<FGuid> Guids;
	for (const FNodeId& NodeId : SelectedNodes)
	{
		if (const URigDependencyGraphNode* Node = FindNode(NodeId))
		{
			if (Node->IslandGuid.IsSet())
			{
				Guids.AddUnique(Node->IslandGuid.GetValue());
			}
		}
	}

	bool bRemovedAnyNodes = false;
	TArray<TObjectPtr<URigDependencyGraphNode>> PreviousNodes = DependencyGraphNodes;
	for (int32 NodeIndex = 0; NodeIndex < PreviousNodes.Num(); ++NodeIndex)
	{
		if (!Guids.Contains(PreviousNodes[NodeIndex]->IslandGuid.Get(FGuid())))
		{
			DependencyGraphNodes.Remove(PreviousNodes[NodeIndex]);
			SelectedNodes.Remove(PreviousNodes[NodeIndex]->NodeId);
			RemoveDependencyGraphNode(PreviousNodes[NodeIndex]);
			bRemovedAnyNodes = true;
		}
	}

	if (bRemovedAnyNodes)
	{
		RequestRefreshLayout();
		NotifyGraphChanged();
	}
}

void URigDependencyGraph::UpdateNodeIndices()
{
	for (int32 NodeIndex = 0; NodeIndex < DependencyGraphNodes.Num(); ++NodeIndex)
	{
		DependencyGraphNodes[NodeIndex]->Index = NodeIndex;
	}
}

void URigDependencyGraph::RemoveDependencyGraphNode(URigDependencyGraphNode* InNode)
{
	check(InNode);
	SelectedNodes.Remove(InNode->GetNodeId());
	DependencyGraphNodes.Remove(InNode);
	NodeIdLookup.Remove(InNode->GetNodeId());
	RemoveNode(InNode);
	UpdateFadeOutStates();
}

void URigDependencyGraph::SelectLinkedNodes(const TArray<FNodeId>& InNodeIds, bool bInputNodes, bool bClearSelection, bool bRecursive)
{
	TArray<FNodeId> NodesToSelect = InNodeIds;
	for (int32 Index = 0; Index < NodesToSelect.Num(); ++Index)
	{
		if (!bRecursive)
		{
			if (!InNodeIds.Contains(NodesToSelect[Index]))
			{
				break;
			}
		}
		if (const URigDependencyGraphNode* Node = FindNode(NodesToSelect[Index]))
		{
			const TArray<UEdGraphPin*>& LinkedPins = bInputNodes ? Node->InputPin->LinkedTo : Node->OutputPin->LinkedTo; 
			for (const UEdGraphPin* LinkedPin : LinkedPins)
			{
				if (const URigDependencyGraphNode* LinkedNode = Cast<URigDependencyGraphNode>(LinkedPin->GetOwningNode()))
				{
					NodesToSelect.AddUnique(LinkedNode->GetNodeId());
				}
			}
		}
	}

	if (!bClearSelection)
	{
		for (const FNodeId& SelectedNodeId : SelectedNodes)
		{
			NodesToSelect.AddUnique(SelectedNodeId);
		}
	}

	SelectNodes(NodesToSelect);
}

void URigDependencyGraph::SelectNodeIsland(const TArray<FNodeId>& InNodeIds, bool bClearSelection)
{
	TArray<FGuid> Guids;
	for (const FNodeId& NodeId : InNodeIds)
	{
		if (const URigDependencyGraphNode* Node = FindNode(NodeId))
		{
			if (Node->IslandGuid.IsSet())
			{
				Guids.AddUnique(Node->IslandGuid.GetValue());
			}
		}
	}
	
	TArray<FNodeId> NodesToSelect;
	for (const FNodeIsland& Island : NodeIslands)
	{
		if (Guids.Contains(Island.Guid))
		{
			for (const FNodeId& NodeId : Island.NodeIds)
			{
				NodesToSelect.AddUnique(NodeId);
			}
		}
	}

	SelectNodes(NodesToSelect);
}

FBox2D URigDependencyGraph::GetAllNodesBounds() const
{
	if (DependencyGraphNodes.IsEmpty())
	{
		return FBox2D();
	}

	FBox2D Bounds(EForceInit::ForceInit);
	for (const TObjectPtr<URigDependencyGraphNode>& Node : DependencyGraphNodes)
	{
		const FVector2D TopLeft(Node->NodePosX, Node->NodePosY);
		Bounds += TopLeft;
		Bounds += TopLeft + Node->Dimensions;
	}
	return Bounds;
}

FBox2D URigDependencyGraph::GetSelectedNodesBounds() const
{
	if (DependencyGraphNodes.IsEmpty() || SelectedNodes.IsEmpty())
	{
		return FBox2D();
	}

	return GetNodesBounds(SelectedNodes);
}

void URigDependencyGraph::CancelLayout()
{
	LayoutIterationsLeft = 0;
	bIsPerformingGridLayout = false;
}

void URigDependencyGraph::OnHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject)
{
	if (InNotification == ERigHierarchyNotification::ElementSelected || 
		InNotification == ERigHierarchyNotification::ElementDeselected)
	{
		UE::TScopeLock Lock(NodesToSelectCriticalSection);

		if (!NodesToSelectDuringTick.IsSet())
		{
			if (BlockSelectionCounter > 0)
			{
				return;
			}
		}
		
		URigHierarchy* Hierarchy = GetRigHierarchy();
		if (Hierarchy == nullptr)
		{
			return;
		}
		
		const TArray<FRigElementKey> SelectedElements = Hierarchy->GetSelectedKeys();

		bool bRequiresReconstruction = false;
		TArray<FNodeId> NodesToSelect;
		for (const FRigElementKey& SelectedKey : SelectedElements)
		{
			const FNodeId NodeId(FNodeId::EType_RigElement, Hierarchy->GetIndex(SelectedKey), NAME_None);
			if (!bRequiresReconstruction && FindNode(NodeId) == nullptr)
			{
				if (bLockContent)
				{
					continue;
				}
				bRequiresReconstruction = true;
			}
			NodesToSelect.Add(NodeId);
		}

		// maintain the selection of non-hierarchy based nodes
		for (const FNodeId& SelectedNode : SelectedNodes)
		{
			if (!SelectedNode.IsElement() && !SelectedNode.IsMetadata())
			{
				NodesToSelect.Add(SelectedNode);
			}
		}

		NodesToSelectDuringTick = NodesToSelect;
		RemoveAllNodesDuringTick = bRequiresReconstruction;
		BlockSelectionCounter = 2;
	}
}

void URigDependencyGraph::OnMetadataChanged(const FRigElementKey& InKey, const FName& InMetadataName)
{
	// todo
}

void URigDependencyGraph::OnSetObjectBeingDebugged(UObject* InObject)
{
	if (!WeakControlRigEditor.IsValid())
	{
		return;
	}
	
	UControlRig* ControlRig = Cast<UControlRig>(InObject);
	if (!ControlRig || !IsValid(ControlRig))
	{
		ControlRig = nullptr;
	}
	Initialize(GetControlRigEditor(), ControlRig);
}

void URigDependencyGraph::OnVMCompiled(UObject* InAsset, URigVM* InVM, FRigVMExtendedExecuteContext& InExtendedContext)
{
	for (const TObjectPtr<URigDependencyGraphNode>& DependencyGraphNode : DependencyGraphNodes)
	{
		DependencyGraphNode->InvalidateCache();
	}

	if (RigDependencyProvider.IsValid())
	{
		RigDependencyProvider->InvalidateCache();
	}
}

void URigDependencyGraph::OnRigVMGraphModified(ERigVMGraphNotifType InNotification, URigVMGraph* InRigVMGraph, UObject* InSubject)
{
	if (!WeakControlRigEditor.IsValid())
	{
		return;
	}

	if (InNotification == ERigVMGraphNotifType::NodeSelectionChanged)
	{
		UE::TScopeLock Lock(NodesToSelectCriticalSection);

		if (!NodesToSelectDuringTick.IsSet())
		{
			if (BlockSelectionCounter > 0)
			{
				return;
			}
		}

		check(InRigVMGraph);
		
		const TArray<FName> SelectNodeNames = InRigVMGraph->GetSelectNodes();
		
		TArray<FNodeId> NodesToSelect;
		for (const FName& SelectedNodeName : SelectNodeNames)
		{
			if (const URigVMNode* SelectedNode = InRigVMGraph->FindNodeByName(SelectedNodeName))
			{
				for (const TObjectPtr<URigDependencyGraphNode>& DependencyGraphNode : DependencyGraphNodes)
				{
					if (DependencyGraphNode->GetRigVMNodeForInstruction() == SelectedNode)
					{
						NodesToSelect.Add(DependencyGraphNode->GetNodeId());
					}
				}
			}
		}

		RemoveAllNodesDuringTick = false;
		
		if (NodesToSelect.IsEmpty() && !bLockContent)
		{
			// we didn't find the nodes selected on our view just yet,
			// let's sync to this new view. first - we need to find the nodes in question.
			for (const FName& SelectedNodeName : SelectNodeNames)
			{
				if (const URigVMNode* SelectedNode = InRigVMGraph->FindNodeByName(SelectedNodeName))
				{
					const FRigHierarchyRecord& Record = GetRecordForRigVMNode(SelectedNode);
					if (Record.IsValid())
					{
						NodesToSelect.Add(Record);
						RemoveAllNodesDuringTick = true;
					}
				}
			}
		}

		// maintain the selection of non-instruction based nodes
		for (const FNodeId& SelectedNode : SelectedNodes)
		{
			if (!SelectedNode.IsInstruction() || !SelectedNode.IsVariable())
			{
				NodesToSelect.Add(SelectedNode);
			}
		}

		NodesToSelectDuringTick = NodesToSelect;
		BlockSelectionCounter = 2;
	}
}

void URigDependencyGraph::ComputeControlToSpaceMap(const URigHierarchy* InHierarchy, TSet<uint32>* OutHashes, TMap<int32, TArray<int32>>* OutControlToSpace, TMap<int32, TArray<int32>>* OutSpaceToControl)
{
	if (OutHashes)
	{
		OutHashes->Reset();
	}
	if (OutControlToSpace)
	{
		OutControlToSpace->Reset();
	}
	if (OutSpaceToControl)
	{
		OutSpaceToControl->Reset();
	}
	if (!InHierarchy || (!OutHashes && !OutControlToSpace && !OutSpaceToControl))
	{
		return;
	}

	TArray<FRigControlElement*> Controls = InHierarchy->GetControls();
	for (const FRigControlElement* Control : Controls)
	{
		for (const FRigElementKeyWithLabel& AvailableSpace : Control->Settings.Customization.AvailableSpaces)
		{
			const int32 SpaceIndex = InHierarchy->GetIndex(AvailableSpace.Key);
			if (SpaceIndex == INDEX_NONE)
			{
				continue;
			}
			if (OutHashes)
			{
				OutHashes->Add(HashCombine(GetTypeHash(SpaceIndex), GetTypeHash(Control->GetIndex())));
			}
			if (OutControlToSpace)
			{
				OutControlToSpace->FindOrAdd(Control->GetIndex()).Add(SpaceIndex);
			}
			if (OutSpaceToControl)
			{
				OutSpaceToControl->FindOrAdd(SpaceIndex).Add(Control->GetIndex());
			}
		}
	}
}

void URigDependencyGraph::SelectNodes(const TArray<FNodeId>& InNodeIds)
{
	BlockSelectionCounter = 2;

	SelectedNodes.Reset();
	
	TSet<const UEdGraphNode*> NodesToSelect;
	for (const FNodeId& NodeId : InNodeIds)
	{
		if (const URigDependencyGraphNode* Node = FindNode(NodeId))
		{
			SelectedNodes.Add(NodeId);
			NodesToSelect.Add(Node);
		}
	}

	UpdateFadeOutStates();
	
	if (NodesToSelect.IsEmpty())
	{
		ClearNodeSelection();
		return;
	}
	SelectNodeSet(NodesToSelect);
}

void URigDependencyGraph::ClearNodeSelection()
{
	(void)OnClearSelection.ExecuteIfBound();
}
