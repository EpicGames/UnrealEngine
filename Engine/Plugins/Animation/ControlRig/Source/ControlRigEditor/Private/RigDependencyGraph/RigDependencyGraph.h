// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDependencyGraphNode.h"
#include "EdGraph/EdGraph.h"
#include "Rigs/RigDependency.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMModel/RigVMNotifications.h"
#include "RigDependencyGraph.generated.h"

class URigDependencyGraphSchema;
class IControlRigBaseEditor;
class UControlRig;
struct FRigVMClient;

UCLASS()
class URigDependencyGraph : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	using FNodeId = URigDependencyGraphNode::FNodeId;
	
	void Initialize(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor, const TWeakObjectPtr<UControlRig>& InControlRig);

	/** Rebuilds graph from selection */
	void RebuildGraph(bool bExpand = false);

	/** Rebuilds the links */
	void RebuildLinks();

	/* Recomputes the islands based on the connectivity */ 
	void RebuildIslands();

	/** Update the input and output pins */
	void UpdatePinVisibility();

	/** Update the nodes' fade out states */
	void UpdateFadeOutStates();

	/** Select the nodes */
	void SelectNodes(const TArray<FNodeId>& InNodeIds);

	/** Removes all nodes from the selection */
	void ClearNodeSelection();

	/** Get the dependency graph nodes we are displaying */
	const TArray<URigDependencyGraphNode*>& GetNodes() const { return DependencyGraphNodes; }

	/** Returns the node given a Node Id */
	const URigDependencyGraphNode* FindNode(const FNodeId& InNodeId) const;
	URigDependencyGraphNode* FindNode(const FNodeId& InNodeId);

	/** Returns true if the node given a Node Id exists */
	bool Contains(const FNodeId& InNodeId) const;

	/** Get the control rig asset editor we are embedded in */
	TSharedRef<IControlRigBaseEditor> GetControlRigEditor() const { return WeakControlRigEditor.Pin().ToSharedRef(); }

	/** Get the control rig we are displaying */
	UControlRig* GetControlRig() const;

	/** Get the hierarchy we are displaying */
	URigHierarchy* GetRigHierarchy() const;

	/** Get the client to resolve a node for */
	const FRigVMClient* GetRigVMClient() const;

	/** Requests a layout refresh */
	void RequestRefreshLayout();
	void RequestRefreshLayout(const TArray<FNodeId>& InNodes);

	/** Get whether a layout refresh was requested */
	bool NeedsRefreshLayout() const;

	/** Get the dependency provider we are editing */
	TSharedRef<IRigDependenciesProvider> GetRigDependencyProvider() const;

	/** Get the dependency graph schema */
	const URigDependencyGraphSchema* GetRigDependencyGraphSchema() const;

	bool DoesFollowParentRelationShips() const { return bFollowParentRelationShips; } 
	bool DoesFollowVMRelationShips() const { return bFollowVMRelationShips; } 
	bool IsContentLocked() const { return bLockContent; } 

	void SetFollowParentRelationShips(bool InFollowParentRelationShips);
	void SetFollowControlSpaceRelationships(bool InFollowControlSpaceRelationships);
	void SetFollowVMRelationShips(bool InFollowVMRelationShips);
	void SetContentLocked(bool InLockContent);

	TArray<FNodeId> GetNodesToExpand(const TArray<FNodeId>& InNodeIds, bool bExpandInputs = true, bool bExpandOutputs = true) const;

	const URigVMNode* GetRigVMNodeForInstruction(const FNodeId& InNodeId) const;
	FRigHierarchyRecord GetRecordForRigVMNode(const URigVMNode* InNode) const;

	const FRigVMExternalVariableDef* GetExternalVariable(const FNodeId& InNodeId) const;

	void Tick(float DeltaTime);

	void SelectAllNodes();
	void RemoveAllNodes();
	TArray<FNodeId> GetAllNodeIds() const;
	const TArray<FNodeId>& GetSelectedNodeIds() const { return SelectedNodes; }
	void RemoveSelectedNodes();
	void RemoveUnselectedNodes();
	void RemoveUnrelatedNodes();
	void SelectLinkedNodes(const TArray<FNodeId>& InNodeIds, bool bInputNodes, bool bClearSelection, bool bRecursive);
	void SelectNodeIsland(const TArray<FNodeId>& InNodeIds, bool bClearSelection);

private:
	void ConstructNodes(const TArray<FNodeId>& InNodesToConstruct, bool bExpand);
	void UpdateNodeIndices();
	void RemoveDependencyGraphNode(URigDependencyGraphNode* InNode);

	FBox2D GetAllNodesBounds() const;
	FBox2D GetSelectedNodesBounds() const;

	template<typename RangeType>
	FBox2D GetNodesBounds(const RangeType& InNodeIds) const
	{
		FBox2D Bounds(EForceInit::ForceInit);
		for (const FNodeId& NodeId : InNodeIds)
		{
			if (const URigDependencyGraphNode* Node = FindNode(NodeId))
			{
				const FVector2D TopLeft(Node->NodePosX, Node->NodePosY);
				Bounds += TopLeft + Node->Dimensions;
			}
		}
		return Bounds;
	}

	void CancelLayout();

	void OnHierarchyModified(ERigHierarchyNotification InNotification, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	FDelegateHandle OnHierarchyModifiedHandle;

	void OnMetadataChanged(const FRigElementKey& InKey, const FName& InMetadataName);
	FDelegateHandle OnMetadataChangedHandle;;
	
	void OnSetObjectBeingDebugged(UObject* InObject);
	FDelegateHandle OnSetObjectBeingDebuggedHandle;

	void OnVMCompiled(UObject* InAsset, URigVM* InVM, FRigVMExtendedExecuteContext& InExtendedContext);
	FDelegateHandle OnVMCompiledHandle;

	void OnRigVMGraphModified(ERigVMGraphNotifType InNotification, URigVMGraph* InRigVMGraph, UObject* InSubject);
	FDelegateHandle OnRigVMGraphModifiedHandle;
	
private:

	static void ComputeControlToSpaceMap(const URigHierarchy* InHierarchy, TSet<uint32>* OutHashes, TMap<int32, TArray<int32>>* OutControlToSpace, TMap<int32, TArray<int32>>* OutSpaceToControl);
	
	TArray<FNodeId> SelectedNodes;

	UPROPERTY()
	TArray<TObjectPtr<URigDependencyGraphNode>> DependencyGraphNodes;

	TMap<FNodeId, URigDependencyGraphNode*> NodeIdLookup;
	
	TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;
	TWeakObjectPtr<UControlRig> WeakControlRig;
	TSharedPtr<IRigDependenciesProvider> RigDependencyProvider;

	mutable FTransactionallySafeCriticalSection NodesToSelectCriticalSection;
	mutable TOptional<TArray<FNodeId>> NodesToSelectDuringTick;
	mutable TOptional<bool> RemoveAllNodesDuringTick;
	mutable TOptional<bool> ZoomAndFitDuringLayout;
	TOptional<uint32> LastRecordsHash;

	int32 DefaultLayoutIterationsPerNode;
	int32 LayoutIterationsLeft;
	int32 LayoutIterationsTotal;
	int32 LayoutIterationsMax;
	bool bFollowParentRelationShips;
	bool bFollowControlSpaceRelationships;
	bool bFollowVMRelationShips;
	bool bLockContent;
	bool bIsPerformingGridLayout;

	TSet<uint32> HierarchySpaceToControlHashes;
	TMap<int32, TArray<int32>> HierarchyControlToSpace;
	TMap<int32, TArray<int32>> HierarchySpaceToControl;

	/** As selection broadcasting is deferred in the graph, we need to block re-broadcasting a for a few frames on refresh */
	int32 BlockSelectionCounter;

	struct FLayoutEdge
	{
		FLayoutEdge()
			: NodeA(INDEX_NONE)
			, NodeB(INDEX_NONE)
			, AnchorA(FVector2D::ZeroVector)
			, AnchorB(FVector2D::ZeroVector)
			, Strength(0.f)
		{
		}
		
		FLayoutEdge(int32 InNodeA, int32 InNodeB, const FVector2D& InAnchorA, const FVector2D& InAnchorB, float InStrength)
			: NodeA(InNodeA)
			, NodeB(InNodeB)
			, AnchorA(InAnchorA)
			, AnchorB(InAnchorB)
			, Strength(InStrength)
		{
			if (NodeA > NodeB)
			{
				Swap(NodeA, NodeB);
				Swap(AnchorA, AnchorB);
			}
		}

		friend uint32 GetTypeHash(const FLayoutEdge& InEdge)
		{
			return HashCombine(GetTypeHash(InEdge.NodeA), GetTypeHash(InEdge.NodeB));
		}

		bool operator==(const FLayoutEdge& InOther) const
		{
			return NodeA == InOther.NodeA && NodeB == InOther.NodeB;
		}

		bool operator>(const FLayoutEdge& InOther) const
		{
			if (NodeA > InOther.NodeA)
			{
				return true;
			}
			if (NodeA < InOther.NodeA)
			{
				return false;
			}
			return NodeB > InOther.NodeB;
		}

		bool operator<(const FLayoutEdge& InOther) const
		{
			if (NodeA < InOther.NodeA)
			{
				return true;
			}
			if (NodeA > InOther.NodeA)
			{
				return false;
			}
			return NodeB < InOther.NodeB;
		}

		int32 NodeA;
		int32 NodeB;
		FVector2D AnchorA;
		FVector2D AnchorB;
		float Strength;
	};

	struct FNodeIsland
	{
		FGuid Guid;
		TSet<FNodeId> NodeIds;
		FBox2D Bounds;
	};

	TSet<FLayoutEdge> LayoutEdges;
	TArray<FNodeIsland> NodeIslands;

	FSimpleDelegate OnClearSelection;

	friend class SRigDependencyGraph;
	friend class URigDependencyGraphSchema;
	friend class FRigDependencyConnectionDrawingPolicy;
};

