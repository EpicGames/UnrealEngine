// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraphCommands.h"

#define LOCTEXT_NAMESPACE "RigDependencyGraphCommands"

void FRigDependencyGraphCommands::RegisterCommands()
{
	UI_COMMAND(FrameSelection, "Frame Selection", "Expands and frames the selection in the tree", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(JumpToSource, "Jump to Source", "Jumps to the instruction node backing up the dependency node", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::J));
	UI_COMMAND(ShowParentChildRelationships, "Show Parent/Child", "Whether to show or hide parent and child relationships", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::P));
	UI_COMMAND(ShowControlSpaceRelationships, "Show Control Spaces", "Whether to show or hide space relationships for controls", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::P));
	UI_COMMAND(ShowInstructionRelationships, "Show VM Nodes", "Whether to show or hide VM based relationships", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::O));
	UI_COMMAND(LockContent, "Lock Content", "Whether or not to sync the content with selection in other views", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::L));
	UI_COMMAND(EnableFlashlight, "Enable Flashlight", "Enable a flashlight around the mouse cursor to brighten up faded out nodes", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SelectAllNodes, "Select All", "Selects all nodes in the graph", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::A));
	UI_COMMAND(SelectInputNodes, "Select Inputs", "Selects the linked inputs", EUserInterfaceActionType::Button, FInputChord(EKeys::MiddleMouseButton));
	UI_COMMAND(SelectOutputNodes, "Select Outputs", "Selects the linked outputs", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::MiddleMouseButton));
	UI_COMMAND(SelectNodeIsland, "Select Island", "Selects the linked island", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::MiddleMouseButton));
	UI_COMMAND(RemoveAllNodes, "Clear View", "Hides all nodes", EUserInterfaceActionType::Button, FInputChord(EKeys::X));
	UI_COMMAND(RemoveSelected, "Hide Selection", "Hides the selected nodes", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
	UI_COMMAND(RemoveUnrelated, "Hide Unrelated", "Hides nodes that not related to the selection", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(IsolateSelected, "Isolate Selection", "Hides all non-selected nodes", EUserInterfaceActionType::Button, FInputChord(EKeys::I));
	UI_COMMAND(RunLayoutSimulation, "Auto Layout", "Starts a layout simulation to relax the nodes", EUserInterfaceActionType::Button, FInputChord(EKeys::R));
	UI_COMMAND(CancelLayoutSimulation, "Cancel Layout", "Cancels the currently running layout simulation", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(FindNodesByName, "Find Nodes By Name", "Finds nodes by name", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));
}

#undef LOCTEXT_NAMESPACE
