// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "ControlRigEditorStyle.h"

class FRigDependencyGraphCommands : public TCommands<FRigDependencyGraphCommands>
{
public:
	FRigDependencyGraphCommands() : TCommands<FRigDependencyGraphCommands>
	(
		"ControlRigDependencyGraph",
		NSLOCTEXT("Contexts", "RigDependencyGraph", "Dependencies"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FControlRigEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	/* frames the selection in the graph */
	TSharedPtr<FUICommandInfo> FrameSelection;

	/* Jumps to the instruction node backing up the dependency node */
	TSharedPtr<FUICommandInfo> JumpToSource;

	/* Show parent-child relationships */
	TSharedPtr< FUICommandInfo > ShowParentChildRelationships;

	/* Show control-space relationships */
	TSharedPtr< FUICommandInfo > ShowControlSpaceRelationships;

	/* Show instruction based relationships */
	TSharedPtr< FUICommandInfo > ShowInstructionRelationships;

	/* Locks the content of the view and stops syncing based on selection changes */
	TSharedPtr< FUICommandInfo > LockContent;

	/* Enable a flashlight around the mouse cursor to brighten up faded out nodes */
	TSharedPtr< FUICommandInfo > EnableFlashlight;

	/* select all nodes */
	TSharedPtr<FUICommandInfo> SelectAllNodes;

	/* select the input nodes of the current selection */
	TSharedPtr<FUICommandInfo> SelectInputNodes;

	/* select the output nodes of the current selection */
	TSharedPtr<FUICommandInfo> SelectOutputNodes;

	/* select the complete node island */
	TSharedPtr<FUICommandInfo> SelectNodeIsland;

	/* remove all nodes */
	TSharedPtr<FUICommandInfo> RemoveAllNodes;

	/* remove selected nodes */
	TSharedPtr<FUICommandInfo> RemoveSelected;

	/* isolate selected nodes */
	TSharedPtr<FUICommandInfo> IsolateSelected;

	/* remove unrealted nodes */
	TSharedPtr<FUICommandInfo> RemoveUnrelated;

	/* starts a layout simulation to relax the nodes */
	TSharedPtr<FUICommandInfo> RunLayoutSimulation;

	/* cancel an ongoing layout simulation */
	TSharedPtr<FUICommandInfo> CancelLayoutSimulation;

	/* search nodes by name */
	TSharedPtr<FUICommandInfo> FindNodesByName;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
