// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/RigDependencyGraphNode.h"

#include "ControlRig.h"
#include "ControlRigEditorStyle.h"
#include "MaterialPropertyHelpers.h"
#include "EdGraph/EdGraphPin.h"
#include "RigDependencyGraph/RigDependencyGraph.h"
#include "Templates/Casts.h"
#include "UObject/UnrealNames.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDependencyGraphNode)

class UObject;

URigDependencyGraphNode::URigDependencyGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Index = INDEX_NONE;
	InputPin = nullptr;
	OutputPin = nullptr;
	NodeBodyColor = FLinearColor::White;

	LayoutPosition = FVector2D::ZeroVector;
	LayoutVelocity = FVector2D::ZeroVector;
	LayoutForce = FVector2D::ZeroVector;
	Dimensions = FVector2D::ZeroVector;
}

void URigDependencyGraphNode::SetupRigDependencyNode(const FNodeId& InNodeId)
{
	NodeId = InNodeId;

	const URigHierarchy* Hierarchy = GetRigHierarchy();
	if (Hierarchy == nullptr)
	{
		return;
	}

	switch (NodeId.Type)
	{
		case FNodeId::EType_RigElement:
		{
			const FRigElementKey ElementKey = Hierarchy->GetKey(NodeId.Index);
			NodeTitle = FText::FromName(ElementKey.Name);
			NodeTooltip = FText::FromString(ElementKey.ToString());

			switch (ElementKey.Type)
			{
				case ERigElementType::Bone:
				{
					NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Bone"));
					break;
				}
				case ERigElementType::Control:
				{
					if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(ElementKey))
					{
						if (ControlElement->IsAnimationChannel())
						{
							NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.AnimationChannel"));
							break;
						}
					}
					NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Control"));
					break;
				}
				case ERigElementType::Null:
				{
					NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Null"));
					break;
				}
				case ERigElementType::Socket:
				{
					NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Socket"));
					break;
				}
				case ERigElementType::Connector:
				{
					NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Connector"));
					break;
				}
				default:
				{
					break;
				}
			}
			break;
		}
		case FNodeId::EType_Metadata:
		{
			const FRigElementKey ElementKey = Hierarchy->GetKey(NodeId.Index);
			NodeTitle = FText::FromString(
				FString::Printf(TEXT("%s\n%s"),
					*NodeId.Name.ToString(),
					*ElementKey.ToString()
				));
			NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Metadata"));
			NodeTooltip = NodeTitle;
			break;
		}
		case FNodeId::EType_Variable:
		{
			if (const FRigVMExternalVariableDef* Variable = GetExternalVariable())
			{
				NodeTitle = FText::FromString(Variable->GetName().ToString());
			}
			NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Variable"));
			NodeTooltip = NodeTitle;
			break;
		}
		case FNodeId::EType_Instruction:
		{
			if (const URigVMNode* Node = GetRigVMNodeForInstruction())
			{
				NodeTitle = FText::FromString(Node->GetNodeTitle());

				const URigVMFunctionLibrary* FunctionLibrary = Node->GetTypedOuter<URigVMFunctionLibrary>();
				if (FunctionLibrary)
				{
					FText FunctionName;
					const UObject* Parent = Node->GetOuter();
					while (Parent && Parent != FunctionLibrary)
					{
						const UObject* GrandParent = Parent->GetOuter();
						if (GrandParent == FunctionLibrary)
						{
							if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Parent))
							{
								FunctionName = FText::FromName(LibraryNode->GetFunctionIdentifier().GetFunctionFName());
							}
							break;
						}
						Parent = GrandParent;
					}

					FText Asset, AssetSuffix;
					if (FunctionLibrary->GetOutermost() != GetRigVMClient()->GetOuter()->GetOutermost())
					{
						FString AssetName = FunctionLibrary->GetOutermost()->GetName();
						AssetName.RightChopInline(1 + AssetName.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromEnd));
						Asset =  FText::FromString(AssetName);
						AssetSuffix = NSLOCTEXT("RigDependencyGraphNode", "AssetSuffix", " - "); 
					}
					
					const FText NodeTitleFormat = NSLOCTEXT("RigDependencyGraphNode", "NodeTitleFormat", "{0}\n{1}{2}{3}");
					NodeTitle = FText::Format(NodeTitleFormat, NodeTitle, Asset, AssetSuffix, FunctionName);
					NodeTooltip = Node->GetToolTipText();
				}
				
				NodeBodyColor = FControlRigEditorStyle::Get().GetColor(TEXT("ControlRig.DependencyGraph.Colors.Instruction"));
			}
			break;
		}
		default:
		{
			NodeTitle = NSLOCTEXT("RigDependencyGraphNode", "Node", "Node");
			NodeTooltip = NodeTitle;
		}
	}

	NodeBodyColor *= FLinearColor(0.2f, 0.2f, 0.2f, 1.f);
}

UObject* URigDependencyGraphNode::GetDetailsObject()
{
	return nullptr;
}

URigDependencyGraph* URigDependencyGraphNode::GetRigDependencyGraph() const
{
	return CastChecked<URigDependencyGraph>(GetOuter());
}

FText URigDependencyGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NodeTitle;
}

FLinearColor URigDependencyGraphNode::GetNodeBodyTintColor() const
{
	return NodeBodyColor;
}

void URigDependencyGraphNode::AllocateDefaultPins()
{
	InputPin = CreatePin(EEdGraphPinDirection::EGPD_Input, NAME_None, NAME_None, NAME_None);
	//InputPin->bHidden = true;
	OutputPin = CreatePin(EEdGraphPinDirection::EGPD_Output, NAME_None, NAME_None, NAME_None);
	//OutputPin->bHidden = true;
}

FSlateIcon URigDependencyGraphNode::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = FLinearColor::White;

	switch (NodeId.Type)
	{
		case FNodeId::EType_RigElement:
		{
			static const FSlateIcon ControlIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.Control"));
			static const FSlateIcon NullIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.Null"));
			static const FSlateIcon BoneIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.BoneImported"));
			static const FSlateIcon SocketIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.Tree.Socket_Open"));
			static const FSlateIcon ConnectorIcon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.ConnectorPrimary"));
			static const FSlateIcon AnimationChannelIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Kismet.VariableList.TypeIcon"));

			if (const URigHierarchy* Hierarchy = GetRigHierarchy())
			{
				const FRigElementKey ElementKey = Hierarchy->GetKey(NodeId.Index);

				switch (ElementKey.Type)
				{
					case ERigElementType::Bone:
					{
						return BoneIcon;
					}
					case ERigElementType::Control:
					{
						if (const UControlRig* ControlRig = GetControlRig())
						{
							if (const FRigControlElement* Control = ControlRig->FindControl(ElementKey.Name))
							{
								if (Control->IsAnimationChannel())
								{
									return AnimationChannelIcon;
								}
							}
						}
						return ControlIcon;
					}
					case ERigElementType::Socket:
					{
						return SocketIcon;
					}
					case ERigElementType::Connector:
					{
						return ConnectorIcon;
					}
					default:
					{
						return NullIcon;
					}
				}
			}
			break;
		}
		case FNodeId::EType_Metadata:
		case FNodeId::EType_Variable:
		{
			static FSlateIcon TypeIcon(FAppStyle::GetAppStyleSetName(), "Kismet.VariableList.TypeIcon");
			return TypeIcon;
		}
		case FNodeId::EType_Instruction:
		{
			static const FSlateIcon NodeIcon(TEXT("RigVMEditorStyle"), TEXT("RigVM.Unit"));
			return NodeIcon;
		}
		default:
		{
			break;
		}
	}
	
	return Super::GetIconAndTint(OutColor);
}

UEdGraphPin& URigDependencyGraphNode::GetInputPin() const
{
	return *InputPin;
}

UEdGraphPin& URigDependencyGraphNode::GetOutputPin() const
{
	return *OutputPin;
}

void URigDependencyGraphNode::SetDimensions(const FVector2D& InDimensions)
{
	Dimensions = InDimensions;
}

int32 URigDependencyGraphNode::GetDependencyDepth() const
{
	if (DependencyDepth.IsSet())
	{
		return DependencyDepth.GetValue();
	}
	
	TArray<bool> Visited;
	return GetDependencyDepth_Impl(Visited);
}

UControlRig* URigDependencyGraphNode::GetControlRig() const
{
	if (const URigDependencyGraph* Graph = GetRigDependencyGraph())
	{
		return Graph->GetControlRig();
	}
	return nullptr;
}

URigHierarchy* URigDependencyGraphNode::GetRigHierarchy() const
{
	if (const URigDependencyGraph* Graph = GetRigDependencyGraph())
	{
		return Graph->GetRigHierarchy();
	}
	return nullptr;
}

const FRigVMClient* URigDependencyGraphNode::GetRigVMClient() const
{
	if (const URigDependencyGraph* Graph = GetRigDependencyGraph())
	{
		return Graph->GetRigVMClient();
	}
	return nullptr;
}

FRigElementKey URigDependencyGraphNode::GetRigElementKey() const
{
	if (NodeId.IsElement() || NodeId.IsMetadata())
	{
		if (const URigHierarchy* Hierarchy = GetRigHierarchy())
		{
			return Hierarchy->GetKey(NodeId.Index);
		}
	}
	return FRigElementKey();
}

const FRigBaseElement* URigDependencyGraphNode::GetRigElement() const
{
	if (NodeId.IsElement() || NodeId.IsMetadata())
	{
		if (const URigHierarchy* Hierarchy = GetRigHierarchy())
		{
			return Hierarchy->Get(NodeId.Index);
		}
	}
	return nullptr;
}

const URigVMNode* URigDependencyGraphNode::GetRigVMNodeForInstruction() const
{
	if (CachedRigVMNode.IsValid())
	{
		return CachedRigVMNode.Get();
	}

	const URigVMNode* SubjectNode = GetRigDependencyGraph()->GetRigVMNodeForInstruction(NodeId);
	if (SubjectNode)
	{
		CachedRigVMNode = SubjectNode;
	}
	return SubjectNode;
}

const FRigVMExternalVariableDef* URigDependencyGraphNode::GetExternalVariable() const
{
	if (!CachedRigVMExternalVariableDef.IsValid())
	{
		if (const FRigVMExternalVariableDef* ExternalVariable = GetRigDependencyGraph()->GetExternalVariable(NodeId))
		{
			CachedRigVMExternalVariableDef = *ExternalVariable;
		}
	}
	return &CachedRigVMExternalVariableDef;
}

bool URigDependencyGraphNode::IsFadedOut() const
{
	return GetFadedOutState() < 1.f - SMALL_NUMBER;
}

float URigDependencyGraphNode::GetFadedOutState() const
{
	if (FadedOutOverride.IsSet())
	{
		return FadedOutOverride.GetValue();
	}
	return bIsFadedOut.Get(false) ? 0.25f : 1.f;
}

void URigDependencyGraphNode::OverrideFadeOutState(TOptional<float> InFadedOutState)
{
	FadedOutOverride = InFadedOutState;
}

const FGuid& URigDependencyGraphNode::GetIslandGuid() const
{
	if (IslandGuid.IsSet())
	{
		return IslandGuid.GetValue();
	}

	// make as invalid
	IslandGuid = FGuid();

	for (int32 Phase = 0; Phase < 2; Phase++)
	{
		for (const UEdGraphPin* LinkedPin : (Phase == 0 ? InputPin : OutputPin)->LinkedTo)
		{
			if (const URigDependencyGraphNode* LinkedNode = Cast<URigDependencyGraphNode>(LinkedPin->GetOwningNode()))
			{
				const FGuid& LinkedGuid = LinkedNode->GetIslandGuid();
				if (LinkedGuid.IsValid())
				{
					IslandGuid = LinkedGuid;
					return LinkedGuid;
				}
			}
		}
	}

	IslandGuid = FGuid::NewGuid();
	return IslandGuid.GetValue();
}

void URigDependencyGraphNode::InvalidateCache()
{
	CachedRigVMExternalVariableDef = FRigVMExternalVariableDef();
	CachedRigVMNode.Reset();
}

int32 URigDependencyGraphNode::GetDependencyDepth_Impl(TArray<bool>& InOutVisited) const
{
	URigDependencyGraph* DependencyGraph = GetRigDependencyGraph();
	if (DependencyGraph == nullptr)
	{
		return INDEX_NONE;
	}

	if (InOutVisited.IsEmpty())
	{
		InOutVisited.AddZeroed(DependencyGraph->GetNodes().Num());
	}

	if (InOutVisited[Index])
	{
		if (!DependencyDepth.IsSet())
		{
			DependencyDepth = 0;
		}
		return DependencyDepth.GetValue();
	}

	InOutVisited[Index] = true;

	if (InputPin->LinkedTo.IsEmpty())
	{
		DependencyDepth = 0;
		return DependencyDepth.GetValue();
	}

	for (const UEdGraphPin* SourcePin : InputPin->LinkedTo)
	{
		const URigDependencyGraphNode* SourceNode = CastChecked<URigDependencyGraphNode>(SourcePin->GetOwningNode());
		DependencyDepth = FMath::Max<int32>(SourceNode->GetDependencyDepth_Impl(InOutVisited) + 1, DependencyDepth.Get(INDEX_NONE));
	}

	check(DependencyDepth.IsSet());
	return DependencyDepth.GetValue();
}
