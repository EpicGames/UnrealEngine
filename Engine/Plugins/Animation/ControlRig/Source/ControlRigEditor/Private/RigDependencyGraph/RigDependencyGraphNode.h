// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Rigs/RigHierarchyElements.h"
#include "Rigs/RigDependencyRecords.h"
#include "RigVMCore/RigVMExternalVariable.h"

#include "RigDependencyGraphNode.generated.h"

class UEdGraphPin;
class UObject;
class UControlRig;
class URigVMNode;
struct FRigVMClient;

UCLASS()
class URigDependencyGraphNode : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	using FNodeId = FRigHierarchyRecord;

	virtual void SetupRigDependencyNode(const FNodeId& InNodeId);
	virtual UObject* GetDetailsObject();

	class URigDependencyGraph* GetRigDependencyGraph() const;

	// UEdGraphNode implementation
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeBodyTintColor() const override;
	virtual FText GetTooltipText() const override { return NodeTooltip; }
	virtual void AllocateDefaultPins() override;
	virtual bool ShowPaletteIconOnNode() const override { return true; }
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const;

	int32 GetIndex() const { return Index; }
	const FNodeId& GetNodeId() const { return NodeId; }
	
	UEdGraphPin& GetInputPin() const;

	UEdGraphPin& GetOutputPin() const;

	void SetDimensions(const FVector2D& InDimensions);
	const FVector2D& GetDimensions() const { return Dimensions; }

	int32 GetDependencyDepth() const;

	/** Get the control rig we are displaying */
	UControlRig* GetControlRig() const;

	/** Get the hierarchy we are displaying */
	URigHierarchy* GetRigHierarchy() const;

	/** Get the client to resolve a node for */
	const FRigVMClient* GetRigVMClient() const;

	FRigElementKey GetRigElementKey() const;
	const FRigBaseElement* GetRigElement() const;
	const URigVMNode* GetRigVMNodeForInstruction() const;
	const FRigVMExternalVariableDef* GetExternalVariable() const;

	bool IsFadedOut() const;
	float GetFadedOutState() const;
	void OverrideFadeOutState(TOptional<float> InFadedOutState);
	void ResetFadedOutState() { OverrideFadeOutState({}); }

	const FGuid& GetIslandGuid() const;

	void InvalidateCache();

protected:

	int32 GetDependencyDepth_Impl(TArray<bool>& InOutVisited) const;

	int32 Index;
	
	FNodeId NodeId;
	
	/** Cached title for the node */
	FText NodeTitle;
	FText NodeTooltip;
	FLinearColor NodeBodyColor;

	/** Our one input pin */
	UEdGraphPin* InputPin;

	/** Our one output pin */
	UEdGraphPin* OutputPin;

	/** Cached dimensions of this node (used for layout) */
	TOptional<bool> FollowLayout;
	FVector2D LayoutPosition;
	FVector2D LayoutVelocity;
	FVector2D LayoutForce;
	FVector2D Dimensions;

	mutable TOptional<int32> DependencyDepth;
	mutable TOptional<FGuid> IslandGuid;
	mutable TOptional<bool> bIsFadedOut;
	mutable TOptional<float> FadedOutOverride;

	mutable TWeakObjectPtr<const URigVMNode> CachedRigVMNode;
	mutable FRigVMExternalVariableDef CachedRigVMExternalVariableDef;

	friend class URigDependencyGraph;
	friend class URigDependencyGraphSchema;
};


