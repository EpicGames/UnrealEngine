// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/RigDependencyGraphPanelNodeFactory.h"

#include "EdGraph/EdGraphNode.h"
#include "RigDependencyGraph/RigDependencyGraph.h"
#include "RigDependencyGraph/RigDependencyGraphNode.h"
#include "RigDependencyGraph/SRigDependencyGraphNode.h"
#include "SGraphNode.h"
#include "Templates/Casts.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

TSharedPtr<SGraphNode> FRigDependencyGraphPanelNodeFactory::CreateNode(UEdGraphNode* Node) const
{
	if (URigDependencyGraphNode* RigDependencyGraphNode = Cast<URigDependencyGraphNode>(Node))
	{
		TSharedRef<SGraphNode> GraphNode = SNew(SRigDependencyGraphNode, RigDependencyGraphNode);
		GraphNode->SlatePrepass();
		RigDependencyGraphNode->SetDimensions(GraphNode->GetDesiredSize());
		return GraphNode;
	}
	return nullptr;
}
