// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "Templates/SharedPointer.h"

class UEdGraphNode;

class FRigDependencyGraphPanelNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<class SGraphNode> CreateNode(UEdGraphNode* Node) const override;
};
