// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/RigDependencyGraphSchema.h"

#include "RigDependencyGraphNode.h"
#include "RigDependencyGraph/RigDependencyGraph.h"
#include "RigDependencyGraph/RigDependencyGraphNode.h"
#include "RigDependencyGraph/RigDependencyConnectionDrawingPolicy.h"
#include "ToolMenus.h"
#include "Toolkits/GlobalEditorCommonCommands.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDependencyGraphSchema)

#define LOCTEXT_NAMESPACE "RigDependencyGraphSchema"

URigDependencyGraphSchema::URigDependencyGraphSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FLinearColor URigDependencyGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return FLinearColor::White;
}

void URigDependencyGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	// Don't allow breaking any links
}

void URigDependencyGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	// Don't allow breaking any links
}

FPinConnectionResponse URigDependencyGraphSchema::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
	// Don't allow moving any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

FPinConnectionResponse URigDependencyGraphSchema::CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy) const
{
	// Don't allow copying any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT(""));
}

FConnectionDrawingPolicy* URigDependencyGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FRigDependencyConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}

URigDependencyGraphNode* URigDependencyGraphSchema::CreateGraphNode(URigDependencyGraph* InGraph, const FNodeId& InNodeId) const
{
	static constexpr bool bSelectNewNode = false;
	
	FGraphNodeCreator<URigDependencyGraphNode> GraphNodeCreator(*InGraph);
	URigDependencyGraphNode* RigDependencyGraphNode = GraphNodeCreator.CreateNode(bSelectNewNode);
	GraphNodeCreator.Finalize();

	RigDependencyGraphNode->SetupRigDependencyNode(InNodeId);

	RigDependencyGraphNode->Index = InGraph->DependencyGraphNodes.Add(RigDependencyGraphNode);
	InGraph->NodeIdLookup.Add(InNodeId, RigDependencyGraphNode);

	return RigDependencyGraphNode;
}

float URigDependencyGraphSchema_Random_Helper(int32& Seed)
{
	Seed = (Seed * 196314165) + 907633515;
	union { float f; int32 i; } Result;
	union { float f; int32 i; } Temp;
	const float SRandTemp = 1.0f;
	Temp.f = SRandTemp;
	Result.i = (Temp.i & 0xff800000) | (Seed & 0x007fffff);
	return FPlatformMath::Fractional(Result.f);
}

void URigDependencyGraphSchema::LayoutNodes(URigDependencyGraph* InGraph, int32 InNumIterations) const
{
	const TArray<URigDependencyGraphNode*>& Nodes = InGraph->GetNodes();
	const int32 NumNodes = Nodes.Num();
	if (NumNodes == 0)
	{
		InGraph->CancelLayout();
		return;
	}
	
	const bool bFirstTime = InGraph->LayoutIterationsTotal == InGraph->LayoutIterationsLeft;
	bool bRandomize = bFirstTime;

	if (InGraph->LayoutIterationsLeft <= 0)
	{
		return;
	}

	static constexpr int32 NodeMarginX = 150;
	static constexpr int32 NodeMarginY = 90;
	static constexpr float DefaultDisplacement = (NodeMarginX + NodeMarginY) * 0.5f;

	if (bFirstTime)
	{
		for (const URigDependencyGraphNode* Node : Nodes)
		{
			// if we found a node which has no dimensions
			// we haven't updated the panel yet - so we need to delay this.
			if (Node->Dimensions.IsNearlyZero())
			{
				return;
			}
		}

		int32 NumNodesRequiringLayout = 0;
		
		// compute the dependency depths
		for (const URigDependencyGraphNode* Node : Nodes)
		{
			Node->DependencyDepth.Reset();
			if (Node->FollowLayout.Get(true))
			{
				NumNodesRequiringLayout++;
			}
		}

		// if all nodes need to be places - let's do an initial placement via
		// a grid.
		if (NumNodesRequiringLayout == NumNodes)
		{
			if (InGraph->bIsPerformingGridLayout)
			{
				TArray<bool> VisitedFlag;
				int32 MaxDepth = 0;
				for (const URigDependencyGraphNode* Node : Nodes)
				{
					MaxDepth = FMath::Max<int32>(MaxDepth, Node->GetDependencyDepth_Impl(VisitedFlag));
				}

				TArray<TArray<URigDependencyGraphNode*>> NodesSortedByDepth;
				NodesSortedByDepth.SetNum(MaxDepth + 1);

				for (URigDependencyGraphNode* Node : Nodes)
				{
					NodesSortedByDepth[Node->GetDependencyDepth()].Add(Node);
				}

				int32 CurrentColumnX = 0;
				for (const TArray<URigDependencyGraphNode*>& Column : NodesSortedByDepth)
				{
					// Lay out root nodes
					float MaxWidth = 0.0f;
					int32 CurrentYOffset = 0;
					int32 TotalRootY = 0;
					for (const URigDependencyGraphNode* Node : Column)
					{
						TotalRootY += (int32)Node->GetDimensions().Y + NodeMarginY;
					}

					for (URigDependencyGraphNode* Node : Column)
					{
						Node->NodePosX = CurrentColumnX;
						Node->NodePosY = CurrentYOffset - (TotalRootY / 2);

						CurrentYOffset += (int32)Node->GetDimensions().Y + NodeMarginY;
						MaxWidth = FMath::Max(MaxWidth, Node->GetDimensions().X);
					}

					CurrentColumnX += (MaxWidth + NodeMarginX);
				}
			}
			else
			{
				bRandomize = false;
			}
		}
		else
		{
			bRandomize = false;
			
			// for each node place it either to the left or right of a neighbor node which may already have a position
			for (URigDependencyGraphNode* Node : Nodes)
			{
				if (Node->FollowLayout.Get(true))
				{
					const bool bUseInputPin = !Node->InputPin->LinkedTo.IsEmpty();

					auto GetPin = [bUseInputPin](const URigDependencyGraphNode* InNode)
					{
						return bUseInputPin ? InNode->InputPin : InNode->OutputPin;
					};
					auto GetOtherPin = [bUseInputPin](const URigDependencyGraphNode* InNode)
					{
						return bUseInputPin ? InNode->OutputPin : InNode->InputPin;
					};

					if (GetPin(Node)->LinkedTo.IsEmpty())
					{
						continue;
					}
					
					const URigDependencyGraphNode* Neighbor = CastChecked<URigDependencyGraphNode>(GetPin(Node)->LinkedTo[0]->GetOwningNode());
					TArray<const URigDependencyGraphNode*> AllNodesForNeighor;
					for (const UEdGraphPin* LinkedPin : GetOtherPin(Neighbor)->LinkedTo)
					{
						check(!LinkedPin->LinkedTo.IsEmpty());
						if (LinkedPin->LinkedTo[0] != GetOtherPin(Neighbor))
						{
							continue;
						}
						const URigDependencyGraphNode* NeighborsNeighbor = CastChecked<URigDependencyGraphNode>(LinkedPin->GetOwningNode());
						AllNodesForNeighor.Add(NeighborsNeighbor);
					}

					float MaxWidth = 0.f;
					float TotalHeight = NodeMarginY * static_cast<float>(AllNodesForNeighor.Num() - 1);
					
					for (const URigDependencyGraphNode* NeighborsNeighbor : AllNodesForNeighor)
					{
						TotalHeight += NeighborsNeighbor->Dimensions.Y;
						MaxWidth = FMath::Max(MaxWidth, NeighborsNeighbor->Dimensions.X);
					}

					Node->NodePosX = Neighbor->NodePosX;
					if (bUseInputPin)
					{
						Node->NodePosX += Neighbor->Dimensions.X + NodeMarginX;
					}
					else
					{
						Node->NodePosX -= NodeMarginX + MaxWidth;
					}
					
					Node->NodePosY = Neighbor->NodePosY - TotalHeight * 0.5f;

					for (const URigDependencyGraphNode* NeighborsNeighbor : AllNodesForNeighor)
					{
						if (NeighborsNeighbor == Node)
						{
							break;
						}
						Node->NodePosY += NodeMarginY + NeighborsNeighbor->Dimensions.Y;
					}
				}
			}
		}
	}

	if (bFirstTime)
	{
		InGraph->LayoutEdges.Reset();

		// update the island bounds
		for (FNodeIsland& Island : InGraph->NodeIslands)
		{
			Island.Bounds = InGraph->GetNodesBounds(Island.NodeIds);
		}

		// compute a deterministic seed
		int32 Seed = 0;
		TArray<FNodeId> NodeIds;
		NodeIds.Reserve(NumNodes);
		for (int32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
		{
			NodeIds.Add(Nodes[NodeIndex]->GetNodeId());
		}
		NodeIds.Sort();
		for (const FNodeId& NodeId : NodeIds)
		{
			Seed += static_cast<int32>(GetTypeHash(NodeId));
		}

		// pseudo-randomize the initial positions
		for (int32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
		{
			URigDependencyGraphNode* Node = Nodes[NodeIndex];
			Node->LayoutPosition = FVector2D(Node->GetPosition()) + Node->Dimensions * 0.5f;

			// always consume the random number to ensure the distribution is deterministic
			const FVector2D RandomDisplacement(
				URigDependencyGraphSchema_Random_Helper(Seed) * 2.f - 1.f,
				URigDependencyGraphSchema_Random_Helper(Seed) * 2.f - 1.f);
			
			if (bRandomize && Node->FollowLayout.Get(true))
			{
				Node->LayoutPosition.X += RandomDisplacement.X * DefaultDisplacement * 0.01f;
				Node->LayoutPosition.Y += RandomDisplacement.Y * DefaultDisplacement;
			}
			
			Node->LayoutVelocity = FVector2D::ZeroVector;
			Node->LayoutForce = FVector2D::ZeroVector;

			for (int32 LinkedToIndex = 0; LinkedToIndex < Nodes[NodeIndex]->OutputPin->LinkedTo.Num(); LinkedToIndex++)
			{
				UEdGraphPin* LinkedPin = Node->OutputPin->LinkedTo[LinkedToIndex];
				if (const URigDependencyGraphNode* LinkedNode = Cast<URigDependencyGraphNode>(LinkedPin->GetOwningNode()))
				{
					if (ensure(LinkedNode->Index != NodeIndex))
					{
						const URigDependencyGraphNode* NodeA = Nodes[NodeIndex];
						const URigDependencyGraphNode* NodeB = LinkedNode;

						const float DesiredDistance = NodeMarginX + (NodeA->Dimensions.X + NodeB->Dimensions.X) * 0.5f;

						auto ComputeDesiredY = [](const int32 Index, const int32 Num, const float NodeHeight) -> float
						{
							if (Num <= 1)
							{
								return 0.f;
							}
							const float NumVertical = static_cast<float>(FMath::Max(Num - 1, 0));
							const float Ratio = static_cast<float>(Index) / NumVertical;
							const float HalfTotalHeight = NumVertical * (NodeHeight + NodeMarginY) * 0.5f;
							return FMath::Lerp(-HalfTotalHeight, HalfTotalHeight, Ratio);
						};

						const int32 IndexInStackA = NodeB->InputPin->LinkedTo.IndexOfByKey(NodeA->OutputPin);
						const int32 SizeOfStackA = NodeB->InputPin->LinkedTo.Num();
						const float DesiredHeightA = ComputeDesiredY(IndexInStackA, SizeOfStackA, NodeB->Dimensions.Y);
						const FVector2D AnchorA = FVector2D(-DesiredDistance, DesiredHeightA);

						const int32 IndexInStackB = NodeA->OutputPin->LinkedTo.IndexOfByKey(NodeB->InputPin);
						const int32 SizeOfStackB = NodeA->OutputPin->LinkedTo.Num();
						const float DesiredHeightB = ComputeDesiredY(IndexInStackB, SizeOfStackB, NodeA->Dimensions.Y);
						const FVector2D AnchorB = FVector2D(DesiredDistance, DesiredHeightB);

						float Strength = 1.f;
						if (NodeA->GetNodeId().IsInstruction() || NodeB->GetNodeId().IsInstruction())
						{
							Strength = 2.f;
						}

						const FLayoutEdge Edge(NodeIndex, LinkedNode->GetIndex(), AnchorA, AnchorB, Strength);
						InGraph->LayoutEdges.Add(Edge);
					}
				}
			}
		}
	}

	// now layout the graph using Fruchterman Reingold
	for (int32 Iteration = 0; Iteration < InNumIterations; ++Iteration, InGraph->LayoutIterationsLeft--)
	{
		const float Ratio = static_cast<float>(InGraph->LayoutIterationsLeft) / static_cast<float>(InGraph->LayoutIterationsTotal);
		const float ForceMagnitude = 0.15f * Ratio;

		// compute repulsive forces, pairwise between all verts
		int32 NumSimulatedNodes = 0;
		for (int32 NodeIndexA = 0; NodeIndexA < NumNodes; ++NodeIndexA)
		{
			if (Nodes[NodeIndexA]->FollowLayout.Get(true))
			{
				NumSimulatedNodes++;
			}
			
			const FVector2D PositionA = Nodes[NodeIndexA]->LayoutPosition;

			for (int32 NodeIndexB = NodeIndexA + 1; NodeIndexB < NumNodes; ++NodeIndexB)
			{
				const FVector2D PositionB = Nodes[NodeIndexB]->LayoutPosition;
				const FVector2D AtoB = PositionB - PositionA;
				const FVector2D AtoBNormal = AtoB.GetSafeNormal().GetAbs();

				const float DesiredDistanceX = (Nodes[NodeIndexA]->Dimensions.X + Nodes[NodeIndexB]->Dimensions.X) * AtoBNormal.X;
				const float DesiredDistanceY = (Nodes[NodeIndexA]->Dimensions.Y + Nodes[NodeIndexB]->Dimensions.Y) * AtoBNormal.Y;
				const float Margin = NodeMarginX * AtoBNormal.X + NodeMarginY * AtoBNormal.Y;
				const float DesiredDistance = 1.5f * Margin + 0.5f * (DesiredDistanceX + DesiredDistanceY);

				const float AtoBDistance = AtoB.Length();
				const float DeltaDistance = AtoBDistance - DesiredDistance;
				if (AtoBDistance > SMALL_NUMBER && DeltaDistance < -SMALL_NUMBER)
				{
					FVector2D Force = ForceMagnitude * DeltaDistance * AtoB / AtoBDistance;
					Nodes[NodeIndexA]->LayoutForce += Force;
					Nodes[NodeIndexB]->LayoutForce += -Force;
				}
			}
		}

		// compute attractive forces along edges
		for (const FLayoutEdge& Edge : InGraph->LayoutEdges)
		{
			URigDependencyGraphNode* NodeA = Nodes[Edge.NodeA];
			URigDependencyGraphNode* NodeB = Nodes[Edge.NodeB];

			const FVector2D A = NodeA->LayoutPosition;
			const FVector2D B = NodeB->LayoutPosition;
			const FVector2D DesiredA = B + Edge.AnchorA;
			const FVector2D DesiredB = A + Edge.AnchorB;
			const FVector2D DirectionA = DesiredA - A;
			const FVector2D DirectionB = DesiredB - B;
			NodeA->LayoutForce += Edge.Strength * ForceMagnitude * DirectionA;
			NodeB->LayoutForce += Edge.Strength * ForceMagnitude * DirectionB;
		}

		for (FNodeIsland& Island : InGraph->NodeIslands)
		{
			bool bHasSimulatedNodes = false;
			for (const FNodeId& NodeId : Island.NodeIds)
			{
				if (const URigDependencyGraphNode* Node = InGraph->FindNode(NodeId))
				{
					if (Node->FollowLayout.Get(true))
					{
						bHasSimulatedNodes = true;
						break;
					}
				}
			}

			if (bHasSimulatedNodes)
			{
				const FBox2D CurrentBounds = InGraph->GetNodesBounds(Island.NodeIds);
				const FVector2D MaintainCenterForce = Island.Bounds.GetCenter() - CurrentBounds.GetCenter();
				if (!MaintainCenterForce.IsNearlyZero())
				{
					for (const FNodeId& NodeId : Island.NodeIds)
					{
						if (URigDependencyGraphNode* Node = InGraph->FindNode(NodeId))
						{
							if (Node->FollowLayout.Get(true))
							{
								Node->LayoutForce += MaintainCenterForce;
							}
						}
					}
				}
			}
		}

		// compute displacements
		// we treat the "forces" as displacements directly
		for (int32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
		{
			Nodes[NodeIndex]->LayoutVelocity = FMath::Lerp(Nodes[NodeIndex]->LayoutVelocity, Nodes[NodeIndex]->LayoutForce, 0.1);
			Nodes[NodeIndex]->LayoutForce = FVector2D::ZeroVector;

			if (Nodes[NodeIndex]->FollowLayout.Get(true))
			{
				Nodes[NodeIndex]->LayoutPosition += Nodes[NodeIndex]->LayoutVelocity;
				Nodes[NodeIndex]->NodePosX = (float)(Nodes[NodeIndex]->LayoutPosition.X - Nodes[NodeIndex]->Dimensions.X * 0.5);
				Nodes[NodeIndex]->NodePosY = (float)(Nodes[NodeIndex]->LayoutPosition.Y - Nodes[NodeIndex]->Dimensions.Y * 0.5);
			}

			Nodes[NodeIndex]->LayoutVelocity *= 0.9;

			// stop the simulation once the node has converged
			if (Nodes[NodeIndex]->LayoutVelocity.Length() < KINDA_SMALL_NUMBER)
			{
				Nodes[NodeIndex]->FollowLayout = false;
			}
		}

		// if no more nodes are simulating
		// the simulation has converged.
		if (NumSimulatedNodes == 0)
		{
			InGraph->CancelLayout();
			break;
		}
	}

	if (InGraph->LayoutIterationsLeft <= 0)
	{
		for (URigDependencyGraphNode* DependencyGraphNode : InGraph->DependencyGraphNodes)
		{
			DependencyGraphNode->FollowLayout = false;
		}
	}
}

const FPinConnectionResponse URigDependencyGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("TheDependencyGraphIsReadOnly", "The dependency graph is read only."));
}

FPinConnectionResponse URigDependencyGraphSchema::CanCreateNewNodes(UEdGraphPin* InSourcePin) const
{
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("TheDependencyGraphIsReadOnly", "The dependency graph is read only."));
}

bool URigDependencyGraphSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	OutErrorMessage = LOCTEXT("TheDependencyGraphIsReadOnly", "The dependency graph is read only.");
	return false;
}

void URigDependencyGraphSchema::SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const
{
	Super::SetNodePosition(Node, Position);

	if (URigDependencyGraphNode* DependencyGraphNode = Cast<URigDependencyGraphNode>(Node))
	{
		DependencyGraphNode->LayoutPosition.X = DependencyGraphNode->NodePosX;
		DependencyGraphNode->LayoutPosition.Y = DependencyGraphNode->NodePosY;
		DependencyGraphNode->FollowLayout = false;
	}
}

#undef LOCTEXT_NAMESPACE
