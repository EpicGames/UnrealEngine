// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDependencyGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "Rigs/RigHierarchy.h"
#include "RigDependencyGraphNode.h"
#include "RigDependencyGraphSchema.generated.h"

class URigDependencyGraph;
class URigDependencyGraphNode;

UCLASS()
class URigDependencyGraphSchema : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphSchema interface
	virtual FName GetParentContextMenuName() const override { return NAME_None; }
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	virtual FPinConnectionResponse MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove = false, bool bNotifyLinkedNodes = false) const override;
	virtual FPinConnectionResponse CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy = false) const override;
	virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	virtual FPinConnectionResponse CanCreateNewNodes(UEdGraphPin* InSourcePin) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	virtual void SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const override;
	// End of UEdGraphSchema interface

	virtual URigDependencyGraphNode* CreateGraphNode(URigDependencyGraph* InGraph, const URigDependencyGraphNode::FNodeId& InNodeId) const;

	virtual void LayoutNodes(URigDependencyGraph* InGraph, int32 InIterations) const;

	using FNodeId = URigDependencyGraphNode::FNodeId;
	using FLayoutEdge = URigDependencyGraph::FLayoutEdge;
	using FNodeIsland = URigDependencyGraph::FNodeIsland;
};

