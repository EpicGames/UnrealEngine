// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/RigDependencyGraphSummoner.h"
#include "IDocumentation.h"
#include "Editor/ControlRigEditor.h"
#include "RigDependencyGraph/SRigDependencyGraph.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "RigDependencyGraphSummoner"

FRigDependencyGraphSummoner::FRigDependencyGraphSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
	: FWorkflowTabFactory("RigDependencyGraphView", InControlRigEditor->GetHostingApp())
	, ControlRigEditor(InControlRigEditor)
{
	TabLabel = LOCTEXT("RigDependencyViewer", "Dependency Viewer");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");

	EnableTabPadding();
	bIsSingleton = false;

	ViewMenuDescription = TabLabel;
	ViewMenuTooltip = LOCTEXT("RigDependencyGraphView_ToolTip", "Shows the Control Rig Dependency Viewer");
}

TSharedPtr<SToolTip> FRigDependencyGraphSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("RigDependencyGraphToolTip", "The Control Rig Dependency Viewer tab lets you debug the relationships caused by your Control Rig."), NULL, TEXT("Shared/Editors/ControlRigEditor"), TEXT("RigDependencyGraph_Window"));
}

TSharedRef<SDockTab> FRigDependencyGraphSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> Tab = FWorkflowTabFactory::SpawnTab(Info);
	Tab->SetTabLabelOverflowPolicy(TOptional<ETextOverflowPolicy>());
	return Tab;
}

TSharedRef<SWidget> FRigDependencyGraphSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	check(ControlRigEditor.IsValid());
	TSharedRef<SRigDependencyGraph> RigDependencyGraph = SNew(SRigDependencyGraph, ControlRigEditor.Pin().ToSharedRef());
	return RigDependencyGraph;
}

#undef LOCTEXT_NAMESPACE
