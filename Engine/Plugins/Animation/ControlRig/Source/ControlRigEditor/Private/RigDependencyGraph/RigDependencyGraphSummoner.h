// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "RigDependencyGraph/SRigDependencyGraph.h"

struct FRigDependencyGraphSummoner : public FWorkflowTabFactory
{
public:
	FRigDependencyGraphSummoner(const TSharedRef<IControlRigBaseEditor>& InControlRigEditor);

	/** FWorkflowTabFactory interface */
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<IControlRigBaseEditor> ControlRigEditor;
};
