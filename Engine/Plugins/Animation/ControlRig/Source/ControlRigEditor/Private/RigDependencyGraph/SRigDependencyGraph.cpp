// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/SRigDependencyGraph.h"
#include "GraphEditor.h"
#include "Toolkits/GlobalEditorCommonCommands.h"

#include "ObjectTools.h"
#include "Engine/Selection.h"
#include "RigDependencyGraph/RigDependencyGraph.h"
#include "RigDependencyGraph/RigDependencyGraphSchema.h"
#include "RigDependencyGraph/RigDependencyGraphCommands.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Algo/Transform.h"
#include "Editor/ControlRigEditor.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Rigs/RigHierarchyController.h"
#include "Editor/RigVMEditorStyle.h"
#include "Settings/ControlRigSettings.h"

#define LOCTEXT_NAMESPACE "RigDependencyGraph"

SRigDependencyGraph::~SRigDependencyGraph()
{
	if (!GExitPurge)
	{
		if ( ensure(GraphObj) )
		{
			GraphObj->RemoveFromRoot();
		}		
	}
}

void SRigDependencyGraph::Construct(const FArguments& InArgs, const TSharedRef<IControlRigBaseEditor>& InControlRigEditor)
{
	WeakControlRigEditor = InControlRigEditor;
	FlashLightRadiusAttribute = InArgs._FlashLightRadius;

	// Create the graph
	GraphObj = NewObject<URigDependencyGraph>();
	GraphObj->Schema = URigDependencyGraphSchema::StaticClass();
	GraphObj->AddToRoot();
	GraphObj->Initialize(InControlRigEditor, InControlRigEditor->GetControlRig());

	BindCommands();

	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_DependencyGraph", "Debug");

	SGraphEditor::FGraphEditorEvents GraphEvents;
	GraphEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SRigDependencyGraph::OnCreateGraphActionMenu);
	GraphEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP(this, &SRigDependencyGraph::HandleSelectionChanged);
	GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SRigDependencyGraph::HandleNodeDoubleClicked);

	// Create the graph editor
	GraphEditor = SNew(SGraphEditor)
		.GraphToEdit(GraphObj)
		.GraphEvents(GraphEvents)
		.Appearance(AppearanceInfo)
		.OnTick(this, &SRigDependencyGraph::OnGraphEditorTick)
		.ShowGraphStateOverlay(false);

	static const FName DefaultForegroundName("DefaultForeground");

	ChildSlot
	[
		SNew(SVerticalBox)
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SComboButton)
				.Visibility(EVisibility::Visible)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
				.ForegroundColor(FSlateColor::UseStyle())
				.OnGetMenuContent(this, &SRigDependencyGraph::CreateTopLeftMenu)
				.ContentPadding(2)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Menu"))
					.DesiredSizeOverride(FVector2D(20, 20))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.Menu"))
				 ]
			]

			+SHorizontalBox::Slot()
			.MinWidth(100)
			.MaxWidth(300)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SAssignNew(FilterBox, SSearchBox)
				.OnTextChanged(this, &SRigDependencyGraph::OnSearchBarTextChanged)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this]() -> FReply
				{
					OnSearchMatchPicked(MatchIndex.Get(0) - 1);
					return FReply::Handled();
				})
				.Visibility_Lambda([this]() -> EVisibility
				{
					return NodesMatchingSearch.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
				})
				.IsEnabled_Lambda([this]() -> bool
				{
					return MatchIndex.Get(0) > 0;
				})
				.ToolTipText(LOCTEXT("SelectPreviousMatch", "Select the previous node matching the search"))
				.ContentPadding(2)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16, 16))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Animation.Backward_Step"))
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this]() -> FReply
				{
					OnSearchMatchPicked(MatchIndex.Get(0) + 1);
					return FReply::Handled();
				})
				.Visibility_Lambda([this]() -> EVisibility
				{
					return NodesMatchingSearch.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
				})
				.IsEnabled_Lambda([this]() -> bool
				{
					return MatchIndex.Get(0) < NodesMatchingSearch.Num() - 1;
				})
				.ToolTipText(LOCTEXT("SelectNextMatch", "Select the next node matching the search"))
				.ContentPadding(2)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16, 16))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Animation.Forward_Step"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 10.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("LockContentTooltip", "Lock the content of this view / stop syncing content based on selection"))
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->bLockContent = !GraphObj->bLockContent;
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (GraphObj->bLockContent)
						{
							return FStyleColors::AccentBlue;
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20, 20))
					.Image_Lambda([this]() -> const FSlateBrush*
					{
						static const FSlateBrush* Unlocked = FAppStyle::GetBrush("Icons.Unlock");
						static const FSlateBrush* Locked = FAppStyle::GetBrush("Icons.Lock");
						return GraphObj->bLockContent ? Locked : Unlocked;
					})
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("EnableFlashLight", "Enable a flashlight around the mouse cursor to brighten up faded out nodes"))
				.OnClicked_Lambda([this]() -> FReply
				{
					UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer = !UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer;
					UControlRigEditorSettings::Get()->SaveConfig();
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer)
						{
							return FStyleColors::AccentBlue;
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20,20))
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.Flashlight"))
				]
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("ShowParentChildRelationShips", "Show the parent-child relationships"))
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->SetFollowParentRelationShips(!GraphObj->bFollowParentRelationShips);
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (GraphObj->bFollowParentRelationShips)
						{
							return FStyleColors::AccentBlue;
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20,20))
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.ParentChild"))
				]
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("ShowVMRelationShips", "Show the relationships introduced by the rig nodes"))
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->SetFollowVMRelationShips(!GraphObj->bFollowVMRelationShips);
					return FReply::Handled();
				})
				.ContentPadding(2)
				[
					SNew(SImage)
					.ColorAndOpacity_Lambda([this]()
					{
						if (GraphObj->bFollowVMRelationShips)
						{
							return FStyleColors::AccentBlue;
						}
						return FSlateColor::UseForeground();
					})
					.DesiredSizeOverride(FVector2D(20,20))
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.VMRelationShips"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 10.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]
		
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->RemoveUnrelatedNodes();
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("HideUnrelatedTooltip", "Hides unrelated nodes from the view"))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(20, 20))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.HideUnrelated"))
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->RemoveUnselectedNodes();
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("IsolateSelectionTooltip", "Isolates the selection by removing unselected nodes"))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(20, 20))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.IsolateSelection"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 10.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
				.Visibility_Lambda([this]() -> EVisibility
				{
					return GraphObj->LayoutIterationsLeft > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 10.0f, 0.0f)
			[
				SNew(SSeparator)
				.Orientation(Orient_Vertical)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->RemoveAllNodes();
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("RemoveAllNodesTooltip", "Removes all of the nodes from the view"))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(20, 20))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.Cleanup"))
				]
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(2.f, 0.f, 4.f, 0.f)
			[
				SAssignNew(LayoutProgressBar, SProgressBar)
				.Visibility_Lambda([this]() -> EVisibility
				{
					return GraphObj->LayoutIterationsLeft > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Percent_Lambda([this]() -> TOptional<float>
				{
					if (GraphObj->LayoutIterationsLeft > 0)
					{
						const float Left = static_cast<float>(GraphObj->LayoutIterationsLeft);
						const float Total = static_cast<float>(GraphObj->LayoutIterationsTotal);
						if (Total > SMALL_NUMBER)
						{
							return (Total - Left) / Total;
						}
					}
					return 0.f;
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.Visibility_Lambda([this]() -> EVisibility
				{
					return GraphObj->LayoutIterationsLeft > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.OnClicked_Lambda([this]() -> FReply
				{
					GraphObj->CancelLayout();
					return FReply::Handled();
				})
				.ToolTipText(LOCTEXT("CancelLayoutTooltip", "Cancels the layout simulation"))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(20, 20))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.X"))
				]
			]
		]

		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			GraphEditor.ToSharedRef()
		]
	];

	GEditor->RegisterForUndo(this);

	GraphObj->OnClearSelection.BindSP(this, &SRigDependencyGraph::OnClearNodeSelection);
}

FActionMenuContent SRigDependencyGraph::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	/*
	if(InDraggedPins.Num() > 0 && InDraggedPins[0]->Direction == EGPD_Output)
	{
		if(URigDependencyGraphNode_Bone* BodyNode = Cast<URigDependencyGraphNode_Bone>(InDraggedPins[0]->GetOwningNode()))
		{
			FMenuBuilder MenuBuilder(true, nullptr);
			TSharedPtr<ISkeletonTree> SkeletonTree = GraphObj->GetRigDependencyEditor()->BuildMenuWidgetNewConstraintForBody(MenuBuilder, BodyNode->BodyIndex, InOnMenuClosed);
			return FActionMenuContent(MenuBuilder.MakeWidget(), SkeletonTree->GetSearchWidget());
		}
	}

	InOnMenuClosed.ExecuteIfBound();
	*/
	return FActionMenuContent();
}

void SRigDependencyGraph::HandleNodeDoubleClicked(UEdGraphNode* InNode)
{
	const URigDependencyGraphNode* RigDependencyGraphNode = Cast<URigDependencyGraphNode>(InNode);
	if (!RigDependencyGraphNode)
	{
		return;
	}

	bool bExpandInputs = true;
	bool bExpandOutputs = true;

	const FGeometry PaintGeometry = GraphEditor->GetPaintSpaceGeometry();
	FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();
	if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(GraphEditor->AsShared()))
	{
		MousePosition -= FVector2D(Window->GetPositionInScreen());
	}

	if(PaintGeometry.IsUnderLocation(MousePosition))
	{
		FVector2f GraphEditorLocation = FVector2f::ZeroVector;
		float GraphEditorZoomAmount = 0;
		GraphEditor->GetViewLocation(GraphEditorLocation, GraphEditorZoomAmount);

		const FVector2D WidgetPosition = FVector2D(PaintGeometry.AbsoluteToLocal(MousePosition) / GraphEditorZoomAmount + GraphEditorLocation);
		const FVector2D Position = FVector2D(RigDependencyGraphNode->GetPosition());
		const FVector2D Dimensions = RigDependencyGraphNode->GetDimensions();
		const double Left = Position.X;
		const double Right = Left + Dimensions.X;
		const double BorderLeft = Left + (Dimensions.X) * 0.3;
		const double BorderRight = Right - (Dimensions.X) * 0.3;
		const double Cursor = WidgetPosition.X;
		bExpandInputs = Cursor < BorderRight;
		bExpandOutputs = Cursor > BorderLeft;
	}

	const TArray<FNodeId> NodesToExpand = GraphObj->GetNodesToExpand({RigDependencyGraphNode->GetNodeId()}, bExpandInputs, bExpandOutputs);
	if (!NodesToExpand.IsEmpty())
	{
		TArray<FNodeId> AllNodeIds;
		for (const TObjectPtr<URigDependencyGraphNode>& ExistingDependencyGraphNode : GraphObj->DependencyGraphNodes)
		{
			AllNodeIds.Add(ExistingDependencyGraphNode->GetNodeId());
		}
		for (const FNodeId& NodeId : NodesToExpand)
		{
			AllNodeIds.AddUnique(NodeId);
		}
		GraphObj->ZoomAndFitDuringLayout = false;
		GraphObj->ConstructNodes(AllNodeIds, false);
		return;
	}
}

void SRigDependencyGraph::OnClearNodeSelection()
{
	if (GraphEditor)
	{
		GraphEditor->ClearSelectionSet();
	}
}

TSharedRef<SWidget> SRigDependencyGraph::CreateTopLeftMenu()
{
	const FRigDependencyGraphCommands& Actions = FRigDependencyGraphCommands::Get();

	const bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

	MenuBuilder.BeginSection("FilterOptions", LOCTEXT("OptionsMenuHeading", "Options"));
	{
		MenuBuilder.AddMenuEntry(Actions.LockContent);
		MenuBuilder.AddMenuEntry(Actions.EnableFlashlight);
		MenuBuilder.AddMenuEntry(Actions.ShowParentChildRelationships);
		MenuBuilder.AddMenuEntry(Actions.ShowControlSpaceRelationships);
		MenuBuilder.AddMenuEntry(Actions.ShowInstructionRelationships);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Navigation", LOCTEXT("NavigationMenuHeading", "Navigation"));
	{
		MenuBuilder.AddMenuEntry(Actions.FrameSelection);
		MenuBuilder.AddMenuEntry(Actions.JumpToSource);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Graph", LOCTEXT("GraphMenuHeading", "Graph"));
	{
		MenuBuilder.AddMenuEntry(Actions.SelectAllNodes);
		MenuBuilder.AddMenuEntry(Actions.SelectInputNodes);
		MenuBuilder.AddMenuEntry(Actions.SelectOutputNodes);
		MenuBuilder.AddMenuEntry(Actions.SelectNodeIsland);
		MenuBuilder.AddMenuEntry(Actions.RemoveAllNodes);
		MenuBuilder.AddMenuEntry(Actions.RemoveSelected);
		MenuBuilder.AddMenuEntry(Actions.IsolateSelected);
		MenuBuilder.AddMenuEntry(Actions.RemoveUnrelated);
		MenuBuilder.AddMenuEntry(Actions.RunLayoutSimulation);
		MenuBuilder.AddMenuEntry(Actions.CancelLayoutSimulation);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRigDependencyGraph::BindCommands()
{
	// create new command
	const FRigDependencyGraphCommands& Commands = FRigDependencyGraphCommands::Get();

	CommandList = MakeShared<FUICommandList>();
	
	CommandList->MapAction(Commands.FrameSelection,
		FExecuteAction::CreateLambda([this] ()
		{
			if (GraphEditor)
			{
				GraphEditor->ZoomToFit(true);
			}
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return GraphObj->GetSelectedNodeIds().Num() > 0;
		})		
	);

	CommandList->MapAction(Commands.JumpToSource,
		FExecuteAction::CreateLambda([this] ()
		{
			for (const FNodeId& SelectedNodeId : GraphObj->GetSelectedNodeIds())
			{
				if (SelectedNodeId.IsInstruction())
				{
					TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakControlRigEditor.Pin();
					if (ControlRigEditor.IsValid())
					{
						if (FControlRigAssetInterfacePtr ControlRigBlueprint = ControlRigEditor->GetControlRigAssetInterface())
						{
							if (FRigVMClient* Client = ControlRigBlueprint->GetRigVMClient())
							{
								if (URigVMController* Controller = Client->GetOrCreateController(Client->GetDefaultModel()))
								{
									if (const URigDependencyGraphNode* RigDependencyGraphNode = GraphObj->FindNode(SelectedNodeId))
									{
										if (const URigVMNode* Node = RigDependencyGraphNode->GetRigVMNodeForInstruction())
										{
											Controller->RequestJumpToHyperLink(Node);
											break;
										}
									}
								}
							}
						}
					}
				}
			}
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			if (GraphObj->GetSelectedNodeIds().Num() > 0)
			{
				if (GraphObj->GetSelectedNodeIds()[0].IsInstruction())
				{
					return true;
				}
			}
			return false;
		})
	);

	CommandList->MapAction(Commands.SelectAllNodes, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->SelectAllNodes();
	}));

	CommandList->MapAction(Commands.SelectInputNodes, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->SelectLinkedNodes(GraphObj->SelectedNodes, true, true, true);
	}));

	CommandList->MapAction(Commands.SelectOutputNodes, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->SelectLinkedNodes(GraphObj->SelectedNodes, false, true, true);
	}));

	CommandList->MapAction(Commands.SelectNodeIsland, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->SelectNodeIsland(GraphObj->SelectedNodes, true);
	}));

	CommandList->MapAction(Commands.RemoveAllNodes, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->RemoveAllNodes();
	}));

	CommandList->MapAction(Commands.RemoveSelected, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->RemoveSelectedNodes();
	}));

	CommandList->MapAction(Commands.IsolateSelected, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->RemoveUnselectedNodes();
	}));

	CommandList->MapAction(Commands.RemoveUnrelated, FExecuteAction::CreateLambda([this] ()
	{
		GraphObj->RemoveUnrelatedNodes();
	}));

	CommandList->MapAction(Commands.RunLayoutSimulation,
		FExecuteAction::CreateLambda([this] ()
		{
			TArray<FNodeId> SelectedNodes = GraphObj->GetSelectedNodeIds();
			if (SelectedNodes.IsEmpty())
			{
				SelectedNodes = GraphObj->GetAllNodeIds();
			}
			if (!SelectedNodes.IsEmpty())
			{
				GraphObj->RequestRefreshLayout(SelectedNodes);
			}
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return GraphObj->GetNodes().Num() > 0 && GraphObj->LayoutIterationsLeft <= 0;
		})
	);

	CommandList->MapAction(Commands.CancelLayoutSimulation,
		FExecuteAction::CreateLambda([this] ()
		{
			GraphObj->CancelLayout();
		}),
		FCanExecuteAction::CreateLambda([this] () -> bool
		{
			return GraphObj->LayoutIterationsLeft > 0;
		})
	);

	CommandList->MapAction(
		Commands.ShowParentChildRelationships,
		FExecuteAction::CreateLambda([this]()
		{
			GraphObj->bFollowParentRelationShips = !GraphObj->bFollowParentRelationShips;
			GraphObj->RebuildLinks();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return GraphObj->bFollowParentRelationShips;
		})
	);

	CommandList->MapAction(
		Commands.ShowControlSpaceRelationships,
		FExecuteAction::CreateLambda([this]()
		{
			GraphObj->bFollowControlSpaceRelationships = !GraphObj->bFollowControlSpaceRelationships;
			GraphObj->RebuildLinks();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return GraphObj->bFollowControlSpaceRelationships;
		})
	);

	CommandList->MapAction(
		Commands.ShowInstructionRelationships,
		FExecuteAction::CreateLambda([this]()
		{
			GraphObj->bFollowVMRelationShips = !GraphObj->bFollowVMRelationShips;
			GraphObj->RebuildLinks();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return GraphObj->bFollowVMRelationShips;
		})
	);

	CommandList->MapAction(
		Commands.EnableFlashlight,
		FExecuteAction::CreateLambda([this]()
		{
			UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer = !UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer;
			UControlRigEditorSettings::Get()->SaveConfig();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer;
		})
	);

	CommandList->MapAction(
		Commands.LockContent,
		FExecuteAction::CreateLambda([this]()
		{
			GraphObj->bLockContent = !GraphObj->bLockContent;
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return GraphObj->bLockContent;
		})
	);
	
	CommandList->MapAction(Commands.FindNodesByName,
	FExecuteAction::CreateLambda([this] ()
	{
		if (FilterBox.IsValid())
		{
			FSlateApplication::Get().ForEachUser([this](FSlateUser& User)
			{
				User.SetFocus(FilterBox.ToSharedRef());
			});
		}
	}));
}

void SRigDependencyGraph::OnSearchBarTextChanged(const FText& NewText)
{
	NodesMatchingSearch.Reset();
	MatchIndex.Reset();

	const FString SearchString = NewText.ToString();
	if (SearchString.Len() <= 1)
	{
		return;
	}
	
	for (const TObjectPtr<URigDependencyGraphNode>& Node : GraphObj->DependencyGraphNodes)
	{
		if (Node->GetNodeTitle(ENodeTitleType::FullTitle).ToString().Contains(SearchString, ESearchCase::IgnoreCase))
		{
			NodesMatchingSearch.Add(Node->GetNodeId());
		}
	}

	OnSearchMatchPicked(0);
}

void SRigDependencyGraph::OnSearchMatchPicked(int32 InIndex)
{
	if (NodesMatchingSearch.IsValidIndex(InIndex))
	{
		const FNodeId NodeToSelect = NodesMatchingSearch[InIndex];
		MatchIndex = InIndex;
		GraphObj->SelectNodes({NodeToSelect});
		if (GraphEditor.IsValid())
		{
			GraphEditor->JumpToNode(GraphObj->FindNode(NodeToSelect), false, false);
		}
	}
}

void SRigDependencyGraph::SelectNodes(const TArray<FNodeId>& InNodeIds)
{
	GraphObj->SelectNodes(InNodeIds);
}

FReply SRigDependencyGraph::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SRigDependencyGraph::HandleSelectionChanged(const FGraphPanelSelectionSet& SelectionSet)
{
	if (GraphObj->BlockSelectionCounter <= 0 && WeakControlRigEditor.IsValid())
	{
		GraphObj->BlockSelectionCounter = 2;

		URigHierarchy* Hierarchy = GraphObj->GetRigHierarchy();
		if (!Hierarchy)
		{
			return;
		}

		GraphObj->SelectedNodes.Reset();
		for (UObject* SelectedObject : SelectionSet)
		{
			if (const URigDependencyGraphNode* RigDependencyGraphNode = Cast<URigDependencyGraphNode>(SelectedObject))
			{
				const FNodeId& NodeId = RigDependencyGraphNode->GetNodeId();
				GraphObj->SelectedNodes.Add(NodeId);
			}
		}

		GraphObj->UpdateFadeOutStates();

		TArray<FRigElementKey> SelectedElements;
		TMap<const URigVMGraph*, TArray<FName>> NodeNamesPerGraph;
		
		for (const TObjectPtr<URigDependencyGraphNode>& DependencyGraphNode : GraphObj->DependencyGraphNodes)
		{
			const FNodeId& NodeId = DependencyGraphNode->GetNodeId();
			if (NodeId.IsElement() && GraphObj->SelectedNodes.Contains(NodeId))
			{
				SelectedElements.Add(Hierarchy->GetKey(NodeId.Index));
			}
			else if (NodeId.IsInstruction())
			{
				if (const URigVMNode* Node = DependencyGraphNode->GetRigVMNodeForInstruction())
				{
					TArray<FName>& SelectedNodesInGraph = NodeNamesPerGraph.FindOrAdd(Node->GetGraph());
					if (GraphObj->SelectedNodes.Contains(NodeId))
					{
						SelectedNodesInGraph.Add(Node->GetFName());
					}
				}
			}
		}

		if (URigHierarchyController* Controller = Hierarchy->GetController())
		{
			Controller->SetSelection(SelectedElements);
		}
		TSharedPtr<IControlRigBaseEditor> ControlRigEditor = WeakControlRigEditor.Pin();
		if (ControlRigEditor.IsValid())
		{
			if (FControlRigAssetInterfacePtr ControlRigBlueprint = ControlRigEditor->GetControlRigAssetInterface())
			{
				if (FRigVMClient* Client = ControlRigBlueprint->GetRigVMClient())
				{
					for (TPair<const URigVMGraph*, TArray<FName>>& Pair : NodeNamesPerGraph)
					{
						if (URigVMController* Controller = Client->GetOrCreateController(Pair.Key))
						{
							Controller->SetNodeSelection(Pair.Value);
						}
					}
				}
			}
		}
	}
}

void SRigDependencyGraph::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	GraphObj->Tick(InDeltaTime);
	
	if (GraphObj->NeedsRefreshLayout())
	{
		const FBox2D OldBounds = GraphObj->GetAllNodesBounds();
		GraphObj->GetRigDependencyGraphSchema()->LayoutNodes(GraphObj, 32);
		if (!GraphObj->GetNodes().IsEmpty() && GraphObj->ZoomAndFitDuringLayout.Get(false) && GraphObj->bIsPerformingGridLayout)
		{
			const FBox2D NewBounds = GraphObj->GetAllNodesBounds();
			if (!NewBounds.GetCenter().Equals(OldBounds.GetCenter(), 10.f) ||
				!NewBounds.GetSize().Equals(OldBounds.GetSize(), 10.f))
			{
				GraphEditor->ZoomToFit(false);
			}
		}
	}
}

void SRigDependencyGraph::OnGraphEditorTick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// flashlight to lighten up nodes in proximity of the mouse
	const FGeometry PaintGeometry = GraphEditor->GetPaintSpaceGeometry();
	FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();
	if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(GraphEditor->AsShared()))
	{
		MousePosition -= FVector2D(Window->GetPositionInScreen());
	}

	const float FlashLightRadius = FlashLightRadiusAttribute.Get();

	if(UControlRigEditorSettings::Get()->bEnableFlashlightInDependencyViewer && PaintGeometry.IsUnderLocation(MousePosition) && FlashLightRadius > SMALL_NUMBER)
	{
		FVector2f GraphEditorLocation = FVector2f::ZeroVector;
		float GraphEditorZoomAmount = 0;
		GraphEditor->GetViewLocation(GraphEditorLocation, GraphEditorZoomAmount);

		const FVector2f WidgetPosition = PaintGeometry.AbsoluteToLocal(MousePosition) / GraphEditorZoomAmount + GraphEditorLocation;
		for (URigDependencyGraphNode* Node : GraphObj->DependencyGraphNodes)
		{
			Node->ResetFadedOutState();

			const float PreviousFadedOutState = Node->GetFadedOutState();
			if (PreviousFadedOutState < 1.f - SMALL_NUMBER)
			{
				FSlateRect Bounds;
				if (GraphEditor->GetBoundsForNode(Node, Bounds, 0.f))
				{
					const FVector2f Center = Bounds.GetCenter();
					const float Radius = 64.f * FlashLightRadius;
					const float Ratio = FVector2f::DistSquared(Center, WidgetPosition) / (Radius * Radius);
					if (Ratio < 1.f)
					{
						const float HalfRatio = 1.f - FMath::Clamp(Ratio - 0.5f, 0.f, .5f) / 0.5f;
						Node->OverrideFadeOutState(FMath::Lerp(PreviousFadedOutState, 1.f, HalfRatio));
					}
				}
			}
		}
	}
	else
	{
		for (URigDependencyGraphNode* Node : GraphObj->DependencyGraphNodes)
		{
			Node->ResetFadedOutState();
		}
	}
}

void SRigDependencyGraph::PostUndo(bool bSuccess)
{
	GraphObj->RebuildGraph();
}

void SRigDependencyGraph::PostRedo(bool bSuccess)
{
	GraphObj->RebuildGraph();
}

#undef LOCTEXT_NAMESPACE
