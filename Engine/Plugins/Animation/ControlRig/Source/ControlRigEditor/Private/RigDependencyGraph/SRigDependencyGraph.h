// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaDelegates.h"
#include "GraphEditor.h"
#include "Containers/ArrayView.h"
#include "EditorUndoClient.h"
#include "RigDependencyGraph.h"
#include "RigDependencyGraphNode.h"

class UObject;
class URigDependencyGraph;
class SRigDependencyGraph;
class SProgressBar;
class SSearchBox;
class IControlRigBaseEditor;

/** Delegate used to communicate graph selection */
DECLARE_DELEGATE_OneParam(FOnRigDependencyGraphObjectsSelected, const TArrayView<UObject*>& /*InObjects*/);

class SRigDependencyGraph : public SCompoundWidget, public FEditorUndoClient
{
public:

	using FNodeId = URigDependencyGraph::FNodeId;
	
	SLATE_BEGIN_ARGS(SRigDependencyGraph)
		: _FlashLightRadius(5.f)
	{}

	SLATE_ATTRIBUTE(float, FlashLightRadius)

	SLATE_END_ARGS()

	~SRigDependencyGraph();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<IControlRigBaseEditor>& InControlRigEditor);

	/** Set the selected nodes */
	void SelectNodes(const TArray<FNodeId>& InNodeIds);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

private:

	void OnGraphEditorTick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	/** Called to create context menu when right-clicking on graph */
	FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Called when the path is being edited */
	void OnSearchBarTextChanged(const FText& NewText);
	void OnSearchMatchPicked(int32 InIndex);

	void HandleSelectionChanged(const FGraphPanelSelectionSet& SelectionSet);

	void HandleNodeDoubleClicked(UEdGraphNode* InNode);

	void OnClearNodeSelection();

	TSharedRef< SWidget > CreateTopLeftMenu();

	void BindCommands();

private:

	TSharedPtr<SGraphEditor> GraphEditor;
	TSharedPtr<SProgressBar> LayoutProgressBar;
	TWeakPtr<IControlRigBaseEditor> WeakControlRigEditor;

	URigDependencyGraph* GraphObj;
	TSharedPtr<FUICommandList> CommandList;

	/** Search box widget */
	TSharedPtr<SSearchBox> FilterBox;

	TArray<FNodeId> NodesMatchingSearch;
	TOptional<int32> MatchIndex;

	TAttribute<float> FlashLightRadiusAttribute;
};
