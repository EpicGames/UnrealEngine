// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDependencyGraph/SRigDependencyGraphNode.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "RigDependencyGraph/RigDependencyGraphNode.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "ControlRigEditorStyle.h"
#include "RigDependencyGraph.h"

class SWidget;
class UEdGraphSchema;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "RigDependencyGraphEditor"

class SRigDependencyGraphNodePin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SRigDependencyGraphNodePin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin)
	{
		this->SetCursor(EMouseCursor::Default);
		SetIsEditable(false);

		bShowLabel = false;
		GraphPinObj = InPin;
		check(GraphPinObj != NULL);

		// Set up a hover for pins that is tinted the color of the pin.
		SBorder::Construct(SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.BorderBackgroundColor(this, &SRigDependencyGraphNodePin::GetPinColor)
			.OnMouseButtonDown(this, &SRigDependencyGraphNodePin::OnPinMouseDown)
			.Cursor(this, &SRigDependencyGraphNodePin::GetPinCursor)
			.Content()
			[
				SNew(SImage)
				.Image(this, &SRigDependencyGraphNodePin::GetPinIcon)
			]
		);
	}

protected:
	/** SGraphPin interface */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		return SNew(SSpacer);
	}
};

void SRigDependencyGraphNode::Construct(const FArguments& InArgs, class URigDependencyGraphNode* InNode)
{
	GraphNode = InNode;
	
	SetCursor(EMouseCursor::CardinalCross);

	if (!ContentWidget.IsValid())
	{
		ContentWidget = SNullWidget::NullWidget;
	}
	
	UpdateGraphNode();
}

void SRigDependencyGraphNode::CreatePinWidgets()
{
	URigDependencyGraphNode* RigDependencyGraphNode = CastChecked<URigDependencyGraphNode>(GraphNode);

	UEdGraphPin& InputPin = RigDependencyGraphNode->GetInputPin();
	if (!InputPin.bHidden)
	{
		this->AddPin(SNew(SRigDependencyGraphNodePin, &InputPin));
	}

	UEdGraphPin& OutputPin = RigDependencyGraphNode->GetOutputPin();
	if (!OutputPin.bHidden)
	{
		this->AddPin(SNew(SRigDependencyGraphNodePin, &OutputPin));
	}
}

void SRigDependencyGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));

	if (PinToAdd->GetDirection() == EGPD_Input)
	{
		LeftNodeBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				PinToAdd
			];
	}
	else
	{
		RightNodeBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				PinToAdd
			];
	}
	OutputPins.Add(PinToAdd);
}

FText SRigDependencyGraphNode::GetNodeTitle() const
{
	return GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
}

FText SRigDependencyGraphNode::GetNodeTooltip() const
{
	return GraphNode->GetTooltipText();
}

const FSlateBrush* SRigDependencyGraphNode::GetNodeTitleIcon() const
{
	FLinearColor Color = FLinearColor::White;
	return GraphNode->GetIconAndTint(Color).GetIcon();
}

FSlateColor SRigDependencyGraphNode::GetNodeBodyColor() const
{
	return GraphNode->GetNodeBodyTintColor();
}

const FSlateBrush* SRigDependencyGraphNode::GetNodeBodyBrush() const
{
	return FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.DependencyGraph.NodeBody"));
}

void SRigDependencyGraphNode::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage( FControlRigEditorStyle::Get().GetBrush("ControlRig.DependencyGraph.NodeBody") )
			.BorderBackgroundColor(this, &SRigDependencyGraphNode::GetNodeBodyColor)
			.Padding(0)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(LeftNodeBox, SVerticalBox)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SImage)
						.Image(this, &SRigDependencyGraphNode::GetNodeTitleIcon)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(this, &SRigDependencyGraphNode::GetNodeTitle)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				]
			]
		];

	CreatePinWidgets();
}

FReply SRigDependencyGraphNode::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		if (URigDependencyGraphNode* DependencyGraphNode = Cast<URigDependencyGraphNode>(GraphNode))
		{
			if (URigDependencyGraph* DependencyGraph = DependencyGraphNode->GetRigDependencyGraph())
			{
				const bool ClearSelection = !MouseEvent.GetModifierKeys().IsShiftDown();
				const bool SelectSourceNodes = !MouseEvent.GetModifierKeys().IsControlDown();
				const bool SelectIsland = MouseEvent.GetModifierKeys().IsAltDown();
				if (SelectIsland)
				{
					DependencyGraph->SelectNodeIsland({DependencyGraphNode->GetNodeId()}, ClearSelection);
				}
				else
				{
					DependencyGraph->SelectLinkedNodes({DependencyGraphNode->GetNodeId()}, SelectSourceNodes, ClearSelection, true);
				}
			}
			return FReply::Handled();
		}
	}
	return SGraphNode::OnMouseButtonUp(MyGeometry, MouseEvent);
}

int32 SRigDependencyGraphNode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FWidgetStyle WidgetStyle = InWidgetStyle;
	
	if (const URigDependencyGraphNode* DependencyGraphNode = Cast<URigDependencyGraphNode>(GraphNode))
	{
		WidgetStyle.BlendOpacity(DependencyGraphNode->GetFadedOutState());
	}
	
	return SGraphNode::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, WidgetStyle, bParentEnabled);
}

#undef LOCTEXT_NAMESPACE
