// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "SGraphNode.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SGraphPin;
class SVerticalBox;
class SWidget;
struct FSlateBrush;

class SRigDependencyGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SRigDependencyGraphNode){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, class URigDependencyGraphNode* InNode);

protected:
	// SGraphNode interface
	virtual void CreatePinWidgets() override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;

	FText GetNodeTitle() const;
	virtual FText GetNodeTooltip() const override;
	const FSlateBrush* GetNodeTitleIcon() const;
	virtual FSlateColor GetNodeBodyColor() const override;
	virtual const FSlateBrush* GetNodeBodyBrush() const override;
	virtual void UpdateGraphNode() override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	
protected:
	/** The content widget for this node - derived classes can insert what they want */
	TSharedPtr<SWidget> ContentWidget;
};
