// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBakeToControlRigDialog.h"
#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "BakeToControlRigSettings.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyEditorModule.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"

class SBakeToControlRigDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SBakeToControlRigDialog) {}
	SLATE_END_ARGS()
	~SBakeToControlRigDialog()
	{
	}
	
	void Construct(const FArguments& InArgs)
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Bake To Control Rig";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		ChildSlot
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
			.Text(NSLOCTEXT("ControlRig", "BakeToControlRig", "Bake To Control Rig"))
			.OnClicked(this, &SBakeToControlRigDialog::OnBakeToControlRig)
			]

			];

		UBakeToControlRigSettings* BakeSettings = GetMutableDefault<UBakeToControlRigSettings>();
		DetailView->SetObject(BakeSettings);
	}

	void SetDelegate(FBakeToControlDelegate& InDelegate)
	{
		Delegate = InDelegate;
	}

private:

	FReply OnBakeToControlRig()
	{
		UBakeToControlRigSettings* BakeSettings = GetMutableDefault<UBakeToControlRigSettings>();
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (BakeSettings && Delegate.IsBound())
		{
			Delegate.Execute(BakeSettings->bReduceKeys, BakeSettings->SmartReduce.TolerancePercentage, BakeSettings->SmartReduce.SampleRate, true /*bResetControls  always*/);
		}
		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return FReply::Handled();
	}
	TSharedPtr<IDetailsView> DetailView;
	FBakeToControlDelegate  Delegate;
};


void BakeToControlRigDialog::GetBakeParams(FBakeToControlDelegate& InDelegate, const FOnWindowClosed& OnClosedDelegate)
{
	const FText TitleText = NSLOCTEXT("ControlRig", "BakeToControlRig", "Bake To Control Rig");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(400.0f, 200.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SBakeToControlRigDialog> DialogWidget = SNew(SBakeToControlRigDialog);
	DialogWidget->SetDelegate(InDelegate);
	Window->SetContent(DialogWidget);
	Window->SetOnWindowClosed(OnClosedDelegate);

	FSlateApplication::Get().AddWindow(Window);

}

