// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Widgets/SWindow.h"
#include "Misc/FrameRate.h"

DECLARE_DELEGATE_FourParams(FBakeToControlDelegate, bool, float, FFrameRate, bool);

struct BakeToControlRigDialog
{
	static void GetBakeParams(FBakeToControlDelegate& Delegate, const FOnWindowClosed& OnClosedDelegate);
};