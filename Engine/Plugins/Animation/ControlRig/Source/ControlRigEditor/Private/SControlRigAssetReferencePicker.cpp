// Copyright Epic Games, Inc. All Rights Reserved.

#include "SControlRigAssetReferencePicker.h"

#include "ControlRigBlueprintLegacy.h"
#include "ControlRigEditorAsset.h"
#include "ControlRigRuntimeAsset.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/VerticalBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"

void SControlRigAssetReferencePicker::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	ControlRigFilter = InArgs._Filter;
	ExtraAssets = InArgs._ExtraAssets;
	SortPredicate = InArgs._SortPredicate;
	
	RefreshFilteredEntries();
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged(this, &SControlRigAssetReferencePicker::OnFilterTextChanged)
		]
		+SVerticalBox::Slot()
		[
			SNew(SBox)
			.WidthOverride(350.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.MaxHeight(400.0f)
				.AutoHeight()
				[
					SAssignNew(AssetList, SListView<TSharedRef<FControlRigAssetSoftReference>>)
						.ListItemsSource(&Entries)
						.OnGenerateRow(this, &SControlRigAssetReferencePicker::GenerateRow)
						.OnSelectionChanged_Lambda([this](TSharedPtr<FControlRigAssetSoftReference> NewChoice, ESelectInfo::Type SelectType)
							{
								SelectedSource = *NewChoice.Get();
								OnSelectionChanged.ExecuteIfBound(SelectedSource);
							})
				]
			]
		]
	];
	
	if (SearchBox.IsValid())
	{
		// set the focus to the search box so you can start typing right away
		SearchBox->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double,float)
		{
			FSlateApplication::Get().ForEachUser([this](FSlateUser& User)
			{
				User.SetFocus(SearchBox.ToSharedRef());
			});
			return EActiveTimerReturnType::Stop;
		}));
	}
}

TSharedRef<ITableRow> SControlRigAssetReferencePicker::GenerateRow(TSharedRef<FControlRigAssetSoftReference> Entry,const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Entry->IsValid());

	const FSlateBrush* Icon = nullptr;
	return SNew(STableRow<TSharedPtr<FControlRigAssetSoftReference>>, OwnerTable)
		.Padding(FMargin(2.f))
		[
			SNew(SHorizontalBox)

			// Icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(Icon)
			]

			// Name
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(6.f, 0.f)
			[
				SNew(STextBlock)
				.Text_Lambda([Entry]()
					{
						return GetDisplayName(*Entry);
					})
				.HighlightText(FText::FromString(FilterText))
				.ToolTipText_Lambda([Entry]()
					{
						return FText::FromString(Entry->GetPathName());
					})
			]
		];
}

FText SControlRigAssetReferencePicker::GetDisplayName(const FControlRigAssetSoftReference& Source)
{
	FString Name = Source.GetName();
	Name.RemoveFromEnd(TEXT("_C"));
	return FText::FromString(Name);
}

void SControlRigAssetReferencePicker::OnFilterTextChanged(const FText& InFilterText)
{
	if (FilterText == InFilterText.ToString())
	{
		return;
	}
	
	FilterText = InFilterText.ToString();
	RefreshFilteredEntries();
	if (AssetList.IsValid())
	{
		AssetList->RequestListRefresh();
	}
}

void SControlRigAssetReferencePicker::RefreshFilteredEntries()
{
	Entries.Reset();

	FAssetRegistryModule& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FARFilter Filter;
	Filter.ClassPaths.Add(UControlRigEditorAssetInterface::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(UControlRigRuntimeAssetInterface::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	TArray<FAssetData> OutAssets;
	AssetRegistry.Get().GetAssets(Filter, OutAssets);
	
	for (FAssetData& AssetData : OutAssets)
	{
		if (ControlRigFilter.IsValid() && !ControlRigFilter->MatchesFilter(AssetData))
		{
			continue;
		}
		
		if (AssetData.GetClass() == UControlRigBlueprint::StaticClass())
		{
			FString GeneratedClassPath;
			if (AssetData.GetTagValue(TEXT("GeneratedClass"), GeneratedClassPath))
			{
				TSharedRef<FControlRigAssetSoftReference> Item = MakeShared<FControlRigAssetSoftReference>();
				FSoftClassPath SoftClassPath(GeneratedClassPath);
				Item->Set(TSoftClassPtr<UControlRig>(SoftClassPath));
				if (FilterText.IsEmpty() || GetDisplayName(*Item).ToString().Contains(FilterText))
				{
					Entries.Add(Item);
				}
			}
		}
		
		if (AssetData.GetClass() == UControlRigRuntimeAsset::StaticClass())
		{
			TSharedRef<FControlRigAssetSoftReference> Item = MakeShared<FControlRigAssetSoftReference>();
			Item->Set(TSoftObjectPtr<UControlRigRuntimeAsset>(AssetData.ToSoftObjectPath()));
			if (FilterText.IsEmpty() || GetDisplayName(*Item).ToString().Contains(FilterText))
			{
				Entries.Add(Item);
			}
		}
	}
	
	for (FControlRigAssetSoftReference& Extra : ExtraAssets)
	{
		TSharedRef<FControlRigAssetSoftReference> Item = MakeShared<FControlRigAssetSoftReference>();
		Item.Get() = Extra;
		if (FilterText.IsEmpty() || GetDisplayName(*Item).ToString().Contains(FilterText))
		{
			Entries.Add(Item);
		}
	}
	
	// Sort
	if (SortPredicate.IsSet())
	{
		Entries.Sort(SortPredicate);
	}
	else
	{
		Entries.Sort([](const TSharedRef<FControlRigAssetSoftReference>& A, const TSharedRef<FControlRigAssetSoftReference>& B)
			{
				return GetDisplayName(*A).CompareTo(GetDisplayName(*B)) <= 0;
			});
	}
}
