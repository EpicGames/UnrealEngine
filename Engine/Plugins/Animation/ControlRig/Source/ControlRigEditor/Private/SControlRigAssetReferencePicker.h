// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigAssetReference.h"
#include "IControlRigEditorModule.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#define UE_API CONTROLRIGEDITOR_API



class SControlRigAssetReferencePicker : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, FControlRigAssetSoftReference);
	SLATE_BEGIN_ARGS(SControlRigAssetReferencePicker)
	{
	}
	SLATE_ARGUMENT(TSharedPtr<FControlRigClassFilter>, Filter)
	SLATE_ARGUMENT(TArray<FControlRigAssetSoftReference>, ExtraAssets)
	SLATE_ARGUMENT(TFunction<bool(const TSharedRef<FControlRigAssetSoftReference> A, const  TSharedRef<FControlRigAssetSoftReference> B)>, SortPredicate)
	SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	
	TSharedRef<ITableRow> GenerateRow(TSharedRef<FControlRigAssetSoftReference> Entry, const TSharedRef<STableViewBase>& OwnerTable);
	static FText GetDisplayName(const FControlRigAssetSoftReference& Source);
	void OnFilterTextChanged(const FText& InFilterText);
	void RefreshFilteredEntries();
	
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SListView<TSharedRef<FControlRigAssetSoftReference>>> AssetList;
	TArray<TSharedRef<FControlRigAssetSoftReference>> Entries;
	FControlRigAssetSoftReference SelectedSource;
	FString FilterText;
	
	FOnSelectionChanged OnSelectionChanged;
	TSharedPtr<FControlRigClassFilter> ControlRigFilter;
	TArray<FControlRigAssetSoftReference> ExtraAssets;
	TFunction<bool(const TSharedRef<FControlRigAssetSoftReference> A, const  TSharedRef<FControlRigAssetSoftReference> B)> SortPredicate;
};

#undef UE_API