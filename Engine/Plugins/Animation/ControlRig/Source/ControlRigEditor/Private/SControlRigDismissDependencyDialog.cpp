// Copyright Epic Games, Inc. All Rights Reserved.

#include "SControlRigDismissDependencyDialog.h"
#include "CoreMinimal.h"
#include "Dialogs/Dialogs.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ControlRig.h"

class SControlRigDismissDependencyDialog : public SModalEditorDialog<bool>
{
	SLATE_BEGIN_ARGS(SControlRigDismissDependencyDialog) {}
	SLATE_END_ARGS()
	
	~SControlRigDismissDependencyDialog()
	{
	}
	
	void Construct(const FArguments& InArgs, const URigHierarchy* InHierarchy, const FRigElementKey& InChild, const FRigElementKey& InParent, const FRigHierarchyDependencyChain& InDependencyChain)
	{
		check(InHierarchy);

		FString ClassName = InHierarchy->GetPathName();
		if (const UControlRig* ControlRig = Cast<UControlRig>(InHierarchy->GetOuter()))
		{
			ClassName = ControlRig->GetClass()->GetPackage()->GetName();
		}

		const FText TitleFormat = NSLOCTEXT("ControlRig", "DismissDependencyTitleFormat", "Potential Cycle relating {1} to {0}");
		const FText Title = FText::Format(TitleFormat, FText::FromString(InChild.ToString()), FText::FromString(InParent.ToString()));
		const FText Format = NSLOCTEXT("ControlRig", "DismissDependencyMessageFormat", "{0}\n\n{1}\n\n{2}");
		Message = FText::Format(
			Format,
			FText::FromString(ClassName),
			Title,
			FText::FromString(InHierarchy->GetMessageFromDependencyChain(InDependencyChain))
		);
		
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("AssetDeleteDialog.Background"))
			.Padding(10.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1)
				.HAlign(HAlign_Fill)
				.Padding(5.f)
				[
					SNew(SScrollBox)
					.Orientation(Orient_Vertical)
					+ SScrollBox::Slot()
					.HAlign(HAlign_Fill)
					.FillSize(1.0)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("NormalFont"))
						.HighlightText(Title)
						.Text(Message)
						.OnDoubleClicked(this, &SControlRigDismissDependencyDialog::OnMessageDoubleClicked)
					]
					+ SScrollBox::Slot()
					.HAlign(HAlign_Fill)
					.AutoSize()
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("SmallFont"))
						.Text(NSLOCTEXT("ControlRig", "DoubleClickThisMessageToCopy", "Ignoring a cycle may cause unexpected behaviour in the rig.\n\nDouble-Click the message above to copy it.\n\nThis dialog can be disabled with 'ControlRig.Hierarchy.AllowToIgnoreCycles'."))
						.OnDoubleClicked(this, &SControlRigDismissDependencyDialog::OnMessageDoubleClicked)
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(5.f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ContentPadding(FMargin(10, 5))
						.Text(NSLOCTEXT("ControlRig", "Cancel", "Cancel"))
						.ToolTipText(NSLOCTEXT("ControlRig", "ActionCancelDismissDependency", "Aborts the action and respects the dependency."))
						.OnClicked(this, &SControlRigDismissDependencyDialog::OnCancel)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
						.ContentPadding(FMargin(10, 5))
						.Text(NSLOCTEXT("ControlRig", "Ignore", "Ignore"))
						.ToolTipText(NSLOCTEXT("ControlRig", "ActionDismissDependency", "Ignore the dependency and proceed with the action."))
						.OnClicked(this, &SControlRigDismissDependencyDialog::OnDismissDependency)
					]
				]
			]
		];
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}
		if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
		{
			OnMessageDoubleClicked(MyGeometry, FPointerEvent());
		}
		return SModalEditorDialog<bool>::OnKeyDown(MyGeometry, InKeyEvent);
	}

private:

	FReply OnCancel()
	{
		ProvideResult(false);
		return FReply::Handled();
	}

	FReply OnDismissDependency()
	{
		ProvideResult(true);
		return FReply::Handled();
	}

	FReply OnMessageDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent) const
	{
		FPlatformApplicationMisc::ClipboardCopy(*Message.ToString());

		FNotificationInfo Info(NSLOCTEXT("ControlRig", "DependencyChainMessageCopiedToClipboard", "Message has been copied to the clipboard!"));
		Info.bFireAndForget = true;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 2.0f;
		TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);

		return FReply::Handled();
	}

	FText Message;
};


bool ControlRigDismissDependencyDialog::LaunchDismissDependencyDialog(const URigHierarchy* InHierarchy, const FRigElementKey& InChild, const FRigElementKey& InParent, const FRigHierarchyDependencyChain& InDependencyChain)
{
	// todo?: compute a key to remember this setting and don't ask in the future
	TSharedPtr<SControlRigDismissDependencyDialog> Dialog = SNew(SControlRigDismissDependencyDialog, InHierarchy, InChild, InParent, InDependencyChain);
	const bool bDismissed = Dialog->ShowModalDialog(NSLOCTEXT("ControlRig", "DismissDependencyDialogTitle","Cyclic Dependency Found"));
	return bDismissed;
}
