// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Rigs/RigHierarchy.h"

struct ControlRigDismissDependencyDialog
{
	static bool LaunchDismissDependencyDialog(const URigHierarchy* InHierarchy, const FRigElementKey& InChild, const FRigElementKey& InParent, const FRigHierarchyDependencyChain& InDependencyChain);
};

class FControlRigDismissDependencyDialogGuard
{
public:
	FControlRigDismissDependencyDialogGuard(URigHierarchy* InHierarchy)
	: Hierarchy(InHierarchy)
	{
		if (Hierarchy.IsValid())
		{
			Hierarchy->OnDependencyDismissed().BindStatic(&ControlRigDismissDependencyDialog::LaunchDismissDependencyDialog);
		}
	}

	~FControlRigDismissDependencyDialogGuard()
	{
		if (Hierarchy.IsValid())
		{
			Hierarchy->OnDependencyDismissed().Unbind();
		}
	}

private:

	TWeakObjectPtr<URigHierarchy> Hierarchy;
};