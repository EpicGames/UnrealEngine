// Copyright Epic Games, Inc. All Rights Reserved.

#include "SControlRigDragDropBox.h"

namespace UE::ControlRigEditor
{
	void SControlRigDragDropBox::Construct(const FArguments& InArgs)
	{
		OnDragDetectedDelegate = InArgs._OnDragDetected;

		ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			InArgs._Content.Widget
		];
	}

	FReply SControlRigDragDropBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return FReply::Handled().DetectDrag(AsShared(), EKeys::LeftMouseButton);
	}

	FReply SControlRigDragDropBox::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (OnDragDetectedDelegate.IsBound())
		{
			return OnDragDetectedDelegate.Execute(MyGeometry, MouseEvent);
		}

		return FReply::Unhandled();
	}
}
