// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Input/DragAndDrop.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ControlRigEditor
{
	/** A box useful to detect drags on arbitrary widgets */
	class SControlRigDragDropBox
		: public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SControlRigDragDropBox) {}

			/** The box content */
			SLATE_DEFAULT_SLOT(FArguments, Content)

			/** Event raised when a drag was detected */
			SLATE_EVENT(FOnDragDetected, OnDragDetected)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:
		//~ Begin SWidget interface
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		//~ End SWidget interface

		/** Delegate to execute when drag was detected */
		FOnDragDetected OnDragDetectedDelegate;
	};
}
