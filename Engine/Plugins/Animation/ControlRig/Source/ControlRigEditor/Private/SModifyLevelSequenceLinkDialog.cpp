// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModifyLevelSequenceLinkDialog.h"
#include "ModifyLevelSequenceLinkSettings.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ControlRig"

class SModifyLevelSequenceLinkDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SModifyLevelSequenceLinkDialog)
		: _LinkItem()
	{}
		SLATE_ARGUMENT(FLevelSequenceAnimSequenceLinkItem, LinkItem)
		SLATE_ARGUMENT(FModifyLevelSequenceLinkDelegate, Delegate)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Delegate = InArgs._Delegate;

		SettingsObject = NewObject<UModifyLevelSequenceLinkSettings>(GetTransientPackage(), NAME_None, RF_Transient);
		SettingsObject->LinkItem = InArgs._LinkItem;
		SettingsObject->AddToRoot();

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Modify Level Sequence Link Settings";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);
		DetailView->SetObject(SettingsObject);

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
				.Text(LOCTEXT("SaveLinkSettings", "Save Settings"))
				.OnClicked(this, &SModifyLevelSequenceLinkDialog::OnSaveClicked)
			]
		];
	}

	~SModifyLevelSequenceLinkDialog()
	{
		if (SettingsObject)
		{
			SettingsObject->RemoveFromRoot();
			SettingsObject = nullptr;
		}
	}

private:
	FReply OnSaveClicked()
	{
		if (SettingsObject)
		{
			Delegate.ExecuteIfBound(SettingsObject->LinkItem);
		}
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	TSharedPtr<IDetailsView> DetailView;
	FModifyLevelSequenceLinkDelegate Delegate;
	UModifyLevelSequenceLinkSettings* SettingsObject = nullptr;
};

void ModifyLevelSequenceLinkDialog::ShowDialog(const FLevelSequenceAnimSequenceLinkItem& InLinkItem, FModifyLevelSequenceLinkDelegate InDelegate)
{
	const FText TitleText = LOCTEXT("ModifyLinkSettingsTitle", "Modify Link Settings");

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(400.f, 500.f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SModifyLevelSequenceLinkDialog> DialogWidget = SNew(SModifyLevelSequenceLinkDialog)
		.LinkItem(InLinkItem)
		.Delegate(InDelegate);

	Window->SetContent(DialogWidget);
	FSlateApplication::Get().AddWindow(Window);
}

#undef LOCTEXT_NAMESPACE
