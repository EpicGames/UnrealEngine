// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "LevelSequenceAnimSequenceLink.h"

DECLARE_DELEGATE_OneParam(FModifyLevelSequenceLinkDelegate, const FLevelSequenceAnimSequenceLinkItem&);

struct ModifyLevelSequenceLinkDialog
{
	static void ShowDialog(const FLevelSequenceAnimSequenceLinkItem& InLinkItem, FModifyLevelSequenceLinkDelegate InDelegate);
};
