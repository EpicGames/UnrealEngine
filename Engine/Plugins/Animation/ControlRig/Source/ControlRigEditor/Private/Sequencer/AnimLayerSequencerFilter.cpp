// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/AnimLayerSequencerFilter.h"
#include "AnimLayers/AnimLayers.h"
#include "Filters/ISequencerTrackFilters.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ITrackLaneExtension.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "MVVM/ViewModels/ChannelModel.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimLayerSequencerFilter)

#define LOCTEXT_NAMESPACE "UnimLayerSequencerFilter"

using namespace UE::Sequencer;
//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_AnimLayerSequencerFilterCommands
	: public TCommands<FSequencerTrackFilter_AnimLayerSequencerFilterCommands>
{
public:

	FSequencerTrackFilter_AnimLayerSequencerFilterCommands()
		: TCommands<FSequencerTrackFilter_AnimLayerSequencerFilterCommands>(
			"FSequencerTrackFilter_AnimLayerSequencerFilter",
			NSLOCTEXT("Contexts", "FSequencerTrackFilter_AnimLayerSequencer", "FSequencerTrackFilter_AnimLayerSequencer"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{ }

	/** Toggle the control rig controls filter */
	TSharedPtr< FUICommandInfo > ToggleSelectedAnimLayer;

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleSelectedAnimLayer, "Selected Anim layer", "Toggle the filter for elected Anim Layer", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::F8));
	}
};

class FSequencerTrackFilter_ToggleSelectedAnimLayer : public FSequencerTrackFilter
{
public:

	FSequencerTrackFilter_ToggleSelectedAnimLayer(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr)
		: FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
		, BindingCount(0)
	{
		FSequencerTrackFilter_AnimLayerSequencerFilterCommands::Register();
	}

	~FSequencerTrackFilter_ToggleSelectedAnimLayer()
	{
		BindingCount--;

		if (BindingCount < 1)
		{
			FSequencerTrackFilter_AnimLayerSequencerFilterCommands::Unregister();
		}
	}

	virtual FString GetName() const override { return TEXT("SelectedAnimLayersFilter"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("SequenceTrackFilter_SelectedAnimLayers", "Selected Anim Layers"); }
	virtual FSlateIcon GetIcon() const override { return FSlateIconFinder::FindIconForClass(UControlRig::StaticClass()); }

	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();

		const TViewModelPtr<IOutlinerExtension> OutlinerExtension = InItem.AsModel()->FindAncestorOfType<IOutlinerExtension>();
		if (!OutlinerExtension.IsValid())
		{
			return false;
		}
		UMovieSceneTrack* Track = nullptr;

		if (const TViewModelPtr<ITrackExtension> AncestorTrackModel = InItem->FindAncestorOfType<ITrackExtension>(true))
		{
			Track = AncestorTrackModel->GetTrack();
		}

		if (Track)
		{
			ULevelSequence* LevelSequence = Track->GetTypedOuter<ULevelSequence>();
			if (LevelSequence)
			{
				if (const UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(LevelSequence))
				{
					TArray<UMovieSceneSection*> Sections = AnimLayers->GetSelectedLayerSections();
					if (TViewModelPtr<FChannelGroupOutlinerModel> ChannelModel = CastViewModel<FChannelGroupOutlinerModel>(InItem))
					{
						for (UMovieSceneSection* Section : Sections)
						{
							if (ChannelModel->GetChannel(Section))
							{
								return true;
							}
						}
					}
				}
			}
		}
		return false;
	}
	virtual FText GetDefaultToolTipText() const override
	{
		return LOCTEXT("SequenceTrackFilter_SelectedAnimLayersTip", "Show Selected Anim Layers");
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override
	{
		return FSequencerTrackFilter_AnimLayerSequencerFilterCommands::Get().ToggleSelectedAnimLayer;
	}


private:
	mutable uint32 BindingCount;
};

//////////////////////////////////////////////////////////////////////////
//


void UAnimLayerSequencerFilter::AddTrackFilterExtensions(ISequencerTrackFilters& InOutFilterInterface
	, const TSharedRef<FFilterCategory>& InPreferredCategory
	, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const
{
	InOutFilterList.Add(MakeShared<FSequencerTrackFilter_ToggleSelectedAnimLayer>(InOutFilterInterface, InPreferredCategory));
}
#undef LOCTEXT_NAMESPACE

