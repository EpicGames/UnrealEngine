// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterExtension.h"
#include "AnimLayerSequencerFilter.generated.h"

UCLASS()
class UAnimLayerSequencerFilter : public USequencerTrackFilterExtension
{
public:
	GENERATED_BODY()

	//~ Begin USequencerTrackFilterExtension
	virtual void AddTrackFilterExtensions(ISequencerTrackFilters& InOutFilterInterface
		, const TSharedRef<FFilterCategory>& InPreferredCategory
		, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const override;
	//~ End USequencerTrackFilterExtension
};
