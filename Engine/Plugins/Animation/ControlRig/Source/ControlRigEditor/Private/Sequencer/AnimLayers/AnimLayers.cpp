// Copyright Epic Games, Inc. All Rights Reserved.
#include "AnimLayers.h"
#include "ControlRig.h"
#include "ISequencer.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "LevelSequence.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/SaveScopedControlRigSelection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "ILevelSequenceEditorToolkit.h"
#include "LevelSequencePlayer.h"
#include "EditMode/ControlRigEditMode.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MovieScene.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "ScopedTransaction.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "EditMode/ControlRigEditMode.h"
#include "ISequencerPropertyKeyedStatus.h"
#include "BakingAnimationKeySettings.h"
#include "Algo/Accumulate.h"
#include "MovieSceneToolHelpers.h"
#include "Tools/EvaluateSequencerTools.h"
#include "LevelEditorViewport.h"

#include "EulerTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimLayers)
#define LOCTEXT_NAMESPACE "AnimLayers"

bool FAnimLayerSelectionSet::MergeWithAnotherSelection(const FAnimLayerSelectionSet& Selection)
{
	if (BoundObject.IsValid() && (BoundObject.Get() == Selection.BoundObject.Get()))
	{
		for (const TPair<FName, FAnimLayerPropertyAndChannels>& Incoming : Selection.Names)
		{
			FAnimLayerPropertyAndChannels& Channels = Names.FindOrAdd(Incoming.Key);
			Channels.Channels |= (Incoming.Value.Channels);
		}
		return true;
	}
	return false;
}

FAnimLayerState::FAnimLayerState() : bKeyed(ECheckBoxState::Unchecked), bActive(true),  bLock(false), Weight(1.0), Type((uint32)EAnimLayerType::Base)
{
	Name = FText(LOCTEXT("BaseLayer", "Base Layer"));
}

FText FAnimLayerState::AnimLayerTypeToText() const
{
	if ((EAnimLayerType)Type == EAnimLayerType::Additive)
	{
		return FText(LOCTEXT("Additive", "Additive"));
	}
	else if ((EAnimLayerType)Type == EAnimLayerType::Override)
	{
		return FText(LOCTEXT("Override", "Override"));
	}
	return FText(LOCTEXT("Base", "Base"));
}

void FAnimLayerItem::MakeCopy(const FGuid& NewGuid, const TWeakObjectPtr<UObject>& NewObject,  FAnimLayerItem& OutCopy) const
{
	OutCopy.SequencerGuid = NewGuid;
	for (const FAnimLayerSectionItem& SectionItem : SectionItems)
	{
		if (SectionItem.Section.IsValid())
		{
			FAnimLayerSectionItem CopySectionItem;
			CopySectionItem.Section = SectionItem.Section;
			CopySectionItem.AnimLayerSet = SectionItem.AnimLayerSet;
			CopySectionItem.AnimLayerSet.BoundObject = NewObject;
			OutCopy.SectionItems.Add(CopySectionItem);
		}
	}
}


UAnimLayer::UAnimLayer(class FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	WeightProxy = NewObject<UAnimLayerWeightProxy>(this, TEXT("Weight"), RF_Transactional);
}



void UAnimLayer::UpdateSceneObjectorGuidsForItems(ISequencer* InSequencer)
{
	TArray<TPair<TWeakObjectPtr<UObject>, FGuid>> DeadObjects;
	for (TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key.IsValid() && Pair.Key->IsA<UControlRig>() == false)
		{
			UObject* Object = Pair.Key.Pin().Get();
			//if guid is not set we need to set it.
			if (Pair.Value.SequencerGuid.IsValid() == false)
			{
				USceneComponent* SceneComponent = nullptr;
				AActor *Actor = nullptr;
				SceneComponent = Cast<USceneComponent>(Object);
				if (SceneComponent)
				{
					Actor = SceneComponent->GetOwner();
				}
				else
				{
					Actor = Cast<AActor>(Object);
				}
				FGuid Binding;
				if (SceneComponent)
				{
					Binding = InSequencer->GetHandleToObject(SceneComponent, false /*bCreateHandle*/);
					if (Binding.IsValid())
					{
						Pair.Value.SequencerGuid = Binding;
					}
					
				}
				if(Binding.IsValid() == false && Actor)
				{
					Binding = InSequencer->GetHandleToObject(Actor, false /*bCreateHandle*/);
					if (Binding.IsValid())
					{
						Pair.Value.SequencerGuid = Binding;
					}
				}
			}
		}
		else if(Pair.Value.SequencerGuid.IsValid())//not valid so update it
		{
			TPair<TWeakObjectPtr<UObject>, FGuid> DeadObject;
			DeadObject.Key = Pair.Key;
			DeadObject.Value = Pair.Value.SequencerGuid;
			DeadObjects.Add(DeadObject);
		}
	}
	for (TPair<TWeakObjectPtr<UObject>, FGuid>& DeadObject : DeadObjects)
	{
		for (TWeakObjectPtr<> BoundObject : InSequencer->FindBoundObjects(DeadObject.Value, InSequencer->GetFocusedTemplateID()))
		{
			if (!BoundObject.IsValid())
			{
				continue;
			}
			if (FAnimLayerItem* Item = AnimLayerItems.Find(DeadObject.Key))
			{
				FAnimLayerItem NewCopy;
				Item->MakeCopy(DeadObject.Value, BoundObject, NewCopy);
				AnimLayerItems.Add(BoundObject, NewCopy);
			}
			AnimLayerItems.Remove(DeadObject.Key);
		}
	}
}

void UAnimLayer::SetKey(TSharedPtr<ISequencer>& Sequencer, const IPropertyHandle& KeyedPropertyHandle)
{
	FScopedTransaction PropertyChangedTransaction(LOCTEXT("KeyAnimLayerWeight", "Key Anim Layer Weight"), !GIsTransacting);
	bool bAnythingKeyed = false;

	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid() && SectionItem.Section.Get()->TryModify())
				{
					FMovieSceneFloatChannel* FloatChannel = nullptr;
					if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
					{
						FloatChannel = &CRSection->Weight;
					}
					else if (UMovieScene3DTransformSection* LayerSection = Cast<UMovieScene3DTransformSection>(SectionItem.Section.Get()))
					{
						FloatChannel = LayerSection->GetWeightChannel();
					}
					if (FloatChannel)
					{
						//don't add key if there!
						FFrameTime LocalTime = Sequencer->GetLocalTime().Time;
						const TRange<FFrameNumber> FrameRange = TRange<FFrameNumber>(LocalTime.FrameNumber);
						TArray<FFrameNumber> KeyTimes;
						FloatChannel->GetKeys(FrameRange, &KeyTimes, nullptr);
						if (KeyTimes.Num() == 0)
						{
							float Value = State.Weight;
							FloatChannel->AddCubicKey(LocalTime.FrameNumber, Value, ERichCurveTangentMode::RCTM_SmartAuto);
							bAnythingKeyed = true;
						}
					}
				}
			}
		}
	}

	if(bAnythingKeyed == false)
	{
		PropertyChangedTransaction.Cancel();
	}
}

EPropertyKeyedStatus UAnimLayer::GetPropertyKeyedStatus(TSharedPtr<ISequencer>& Sequencer, const IPropertyHandle& PropertyHandle)
{
	EPropertyKeyedStatus KeyedStatus = EPropertyKeyedStatus::NotKeyed;

	if (Sequencer.IsValid() == false || Sequencer->GetFocusedMovieSceneSequence() == nullptr)
	{
		return KeyedStatus;
	}
	const TRange<FFrameNumber> FrameRange = TRange<FFrameNumber>(Sequencer->GetLocalTime().Time.FrameNumber);
	int32 NumKeyed = 0;
	int32 NumToCheck = 0;
	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					FMovieSceneFloatChannel* FloatChannel = nullptr;
					if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
					{
						FloatChannel = &CRSection->Weight;
					}
					else if (UMovieScene3DTransformSection* LayerSection = Cast<UMovieScene3DTransformSection>(SectionItem.Section.Get()))
					{
						FloatChannel = LayerSection->GetWeightChannel();
					}
					if (FloatChannel)
					{
						++NumToCheck;
						EPropertyKeyedStatus NewKeyedStatus = EPropertyKeyedStatus::NotKeyed;
						if (FloatChannel->GetNumKeys() > 0)
						{
							TArray<FFrameNumber> KeyTimes;
							FloatChannel->GetKeys(FrameRange, &KeyTimes, nullptr);
							if (KeyTimes.Num() > 0)
							{
								++NumKeyed;
								NewKeyedStatus = EPropertyKeyedStatus::PartiallyKeyed;
							}
							else
							{
								NewKeyedStatus = EPropertyKeyedStatus::KeyedInOtherFrame;
							}
						}
						KeyedStatus = FMath::Max(KeyedStatus, NewKeyedStatus);
					}
				}
			}
		}
	}
	if (KeyedStatus == EPropertyKeyedStatus::PartiallyKeyed && NumToCheck == NumKeyed)
	{
		KeyedStatus = EPropertyKeyedStatus::KeyedInFrame;
	}
	return KeyedStatus;
}

ECheckBoxState UAnimLayer::GetKeyed() const
{
	TOptional<ECheckBoxState> CurrentVal;
	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneTrack>()))
					{
						if (UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
						{
							for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlName : SectionItem.AnimLayerSet.Names)
							{
								if (ControlRigTrack->GetSectionToKey(ControlName.Key) == SectionItem.Section.Get())
								{
									if (CurrentVal.IsSet() && CurrentVal.GetValue() != ECheckBoxState::Checked)
									{
										CurrentVal = ECheckBoxState::Undetermined;
									}
									if (CurrentVal.IsSet() == false)
									{
										CurrentVal = ECheckBoxState::Checked;
									}
								}
								else
								{
									if (CurrentVal.IsSet() && CurrentVal.GetValue() != ECheckBoxState::Unchecked)
									{
										CurrentVal = ECheckBoxState::Undetermined;
									}
									if (CurrentVal.IsSet() == false)
									{
										CurrentVal = ECheckBoxState::Unchecked;
									}
								}
							}
						}
						else
						{
							if (Track->GetSectionToKey() == SectionItem.Section.Get() ||
								(Track->GetAllSections().Num() == 1 && (Track->GetAllSections()[0] == SectionItem.Section.Get())))
							{
								if (CurrentVal.IsSet() && CurrentVal.GetValue() != ECheckBoxState::Checked)
								{
									CurrentVal = ECheckBoxState::Undetermined;
								}
								if (CurrentVal.IsSet() == false)
								{
									CurrentVal = ECheckBoxState::Checked;
								}
							}
							else
							{
								if (CurrentVal.IsSet() && CurrentVal.GetValue() != ECheckBoxState::Unchecked)
								{
									CurrentVal = ECheckBoxState::Undetermined;
								}
								if (CurrentVal.IsSet() == false)
								{
									CurrentVal = ECheckBoxState::Unchecked;
								}
							}
						}
					}
				}
			}
		}
	}
	if (CurrentVal.IsSet())
	{
		State.bKeyed = CurrentVal.GetValue();
		return CurrentVal.GetValue();
	}
	//has no sections and is base so it's keyed, else it's not
	return State.Type == EAnimLayerType::Base ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void UAnimLayer::SetSectionToKey() const
{

	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneTrack>()))
					{
						Track->Modify();
						if (UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
						{
							for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlName : SectionItem.AnimLayerSet.Names)
							{
								ControlRigTrack->SetSectionToKey(SectionItem.Section.Get(), ControlName.Key);
							}
						}
						else
						{
							Track->SetSectionToKey(SectionItem.Section.Get());
						}
					}
				}
			}
		}
	}
}

void UAnimLayer::SetKeyed()
{
	State.bKeyed = ECheckBoxState::Checked;
	SetSectionToKey();

}

bool UAnimLayer::GetActive() const
{
	TOptional<bool> CurrentVal;
	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneTrack>()))
					{
						const bool bActive = !Track->IsRowEvalDisabled(SectionItem.Section->GetRowIndex());

						//make sure other track rows match state, and if active the sections are active
						if (CurrentVal.IsSet() && CurrentVal.GetValue() != bActive)
						{
							Track->SetRowEvalDisabled(!CurrentVal.GetValue(), SectionItem.Section->GetRowIndex());
						}
						//make sure section is active
						if (bActive  && SectionItem.Section->IsActive() == false)
						{
							SectionItem.Section->SetIsActive(bActive);
						}
						if (CurrentVal.IsSet() == false)
						{
							CurrentVal = bActive;
						}
					}
				}
			}
		}
	}
	if (CurrentVal.IsSet())
	{
		State.bActive = CurrentVal.GetValue();
	}
	return State.bActive;
}

void UAnimLayer::PostLoad()
{
	Super::PostLoad();

	TWeakObjectPtr<UAnimLayer> WeakThis = this;

	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([WeakThis, this]()
		{
			if (WeakThis.IsValid())
			{
				const bool bTrackRowDisabled = !WeakThis->State.bActive;
				for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : WeakThis->AnimLayerItems)
				{
					if (Pair.Key != nullptr)
					{
						for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
						{
							if (SectionItem.Section.IsValid())
							{
								if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneTrack>()))
								{ 
									const int32 SectionRowIndex = SectionItem.Section->GetRowIndex();
									bool bTrackModified = false;
									if (Track->IsRowEvalDisabled(SectionItem.Section->GetRowIndex()) != bTrackRowDisabled)
									{
										bTrackModified = true;
										Track->Modify();
										Track->SetRowEvalDisabled(bTrackRowDisabled, SectionRowIndex);
									}
									//make sure all of the sections are active
									const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
									if (Sections.Num() > 0)
									{
										TArray<UMovieSceneSection*> SectionsToDelete;
										for (UMovieSceneSection* Section : Sections)
										{
											if (Section)
											{
												if (Section != SectionItem.Section.Get() && Section->GetRowIndex() == SectionRowIndex)
												{
													SectionsToDelete.Add(Section);
												}
												else if (Section->IsActive() != true)
												{
													if (Section->TryModify())
													{
														Section->SetIsActive(true);
													}
												}
											}
										}
										if (SectionsToDelete.Num() > 0)
										{
											if (bTrackModified == false)
											{
												Track->Modify();
											}
											for (UMovieSceneSection* SectionToDelete : SectionsToDelete)
											{
												Track->RemoveSection(*SectionToDelete);
											}
										}
									}
								}
							}
						}
					}
				}
			}
		}
	));
}

void UAnimLayer::SetActive(bool bInActive)
{
	const FScopedTransaction Transaction(LOCTEXT("SetActive_Transaction", "Set Active"), !GIsTransacting);
	Modify();
	State.bActive = bInActive;
	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneTrack>()))
					{
						Track->Modify();
						Track->SetRowEvalDisabled(!State.bActive, SectionItem.Section->GetRowIndex());
						if (State.bActive)
						{
							//make sure to turn on, may be off from legacy
							SectionItem.Section->Modify();
							SectionItem.Section.Get()->SetIsActive(State.bActive);
						}

						if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
						{
							using namespace UE::Sequencer;
							FOutlinerSelection& SelectedOutlinerItems = SequencerPtr->GetViewModel()->GetSelection()->Outliner;

							FTrackRowModelStorageExtension* TrackRowModelStorage = SequencerPtr->GetViewModel()->GetRootModel()->CastDynamic<FTrackRowModelStorageExtension>();
							check(TrackRowModelStorage);

							const UControlRig* ControlRig = Cast<UControlRig>(Pair.Key.Get());
							if (ControlRig)
							{
								if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
								{
									for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlName : SectionItem.AnimLayerSet.Names)
									{
										if (State.bActive && ControlRig->IsControlSelected(ControlName.Key) == false)
										{
											continue; //don't select it if not selected
										}
										FChannelMapInfo* pChannelIndex = CRSection->ControlChannelMap.Find(ControlName.Key);
										if (pChannelIndex != nullptr)
										{
											if (pChannelIndex->ParentControlIndex == INDEX_NONE)
											{
												int32 CategoryIndex = CRSection->GetActiveCategoryIndex(ControlName.Key);
												if (CategoryIndex != INDEX_NONE)
												{
													SequencerPtr->SelectByNthCategoryNode(CRSection, CategoryIndex, State.bActive);
												}
											}
										}
									}
								}
							}
							if (State.bActive == false || ControlRig == nullptr) //if not a control rig we select the whole trackrow, always make sure to deselect it
							{
								const int32 RowIndex = SectionItem.Section->GetRowIndex();
								if (TSharedPtr<FTrackRowModel> TrackRowModel = TrackRowModelStorage->FindModelForTrackRow(Track, RowIndex))
								{
									if (State.bActive)
									{
										if (ControlRig == nullptr && RowIndex != 0) //first deselect the 0th track row model which is the parent, yes this will deselect the actor but that is okay for now
										{
											if (TSharedPtr<FTrackRowModel> ParentTrackRwtModel = TrackRowModelStorage->FindModelForTrackRow(Track, 0))
											{
												SelectedOutlinerItems.Deselect(ParentTrackRwtModel);
											}
										}
										SelectedOutlinerItems.Select(TrackRowModel);

									}
									else
									{
										SelectedOutlinerItems.Deselect(TrackRowModel);
									}
								}
							}
						}

					}
				}
			}
		}
	}
}

bool UAnimLayer::AddSelectedInSequencer() 
{
	using namespace UE::AIE;

	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (SequencerPtr.IsValid() == false)
	{
		return false;
	}
	TArray<FControlRigAndControlsAndTrack> SelectedCRs;
	TArray<FObjectAndTrack> SelectedBoundObjects;
	FSequencerSelected::GetSelectedControlRigsAndBoundObjects(SequencerPtr.Get(), SelectedCRs, SelectedBoundObjects);
	if (SelectedCRs.Num() <= 0 && SelectedBoundObjects.Num() <= 0)
	{
		return false;
	}
	bool bAddedSomething = false;
	const FScopedTransaction Transaction(LOCTEXT("AddSelectedAnimLayer_Transaction", "Add Selected"),!GIsTransacting);
	Modify();
	for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
	{
		if (FAnimLayerItem* ExistingAnimLayerItem = AnimLayerItems.Find(CRControls.ControlRig))
		{
			for (FAnimLayerSectionItem& SectionItem : ExistingAnimLayerItem->SectionItems)
			{
				if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section))
				{
					for (FName& ControlName : CRControls.Controls)
					{
						if (SectionItem.AnimLayerSet.Names.Contains(ControlName) == false)
						{
							FAnimLayerPropertyAndChannels Channels;
							Channels.Name = ControlName;
							Channels.Channels = (uint32)EControlRigContextChannelToKey::AllTransform;
							SectionItem.AnimLayerSet.Names.Add(ControlName, Channels);
						}
					}
					TArray<FName> AllControls;
					SectionItem.AnimLayerSet.Names.GenerateKeyArray(AllControls);
					UAnimLayers::SetUpControlRigSection(CRSection, AllControls);
					bAddedSomething = true;
				}
			}
		}
		else
		{
			//add new section
			FAnimLayerItem AnimLayerItem;
			FAnimLayerSectionItem SectionItem;
			SectionItem.AnimLayerSet.BoundObject = CRControls.ControlRig;
			for (FName& ControlName : CRControls.Controls)
			{
				FAnimLayerPropertyAndChannels Channels;
				Channels.Name = ControlName;
				Channels.Channels = (uint32)EControlRigContextChannelToKey::AllTransform;
				SectionItem.AnimLayerSet.Names.Add(ControlName, Channels);
			}
			// Add a new section that starts and ends at the same time
			TGuardValue<bool> GuardSetSection(CRControls.Track->bSetSectionToKeyPerControl, false);
			if (UMovieSceneControlRigParameterSection* NewSection = Cast<UMovieSceneControlRigParameterSection>(CRControls.Track->CreateNewSection()))
			{
				ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
				NewSection->SetFlags(RF_Transactional);
				NewSection->SetTransformMask(FMovieSceneTransformMask{ EMovieSceneTransformChannel::All });
				FMovieSceneFloatChannel* FloatChannel = &NewSection->Weight;
				SectionItem.Section = NewSection;
				AnimLayerItem.SectionItems.Add(SectionItem);
				AnimLayerItems.Add(CRControls.ControlRig, AnimLayerItem);
				UAnimLayers::SetUpSectionDefaults(SequencerPtr.Get(), this, CRControls.Track, NewSection,FloatChannel);
				UAnimLayers::SetUpControlRigSection(NewSection, CRControls.Controls);
				bAddedSomething = true;
			}
		}
	}

	for (FObjectAndTrack& ObjectAndTrack : SelectedBoundObjects)
	{
		FAnimLayerItem& AnimLayerItem = AnimLayerItems.FindOrAdd(ObjectAndTrack.BoundObject);
		AnimLayerItem.SequencerGuid = ObjectAndTrack.SequencerGuid;
		FAnimLayerSectionItem SectionItem;
		SectionItem.AnimLayerSet.BoundObject = ObjectAndTrack.BoundObject;
		/* if we ever support channels
		for (FName& ControlName : SelectedControls)
		{
			FAnimLayerPropertyAndChannels Channels;
			Channels.Name = ControlName;
			Channels.Channels = (uint32)EControlRigContextChannelToKey::AllTransform;
			AnimLayerItem.AnimLayerSet.Names.Add(ControlName, Channels);

		}*/
		// Add a new section that starts and ends at the same time
		ObjectAndTrack.Track->Modify();
		if (UMovieSceneSection* NewSection = ObjectAndTrack.Track->CreateNewSection())
		{
			ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
			NewSection->SetFlags(RF_Transactional);
			FMovieSceneFloatChannel* FloatChannel = nullptr;
			if (UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(NewSection))
			{
				TransformSection->SetMask(FMovieSceneTransformMask{ EMovieSceneTransformChannel::All });
				FloatChannel = TransformSection->GetWeightChannel();
			}
			SectionItem.Section = NewSection;
			AnimLayerItem.SectionItems.Add(SectionItem);
			UAnimLayers::SetUpSectionDefaults(SequencerPtr.Get(), this, ObjectAndTrack.Track, NewSection, FloatChannel);
			bAddedSomething = true;
		}
	}
	if (bAddedSomething)
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			AnimLayers->SetUpBaseLayerSections();
		}
		SetKeyed();
	}
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	return true;
}

void UAnimLayer::GetSections(TArray<UMovieSceneSection*>& OutSections) const
{
	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					OutSections.Add(SectionItem.Section.Get());
				}
			}
		}
	}
}

void UAnimLayer::SetSelectedInList(const FAnimLayersScopedSelection& ScopedSelection, bool bInValue)
{
	if (bIsSelectedInList != bInValue)
	{
		bIsSelectedInList = bInValue;
	}
}

void UAnimLayer::ApplyControlSelectionToSequencer(bool bSelect) const
{
	using namespace UE::Sequencer;

	if (TSharedPtr<ISequencer> Sequencer = UAnimLayers::GetSequencerFromAsset())
	{
		for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
		{
			if (!Pair.Key.IsValid())
			{
				continue;
			}
			
			const UControlRig* ControlRig = Cast<UControlRig>(Pair.Key.Get());			
			if (!ControlRig)
			{
				continue;
			}
			
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (!SectionItem.Section.IsValid())
				{
					continue;
				}

				UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get());				
				if (!CRSection)
				{
					continue;
				}

				UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(CRSection->GetTypedOuter<UMovieSceneTrack>());
				if (!CRTrack)
				{
					continue;
				}

				for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlEntry : SectionItem.AnimLayerSet.Names)
				{
					const FName& ControlName = ControlEntry.Key;

					// When selecting, only show currently selected controls.
					// When deselecting, clear all, control selection may have changed since the last select pass.
					if (bSelect && 
						!ControlRig->IsControlSelected(ControlName))
					{
						continue;
					}

					// Only select sections to key
					if (CRTrack->GetSectionToKey(ControlName) != CRSection)
					{
						continue;
					}

					// Only select root controls, SelectByNthCategoryNode selects children implicitly
					FChannelMapInfo* ChannelMapInfo = CRSection->ControlChannelMap.Find(ControlName);
					if (!ChannelMapInfo || 
						ChannelMapInfo->ParentControlIndex != INDEX_NONE)
					{
						continue;
					}

					const int32 CategoryIndex = CRSection->GetActiveCategoryIndex(ControlName);
					if (CategoryIndex != INDEX_NONE)
					{
						Sequencer->SelectByNthCategoryNode(CRSection, CategoryIndex, bSelect);
					}
				}
			}
		}
	}
}

bool UAnimLayer::HasSectionsInCurrentEditorSelection() const
{
	TArray<AActor*> SelectedActors;
	USelection* EditorSelection = GEditor ? GEditor->GetSelectedActors() : nullptr;
	if (EditorSelection)
	{
		EditorSelection->GetSelectedObjects<AActor>(SelectedActors);
	}

	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& LayerItem : AnimLayerItems)
	{
		if (!LayerItem.Key.IsValid())
		{
			continue;
		}

		for (const FAnimLayerSectionItem& SectionItem : LayerItem.Value.SectionItems)
		{
			UMovieSceneSection* Section = SectionItem.Section.Get();
			if (!Section)
			{
				continue;
			}

			UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Section->GetTypedOuter<UMovieSceneControlRigParameterTrack>();
			if (ControlRigParameterTrack)
			{
				const UControlRig* ControlRig = ControlRigParameterTrack->GetControlRig();
				if (!ControlRig)
				{
					continue;
				}

				for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlName : SectionItem.AnimLayerSet.Names)
				{
					if (ControlRig->IsControlSelected(ControlName.Key))
					{
						return true;
					}
				}
			}
			else if (SectionItem.AnimLayerSet.BoundObject.IsValid())
			{
				if (AActor* OuterActor = SectionItem.AnimLayerSet.BoundObject->GetTypedOuter<AActor>())
				{
					if (SelectedActors.Contains(OuterActor))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}
bool UAnimLayer::RemoveAnimLayerItem(UObject* InObject)
{
	if (FAnimLayerItem* Item =  AnimLayerItems.Find(InObject))
	{
		for (const FAnimLayerSectionItem& SectionItem : Item->SectionItems)
		{
			if (SectionItem.Section.IsValid())
			{
				if (UMovieSceneTrack* Track = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>())
				{
					if (Track->GetAllSections().Find(SectionItem.Section.Get()) != 0)
					{
						Track->Modify();
						Track->RemoveSection(*(SectionItem.Section.Get()));
					}
				}
			}
		}
		AnimLayerItems.Remove(InObject);
		return true;
	}
	return false;
	
}
bool UAnimLayer::RemoveSelectedInSequencer() 
{
	using namespace UE::AIE;

	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (SequencerPtr.IsValid() == false)
	{
		return false;
	}

	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
	{
		if (AnimLayers->AnimLayers[0] == this)
		{
			return false;
		}
	}

	TArray<FControlRigAndControlsAndTrack> SelectedCRs;
	TArray<FObjectAndTrack> SelectedBoundObjects;
	FSequencerSelected::GetSelectedControlRigsAndBoundObjects(SequencerPtr.Get(), SelectedCRs, SelectedBoundObjects);
	if (SelectedCRs.Num() <= 0 && SelectedBoundObjects.Num() < 0)
	{
		return false;
	}
	bool bRemovedSomething = false;
	UAnimLayer& AnimLayer = *this;
	const FScopedTransaction Transaction(LOCTEXT("RemoveSelected_Transaction", "Remove Selected"), !GIsTransacting);
	Modify();
	for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
	{
		if (FAnimLayerItem* ExistingAnimLayerItem = AnimLayer.AnimLayerItems.Find(CRControls.ControlRig))
		{
			for (FAnimLayerSectionItem& SectionItem : ExistingAnimLayerItem->SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section))
					{
						for (FName& ControlName : CRControls.Controls)
						{
							if (SectionItem.AnimLayerSet.Names.Contains(ControlName) == true)
							{
								SectionItem.AnimLayerSet.Names.Remove(ControlName);
							}
							TArray<FName> ControlNames;
							SectionItem.AnimLayerSet.Names.GetKeys(ControlNames);
							UAnimLayers::SetUpControlRigSection(CRSection, ControlNames);
							bRemovedSomething = true;
						}
					}
					if (SectionItem.AnimLayerSet.Names.Num() == 0)
					{
						AnimLayer.RemoveAnimLayerItem(CRControls.ControlRig);
						bRemovedSomething = true;
						break;
					}
				}
			}
		}
	}
	for (FObjectAndTrack& ObjectAndTrack : SelectedBoundObjects)
	{
		if (AnimLayer.AnimLayerItems.Find(ObjectAndTrack.BoundObject))
		{
			AnimLayer.RemoveAnimLayerItem(ObjectAndTrack.BoundObject);
			bRemovedSomething = true;
		}
	}
	if (bRemovedSomething)
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			AnimLayers->SetUpBaseLayerSections();
		}
	}
	if (AnimLayer.AnimLayerItems.Num() == 0)
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			int32 Index = AnimLayers->GetAnimLayerIndex(this);
			if (Index != INDEX_NONE)
			{
				AnimLayers->DeleteAnimLayer(SequencerPtr.Get(), Index);
			}
		}
	}
	
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemRemoved);
	return true;

}

void UAnimLayer::SelectWeightChannel(bool bClearSelection)
{
	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (!SequencerPtr)
	{
		return;
	}
	if (bClearSelection)
	{
		SequencerPtr->EmptySelection();
	}
	const TArray<FName> WeightChannelNames = { FName("Weight") };
	for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (!Pair.Key.IsValid())
		{
			continue;
		}
		for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
		{
			if (!SectionItem.Section.IsValid())
			{
				continue;
			}
			UMovieSceneSection* Section = SectionItem.Section.Get();
			if (Cast<UMovieSceneControlRigParameterSection>(Section) || Cast<UMovieScene3DTransformSection>(Section))
			{
				SequencerPtr->SelectByChannels(Section, WeightChannelNames, false, true);
			}
		}
	}
}

void UAnimLayer::GetAnimLayerObjects(FAnimLayerObjects& InLayerObjects) const
{
	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (Pair.Key->IsA<UControlRig>())
					{
						if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
						{
							FAnimLayerControlRigObject ControlRigObject;
							ControlRigObject.ControlRig = Cast<UControlRig>(Pair.Key.Get());
							for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlName : SectionItem.AnimLayerSet.Names)
							{
								ControlRigObject.ControlNames.Add(ControlName.Key);
							}
							InLayerObjects.ControlRigObjects.Add(ControlRigObject);
						}
					}
					else if (IsAccepableNonControlRigObject(Pair.Key.Get()))
					{
						FAnimLayerSceneObject SceneObject;
						SceneObject.SceneObjectOrComponent = Pair.Key;
						InLayerObjects.SceneObjects.Add(SceneObject);
					}
				}
			}
		}
	}
}

bool UAnimLayer::IsAccepableNonControlRigObject(UObject* InObject)
{
	return (InObject->IsA<AActor>() || InObject->IsA<USceneComponent>());
}

ECheckBoxState UAnimLayer::GetSelected() const
{
	TSet<UObject*> SelectedObjects; 
	TMap<UControlRig*, TArray<FName>> SelectedControls;
	return GetSelected(SelectedObjects, SelectedControls);
}

ECheckBoxState UAnimLayer::GetSelected(TSet<UObject*>& OutSelectedObjects, TMap<UControlRig*, TArray<FName>>& OutSelectedControls) const
{
	FAnimLayerObjects LayerObjects;
	GetAnimLayerObjects(LayerObjects);
	TOptional<ECheckBoxState>  SelectionState;
	for (const FAnimLayerControlRigObject& ControlRigObject : LayerObjects.ControlRigObjects)
	{
		if (ControlRigObject.ControlRig.IsValid())
		{
			TArray<FName> SelectedControls = ControlRigObject.ControlRig->CurrentControlSelection();
			for (const FName& ControlName : ControlRigObject.ControlNames)
			{
				if (SelectedControls.Contains(ControlName))
				{
					if (TArray<FName>* ControlsArray = OutSelectedControls.Find(ControlRigObject.ControlRig.Get()))
					{
						ControlsArray->Add(ControlName);
					}
					else
					{
						TArray<FName>& ControlsRefArray = OutSelectedControls.Add(ControlRigObject.ControlRig.Get());
						ControlsRefArray.Add(ControlName);
					}
					if (SelectionState.IsSet() == false)
					{
						SelectionState = ECheckBoxState::Checked;
					}
					else if (SelectionState.GetValue() != ECheckBoxState::Checked)
					{
						SelectionState = ECheckBoxState::Undetermined;
					}
				}
				else
				{
					if (SelectionState.IsSet() == false)
					{
						SelectionState = ECheckBoxState::Unchecked;
					}
					else if (SelectionState.GetValue() != ECheckBoxState::Unchecked)
					{
						SelectionState = ECheckBoxState::Undetermined;
					}
				}
			}
		}
		else
		{
			if (SelectionState.IsSet())
			{
				if (SelectionState.GetValue() == ECheckBoxState::Checked)
				{
					SelectionState = ECheckBoxState::Undetermined;
				}
			}
			else
			{
				SelectionState = ECheckBoxState::Unchecked;
			}
		}
	}
	USelection* ComponentSelection = GEditor->GetSelectedComponents();
	TArray<TWeakObjectPtr<UObject>> SelectedComponents;
	ComponentSelection->GetSelectedObjects(SelectedComponents);
	USelection* ActorSelection = GEditor->GetSelectedActors();
	TArray<TWeakObjectPtr<UObject>> SelectedActors;
	ActorSelection->GetSelectedObjects(SelectedActors);

	for (const FAnimLayerSceneObject& SceneObject : LayerObjects.SceneObjects)
	{
		if (SceneObject.SceneObjectOrComponent.IsValid() &&
			SceneObject.SceneObjectOrComponent->IsA<AActor>())
		{
			if (SelectedActors.Contains(SceneObject.SceneObjectOrComponent))
			{
				OutSelectedObjects.Add(SceneObject.SceneObjectOrComponent.Get());
				if (SelectionState.IsSet() == false)
				{
					SelectionState = ECheckBoxState::Checked;
				}
				else if (SelectionState.GetValue() != ECheckBoxState::Checked)
				{
					SelectionState = ECheckBoxState::Undetermined;
				}
			}
			else
			{
				if (SelectionState.IsSet() == false)
				{
					SelectionState = ECheckBoxState::Unchecked;
				}
				else if (SelectionState.GetValue() != ECheckBoxState::Unchecked)
				{
					SelectionState = ECheckBoxState::Undetermined;
				}
			}
		}
		else if (SceneObject.SceneObjectOrComponent.IsValid() &&
			SceneObject.SceneObjectOrComponent->IsA<USceneComponent>())
		{
			if (SelectedComponents.Contains(SceneObject.SceneObjectOrComponent))
			{
				OutSelectedObjects.Add(SceneObject.SceneObjectOrComponent.Get());
				if (SelectionState.IsSet() == false)
				{
					SelectionState = ECheckBoxState::Checked;
				}
				else if (SelectionState.GetValue() != ECheckBoxState::Checked)
				{
					SelectionState = ECheckBoxState::Undetermined;
				}
			}
			else
			{
				if (SelectionState.IsSet() == false)
				{
					SelectionState = ECheckBoxState::Unchecked;
				}
				else if (SelectionState.GetValue() != ECheckBoxState::Unchecked)
				{
					SelectionState = ECheckBoxState::Undetermined;
				}
			}

		}
	}
	if (SelectionState.IsSet())
	{
		return SelectionState.GetValue();
	}
	return ECheckBoxState::Unchecked;
}

void UAnimLayer::SetSelected(bool bInSelected, bool bClearSelection)
{
	if (!GEditor)
	{
		return;
	}
	FAnimLayerObjects LayerObjects;
	GetAnimLayerObjects(LayerObjects);
	if (LayerObjects.ControlRigObjects.Num() <= 0 && LayerObjects.SceneObjects.Num() <= 0)
	{
		return;
	}
	const FScopedTransaction Transaction(LOCTEXT("SetSelected_Transaction", "Set Selection"), !GIsTransacting);
	Modify();
	State.bSelected = bInSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	if (bClearSelection)
	{
		if ((GEditor->GetSelectedActorCount() || GEditor->GetSelectedComponentCount()))
		{
			GEditor->SelectNone(false, true);
			GEditor->NoteSelectionChange();
		}

		TArray<UControlRig*> ControlRigs;
		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (ControlRigEditMode)
		{
			ControlRigs = ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/);
			for (UControlRig* ControlRig : ControlRigs)
			{
				ControlRig->ClearControlSelection();
			}
		}
		if (bInSelected == false)
		{
			return; //clearing and not selecting so we are done.
		}
	}

	for (const FAnimLayerSceneObject& SceneObject : LayerObjects.SceneObjects)
	{
		if (AActor* Actor = Cast<AActor>(SceneObject.SceneObjectOrComponent.Get()))
		{
			GEditor->SelectActor(Actor, bInSelected, true);
		}
		else if (UActorComponent* Component = Cast<UActorComponent>(SceneObject.SceneObjectOrComponent.Get()))
		{
			GEditor->SelectComponent(Component, bInSelected, true);
		}
	}

	for (const FAnimLayerControlRigObject& ControlRigObject : LayerObjects.ControlRigObjects)
	{
		if (ControlRigObject.ControlRig.IsValid())
		{
			for (const FName& ControlName : ControlRigObject.ControlNames)
			{
				ControlRigObject.ControlRig->SelectControl(ControlName, bInSelected);
			}
		}
	}	
}

bool UAnimLayer::GetLock() const
{
	TOptional<bool> CurrentVal;
	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					const bool bIsLocked = SectionItem.Section.Get()->IsLocked();
					if (CurrentVal.IsSet() && CurrentVal.GetValue() != bIsLocked)
					{
						SectionItem.Section.Get()->SetIsLocked(CurrentVal.GetValue());
					}
					if (CurrentVal.IsSet() == false)
					{
						CurrentVal = bIsLocked;
					}
				}
			}
		}
	}
	if (CurrentVal.IsSet())
	{
		State.bLock = CurrentVal.GetValue();
	}
	return State.bLock;
}

void UAnimLayer::SetLock(bool bInLock)
{
	const FScopedTransaction Transaction(LOCTEXT("SetLock_Transaction", "Set Lock"), !GIsTransacting);
	Modify();
	State.bLock = bInLock;
	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{

					SectionItem.Section->Modify();
					SectionItem.Section.Get()->SetIsLocked(State.bLock);
				}
			}
		}
	}
}

FText UAnimLayer::GetName() const
{
	return State.Name;
}

void UAnimLayer::SetName(const FText& InName)
{
	const FScopedTransaction Transaction(LOCTEXT("SetName_Transaction", "Set Name"), !GIsTransacting);
	Modify();
	State.Name = InName;

	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneNameableTrack>()))
					{
						NameableTrack->Modify();
						NameableTrack->SetTrackRowDisplayName(State.Name, SectionItem.Section.Get()->GetRowIndex());
					}
				}
			}
		}
	}
}

double UAnimLayer::GetWeight() const
{
	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (SequencerPtr)
	{
		TOptional<float> DifferentWeightValue;
		for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
		{
			if (Pair.Key != nullptr)
			{
				for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
				{
					if (SectionItem.Section.IsValid())
					{
						FMovieSceneFloatChannel* FloatChannel = nullptr;
						if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
						{
							FloatChannel = &CRSection->Weight;
						}
						else if (UMovieScene3DTransformSection* LayerSection = Cast<UMovieScene3DTransformSection>(SectionItem.Section.Get()))
						{
							FloatChannel = LayerSection->GetWeightChannel();
						}
						if (FloatChannel)
						{
							const FFrameNumber CurrentTime = SequencerPtr->GetLocalTime().Time.FloorToFrame();
							float Value = 0.f;
							FloatChannel->Evaluate(CurrentTime, Value);
							if (State.Weight != Value || WeightProxy->Weight != Value)
							{
								DifferentWeightValue = Value;
							}

						}
					}
				}
			}
		}
		if (DifferentWeightValue.IsSet())
		{
			State.Weight = DifferentWeightValue.GetValue();
			WeightProxy->Weight = State.Weight;
		}
	}
	return State.Weight;
}

static void SetFloatWeightValue(float InValue, ISequencer * Sequencer, UMovieSceneSection* OwningSection,FMovieSceneFloatChannel* Channel)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;


	if (!OwningSection)
	{
		return;
	}
	OwningSection->SetFlags(RF_Transactional);;

	if (!OwningSection->TryModify() || !Channel || !Sequencer)
	{
		return;
	}

	const bool  bAutoSetTrackDefaults = Sequencer->GetAutoSetTrackDefaults();	
	const FFrameNumber CurrentTime = Sequencer->GetLocalTime().Time.FloorToFrame();

	EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(Channel, CurrentTime, Sequencer->GetKeyInterpolation());

	TArray<FKeyHandle> KeysAtCurrentTime;
	Channel->GetKeys(TRange<FFrameNumber>(CurrentTime), nullptr, &KeysAtCurrentTime);

	if (KeysAtCurrentTime.Num() > 0)
	{
		AssignValue(Channel, KeysAtCurrentTime[0], InValue);
	}
	else
	{
		bool bHasAnyKeys = Channel->GetNumKeys() != 0;

		if (bHasAnyKeys || bAutoSetTrackDefaults == false)
		{
			// When auto setting track defaults are disabled, add a key even when it's empty so that the changed
			// value is saved and is propagated to the property.
			AddKeyToChannel(Channel, CurrentTime, InValue, Interpolation);
			bHasAnyKeys = Channel->GetNumKeys() != 0;
		}

		if (bHasAnyKeys)
		{
			TRange<FFrameNumber> KeyRange = TRange<FFrameNumber>(CurrentTime);
			TRange<FFrameNumber> SectionRange = OwningSection->GetRange();

			if (!SectionRange.Contains(KeyRange))
			{
				OwningSection->SetRange(TRange<FFrameNumber>::Hull(KeyRange, SectionRange));
			}
		}
	}

	// Always update the default value when auto-set default values is enabled so that the last changes
	// are always saved to the track.
	if (bAutoSetTrackDefaults)
	{
		SetChannelDefault(Channel, InValue);
	}

	Channel->AutoSetTangents();
	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

void UAnimLayer::SetWeight(double InWeight)
{
	State.Weight = InWeight;
	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (SequencerPtr)
	{
		for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
		{
			if (Pair.Key != nullptr)
			{
				for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
				{
					if (SectionItem.Section.IsValid())
					{
						if (SectionItem.Section.Get()->TryModify())
						{
							FMovieSceneFloatChannel* FloatChannel = nullptr;
							if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
							{
								FloatChannel = &CRSection->Weight;
							}
							else if (UMovieScene3DTransformSection* LayerSection = Cast<UMovieScene3DTransformSection>(SectionItem.Section.Get()))
							{
								FloatChannel = LayerSection->GetWeightChannel();
							}
							if (FloatChannel)
							{
								float WeightValue = (float)InWeight;
								SetFloatWeightValue(WeightValue, SequencerPtr.Get(), SectionItem.Section.Get(), FloatChannel);
							}
						}
					}
				}
			}
		}
	}
}

void UAnimLayerWeightProxy::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
#if WITH_EDITOR
	if (UAnimLayer* AnimLayer = GetTypedOuter<UAnimLayer>())
	{
		if (PropertyChangedEvent.Property && (
			(PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet) ||
			(PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) ||
			(PropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault) ||
			(PropertyChangedEvent.ChangeType & EPropertyChangeType::Unspecified))
			)
		{
			//set values
			FProperty* Property = PropertyChangedEvent.Property;
			FProperty* MemberProperty = nullptr;
			if ((Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimLayerWeightProxy, Weight)) ||
				(MemberProperty && MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAnimLayerWeightProxy, Weight)))
			{
				Modify();
				AnimLayer->SetWeight(Weight);
			}
		}
	}
#endif
}

#if WITH_EDITOR
void UAnimLayerWeightProxy::PostEditUndo()
{
	if (UAnimLayer* AnimLayer = GetTypedOuter<UAnimLayer>())
	{
		AnimLayer->SetWeight(Weight);
	}
}
#endif
EAnimLayerType UAnimLayer::GetType() const
{
	TOptional<EMovieSceneBlendType> CurrentVal;
	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					const EMovieSceneBlendType BlendType = SectionItem.Section.Get()->GetBlendType().BlendType;
					if (CurrentVal.IsSet() && CurrentVal.GetValue() != BlendType)
					{
						SectionItem.Section.Get()->SetBlendType(CurrentVal.GetValue());
					}
					if (CurrentVal.IsSet() == false)
					{
						CurrentVal = BlendType;
					}
				}
			}
		}
	}
	if (CurrentVal.IsSet())
	{
		switch (CurrentVal.GetValue())
		{
		case EMovieSceneBlendType::Additive:
			State.Type = (int32)EAnimLayerType::Additive;
			break;
		case EMovieSceneBlendType::Override:
			State.Type = (int32)EAnimLayerType::Override;
			break;
		case EMovieSceneBlendType::Absolute:
			State.Type = (int32)EAnimLayerType::Base;
			break;
		}
	}

	return (EAnimLayerType)(State.Type);
}

static void SetDefaultsForOverride(UMovieSceneSection* InSection)
{
	if (InSection->IsA<UMovieSceneControlRigParameterSection>())
	{
		return; //control rig sections already handle this
	}
	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (SequencerPtr.IsValid() == false)
	{
		return;
	}
	FFrameNumber FrameNumber = SequencerPtr->GetLocalTime().Time.GetFrame();
	if (UMovieSceneTrack* OwnerTrack = InSection->GetTypedOuter<UMovieSceneTrack>())
	{
		TArray<UMovieSceneSection*> TrackSections = OwnerTrack->GetAllSections();
		int32 SectionIndex = TrackSections.Find((InSection));
		if (SectionIndex != INDEX_NONE)
		{
			InSection->Modify();
			TrackSections.SetNum(SectionIndex); //this will gives us up to the section 
			TArray<UMovieSceneSection*> Sections;
			TArray<UMovieSceneSection*> AbsoluteSections;
			MovieSceneToolHelpers::SplitSectionsByBlendType(EMovieSceneBlendType::Absolute, TrackSections, Sections, AbsoluteSections);
			TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = InSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			if (BaseDoubleChannels.Num() > 0)
			{
				int32 NumChannels = BaseDoubleChannels.Num();
				const int32 StartIndex = 0;
				const int32 EndIndex = NumChannels - 1;
				TArray<double> ChannelValues = MovieSceneToolHelpers::GetChannelValues<FMovieSceneDoubleChannel,
					double>(StartIndex,EndIndex, Sections, AbsoluteSections, FrameNumber);
				for (int32 Index = 0; Index < NumChannels; ++Index)
				{
					FMovieSceneDoubleChannel* DoubleChannel = BaseDoubleChannels[Index];
					const double Value = ChannelValues[Index];
					DoubleChannel->SetDefault(Value);
				}
			}
			else if (BaseFloatChannels.Num() > 0)
			{
				int32 NumChannels = BaseFloatChannels.Num();
				const int32 StartIndex = 0;
				const int32 EndIndex = NumChannels - 1;
				TArray<float> ChannelValues = MovieSceneToolHelpers::GetChannelValues<FMovieSceneFloatChannel,
					float>(StartIndex, EndIndex, Sections, AbsoluteSections, FrameNumber);
				for (int32 Index = 0; Index < NumChannels; ++Index)
				{
					FMovieSceneFloatChannel* FloatChannel = BaseFloatChannels[Index];
					const float Value = ChannelValues[Index];
					FloatChannel->SetDefault(Value);
				}
			}
			SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
		}
	}
}

void UAnimLayer::SetType(EAnimLayerType LayerType)
{
	const FScopedTransaction Transaction(LOCTEXT("SetType_Transaction", "Set Type"), !GIsTransacting);
	Modify();

	State.Type = (uint32)(LayerType);
	for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
			{
				if (SectionItem.Section.IsValid())
				{
					switch (LayerType)
					{
					case EAnimLayerType::Additive:
						SectionItem.Section.Get()->SetBlendType(EMovieSceneBlendType::Additive);
						break;
					case EAnimLayerType::Override:
						SectionItem.Section.Get()->SetBlendType(EMovieSceneBlendType::Override);
						SetDefaultsForOverride(SectionItem.Section.Get());
						break;
					case EAnimLayerType::Base:
						SectionItem.Section.Get()->SetBlendType(EMovieSceneBlendType::Absolute);
						break;
					}
				}
			}
		}
	}
}

UAnimLayers::UAnimLayers(class FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

#if WITH_EDITOR
void UAnimLayers::PostEditUndo()
{
	Super::PostEditUndo();
	SetAnimLayerIndices(false /*bResetSectionOrders*/);
	TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>> TracksAndOrderedSections;
	GetTracksAndSections(TracksAndOrderedSections);
	SetTrackRowForSections(TracksAndOrderedSections);
	AnimLayerListChangedBroadcast();
}
#endif

void UAnimLayers::PostLoad()
{
	Super::PostLoad();
	SetAnimLayerIndices(false /*bResetSectionOrders*/);
	TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>> TracksAndOrderedSections;
	GetTracksAndSections(TracksAndOrderedSections);
	SetTrackRowForSections(TracksAndOrderedSections);
}

void UAnimLayers::GetTracksAndSections(TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>>& OutTracksAndOrderedSections)
{
	for (UAnimLayer* AnimLayer : AnimLayers)
	{
		if (AnimLayer == nullptr)
		{
			continue;
		}
		for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
		{
			if (Pair.Key != nullptr)
			{
				for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
				{
					if (SectionItem.Section.IsValid())
					{
						if (UMovieSceneTrack* Track = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>())
						{
							TArray<TPair<UAnimLayer*, UMovieSceneSection*>>& TrackSections = OutTracksAndOrderedSections.FindOrAdd(Track);
							TPair<UAnimLayer*, UMovieSceneSection*> NameSection;
							NameSection.Key = AnimLayer;
							NameSection.Value = SectionItem.Section.Get();
							TrackSections.Add(NameSection);
						}
					}
				}
			}
		}
	}
}

void UAnimLayers::SetTrackRowForSections(const TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>>& TracksAndOrderedSections)
{
	for (const TPair<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>>& TrackSections : TracksAndOrderedSections)
	{
		if (UMovieSceneTrack* Track = TrackSections.Key)
		{
			UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track);
			int32 Index = 0;
			for (const TPair<UAnimLayer*, UMovieSceneSection*>& Pair : TrackSections.Value)
			{
				Pair.Value->SetOverlapPriority(Index);
				Pair.Value->SetRowIndex(Index);
				if (NameableTrack)
				{
					NameableTrack->SetTrackRowDisplayName(Pair.Key->GetName(), Index);
				}
				Track->SetRowEvalDisabled(!Pair.Key->State.bActive, Index);
				++Index;
			}
			Track->UpdateEasing();
		}
	}
}


void UAnimLayers::SetAnimLayerIndices(bool bResetSectionOrders)
{
	for (int32 Index = 0; Index < AnimLayers.Num(); ++Index)
	{
		if (UAnimLayer* AnimLayer = AnimLayers[Index].Get())
		{
			AnimLayer->Modify();
			AnimLayer->LayerIndex = Index;
		}
	}
	if (bResetSectionOrders)
	{
		//go through and cache out the sections in order, we need to do this since there may be missing sections
		//since every layer doesn't contain every section 
		
		TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*,UMovieSceneSection*>>> TracksAndOrderedSections;
		GetTracksAndSections(TracksAndOrderedSections);
		//now go through and set up tracks
		TArray<UMovieSceneSection*> Sections;
		int32 Index = 0;
		for (TPair<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>>& TrackSections : TracksAndOrderedSections)
		{
			if (UMovieSceneTrack* Track = TrackSections.Key)
			{
				Track->Modify();
				UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track);
				
				Sections.SetNum(TrackSections.Value.Num());
				Index = 0;
				for (TPair<UAnimLayer*, UMovieSceneSection*>& Pair : TrackSections.Value)
				{
					Sections[Index++] = Pair.Value;
				}
				MovieSceneHelpers::ResetSectionOrder(Track, Sections);
			}
		}
		SetTrackRowForSections(TracksAndOrderedSections);
	}
}

bool UAnimLayers::ReorderAnimLayers(ISequencer* SequencerPtr, int32 OriginalIndex, int32 NewIndex)
{
	//can't move over  or move base index
	if (NewIndex == 0 || OriginalIndex == 0 || SequencerPtr == nullptr)
	{
		return false;
	}
	if(OriginalIndex != NewIndex && AnimLayers.IsValidIndex(OriginalIndex) && AnimLayers.IsValidIndex(NewIndex))
	{ 
		const FScopedTransaction Transaction(LOCTEXT("ReorderLayers_Transaction", "Reorder Layers"), !GIsTransacting);
		Modify();
		UAnimLayer* AnimLayer = AnimLayers[OriginalIndex];
		AnimLayers.RemoveAt(OriginalIndex);
		AnimLayers.Insert(AnimLayer, NewIndex);
		//reset layer indices need to go through and set every
		SetAnimLayerIndices(true /*bResetSectionOrders*/);
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemRemoved);
		AnimLayerListChangedBroadcast();
		return true;
	}
	return false;
}


TSharedPtr<ISequencer> UAnimLayers::GetSequencerFromAsset()
{
	ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, false);
	ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
	TSharedPtr<ISequencer> SequencerPtr = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	return SequencerPtr;
}

void UAnimLayers::AddBaseLayer()
{
	UAnimLayer* AnimLayer = NewObject<UAnimLayer>(this, TEXT("BaseLayer"), RF_Transactional);
	AnimLayer->State.Type = EAnimLayerType::Base;
	AnimLayer->SetKeyed();
	AnimLayers.Add(AnimLayer);
	SetAnimLayerIndices(false /*bResetSectionOrders*/);
}

bool UAnimLayers::HasAnimLayers(ISequencer* InSequencer)
{
	if (!InSequencer)
	{
		return false;
	}
	if(ULevelSequence* LevelSequence = Cast<ULevelSequence>(InSequencer->GetFocusedMovieSceneSequence()))
	{ 
		if (LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
		{
			if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
			{
				if (UAnimLayers* AnimLayers = AssetUserDataInterface->GetAssetUserData< UAnimLayers >())
				{
					return true;
				}
			}
		}
	}
	return false;
}

UAnimLayers* UAnimLayers::GetAnimLayers(ISequencer* SequencerPtr, bool bAddIfDoesNotExist)
{
	if (!SequencerPtr)
	{
		return nullptr;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	return UAnimLayers::GetAnimLayers(LevelSequence,bAddIfDoesNotExist);
}

UAnimLayers* UAnimLayers::GetAnimLayers(ULevelSequence* LevelSequence, bool bAddIfDoesNotExist)
{
	if(!LevelSequence)
	{
		return nullptr;
	}
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			UAnimLayers* AnimLayers = AssetUserDataInterface->GetAssetUserData< UAnimLayers >();
			if (!AnimLayers && bAddIfDoesNotExist)
			{
				AnimLayers = NewObject<UAnimLayers>(LevelSequence, NAME_None, RF_Public | RF_Transactional);
				AssetUserDataInterface->AddAssetUserData(AnimLayers);
			}
			return AnimLayers;
		}
	}
	return nullptr;
}

int32 UAnimLayers::GetAnimLayerIndex(UAnimLayer* InAnimLayer) const
{
	if (InAnimLayer != nullptr)
	{
		return AnimLayers.Find(InAnimLayer);
	}
	return INDEX_NONE;
}

bool UAnimLayers::DeleteAnimLayer(ISequencer* SequencerPtr, int32 Index)
{
	if (Index >= 1 && Index < AnimLayers.Num()) //don't delete base
	{
		if (UAnimLayer* AnimLayer = AnimLayers[Index])
		{
			const FScopedTransaction Transaction(LOCTEXT("DeleteAnimLayer_Transaction", "Delete Anim Layer"), !GIsTransacting);
			Modify();
			for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
			{
				if (Pair.Key != nullptr)
				{
					for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
					{
						if (SectionItem.Section.IsValid())
						{
							if (UMovieSceneTrack* Track = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>())
							{
								if (Track->GetAllSections().Find(SectionItem.Section.Get()) != 0)
								{
									Track->Modify();
									Track->RemoveSection(*(SectionItem.Section.Get()));
								}
							}
						}
					}
				}
			}
			AnimLayers.RemoveAt(Index);
			SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemRemoved);
		}
		if (UAnimLayer* BaseAnimLayer = AnimLayers[0]) //set base as keyed
		{
			BaseAnimLayer->SetKeyed();
		}
		SetAnimLayerIndices(false /*bResetSectionOrders*/);
		AnimLayerListChangedBroadcast();
	}
	else
	{
		return false;
	}
	return true;
}

static void CopySectionIntoAnother(UMovieSceneSection* ToSection, UMovieSceneSection* FromSection)
{
	const FFrameNumber Min = TNumericLimits<FFrameNumber>::Lowest();
	const FFrameNumber Max = TNumericLimits<FFrameNumber>::Max();
	TRange<FFrameNumber> Range(Min, Max);
	TArray<UMovieSceneSection*> AbsoluteSections;
	TArray<UMovieSceneSection*> AdditiveSections;

	AdditiveSections.Add(ToSection);
	AdditiveSections.Add(FromSection);
	

	FMovieSceneChannelProxy& ChannelProxy = ToSection->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : ToSection->GetChannelProxy().GetAllEntries())
	{
		const FName ChannelTypeName = Entry.GetChannelTypeName();

		if (ChannelTypeName == FMovieSceneFloatChannel::StaticStruct()->GetFName())
		{
			TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = ToSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			const int32 StartIndex = 0;
			const int32 EndOffset = ToSection->IsA<UMovieSceneControlRigParameterSection>() ? 2: 1; //if CR section skip weight
			const int32 EndIndex = BaseFloatChannels.Num() - EndOffset;
			MovieSceneToolHelpers::MergeSections<FMovieSceneFloatChannel>(ToSection, AbsoluteSections, AdditiveSections, 
				StartIndex, EndIndex, Range);
		}
		else if (ChannelTypeName == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
		{
			TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = ToSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			const int32 StartIndex = 0;
			const int32 EndIndex = BaseDoubleChannels.Num() - 1;
			MovieSceneToolHelpers::MergeSections<FMovieSceneDoubleChannel>(ToSection, AbsoluteSections, AdditiveSections, 
				StartIndex, EndIndex, Range);
		}
	}
}

//get the set of sections active, or with this control name in it
static void GetValidCRSections(UMovieSceneSection* InSection, const FName& ControlName, TArray<UMovieSceneSection*>& OutSections)
{
	if (UMovieSceneTrack* OwnerTrack = InSection->GetTypedOuter<UMovieSceneTrack>())
	{
		TArray<UMovieSceneSection*> TrackSections = OwnerTrack->GetAllSections();
		for (UMovieSceneSection* Section : TrackSections)
		{
			if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
			{
				if (Section && (Section->IsActive() || Section == InSection) &&
					CRSection->GetControlNameMask(ControlName))
				{
					OutSections.Add(Section);
				}
			}
		}
	}
}

static void GetValidSections(UMovieSceneSection* InSection, TArray<UMovieSceneSection*>& OutSections)
{
	if (UMovieSceneTrack* OwnerTrack = InSection->GetTypedOuter<UMovieSceneTrack>())
	{
		TArray<UMovieSceneSection*> TrackSections = OwnerTrack->GetAllSections();
		for (UMovieSceneSection* Section : TrackSections)
		{
			if (Section && (Section->IsActive() || Section == InSection))
			{
				OutSections.Add(Section);
			}
		}
	}
}
static void RevertWeightChannelToOne(FMovieSceneFloatChannel* FloatChannel, const TRange<FFrameNumber>& FrameRange)
{
	if (FloatChannel)
	{
		//don't add key if there!
		TArray<FFrameNumber> KeyTimes;
		TArray<FKeyHandle> KeyHandles;
		FloatChannel->GetKeys(FrameRange, &KeyTimes, &KeyHandles);
		if (KeyTimes.Num() > 0)
		{
			FloatChannel->DeleteKeys(KeyHandles);
			if (FloatChannel->GetNumKeys() > 0) //if it still has keys see if the weight is not one 
			{
				float StartValue = 0.0, EndValue = 0.0;
				FFrameTime StartFrameTime = FrameRange.HasLowerBound() ? FrameRange.GetLowerBoundValue() : FFrameTime(KeyTimes[0]);
				FFrameTime EndFrameTime = FrameRange.HasUpperBound() ? FrameRange.GetUpperBoundValue() : FFrameTime(KeyTimes[KeyTimes.Num() -1]);
				FloatChannel->Evaluate(StartFrameTime, StartValue);
				FloatChannel->Evaluate(EndFrameTime, EndValue);
				//if not zero at boundaries set one keys there
				if (FMath::IsNearlyEqual(StartValue, 1.0f) == false ||
					FMath::IsNearlyEqual(EndValue, 1.0f) == false)
				{
					FloatChannel->AddCubicKey(StartFrameTime.FrameNumber, 1.0f, ERichCurveTangentMode::RCTM_SmartAuto);
					FloatChannel->AddCubicKey(EndFrameTime.FrameNumber, 1.0f, ERichCurveTangentMode::RCTM_SmartAuto);
				}
			}
			else
			{
				FloatChannel->SetDefault(1.0f);
			}
		}
	}
}

namespace UE::AIE
{ 
// disable section if no control in the section and override, need to make sure we skip it's eval or it will incorrectly
//override when merging
static bool ShouldDisableSection(UMovieSceneControlRigParameterSection* CRSection, FRigControlElement* ControlElement)
{
	return (CRSection->GetControlNameMask(ControlElement->GetFName()) == false &&
		CRSection->GetBlendType().IsValid() && CRSection->GetBlendType() == EMovieSceneBlendType::Override);
}

static void MergeControlRigSections(UMovieSceneControlRigParameterSection* BaseSection, UMovieSceneControlRigParameterSection* Section, const TRange<FFrameNumber>& Range,
	const int32* Increment)
{

	if (!BaseSection || !Section)
	{
		return;
	}
	TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = BaseSection->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
	if (BaseFloatChannels.Num() > 0)
	{
		//need to go through each control and merge that
		TArray<FRigControlElement*> Controls;
		UControlRig* ControlRig = BaseSection->GetControlRig();
		if (ControlRig == nullptr)
		{
			return;
		}
		ControlRig->GetControlsInOrder(Controls);
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		if (Hierarchy == nullptr)
		{
			return;
		}

		BaseSection->Modify();

		const bool bIsOverride = (Section->GetBlendType().IsValid() && Section->GetBlendType() == EMovieSceneBlendType::Override) ||
			(BaseSection->GetBlendType().IsValid() && BaseSection->GetBlendType() == EMovieSceneBlendType::Override);


		for (int32 LocalControlIndex = 0; LocalControlIndex < Controls.Num(); ++LocalControlIndex)
		{
			FRigControlElement* ControlElement = Controls[LocalControlIndex];
			check(ControlElement);
			if (!Hierarchy->IsAnimatable(ControlElement))
			{
				continue;
			}

			const FName& ControlName = ControlElement->GetFName();

			FChannelMapInfo* pTopChannelIndex = Section->ControlChannelMap.Find(ControlName);
			if (!pTopChannelIndex)
			{
				continue;
			}
			if (FChannelMapInfo* pChannelIndex = BaseSection->ControlChannelMap.Find(ControlName))
			{
				//these may be different due to rig changes
				const int32 ChannelIndex = pChannelIndex->ChannelIndex;
				const int32 TopChannelIndex = pTopChannelIndex->ChannelIndex;

				//if override we mask out if both are masked out, if not override we mask out if the top section is masked out
				const bool bMaskedOutOfBase = BaseSection->GetControlNameMask(ControlName) == false;
				const bool bMaskedOutOfSection = Section->GetControlNameMask(ControlName) == false;
				const bool bMaskKeyOut = bIsOverride ? (bMaskedOutOfBase && bMaskedOutOfSection)
					: bMaskedOutOfSection;
				if (bMaskKeyOut)
				{
					continue;
				}
				TArray<UMovieSceneSection*> TrackSections;
				GetValidCRSections(BaseSection, ControlName, TrackSections);

				TOptional<bool> bBaseSectionResetActive;
				TOptional<bool> bSectionResetActive;
				if (ShouldDisableSection(BaseSection, ControlElement))
				{
					bBaseSectionResetActive = BaseSection->IsActive();
					BaseSection->SetIsActive(false);
				}

				if (ShouldDisableSection(Section, ControlElement))
				{
					bSectionResetActive = Section->IsActive();
					Section->SetIsActive(false);
				}


				switch (ControlElement->Settings.ControlType)
				{

				case ERigControlType::Float:
				case ERigControlType::ScaleFloat:
				{
					int32 StartIndex = ChannelIndex;
					int32 EndIndex = ChannelIndex;
					int32 TopEndIndex = TopChannelIndex;
					MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
						Section, StartIndex, EndIndex, TopChannelIndex, TopEndIndex);
					MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);
					break;
				}
				case ERigControlType::Vector2D:
				{
					int32 StartIndex = ChannelIndex;
					int32 EndIndex = ChannelIndex + 1;
					int32 TopEndIndex = TopChannelIndex + 1;
					MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
						Section, StartIndex, EndIndex, TopChannelIndex, TopEndIndex);
					MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

					break;
				}
				case ERigControlType::Position:
				case ERigControlType::Scale:
				case ERigControlType::Rotator:
				{
					int32 StartIndex = ChannelIndex;
					int32 EndIndex = ChannelIndex + 2;
					int32 TopEndIndex = TopChannelIndex + 2;
					MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
						Section, StartIndex, EndIndex, TopChannelIndex, TopEndIndex);
					MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

					break;
				}

				case ERigControlType::Transform:
				case ERigControlType::TransformNoScale:
				case ERigControlType::EulerTransform:
				{
					EMovieSceneTransformChannel BaseChannelMask = BaseSection->GetTransformMask().GetChannels();
					EMovieSceneTransformChannel ChannelMask = Section->GetTransformMask().GetChannels();

					const bool bDoAllTransform = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::AllTransform);
					if (bDoAllTransform)
					{
						if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
						{
							int32 StartIndex = ChannelIndex;
							int32 EndIndex = ChannelIndex + 5;
							int32 TopEndIndex = TopChannelIndex + 5;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopChannelIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);
						}
						else
						{
							int32 StartIndex = ChannelIndex;
							int32 EndIndex = ChannelIndex + 8;
							int32 TopEndIndex = TopChannelIndex + 8;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopChannelIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);
						}
					}
					else
					{
						if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::TranslationX)
							&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationX))
						{
							int32 StartIndex = ChannelIndex;
							int32 EndIndex = StartIndex;
							int32 TopStartIndex = TopChannelIndex;
							int32 TopEndIndex = TopStartIndex;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

						}
						if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::TranslationY)
							&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationY))
						{
							int32 StartIndex = ChannelIndex + 1;
							int32 EndIndex = StartIndex;
							int32 TopStartIndex = TopChannelIndex + 1;
							int32 TopEndIndex = TopStartIndex;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

						}
						if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::TranslationZ)
							&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationZ))
						{
							int32 StartIndex = ChannelIndex + 2;
							int32 EndIndex = StartIndex;
							int32 TopStartIndex = TopChannelIndex + 2;
							int32 TopEndIndex = TopStartIndex;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

						}
						if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::RotationX)
							&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationX))
						{
							int32 StartIndex = ChannelIndex + 3;
							int32 EndIndex = StartIndex;
							int32 TopStartIndex = TopChannelIndex + 3;
							int32 TopEndIndex = TopStartIndex;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

						}
						if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::RotationY)
							&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationY))
						{
							int32 StartIndex = ChannelIndex + 4;
							int32 EndIndex = StartIndex;
							int32 TopStartIndex = TopChannelIndex + 4;
							int32 TopEndIndex = TopStartIndex;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

						}
						if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::RotationZ)
							&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationZ))
						{
							int32 StartIndex = ChannelIndex + 5;
							int32 EndIndex = StartIndex;
							int32 TopStartIndex = TopChannelIndex + 5;
							int32 TopEndIndex = TopStartIndex;
							MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
								Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
							MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

						}
						if (ControlElement->Settings.ControlType != ERigControlType::TransformNoScale)
						{
							if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::ScaleX)
								&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleX))
							{
								int32 StartIndex = ChannelIndex + 6;
								int32 EndIndex = StartIndex;
								int32 TopStartIndex = TopChannelIndex + 6;
								int32 TopEndIndex = TopStartIndex;
								MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
									Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
								MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

							}
							if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::ScaleY)
								&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleY))
							{
								int32 StartIndex = ChannelIndex + 7;
								int32 EndIndex = StartIndex;
								int32 TopStartIndex = TopChannelIndex + 7;
								int32 TopEndIndex = TopStartIndex;
								MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
									Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
								MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

							}
							if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::ScaleZ)
								&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleZ))
							{
								int32 StartIndex = ChannelIndex + 8;
								int32 EndIndex = StartIndex;
								int32 TopStartIndex = TopChannelIndex + 8;
								int32 TopEndIndex = TopStartIndex;
								MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
									Section, StartIndex, EndIndex, TopStartIndex, TopEndIndex);
								MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);

							}
						}
					}
					break;
				}
				default:
					break;
				}
				if (bBaseSectionResetActive.IsSet())
				{
					BaseSection->SetIsActive(bBaseSectionResetActive.GetValue());
				}
				if (bSectionResetActive.IsSet())
				{
					Section->SetIsActive(bSectionResetActive.GetValue());
				}
			}
		}
		if (BaseSection && BaseSection->GetBlendType() == EMovieSceneBlendType::Override)
		{
			if (FMovieSceneFloatChannel* FloatChannel = &BaseSection->Weight)
			{
				RevertWeightChannelToOne(FloatChannel, Range);
			}
		}
	}
}

static void MergeTransformSections(UMovieScene3DTransformSection* BaseSection, UMovieScene3DTransformSection* Section, const TRange<FFrameNumber>& Range, const int32* Increment)
{
	if (!BaseSection || !Section)
	{
		return;
	}
	TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = BaseSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
	if (BaseDoubleChannels.Num() > 0)
	{
		EMovieSceneTransformChannel BaseChannelMask = BaseSection->GetMask().GetChannels();
		EMovieSceneTransformChannel ChannelMask = Section->GetMask().GetChannels();
		BaseSection->Modify();
		TArray<UMovieSceneSection*> TrackSections;
		GetValidSections(BaseSection, TrackSections);
		const bool bDoAllTransform = EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::AllTransform);
		if (bDoAllTransform)
		{
			int32 StartIndex = 0;
			int32 EndIndex = BaseDoubleChannels.Num() - 1;
			MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
				Section, StartIndex, EndIndex, StartIndex, EndIndex);
			MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
		}
		else
		{
			const int32 ChannelIndex = 0;
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::TranslationX)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationX))
			{
				int32 StartIndex = ChannelIndex;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::TranslationY)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationY))
			{
				int32 StartIndex = ChannelIndex + 1;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::TranslationZ)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::TranslationZ))
			{
				int32 StartIndex = ChannelIndex + 2;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::RotationX)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationX))
			{
				int32 StartIndex = ChannelIndex + 3;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::RotationY)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationY))
			{
				int32 StartIndex = ChannelIndex + 4;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::RotationZ)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::RotationZ))
			{
				int32 StartIndex = ChannelIndex + 5;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::ScaleX)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleX))
			{
				int32 StartIndex = ChannelIndex + 6;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::ScaleY)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleY))
			{
				int32 StartIndex = ChannelIndex + 7;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
			if (EnumHasAllFlags(BaseChannelMask, EMovieSceneTransformChannel::ScaleZ)
				&& EnumHasAllFlags(ChannelMask, EMovieSceneTransformChannel::ScaleZ))
			{
				int32 StartIndex = ChannelIndex + 8;
				int32 EndIndex = StartIndex;
				MovieSceneToolHelpers::FBaseTopSections BT(BaseSection,
					Section, StartIndex, EndIndex, StartIndex, EndIndex);
				MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);
			}
		}
		if (BaseSection && BaseSection->GetBlendType() == EMovieSceneBlendType::Override)
		{
			if (FMovieSceneFloatChannel* FloatChannel = BaseSection->GetWeightChannel())
			{
				RevertWeightChannelToOne(FloatChannel, Range);
			}
		}
	}
}
}

int32 UAnimLayers::DuplicateAnimLayer(ISequencer* SequencerPtr, int32 Index)
{
	using namespace UE::AIE;
	int32 NewIndex = INDEX_NONE;
	if (Index >= 1 && Index < AnimLayers.Num()) //don't duplicate base
	{
		if (UAnimLayer* ExistingAnimLayer = AnimLayers[Index])
		{
			if (ExistingAnimLayer->AnimLayerItems.Num() == 0)
			{
				UE_LOGF(LogControlRig, Error, "Anim Layers: Can not duplicate empty layer");
				return INDEX_NONE;
			}
			TRangeBound<FFrameNumber> OpenBound;
			TRange<FFrameNumber> InfiniteRange(OpenBound, OpenBound);

			const FScopedTransaction Transaction(LOCTEXT("DuplicateAnimLayer_Transaction", "Duplicate Anim Layer"), !GIsTransacting);
			Modify();
			UAnimLayer* NewAnimLayer = NewObject<UAnimLayer>(this, NAME_None, RF_Transactional);
			NewAnimLayer->SetType(ExistingAnimLayer->GetType());
			bool bItemAdded = false;
			for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : ExistingAnimLayer->AnimLayerItems)
			{
				if (Pair.Key != nullptr)
				{
					for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
					{
						if (SectionItem.Section.IsValid())
						{
							if (UMovieSceneTrack* Track = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>())
							{
								Track->Modify();
								if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
								{
									if (UControlRig* ControlRig = Cast<UControlRig>(Pair.Key))
									{
										FAnimLayerItem AnimLayerItem;
										FAnimLayerSectionItem NewSectionItem;
										NewSectionItem.AnimLayerSet.BoundObject = ControlRig;
										NewSectionItem.AnimLayerSet = SectionItem.AnimLayerSet;
										// Add a new section that starts and ends at the same time
										if (UMovieSceneControlRigParameterTrack* CRTack = Cast<UMovieSceneControlRigParameterTrack>(Track))
										{
											TGuardValue<bool> GuardSetSection(CRTack->bSetSectionToKeyPerControl, false);
											if (UMovieSceneControlRigParameterSection* NewSection = Cast<UMovieSceneControlRigParameterSection>(Track->CreateNewSection()))
											{
												if (bItemAdded == false)
												{
													NewAnimLayer->State.Weight = 1.0;
													NewAnimLayer->State.Type = (EAnimLayerType::Additive);
													bItemAdded = true;
												}
												ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
												NewSection->SetFlags(RF_Transactional);
												NewSection->SetTransformMask(FMovieSceneTransformMask{ EMovieSceneTransformChannel::All });
												FMovieSceneFloatChannel* FloatChannel = &NewSection->Weight;
												NewSectionItem.Section = NewSection;
												AnimLayerItem.SectionItems.Add(NewSectionItem);
												NewAnimLayer->AnimLayerItems.Add(ControlRig, AnimLayerItem);
												SetUpSectionDefaults(SequencerPtr, NewAnimLayer, Track, NewSection, FloatChannel);
												NewSection->SetBlendType(Section->GetBlendType().Get());
												TArray<FName> ControlNames;
												NewSectionItem.AnimLayerSet.Names.GetKeys(ControlNames);
												SetUpControlRigSection(NewSection, ControlNames);
												//current copy keys
												int32* Increment = nullptr;
												MergeControlRigSections(NewSection, Section, InfiniteRange, Increment);
											}
										}
									}
								}
								else
								{
									FAnimLayerItem AnimLayerItem;
									FAnimLayerSectionItem NewSectionItem;
									NewSectionItem.AnimLayerSet.BoundObject = Pair.Key;
									if (UMovieSceneSection* NewSection = Track->CreateNewSection())
									{
										if (bItemAdded == false)
										{
											NewAnimLayer->State.Weight = 1.0;
											NewAnimLayer->State.Type = (EAnimLayerType::Additive);
											bItemAdded = true;
										}

										ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
										NewSection->SetFlags(RF_Transactional);
										FMovieSceneFloatChannel* FloatChannel = nullptr;
										if (UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(NewSection))
										{
											TransformSection->SetMask(FMovieSceneTransformMask{ EMovieSceneTransformChannel::All });
											FloatChannel = TransformSection->GetWeightChannel();
										}
										NewSectionItem.Section = NewSection;
										AnimLayerItem.SectionItems.Add(NewSectionItem);
										NewAnimLayer->AnimLayerItems.Add(Pair.Key, AnimLayerItem);
										SetUpSectionDefaults(SequencerPtr, NewAnimLayer, Track, NewSection, FloatChannel);
										NewSection->SetBlendType(SectionItem.Section->GetBlendType().Get());
										//current copy keys
										CopySectionIntoAnother(NewSection, SectionItem.Section.Get());
									}
								}
							}
						}
					}
				}
			}
			if (bItemAdded)
			{
				FString ExistingName = ExistingAnimLayer->GetName().ToString();
				FString NewLayerName = FString::Printf(TEXT("%s_Duplicate"), *ExistingName);

				FText LayerText;
				LayerText = LayerText.FromString(NewLayerName);
				NewAnimLayer->SetName(LayerText); //need items/sections to be added so we can change their track row names
				NewIndex = AnimLayers.Add(NewAnimLayer);
				NewAnimLayer->SetKeyed();
				AnimLayerListChangedBroadcast();
				// no need to since it's a dup SetUpBaseLayerSections();
			}
		}
	}
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	SetAnimLayerIndices(false /*bResetSectionOrders*/);
	return NewIndex;
}

static void AddNamesToMask(FAnimLayerSectionItem* Owner, UMovieSceneControlRigParameterSection* CRSection,const FAnimLayerSelectionSet& NewSet)
{
	bool bNameAdded = false;
	for (TPair<FName, FAnimLayerPropertyAndChannels> NameAndChannels : NewSet.Names)
	{
		if (Owner->AnimLayerSet.Names.Contains(NameAndChannels.Key) == false)
		{
			bNameAdded = true;
			Owner->AnimLayerSet.Names.Add(NameAndChannels);
		}
	}
	if (bNameAdded)
	{
		TArray<FName> AllControls;
		Owner->AnimLayerSet.Names.GenerateKeyArray(AllControls);
		UAnimLayers::SetUpControlRigSection(CRSection, AllControls);
	}
}

void FAnimLayerItem::SetSectionsActive(bool bIsActive)
{
	for (FAnimLayerSectionItem& SectionItem : SectionItems)
	{
		if (SectionItem.Section.IsValid())
		{
			SectionItem.Section->SetIsActive(bIsActive);
		}
	}
}

FAnimLayerSectionItem* FAnimLayerItem::FindMatchingSectionItem(UMovieSceneSection* InMovieSceneSection)
{
	if (InMovieSceneSection)
	{
		for (FAnimLayerSectionItem& CurrentItem : SectionItems)
		{
			if (CurrentItem.Section.IsValid())
			{
				UMovieSceneTrack* InTrack = InMovieSceneSection->GetTypedOuter<UMovieSceneTrack>();
				UMovieSceneTrack* CurrentTrack = CurrentItem.Section->GetTypedOuter<UMovieSceneTrack>();
				if (CurrentTrack && InTrack && CurrentTrack == InTrack)
				{
					return &CurrentItem;
				}
			}
		}
	}
	return nullptr;
}

namespace UE::AIE 
{
	template<typename ChannelType, typename ValueType>
	void SetPassthroughKeys(TArrayView<ChannelType*> Channels, int32 StartIndex, int32 EndIndex, EAnimLayerType BlendType, const TArray<UMovieSceneSection*>& Sections, const TArray<UMovieSceneSection*>& AbsoluteSections,
		EMovieSceneKeyInterpolation DefaultInterpolation, const FFrameNumber& FrameNumber)
	{
		switch (BlendType)
		{
		case EAnimLayerType::Override:
		{
			TArray<ValueType> ChannelValues = MovieSceneToolHelpers::GetChannelValues<ChannelType,
				ValueType>(StartIndex, EndIndex, Sections, AbsoluteSections, FrameNumber);
			int32  ChannelValueIndex = 0;
			for (int32 Index = StartIndex; Index <= EndIndex; ++Index)
			{
				ChannelType* Channel = Channels[Index];
				ValueType Value = ChannelValues[ChannelValueIndex];
				++ChannelValueIndex;
				AssigneOrSetValue(Channel, Value, FrameNumber, DefaultInterpolation);
			}
		}
		break;
		case EAnimLayerType::Additive:
		{
			const ValueType Value = 0.0;
			for (int32 Index = StartIndex; Index <= EndIndex; ++Index)
			{
				ChannelType* Channel = Channels[Index];
				AssigneOrSetValue(Channel, Value, FrameNumber, DefaultInterpolation);
			}
		}
		break;
		}
	}
}
bool UAnimLayers::SetPassthroughKey(ISequencer* InSequencer, int32 InIndex)
{
	return SetKeyValueOrPassthrough(InSequencer, InIndex, false /*IsValue*/);
}

bool UAnimLayers::SetKey(ISequencer* InSequencer, int32 InIndex)
{
	return SetKeyValueOrPassthrough(InSequencer, InIndex, true /*IsValue*/);
}



bool UAnimLayers::SetKeyValueOrPassthrough(ISequencer* InSequencer, int32 InIndex, bool bJustValue)
{
	using namespace UE::AIE;
	auto GetSelectedRigElements = [](UControlRig* ControlRig, TArray<FRigElementKey>& OutSelectedKeys) 
	{
		if (ControlRig)
		{
			TArray<FRigElementKey> SelectedRigElements = ControlRig->GetHierarchy()->GetSelectedKeys();
			if (ControlRig->IsAdditive())
			{
				// For additive rigs, ignore boolean controls
				SelectedRigElements = SelectedRigElements.FilterByPredicate([ControlRig](const FRigElementKey& Key)
					{
						if (FRigControlElement* Element = ControlRig->FindControl(Key.Name))
						{
							return Element->CanTreatAsAdditive();
						}
						return true;
					});
			}
		}

	};
	if (InSequencer == nullptr || InIndex <= -1 || InIndex >= AnimLayers.Num())
	{
		return false;
	}
	FFrameNumber FrameNumber = InSequencer->GetLocalTime().Time.GetFrame();
	EMovieSceneKeyInterpolation DefaultInterpolation = InSequencer->GetKeyInterpolation();

	FText TransactionText = bJustValue ? FText(LOCTEXT("SetKeyValue_Transaction", "Set Key")) : FText(LOCTEXT("SetPassthroughKey_Transaction", "Set Passthrough Key"));
	if (UAnimLayer* AnimLayer = AnimLayers[InIndex].Get())
	{
		TSet<UObject*> SelectedObjects;
		TMap<UControlRig*, TArray<FName>> SelectedControls;
		bool bHasSelected = AnimLayer->GetSelected(SelectedObjects,SelectedControls) == ECheckBoxState::Unchecked ? false : true;

		for (TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
		{
			if (Pair.Key != nullptr)
			{
				for (FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
				{
					if (SectionItem.Section.IsValid())
					{
						if (UMovieSceneTrack* OwnerTrack = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>())
						{
							TArray<UMovieSceneSection*> TrackSections = OwnerTrack->GetAllSections();
							int32 SectionIndex = TrackSections.Find((SectionItem.Section.Get()));
							if (SectionIndex != INDEX_NONE)
							{
								const FScopedTransaction Transaction(TransactionText, !GIsTransacting);
								SectionItem.Section->Modify();
								TrackSections.SetNum(SectionIndex); //this will gives us up to the section 
								TArray<UMovieSceneSection*> Sections;
								TArray<UMovieSceneSection*> AbsoluteSections;
								MovieSceneToolHelpers::SplitSectionsByBlendType(EMovieSceneBlendType::Absolute, TrackSections, Sections, AbsoluteSections);
								TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = SectionItem.Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
								TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = SectionItem.Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
								if (bHasSelected == false)
								{
									if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
									{
										if (BaseFloatChannels.Num() > 0 && CRSection->GetControlRig())
										{
											//passthrough and base is separate case we call edit mode function since it does lots of special case stuff
											//need to make sure the base section is section to key and then set it back if not
											if (bJustValue == false && AnimLayer->GetType() == EAnimLayerType::Base)
											{
												if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(OwnerTrack))
												{
													TMap<FName, TWeakObjectPtr<UMovieSceneSection>> Empty;
													FControlRigParameterTrackSectionToKeyRestore Restore(CRTrack, CRSection, Empty);

													FRigControlModifiedContext Context;
													Context.SetKey = EControlRigSetKey::Always;

													FControlRigEditMode::InvertInputPose(CRSection->GetControlRig(), Context, false/*selection only*/, false/* include channels*/);
												}
											}
											else //not base and passthrough do each control 
											{
												for (const TPair<FName, FAnimLayerPropertyAndChannels>& Set : SectionItem.AnimLayerSet.Names)
												{
													FName ControlName = Set.Key;
													if (FRigControlElement* Control = CRSection->GetControlRig()->FindControl(ControlName))
													{
														int32 StartIndex = 0, EndIndex = 0;
														if (FControlRigKeys::GetStartEndIndicesForControl(CRSection, Control, StartIndex, EndIndex))
														{
															if (bJustValue)
															{
																SetCurrentKeys<FMovieSceneFloatChannel, float>(BaseFloatChannels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
															}
															else
															{
																SetPassthroughKeys<FMovieSceneFloatChannel, float>(BaseFloatChannels, StartIndex, EndIndex, AnimLayer->GetType(),
																	Sections, AbsoluteSections, DefaultInterpolation, FrameNumber);
															}
														}
													}
												}
											}	
										}
									}
									else if (BaseDoubleChannels.Num() > 0)
									{
										const int32 NumChannels = BaseDoubleChannels.Num();
										const int32 StartIndex = 0;
										const int32 EndIndex = NumChannels - 1;

										if (bJustValue)
										{
											SetCurrentKeys<FMovieSceneDoubleChannel, double>(BaseDoubleChannels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
										}
										else
										{
											SetPassthroughKeys<FMovieSceneDoubleChannel, double>(BaseDoubleChannels, StartIndex, EndIndex, AnimLayer->GetType(), Sections, AbsoluteSections,
												DefaultInterpolation, FrameNumber);
										}
									}
								}
								else //bHasSelected == true
								{
									if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
									{
										for (const TPair <UControlRig*, TArray<FName>>& ControlPair : SelectedControls)
										{
											if (CRSection->GetControlRig() == ControlPair.Key)
											{
												//passthrough and base is separate case we call edit mode function since it does lots of special case stuff
												//need to make sure the base section is section to key and then set it back if not
												if (bJustValue == false && AnimLayer->GetType() == EAnimLayerType::Base)
												{
													if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(OwnerTrack))
													{
														TMap<FName, TWeakObjectPtr<UMovieSceneSection>> Empty;
														FControlRigParameterTrackSectionToKeyRestore Restore(CRTrack, CRSection, Empty);
														FRigControlModifiedContext Context;
														Context.SetKey = EControlRigSetKey::Always;

														FControlRigEditMode::InvertInputPose(CRSection->GetControlRig(), Context, true/*selection only*/, false/* include channels*/);
													}
												}
												else
												{
													for (const TPair<FName, FAnimLayerPropertyAndChannels>& Set : SectionItem.AnimLayerSet.Names)
													{
														FName ControlName = Set.Key;
														if (ControlPair.Value.Contains(ControlName))
														{
															if (FRigControlElement* Control = ControlPair.Key->FindControl(ControlName))
															{
																int32 StartIndex = 0, EndIndex = 0;
																if (FControlRigKeys::GetStartEndIndicesForControl(CRSection, Control, StartIndex, EndIndex))
																{
																	if (bJustValue)
																	{
																		SetCurrentKeys<FMovieSceneFloatChannel, float>(BaseFloatChannels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
																	}
																	else
																	{
																		SetPassthroughKeys<FMovieSceneFloatChannel, float>(BaseFloatChannels, StartIndex, EndIndex, AnimLayer->GetType(), Sections, AbsoluteSections,
																			DefaultInterpolation, FrameNumber);
																	}
																}
															}
														}
													}
												}
											}
										}
									}
									else 
									{
										for (UObject* SelectedObject : SelectedObjects)
										{
											if (SectionItem.AnimLayerSet.BoundObject == SelectedObject) //okay selected
											{
												if (BaseDoubleChannels.Num() > 0)
												{
													const int32 NumChannels = BaseDoubleChannels.Num();
													const int32 StartIndex = 0;
													const int32 EndIndex = NumChannels - 1;
													if (bJustValue)
													{
														SetCurrentKeys<FMovieSceneDoubleChannel, double>(BaseDoubleChannels, StartIndex, EndIndex, DefaultInterpolation, FrameNumber);
													}
													else
													{
														SetPassthroughKeys<FMovieSceneDoubleChannel, double>(BaseDoubleChannels, StartIndex, EndIndex, AnimLayer->GetType(), Sections, AbsoluteSections,
															DefaultInterpolation, FrameNumber);
													}
												}
											}
										}
									}
								}
								InSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
							}
						}
						else
						{
							return false;
						}
					}
				}
			}
		}
	}
	else
	{
		return false;
	}
	return true;
}

bool UAnimLayers::MergeAnimLayers(TSharedPtr<ISequencer>& InSequencerPtr, const TArray<int32>& Indices, const FMergeAnimLayerSettings* InSettings)
{
	using namespace UE::AIE;

	if (InSequencerPtr.IsValid() == false)
	{
		return false;
	}

	const UE::ControlRig::FSaveScopedControlRigSelection SaveScopedControlRigSelection(InSequencerPtr);

	ISequencer* InSequencer = InSequencerPtr.Get();

	TArray<UAnimLayer*> LayersToMerge;
	const FFrameNumber Min = TNumericLimits<FFrameNumber>::Lowest();
	const FFrameNumber Max = TNumericLimits<FFrameNumber>::Max();

	TRange<FFrameNumber> Range = InSequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
	TOptional<TRange<FFrameNumber>> OptionalRange = InSequencerPtr->GetSubSequenceRange();
	if (OptionalRange.IsSet())
	{
		Range = TRange<FFrameNumber>(OptionalRange.GetValue().GetLowerBoundValue(), OptionalRange.GetValue().GetUpperBoundValue());
	}
	FScopedTransaction Transaction(LOCTEXT("Merge Anim Layers", "Merge Anim Layers"), !GIsTransacting);

	TArray<int32> SortedIndices = Indices;
	//we go backwards to the first one
	SortedIndices.Sort([](const int32& Index1, const int32& Index2) {
		return  Index1 > Index2;
		});

	for (int32 Index : SortedIndices)
	{
		if (Index >= 0 && Index < AnimLayers.Num())
		{
			if (UAnimLayer* AnimLayer = AnimLayers[Index].Get())
			{
				LayersToMerge.Add(AnimLayer);
			}
		}
	}
	if (LayersToMerge.Num() < 1)
	{
		return false;
	}
	const FAnimLayersScopedSelection ScopedSelection(*this);

	Modify();
	//set up Increment if we are baking increments
	const int32* Increment = (InSettings && InSettings->BakingKeySettings == EBakingKeySettings::AllFrames) ? &InSettings->FrameIncrement : nullptr;
	for (int32 Index = 0; Index < LayersToMerge.Num() -1; ++Index)
	{
		UAnimLayer* BaseLayer = LayersToMerge[Index + 1];
		UAnimLayer* AnimLayer = LayersToMerge[Index];
		BaseLayer->Modify();
		AnimLayer->Modify();
		for (TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
		{
			if (Pair.Key != nullptr)
			{
				for (FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
				{
					if (SectionItem.Section.IsValid())
					{
						FAnimLayerSectionItem* BaseSectionItem = nullptr;
						if (FAnimLayerItem* Owner = BaseLayer->AnimLayerItems.Find(Pair.Key))
						{
							BaseSectionItem = Owner->FindMatchingSectionItem(SectionItem.Section.Get());

						}
						if (BaseSectionItem && BaseSectionItem->Section.IsValid())
						{
							if (SectionItem.Section->IsActive())//active sections merge them
							{
								//if transform or control rig section we need to handle masking
								if (UMovieSceneControlRigParameterSection* BaseCRSection = Cast< UMovieSceneControlRigParameterSection>(BaseSectionItem->Section.Get()))
								{
									UMovieSceneControlRigParameterSection* CRSection = Cast< UMovieSceneControlRigParameterSection>(SectionItem.Section.Get());
									MergeControlRigSections(BaseCRSection, CRSection, Range, Increment);
								}
								else if (UMovieScene3DTransformSection* BaseTRSection = Cast< UMovieScene3DTransformSection>(BaseSectionItem->Section.Get()))
								{
									UMovieScene3DTransformSection* TRSection = Cast< UMovieScene3DTransformSection>(SectionItem.Section.Get());
									MergeTransformSections(BaseTRSection, TRSection, Range, Increment);
								}
								else
								{
									TArray<UMovieSceneSection*> TrackSections;
									GetValidSections(BaseSectionItem->Section.Get(), TrackSections);
									TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = BaseSectionItem->Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
									TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = BaseSectionItem->Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
									if (BaseDoubleChannels.Num() > 0)
									{
										int32 StartIndex = 0;
										int32 EndIndex = BaseDoubleChannels.Num() - 1;

										MovieSceneToolHelpers::FBaseTopSections BT(BaseSectionItem->Section.Get(),
											SectionItem.Section.Get(), StartIndex, EndIndex, StartIndex,EndIndex);
										MovieSceneToolHelpers::MergeTwoSections<FMovieSceneDoubleChannel>(BT, Range, TrackSections, Increment);

									}
									else if (BaseFloatChannels.Num() > 0)
									{
										int32 StartIndex = 0;
										int32 EndIndex = BaseFloatChannels.Num() - 1;
										MovieSceneToolHelpers::FBaseTopSections BT(BaseSectionItem->Section.Get(),
											SectionItem.Section.Get(), StartIndex, EndIndex,StartIndex, EndIndex);
										MovieSceneToolHelpers::MergeTwoSections<FMovieSceneFloatChannel>(BT, Range, TrackSections, Increment);
									}
								}
							}
							if (BaseLayer != AnimLayers[0]) //if not base layer
							{
								if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(BaseSectionItem->Section))
								{
									if (SectionItem.AnimLayerSet.Names.Num() > 0)
									{
										AddNamesToMask(BaseSectionItem, CRSection, SectionItem.AnimLayerSet);
										if (SortedIndices[0] != 0) //if not base then make sure mask is set up
										{
											TArray<FName> AllControls;
											BaseSectionItem->AnimLayerSet.Names.GenerateKeyArray(AllControls);
											UAnimLayers::SetUpControlRigSection(CRSection, AllControls);
										}
									}
								}
							}
							//merging so do a key reduction possibly.
							if (InSettings->bReduceKeys)
							{
								FSmartReduceParams SmartParams;
								SmartParams.SampleRate = InSequencer->GetFocusedDisplayRate();
								SmartParams.TolerancePercentage = InSettings->TolerancePercentage;
								FControlRigParameterTrackEditor::SmartReduce(InSequencerPtr, SmartParams, BaseSectionItem->Section.Get());
								return true;
							}
						}
						else //okay this object doesn't exist in the first layer we are merging into so we need to move it to the other one 
						{
							FAnimLayerItem AnimLayerItem;
							FAnimLayerSectionItem NewSectionItem;
							NewSectionItem.AnimLayerSet.BoundObject = Pair.Key;
							NewSectionItem.AnimLayerSet.Names = SectionItem.AnimLayerSet.Names;
							NewSectionItem.Section = SectionItem.Section;
							if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(SectionItem.Section.Get()->GetTypedOuter< UMovieSceneNameableTrack>()))
							{
								NameableTrack->Modify();
								NameableTrack->SetTrackRowDisplayName(BaseLayer->State.Name, SectionItem.Section.Get()->GetRowIndex());
							}
							AnimLayerItem.SectionItems.Add(NewSectionItem);
							BaseLayer->AnimLayerItems.Add(Pair.Key, AnimLayerItem);
							//since we moved the section over we reset it on the merged so we don't delete it when we remove the layer
							if (FAnimLayerItem* OldAnimLayeritem = AnimLayer->AnimLayerItems.Find(Pair.Key))
							{
								SectionItem.Section.Reset();
							}
						}
					}
				}
			}
		}
		if (AnimLayer->GetType() == EAnimLayerType::Override &&
			(BaseLayer->GetType() == EAnimLayerType::Additive))
		{
			BaseLayer->SetType(EAnimLayerType::Override);
		}
		int32 LayerIndex = GetAnimLayerIndex(AnimLayer);
		if (LayerIndex != INDEX_NONE)
		{
			DeleteAnimLayer(InSequencer, LayerIndex);
		}
	}
	UAnimLayer* BaseLayer = LayersToMerge[LayersToMerge.Num() - 1];
	if (BaseLayer != AnimLayers[0]) //if not base layer
	{
		FString Merged(TEXT("Merged"));
		FString ExistingName = BaseLayer->GetName().ToString();
		if (ExistingName.Contains(Merged) == false)
		{
			FString NewLayerName = FString::Printf(TEXT("%s_Merged"), *ExistingName);
			FText LayerText;
			LayerText = LayerText.FromString(NewLayerName);
			BaseLayer->SetName(LayerText); //need items/sections to be added so we can change their track row names
		}
	}
	else
	{
		SetUpBaseLayerSections(); //if it is the base reset it
	}
	SetAnimLayerIndices(false /*bResetSectionOrders*/);

	InSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
	return true;
}

void UAnimLayers::AnimLayerListChangedBroadcast()
{
	OnAnimLayerListChanged.Broadcast(this);
}

TArray<UMovieSceneSection*> UAnimLayers::GetSelectedLayerSections() const
{
	TArray<UMovieSceneSection*> Sections;
	for (const UAnimLayer* AnimLayer : AnimLayers)
	{
		if (AnimLayer && AnimLayer->bIsSelectedInList)
		{
			for (const TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
			{
				if (Pair.Key != nullptr)
				{
					for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
					{
						if (SectionItem.Section.IsValid() )
						{
							Sections.Add(SectionItem.Section.Get());
						}
					}
				}
			}
		}
	}
	return Sections;
}

bool UAnimLayers::IsTrackOnSelectedLayer(const UMovieSceneTrack* InTrack)const
{
	TSet<UMovieSceneTrack*> CurrentTracks;
	for (const UAnimLayer* AnimLayer : AnimLayers)
	{
		if (AnimLayer && AnimLayer->bIsSelectedInList)
		{
			for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
			{
				if (Pair.Key != nullptr)
				{
					for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
					{
						if (SectionItem.Section.IsValid())
						{
							UMovieSceneTrack* OwnerTrack = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>();
							if (OwnerTrack == InTrack)
							{
								return true;
							}
						}
					}
				}
			}
		}
	}
	return false;
}

int32 UAnimLayers::AddAnimLayerFromSelection(ISequencer* SequencerPtr)
{
	using namespace UE::AIE;

	int32 NewIndex = INDEX_NONE;
	//wrap scoped transaction since it can deselect control rigs
	TArray<FControlRigAndControlsAndTrack> SelectedCRs;
	TArray<FObjectAndTrack> SelectedBoundObjects; 
	{
		const FScopedTransaction Transaction(LOCTEXT("AddAnimLayer_Transaction", "Add Anim Layer"), !GIsTransacting);
		Modify();
		if (AnimLayers.Num() == 0)
		{
			AddBaseLayer();
		}
		UAnimLayer* AnimLayer = NewObject<UAnimLayer>(this, NAME_None, RF_Transactional);

		FSequencerSelected::GetSelectedControlRigsAndBoundObjects(SequencerPtr, SelectedCRs, SelectedBoundObjects);

		if (SelectedCRs.Num() <= 0 && SelectedBoundObjects.Num() <= 0)
		{
			FString LayerName = FString::Printf(TEXT("Empty Layer %d"), AnimLayers.Num());
			FText LayerText;
			LayerText = LayerText.FromString(LayerName);
			AnimLayer->SetName(LayerText); //need items/sections to be added so we can change their track row names
			AnimLayer->State.Weight = 1.0;
			AnimLayer->State.Type = (EAnimLayerType::Additive);
			int32 Index = AnimLayers.Add(AnimLayer);
			SetAnimLayerIndices(false /*bResetSectionOrders*/);
			AnimLayerListChangedBroadcast();

			return Index;
		}

		bool bItemAdded = false;

		for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
		{
			Modify();
			FAnimLayerItem AnimLayerItem;
			FAnimLayerSectionItem SectionItem;
			SectionItem.AnimLayerSet.BoundObject = CRControls.ControlRig;
			for (FName& ControlName : CRControls.Controls)
			{
				FAnimLayerPropertyAndChannels Channels;
				Channels.Name = ControlName;
				Channels.Channels = (uint32)EControlRigContextChannelToKey::AllTransform;
				SectionItem.AnimLayerSet.Names.Add(ControlName, Channels);

			}
			CRControls.Track->Modify();
			// Add a new section that starts and ends at the same time
			TGuardValue<bool> GuardSetSection(CRControls.Track->bSetSectionToKeyPerControl, false);
			if (UMovieSceneControlRigParameterSection* NewSection = Cast<UMovieSceneControlRigParameterSection>(CRControls.Track->CreateNewSection()))
			{
				if (bItemAdded == false)
				{
					AnimLayer->State.Weight = 1.0;
					AnimLayer->State.Type = (EAnimLayerType::Additive);
					bItemAdded = true;
				}

				ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
				NewSection->SetFlags(RF_Transactional);
				NewSection->SetTransformMask(FMovieSceneTransformMask{ EMovieSceneTransformChannel::All });
				FMovieSceneFloatChannel* FloatChannel = &NewSection->Weight;
				SectionItem.Section = NewSection;
				AnimLayerItem.SectionItems.Add(SectionItem);
				AnimLayer->AnimLayerItems.Add(CRControls.ControlRig, AnimLayerItem);
				SetUpSectionDefaults(SequencerPtr, AnimLayer, CRControls.Track, NewSection, FloatChannel);
				SetUpControlRigSection(NewSection, CRControls.Controls);

			}
		}
		for (FObjectAndTrack& ObjectAndTrack : SelectedBoundObjects)
		{
			Modify();

			FAnimLayerItem& AnimLayerItem = AnimLayer->AnimLayerItems.FindOrAdd(ObjectAndTrack.BoundObject);
			FAnimLayerSectionItem SectionItem;
			SectionItem.AnimLayerSet.BoundObject = ObjectAndTrack.BoundObject;
			AnimLayerItem.SequencerGuid = ObjectAndTrack.SequencerGuid;

			/*
			for (FName& ControlName : SelectedControls)
			{
				FAnimLayerPropertyAndChannels Channels;
				Channels.Name = ControlName;
				Channels.Channels = (uint32)EControlRigContextChannelToKey::AllTransform;
				AnimLayerItem.AnimLayerSet.Names.Add(ControlName, Channels);

			}*/

			// Add a new section that starts and ends at the same time
			ObjectAndTrack.Track->Modify();
			if (UMovieSceneSection* NewSection = ObjectAndTrack.Track->CreateNewSection())
			{
				if (bItemAdded == false)
				{
					AnimLayer->State.Weight = 1.0;
					AnimLayer->State.Type = (EAnimLayerType::Additive);
					bItemAdded = true;
				}

				ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
				NewSection->SetFlags(RF_Transactional);
				SectionItem.Section = NewSection;
				FMovieSceneFloatChannel* FloatChannel = nullptr;
				if (UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(NewSection))
				{
					TransformSection->SetMask(FMovieSceneTransformMask{ EMovieSceneTransformChannel::All });
					FloatChannel = TransformSection->GetWeightChannel();
				}
				AnimLayerItem.SectionItems.Add(SectionItem);
				SetUpSectionDefaults(SequencerPtr, AnimLayer, ObjectAndTrack.Track, NewSection, FloatChannel);
			}
		}

		if (bItemAdded)
		{
			FString LayerName = FString::Printf(TEXT("Anim Layer %d"), AnimLayers.Num());
			FText LayerText;
			LayerText = LayerText.FromString(LayerName);
			AnimLayer->SetName(LayerText); //need items/sections to be added so we can change their track row names
			int32 Index = AnimLayers.Add(AnimLayer);
			SetUpBaseLayerSections();
			AnimLayer->SetKeyed();
			SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

			NewIndex = Index;
			SetAnimLayerIndices(false /*bResetSectionOrders*/);
			AnimLayerListChangedBroadcast();
		}
	}

	for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
	{
		for (const FName& Control : CRControls.Controls)
		{
			if (CRControls.ControlRig->IsControlSelected(Control) == false)
			{
				CRControls.ControlRig->SelectControl(Control, true);
			}
		}
	}

	return NewIndex;
}

static void AddSectionToAnimLayerItem(const FAnimLayerSelectionSet& CurrentAnimLayerSet, FAnimLayerItem* AnimLayerItem, UObject* BoundObject, UMovieSceneSection* InSection)
{
	FAnimLayerSectionItem NewSectionItem;
	NewSectionItem.AnimLayerSet.BoundObject = BoundObject;
	NewSectionItem.Section = InSection;
	if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(NewSectionItem.Section))
	{
		FMovieSceneFloatChannel* FloatChannel = &CRSection->Weight;
		FloatChannel->SetDefault(1.0f);
		CRSection->SetTransformMask(CRSection->GetTransformMask().GetChannels() | EMovieSceneTransformChannel::Weight);
	}
	else if (UMovieScene3DTransformSection* Section = Cast<UMovieScene3DTransformSection>(NewSectionItem.Section))
	{
		FMovieSceneFloatChannel* FloatChannel = Section->GetWeightChannel();
		FloatChannel->SetDefault(1.0f);
		Section->SetMask(Section->GetMask().GetChannels() | EMovieSceneTransformChannel::Weight);
	}
	for (const TPair<FName, FAnimLayerPropertyAndChannels>& Set : CurrentAnimLayerSet.Names)
	{
		FAnimLayerPropertyAndChannels Channels;
		Channels.Name = Set.Value.Name;
		Channels.Channels = Set.Value.Channels;
		NewSectionItem.AnimLayerSet.Names.Add(Set.Key, Channels);
	}
	AnimLayerItem->SectionItems.Add(NewSectionItem);
}

void UAnimLayers::SetUpBaseLayerSections()
{
	if (AnimLayers.Num() > 0)
	{
		if (UAnimLayer* BaseAnimLayer = AnimLayers[0])
		{
			BaseAnimLayer->Modify();
			BaseAnimLayer->AnimLayerItems.Reset(); //clear it out
			for (int32 Index = 1; Index < AnimLayers.Num(); ++Index)
			{
				UAnimLayer* AnimLayer = AnimLayers[Index];
				AnimLayer->Modify();
				for (const TPair<TWeakObjectPtr<UObject>,FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
				{
					if (Pair.Key != nullptr)
					{
						for (const FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
						{
							if (SectionItem.Section.IsValid())
							{
								if (UMovieSceneTrack* Track = SectionItem.Section->GetTypedOuter<UMovieSceneTrack>())
								{
									const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
									if (Sections.Num() > 1 && Sections[0]->GetBlendType().IsValid() && Sections[0]->GetBlendType() == EMovieSceneBlendType::Absolute)
									{
										if (FAnimLayerItem* Existing = BaseAnimLayer->AnimLayerItems.Find(Pair.Key))
										{
											if (Pair.Key->IsA<UControlRig>()) //if control rig just merge over control names
											{
												for (const TPair<FName, FAnimLayerPropertyAndChannels>& Set : SectionItem.AnimLayerSet.Names)
												{
													for (FAnimLayerSectionItem& ExistingSectionItem : Existing->SectionItems)
													{
														if (ExistingSectionItem.AnimLayerSet.Names.Find(Set.Key) == nullptr)
														{
															FAnimLayerPropertyAndChannels Channels;
															Channels.Name = Set.Value.Name;
															Channels.Channels = Set.Value.Channels;
															ExistingSectionItem.AnimLayerSet.Names.Add(Set.Key, Channels);
														}
													}
												}
											}
											else
											{
												AddSectionToAnimLayerItem(SectionItem.AnimLayerSet,Existing, Pair.Key.Get(), Sections[0]);
											}
										}
										else
										{
											FAnimLayerItem AnimLayerItem;
											AddSectionToAnimLayerItem(SectionItem.AnimLayerSet, &AnimLayerItem, Pair.Key.Get(), Sections[0]);
											BaseAnimLayer->AnimLayerItems.Add(Pair.Key, AnimLayerItem);
										}
									}
								}
							}
						}
					}
				}
			}

			// Track controls in the base layer for any control present in the absolute section
			// but not covered by any non-base layer — so the base layer is visible for those controls.
			for (TTuple<TWeakObjectPtr<UObject>, FAnimLayerItem>& Item : BaseAnimLayer->AnimLayerItems)
			{
				if (!Item.Key.IsValid() || 
					!Item.Key->IsA<UControlRig>())
				{
					continue;
				}

				for (FAnimLayerSectionItem& BaseSectionItem : Item.Value.SectionItems)
				{
					if (!BaseSectionItem.Section.IsValid())
					{
						continue;
					}
					
					UMovieSceneControlRigParameterSection* AbsSection = Cast<UMovieSceneControlRigParameterSection>(BaseSectionItem.Section.Get());
					if (!AbsSection)
					{
						continue;
					}

					for (const TPair<FName, FChannelMapInfo>& ChannelPair : AbsSection->ControlChannelMap)
					{
						if (!BaseSectionItem.AnimLayerSet.Names.Contains(ChannelPair.Key))
						{
							FAnimLayerPropertyAndChannels Channels;
							Channels.Name = ChannelPair.Key;
							Channels.Channels = (uint32)EControlRigContextChannelToKey::AllTransform;

							BaseSectionItem.AnimLayerSet.Names.Add(ChannelPair.Key, Channels);
						}
					}
				}
			}

			//set the name will set it on all base sections
			FText LayerText = BaseAnimLayer->State.Name;
			BaseAnimLayer->SetName(LayerText);
		}
	}
	else
	{
		AddBaseLayer();
	}
}

void UAnimLayers::GetAnimLayerStates(TArray<FAnimLayerState>& OutStates)
{
	OutStates.Reset();
	for (UAnimLayer* AnimLayer : AnimLayers)
	{
		if (AnimLayer)
		{
			OutStates.Emplace(AnimLayer->State);
		}
	}
}

void UAnimLayers::SetUpSectionDefaults(ISequencer* SequencerPtr, UAnimLayer* Layer, UMovieSceneTrack* Track, UMovieSceneSection* NewSection, FMovieSceneFloatChannel* WeightChannel)
{
	int32 OverlapPriority = 0;
	TMap<int32, int32> NewToOldRowIndices;
	int32 RowIndex = Track->GetMaxRowIndex() + 1;
	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		OverlapPriority = FMath::Max(Section->GetOverlapPriority() + 1, OverlapPriority);

		// Move existing sections on the same row or beyond so that they don't overlap with the new section
		if (Section != NewSection && Section->GetRowIndex() >= RowIndex)
		{
			int32 OldRowIndex = Section->GetRowIndex();
			int32 NewRowIndex = Section->GetRowIndex() + 1;
			NewToOldRowIndices.FindOrAdd(NewRowIndex, OldRowIndex);
			Section->Modify();
			Section->SetRowIndex(NewRowIndex);
		}
	}

	Track->Modify();

	Track->OnRowIndicesChanged(NewToOldRowIndices);
	NewSection->SetRange(TRange<FFrameNumber>::All());
	
	NewSection->SetOverlapPriority(OverlapPriority);
	NewSection->SetRowIndex(RowIndex);

	Track->AddSection(*NewSection);
	Track->UpdateEasing();

	if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track))
	{
		NameableTrack->SetTrackRowDisplayName(Layer->GetName(), RowIndex);
	}

	switch (Layer->State.Type)
	{
	case EAnimLayerType::Additive:
		NewSection->SetBlendType(EMovieSceneBlendType::Additive);
		break;
	case EAnimLayerType::Override:
		NewSection->SetBlendType(EMovieSceneBlendType::Override);
		SetDefaultsForOverride(NewSection);
		break;
	case EAnimLayerType::Base:
		NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
		break;
	}
	if (WeightChannel)
	{
		WeightChannel->SetDefault(1.0f);
	}
}

void UAnimLayers::SetUpControlRigSection(UMovieSceneControlRigParameterSection* ParameterSection, TArray<FName>& ControlNames)
{
	if (ParameterSection)
	{
		UControlRig* ControlRig = ParameterSection ? ParameterSection->GetControlRig() : nullptr;

		if (ParameterSection && ControlRig)
		{
			ParameterSection->Modify();
			ParameterSection->FillControlNameMask(false);
			constexpr bool bReconstructChannelProxy = false;

			TArray<FRigControlElement*> Controls;
			ControlRig->GetControlsInOrder(Controls);
			for (const FName& RigName : ControlNames)
			{
				ParameterSection->SetControlNameMask(RigName, true, bReconstructChannelProxy);
			}
			ParameterSection->CacheChannelProxy();
		}
	}
}

namespace UE
{
namespace AIE
{

struct FKeyInterval
{
	FFrameNumber StartFrame;
	double StartValue;
	FFrameNumber EndFrame;
	double EndValue;
};

template<typename ChannelType>
void GetPairs(ChannelType* Channel, TArray<FKeyInterval>& OutKeyIntervals)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	TMovieSceneChannelData<ChannelValueType> ChannelData = Channel->GetData();

	TArrayView<FFrameNumber> Times = ChannelData.GetTimes();
	TArrayView <ChannelValueType> Values = ChannelData.GetValues();
	OutKeyIntervals.SetNum(0);
	for (int32 Index = 0; Index < Times.Num() -1; ++Index)
	{
		FKeyInterval KeyInterval;
		KeyInterval.StartFrame = Times[Index];
		KeyInterval.StartValue = Values[Index].Value;
		KeyInterval.EndFrame = Times[Index + 1];
		KeyInterval.EndValue = Values[Index + 1].Value;
		OutKeyIntervals.Add(KeyInterval);
	}
}

template<typename ChannelType>
void GetFrames(ChannelType* Channel, TSet<FFrameNumber>& OutKeyTimes)
{
	using ChannelValueType = typename ChannelType::ChannelValueType;
	TMovieSceneChannelData<ChannelValueType> ChannelData = Channel->GetData();

	TArrayView<FFrameNumber> Times = ChannelData.GetTimes();
	for (int32 Index = 0; Index < Times.Num(); ++Index)
	{
		OutKeyTimes.Add(Times[Index]);
	}
}

template<typename ChannelType>
void  EvaluateCurveOverRange(ChannelType* Channel, const FFrameNumber& StartTime, const FFrameNumber& EndTime,
	const FFrameNumber& Interval, TArray<TPair<FFrameNumber, double>>& OutKeys)
{
	using CurveValueType = typename ChannelType::CurveValueType;
	CurveValueType Value;
	FFrameNumber CurrentTime = StartTime;
	OutKeys.SetNum(0);
	while (CurrentTime < EndTime)
	{
		Channel->Evaluate(CurrentTime, Value);
		CurrentTime += Interval;
		TPair<FFrameNumber, double> OutValue;
		OutValue.Key = CurrentTime;
		OutValue.Value = (double) Value;
		OutKeys.Add(OutValue);
	}
}

void GetPercentageOfChange(const TArray<TPair<FFrameNumber, double>>& InKeys, TArray<double>& ValueDifferences,
TArray<TPair<FFrameNumber, double>>& PercentageDifferences)
{
	ValueDifferences.SetNum(InKeys.Num());
	for (int32 Index = 0; Index < InKeys.Num() -1 ; ++Index)
	{
		const double ValueDifference = InKeys[Index + 1].Value - InKeys[Index].Value;
		ValueDifferences[Index] = ValueDifference;
	}
	const double TotalChange = Algo::Accumulate(ValueDifferences, 0.);
	if (FMath::IsNearlyZero(TotalChange) == false)
	{
		const double TotalChangePercentage = (100.0 / TotalChange);
		PercentageDifferences.SetNum(InKeys.Num());
		for (int32 Index = 0; Index < InKeys.Num(); ++Index)
		{
			PercentageDifferences[Index].Key = InKeys[Index].Key;
			PercentageDifferences[Index].Value = TotalChangePercentage * ValueDifferences[Index];
		}
	}
	else
	{
		PercentageDifferences.SetNum(0);
	}
}

template<typename ChannelType, typename ValueType>
void AdjustmentBlend(UMovieSceneSection* Section, TArrayView<ChannelType*>& BaseChannels, 
	TArrayView<ChannelType*>& LayerChannels,ISequencer* InSequencer)
{
	using namespace UE::AIE;
	using namespace UE::MovieScene;

	if (BaseChannels.Num() != LayerChannels.Num())
	{
		return;
	}
	const FFrameRate& FrameRate = InSequencer->GetFocusedDisplayRate();
	const FFrameRate& TickResolution = InSequencer->GetFocusedTickResolution();
	const FFrameNumber Interval = TickResolution.AsFrameNumber(FrameRate.AsInterval());
	EMovieSceneKeyInterpolation DefaultInterpolation = InSequencer->GetKeyInterpolation();

	TArray<FKeyInterval> KeyIntervals;
	TArray <TPair<FFrameNumber, double>> Keys;
	TArray<double> ValueDifferences;
	TArray<TPair<FFrameNumber, double>> PercentageDifferences;
	for (int32 Index = 0; Index < BaseChannels.Num(); ++Index)
	{
		ChannelType* BaseChannel = BaseChannels[Index];
		ChannelType* LayerChannel = LayerChannels[Index];
		KeyIntervals.SetNum(0);
		Keys.SetNum(0);
		GetPairs(LayerChannel, KeyIntervals);
		for (FKeyInterval& KeyInterval : KeyIntervals)
		{
			EvaluateCurveOverRange(BaseChannel, KeyInterval.StartFrame, KeyInterval.EndFrame,
				Interval, Keys);
			GetPercentageOfChange(Keys, ValueDifferences, PercentageDifferences);
			const double TotalPoseLayerChange = FMath::Abs(KeyInterval.EndValue - KeyInterval.StartValue);
			double PreviousValue = KeyInterval.StartValue;
			for (TPair<FFrameNumber, double>& TimeValue : PercentageDifferences)
			{
				const double ValueDelta = (TotalPoseLayerChange / 100.0) * TimeValue.Value;
				const double CurrentValue = (KeyInterval.EndValue > KeyInterval.StartValue) ?
					PreviousValue + ValueDelta : PreviousValue - ValueDelta;
				ValueType ValueTypeValue(CurrentValue);
				AddKeyToChannel(LayerChannel, TimeValue.Key, ValueTypeValue, DefaultInterpolation);
				PreviousValue = CurrentValue;
			}
		}
	}
}

UMovieSceneSection* GetSectionFromAnimLayerItem(const FAnimLayerItem& AnimLayerItem, UControlRig* OptionalControlRig)
{
	for (const FAnimLayerSectionItem& SectionItem : AnimLayerItem.SectionItems)
	{
		if (SectionItem.Section.IsValid())
		{
			if (OptionalControlRig)
			{
				if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
				{
					if (OptionalControlRig == CRSection->GetControlRig())
					{
						return CRSection;
					}
				}
			}
			else
			{
				return SectionItem.Section.Get();
			}
		}
	}
	return nullptr;
}

void GetEvaluatingItemsFromLayer(const FAnimLayerItem& AnimLayerItem, int32 NumOfTransforms, TArray<UE::AIE::FActorAndWorldTransforms>& OutEvaluatingActors, TMap<UControlRig*, UE::AIE::FControlRigAndWorldTransforms>& OutEvaluatingControlRigs)
{
	for (const FAnimLayerSectionItem& SectionItem : AnimLayerItem.SectionItems)
	{
		if (SectionItem.Section.IsValid())
		{
			if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(SectionItem.Section.Get()))
			{
				if (UControlRig* ControlRig = CRSection->GetControlRig())
				{
					FControlRigAndWorldTransforms ControlRigAndWorldTransforms;
					ControlRigAndWorldTransforms.ControlRig = ControlRig;
					ControlRigAndWorldTransforms.ParentTransforms =  MakeShared<FArrayOfTransforms>();
					ControlRigAndWorldTransforms.ParentTransforms->SetNum(NumOfTransforms);

					for (const TPair<FName, FAnimLayerPropertyAndChannels>& ControlName : SectionItem.AnimLayerSet.Names)
					{
						TSharedPtr<FArrayOfTransforms> WorldTransforms = MakeShared<FArrayOfTransforms>();
						WorldTransforms->SetNum(NumOfTransforms);
						ControlRigAndWorldTransforms.ControlAndWorldTransforms.Add(ControlName.Key, WorldTransforms);

					}
					OutEvaluatingControlRigs.Add(ControlRig, ControlRigAndWorldTransforms);		
				}
			}
			else if (UMovieScene3DTransformSection* LayerSection = Cast<UMovieScene3DTransformSection>(SectionItem.Section.Get()))
			{
				//tbd if we do actors or not yet currently will use curve based adjustment blending that only works for world space curves
			}
		}
	}
}


void CalculateTransformDiff(TSharedPtr<UE::AIE::FArrayOfTransforms>& Base, TSharedPtr<UE::AIE::FArrayOfTransforms> Layer, TSharedPtr<UE::AIE::FArrayOfTransforms>& OutDiff)
{
	if (Base->Num() > 0 && (Base->Num() == Layer->Num()))
	{
		OutDiff->SetNum(Base->Num());
		for (int32 Index = 0; Index < Base->Num(); ++Index)
		{

			OutDiff->Transforms[Index] = Layer->Transforms[Index].GetRelativeTransform(Base->Transforms[Index]);
		}

	}
}

struct FTransformAsArray
{
	double Values[9];
	FTransformAsArray()
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			const int32 RealIndex = Index * 3;
			const int32 RealIndex1 = RealIndex + 1;
			const int32 RealIndex2 = RealIndex + 2;
			Values[RealIndex] = 0;
			Values[RealIndex1] = 0;
			Values[RealIndex2] = 0;
		}
	}
	FTransformAsArray(const FTransformAsArray&) = default;
	FTransformAsArray(const FTransform& Transform)
	{
		const FVector& Location = Transform.GetLocation();
		const FVector& EulerAngles = Transform.GetRotation().Euler();
		const FVector& Scale = Transform.GetScale3D();
		Values[0] = Location.X;
		Values[1] = Location.Y;
		Values[2] = Location.Z;
		Values[3] = EulerAngles.X;
		Values[4] = EulerAngles.Y;
		Values[5] = EulerAngles.Z;
		Values[6] = Scale.X;
		Values[7] = Scale.Y;
		Values[8] = Scale.Z;

	}

	FEulerTransform AsEulerTransform() const
	{
		FVector Location(Values[0], Values[1], Values[2]);
		FRotator Rotator(Values[4], Values[5], Values[3]);
		FVector Scale(Values[6], Values[7], Values[8]);

		return FEulerTransform(Location, Rotator, Scale);
	}

	FTransform AsTransform() const
	{
		FVector Location(Values[0], Values[1], Values[2]);
		FRotator Rotator(Values[4], Values[5], Values[3]);
		FVector Scale(Values[6], Values[7], Values[8]);

		return FTransform(Rotator, Location, Scale);
	}

	double& operator[](int32 Index)
	{
		return Values[Index];
	}

	const double& operator[](int32 Index)const
	{
		return Values[Index];
	}

	FTransformAsArray Abs() const
	{
		FTransformAsArray Val;
		for (int32 Index = 0; Index < 3; ++Index)
		{
			const int32 RealIndex = Index * 3;
			const int32 RealIndex1 = RealIndex + 1;
			const int32 RealIndex2 = RealIndex + 2;

			Val.Values[RealIndex]  = FMath::Abs(Values[RealIndex]);
			Val.Values[RealIndex1] = FMath::Abs(Values[RealIndex1]);
			Val.Values[RealIndex2] = FMath::Abs(Values[RealIndex2]);
		}
		return Val;
	}

	FTransformAsArray operator+=(const FTransformAsArray& RHS)
	{
		for (int32 Index = 0; Index < 3; ++Index)
		{
			const int32 RealIndex = Index * 3;
			const int32 RealIndex1 = RealIndex + 1;
			const int32 RealIndex2 = RealIndex + 2;

			Values[RealIndex]  +=  RHS[RealIndex];
			Values[RealIndex1] += RHS[RealIndex1];
			Values[RealIndex2] += RHS[RealIndex2];
		}
		return *this;
	}
};

FTransformAsArray operator-(const FTransformAsArray& LHS, const FTransformAsArray& RHS) {
	FTransformAsArray ReturnVal;
	for (int32 Index = 0; Index < 3; ++Index)
	{
		const int32 RealIndex = Index * 3;
		const int32 RealIndex1 = RealIndex + 1;
		const int32 RealIndex2 = RealIndex + 2;

		ReturnVal[RealIndex] = LHS[RealIndex] - RHS[RealIndex];
		ReturnVal[RealIndex1] = LHS[RealIndex1] - RHS[RealIndex1];
		ReturnVal[RealIndex2] = LHS[RealIndex2] - RHS[RealIndex2];
	}
	return ReturnVal;
}

FTransformAsArray operator+(const FTransformAsArray& LHS, const FTransformAsArray& RHS) {
	FTransformAsArray ReturnVal;
	for (int32 Index = 0; Index < 3; ++Index)
	{
		const int32 RealIndex = Index * 3;
		const int32 RealIndex1 = RealIndex + 1;
		const int32 RealIndex2 = RealIndex + 2;

		ReturnVal[RealIndex] = LHS[RealIndex] + RHS[RealIndex];
		ReturnVal[RealIndex1] = LHS[RealIndex1] + RHS[RealIndex1];
		ReturnVal[RealIndex2] = LHS[RealIndex2] + RHS[RealIndex2];
	}
	return ReturnVal;
}

void GetWorldTransformPercentageOfChange(int32 FrameStartIndex, int32 FrameEndIndex, const TSharedPtr<UE::AIE::FArrayOfTransforms>& ArrayOfTransforms, TArray<FTransformAsArray>& ValueDifferences, TArray<TArray<double>>& PercentageDifferences)
{
	const int32 NumKeys = FrameEndIndex - FrameStartIndex + 1;
	FTransformAsArray CurrentArray;
	FTransformAsArray NextArray(FTransformAsArray(ArrayOfTransforms->Transforms[FrameStartIndex]));
	FTransformAsArray TotalDifferences,TotalChange;
	for (int32 Index = FrameStartIndex; Index < FrameEndIndex; ++Index)
	{
		CurrentArray = NextArray;
		FTransform& NextTransform = (ArrayOfTransforms->Transforms)[Index + 1];
		NextArray = FTransformAsArray(NextTransform);
		ValueDifferences[Index] = (NextArray - CurrentArray).Abs();
		TotalChange += ValueDifferences[Index];
	}
	PercentageDifferences.Reset(0);
	PercentageDifferences.SetNum(9); //one per channel
	for (int32 ChannelIndex = 0; ChannelIndex < 9; ++ChannelIndex)
	{
		TArray<double>& PercentageDiffs = PercentageDifferences[ChannelIndex];
		if (FMath::IsNearlyZero(TotalChange[ChannelIndex]) == false)
		{
			const double TotalChangePercentage = (100.0 / TotalChange[ChannelIndex]);
			PercentageDiffs.SetNum(ValueDifferences.Num());
			for (int32 Index = FrameStartIndex; Index <= FrameEndIndex; ++Index)
			{
				PercentageDiffs[Index] = TotalChangePercentage * ValueDifferences[Index][ChannelIndex];
			}
		}
		else
		{
			PercentageDiffs.SetNum(0);
		}
	}

}
//get evaluating rigs and actors for the layer we will adjust
//make a copy for the base
//mute adjustment , calculate base curves
//unmet adjustment, calculate modified curves

bool ControlTypeValidForAdjustment(const FRigControlElement* ControlElement)
{
	switch (ControlElement->Settings.ControlType)
	{
		case ERigControlType::Position:
		case ERigControlType::Rotator:
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
			return true;
	}
	return false;
}

void AdjustmentBlendWorldSpace(ISequencer* Sequencer,  TArray<FAnimLayerItem*>& LayersToMute,  FAnimLayerItem& AnimLayerItem)
{
	if (!Sequencer)
	{
		return;
	}

	TRange<FFrameNumber> Range = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
	TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer->GetSubSequenceRange();
	if (OptionalRange.IsSet())
	{
		Range = TRange<FFrameNumber>(OptionalRange.GetValue().GetLowerBoundValue(), OptionalRange.GetValue().GetUpperBoundValue());
	}

	FFrameNumber FramesPerTick = FFrameRate::TransformTime(FFrameNumber(1), Sequencer->GetFocusedDisplayRate(),
		Sequencer->GetFocusedTickResolution()).RoundToFrame();

	FFrameTimeByIndex CurrentFrameTimes;
	CurrentFrameTimes = FFrameTimeByIndex(Range.GetLowerBoundValue(), Range.GetUpperBoundValue(), FramesPerTick);

	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;

	TArray<FActorAndWorldTransforms> EvaluatingActors, BaseEvaluatingActors;
	TMap<UControlRig*, FControlRigAndWorldTransforms> EvaluatingControlRigs, BaseEvaluatingControlRigs;

	GetEvaluatingItemsFromLayer(AnimLayerItem, CurrentFrameTimes.NumFrames, EvaluatingActors, EvaluatingControlRigs);

	//set up Base to evaluate from current
	for (const FActorAndWorldTransforms& ActorAndWorld : EvaluatingActors)
	{
		FActorAndWorldTransforms Copy;
		ActorAndWorld.MakeCopy(Copy);
		BaseEvaluatingActors.Add(ActorAndWorld);
	}
	for (const TPair< UControlRig*, FControlRigAndWorldTransforms>& ControlRigAndWorld : EvaluatingControlRigs)
	{
		FControlRigAndWorldTransforms Copy;
		ControlRigAndWorld.Value.MakeCopy(Copy);
		BaseEvaluatingControlRigs.Add(ControlRigAndWorld.Key, Copy);
	}
	
	//get base world space for every frame
	AnimLayerItem.SetSectionsActive(false);
	for (FAnimLayerItem* MuteLayerItem : LayersToMute)
	{
		if (MuteLayerItem)
		{
			MuteLayerItem->SetSectionsActive(false);
		}
	}
	FEvalHelpers::CalculateWorldTransforms(World, Sequencer, CurrentFrameTimes,
		CurrentFrameTimes.GetFullIndexArray(), BaseEvaluatingActors, BaseEvaluatingControlRigs);
	//get the values from the additive, just from it's frames...
	TArray<int32> AdditiveKeyIndices;
	AdditiveKeyIndices.SetNum(2);
	AdditiveKeyIndices[0] = 0;
	AdditiveKeyIndices[1] = CurrentFrameTimes.NumFrames - 1;

	AnimLayerItem.SetSectionsActive(true);
	for (FAnimLayerItem* UnMuteLayerItem : LayersToMute)
	{
		if (UnMuteLayerItem)
		{
			UnMuteLayerItem->SetSectionsActive(true);
		}
	}
	FEvalHelpers::CalculateWorldTransforms(World, Sequencer, CurrentFrameTimes,
		CurrentFrameTimes.GetFullIndexArray(),
		EvaluatingActors, EvaluatingControlRigs);
	
	
	FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransform();
	constexpr ERigTransformType::Type TransformType = ERigTransformType::CurrentGlobal;
	FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

	FRigControlModifiedContext Context;
	Context.SetKey = EControlRigSetKey::Always;
	Context.KeyMask = (uint32)EControlRigContextChannelToKey::AllTransform;

	TArray <TPair<FFrameNumber, FTransformAsArray>> Keys;
	TArray<FTransformAsArray> ValueDifferences;
	ValueDifferences.SetNum(CurrentFrameTimes.NumFrames);
	TArray<TArray<double>> PercentageDifferences;
	FTransformAsArray StartValue, EndValue;


	for (const TPair< UControlRig*, FControlRigAndWorldTransforms>& BaseControlRigAndWorld : BaseEvaluatingControlRigs)
	{
		UControlRig* ControlRig = BaseControlRigAndWorld.Key;
		if (FControlRigAndWorldTransforms* EvalCRandWT = EvaluatingControlRigs.Find(ControlRig))
		{
			URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();
			if (!RigHierarchy)
			{
				continue;
			}
			for (int32 KeyIndex = 0; KeyIndex < AdditiveKeyIndices.Num() - 1; ++KeyIndex)
			{
				int32 FrameStartIndex = AdditiveKeyIndices[KeyIndex];
				int32 FrameEndIndex = AdditiveKeyIndices[KeyIndex + 1];
				if (FrameStartIndex == FrameEndIndex)
				{
					continue;  //should never happen
				}
				//needed controls sorted from root to leaves since we are setting things in global/world space
				TArray <FName> ControlNames;
				BaseControlRigAndWorld.Value.ControlAndWorldTransforms.GenerateKeyArray(ControlNames);
				TArray<FRigElementKey> ControlKeys;
				for (const FName& ControlName : ControlNames)
				{
					ControlKeys.Add(FRigElementKey(ControlName, ERigElementType::Control));
				}
				ControlKeys = RigHierarchy->SortKeys(ControlKeys);

				//for (const TPair<FName, TSharedPtr<FArrayOfTransforms>>& BasePair : BaseControlRigAndWorld.Value.ControlAndWorldTransforms)
				for(const FRigElementKey& Key: ControlKeys)
				{
					if (const TSharedPtr<FArrayOfTransforms>* Transforms = BaseControlRigAndWorld.Value.ControlAndWorldTransforms.Find(Key.Name))
					{
						if (FRigControlElement* ControlElement = ControlRig->FindControl(Key.Name))
						{
							if (ControlTypeValidForAdjustment(ControlElement) == false)
							{
								continue;
							}
							if (TSharedPtr<FArrayOfTransforms>* EvalTransforms = EvalCRandWT->ControlAndWorldTransforms.Find(Key.Name))
							{
								ValueDifferences.Reset();
								ValueDifferences.SetNum(CurrentFrameTimes.NumFrames);
								//get base layer values as diffs and percentages
								GetWorldTransformPercentageOfChange(FrameStartIndex, FrameEndIndex, *Transforms, ValueDifferences, PercentageDifferences);
								//get new layer change
								FTransform STBase = (*Transforms)->Transforms[FrameStartIndex];
								FTransformAsArray StartTransformBase(STBase);
								FTransformAsArray StartTransformSubtractBase((*EvalTransforms)->Transforms[FrameStartIndex]);
								StartTransformSubtractBase = StartTransformSubtractBase - StartTransformBase;

								FTransform ETBase = (*Transforms)->Transforms[FrameEndIndex];
								FTransformAsArray EndTransformBase(ETBase);
								FTransformAsArray EndTransformSubtractBase((*EvalTransforms)->Transforms[FrameEndIndex]);
								EndTransformSubtractBase = EndTransformSubtractBase - EndTransformBase;

								FTransformAsArray CurrentValue;
								FTransformAsArray PreviousValue;
								FTransformAsArray TotalPoseLayerChange;

								for (int32 ChannelIndex = 0; ChannelIndex < 9; ++ChannelIndex)
								{
									TotalPoseLayerChange[ChannelIndex] = FMath::Abs(StartTransformSubtractBase[ChannelIndex] - EndTransformSubtractBase[ChannelIndex]);
									PreviousValue[ChannelIndex] = StartTransformSubtractBase[ChannelIndex];
								}
								FTransform CT, NT;
								FTransformAsArray CurrentTransform, NewTransform;
								for (int32 FrameIndex = FrameStartIndex + 1; FrameIndex <= FrameEndIndex; ++FrameIndex)
								{

									const FFrameNumber FrameNumber = CurrentFrameTimes.CalculateFrame(FrameIndex);
									FFrameTime GlobalTime(FrameNumber);
									GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime); //player evals in root time so need to go back to it.

									FMovieSceneContext MovieSceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
									Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(MovieSceneContext);
									ControlRig->Evaluate_AnyThread();
									for (int32 ChannelIndex = 0; ChannelIndex < 9; ++ChannelIndex)
									{
										if (FMath::IsNearlyZero(TotalPoseLayerChange[ChannelIndex], (1.e-4f)))
										{
											CurrentValue[ChannelIndex] = PreviousValue[ChannelIndex];
										}
										else if (FrameIndex < PercentageDifferences[ChannelIndex].Num())
										{
											double Value = PercentageDifferences[ChannelIndex][FrameIndex - 1];
											const double ValueDelta = (TotalPoseLayerChange[ChannelIndex] / 100.0) * Value;
											CurrentValue[ChannelIndex] = (EndTransformSubtractBase[ChannelIndex] > StartTransformSubtractBase[ChannelIndex]) ?
												PreviousValue[ChannelIndex] + ValueDelta : PreviousValue[ChannelIndex] - ValueDelta;
										}
										else
										{
											CurrentValue[ChannelIndex] = StartTransformSubtractBase[ChannelIndex] + (EndTransformSubtractBase[ChannelIndex] - StartTransformSubtractBase[ChannelIndex]) * (FrameIndex - FrameStartIndex) / (FrameEndIndex - FrameStartIndex);
										}
										PreviousValue[ChannelIndex] = CurrentValue[ChannelIndex];

									}
									//get current transform of the base
									CT = (*Transforms)->Transforms[FrameIndex];
									CT = CT.GetRelativeTransform((*EvalCRandWT->ParentTransforms)[FrameIndex]);
									CurrentTransform = FTransformAsArray(CT);
									NewTransform = CurrentTransform + CurrentValue;
									NT = NewTransform.AsTransform();;
									Context.LocalTime = TickResolution.AsSeconds(FFrameTime(FrameNumber));
									FRigControlValue Value = ControlRig->GetControlValueFromGlobalTransform(ControlElement->GetKey().Name, NT, TransformType);
									NT = Value.GetAsTransform(ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
									FEulerTransform EulerTransform(NT);;
									UE::AIE::FSetTransformHelpers::SetControlTransform(EulerTransform, ControlRig, ControlElement, Context);

								}
							}
						}
					}
				}
			}
		}
	}
	
	//make sure to reset things
	Sequencer->ForceEvaluate();
	//we now need to make sure the control rigs are back up to date.
	//we do this by going through the control rig tracks that the actors/rigs are dependent up

	for (const TPair< UControlRig*, FControlRigAndWorldTransforms>& ControlRigAndWorldAgain : EvaluatingControlRigs)
	{
		if (UControlRig * ControlRig = ControlRigAndWorldAgain.Key)
		{
			ControlRig->Evaluate_AnyThread();
			if (ControlRig->GetObjectBinding())
			{
				ControlRig->EvaluateSkeletalMeshComponent(0.0);
			}
		}
	}
}

} //AIE
} //UE


bool UAnimLayers::AdjustmentBlendLayers(ISequencer* InSequencer, int32 LayerIndex)
{
	using namespace UE::AIE;
	if (InSequencer == nullptr)
	{
		return false;
	}
	if (LayerIndex < 1 || LayerIndex >= AnimLayers.Num())
	{
		return false;
	}
	TArray<FAnimLayerItem*> OtherActiveLayers;
	for (int32 Index = 0; Index < AnimLayers.Num(); ++Index)
	{
		if (Index != 0 && Index != LayerIndex && AnimLayers[Index] != nullptr && AnimLayers[Index]->State.bActive)
		{
			for (TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayers[Index]->AnimLayerItems)
			{
				if (Pair.Key != nullptr)
				{
					OtherActiveLayers.Add(&Pair.Value);
				}
			}
		}
	}
	UAnimLayer* BaseLayer = AnimLayers[0];
	UAnimLayer* AnimLayer = AnimLayers[LayerIndex];
	const FFrameNumber Min = TNumericLimits<FFrameNumber>::Lowest();
	const FFrameNumber Max = TNumericLimits<FFrameNumber>::Max();
	TRange<FFrameNumber> Range(Min, Max);
	FScopedTransaction Transaction(LOCTEXT("AdjustmentBlendLayer", "Adjustment Blend layer"), !GIsTransacting);
	const FAnimLayersScopedSelection ScopedSelection(*this);

	Modify();
	AnimLayer->Modify();


	for (TPair<TWeakObjectPtr<UObject>, FAnimLayerItem>& Pair : AnimLayer->AnimLayerItems)
	{
		if (Pair.Key != nullptr)
		{
			if (Pair.Key->IsA<UControlRig>())
			{
				if (FAnimLayerItem* BaseLayerItem = BaseLayer->AnimLayerItems.Find(Pair.Key))
				{
					AdjustmentBlendWorldSpace(InSequencer, OtherActiveLayers, Pair.Value);
				}
			}
			else
			{

				for (FAnimLayerSectionItem& SectionItem : Pair.Value.SectionItems)
				{
					if (SectionItem.Section.IsValid())
					{
						FAnimLayerSectionItem* BaseSectionItem = nullptr;
						if (FAnimLayerItem* Owner = BaseLayer->AnimLayerItems.Find(Pair.Key))
						{
							BaseSectionItem = Owner->FindMatchingSectionItem(SectionItem.Section.Get());

						}
						if (BaseSectionItem && BaseSectionItem->Section.IsValid())
						{
							if (SectionItem.Section->IsActive())//active sections merge them
							{
								TArrayView<FMovieSceneFloatChannel*> BaseFloatChannels = BaseSectionItem->Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
								TArrayView<FMovieSceneDoubleChannel*> BaseDoubleChannels = BaseSectionItem->Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
								TArrayView<FMovieSceneFloatChannel*> LayerFloatChannels = SectionItem.Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
								TArrayView<FMovieSceneDoubleChannel*> LayerDoubleChannels = SectionItem.Section->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
								SectionItem.Section->Modify();

								if (BaseDoubleChannels.Num() > 0)
								{
									AdjustmentBlend<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>(SectionItem.Section.Get(), BaseDoubleChannels, LayerDoubleChannels, InSequencer);
								}
								else if (BaseFloatChannels.Num() > 0)
								{
									AdjustmentBlend<FMovieSceneFloatChannel, FMovieSceneFloatValue>(SectionItem.Section.Get(), BaseFloatChannels, LayerFloatChannels, InSequencer);
								}
							}
						}
					}
				}
			}
		}
	}
	SetAnimLayerIndices(false /*bResetSectionOrders*/);
	InSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	return true;
}
#undef LOCTEXT_NAMESPACE



