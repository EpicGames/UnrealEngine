// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/SparseDelegate.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/AssetUserData.h"
#include "Rigs/RigHierarchyDefines.h"
#include "ControlRig.h"
#include "Styling/SlateTypes.h"
#include "ISequencerPropertyKeyedStatus.h"
#include "BakingAnimationKeySettings.h"
#include "Misc/Guid.h"
#include "AnimLayers.generated.h"

class ULevelSequence;
class UControlRig;
class UMovieSceneTrack;
class UMovieSceneSection;
class ISequencer;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;
struct FMovieSceneFloatChannel;
class IPropertyHandle;
namespace UE::Sequencer { class FOutlinerSelection; }

USTRUCT(BlueprintType)
struct FMergeAnimLayerSettings
{
	GENERATED_BODY();

	FMergeAnimLayerSettings()
	{
		BakingKeySettings = EBakingKeySettings::KeysOnly;
		FrameIncrement = 1;
		bReduceKeys = false;
		TolerancePercentage = 5.0;
	}
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake")
	EBakingKeySettings BakingKeySettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (ClampMin = "1", UIMin = "1", EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames"))
	int32 FrameIncrement;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames"))
	bool bReduceKeys;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames && bReduceKeys"))
	float TolerancePercentage;
};



USTRUCT(BlueprintType)
// Name of a property and control and the specific channels that make up the layer
struct FAnimLayerPropertyAndChannels
{
	GENERATED_BODY()

	FAnimLayerPropertyAndChannels() : Name(NAME_None) , Channels((uint32)EControlRigContextChannelToKey::AllTransform) {};
	
	//Name of the property or control
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	FName Name; 

	//Mask of channels for that property/control, currently not used
	uint32 Channels;
};

//Bound object/control rig and the properties/channels it is made of
//A layer will consist of a bunch of these
USTRUCT(BlueprintType)
struct FAnimLayerSelectionSet
{
	GENERATED_BODY()
	FAnimLayerSelectionSet() :BoundObject(nullptr) {};

	FAnimLayerSelectionSet& operator = (const FAnimLayerSelectionSet& Other)
	{
		BoundObject = Other.BoundObject;
		for (const TPair<FName, FAnimLayerPropertyAndChannels>& Pair : Other.Names)
		{
			Names.Add(Pair.Key, Pair.Value);
		}
		return *this;
	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TWeakObjectPtr<UObject> BoundObject; //bound object is either a CR or a bound sequencer object

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TMap<FName, FAnimLayerPropertyAndChannels> Names; //property/control name and channels

	bool MergeWithAnotherSelection(const FAnimLayerSelectionSet& Selection);
};

//The set with it's section
USTRUCT(BlueprintType)
struct FAnimLayerSectionItem
{
	GENERATED_BODY()
	FAnimLayerSectionItem() : Section(nullptr) {};

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	FAnimLayerSelectionSet AnimLayerSet;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TWeakObjectPtr<UMovieSceneSection> Section;
};

//individual layer items that make up the layer
USTRUCT(BlueprintType)
struct FAnimLayerItem
{
	GENERATED_BODY()
	FAnimLayerItem()  {};

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TArray< FAnimLayerSectionItem>  SectionItems;

	UPROPERTY()
	FGuid SequencerGuid;

	//make copy with new guid
	void MakeCopy(const FGuid& NewGuid, const TWeakObjectPtr<UObject>& NewObject,  FAnimLayerItem& OutCopy) const;

	//find section that matches based upon incoming section (same type and property)
	FAnimLayerSectionItem* FindMatchingSectionItem(UMovieSceneSection* InMovieSceneSection);

	void SetSectionsActive(bool bIsActive);
};

UENUM()
enum EAnimLayerType : uint32
{
	Base = 0x0,
	Additive = 0x1,
	Override = 0x2,
};
ENUM_CLASS_FLAGS(EAnimLayerType)

//state and properties of a layer
USTRUCT(BlueprintType)
struct FAnimLayerState
{

	GENERATED_BODY()

	FAnimLayerState();
	FText AnimLayerTypeToText()const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	mutable ECheckBoxState bKeyed = ECheckBoxState::Unchecked;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	mutable ECheckBoxState bSelected = ECheckBoxState::Unchecked;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	mutable bool bActive;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	mutable bool bLock;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	FText Name;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	mutable double Weight;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data", meta = (Bitmask, BitmaskEnum = "/Script/ControlRigEditor.EAnimLayerType"))
	mutable int32 Type;

};

USTRUCT(BlueprintType)
struct FAnimLayerControlRigObject
{
	GENERATED_BODY()
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TWeakObjectPtr<UControlRig> ControlRig;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TArray<FName>  ControlNames;
};

USTRUCT(BlueprintType)
struct FAnimLayerSceneObject
{
	GENERATED_BODY()
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TWeakObjectPtr<UObject> SceneObjectOrComponent;
	
	//just doing transform for now
	//UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	//TArray<FName>  PropertyNames;
};



USTRUCT(BlueprintType)
struct FAnimLayerObjects
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TArray<FAnimLayerControlRigObject> ControlRigObjects;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	TArray<FAnimLayerSceneObject> SceneObjects;
};

UCLASS(EditInlineNew, CollapseCategories, HideDropdown)
class UAnimLayerWeightProxy : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Interp, DisplayName = "", Category = "NoCategory", meta = (SliderExponent = "1.0"))
	double Weight = 1.0;

	//UObect overrides for setting values when weight changes
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

//for friend access to State variables directly
class UAnimLayers;
struct FAnimLayerSourceUIEntry;
struct FAnimLayersScopedSelection;

UCLASS(BlueprintType, MinimalAPI)
class UAnimLayer : public UObject
{
	GENERATED_UCLASS_BODY()
public:
	UAnimLayer() {};
	virtual void PostLoad() override;

	void GetAnimLayerObjects(FAnimLayerObjects& InLayerObjects) const;
	void GetSections(TArray<UMovieSceneSection*>& OutSections) const;

	UFUNCTION(BlueprintCallable, Category = "State")
	ECheckBoxState GetKeyed() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetKeyed();

	UFUNCTION(BlueprintCallable, Category = "State")
	ECheckBoxState GetSelected() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetSelected(bool bInSelected, bool bClearSelection);

	UFUNCTION(BlueprintCallable, Category = "State")
	bool GetActive() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetActive(bool bInActive);

	UFUNCTION(BlueprintCallable, Category = "State")
	bool GetLock() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetLock(bool bInLock);

	UFUNCTION(BlueprintCallable, Category = "State")
	FText GetName() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetName(const FText& InName);

	UFUNCTION(BlueprintCallable, Category = "State")
	double GetWeight() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetWeight(double InWeight);

	UFUNCTION(BlueprintCallable, Category = "State")
	EAnimLayerType GetType() const;
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetType(EAnimLayerType LayerType);

	UFUNCTION(BlueprintCallable, Category = "State")
	bool AddSelectedInSequencer();
	UFUNCTION(BlueprintCallable, Category = "State")
	bool RemoveSelectedInSequencer();

	UFUNCTION(BlueprintCallable, Category = "State")
	void SelectWeightChannel(bool bClearSelection);

	int32 GetAnimLayerIndex() const { return LayerIndex; }
	bool RemoveAnimLayerItem(UObject* InObject);
	void SetSelectedInList(const FAnimLayersScopedSelection& ScopedSelection, bool bInValue);
	bool GetSelectedInList() const { return bIsSelectedInList; }
	ECheckBoxState GetSelected(TSet<UObject*>& OutSelectedObjects, TMap<UControlRig*, TArray<FName>>& OutSelectedControls) const;

	//for key property status for weights
	void SetKey(TSharedPtr<ISequencer>& Sequencer, const IPropertyHandle& KeyedPropertyHandle);
	EPropertyKeyedStatus GetPropertyKeyedStatus(TSharedPtr<ISequencer>& Sequencer, const IPropertyHandle& PropertyHandle);
	
	//function to make sure we have Guids stored on the anim layers now so we can correctly keep track of spawned actors.
	void UpdateSceneObjectorGuidsForItems(ISequencer* InSequencer);

	/** Sets sequencer selection for this layer's currently selected controls */
	void ApplyControlSelectionToSequencer(bool bSelect = true) const;
	
	/** Returns true if the layer has sections that match the current editor selection */
	bool HasSectionsInCurrentEditorSelection() const;

private:

	UPROPERTY(VisibleAnywhere, Category = "Data")
	TMap<TWeakObjectPtr<UObject>, FAnimLayerItem> AnimLayerItems;

	UPROPERTY(VisibleAnywhere, Category = "Data")
	FAnimLayerState State;

	UPROPERTY(Transient)
	TObjectPtr<UAnimLayerWeightProxy> WeightProxy;
	
	//cache of index in layer array
	UPROPERTY()
	int32 LayerIndex = INDEX_NONE;

	void SetSectionToKey() const;

	//static helper functions
	static bool IsAccepableNonControlRigObject(UObject* InObject);

	bool bIsSelectedInList = false;

	friend class UAnimLayers;
	friend struct FAnimLayerSourceUIEntry;
};

/** Link To Set of Anim Sequences that we may belinked to.*/
UCLASS(BlueprintType, MinimalAPI)
class UAnimLayers : public UAssetUserData
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	virtual bool IsEditorOnly() const override { return true; }
	virtual void PostLoad() override;

	UPROPERTY(BlueprintReadWrite, Category = Layers)
	TArray<TObjectPtr<UAnimLayer>> AnimLayers;

	UPROPERTY()
	bool bOpenUIOnOpen = true;

	int32 GetAnimLayerIndex(UAnimLayer* InAnimLayer) const;
	bool DeleteAnimLayer(ISequencer* InSequencer, int32 Index);
	int32 DuplicateAnimLayer(ISequencer* InSequencer, int32 Index);
	int32 AddAnimLayerFromSelection(ISequencer* InSequencer);
	void GetAnimLayerStates(TArray<FAnimLayerState>& OutStates);
	bool MergeAnimLayers(TSharedPtr<ISequencer>& InSequencerPtr, const TArray<int32>& Indices, const FMergeAnimLayerSettings* InSettings);
	bool SetPassthroughKey(ISequencer* InSequencer, int32 Index);
	bool SetKey(ISequencer* InSequencer, int32 Index);
	bool ReorderAnimLayers(ISequencer* InSequencer, int32 OldIndex, int32 NewIndex);
	//will always blend to base for now
	bool AdjustmentBlendLayers(ISequencer* InSequencer, int32 LayerIndex);

	bool IsTrackOnSelectedLayer(const UMovieSceneTrack* InTrack)const;
	
	TArray<UMovieSceneSection*> GetSelectedLayerSections() const;

	//Get
	static UAnimLayers* GetAnimLayers(ISequencer* InSequencer, bool bAddIfDoesNotExist = false);
	static UAnimLayers* GetAnimLayers(ULevelSequence* InLevelSequence, bool bAddIfDoesNotExist = false);
	static bool HasAnimLayers(ISequencer* InSequencer);

	static 	TSharedPtr<ISequencer> GetSequencerFromAsset();

public:
	DECLARE_EVENT_OneParam(UAnimLayers, FAnimLayerListChanged, UAnimLayers*);

	FAnimLayerListChanged& AnimLayerListChanged() { return OnAnimLayerListChanged; };

	/** Returns a delegate broadcast when anim outliner changed selection, both for anim layers self as well as for sequencer. */
	FSimpleMulticastDelegate& GetOnSelectionChanged() { return OnSelectionChanged; }

private:
	void AnimLayerListChangedBroadcast();
	FAnimLayerListChanged OnAnimLayerListChanged;
	void AddBaseLayer();
	bool SetKeyValueOrPassthrough(ISequencer* InSequencer, int32 Index, bool bJustValue);
	//Set the anim layer index values, if bResetSectionOrders is true we will also go through and make sure the sections are
	//also in order
	void SetAnimLayerIndices(bool bResetSectionOrders);
	//get set of tracks and their sections for each anim layer
	void GetTracksAndSections(TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>>& OutTracksAndOrderedSections);
	//setup track information (name, mute state etc), for sections on the track per anim layer
	void SetTrackRowForSections(const TMap<UMovieSceneTrack*, TArray<TPair<UAnimLayer*, UMovieSceneSection*>>>& TracksAndOrderedSections);
	/** A delegate broadcast when the selected layer sections changed */
	FSimpleMulticastDelegate OnSelectionChanged;

public:
	void SetUpBaseLayerSections();

	static void SetUpSectionDefaults(ISequencer* SequencerPtr, UAnimLayer* Layer, UMovieSceneTrack* InTrack, UMovieSceneSection* InSection, FMovieSceneFloatChannel* FloatChannel);
	static void SetUpControlRigSection(UMovieSceneControlRigParameterSection* InSection, TArray<FName>& ControlNames);
};

/** A scope in which selection can occur in anim layers. Raises OnSelectionChanged when destructed. */
struct FAnimLayersScopedSelection
{
	FAnimLayersScopedSelection(UAnimLayers & AnimLayers)
		: WeakAnimLayers(&AnimLayers)
	{
	}

	~FAnimLayersScopedSelection()
	{
		if (WeakAnimLayers.IsValid())
		{
			WeakAnimLayers->GetOnSelectionChanged().Broadcast();
		}
	}

private:
	TWeakObjectPtr<UAnimLayers> WeakAnimLayers;
};


