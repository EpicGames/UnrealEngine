// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimLayerComboBox.h"
#include "ControlRigEditorStyle.h"
#include "DetailLayoutBuilder.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "SAnimLayerComboBoxItem.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SAnimLayerComboBox"

SAnimLayerComboBox::~SAnimLayerComboBox()
{
	RemoveAnimLayerBindings();
}

void SAnimLayerComboBox::Construct(const FArguments& InArgs, const TWeakPtr<ISequencer>& InWeakSequencer)
{
	WeakSequencer = InWeakSequencer;

	AnimLayerChangedDelegate = InArgs._OnAnimLayerChanged;

	static constexpr float ComboHeight = 26.f;
	static const FVector2D IconSize(16.f, 16.f);

	ChildSlot
	.VAlign(VAlign_Center)
	.Padding(0.f)
	[
		SNew(SBox)
		.MinDesiredWidth(InArgs._MinDesiredWidth)
		.MaxDesiredWidth(FMath::Max(200.f, InArgs._MinDesiredWidth.Get(0.).Get()))
		.HeightOverride(ComboHeight)
		[
			SAssignNew(ComboButton, SComboButton)
			.ToolTipText(LOCTEXT("AnimationLayersTooltip", "Animation Layers"))
			.OnMenuOpenChanged(this, &SAnimLayerComboBox::HandleComboMenuOpenChanged)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 2.f, 0.f)
				[
					SNew(SImage)
					.DesiredSizeOverride(IconSize)
					.ColorAndOpacity(FStyleColors::Foreground)
					.Image(FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.AnimLayers")))
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 0.f, 0.f)
				[
					SAssignNew(SelectedContent, SBox)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
				]
			]
			.MenuContent()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Menu.Background")))
				[
					SNew(SBox)
					.MinDesiredWidth(InArgs._MinDesiredWidth)
					.MaxDesiredHeight(300.f)
					[
						SAssignNew(ListView, SListView<FAnimLayerItemPtr>)
						.ListItemsSource(&AnimLayerItems)
						.SelectionMode(ESelectionMode::Multi)
						.ClearSelectionOnClick(false)
						.OnGenerateRow(this, &SAnimLayerComboBox::GenerateListRow)
						.OnSelectionChanged(this, &SAnimLayerComboBox::HandleSelectionChanged)
					]
				]
			]
		]
	];

	ReloadAnimLayerData();
}

void SAnimLayerComboBox::Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if the movie scene has changed and reload layer data if it has
	const UMovieScene* const MovieScene = GetFocusedMovieScene();
	if (!MovieScene)
	{
		if (LastMovieSceneSig.IsValid())
		{
			LastMovieSceneSig.Invalidate();

			RemoveAnimLayerBindings();
			ResetCache();

			if (ListView.IsValid())
			{
				ListView->RebuildList();
				ListView->ClearSelection();
			}

			UpdateSelectedContent();
		}
		return;
	}

	const FGuid CurrentMovieSceneSig = MovieScene->GetSignature();
	if (LastMovieSceneSig != CurrentMovieSceneSig)
	{
		LastMovieSceneSig = CurrentMovieSceneSig;
		ReloadAnimLayerData();
	}
}

TSharedRef<ITableRow> SAnimLayerComboBox::GenerateListRow(const FAnimLayerItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(STableRow<FAnimLayerItemPtr>, InOwnerTable)
		.Padding(FMargin(6.f, 6.f))
		[
			SNew(SAnimLayerComboBoxItem)
			.Item(InItem)
		];
}

void SAnimLayerComboBox::ReloadAnimLayerData()
{
	RemoveAnimLayerBindings();
	ResetCache();

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;
	if (!AnimLayers)
	{
		if (ListView.IsValid())
		{
			ListView->RebuildList();
			ListView->ClearSelection();
		}

		UpdateSelectedContent();

		return;
	}

	RebuildSectionToLayerMap();

	for (const TObjectPtr<UAnimLayer>& Layer : AnimLayers->AnimLayers)
	{
		if (Layer)
		{
			AnimLayerItems.Add(MakeShared<FItem>(Layer));
		}
	}

	if (ListView.IsValid())
	{
		ListView->RebuildList();
	}

	AnimLayerSelectionChangedDelegate = AnimLayers->GetOnSelectionChanged().AddSP(this, &SAnimLayerComboBox::HandleAnimLayerSelectionChanged);
	AnimLayersChangedDelegate = AnimLayers->AnimLayerListChanged().AddSP(this, &SAnimLayerComboBox::HandleAnimLayersChanged);

	if (AnimLayerItems.IsEmpty())
	{
		if (ListView.IsValid())
		{
			ListView->ClearSelection();
		}

		UpdateSelectedContent();

		return;
	}

	HandleAnimLayerSelectionChanged();
}

void SAnimLayerComboBox::RemoveAnimLayerBindings()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;

	if (!AnimLayers)
	{
		return;
	}

	if (AnimLayerSelectionChangedDelegate.IsValid())
	{
		AnimLayers->GetOnSelectionChanged().Remove(AnimLayerSelectionChangedDelegate);
		AnimLayerSelectionChangedDelegate.Reset();
	}

	if (AnimLayersChangedDelegate.IsValid())
	{
		AnimLayers->AnimLayerListChanged().Remove(AnimLayersChangedDelegate);
		AnimLayersChangedDelegate.Reset();
	}
}

void SAnimLayerComboBox::ResetCache()
{
	AnimLayerItems.Reset();
	SectionToLayerMap.Reset();
}

void SAnimLayerComboBox::SyncListSelectionToAnimLayerSelection()
{
	if (!ListView.IsValid())
	{
		return;
	}

	ListView->ClearSelection();

	const TSet<FAnimLayerItemPtr> SelectedItems = GetSelectedItems();
	for (const FAnimLayerItemPtr& Item : SelectedItems)
	{
		ListView->SetItemSelection(Item, true, ESelectInfo::Direct);
	}
}

void SAnimLayerComboBox::HandleComboMenuOpenChanged(const bool bInOpen)
{
	if (!bInOpen || !ListView.IsValid())
	{
		return;
	}

	RegisterActiveTimer(0.f
		, FWidgetActiveTimerDelegate::CreateSP(this, &SAnimLayerComboBox::HandleDeferredSyncSelection));
}

EActiveTimerReturnType SAnimLayerComboBox::HandleDeferredSyncSelection(const double InCurrentTime, const float InDeltaTime)
{
	SyncListSelectionToAnimLayerSelection();
	return EActiveTimerReturnType::Stop;
}

void SAnimLayerComboBox::HandleAnimLayerSelectionChanged()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;
	if (!AnimLayers)
	{
		return;
	}

	UpdateSelectedContent();
	SyncListSelectionToAnimLayerSelection();
}

void SAnimLayerComboBox::HandleAnimLayersChanged(UAnimLayers* const InAnimLayers)
{
	ReloadAnimLayerData();
}

void SAnimLayerComboBox::HandleSelectionChanged(const FAnimLayerItemPtr InItem, const ESelectInfo::Type InSelectInfo)
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;
	if (!AnimLayers)
	{
		return;
	}
	if (!AnimLayers || InSelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	if (InItem.IsValid() && InItem->WeakAnimLayer.IsValid())
	{
		UAnimLayer* const SelectedAnimLayer = InItem->WeakAnimLayer.Get();

		SetSelectedAnimLayers({ SelectedAnimLayer });

		AnimLayerChangedDelegate.ExecuteIfBound(SelectedAnimLayer);
	}
	else
	{
		SetSelectedAnimLayers({});

		AnimLayerChangedDelegate.ExecuteIfBound(nullptr);
	}

	SyncListSelectionToAnimLayerSelection();
	UpdateSelectedContent();

	if (ComboButton.IsValid())
	{
		ComboButton->SetIsOpen(false);
	}
}

void SAnimLayerComboBox::UpdateSelectedContent()
{
	if (!SelectedContent.IsValid())
	{
		return;
	}

	const TSet<FAnimLayerItemPtr> SelectedItems = GetSelectedItems();
	const int32 SelectedItemCount = SelectedItems.Num();

	FAnimLayerItemPtr NewItem = nullptr;

	if (SelectedItemCount == 1)
	{
		NewItem = SelectedItems.Array()[0];
	}

	// Early out if nothing actually changed
	if (CachedSelectedCount == SelectedItemCount)
	{
		if (SelectedItemCount != 1 || CachedSelectedItem == NewItem)
		{
			return;
		}
	}

	CachedSelectedCount = SelectedItemCount;
	CachedSelectedItem = NewItem;

	TSharedPtr<SWidget> NewContentWidget;

	if (SelectedItemCount == 1)
	{
		NewContentWidget = SNew(SAnimLayerComboBoxItem)
			.Item(SelectedItems.Array()[0]);
	}
	else
	{
		const FText ComboBoxText = SelectedItemCount > 1
			? LOCTEXT("MultipleAnimLayers", "- Multiple -")
			: LOCTEXT("NoAnimLayer", "- None -");

		NewContentWidget = SNew(SBox)
			.Padding(6.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.Font(DEFAULT_FONT("Italic", 10))
				.Text(ComboBoxText)
				.ColorAndOpacity(FStyleColors::Foreground)
			];
	}

	SelectedContent->SetContent(NewContentWidget.ToSharedRef());
}

SAnimLayerComboBox::FAnimLayerItemPtr SAnimLayerComboBox::FindItemByAnimLayer(UAnimLayer* const InAnimLayer) const
{
	const FAnimLayerItemPtr* const FoundItem = AnimLayerItems.FindByPredicate(
		[InAnimLayer](const FAnimLayerItemPtr& InItem)
		{
			return InItem.IsValid() && InItem->WeakAnimLayer == InAnimLayer;
		});
	return FoundItem ? *FoundItem : nullptr;
}

void SAnimLayerComboBox::SetSelectedAnimLayer(UAnimLayer* const InAnimLayer)
{
	SetSelectedAnimLayers(InAnimLayer ? TSet{ InAnimLayer } : TSet<UAnimLayer*>{});
	SyncListSelectionToAnimLayerSelection();
	UpdateSelectedContent();
}

UMovieScene* SAnimLayerComboBox::GetFocusedMovieScene() const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	const UMovieSceneSequence* const FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return nullptr;
	}

	return FocusedSequence->GetMovieScene();
}

UAnimLayer* SAnimLayerComboBox::FindLayerForSection(UMovieSceneSection* const InSection) const
{
	if (!InSection)
	{
		return nullptr;
	}

	const TWeakObjectPtr<UAnimLayer>* const FoundLayer = SectionToLayerMap.Find(InSection);
	return (FoundLayer && FoundLayer->IsValid()) ? FoundLayer->Get() : nullptr;
}

void SAnimLayerComboBox::RebuildSectionToLayerMap()
{
	SectionToLayerMap.Reset();

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;
	if (!AnimLayers)
	{
		return;
	}

	for (UAnimLayer* const Layer : AnimLayers->AnimLayers)
	{
		if (!Layer)
		{
			continue;
		}

		TArray<UMovieSceneSection*> LayerSections;
		Layer->GetSections(LayerSections);

		for (UMovieSceneSection* const Section : LayerSections)
		{
			if (Section)
			{
				SectionToLayerMap.Add(Section, Layer);
			}
		}
	}
}

bool SAnimLayerComboBox::GetSingleAnimLayerFromSections(const TArray<UMovieSceneSection*>& InSections
	, UAnimLayer*& OutLayer, bool& bOutMultiple) const
{
	OutLayer = nullptr;
	bOutMultiple = false;

	for (UMovieSceneSection* const Section : InSections)
	{
		UAnimLayer* const Layer = FindLayerForSection(Section);
		if (!Layer)
		{
			OutLayer = nullptr;
			bOutMultiple = false;

			return false;
		}

		if (!OutLayer)
		{
			OutLayer = Layer;
		}
		else if (OutLayer != Layer)
		{
			bOutMultiple = true;
			OutLayer = nullptr;

			return false;
		}
	}

	return OutLayer != nullptr;
}

SAnimLayerComboBox::FAnimLayerItemPtr SAnimLayerComboBox::GetBaseLayerItem() const
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;

	if (!AnimLayers || AnimLayers->AnimLayers.IsEmpty())
	{
		return nullptr;
	}

	UAnimLayer* const BaseLayer = AnimLayers->AnimLayers[0];
	return FindItemByAnimLayer(BaseLayer);
}

void SAnimLayerComboBox::SetSelectedAnimLayers(const TSet<UAnimLayer*>& InSelectedLayers)
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;
	if (!AnimLayers)
	{
		return;
	}

	const FAnimLayersScopedSelection ScopedSelection(*AnimLayers);

	for (UAnimLayer* const AnimLayer : AnimLayers->AnimLayers)
	{
		if (AnimLayer)
		{
			const bool bShouldBeSelectedInList = InSelectedLayers.Contains(AnimLayer);
			AnimLayer->SetSelectedInList(ScopedSelection, bShouldBeSelectedInList);
		}
	}

	// IMPORTANT: do NOT call SetSelected(..., true) for every layer in the loop,
	// because later unselected layers will clear the earlier selected one.
	bool bClearedSelection = false;

	for (UAnimLayer* const AnimLayer : AnimLayers->AnimLayers)
	{
		if (!AnimLayer || !InSelectedLayers.Contains(AnimLayer))
		{
			continue;
		}

		AnimLayer->SetSelected(true, !bClearedSelection);
		bClearedSelection = true;

		AnimLayer->SetKeyed();
	}

	// If nothing should be selected, explicitly clear once
	if (!bClearedSelection)
	{
		for (UAnimLayer* const AnimLayer : AnimLayers->AnimLayers)
		{
			if (AnimLayer)
			{
				AnimLayer->SetSelected(false, true);
				break;
			}
		}
	}

	SyncListSelectionToAnimLayerSelection();
	UpdateSelectedContent();

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshTree);
}

TSet<UAnimLayer*> SAnimLayerComboBox::GetSelectedAnimLayers() const
{
	TSet<UAnimLayer*> SelectedLayers;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;

	if (!AnimLayers)
	{
		return SelectedLayers;
	}

	for (UAnimLayer* const AnimLayer : AnimLayers->AnimLayers)
	{
		if (AnimLayer && AnimLayer->GetSelectedInList())
		{
			SelectedLayers.Add(AnimLayer);
		}
	}

	return SelectedLayers;
}

TSet<SAnimLayerComboBox::FAnimLayerItemPtr> SAnimLayerComboBox::GetSelectedItems() const
{
	TSet<FAnimLayerItemPtr> SelectedItems;

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	UAnimLayers* AnimLayers = Sequencer.IsValid() ? UAnimLayers::GetAnimLayers(Sequencer.Get()) : nullptr;


	if (!AnimLayers)
	{
		return SelectedItems;
	}

	for (const FAnimLayerItemPtr& Item : AnimLayerItems)
	{
		if (!Item.IsValid() || !Item->WeakAnimLayer.IsValid())
		{
			continue;
		}

		if (Item->WeakAnimLayer->GetSelectedInList())
		{
			SelectedItems.Add(Item);
		}
	}

	return SelectedItems;
}

#undef LOCTEXT_NAMESPACE
