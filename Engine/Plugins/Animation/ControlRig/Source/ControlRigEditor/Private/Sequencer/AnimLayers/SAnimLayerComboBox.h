// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimLayers.h"
#include "Styling/StyleColors.h"
#include "Widgets/SCompoundWidget.h"

class ISequencer;
class ITableRow;
class SBox;
class SComboButton;
class STableViewBase;
class UAnimLayer;
class UMovieScene;
class UMovieSceneSection;
template <typename ItemType> class SListView;

class SAnimLayerComboBox : public SCompoundWidget
{
public:
	struct FItem
	{
		FItem(const TWeakObjectPtr<UAnimLayer>& InAnimLayer)
			: WeakAnimLayer(InAnimLayer)
		{}

		TWeakObjectPtr<UAnimLayer> WeakAnimLayer;
	};
	using FAnimLayerItemPtr = TSharedPtr<FItem>;

	DECLARE_DELEGATE_OneParam(FOnAnimLayerChanged, UAnimLayer*)

	SLATE_BEGIN_ARGS(SAnimLayerComboBox)
		: _MinDesiredWidth(120.f)
	{}
		SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)

		/** Event that broadcasts when the selected animation layer has changed */
		SLATE_EVENT(FOnAnimLayerChanged, OnAnimLayerChanged)

	SLATE_END_ARGS()

	virtual ~SAnimLayerComboBox() override;

	void Construct(const FArguments& InArgs, const TWeakPtr<ISequencer>& InWeakSequencer);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End Widget

	void SetSelectedAnimLayer(UAnimLayer* const InAnimLayer);
	void SetSelectedAnimLayers(const TSet<UAnimLayer*>& InSelectedLayers);
	TSet<UAnimLayer*> GetSelectedAnimLayers() const;

protected:
	TSharedRef<ITableRow> GenerateListRow(const FAnimLayerItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	void ReloadAnimLayerData();

	void RemoveAnimLayerBindings();

	void ResetCache();

	void SyncListSelectionToAnimLayerSelection();

	void HandleComboMenuOpenChanged(const bool bInOpen);

	EActiveTimerReturnType HandleDeferredSyncSelection(const double InCurrentTime, const float InDeltaTime);

	void HandleAnimLayerSelectionChanged();
	void HandleAnimLayersChanged(UAnimLayers* const InAnimLayers);

	void HandleSelectionChanged(const FAnimLayerItemPtr InItem, const ESelectInfo::Type InSelectInfo);

	void UpdateSelectedContent();

	FAnimLayerItemPtr FindItemByAnimLayer(UAnimLayer* const InAnimLayer) const;

	UMovieScene* GetFocusedMovieScene() const;

	UAnimLayer* FindLayerForSection(UMovieSceneSection* const InSection) const;

	void RebuildSectionToLayerMap();

	bool GetSingleAnimLayerFromSections(const TArray<UMovieSceneSection*>& InSections
		, UAnimLayer*& OutLayer, bool& bOutMultiple) const;

	FAnimLayerItemPtr GetBaseLayerItem() const;

	void ShowMultipleSelectedContent();

	TSet<FAnimLayerItemPtr> GetSelectedItems() const;

	TWeakPtr<ISequencer> WeakSequencer;

	FOnAnimLayerChanged AnimLayerChangedDelegate;

	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SListView<FAnimLayerItemPtr>> ListView;
	TSharedPtr<SBox> SelectedContent;

	FGuid LastMovieSceneSig;

	TArray<FAnimLayerItemPtr> AnimLayerItems;

	FDelegateHandle AnimLayersChangedDelegate;
	FDelegateHandle AnimLayerSelectionChangedDelegate;

	TMap<TWeakObjectPtr<UMovieSceneSection>, TWeakObjectPtr<UAnimLayer>> SectionToLayerMap;

	FAnimLayerItemPtr CachedSelectedItem;
	int32 CachedSelectedCount = INDEX_NONE;
};
