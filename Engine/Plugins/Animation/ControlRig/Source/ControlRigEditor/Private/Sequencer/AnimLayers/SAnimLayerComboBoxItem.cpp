// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimLayerComboBoxItem.h"
#include "AnimLayers.h"
#include "ControlRigEditorStyle.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAnimLayerComboBoxItem"

void SAnimLayerComboBoxItem::Construct(const FArguments& InArgs)
{
	Item = InArgs._Item;
	TextColor = InArgs._TextColor;

	ChildSlot
	.Padding(FMargin(6.f, 1.f))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.AnimLayerSelected")))
			.ColorAndOpacity(this, &SAnimLayerComboBoxItem::GetIconColor)
			.Visibility(this, &SAnimLayerComboBoxItem::GetIconVisibility)
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(this, &SAnimLayerComboBoxItem::GetDisplayTextColor)
			.Text(this, &SAnimLayerComboBoxItem::GetDisplayText)
		]
	];
}

FSlateColor SAnimLayerComboBoxItem::GetIconColor() const
{
	FSlateColor SlateColor = FStyleColors::Foreground;

	if (UAnimLayer* const AnimLayer = GetAnimLayer())
	{
		const bool bSelected = AnimLayer->GetSelected() != ECheckBoxState::Unchecked;
		if (bSelected)
		{
			if (AnimLayer->GetSelectedInList())
			{
				SlateColor = FStyleColors::Foreground;
			}
			else
			{
				SlateColor = FStyleColors::Select;
			}										
		}
		else
		{
			SlateColor = FStyleColors::Dropdown;
		}
	}

	return SlateColor;
}

EVisibility SAnimLayerComboBoxItem::GetIconVisibility() const
{
	const TSharedPtr<SAnimLayerComboBox::FItem> ActualItem = Item.IsSet() ? Item.Get() : nullptr;
	return ActualItem.IsValid() && ActualItem->WeakAnimLayer.IsValid()
		? EVisibility::Visible : EVisibility::Collapsed;
}

FText SAnimLayerComboBoxItem::GetDisplayText() const
{
	UAnimLayer* const AnimLayer = GetAnimLayer();
	return AnimLayer ? AnimLayer->GetName() : LOCTEXT("NoSelection", "Anim Layers");
}

FSlateColor SAnimLayerComboBoxItem::GetDisplayTextColor() const
{
	if (!Item.IsSet() || !Item.Get().IsValid())
	{
		return FStyleColors::AccentGray;
	}

	if (TextColor.IsSet())
	{
		return TextColor.Get();
	}

	return FStyleColors::Foreground;
}

UAnimLayer* SAnimLayerComboBoxItem::GetAnimLayer() const
{
	const TSharedPtr<SAnimLayerComboBox::FItem> ActualItem = Item.IsSet() ? Item.Get() : nullptr;
	return ActualItem.IsValid() ? ActualItem->WeakAnimLayer.Get() : nullptr;
}

#undef LOCTEXT_NAMESPACE
