// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SAnimLayerComboBox.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class UAnimLayer;
struct FSlateColor;

class SAnimLayerComboBoxItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimLayerComboBoxItem) {}
		SLATE_ATTRIBUTE(TSharedPtr<SAnimLayerComboBox::FItem>, Item)
		SLATE_ATTRIBUTE(FSlateColor, TextColor)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	FSlateColor GetIconColor() const;
	EVisibility GetIconVisibility() const;

	FText GetDisplayText() const;
	FSlateColor GetDisplayTextColor() const;

	UAnimLayer* GetAnimLayer() const;

	TAttribute<TSharedPtr<SAnimLayerComboBox::FItem>> Item;
	TAttribute<FSlateColor> TextColor;
};
