// Copyright Epic Games, Inc. All Rights Reserved.
#include "SAnimLayers.h"
#include "AnimLayers.h"
#include "Templates/SharedPointer.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "DetailWidgetRow.h"
#include <DetailsNameWidgetOverrideCustomization.h>
#include "IDetailCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "IStructureDetailsView.h"
#include "Containers/Set.h"
#include "PropertyPath.h"
#include "Widgets/SWidget.h"
#include "EditMode/ControlRigEditMode.h"
#include "Toolkits/IToolkitHost.h"
#include "EditorModeManager.h"
#include "ControlRig.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MovieScene.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "SceneOutlinerPublicTypes.h"
#include "ControlRigEditorStyle.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Sequencer/SaveScopedControlRigSelection.h"
#include "Algo/IndexOf.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"

#define LOCTEXT_NAMESPACE "AnimLayers"

struct FAnimLayerSourceUIEntry;

typedef TSharedPtr<FAnimLayerSourceUIEntry> FAnimLayerSourceUIEntryPtr;

namespace UE::AIE
{

	bool bBaseLayerOnBottom = false;
	FAutoConsoleVariableRef CVarUseLegacyControlRigTemplate(
		TEXT("ControlRig.Sequencer.BaseAnimLayerOnBottom"),
		bBaseLayerOnBottom,
		TEXT("(Default: false) Puts Base Anim Layer on bottom of stack, if false(default) it puts base layer on top."),
		ECVF_Default
	);

	int32 ConvertUIIndexIntoAnimIndex(int32 Index, int32 Num)
	{
		if (bBaseLayerOnBottom)
		{
			return (Num - 1) - Index;
		}
		return Index;
	}

	void SetAnimLayerIndices(TArray<int32>& Indices, int32 Num)
	{
		Indices.SetNum(Num);
		for (int32 Index = 0; Index < Num; ++Index)
		{
			Indices[Index] = UE::AIE::ConvertUIIndexIntoAnimIndex(Index, Num);
		}
	}

}

namespace AnimLayerSourceListUI
{
	static const FName LayerColumnName(TEXT("Layer"));
	static const FName ActionColumnName(TEXT("Action"));
	static const FName StatusColumnName(TEXT("Status"));
	static const FName WeightColumnName(TEXT("Weight"));
	static const FName TypeColumnName(TEXT("Type"));
};


// Structure that defines a single entry in the source UI
struct FAnimLayerSourceUIEntry
{
public:
	FAnimLayerSourceUIEntry(UAnimLayer* InAnimLayer): AnimLayer(InAnimLayer)
	{}

	void SetSelectedInList(const FAnimLayersScopedSelection& ScopedSelection, bool bInValue);
	bool GetSelectedInList() const;

	void SelectObjects() const;
	void AddSelected() const;
	void RemoveSelected() const;
	void Duplicate() const;
	void DeleteAnimLayer() const;
	void SetPassthroughKey() const;
	void SetKey() const;
	void SelectWeightChannel() const;

	ECheckBoxState GetKeyed() const;
	void SetKeyed();
	FReply OnKeyedColor();
	FSlateColor GetKeyedColor() const;

	ECheckBoxState GetSelected() const;
	void SetSelected(bool bInSelected);

	bool GetActive() const;
	void SetActive(bool bInActive);

	bool GetLock() const;
	void SetLock(bool bInLock);

	FText GetName() const;
	void SetName(const FText& InName);

	double GetWeight() const;
	void SetWeight(double InWeight);

	EAnimLayerType GetType() const;
	FText GetTypeToText() const;
	void SetType(EAnimLayerType InType);

	UObject* GetWeightObject() const;

	int32 GetAnimLayerIndex() const;
	void ClearCaches() const { bSelectionStateValid = false; bKeyedStateIsValid = false; }
private:
	//cached values
	mutable bool bSelectionStateValid = false;
	mutable bool bKeyedStateIsValid = false;
	mutable ECheckBoxState SelectionState = ECheckBoxState::Unchecked;
	mutable ECheckBoxState KeyedState = ECheckBoxState::Unchecked;

	UAnimLayer* AnimLayer = nullptr;
};

int32 FAnimLayerSourceUIEntry::GetAnimLayerIndex() const
{
	if (AnimLayer)
	{
		return AnimLayer->GetAnimLayerIndex();
	}
	return INDEX_NONE;
}

UObject* FAnimLayerSourceUIEntry::GetWeightObject() const
{
	if (AnimLayer)
	{
		return AnimLayer->WeightProxy.Get();
	}
	return nullptr;
}

void FAnimLayerSourceUIEntry::SelectObjects() const
{
	if (AnimLayer)
	{
		AnimLayer->SetSelected(true, !(FSlateApplication::Get().GetModifierKeys().IsShiftDown()));
		ClearCaches();
	}
}

void FAnimLayerSourceUIEntry::AddSelected() const
{
	if (AnimLayer)
	{
		AnimLayer->AddSelectedInSequencer();
		ClearCaches();
	}
}

void FAnimLayerSourceUIEntry::RemoveSelected() const
{
	if (AnimLayer)
	{
		AnimLayer->RemoveSelectedInSequencer();
		ClearCaches();
	}
}

void FAnimLayerSourceUIEntry::DeleteAnimLayer() const
{
	if (AnimLayer)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
		{
			if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
			{
				int32 Index = GetAnimLayerIndex();
				if (Index != INDEX_NONE)
				{
					AnimLayers->DeleteAnimLayer(SequencerPtr.Get(), Index);
				}
			}
		}
		ClearCaches();
	}
}
void FAnimLayerSourceUIEntry::Duplicate() const
{
	if (AnimLayer)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
		{
			if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
			{
				int32 Index = GetAnimLayerIndex();
				if (Index != INDEX_NONE)
				{
					AnimLayers->DuplicateAnimLayer(SequencerPtr.Get(), Index);
				}
			}
		}
		ClearCaches();
	}
}

void FAnimLayerSourceUIEntry::SetPassthroughKey() const
{
	if (AnimLayer)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
		{
			if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
			{
				int32 Index = GetAnimLayerIndex();
				if (Index != INDEX_NONE)
				{
					AnimLayers->SetPassthroughKey(SequencerPtr.Get(), Index);
				}
			}
		}
		ClearCaches();
	}
}

void FAnimLayerSourceUIEntry::SetKey() const
{
	if (AnimLayer)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
		{
			if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
			{
				int32 Index = GetAnimLayerIndex();
				if (Index != INDEX_NONE)
				{
					AnimLayers->SetKey(SequencerPtr.Get(), Index);
				}
			}
		}
		ClearCaches();
	}
}

void FAnimLayerSourceUIEntry::SelectWeightChannel() const
{
	if (AnimLayer)
	{
		AnimLayer->SelectWeightChannel(!(FSlateApplication::Get().GetModifierKeys().IsShiftDown()));
	}
}

ECheckBoxState FAnimLayerSourceUIEntry::GetKeyed() const
{
	if (bKeyedStateIsValid == false)
	{
		bKeyedStateIsValid = true;
		if (AnimLayer)
		{
			KeyedState = AnimLayer->GetKeyed();
		}
	}
	return KeyedState;
}

void FAnimLayerSourceUIEntry::SetKeyed()
{
	if (AnimLayer)
	{
		bKeyedStateIsValid = false;
		AnimLayer->SetKeyed();
	}
}

FReply FAnimLayerSourceUIEntry::OnKeyedColor()
{
	if (AnimLayer)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetKeyed_Transaction", "Set Keyed"), !GIsTransacting);
		AnimLayer->SetKeyed();
	}
	return FReply::Handled();
}

FSlateColor FAnimLayerSourceUIEntry::GetKeyedColor() const
{
	ECheckBoxState State = GetKeyed();
	switch (State)
	{
		case ECheckBoxState::Undetermined:
		{
			return (FLinearColor::Green /2);
		};
		case ECheckBoxState::Checked:
		{
			return FLinearColor::Green;
		};
	}
	return FLinearColor::Transparent;
}

ECheckBoxState FAnimLayerSourceUIEntry::GetSelected() const
{
	if (bSelectionStateValid == false)
	{
		bSelectionStateValid = true;
		if (AnimLayer)
		{
			SelectionState = AnimLayer->GetSelected();
		}
	}
	return SelectionState;
}

void FAnimLayerSourceUIEntry::SetSelected(bool bInSelected)
{
	if (AnimLayer)
	{
		bSelectionStateValid = false;
		AnimLayer->SetSelected(bInSelected, !(FSlateApplication::Get().GetModifierKeys().IsShiftDown()));
	}
}

void FAnimLayerSourceUIEntry::SetSelectedInList(const FAnimLayersScopedSelection& ScopedSelection, bool bInValue)
{
	if (AnimLayer)
	{
		AnimLayer->SetSelectedInList(ScopedSelection, bInValue);
		if (bInValue)
		{
			AnimLayer->SetKeyed();//selection also sets keyed
		}
	}
}

bool FAnimLayerSourceUIEntry::GetSelectedInList() const 
{ 
	if (AnimLayer)
	{
		return AnimLayer->GetSelectedInList();
	}
	return false;
}

bool FAnimLayerSourceUIEntry::GetActive() const
{
	if (AnimLayer)
	{
		return AnimLayer->GetActive();
	}
	return false;
}

void FAnimLayerSourceUIEntry::SetActive(bool bInActive)
{
	if (AnimLayer)
	{
		AnimLayer->SetActive(bInActive);
	}
}

bool FAnimLayerSourceUIEntry::GetLock() const
{
	if (AnimLayer)
	{
		return AnimLayer->GetLock();
	}
	return false;
}

void FAnimLayerSourceUIEntry::SetLock(bool bInLock)
{
	if (AnimLayer)
	{
		AnimLayer->SetLock(bInLock);
	}
}

FText FAnimLayerSourceUIEntry::GetName() const
{
	if (AnimLayer)
	{
		return AnimLayer->GetName();
	}
	return FText();
}

void FAnimLayerSourceUIEntry::SetName(const FText& InName)
{
	if (AnimLayer)
	{
		AnimLayer->SetName(InName);
	}
}

double FAnimLayerSourceUIEntry::GetWeight() const
{
	if (AnimLayer)
	{
		return AnimLayer->GetWeight();
	}
	return 0.0;
}

void FAnimLayerSourceUIEntry::SetWeight(double InWeight)
{
	if (AnimLayer)
	{
		AnimLayer->SetWeight(InWeight);
	}
}

EAnimLayerType FAnimLayerSourceUIEntry::GetType() const
{
	if (AnimLayer)
	{
		return AnimLayer->GetType();
	}
	return EAnimLayerType::Base;
}


void FAnimLayerSourceUIEntry::SetType(EAnimLayerType InType)
{
	if (AnimLayer)
	{
		AnimLayer->SetType(InType);
	}
}

FText FAnimLayerSourceUIEntry::GetTypeToText() const
{
	if (AnimLayer)
	{
		return AnimLayer->State.AnimLayerTypeToText();
	}
	return FText();
}

class FAnimLayerSourcesView;

/** List view that routes Delete/Backspace through the owning view for a unified transaction + refresh. */
class SAnimLayerListView : public STreeView<FAnimLayerSourceUIEntryPtr>
{
public:
	void SetSourcesView(TSharedPtr<FAnimLayerSourcesView> InSourcesView) { WeakSourcesView = InSourcesView; }

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TWeakPtr<FAnimLayerSourcesView> WeakSourcesView;
};

class FAnimLayerSourcesView : public TSharedFromThis<FAnimLayerSourcesView>
{
	// Routes keyboard Delete through the private menu handler.
	friend class SAnimLayerListView;

public:
	FAnimLayerSourcesView();

	// Gather information about all sources and update the list view 
	void RefreshSourceData(bool bRefreshUI);
	// Handler that creates a widget row for a given ui entry
	TSharedRef<ITableRow> MakeSourceListViewWidget(FAnimLayerSourceUIEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable);
	// Handles constructing a context menu for the sources
	TSharedPtr<SWidget> OnSourceConstructContextMenu();;
	// Handle selection change, triggering the OnSourceSelectionChangedDelegate delegate.
	void OnSourceListSelectionChanged(FAnimLayerSourceUIEntryPtr Entry, ESelectInfo::Type SelectionType) const;
	// Focus on the added layer name
	void FocusRenameLayer(int32 Index)
	{
		FocusOnIndex = Index;
	}

	void AddController(FAnimLayerController* InController);

	void RenameItem(int32 Index) const;

	//drag and drop
	FReply OnDragDetected(
		const FGeometry& MyGeometry,
		const FPointerEvent& MouseEvent);
	TOptional<EItemDropZone> OnCanAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		FAnimLayerSourceUIEntryPtr TargetElement);
	FReply OnAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		FAnimLayerSourceUIEntryPtr TargetElement);
private:
	// Create the sources list view
	void CreateSourcesListView();

	mutable int32  FocusOnIndex = INDEX_NONE;

	void AddSelected();
	void RemoveSelected();
	void SelectObjects();
	void Duplicate();
	void MergeLayers();
	void AdjustmentBlend();
	void SetPassthroughKey();
	void DeleteAnimLayer();
	void Rename();

	mutable bool bWaitingToUpdateSourceList = false;

public:
	// Holds the data that will be displayed in the list view
	TArray<FAnimLayerSourceUIEntryPtr> SourceData;
	// Holds the sources list view widget
	TSharedPtr<SAnimLayerListView> SourcesListView;
	//pointer to controller safe to be raw since it's the parent.
	FAnimLayerController* Controller = nullptr;


};

struct  FAnimLayerController : public TSharedFromThis<FAnimLayerController>
{
public:
	FAnimLayerController();
	~FAnimLayerController();
private:
	bool bSelectionFilterActive = false;

private:

	void RebuildSourceList();

	// Handles source selection changing.
	void OnSourceSelectionChangedHandler(FAnimLayerSourceUIEntryPtr Entry, ESelectInfo::Type SelectionType) const;
public:
	// Handles the source collection changing.
	void RefreshSourceData(bool bRefreshUI);
	void RefreshTimeDependantData();
	void RefreshSelectionData();
	void HandleOnAnimLayerListChanged(UAnimLayers*) { RebuildSourceList(); }
	void FocusRenameLayer(int32 Index) { if (SourcesView.IsValid()) SourcesView->FocusRenameLayer(Index); }
	void SelectItem(const FAnimLayersScopedSelection& ScopedSelection, int32 Index, bool bClear = true);
	void ToggleSelectionFilterActive();
	bool IsSelectionFilterActive() const;
public:
	// Sources view
	TSharedPtr<FAnimLayerSourcesView> SourcesView;
	// Guard from reentrant selection
	mutable bool bSelectionChangedGuard = false;
};

class FAnimLayersSourceViewDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAnimLayersSourceViewDragDropOp, FDecoratedDragDropOp)
		static TSharedRef<FAnimLayersSourceViewDragDropOp> New(TWeakPtr<FAnimLayerSourceUIEntry> InElement);
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	TWeakPtr<FAnimLayerSourceUIEntry> Element;
};

TSharedRef<FAnimLayersSourceViewDragDropOp> FAnimLayersSourceViewDragDropOp::New(TWeakPtr<FAnimLayerSourceUIEntry> InElement)
{
	TSharedRef<FAnimLayersSourceViewDragDropOp> Operation = MakeShared<FAnimLayersSourceViewDragDropOp>();
	Operation->Element = InElement;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FAnimLayersSourceViewDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
				.Text(Element.Pin()->GetName())
		];
}

class SAnimLayerSourcesRow : public SMultiColumnTableRow<FAnimLayerSourceUIEntryPtr>
{
public:
	SLATE_BEGIN_ARGS(SAnimLayerSourcesRow) {}
	/** The list item for this row */
	SLATE_ARGUMENT(FAnimLayerSourceUIEntryPtr, Entry)
	SLATE_ARGUMENT(TSharedPtr<FAnimLayerSourcesView>, ListView)
	SLATE_END_ARGS()


	void Construct(const FArguments& Args, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		EntryPtr = Args._Entry;
		ListViewPtr = Args._ListView;
		LayerTypeTextList.Emplace(new FText(LOCTEXT("Additive", "Additive")));
		LayerTypeTextList.Emplace(new FText(LOCTEXT("Override", "Override")));
		SMultiColumnTableRow<FAnimLayerSourceUIEntryPtr>::Construct(
			FSuperRowType::FArguments()
			.OnDragDetected(ListViewPtr.Get(), &FAnimLayerSourcesView::OnDragDetected)
			.OnCanAcceptDrop(ListViewPtr.Get(), &FAnimLayerSourcesView::OnCanAcceptDrop)
			.OnAcceptDrop(ListViewPtr.Get(), &FAnimLayerSourcesView::OnAcceptDrop)
			.Padding(2.f),
			OwnerTableView
		);
	}

	// Static Source UI FNames
	void BeginEditingName()
	{
		if (LayerNameTextBlock.IsValid())
		{
			LayerNameTextBlock->EnterEditingMode();
		}
	}

	FSlateColor GetKeyedColor() const
	{
		if (EntryPtr)
		{
			return EntryPtr->GetKeyedColor();
		}
		return FLinearColor::Transparent;
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == AnimLayerSourceListUI::LayerColumnName)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(6)
				.MinWidth(6)
				[
					SNew(SBox)
						.WidthOverride(6)
						[
							SNew(SButton)
								.ContentPadding(0)
								.VAlign(VAlign_Fill)
								.IsFocusable(true)
								.IsEnabled(true)
								.ButtonStyle(FAppStyle::Get(), "Sequencer.AnimationOutliner.ColorStrip")
								.OnClicked_Lambda([this]()
									{
										return EntryPtr->OnKeyedColor();
									})
								.Content()
								[
									SNew(SImage)
										.Image(FAppStyle::GetBrush("WhiteBrush"))
										.ColorAndOpacity(this, &SAnimLayerSourcesRow::GetKeyedColor)
								]
						]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.IsFocusable(false)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ButtonColorAndOpacity_Lambda([this]()
						{
							return FLinearColor::White;
						})
					.OnClicked_Lambda([this]()
						{
							bool bValue = EntryPtr->GetSelected() == ECheckBoxState::Unchecked ? false : true;
							EntryPtr->SetSelected(!bValue);
							return FReply::Handled();
						})
					.ContentPadding(1.f)
					.ToolTipText(LOCTEXT("AnimLayerSelectionTooltip", "Selection In Layer"))
					[
						SNew(SImage)
							.ColorAndOpacity_Lambda([this]()
								{
									const FColor Selected(38, 187, 255);
									const FColor NotSelected(56, 56, 56);
									//FLinearColor::White;
									bool bValue = EntryPtr->GetSelected() == ECheckBoxState::Unchecked ? false : true;
									FSlateColor SlateColor;
									if (bValue == true)
									{
										if (EntryPtr->GetSelectedInList())
										{
											SlateColor = FLinearColor::White;
										}
										else
										{
											SlateColor = FSlateColor(Selected);
										}										
									}
									else
									{
										SlateColor = FSlateColor(NotSelected);
									}
									return SlateColor;
								})
							.Image(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.AnimLayerSelected").GetIcon())
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(10.0)
				.Padding(10.0,0)
				[	SNew(SBox)
					.HAlign(EHorizontalAlignment::HAlign_Left)
					[
						SAssignNew(LayerNameTextBlock, SInlineEditableTextBlock)
							.Justification(ETextJustify::Left)
							.Text_Lambda([this]()
								{
									return EntryPtr->GetName();
								})
							.OnTextCommitted(this, &SAnimLayerSourcesRow::OnLayerNameCommited)
					]
				];
		}
		else if (ColumnName == AnimLayerSourceListUI::ActionColumnName)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.IsFocusable(false)
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.OnClicked_Lambda([this]()
							{
								EntryPtr->SetKey();
								return FReply::Handled();
							})
						.ContentPadding(1.f)
						.ToolTipText(LOCTEXT("AnimLayerKeyTooltip", "Key selected controls or all controls if none selected"))
						[
							SNew(SImage)
								.Image(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.KeyAdd").GetIcon())
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.IsFocusable(false)
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.OnClicked_Lambda([this]()
							{
								EntryPtr->SetPassthroughKey();
								return FReply::Handled();
							})
						.ContentPadding(1.f)
						.ToolTipText(LOCTEXT("AnimLayerPassthroughTooltip", "Set Default Pose(Base), Zero Key(Additive) or Passthrough(Override) key"))
						[
							SNew(SImage)
								.Image(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.KeySpecial").GetIcon())
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.IsFocusable(false)
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.OnClicked_Lambda([this]()
							{
								EntryPtr->SelectWeightChannel();
								return FReply::Handled();
							})
						.ContentPadding(1.f)
						.ToolTipText(LOCTEXT("AnimLayerSelectWeightChannelTooltip", "Select the weight channel for this layer in the Sequencer"))
						[
							SNew(SImage)
								.Image(FAppStyle::GetBrush("Sequencer.Weight"))
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				];


		}
		else if (ColumnName == AnimLayerSourceListUI::StatusColumnName)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1, 0)
					.VAlign(VAlign_Center)
					[
						SAssignNew(MuteButton, SButton)
						.IsFocusable(false)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ButtonColorAndOpacity_Lambda([this] ()
						{
							const bool bIsActive = !EntryPtr->GetActive();
							const bool bIsHovered = MuteButton->IsHovered();
							return GetStatusImageColorAndOpacity(bIsActive, bIsHovered);		
						})
						.OnClicked_Lambda([this]()
						{
							bool bValue = EntryPtr->GetActive();
							EntryPtr->SetActive(!bValue);
							return FReply::Handled();
						})
						.ContentPadding(1.f)
						.ToolTipText(LOCTEXT("AnimLayerMuteTooltip", "Mute Layer"))
						[
							SNew(SImage)
								.ColorAndOpacity_Lambda([this]() 
								{ 
									const bool bIsActive = !EntryPtr->GetActive();
									const bool bIsHovered = MuteButton->IsHovered();
									return GetStatusImageColorAndOpacity(bIsActive, bIsHovered);
								})
								.Image(FAppStyle::GetBrush("Sequencer.Column.Mute"))
						]
					]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1, 0)
					.VAlign(VAlign_Center)
					[
						SAssignNew(LockButton, SButton)
						.IsFocusable(false)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ButtonColorAndOpacity_Lambda([this] ()
						{
							const bool bIsLock = EntryPtr->GetLock();
							const bool bIsHovered = LockButton->IsHovered();
							return GetStatusImageColorAndOpacity(bIsLock, bIsHovered);
						})
						.OnClicked_Lambda([this]()
						{
							bool bValue = EntryPtr->GetLock();
							EntryPtr->SetLock(!bValue);
							return FReply::Handled();
						})
						.ContentPadding(1.f)
						.ToolTipText(LOCTEXT("AnimLayerLockTooltip", "Lock Layer"))
						[
							SNew(SImage)
								.ColorAndOpacity_Lambda([this]() 
								{ 
									const bool bIsLock = EntryPtr->GetLock();
									const bool bIsHovered = LockButton->IsHovered();
									return GetStatusImageColorAndOpacity(bIsLock, bIsHovered);
								})
							.Image(FAppStyle::GetBrush("Sequencer.Column.Locked"))
						]
					];


		}
		else if (ColumnName == AnimLayerSourceListUI::WeightColumnName)
		{
			UObject* WeightObject = nullptr;
			FControlRigEditMode* EditMode = nullptr;
			if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
			{
				WeightObject = EntryPtr->GetWeightObject();

				TSharedPtr<IToolkitHost> ToolkitHost = SequencerPtr->GetToolkitHost();
				if (ToolkitHost.IsValid())
				{
					FEditorModeTools& EditorModeTools = ToolkitHost->GetEditorModeManager();			
					if (!EditorModeTools.IsModeActive(FControlRigEditMode::ModeName))
					{
						EditorModeTools.ActivateMode(FControlRigEditMode::ModeName);

						EditMode = static_cast<FControlRigEditMode*>(EditorModeTools.GetActiveMode(FControlRigEditMode::ModeName));
						if (EditMode && EditMode->GetToolkit().IsValid() == false)
						{
							EditMode->Enter();
						}
					}
					EditMode = static_cast<FControlRigEditMode*>(EditorModeTools.GetActiveMode(FControlRigEditMode::ModeName));	
				}
			}
			return SAssignNew(WeightDetails, SAnimWeightDetails, EditMode, WeightObject);
			
		}
		else if (ColumnName == AnimLayerSourceListUI::TypeColumnName)
		{
			if(EntryPtr->GetType() != (int32)(EAnimLayerType::Base))
			{ 
				return SAssignNew(LayerTypeCombo, SComboBox<TSharedPtr<FText>>)
					.ContentPadding(FMargin(10.0f, 2.0f))
					.OptionsSource(&LayerTypeTextList)
					.HasDownArrow(false)
					.OnGenerateWidget_Lambda([this](TSharedPtr<FText> Item)
					{
						return SNew(SBox)
							.MaxDesiredWidth(100.0f)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
								.Text(*Item)
							];
					})
					.OnSelectionChanged(this, &SAnimLayerSourcesRow::OnLayerTypeChanged)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
						.Text(this, &SAnimLayerSourcesRow::GetLayerTypeText)
					];
			}
			else
			{
				return SNullWidget::NullWidget;
			}
		}

		return SNullWidget::NullWidget;
	}

private:
	
	FSlateColor GetStatusImageColorAndOpacity(bool bIsActive, bool bIsHovered) const
	{
		FLinearColor OutColor = FLinearColor::White;
		float Opacity = 0.0f;

		if (bIsActive)
		{
			// Directly active, full opacity
			Opacity = 1.0f;
		}
		else if (bIsHovered)
		{
			// Mouse is over widget and it is not directly active.
			Opacity = .8f;
		}
		else 
		{
			// Not active in any way and mouse is not over widget or item.
			Opacity = 0.1f;
		}
		OutColor.A = Opacity;
		return OutColor;
	}


	void OnLayerNameCommited(const FText& InNewText, ETextCommit::Type InTextCommit)
	{
		if (InNewText.IsEmpty())
		{
			return;
		}
		EntryPtr->SetName(InNewText);
	}

	FText GetLayerTypeText() const
	{
		FText CurrentLayerTypeText = EntryPtr->GetTypeToText();

		if (LayerTypeCombo.IsValid() && LayerTypeCombo->GetSelectedItem().IsValid() && LayerTypeCombo->GetSelectedItem()->IdenticalTo(CurrentLayerTypeText) == false)
		{
			int32 constexpr AdditiveIndex = 0;
			int32 constexpr OverrideIndex = 1;
			if (EntryPtr->GetType() == EAnimLayerType::Additive)
			{
				LayerTypeCombo->SetSelectedItem(LayerTypeTextList[AdditiveIndex]);
			}
			else
			{
				LayerTypeCombo->SetSelectedItem(LayerTypeTextList[OverrideIndex]);
			}
		}
		return CurrentLayerTypeText;

	}
	void OnLayerTypeChanged(TSharedPtr<FText> InNewText, ESelectInfo::Type SelectInfo)
	{
		if (InNewText.IsValid() == false || InNewText->IsEmpty())
		{
			return;
		}
		FText Additive(LOCTEXT("Additive", "Additive"));
		FText Override(LOCTEXT("Override", "Override"));
		if (InNewText->IdenticalTo(Additive))
		{
			EntryPtr->SetType(EAnimLayerType::Additive);
		}
		else if (InNewText->IdenticalTo(Override))
		{
			EntryPtr->SetType(EAnimLayerType::Override);
		}
	}


private:
	FAnimLayerSourceUIEntryPtr EntryPtr;
	TSharedPtr<FAnimLayerSourcesView> ListViewPtr;
	TSharedPtr<SInlineEditableTextBlock> LayerNameTextBlock;
	mutable TArray<TSharedPtr<FText>> LayerTypeTextList;
	TSharedPtr<SComboBox<TSharedPtr<FText>>> LayerTypeCombo;
	//TSharedPtr<SNumericEntryBox<double>> WeightBox;
	TSharedPtr< SAnimWeightDetails> WeightDetails;
	TSharedPtr<SButton> MuteButton;
	TSharedPtr<SButton> LockButton;
};

FAnimLayerSourcesView::FAnimLayerSourcesView()
{
	CreateSourcesListView();
}

void FAnimLayerSourcesView::AddController(FAnimLayerController* InController)
{
	Controller = InController;
}

TSharedRef<ITableRow> FAnimLayerSourcesView::MakeSourceListViewWidget(FAnimLayerSourceUIEntryPtr Entry, const TSharedRef<STableViewBase>& OwnerTable) 
{
	if (FocusOnIndex != INDEX_NONE)
	{
		int32 NextIndex = FocusOnIndex;
		FocusOnIndex = INDEX_NONE;
		GEditor->GetTimerManager()->SetTimerForNextTick([this, NextIndex]()
			{
				if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
				{
					if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
					{
						for (int32 Index = 0; Index < SourceData.Num(); ++Index)
						{
							const FAnimLayerSourceUIEntryPtr& Item = SourceData[Index];
							if (Item->GetAnimLayerIndex() == NextIndex) //mz to fix
							{
								int32 RenameIndex = UE::AIE::ConvertUIIndexIntoAnimIndex(Item->GetAnimLayerIndex(), AnimLayers->AnimLayers.Num());
								RenameItem(RenameIndex);
								FocusOnIndex = INDEX_NONE;
							}
						}
					}
				}
			});
	}
	return SNew(SAnimLayerSourcesRow, OwnerTable)
		.Entry(Entry)
		.ListView(AsShared());
}

void FAnimLayerSourcesView::RenameItem(int32 Index) const
{
	if (Index < SourceData.Num())
	{
		if (TSharedPtr<ITableRow> Row = SourcesListView->WidgetFromItem(SourceData[Index]))
		{
			TWeakPtr<SAnimLayerSourcesRow> Widget = StaticCastSharedRef<SAnimLayerSourcesRow>(Row->AsWidget());
			if (Widget.IsValid())
			{
				Widget.Pin()->BeginEditingName();
			}
		}
	}
}

void FAnimLayerSourcesView::CreateSourcesListView()
{
	bWaitingToUpdateSourceList = false;
	SAssignNew(SourcesListView, SAnimLayerListView)
	//	.ListItemsSource(&SourceData)
		.TreeItemsSource(&SourceData)
		.OnGetChildren_Lambda([](FAnimLayerSourceUIEntryPtr ChannelItem, TArray<FAnimLayerSourceUIEntryPtr>& OutChildren)
		{
		})
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow_Raw(this, &FAnimLayerSourcesView::MakeSourceListViewWidget)
		.OnContextMenuOpening_Raw(this, &FAnimLayerSourcesView::OnSourceConstructContextMenu)
		.OnSelectionChanged_Raw(this, &FAnimLayerSourcesView::OnSourceListSelectionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(AnimLayerSourceListUI::LayerColumnName)
			.FillSized(160.f)
			.HAlignCell(EHorizontalAlignment::HAlign_Left)
			.DefaultLabel(LOCTEXT("LayerColumnName", "Name"))
			+ SHeaderRow::Column(AnimLayerSourceListUI::ActionColumnName)
			.FillWidth(40.f)
			.HAlignCell(EHorizontalAlignment::HAlign_Center)
			.DefaultLabel(LOCTEXT("ActionColumnName", "Action"))
			+ SHeaderRow::Column(AnimLayerSourceListUI::StatusColumnName)
			.FillWidth(40.f)
			.HAlignCell(EHorizontalAlignment::HAlign_Center)
			.DefaultLabel(LOCTEXT("StatusColumnName", "Status"))
			+ SHeaderRow::Column(AnimLayerSourceListUI::WeightColumnName)
			.FillWidth(60.f)
			.HAlignCell(EHorizontalAlignment::HAlign_Center)
			.DefaultLabel(LOCTEXT("WeightColumnName", "Weight"))
			+ SHeaderRow::Column(AnimLayerSourceListUI::TypeColumnName)
			.FillSized(80.f)
			.HAlignCell(EHorizontalAlignment::HAlign_Right)
			.DefaultLabel(LOCTEXT("TypeColumnName", "Type"))
		);
}

FReply SAnimLayerListView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Delete || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		if (TSharedPtr<FAnimLayerSourcesView> PinnedView = WeakSourcesView.Pin())
		{
			PinnedView->DeleteAnimLayer();
			return FReply::Handled();
		}
	}
	return STreeView<FAnimLayerSourceUIEntryPtr>::OnKeyDown(MyGeometry, InKeyEvent);
}

void FAnimLayerSourcesView::AddSelected()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddSelectedAnimLayer_Transaction", "Add Selected"), !GIsTransacting);
		for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
		{
			Ptr->AddSelected();
		}
	}
}
void FAnimLayerSourcesView::RemoveSelected()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveSelected_Transaction", "Remove Selected"), !GIsTransacting);
		for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
		{
			Ptr->RemoveSelected();
		}

		if (Controller)
		{
			Controller->RefreshSourceData(true);
		}
	}
}

void FAnimLayerSourcesView::SelectObjects()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetSelected_Transaction", "Set Selection"), !GIsTransacting);
		for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
		{
			Ptr->SelectObjects();
		}
	}
}

void FAnimLayerSourcesView::DeleteAnimLayer()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteAnimLayer_Transaction", "Delete Anim Layer"), !GIsTransacting);
		for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
		{
			Ptr->DeleteAnimLayer();
		}

		if (Controller)
		{
			Controller->RefreshSourceData(true);
		}
	}
}
void FAnimLayerSourcesView::Duplicate()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateAnimLayer_Transaction", "Duplicate Anim Layer"), !GIsTransacting);
		for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
		{
			Ptr->Duplicate();
		}

		if (Controller)
		{
			Controller->RefreshSourceData(true);
		}
	}
}

void FAnimLayerSourcesView::AdjustmentBlend()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			TArray<FAnimLayerSourceUIEntryPtr> Selected;
			SourcesListView->GetSelectedItems(Selected);
			if (Selected.Num() != 1)
			{
				return;
			}
			int32 Index = Selected[0]->GetAnimLayerIndex();
			if (Index != INDEX_NONE && Index != 0) //not base
			{
				AnimLayers->AdjustmentBlendLayers(SequencerPtr.Get(), Index);
			}
		}
	}
}


FReply FAnimLayerSourcesView::OnDragDetected(
	const FGeometry& MyGeometry,
	const FPointerEvent& MouseEvent)
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() != 1)
	{
		return FReply::Unhandled();
	}

	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const FAnimLayerSourceUIEntryPtr DraggedElement = Selected[0];
		const TSharedRef<FAnimLayersSourceViewDragDropOp> DragDropOp = FAnimLayersSourceViewDragDropOp::New(DraggedElement);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> FAnimLayerSourcesView::OnCanAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	FAnimLayerSourceUIEntryPtr TargetItem)
{
	TOptional<EItemDropZone> ReturnedDropZone;

	const TSharedPtr<FAnimLayersSourceViewDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAnimLayersSourceViewDragDropOp>();
	if (DragDropOp.IsValid())
	{
		ReturnedDropZone = DropZone == EItemDropZone::BelowItem ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;
	}

	return ReturnedDropZone;
}

FReply FAnimLayerSourcesView::OnAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	FAnimLayerSourceUIEntryPtr TargetItem)
{
	TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset();
	if (SequencerPtr.IsValid() == false)
	{
		return FReply::Handled();
	}
	UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get());
	if (AnimLayers == nullptr)
	{
		return FReply::Handled();
	}
	const TSharedPtr<FAnimLayersSourceViewDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAnimLayersSourceViewDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}

	if (Controller == nullptr)
	{
		return FReply::Handled();
	}
	
	FAnimLayerSourceUIEntryPtr DraggedElement = DragDropOp.Get()->Element.Pin();
	if (DraggedElement.IsValid() == false)
	{
		return FReply::Handled();
	}

	if (DraggedElement->GetAnimLayerIndex() == TargetItem->GetAnimLayerIndex() || 
		DraggedElement->GetAnimLayerIndex() == 0)
	{
		return FReply::Handled();
	}

	//put into correct space
	int32 TargetIndex = UE::AIE::ConvertUIIndexIntoAnimIndex(TargetItem->GetAnimLayerIndex(), AnimLayers->AnimLayers.Num());
	const int32 DraggedIndex = UE::AIE::ConvertUIIndexIntoAnimIndex(DraggedElement->GetAnimLayerIndex(), AnimLayers->AnimLayers.Num());

	if (DropZone == EItemDropZone::BelowItem)
	{
		if (TargetIndex < DraggedIndex)
		{
			TargetIndex++;
		}
	}
	else if (DropZone == EItemDropZone::AboveItem)
	{
		if (TargetIndex > DraggedIndex)
		{
			TargetIndex--;
		}
	}

	//put target back out
	TargetIndex = UE::AIE::ConvertUIIndexIntoAnimIndex(TargetIndex, AnimLayers->AnimLayers.Num());

	AnimLayers->ReorderAnimLayers(SequencerPtr.Get(), DraggedElement->GetAnimLayerIndex(), TargetIndex);

	if (Controller)
	{
		Controller->RefreshSourceData(true);
	}

	return FReply::Handled();
}
//////////////////////////////////////////////////////////////
/// SMergeAnimLayersWidget
///////////////////////////////////////////////////////////
DECLARE_DELEGATE_TwoParams(FMergeAnimLayersCB, TSharedPtr<ISequencer>& InSequencer, const FMergeAnimLayerSettings& InSettings);

/** Widget allowing collapsing of controls */
class SMergeAnimLayersWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMergeAnimLayersWidget)
		: _Sequencer(nullptr), _bSmartReduce(false), _TolerancePercentage(TNumericLimits<float>::Max())
		{}
		SLATE_ARGUMENT(TWeakPtr<ISequencer>, Sequencer)
		SLATE_ARGUMENT(bool, bSmartReduce)
		SLATE_ARGUMENT(float, TolerancePercentage)


	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SMergeAnimLayersWidget() override {}

	FReply OpenDialog(bool bModal = true);
	void CloseDialog();

	void SetMergeCB(FMergeAnimLayersCB& InCB) { MergeCB = InCB; }
private:
	void Merge();

	TWeakPtr<ISequencer> Sequencer;
	//static to be reused
	static TOptional<FMergeAnimLayerSettings> MergeAnimLayersSettings;
	//structonscope for details panel
	TSharedPtr < TStructOnScope<FMergeAnimLayerSettings>> Settings;
	TWeakPtr<SWindow> DialogWindow;
	TSharedPtr<IStructureDetailsView> DetailsView;
	FMergeAnimLayersCB MergeCB;
};


TOptional<FMergeAnimLayerSettings> SMergeAnimLayersWidget::MergeAnimLayersSettings;

void SMergeAnimLayersWidget::Construct(const FArguments& InArgs)
{
	Sequencer = InArgs._Sequencer;

	if (MergeAnimLayersSettings.IsSet() == false)
	{
		TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
		MergeAnimLayersSettings = FMergeAnimLayerSettings();
	}

	if (InArgs._TolerancePercentage != TNumericLimits<float>::Max())
	{
		MergeAnimLayersSettings.GetValue().bReduceKeys = InArgs._bSmartReduce;
		MergeAnimLayersSettings.GetValue().TolerancePercentage = InArgs._TolerancePercentage;
	}

	Settings = MakeShared<TStructOnScope<FMergeAnimLayerSettings>>();
	Settings->InitializeAs<FMergeAnimLayerSettings>(MergeAnimLayersSettings.GetValue());

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	DetailsView = PropertyEditor.CreateStructureDetailView(ViewArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());
	DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber",
		FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.Pin().ToSharedRef(), &ISequencer::MakeFrameNumberDetailsCustomization));
	DetailsView->SetStructureData(Settings);

	ChildSlot
		[
			SNew(SBorder)
				.Visibility(EVisibility::Visible)
				[
					SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.f)
						[
							DetailsView->GetWidget().ToSharedRef()
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(16.f)
						[

							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.FillWidth(1.f)
								.HAlign(HAlign_Fill)
								[
									SNew(SSpacer)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Right)
								.Padding(0.f)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.Text(LOCTEXT("OK", "OK"))
										.OnClicked_Lambda([this, InArgs]()
											{
												Merge();
												CloseDialog();
												return FReply::Handled();

											})
										.IsEnabled_Lambda([this]()
											{
												return (Settings.IsValid());
											})
								]
						]
				]
		];
}

void  SMergeAnimLayersWidget::Merge()
{
	FMergeAnimLayerSettings* BakeSettings = Settings->Get();
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	MergeCB.ExecuteIfBound(SequencerPtr, *BakeSettings);
	MergeAnimLayersSettings = *BakeSettings;
}

class SMergeAnimLayersWidgetWindow : public SWindow
{
};

FReply SMergeAnimLayersWidget::OpenDialog(bool bModal)
{
	check(!DialogWindow.IsValid());

	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

	TSharedRef<SMergeAnimLayersWidgetWindow> Window = SNew(SMergeAnimLayersWidgetWindow)
		.Title(LOCTEXT("MergeAnimLayers", "Merge Anim Layer"))
		.CreateTitleBar(true)
		.Type(EWindowType::Normal)
		.SizingRule(ESizingRule::Autosized)
		.ScreenPosition(CursorPos)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		[
			AsShared()
		];

	Window->SetWidgetToFocusOnActivate(AsShared());

	DialogWindow = Window;

	Window->MoveWindowTo(CursorPos);

	if (bModal)
	{
		GEditor->EditorAddModalWindow(Window);
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}

	return FReply::Handled();
}

void SMergeAnimLayersWidget::CloseDialog()
{
	if (DialogWindow.IsValid())
	{
		DialogWindow.Pin()->RequestDestroyWindow();
		DialogWindow.Reset();
	}
}
///////////////
//end of SMergeAnimLayersWidget
/////////////////////


void FAnimLayerSourcesView::MergeLayers()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			TArray<FAnimLayerSourceUIEntryPtr> Selected;
			SourcesListView->GetSelectedItems(Selected);
			if (Selected.Num() < 2)
			{
				return;
			}
			TArray<int32> LayersToMerge;
			for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
			{
				int32 Index = Ptr->GetAnimLayerIndex();
				if (Index != INDEX_NONE)
				{
					LayersToMerge.Add(Index);
				}
			}
			if (LayersToMerge.Num() > 1)
			{
			
				FMergeAnimLayersCB MergeCB = FMergeAnimLayersCB::CreateLambda([AnimLayers, LayersToMerge, WeakThis = AsWeak()](TSharedPtr<ISequencer>& InSequencer, const FMergeAnimLayerSettings& InSettings)
				{
					AnimLayers->MergeAnimLayers(InSequencer, LayersToMerge, &InSettings);
					if (TSharedPtr<FAnimLayerSourcesView> PinnedView = WeakThis.Pin())
					{
						if (PinnedView->Controller)
						{
							PinnedView->Controller->RefreshSourceData(true);
						}
					}
				});


				TSharedRef<SMergeAnimLayersWidget> BakeWidget =
					SNew(SMergeAnimLayersWidget)
					.Sequencer(SequencerPtr)
					.bSmartReduce(false)
					.TolerancePercentage(5.0);
				
				BakeWidget->SetMergeCB(MergeCB);
				BakeWidget->OpenDialog(false);
			}
		}
	}
}

void FAnimLayerSourcesView::SetPassthroughKey()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() > 0)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetPassthroughKey_Transaction", "Set Passthrough Key"), !GIsTransacting);
		for (const FAnimLayerSourceUIEntryPtr& Ptr : Selected)
		{
			Ptr->SetPassthroughKey();
		}
	}
}

void FAnimLayerSourcesView::Rename()
{
	TArray<FAnimLayerSourceUIEntryPtr> Selected;
	SourcesListView->GetSelectedItems(Selected);
	if (Selected.Num() == 1)
	{
		int32 Index = INDEX_NONE;
		if (SourceData.Find(Selected[0], Index))
		{
			if (Index != INDEX_NONE)
			{
				RenameItem(Index);
			}
		}
	}
}

TSharedPtr<SWidget> FAnimLayerSourcesView::OnSourceConstructContextMenu()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

			TArray<FAnimLayerSourceUIEntryPtr> Selected;
			SourcesListView->GetSelectedItems(Selected);
			int32 BaseLayerIndex = Algo::IndexOfByPredicate(Selected, [AnimLayers](const FAnimLayerSourceUIEntryPtr& Key)
				{
					return (Key.IsValid() && Key->GetAnimLayerIndex() == 0);
				});
			//if we have a base layer selected only show Merge 
			if (BaseLayerIndex != INDEX_NONE)
			{
				if (Selected.Num() > 1)
				{
					MenuBuilder.BeginSection("AnimLayerContextMenuLayer", LOCTEXT("AnimLayerContextMenuLayer", "Layer"));

					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::MergeLayers));
					const FText Label = LOCTEXT("MergeLayers", "Merge Layers");
					const FText ToolTipText = LOCTEXT("MergeLayerstooltip", "Merge selected layers");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				MenuBuilder.AddMenuSeparator();
				return MenuBuilder.MakeWidget();

			}
			else if (Selected.Num() > 0)
			{
				MenuBuilder.BeginSection("AnimLayerContextMenuLayer", LOCTEXT("AnimLayerContextMenuLayer", "Layer"));
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::AddSelected));
					const FText Label = LOCTEXT("AddSelected", "Add Selected");
					const FText ToolTipText = LOCTEXT("AddSelectedTooltip", "Add selection to objects to selected layers");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::RemoveSelected));
					const FText Label = LOCTEXT("RemoveSelected", "Remove Selected");
					const FText ToolTipText = LOCTEXT("RemoveSelectedtooltip", "Remove selection from selected layers");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::SelectObjects));
					const FText Label = LOCTEXT("SelectObjects", "Select Objects");
					const FText ToolTipText = LOCTEXT("SelectObjectsTooltip", "Select all objects in this layer");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}

				MenuBuilder.AddMenuSeparator();

				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::Duplicate));
					const FText Label = LOCTEXT("Duplicate", "Duplicate");
					const FText ToolTipText = LOCTEXT("Duplicatetooltip", "Duplicate to new layer");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				if (Selected.Num() > 1)
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::MergeLayers));
					const FText Label = LOCTEXT("MergeLayers", "Merge Layers");
					const FText ToolTipText = LOCTEXT("MergeLayerstooltip", "Merge selected layers");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::SetPassthroughKey));
					const FText Label = LOCTEXT("SetPassthroughKey", "Passthrough Key");
					const FText ToolTipText = LOCTEXT("SetPassthroughKeytooltip", "Set zero key(Additive) or previous value(Override)");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				//adjustment blending 
				if (Selected.Num() == 1 && Selected[0]->GetAnimLayerIndex() != 0)
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::AdjustmentBlend));
					const FText Label = LOCTEXT("AdjustmentBlend", "Adjustment Blend");
					const FText ToolTipText = LOCTEXT("AdjustmentBlendtooltip", "AdustmentBlend");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
				
				MenuBuilder.AddMenuSeparator();
				if (Selected.Num() == 1)
				{
					{
						FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::Rename));
						const FText Label = LOCTEXT("Rename", "Rename");
						const FText ToolTipText = LOCTEXT("RenameLayerTooltip", "Rename selected layer");
						MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
					}

				}
				{
					FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &FAnimLayerSourcesView::DeleteAnimLayer));
					const FText Label = LOCTEXT("DeletaLayer", "Delete Layer");
					const FText ToolTipText = LOCTEXT("DeleteLayertooltip", "Delete selected layers");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}

				MenuBuilder.EndSection();

				return MenuBuilder.MakeWidget();
			}
		}
	}
	return nullptr;
}

void FAnimLayerSourcesView::OnSourceListSelectionChanged(FAnimLayerSourceUIEntryPtr Entry, ESelectInfo::Type SelectionType) const
{
	if ((Controller && Controller->bSelectionChangedGuard) || bWaitingToUpdateSourceList == true)
	{
		return;
	}
	using namespace UE::Sequencer;
	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		bWaitingToUpdateSourceList = true;
		GEditor->GetTimerManager()->SetTimerForNextTick([WeakThis = AsWeak(), this, WeakSequencer = SequencerPtr.ToWeakPtr()]()
			{
				if (WeakThis.IsValid())
				{
					if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
					{
						if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
						{
							TArray<FAnimLayerSourceUIEntryPtr> Selected;
							SourcesListView->GetSelectedItems(Selected);
							const FAnimLayersScopedSelection ScopedSelection(*AnimLayers);
							FSelectionEventSuppressor SuppressSelectionEvents = SequencerPtr->GetViewModel()->GetSelection()->SuppressEvents();

							for (UAnimLayer* AnimLayer : AnimLayers->AnimLayers)
							{
								AnimLayer->SetSelectedInList(ScopedSelection, false);
							}
							for (const FAnimLayerSourceUIEntryPtr& Item : Selected)
							{
								Item->SetSelectedInList(ScopedSelection, true);
							}
						}
					}
					bWaitingToUpdateSourceList = false;
				}
			});
	}
}


void FAnimLayerSourcesView::RefreshSourceData(bool bRefreshUI)
{
	SourceData.Reset();
	FocusOnIndex = INDEX_NONE;
	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			TArray<int32> Indices;
			UE::AIE::SetAnimLayerIndices(Indices, AnimLayers->AnimLayers.Num());
			for (int32 I = 0; I < AnimLayers->AnimLayers.Num(); ++I)
			{
				int32 Index = Indices[I];
				UAnimLayer* AnimLayer = AnimLayers->AnimLayers[Index];

				if (Controller == nullptr || Controller->IsSelectionFilterActive() == false
					|| AnimLayer->GetSelected() != ECheckBoxState::Unchecked)
				{
					SourceData.Add(MakeShared<FAnimLayerSourceUIEntry>(AnimLayer));
				}
			}
			/*
			for (FGuid SourceGuid : Client->GetDisplayableSources())
			{
				SourceData.Add(MakeShared<FAnimLayerSourceUIEntry>(SourceGuid, Client));
			}
			*/
		}
	}
	if (bRefreshUI)
	{
		SourcesListView->RequestListRefresh();
		TArray< FAnimLayerSourceUIEntryPtr> Selected;
		for (const FAnimLayerSourceUIEntryPtr& Item : SourceData) 
		{
			if (Item->GetSelectedInList())
			{
				Selected.Add(Item);
			}
		}
		if (Selected.Num())
		{
			SourcesListView->SetItemSelection(Selected, true);
		}
	}
}

//////////////////////////////////////////////////////////////
/// FAnimLayerController
///////////////////////////////////////////////////////////

FAnimLayerController::FAnimLayerController()
{

	SourcesView = MakeShared<FAnimLayerSourcesView>();
	SourcesView->SourcesListView->SetSourcesView(SourcesView);
	RebuildSourceList();
}

FAnimLayerController::~FAnimLayerController()
{
	/*
	if (Client)
	{
		Client->OnLiveLinkSourcesChanged().Remove(OnSourcesChangedHandle);
		OnSourcesChangedHandle.Reset();
	}
	*/
}

void FAnimLayerController::RefreshTimeDependantData()
{
	if (SourcesView.IsValid())
	{
		for (const FAnimLayerSourceUIEntryPtr& Item : SourcesView->SourceData)
		{
			Item->GetWeight();
		}
	}
}
void FAnimLayerController::SelectItem(const FAnimLayersScopedSelection& ScopedSelection, int32 Index, bool bClear)
{
	if (SourcesView.IsValid() && SourcesView->SourcesListView.IsValid())
	{
		// SetSelectedInList can trigger a ControlRig selection notification via SelectByNthCategoryNode -> ERigHierarchyNotification ->
		// HandleControlSelected) which calls RefreshSourceData and resets the array mid-loop.
		const TArray<FAnimLayerSourceUIEntryPtr> SourceDataSnapshot = SourcesView->SourceData;
		
		if (bClear)
		{
			SourcesView->SourcesListView->ClearSelection();
			for (const FAnimLayerSourceUIEntryPtr& Item : SourceDataSnapshot)
			{
				Item->SetSelectedInList(ScopedSelection, false);
			}
		}
		
		if (SourceDataSnapshot.IsValidIndex(Index))
		{
			SourceDataSnapshot[Index]->SetSelectedInList(ScopedSelection, true);
			SourcesView->SourcesListView->SetItemSelection(SourceDataSnapshot[Index], true);
		}
	}
}

void FAnimLayerController::RefreshSelectionData()
{
	if (!SourcesView.IsValid() || !SourcesView->SourcesListView.IsValid())
	{
		return;
	}

	bSelectionChangedGuard = true;

	SourcesView->SourcesListView->ClearSelection();

	for (const FAnimLayerSourceUIEntryPtr& Item : SourcesView->SourceData)
	{
		Item->ClearCaches();

		if (Item->GetSelectedInList())
		{
			SourcesView->SourcesListView->SetItemSelection(Item, true);
		}
	}

	bSelectionChangedGuard = false;
}

void FAnimLayerController::RefreshSourceData(bool bRefreshUI)
{
	if (SourcesView.IsValid())
	{
		SourcesView->RefreshSourceData(bRefreshUI);
	}
}
void FAnimLayerController::ToggleSelectionFilterActive()
{
	bSelectionFilterActive = !bSelectionFilterActive;
	RebuildSourceList();
}

bool FAnimLayerController::IsSelectionFilterActive() const
{
	return bSelectionFilterActive;
}

void FAnimLayerController::OnSourceSelectionChangedHandler(FAnimLayerSourceUIEntryPtr Entry, ESelectInfo::Type SelectionType) const
{
	if (bSelectionChangedGuard)
	{
		return;
	}
}

void FAnimLayerController::RebuildSourceList()
{
	SourcesView->RefreshSourceData(true);
}

//////////////////////////////////////////////////////////////
/// SAnimLayers
///////////////////////////////////////////////////////////

void SAnimLayers::Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode)
{
	AnimLayerController = MakeShared<FAnimLayerController>();
	AnimLayerController->SourcesView->AddController(AnimLayerController.Get());
	LastMovieSceneSig = FGuid();
	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		SequencerPtr->OnActivateSequence().AddRaw(this, &SAnimLayers::OnActivateSequence);
		SequencerPtr->OnMovieSceneDataChanged().AddRaw(this, &SAnimLayers::OnMovieSceneDataChanged);
		SequencerPtr->OnGlobalTimeChanged().AddRaw(this, &SAnimLayers::OnGlobalTimeChanged);
		SequencerPtr->OnEndScrubbingEvent().AddRaw(this, &SAnimLayers::OnGlobalTimeChanged);
		SequencerPtr->OnStopEvent().AddRaw(this, &SAnimLayers::OnGlobalTimeChanged);

		if (UAnimLayers* AnimLayersPtr = UAnimLayers::GetAnimLayers(SequencerPtr.Get()))
		{
			OnAnimLayerListChangedHandle = AnimLayersPtr->AnimLayerListChanged().AddRaw(AnimLayerController.Get(), &FAnimLayerController::HandleOnAnimLayerListChanged);
			AnimLayers = AnimLayersPtr;
		}
	}
	ChildSlot
	[
		SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(4.0f, 0.f,0.f,0.f))
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(4.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(.0f)
					.HAlign(HAlign_Fill)
					.FillWidth(1.0f)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.Padding(.0f)
						[
							SNew(SPositiveActionButton)
							.OnClicked(this, &SAnimLayers::OnAddClicked)
							.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
							.Text(LOCTEXT("AnimLayer", "Layer"))
							.ToolTipText(LOCTEXT("AnimLayerTooltip", "Add a new Animation Layer"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(10.f)
						[
							SNew(SSpacer)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.Padding(5.0f)
						.VAlign(VAlign_Center)
						[
							SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "NoBorder")
								.ButtonColorAndOpacity_Lambda([this]()
									{
										return FSlateColor(FLinearColor(1.f, 1.f, 1.f, 1));
									})
								.OnClicked_Lambda([this]()
									{
										AnimLayerController->ToggleSelectionFilterActive();
										return FReply::Handled();
									})
								.ContentPadding(1.f)
								.ToolTipText(LOCTEXT("AnimLayerSelectionFilerTooltip", "Only show Anim Layers with selected objects"))
								[
									SNew(SImage)
										.ColorAndOpacity_Lambda([this]()
											{

												const FLinearColor Selected = FLinearColor::White;
												const FColor NotSelected(56, 56, 56);
												FSlateColor SlateColor;
												if (AnimLayerController->IsSelectionFilterActive() == true)
												{
													SlateColor = FSlateColor(Selected);
												}
												else
												{
													SlateColor = FSlateColor(NotSelected);
												}
												return SlateColor;
											})
										.Image(FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.FilterAnimLayerSelected").GetIcon())
								]
						]	
					]
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				AnimLayerController->SourcesView->SourcesListView.ToSharedRef()
			]
	];
	SetEditMode(InEditMode);
	RegisterSelectionChanged();
	SetCanTick(true);

}

void SAnimLayers::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (ISequencer* Sequencer = GetSequencer())
	{
		FGuid CurrentMovieSceneSig = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
		if (LastMovieSceneSig != CurrentMovieSceneSig)
		{
			LastMovieSceneSig = CurrentMovieSceneSig;

			if (AnimLayers.IsValid() == false)
			{
				if (UAnimLayers* AnimLayersPtr = UAnimLayers::GetAnimLayers(Sequencer))
				{
					OnAnimLayerListChangedHandle = AnimLayersPtr->AnimLayerListChanged().AddRaw(AnimLayerController.Get(), &FAnimLayerController::HandleOnAnimLayerListChanged);
					AnimLayers = AnimLayersPtr;
					AnimLayerController->RefreshSourceData(true);
				}
			}
			AnimLayerController->RefreshSelectionData();
		}
	}
}


FReply SAnimLayers::OnSelectionFilterClicked()
{
	AnimLayerController->ToggleSelectionFilterActive();
	return FReply::Handled();
}

bool SAnimLayers::IsSelectionFilterActive() const
{
	return AnimLayerController->IsSelectionFilterActive();
}

//mz todo, if in layers with control rig's need to replace them.
void SAnimLayers::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	//if there's a control rig recreate the tree, controls may have changed
	bool bNewControlRig = false;
	for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
	{
		if (Pair.Key && Pair.Value)
		{
			if (Pair.Key->IsA<UControlRig>() && Pair.Value->IsA<UControlRig>())
			{
				bNewControlRig = false;
				break;
			}
		}
	}
}

SAnimLayers::SAnimLayers()
{
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &SAnimLayers::OnObjectsReplaced);
}

SAnimLayers::~SAnimLayers()
{
	if (AnimLayers.IsValid())
	{
		AnimLayers.Get()->AnimLayerListChanged().Remove(OnAnimLayerListChangedHandle);
		OnAnimLayerListChangedHandle.Reset();
	}

	if (TSharedPtr<ISequencer> SequencerPtr = UAnimLayers::GetSequencerFromAsset())
	{
		SequencerPtr->OnActivateSequence().RemoveAll(this);
		SequencerPtr->OnMovieSceneDataChanged().RemoveAll(this);
		SequencerPtr->OnGlobalTimeChanged().RemoveAll(this);
		SequencerPtr->OnEndScrubbingEvent().RemoveAll(this);
		SequencerPtr->OnStopEvent().RemoveAll(this);

	}
	
	// unregister previous one
	if (OnSelectionChangedHandle.IsValid())
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		FLevelEditorModule::FActorSelectionChangedEvent& ActorSelectionChangedEvent = LevelEditor.OnActorSelectionChanged();
		ActorSelectionChangedEvent.Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}	
	
	for (TWeakObjectPtr<UControlRig>& ControlRig : BoundControlRigs)
	{
		if (ControlRig.IsValid())
		{
			ControlRig.Get()->ControlRigBound().RemoveAll(this);
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig.Get()->GetObjectBinding();
			if (Binding)
			{
				Binding->OnControlRigBind().RemoveAll(this);
			}
		}
	}
	BoundControlRigs.SetNum(0);
}

FReply SAnimLayers::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TWeakPtr<ISequencer> Sequencer = EditMode->GetWeakSequencer();
		if (Sequencer.IsValid())
		{

			if (Sequencer.Pin()->GetCommandBindings(ESequencerCommandBindings::CurveEditor).Get()->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
		}
	}
	return FReply::Unhandled();
}

void SAnimLayers::HandleControlSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected)
{
	FControlRigBaseDockableView::HandleControlSelected(Subject, ControlElement, bSelected);
	if (AnimLayerController)
	{
		if (AnimLayerController->IsSelectionFilterActive())
		{
			AnimLayerController->RefreshSourceData(true);
		}
		AnimLayerController->RefreshSelectionData();
	}
}

void SAnimLayers::RegisterSelectionChanged()
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	FLevelEditorModule::FActorSelectionChangedEvent& ActorSelectionChangedEvent = LevelEditor.OnActorSelectionChanged();

	// unregister previous one
	if (OnSelectionChangedHandle.IsValid())
	{
		ActorSelectionChangedEvent.Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}

	// register
	OnSelectionChangedHandle = ActorSelectionChangedEvent.AddRaw(this, &SAnimLayers::OnActorSelectionChanged);
}

void SAnimLayers::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	if (AnimLayerController)
	{
		if (AnimLayerController->IsSelectionFilterActive())
		{
			AnimLayerController->RefreshSourceData(true);
		}
		AnimLayerController->RefreshSelectionData();
	}
}

void SAnimLayers::OnActivateSequence(FMovieSceneSequenceIDRef ID)
{
	if (!GIsTransacting && AnimLayerController) //don't do this if in UNDO, Sequencer unfortunately calls this delegate then
	{
		AnimLayerController->RefreshSourceData(true);
		AnimLayerController->RefreshSelectionData();
	}
}

void SAnimLayers::OnGlobalTimeChanged()
{
	if (AnimLayerController)
	{
		AnimLayerController->RefreshTimeDependantData();
	}
}

void SAnimLayers::OnMovieSceneDataChanged(EMovieSceneDataChangeType)
{
	if (AnimLayerController)
	{
		AnimLayerController->RefreshTimeDependantData();
		AnimLayerController->RefreshSelectionData();
	}
}

FReply SAnimLayers::OnAddClicked()
{
	if (ISequencer* SequencerPtr = GetSequencer())
	{
		const UE::ControlRig::FSaveScopedControlRigSelection SaveScopedControlRigSelection(SequencerPtr->AsWeak());

		if (UAnimLayers* AnimLayersPtr = UAnimLayers::GetAnimLayers(SequencerPtr, true/*bAddIfDoesNotExist */ ))
		{
			const FAnimLayersScopedSelection ScopedSelection(*AnimLayersPtr);

			int32 Index = AnimLayersPtr->AddAnimLayerFromSelection(SequencerPtr);
			if (Index != INDEX_NONE)
			{
				if (AnimLayerController.IsValid())
				{
					constexpr bool bRefreshUI = true;
					AnimLayerController->RefreshSourceData(bRefreshUI);
					AnimLayerController->FocusRenameLayer(Index);
					AnimLayerController->SelectItem(ScopedSelection, Index, true);
				}
			}
		}
	}

	return FReply::Handled();
}

void SAnimLayers::SetEditMode(FControlRigEditMode& InEditMode)
{
	FControlRigBaseDockableView::SetEditMode(InEditMode);
	ModeTools = InEditMode.GetModeManager();
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TArrayView<TWeakObjectPtr<UControlRig>> ControlRigs = EditMode->GetControlRigs();
		for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
		{
			if (ControlRig.IsValid())
			{
				if (!ControlRig.Get()->ControlRigBound().IsBoundToObject(this))
				{
					ControlRig.Get()->ControlRigBound().AddRaw(this, &SAnimLayers::HandleOnControlRigBound);
					BoundControlRigs.Add(ControlRig);
				}
				const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig.Get()->GetObjectBinding();
				if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
				{
					Binding->OnControlRigBind().AddRaw(this, &SAnimLayers::HandleOnObjectBoundToControlRig);
				}
			}
		}
	}
}

void SAnimLayers::HandleControlAdded(UControlRig* ControlRig, bool bIsAdded)
{
	FControlRigBaseDockableView::HandleControlAdded(ControlRig, bIsAdded);
	if (ControlRig)
	{
		if (bIsAdded == true)
		{
			if (!ControlRig->ControlRigBound().IsBoundToObject(this))
			{
				ControlRig->ControlRigBound().AddRaw(this, &SAnimLayers::HandleOnControlRigBound);
				BoundControlRigs.Add(ControlRig);
			}
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
			{
				Binding->OnControlRigBind().AddRaw(this, &SAnimLayers::HandleOnObjectBoundToControlRig);
			}
		}
		else
		{
			BoundControlRigs.Remove(ControlRig);
			ControlRig->ControlRigBound().RemoveAll(this);
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			if (Binding)
			{
				Binding->OnControlRigBind().RemoveAll(this);
			}
		}
	}
}

void SAnimLayers::PostUndo(bool bSuccess)
{
	if (AnimLayerController.IsValid())
	{
		constexpr bool bRefreshUI = true;
		AnimLayerController->RefreshSourceData(bRefreshUI);
	}
}

void SAnimLayers::PostRedo(bool bSuccess)
{
	if (AnimLayerController.IsValid())
	{
		constexpr bool bRefreshUI = true;
		AnimLayerController->RefreshSourceData(bRefreshUI);
	}
}

void SAnimLayers::HandleOnControlRigBound(UControlRig* InControlRig)
{
	if (!InControlRig)
	{
		return;
	}

	const TSharedPtr<IControlRigObjectBinding> Binding = InControlRig->GetObjectBinding();

	if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
	{
		Binding->OnControlRigBind().AddRaw(this, &SAnimLayers::HandleOnObjectBoundToControlRig);
	}
}

//mz todo need to test recompiling
void SAnimLayers::HandleOnObjectBoundToControlRig(UObject* InObject)
{
	
}

class SInvalidWeightNameDetailWidget : public SSpacer
{
	SLATE_BEGIN_ARGS(SInvalidWeightNameDetailWidget)
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SetVisibility(EVisibility::Collapsed);
	}

};

class  FWeightNameOverride : public FDetailsNameWidgetOverrideCustomization
{
public:
	FWeightNameOverride() {};
	virtual ~FWeightNameOverride() override = default;
	virtual TSharedRef<SWidget> CustomizeName(TSharedRef<SWidget> InnerNameContent, FPropertyPath& Path) override 
	{
		const TSharedRef<SWidget> NameContent = SNew(SInvalidWeightNameDetailWidget);
		return NameContent;
	}
};

void SAnimWeightDetails::Construct(const FArguments& InArgs, FControlRigEditMode* InEditMode, UObject* InWeightObject)
{
	if (InEditMode == nullptr || InWeightObject == nullptr)
	{
		return;
	}
	using namespace UE::Sequencer;

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bCustomNameAreaLocation = false;
		DetailsViewArgs.bCustomFilterAreaLocation = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = true;
		DetailsViewArgs.bShowScrollBar = false; // Don't need to show this, as we are putting it in a scroll box
		DetailsViewArgs.DetailsNameWidgetOverrideCustomization = MakeShared<FWeightNameOverride>();

	}

	WeightView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	WeightView->SetKeyframeHandler(InEditMode->DetailKeyFrameCache);


	ChildSlot
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				WeightView.ToSharedRef()
			]
		]
	];
	TArray<UObject*> Objects;
	Objects.Add(InWeightObject);
	WeightView->SetObjects(Objects, true);
}

SAnimWeightDetails::~SAnimWeightDetails()
{
}



#undef LOCTEXT_NAMESPACE



