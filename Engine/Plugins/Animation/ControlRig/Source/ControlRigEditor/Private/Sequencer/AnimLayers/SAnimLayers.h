// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "EditMode/ControlRigBaseDockableView.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "ISequencer.h"
#include "Misc/Guid.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UControlRig;
class FControlRigEditMode;
struct FRigControlElement;
struct FRigElementKey;
struct FAnimLayerController;
class UAnimLayers;
enum class EMovieSceneDataChangeType;


class SAnimLayers : public FControlRigBaseDockableView, public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
	SLATE_BEGIN_ARGS(SAnimLayers) {}
	SLATE_END_ARGS()
	SAnimLayers();
	~SAnimLayers();

	void Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode);
	//SCompuntWidget
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//FControlRigBaseDockableView overrides
	virtual TSharedRef<FControlRigBaseDockableView> AsSharedWidget() override { return SharedThis(this); }
	virtual void SetEditMode(FControlRigEditMode& InEditMode) override;
	virtual void HandleControlAdded(UControlRig* ControlRig, bool bIsAdded) override;

private:
	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

	void HandleOnControlRigBound(UControlRig* InControlRig);
	void HandleOnObjectBoundToControlRig(UObject* InObject);
	virtual void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected) override;
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	//set of control rigs we are bound too and need to clear delegates from
	TArray<TWeakObjectPtr<UControlRig>> BoundControlRigs;
private:
	//actor selection changing
	void RegisterSelectionChanged();
	void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);
	FDelegateHandle OnSelectionChangedHandle;
	FDelegateHandle OnAnimLayerListChangedHandle;

	//sequencer changing so need to refresh list or update weight
	void OnActivateSequence(FMovieSceneSequenceIDRef ID);
	void OnGlobalTimeChanged();
	void OnMovieSceneDataChanged(EMovieSceneDataChangeType);
	FGuid LastMovieSceneSig = FGuid();

private:
	FReply OnAddClicked();
	FReply OnSelectionFilterClicked();
	bool IsSelectionFilterActive() const;
	TSharedPtr<FAnimLayerController> AnimLayerController;
	TWeakObjectPtr<UAnimLayers> AnimLayers;
private:
	//Keyboard interaction for Sequencer hotkeys
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
};

class SAnimWeightDetails : public SCompoundWidget
{

	SLATE_BEGIN_ARGS(SAnimWeightDetails)
	{}
	SLATE_END_ARGS()
	~SAnimWeightDetails();

	void Construct(const FArguments& InArgs, FControlRigEditMode* InEditMode, UObject* InWeightObject);

private:

	TSharedPtr<IDetailsView> WeightView;

};


