// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/AnimationAuthoringSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationAuthoringSettings)

UAnimationAuthoringSettings::UAnimationAuthoringSettings(const FObjectInitializer& ObjectInitializer)
	: UDeveloperSettings(ObjectInitializer)
{}

UAnimationAuthoringSettings::FOnUpdateSettings UAnimationAuthoringSettings::OnSettingsChange;

void UAnimationAuthoringSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	OnSettingsChange.Broadcast(this);
}
