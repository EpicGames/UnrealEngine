// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlConstraintChannelInterface.h"

#include "ControlRigSpaceChannelEditors.h"
#include "ISequencer.h"

#include "Constraints/ControlRigTransformableHandle.h"
#include "Transform/TransformConstraintUtil.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "LevelSequence.h"
#include "MovieSceneToolHelpers.h"

#include "Transform/TransformConstraint.h"
#include "Algo/Copy.h"
#include "ConstraintsManager.h"
#include "ScopedTransaction.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Constraints/MovieSceneConstraintChannelHelper.inl"

#define LOCTEXT_NAMESPACE "Constraints"

namespace
{
	UMovieScene* GetMovieScene(const TSharedPtr<ISequencer>& InSequencer)
	{
		const UMovieSceneSequence* MovieSceneSequence = InSequencer ? InSequencer->GetFocusedMovieSceneSequence() : nullptr;
		return MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	}
}

UMovieSceneSection* FControlConstraintChannelInterface::GetHandleSection(
	const UTransformableHandle* InHandle,
	const TSharedPtr<ISequencer>& InSequencer)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}
	
	const UTransformableControlHandle* ControlHandle = static_cast<const UTransformableControlHandle*>(InHandle);
	static constexpr bool bConstraintSection = false;
	return GetControlSection(ControlHandle, InSequencer, bConstraintSection);
}

UMovieSceneSection* FControlConstraintChannelInterface::GetHandleConstraintSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}

	const UTransformableControlHandle* ControlHandle = static_cast<const UTransformableControlHandle*>(InHandle);
	static constexpr bool bConstraintSection = true;
	return GetControlSection(ControlHandle, InSequencer, bConstraintSection);
}

UWorld* FControlConstraintChannelInterface::GetHandleWorld(UTransformableHandle* InHandle)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}
	
	const UTransformableControlHandle* ControlHandle = static_cast<UTransformableControlHandle*>(InHandle);
	const UControlRig* ControlRig = ControlHandle->ControlRig.LoadSynchronous();

	return ControlRig ? ControlRig->GetWorld() : nullptr;
}

bool FControlConstraintChannelInterface::SmartConstraintKey(
	UTickableTransformConstraint* InConstraint, const TOptional<bool>& InOptActive,
	const TRange<FFrameNumber>& InTimeRange, const TSharedPtr<ISequencer>& InSequencer)
{
	if (!InConstraint)
	{
		return false;
	}

	// TODO handle range
	const bool bInfiniteInput = !InTimeRange.HasLowerBound();
	const TOptional<FFrameNumber> Time = bInfiniteInput ? TOptional<FFrameNumber>() : InTimeRange.GetLowerBoundValue();

	const UTransformableControlHandle* ControlHandle = Cast<UTransformableControlHandle>(InConstraint->ChildTRSHandle);
	if (!ControlHandle)
	{
		return false;
	}

	UMovieSceneControlRigParameterSection* ConstraintSection = GetControlSection(ControlHandle, InSequencer, true);
	UMovieSceneControlRigParameterSection* TransformSection = GetControlSection(ControlHandle, InSequencer, false);
	if ((!ConstraintSection) || (!TransformSection))
	{
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("KeyControlConstraintKey", "Key Control Constraint Key"));
	ConstraintSection->Modify();
	TransformSection->Modify();

	// set constraint as dynamic
	InConstraint->bDynamicOffset = true;
	
	//check if static if so we need to delete it from world, will get added later again
	if (UConstraintsManager* Manager = InConstraint->GetTypedOuter<UConstraintsManager>())
	{
		Manager->RemoveStaticConstraint(InConstraint);
	}

	// add the channel
	const bool bHadConstraintChannel = InConstraint && ConstraintSection->HasConstraintChannel(InConstraint->ConstraintID);
	ConstraintSection->AddConstraintChannel(InConstraint);

	// add key if needed
	if (FConstraintAndActiveChannel* Channel = ConstraintSection->GetConstraintChannel(InConstraint->ConstraintID))
	{
		if (bInfiniteInput)
		{
			ensure(Channel->ActiveChannel.GetNumKeys() == 0);
		}
		
		bool ActiveValueToBeSet = bInfiniteInput;
		//add key if we can and make sure the key we are setting is what we want
		if (bInfiniteInput || (CanAddKey(Channel->ActiveChannel, *Time, ActiveValueToBeSet) && (InOptActive.IsSet() == false || InOptActive.GetValue() == ActiveValueToBeSet)))
		{
			const bool bNeedsCompensation = InConstraint->NeedsCompensation();
				
			TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
			TGuardValue<bool> RemoveConstraintGuard(FConstraintsManagerController::bDoNotRemoveConstraint, true);

			UControlRig* ControlRig = ControlHandle->ControlRig.Get();
			const FName& ControlName = ControlHandle->ControlName;
				
			// store the frames to compensate
			const TArrayView<FMovieSceneFloatChannel*> Channels = ControlHandle->GetFloatChannels(TransformSection);
			TArray<FFrameNumber> FramesToCompensate;
			if (bNeedsCompensation)
			{
				FMovieSceneConstraintChannelHelper::GetFramesToCompensate<FMovieSceneFloatChannel>(Channel->ActiveChannel, ActiveValueToBeSet, Time, Channels, FramesToCompensate);
			}
			else
			{
				if (!bInfiniteInput)
				{
					FramesToCompensate.Add(*Time);
				}
			}

			const bool bIsActiveByDefault = Channel->ActiveChannel.GetDefault() && *Channel->ActiveChannel.GetDefault();
			const bool bWasInfinite = bIsActiveByDefault && Channel->ActiveChannel.GetNumKeys() == 0;
			
			const FFrameRate TickResolution = InSequencer->GetFocusedTickResolution();
			const FFrameTime CurrentTime = InSequencer->GetLocalTime().ConvertTo(TickResolution);
			const FFrameNumber CurrentFrame = CurrentTime.GetFrame();
			
			TOptional<FFrameNumber> OptCurrentTime;
			bool bPropagateRelative = true;
			if (!bHadConstraintChannel && ActiveValueToBeSet)
			{
				if (!bInfiniteInput && !bWasInfinite)
				{
					// check if the first active key will be added before the current frame
                    // if that's the case then the current frame must be added and the compensation evaluator notified
					Tie(OptCurrentTime, bPropagateRelative) =
						IsActiveKeyAddedBeforeCurrentTime(CurrentFrame, *Time, Channels, FramesToCompensate);
				}
				else if (bInfiniteInput && !FramesToCompensate.IsEmpty())
				{
					// make sure to add the current frame first if the child has transforms to compensate
					FramesToCompensate.Remove(CurrentFrame);
					FramesToCompensate.Insert(CurrentFrame, 0);
					OptCurrentTime = CurrentFrame;
					bPropagateRelative = false;
				}
			}
			
			FCompensationEvaluator::FEvalParameters EvalParams(InSequencer, FramesToCompensate);
			EvalParams.bToActive = ActiveValueToBeSet;
			EvalParams.OptCurrentFrame = OptCurrentTime;
			EvalParams.bPropagateRelative = bPropagateRelative;

			if (bWasInfinite)
			{
				EvalParams.bToActive = false;
			}
			else if (bIsActiveByDefault && Time)
			{
				const TMovieSceneChannelData<const bool> ChannelData = Channel->ActiveChannel.GetData();
				const TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
				const TArrayView<const bool> Values = ChannelData.GetValues();
				
				const bool bInfiniteAtLeft = *Time < Times[0] && !Values[0];
				EvalParams.bInfiniteAtLeft = bInfiniteAtLeft;
			}
			
			if (bHadConstraintChannel && ActiveValueToBeSet && !bWasInfinite)
			{
				const TMovieSceneChannelData<const bool> ChannelData = Channel->ActiveChannel.GetData();
				if (ChannelData.GetTimes().IsEmpty() && !bInfiniteInput)
				{
					// this means that the channel already existed but there was no existing key
					// e.g., when the user deletes keys directly in sequencer without deleting the constraint
					EvalParams.bKeepCurrent = !FramesToCompensate.IsEmpty() && FramesToCompensate[0] == CurrentTime.GetFrame();
				}
			}
			
			// store child and space transforms for these frames
			FCompensationEvaluator Evaluator(InConstraint);
			Evaluator.ComputeLocalTransforms(ControlRig->GetWorld(), EvalParams);
			TArray<FTransform>& ChildLocals = Evaluator.ChildLocals;

			const bool bUpdateDefault = bNeedsCompensation && ChildLocals.IsEmpty() && bInfiniteInput;
			
			// store tangents at this time
			TArray<FMovieSceneTangentData> Tangents;
			int32 ChannelIndex = 0, NumChannels = 0;

			FChannelMapInfo* pChannelIndex = nullptr;
			FRigControlElement* ControlElement = nullptr;
			Tie(ControlElement, pChannelIndex) = FControlRigSpaceChannelHelpers::GetControlAndChannelInfo(ControlRig, TransformSection, ControlName);

			if (pChannelIndex && ControlElement)
			{
				// get the number of float channels to treat
				NumChannels = FControlRigSpaceChannelHelpers::GetNumFloatChannels(ControlElement->Settings.ControlType);
				if (bNeedsCompensation && !bInfiniteInput && NumChannels > 0)
				{
					ChannelIndex = pChannelIndex->ChannelIndex;
					EvaluateTangentAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, TransformSection, *Time, Tangents);
				}
			}
		
			const EMovieSceneTransformChannel ChannelsToKey =InConstraint->GetChannelsToKey();
			
			// add child's transform key at Time-1 to keep animation
			if (bNeedsCompensation && !bInfiniteInput && !ChildLocals.IsEmpty())
			{
				const FFrameNumber TimeMinusOne(*Time - 1);

				ControlHandle->AddTransformKeys({ TimeMinusOne },
					{ ChildLocals[0] }, ChannelsToKey, TickResolution, ConstraintSection, true);

				// set tangents at Time-1
				if (NumChannels > 0) //-V547
				{
					SetTangentsAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, TransformSection, TimeMinusOne, Tangents);
				}
			}

			// add active key
			if (bInfiniteInput)
			{
				constexpr bool bActiveByDefault = true;
				Channel->ActiveChannel.SetDefault(bActiveByDefault);
			}
			else
			{
				TMovieSceneChannelData<bool> ChannelData = Channel->ActiveChannel.GetData();

				const TArrayView<const FFrameNumber> FrameNumbers = ChannelData.GetTimes();
				const bool bRemoveInfinite = bIsActiveByDefault && ActiveValueToBeSet && (FrameNumbers.IsEmpty() || *Time < FrameNumbers[0]); 
				
				ChannelData.UpdateOrAddKey(*Time, ActiveValueToBeSet);

				if (bRemoveInfinite)
				{
					Channel->ActiveChannel.RemoveDefault();
				}
			}

			// compensate
			{
				// we need to remove the first transforms as we store NumFrames+1 transforms
				if (!ChildLocals.IsEmpty())
				{
					ChildLocals.RemoveAt(0);
				}

				// add keys
				constexpr bool bLocal = true;
				ControlHandle->AddTransformKeys(FramesToCompensate, ChildLocals, ChannelsToKey, TickResolution, ConstraintSection, bLocal);

				// update default if needed
				if (bUpdateDefault)
				{
					const FTransform Local = UE::TransformConstraintUtil::ComputeRelativeTransform(
						ControlHandle->GetLocalTransform(),
						ControlHandle->GetGlobalTransform(),
						InConstraint->GetParentGlobalTransform(),
						InConstraint);

					FRigControlModifiedContext Context;
					Context.KeyMask = static_cast<uint32>(ChannelsToKey);
					
					static constexpr bool bNotify = true, bUndo = false, bFixEuler = true;
					ControlRig->SetControlLocalTransform(ControlName, Local, bNotify, Context, bUndo, bFixEuler);
					if (ControlRig->IsAdditive())
					{
						ControlRig->Evaluate_AnyThread();
					}
				}

				// set tangents at Time
				if (bNeedsCompensation && !bInfiniteInput && NumChannels > 0)
				{
					SetTangentsAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, TransformSection, *Time, Tangents);
				}
			}

			// cleanup next keys that have the same value as they are useless
			if (!bInfiniteInput)
			{
				TArray<FFrameNumber> TimesRemoved;
				CleanDuplicates(Channel->ActiveChannel, TimesRemoved);

				// as compensation has been disabled earlier (FMovieSceneConstraintChannelHelper::bDoNotCompensate set to true),
				// previous transform compensation keys must be explicitly removed as HandleConstraintKeyDeleted won't do it
				for (const FFrameNumber& TimeToRemove: TimesRemoved)
				{
					const FFrameNumber TimeMinusOne(TimeToRemove - 1);
					FMovieSceneConstraintChannelHelper::DeleteTransformKeys(Channels, TimeMinusOne);
				}
			}
			
			return true;
		}
	}
	return false;
}

void FControlConstraintChannelInterface::AddHandleTransformKeys(
	const TSharedPtr<ISequencer>& InSequencer,
	const UTransformableHandle* InHandle,
	const TArray<FFrameNumber>& InFrames,
	const TArray<FTransform>& InLocalTransforms,
	const EMovieSceneTransformChannel& InChannels)
{
	ensure(InLocalTransforms.Num());

	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return;
	}

	const UTransformableControlHandle* Handle = static_cast<const UTransformableControlHandle*>(InHandle);
	const UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	UMovieSceneSection* Section = GetHandleSection(InHandle, InSequencer);
	
	Handle->AddTransformKeys(InFrames, InLocalTransforms, InChannels, MovieScene->GetTickResolution(), Section, true);
}

UMovieSceneControlRigParameterSection* FControlConstraintChannelInterface::GetControlSection(
	const UTransformableControlHandle* InHandle,
	const TSharedPtr<ISequencer>& InSequencer,
	const bool bIsConstraint)
{
	if (!IsValid(InHandle) || !InHandle->IsValid())
	{
		return nullptr;
	}
	
	const UControlRig* ControlRig = InHandle->ControlRig.LoadSynchronous();
	if (!ControlRig)
	{
		return nullptr;
	}
	
	const UMovieScene* MovieScene = GetMovieScene(InSequencer);
	if (!MovieScene)
	{
		return nullptr;
	}

	auto GetControlRigTrack = [InHandle, MovieScene]()->UMovieSceneControlRigParameterTrack*
	{
		const TWeakObjectPtr<UControlRig> ControlRig = InHandle->ControlRig.LoadSynchronous();
		if (ControlRig.IsValid())
		{	
			const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
			for (const FMovieSceneBinding& Binding : Bindings)
			{
				for (UMovieSceneTrack* Track : MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid()))
				{
					UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
					if (ControlRigTrack && ControlRigTrack->GetControlRig() == ControlRig)
					{
						return ControlRigTrack;
					}
				}
			}
		}
		return nullptr;
	};

	UMovieSceneControlRigParameterTrack* ControlRigTrack = GetControlRigTrack();
	if (!ControlRigTrack)
	{
		return nullptr;
	}

	UMovieSceneSection* Section = ControlRigTrack->FindSection(0);
	if (bIsConstraint)
	{
		const TArray<UMovieSceneSection*>& AllSections = ControlRigTrack->GetAllSections();
		if (!AllSections.IsEmpty())
		{
			Section = AllSections[0]; 
		}
	}

	return Cast<UMovieSceneControlRigParameterSection>(Section);
}

void FControlConstraintChannelInterface::UnregisterTrack(UMovieSceneTrack* InTrack, UWorld* InWorld)
{
	ITransformConstraintChannelInterface::UnregisterTrack(InTrack, InWorld);
	
	UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(InTrack);
	if (!ControlRigTrack)
	{
		return;
	}

	const TArray<UMovieSceneSection*>& AllSections = ControlRigTrack->GetAllSections();
	UMovieSceneControlRigParameterSection* Section =
		AllSections.IsEmpty() ? nullptr : Cast<UMovieSceneControlRigParameterSection>(AllSections[0]);
	
	if (!Section)
	{
		return;
	}

	UnregisterConstraints(Section, InWorld);
}

#undef LOCTEXT_NAMESPACE
