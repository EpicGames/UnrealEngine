// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Styling/SlateStyleRegistry.h"
#include "Sequencer/MovieSceneControlRigSystem.h"
#include "Baker/SequencerBakerSubsystem.h"
#include "Baker/SequencerBaker.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/Extensions/ISectionOwnerExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Animation/AnimMontage.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Framework/Commands/Commands.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SSpinBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "SequencerSectionPainter.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor.h"
#include "ISequencerChannelInterface.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SequencerUtilities.h"
#include "MVVM/Views/ViewUtilities.h"
#include "ISectionLayoutBuilder.h"
#include "Styling/AppStyle.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Fonts/FontMeasure.h"
#include "AnimationEditorUtils.h"
#include "Misc/AxisDisplayInfo.h"
#include "Misc/MessageDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/Blueprint.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "ControlRigObjectBinding.h"
#include "LevelEditorViewport.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "ControlRigEditorModule.h"
#include "SequencerSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Channels/FloatChannelCurveModel.h"
#include "TransformNoScale.h"
#include "ControlRigComponent.h"
#include "ISequencerObjectChangeListener.h"
#include "MovieSceneToolHelpers.h"
#include "Rigs/FKControlRig.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Units/Execution/RigUnit_InverseExecution.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Exporters/AnimSeqExportOption.h"
#include "SBakeToControlRigDialog.h"
#include "ControlRigBlueprintLegacy.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Animation/SkeletalMeshActor.h"
#include "TimerManager.h"
#include "BakeToControlRigSettings.h"
#include "BakeToControlRigHelper.h"
#include "LoadAnimToControlRigSettings.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "Toolkits/IToolkitHost.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "ControlRigSpaceChannelEditors.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/TransactionObjectEvent.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "PropertyEditorModule.h"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "BakingAnimationKeySettings.h"
#include "ControlRigSelectionUtils.h"
#include "FrameNumberDetailsCustomization.h"
#include "Editor/UnrealEd/Private/FbxExporter.h"
#include "Sequencer/ControlRigSequencerHelpers.h"
#include "Widgets/Layout/SSpacer.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "LevelSequence.h"
#include "ISequencerModule.h"
#include "CurveModel.h"
#include "Sequencer/SLoadAnimToControlRig.h"
#include "FrontendFilterBase.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "Editor/RigVMEditorTools.h"
#include "Sequencer/AnimLayers/AnimLayers.h"
#include "Tools/EvaluateSequencerTools.h"
#include "Decorations/MovieSceneMuteSoloDecoration.h"
#include "Editor/Constraints/ConstraintCreationOptions.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"
#include "SControlRigAssetReferencePicker.h"
#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "Tracks/MovieSceneCommonAnimationTrack.h"


#define LOCTEXT_NAMESPACE "FControlRigParameterTrackEditor"

bool FControlRigParameterTrackEditor::bControlRigEditModeWasOpen = false;
TArray <TPair<FControlRigAssetStrongReference, TArray<FName>>> FControlRigParameterTrackEditor::PreviousSelectedControlRigs;

TAutoConsoleVariable<bool> CVarAutoGenerateControlRigTrack(TEXT("ControlRig.Sequencer.AutoGenerateTrack"), true, TEXT("When true automatically create control rig tracks in Sequencer when a control rig is added to a level."));

TAutoConsoleVariable<bool>	
CVarSelectedKeysSelectControls(TEXT("ControlRig.Sequencer.SelectedKeysSelectControls"), false, TEXT("When true when we select a key in Sequencer it will select the Control, by default false."));

TAutoConsoleVariable<bool> CVarSelectedSectionSetsSectionToKey(TEXT("ControlRig.Sequencer.SelectedSectionSetsSectionToKey"), true, TEXT("When true when we select a channel in a section, if it's the only section selected we set it as the Section To Key, by default false."));

TAutoConsoleVariable<bool> CVarEnableAdditiveControlRigs(TEXT("ControlRig.Sequencer.EnableAdditiveControlRigs"), true, TEXT("When true it is possible to add an additive control rig to a skeletal mesh component."));

TAutoConsoleVariable<bool> CVarSelectChildChannels(TEXT("ControlRig.Sequencer.SelectChildChannels"), true, TEXT("When true selecting a control will search for child channels and constraint controls and select them also, default to true."));


static USkeletalMeshComponent* AcquireSkeletalMeshFromObject(UObject* BoundObject, TSharedPtr<ISequencer> SequencerPtr)
{
	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
		{
			return SkeletalMeshComponent;
		}

		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		Actor->GetComponents(SkeletalMeshComponents);

		if (SkeletalMeshComponents.Num() == 1)
		{
			return SkeletalMeshComponents[0];
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return SkeletalMeshComponent;
		}
	}

	return nullptr;
}


static USkeleton* GetSkeletonFromComponent(UActorComponent* InComponent)
{
	USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(InComponent);
	if (SkeletalMeshComp && SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
	{
		// @todo Multiple actors, multiple components
		return SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
	}

	return nullptr;
}

static USkeleton* AcquireSkeletonFromObjectGuid(const FGuid& Guid, UObject** Object, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;
	*Object = BoundObject;
	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
		{
			return GetSkeletonFromComponent(SkeletalMeshComponent);
		}

		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		Actor->GetComponents(SkeletalMeshComponents);
		if (SkeletalMeshComponents.Num() == 1)
		{
			return GetSkeletonFromComponent(SkeletalMeshComponents[0]);
		}
		SkeletalMeshComponents.Empty();

		AActor* ActorCDO = Cast<AActor>(Actor->GetClass()->GetDefaultObject());
		if (ActorCDO)
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ActorCDO->GetRootComponent()))
			{
				return GetSkeletonFromComponent(SkeletalMeshComponent);
			}

			ActorCDO->GetComponents(SkeletalMeshComponents);
			if (SkeletalMeshComponents.Num() == 1)
			{
				return GetSkeletonFromComponent(SkeletalMeshComponents[0]);
			}
			SkeletalMeshComponents.Empty();
		}

		UBlueprintGeneratedClass* ActorBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(Actor->GetClass());
		if (ActorBlueprintGeneratedClass && ActorBlueprintGeneratedClass->SimpleConstructionScript)
		{
			const TArray<USCS_Node*>& ActorBlueprintNodes = ActorBlueprintGeneratedClass->SimpleConstructionScript->GetAllNodes();

			for (USCS_Node* Node : ActorBlueprintNodes)
			{
				if (Node->ComponentClass && Node->ComponentClass->IsChildOf(USkeletalMeshComponent::StaticClass()))
				{
					if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Node->GetActualComponentTemplate(ActorBlueprintGeneratedClass)))
					{
						SkeletalMeshComponents.Add(SkeletalMeshComponent);
					}
				}
			}

			if (SkeletalMeshComponents.Num() == 1)
			{
				return GetSkeletonFromComponent(SkeletalMeshComponents[0]);
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		if (USkeleton* Skeleton = GetSkeletonFromComponent(SkeletalMeshComponent))
		{
			return Skeleton;
		}
	}

	return nullptr;
}

static USkeletalMeshComponent* AcquireSkeletalMeshFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;
	return AcquireSkeletalMeshFromObject(BoundObject, SequencerPtr);
}

static bool DoesControlRigAllowMultipleInstances(const FTopLevelAssetPath& InGeneratedClassPath)
{
	const IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	
	const FString BlueprintPath = InGeneratedClassPath.ToString().LeftChop(2); // Chop off _C
	const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(BlueprintPath));

	if (AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UControlRigBlueprint, bAllowMultipleInstances)))
	{
		return true;
	}

	return false;
}

namespace UE::ControlRigEditor::Private
{
	
void ClearTimer(FTimerHandle& InOutHandle)
{
	if (ensure(GEditor))
	{
		if (InOutHandle.IsValid())
		{
			GEditor->GetTimerManager()->ClearTimer(InOutHandle);
		}
	}
	else
	{
		InOutHandle.Invalidate();
	}
}

void DeferToNextTick(FTimerHandle& InOutHandle, TFunction<void(void)>&& Callback)
{
	ClearTimer(InOutHandle);
	if (ensure(GEditor))
	{
		InOutHandle = GEditor->GetTimerManager()->SetTimerForNextTick(MoveTemp(Callback));
	}
}
	
void DeferToNextTick(TFunction<void(void)>&& Callback)
{
	if (ensure(GEditor))
	{
		GEditor->GetTimerManager()->SetTimerForNextTick(MoveTemp(Callback));
	}
}
	
}

bool FControlRigParameterTrackEditor::bAutoGenerateControlRigTrack = true;
FCriticalSection FControlRigParameterTrackEditor::ControlUndoTransactionMutex;
FControlRigParameterTrackEditor* FControlRigParameterTrackEditor::CurrentEditor = nullptr;

FControlRigParameterTrackEditor::FControlRigParameterTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FKeyframeTrackEditor<UMovieSceneControlRigParameterTrack>(InSequencer)
	, bCurveDisplayTickIsPending(false)
	, bIsDoingSelection(false)
	, bSkipNextSelectionFromTimer(false)
	, bIsLayeredControlRig(false)
	, bFilterAssetBySkeleton(true)
	, bFilterAssetByAnimatableControls(false)
	, ControlUndoBracket(0)
	, ControlChangedDuringUndoBracket(0)
{
	FMovieSceneToolsModule::Get().RegisterAnimationBakeHelper(this);

	if (GEditor != nullptr)
	{
		GEditor->RegisterForUndo(this);
	}
}

FText FControlRigParameterTrackEditor::GetDisplayName() const
{
	return LOCTEXT("ControlRigParameterTrackEditor_DisplayName", "Control Rig Parameter");
}

void FControlRigParameterTrackEditor::OnInitialize()
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}
	FControlRigParameterTrackEditor::CurrentEditor = this;

	UMovieSceneSequence* OwnerSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
	
	SelectionChangedHandle = Sequencer->GetSelectionChangedTracks().AddRaw(this, &FControlRigParameterTrackEditor::OnSelectionChanged);
	SequencerChangedHandle = Sequencer->OnMovieSceneDataChanged().AddRaw(this, &FControlRigParameterTrackEditor::OnSequencerDataChanged);
	OnActivateSequenceChangedHandle = Sequencer->OnActivateSequence().AddRaw(this, &FControlRigParameterTrackEditor::OnActivateSequenceChanged);
	OnChannelChangedHandle = Sequencer->OnChannelChangedWithTime().AddRaw(this, &FControlRigParameterTrackEditor::OnChannelChanged);
	OnMovieSceneBindingsChangeHandle = Sequencer->OnMovieSceneBindingsChanged().AddRaw(this, &FControlRigParameterTrackEditor::OnMovieSceneBindingsChanged);


	if (MovieScene)
	{
		OnMovieSceneChannelChangedHandle = MovieScene->OnChannelChangedWithTime().AddRaw(this, &FControlRigParameterTrackEditor::OnChannelChanged);
	}
	OnActorAddedToSequencerHandle = Sequencer->OnActorAddedToSequencer().AddRaw(this, &FControlRigParameterTrackEditor::HandleActorAdded);

	{
		//we check for two things, one if the control rig has been replaced if so we need to switch.
		//the other is if bound object on the edit mode is null we request a re-evaluate which will reset it up.
		const FDelegateHandle OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddLambda([this](const TMap<UObject*, UObject*>& ReplacementMap)
		{
			TSharedPtr<ISequencer> Sequencer = GetSequencer();
			UMovieSceneSequence* OwnerSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
			UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;

			if (!MovieScene)
			{
				return;
			}

			static bool bHasEnteredSilent = false;
			
			bool bRequestEvaluate = false;
			TMap<UControlRig*, UControlRig*> OldToNewControlRigs;
			
			FControlRigEditMode* ControlRigEditMode = GetEditMode();
			if (ControlRigEditMode == nullptr)
			{
				return;
			}
							
			//Reset Bindings for replaced objects.
			for (TPair<UObject*, UObject*> ReplacedObject : ReplacementMap)
			{
				if (UControlRigComponent* OldControlRigComponent = Cast<UControlRigComponent>(ReplacedObject.Key))
				{
					UControlRigComponent* NewControlRigComponent = Cast<UControlRigComponent>(ReplacedObject.Value);
					if (OldControlRigComponent->GetControlRig())
					{
						UControlRig* NewControlRig = nullptr;
						if (NewControlRigComponent)
						{
							NewControlRig = NewControlRigComponent->GetControlRig();
						}
						OldToNewControlRigs.Emplace(OldControlRigComponent->GetControlRig(), NewControlRig);
					}
				}
				else if (UControlRig* OldControlRig = Cast<UControlRig>(ReplacedObject.Key))
				{
					UControlRig* NewControlRig = Cast<UControlRig>(ReplacedObject.Value);
					OldToNewControlRigs.Emplace(OldControlRig, NewControlRig);
				}
			}
			
			if (OldToNewControlRigs.Num() > 0)
			{
				//need to avoid any evaluations when doing this replacement otherwise we will evaluate sequencer
				if (bHasEnteredSilent == false)
				{
					Sequencer->EnterSilentMode();
					bHasEnteredSilent = true;
				}

				IterateTracks([this, &OldToNewControlRigs, ControlRigEditMode, &bRequestEvaluate](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
				{
					if (UControlRig* OldControlRig = Track->GetControlRig())
					{
						UControlRig** FoundNewControlRig = OldToNewControlRigs.Find(OldControlRig);
						if (FoundNewControlRig)
						{
							UControlRig* NewControlRig = (*FoundNewControlRig); 
							
							TArray<FName> SelectedControls = OldControlRig->CurrentControlSelection();
							OldControlRig->ClearControlSelection();
							UnbindControlRig(OldControlRig);

							if (NewControlRig)
							{
								Track->Modify();
								Track->ReplaceControlRig(NewControlRig, OldControlRig->GetAssetReference() != NewControlRig->GetAssetReference());
								BindControlRig(NewControlRig,InGuid);
								bRequestEvaluate = true;
							}
							else
							{
								Track->ReplaceControlRig(nullptr, true);
							}
							
							if (ControlRigEditMode)
							{
								ControlRigEditMode->ReplaceControlRig(OldControlRig, NewControlRig);

								auto UpdateSelectionDelegate = [this, SelectedControls, NewControlRig, bRequestEvaluate]()
								{
									using namespace UE::ControlRigEditor::Private;
									if (!(FSlateApplication::Get().HasAnyMouseCaptor() || GUnrealEd->IsUserInteracting()))
									{
										TSharedPtr<ISequencer> Sequencer = GetSequencer();
										
										TGuardValue<bool> Guard(bIsDoingSelection, true);
										if (Sequencer)
										{
											Sequencer->ExternalSelectionHasChanged();
										}

										if (NewControlRig)
										{
											DeferToNextTick([WeakRig = TWeakObjectPtr<UControlRig>(NewControlRig), SelectedControls]()
											{
												if (UControlRig* Rig = WeakRig.Get())
												{
													Rig->ClearControlSelection();
													for (const FName& ControlName : SelectedControls)
													{
														Rig->SelectControl(ControlName, true);
													}
												}
											});
										}
										
										if (bHasEnteredSilent)
										{
											if (Sequencer)
											{
												Sequencer->ExitSilentMode();
											}
											bHasEnteredSilent = false;
										}
										
										if (bRequestEvaluate)
										{
											if (Sequencer)
											{
												Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
											}
										}
										ClearTimer(UpdateSelectionTimerHandle);
									}
								};

								GEditor->GetTimerManager()->SetTimer(UpdateSelectionTimerHandle, UpdateSelectionDelegate, 0.01f, true);
							}
						}
					}

					return false;
				});

				if (!ControlRigEditMode)
				{
					if (bRequestEvaluate)
					{
						Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
					}
				}
			}

			if (ControlRigEditMode && bRequestEvaluate)
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
			}

			// ensure we exit silent mode if it has been entered
			if (bHasEnteredSilent)
			{
				Sequencer->ExitSilentMode();
				bHasEnteredSilent = false;
			}
		});
	
		AcquiredResources.Add([OnObjectsReplacedHandle] { FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle); });
	}

	// Register all modified/selections for control rigs
	IterateTracks([this](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
	{
			
		if (UControlRig* ControlRig = Track->GetControlRig())
		{
			// Make sure the delegates are bound before the control rig initializes
			// Usually delegates should have been bound during Track PostLoad, and they stay bound even if the rig is recompiled via FCoreUObjectDelegates::OnObjectsReplaced above
			// However, the logic in OnObjectsReplaced only runs if the sequencer is running. When the rig is recompiled while the sequencer is closed
			// the new Control Rig received by the Track won't have its delegates bound to the Track, so we have to rebind them here such
			// that the track can pull information from latest version of the rig after construction event
			Track->BindControlRigDelegates();
			
			BindControlRig(ControlRig, InGuid);
		}

		// Mark layered mode on track color and display name 
		UControlRigSequencerEditorLibrary::MarkLayeredModeOnTrackDisplay(Track);

		return false;
	});

	// Register as animation mixer item menu provider
	IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);
}

FControlRigParameterTrackEditor::~FControlRigParameterTrackEditor()
{
	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}
	FControlRigParameterTrackEditor::CurrentEditor = nullptr;

	FMovieSceneToolsModule::Get().UnregisterAnimationBakeHelper(this);

	UE::ControlRigEditor::Private::ClearTimer(DeactivateModeHandle);
	UE::ControlRigEditor::Private::ClearTimer(ActivateModeHandle);
}


void FControlRigParameterTrackEditor::BindControlRig(UControlRig* ControlRig, FGuid InGuid)
{
	if (ControlRig)
	{
		if (!BoundControlRigs.Contains(ControlRig))
		{
			ControlRig->ControlModified().AddRaw(this, &FControlRigParameterTrackEditor::HandleControlModified);
			ControlRig->OnPostConstruction_AnyThread().AddRaw(this, &FControlRigParameterTrackEditor::HandleOnPostConstructed);
			ControlRig->ControlSelected().AddRaw(this, &FControlRigParameterTrackEditor::HandleControlSelected);
			ControlRig->ControlUndoBracket().AddRaw(this, &FControlRigParameterTrackEditor::HandleControlUndoBracket);
			ControlRig->ControlRigBound().AddRaw(this, &FControlRigParameterTrackEditor::HandleOnControlRigBound);
		
			BoundControlRigs.Add(ControlRig, InGuid);
		}

		if (UMovieSceneControlRigParameterTrack* Track = FindTrack(ControlRig))
		{
			for (UMovieSceneSection* BaseSection : Track->GetAllSections())
			{
				if (UMovieSceneControlRigParameterSection* Section = Cast< UMovieSceneControlRigParameterSection>(BaseSection))
				{
					if (Section->GetControlRig())
					{
						TArray<FSpaceControlNameAndChannel>& SpaceChannels = Section->GetSpaceChannels();
						for (FSpaceControlNameAndChannel& Channel : SpaceChannels)
						{
							HandleOnSpaceAdded(Section, Channel.ControlName, &(Channel.SpaceCurve));
						}
						
						TArray<FConstraintAndActiveChannel>& ConstraintChannels = Section->GetConstraintsChannels();
						for (FConstraintAndActiveChannel& Channel: ConstraintChannels)
						{ 
							HandleOnConstraintAdded(Section, &(Channel.ActiveChannel));
						}
					}
				}
			}

			if (!Track->SpaceChannelAdded().IsBoundToObject(this))
			{
				Track->SpaceChannelAdded().AddRaw(this, &FControlRigParameterTrackEditor::HandleOnSpaceAdded);
			}

			if (!Track->ConstraintChannelAdded().IsBoundToObject(this))
			{
				Track->ConstraintChannelAdded().AddRaw(this, &FControlRigParameterTrackEditor::HandleOnConstraintAdded);
			}
		}
	}
}
void FControlRigParameterTrackEditor::UnbindControlRig(UControlRig* ControlRig)
{
	if (ControlRig && BoundControlRigs.Contains(ControlRig) == true)
	{
		UMovieSceneControlRigParameterTrack* Track = FindTrack(ControlRig);
		if (Track)
		{
			Track->SpaceChannelAdded().RemoveAll(this);
			Track->ConstraintChannelAdded().RemoveAll(this);
		}
		ControlRig->ControlModified().RemoveAll(this);
		ControlRig->OnPostConstruction_AnyThread().RemoveAll(this);
		ControlRig->ControlSelected().RemoveAll(this);
		if (const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding())
		{
			Binding->OnControlRigBind().RemoveAll(this);
		}
		ControlRig->ControlUndoBracket().RemoveAll(this);
		ControlRig->ControlRigBound().RemoveAll(this);
		
		BoundControlRigs.Remove(ControlRig);
		ClearOutAllSpaceAndConstraintDelegates(ControlRig);
	}
}
void FControlRigParameterTrackEditor::UnbindAllControlRigs()
{
	ClearOutAllSpaceAndConstraintDelegates();
	TMap<TWeakObjectPtr<UControlRig>,FGuid> ControlRigs = BoundControlRigs;
	for (TPair<TWeakObjectPtr<UControlRig>, FGuid>& ObjectPtr : ControlRigs)
	{
		if (ObjectPtr.Key.IsValid())
		{
			UControlRig* ControlRig = ObjectPtr.Key.Get();
			UnbindControlRig(ControlRig);
		}
	}
	BoundControlRigs.Reset();
}


void FControlRigParameterTrackEditor::ObjectImplicitlyAdded(UObject* InObject)
{
	UControlRig* ControlRig = Cast<UControlRig>(InObject);
	if (ControlRig)
	{
		IterateTracks([this, ControlRig](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
			{
				if (ControlRig == Track->GetControlRig())
				{
					BindControlRig(ControlRig, InGuid);
				}

				return false;
			});
	}
}

void FControlRigParameterTrackEditor::ObjectImplicitlyRemoved(UObject* InObject)
{
	UControlRig* ControlRig = Cast<UControlRig>(InObject);
	if (ControlRig)
	{
		UnbindControlRig(ControlRig);
	}

	if (RecreateRigOperator.IsValid() && RecreateRigOperator->IsEditingObject(InObject))
	{
		RecreateRigOperator->Abort();
	}
}

void FControlRigParameterTrackEditor::OnRelease()
{
	FControlRigParameterTrackEditor::CurrentEditor = nullptr;

	for (FDelegateHandle& Handle : ConstraintHandlesToClear)
	{
		if (Handle.IsValid())
		{
			FConstraintsManagerController::GetNotifyDelegate().Remove(Handle);
		}
	}
	ConstraintHandlesToClear.Reset();

	FControlRigEditMode* ControlRigEditMode = GetEditMode();
	PreviousSelectedControlRigs.Reset();
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	
	if (ControlRigEditMode)
	{
		bControlRigEditModeWasOpen = true;
		for (TPair<TWeakObjectPtr<UControlRig>,FGuid>& ControlRig : BoundControlRigs)
		{
			if (ControlRig.Key.IsValid())
			{
				TPair<FControlRigAssetStrongReference, TArray<FName>> ClassAndName;
				ClassAndName.Key = ControlRig.Key->GetAssetReference();
				ClassAndName.Value = ControlRig.Key->CurrentControlSelection();
				PreviousSelectedControlRigs.Add(ClassAndName);
			}
		}
		ControlRigEditMode->Exit(); //deactive mode below doesn't exit for some reason so need to make sure things are cleaned up

		if (FEditorModeTools* Tools = GetEditorModeTools())
		{
			Tools->DeactivateMode(FControlRigEditMode::ModeName);
		}

		ControlRigEditMode->SetObjects(nullptr, nullptr, Sequencer);
	}
	else
	{
		bControlRigEditModeWasOpen = false;
	}
	
	UnbindAllControlRigs();

	if (Sequencer)
	{
		if (SelectionChangedHandle.IsValid())
		{
			Sequencer->GetSelectionChangedTracks().Remove(SelectionChangedHandle);
			SelectionChangedHandle.Reset();
		}
		
		if (SequencerChangedHandle.IsValid())
		{
			Sequencer->OnMovieSceneDataChanged().Remove(SequencerChangedHandle);
			SequencerChangedHandle.Reset();
		}
		
		if (OnActivateSequenceChangedHandle.IsValid())
		{
			Sequencer->OnActivateSequence().Remove(OnActivateSequenceChangedHandle);
			OnActivateSequenceChangedHandle.Reset();
		}
		
		if (CurveChangedHandle.IsValid())
		{
			Sequencer->GetCurveDisplayChanged().Remove(CurveChangedHandle);
			CurveChangedHandle.Reset();
		}
		
		if (OnActorAddedToSequencerHandle.IsValid())
		{
			Sequencer->OnActorAddedToSequencer().Remove(OnActorAddedToSequencerHandle);
			OnActorAddedToSequencerHandle.Reset();
		}
		
		if (OnChannelChangedHandle.IsValid())
		{
			Sequencer->OnChannelChangedWithTime().Remove(OnChannelChangedHandle);
			OnChannelChangedHandle.Reset();
		}

		if (OnMovieSceneBindingsChangeHandle.IsValid())
		{
			Sequencer->OnMovieSceneBindingsChanged().Remove(OnMovieSceneBindingsChangeHandle);
			OnMovieSceneBindingsChangeHandle.Reset();
		}

		if (OnMovieSceneChannelChangedHandle.IsValid())
		{
			UMovieSceneSequence* OwnerSequence = Sequencer->GetFocusedMovieSceneSequence();
			if (UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr)
			{
				MovieScene->OnChannelChangedWithTime().Remove(OnMovieSceneChannelChangedHandle);
			}
			OnMovieSceneChannelChangedHandle.Reset();
		}
	}

	// Unregister as animation mixer item menu provider
	IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);

	AcquiredResources.Release();
	
	UE::ControlRigEditor::Private::ClearTimer(DeactivateModeHandle);
	UE::ControlRigEditor::Private::ClearTimer(ActivateModeHandle);
}

TSharedRef<ISequencerTrackEditor> FControlRigParameterTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FControlRigParameterTrackEditor(InSequencer));
}

bool FControlRigParameterTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneControlRigParameterTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FControlRigParameterTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneControlRigParameterTrack::StaticClass();
}


const FSlateBrush* FControlRigParameterTrackEditor::GetIconBrush() const
{
	static const FName StyleSetName(TEXT("ControlRigEditorStyle"));
	const ISlateStyle* Style = FSlateStyleRegistry::FindSlateStyle(StyleSetName);
	return Style ? Style->GetOptionalBrush(TEXT("ClassIcon.ControlRigBlueprint")) : nullptr;
}

TSharedRef<ISequencerSection> FControlRigParameterTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FControlRigParameterSection(SectionObject, GetSequencer()));
}

TSharedPtr<SWidget> FControlRigParameterTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	TWeakPtr<ISequencer> WeakSequencer = GetSequencer();

	const int32 RowIndex = Params.TrackInsertRowIndex;
	UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel = Params.ViewModel.AsWeak();
	auto SubMenuCallback = [=]() -> TSharedRef<SWidget>
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarp_Label", "Time Warp"));
		{
			FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, WeakViewModel);
		}
		MenuBuilder.EndSection();
		FSequencerUtilities::PopulateMenu_CreateNewSection(MenuBuilder, RowIndex, Track, WeakSequencer);

		return MenuBuilder.MakeWidget();
	};

	return UE::Sequencer::MakeAddButton(LOCTEXT("AddSection", "Section"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);

}

void FControlRigParameterTrackEditor::BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if(!ObjectClass)
	{
		return;
	}
	
	if (ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) ||
		ObjectClass->IsChildOf(AActor::StaticClass()) ||
		ObjectClass->IsChildOf(UChildActorComponent::StaticClass()))
	{
		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		UObject* BoundObject = nullptr;
		USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], &BoundObject, Sequencer);
		USkeletalMeshComponent* SkelMeshComp = AcquireSkeletalMeshFromObject(BoundObject, Sequencer);

		if (Skeleton && SkelMeshComp)
		{
			const TSharedPtr<ISequencer> SequencerForBake = Sequencer;
			const FGuid BakeObjectBinding = ObjectBindings[0];

			MenuBuilder.BeginSection("Control Rig", LOCTEXT("ControlRig", "Control Rig"));
			{
				UE::ControlRig::BuildBakeToControlRigMenu(
					MenuBuilder, Skeleton, bFilterAssetBySkeleton,
					// PopulateDelegate: export from skel mesh via sequencer evaluation
					UE::ControlRig::FPopulateAnimSequenceDelegate::CreateLambda(
						[SequencerForBake](UAnimSequence* AnimSeq, UAnimSeqExportOption* ExportOpts, USkeletalMeshComponent* Mesh) -> bool
						{
							UMovieSceneSequence* OwnerSeq = SequencerForBake->GetFocusedMovieSceneSequence();
							UMovieSceneSequence* RootSeq = SequencerForBake->GetRootMovieSceneSequence();
							FMovieSceneSequenceTransform RootToLocal = SequencerForBake->GetFocusedMovieSceneSequenceTransformUnwarped();
							FAnimExportSequenceParameters AESP;
							AESP.Player = SequencerForBake.Get();
							AESP.RootToLocalTransform = RootToLocal;
							AESP.MovieSceneSequence = OwnerSeq;
							AESP.RootMovieSceneSequence = RootSeq;
							AESP.bForceUseOfMovieScenePlaybackRange = SequencerForBake->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();
							return MovieSceneToolHelpers::ExportToAnimSequence(AnimSeq, ExportOpts, AESP, Mesh);
						}),
					SequencerForBake, BakeObjectBinding, BoundObject, SkelMeshComp,
					// TrackFactory: create standalone track on the binding
					[this, BakeObjectBinding](UMovieScene* MovieScene) -> UMovieSceneControlRigParameterTrack*
					{
						// Remove existing CR tracks
						TArray<UMovieSceneTrack*> ExistingTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), BakeObjectBinding, NAME_None);
						for (UMovieSceneTrack* Existing : ExistingTracks)
						{
							MovieScene->RemoveTrack(*Existing);
						}
						return Cast<UMovieSceneControlRigParameterTrack>(AddTrack(MovieScene, BakeObjectBinding, UMovieSceneControlRigParameterTrack::StaticClass(), NAME_None));
					},
					[this, BakeObjectBinding, WeakSkelMeshComp = TWeakObjectPtr<USkeletalMeshComponent>(SkelMeshComp)]
					(const UE::ControlRig::FBakeToControlRigResult& Result)
					{
						const TSharedPtr<ISequencer> Seq = GetSequencer();
						if (!Seq) { return; }
						UMovieScene* MovieScene = Seq->GetFocusedMovieSceneSequence() ? Seq->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;
						if (!MovieScene) { return; }

						TArray<UMovieSceneTrack*> TracksToDisable;
						auto CollectSkelAnimAndMixerTracks = [&TracksToDisable, MovieScene](const FGuid& InBindingGuid)
						{
							const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBindingGuid);
							if (!Binding) { return; }
							for (UMovieSceneTrack* Track : Binding->GetTracks())
							{
								if (Cast<UMovieSceneSkeletalAnimationTrack>(Track))
								{
									TracksToDisable.Add(Track);
								}
								else if (Track && Track->GetClass()->GetName() == TEXT("MovieSceneAnimationMixerTrack"))
								{
									TracksToDisable.Add(Track);
								}
							}
						};
						CollectSkelAnimAndMixerTracks(BakeObjectBinding);
						if (USkeletalMeshComponent* SkelMeshComp = WeakSkelMeshComp.Get())
						{
							CollectSkelAnimAndMixerTracks(Seq->FindObjectId(*SkelMeshComp, Seq->GetFocusedTemplateID()));
						}

						for (UMovieSceneTrack* Track : TracksToDisable)
						{
							MovieSceneHelpers::DisableTrack(Track);
						}

						FControlRigEditMode* ControlRigEditMode = GetEditMode();
						if (!ControlRigEditMode)
						{
							ControlRigEditMode = GetEditMode(true);
						}
						if (ControlRigEditMode) { ControlRigEditMode->AddControlRigObject(Result.ControlRig, Seq); }
						BindControlRig(Result.ControlRig, BakeObjectBinding);

						if (Result.Section)
						{
							Seq->EmptySelection();
							Seq->SelectSection(Result.Section);
							Seq->ThrobSectionSelection();
						}
						Seq->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
					});
			}
			MenuBuilder.EndSection();
		}
	}
}

static bool AssetReferenceSortPredicate(const TSharedRef<FControlRigAssetSoftReference> A, const  TSharedRef<FControlRigAssetSoftReference> B)
{
	const bool bAIsFK = A->Get() == UFKControlRig::StaticClass();
	const bool bBIsFK = B->Get() == UFKControlRig::StaticClass();
	if ((!bAIsFK && !bBIsFK))
	{
		return  A->GetName().Compare(B->GetName(), ESearchCase::IgnoreCase) < 0;
	}
	else
	{
		return A->Get() == UFKControlRig::StaticClass();
	}
}

/** Filter class does not allow classes that already exist in a skeletal mesh component. */
class FClassViewerHideAlreadyAddedRigsFilter : public IClassViewerFilter
{
public:
	FClassViewerHideAlreadyAddedRigsFilter(const TArray<UClass*> InExistingClasses)
		: AlreadyAddedRigs(InExistingClasses)
	{}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs ) override
	{
		return IsClassAllowed_Internal(InClass->GetClassPathName());
	}
	
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions,
										const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData,
										TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
	{
		return IsClassAllowed_Internal(InUnloadedClassData.Get().GetClassPathName());
	}
	
private:
	bool IsClassAllowed_Internal(const FTopLevelAssetPath& InGeneratedClassPath)
	{
		if (DoesControlRigAllowMultipleInstances(InGeneratedClassPath))
		{
			return true;
		}
		
		return !AlreadyAddedRigs.ContainsByPredicate([InGeneratedClassPath](const UClass* Class)
			{
				return InGeneratedClassPath == Class->GetClassPathName();
			});
	}

	
	TArray<UClass*> AlreadyAddedRigs;
};

void FControlRigParameterTrackEditor::BakeToControlRigSubMenu(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, UObject* BoundObject, USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton)
{
	if (Skeleton)
	{
		const TSharedPtr<FControlRigClassFilter> ClassFilter = MakeShareable(new FControlRigClassFilter(bFilterAssetBySkeleton, false, true, Skeleton));		
		TArray<FControlRigAssetSoftReference> ExtraAssets = {UFKControlRig::StaticClass()};

		MenuBuilder.AddWidget(SNew(SControlRigAssetReferencePicker)
			.OnSelectionChanged_Lambda([this, ObjectBinding, BoundObject, SkelMeshComp, Skeleton](FControlRigAssetSoftReference InReference)
				{
					BakeToControlRig(InReference.LoadStrongReference(), ObjectBinding, BoundObject, SkelMeshComp, Skeleton);
				})
			.Filter(ClassFilter)
			.ExtraAssets(ExtraAssets)
			.SortPredicate(AssetReferenceSortPredicate)
		,FText::GetEmpty());
	}
}

class SBakeToAnimAndControlRigOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBakeToAnimAndControlRigOptionsWindow)
		: _ExportOptions(nullptr), _BakeSettings(nullptr)
		, _WidgetWindow()
	{}

	SLATE_ARGUMENT(UAnimSeqExportOption*, ExportOptions)
		SLATE_ARGUMENT(UBakeToControlRigSettings*, BakeSettings)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnExport()
	{
		bShouldExport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}


	FReply OnCancel()
	{
		bShouldExport = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldExport() const
	{
		return bShouldExport;
	}


	SBakeToAnimAndControlRigOptionsWindow()
		: ExportOptions(nullptr)
		, BakeSettings(nullptr)
		, bShouldExport(false)
	{}

private:

	FReply OnResetToDefaultClick() const;

private:
	UAnimSeqExportOption* ExportOptions;
	UBakeToControlRigSettings* BakeSettings;
	TSharedPtr<class IDetailsView> DetailsView;
	TSharedPtr<class IDetailsView> DetailsView2;
	TWeakPtr< SWindow > WidgetWindow;
	bool			bShouldExport;
};


void SBakeToAnimAndControlRigOptionsWindow::Construct(const FArguments& InArgs)
{
	ExportOptions = InArgs._ExportOptions;
	BakeSettings = InArgs._BakeSettings;
	WidgetWindow = InArgs._WidgetWindow;

	check(ExportOptions);

	FText CancelText = LOCTEXT("AnimSequenceOptions_Cancel", "Cancel");
	FText CancelTooltipText = LOCTEXT("AnimSequenceOptions_Cancel_ToolTip", "Cancel control rig creation");

	TSharedPtr<SBox> HeaderToolBox;
	TSharedPtr<SHorizontalBox> AnimHeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	TSharedPtr<SBox> InspectorBox2;
	this->ChildSlot
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(HeaderToolBox, SBox)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
		.Text(LOCTEXT("Export_CurrentFileTitle", "Current File: "))
		]
		]
		]
	+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2)
		[
			SAssignNew(InspectorBox, SBox)
		]
	+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(2)
		[
			SAssignNew(InspectorBox2, SBox)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
		+ SUniformGridPanel::Slot(1, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
		.Text(LOCTEXT("Create", "Create"))
		.OnClicked(this, &SBakeToAnimAndControlRigOptionsWindow::OnExport)
		]
	+ SUniformGridPanel::Slot(2, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
		.Text(CancelText)
		.ToolTipText(CancelTooltipText)
		.OnClicked(this, &SBakeToAnimAndControlRigOptionsWindow::OnCancel)
		]
		]
			]
		];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView2 = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InspectorBox->SetContent(DetailsView->AsShared());
	InspectorBox2->SetContent(DetailsView2->AsShared());
	HeaderToolBox->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
		[
			SAssignNew(AnimHeaderButtons, SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(SButton)
			.Text(LOCTEXT("AnimSequenceOptions_ResetOptions", "Reset to Default"))
		.OnClicked(this, &SBakeToAnimAndControlRigOptionsWindow::OnResetToDefaultClick)
		]
		]
		]
		]
	);

	DetailsView->SetObject(ExportOptions);
	DetailsView2->SetObject(BakeSettings);
}

FReply SBakeToAnimAndControlRigOptionsWindow::OnResetToDefaultClick() const
{
	if (ExportOptions)
	{
		ExportOptions->ResetToDefault();
		//Refresh the view to make sure the custom UI are updating correctly
		DetailsView->SetObject(ExportOptions, true);
	}

	if (BakeSettings)
	{
		BakeSettings->Reset();
		DetailsView2->SetObject(BakeSettings, true);
	}
	
	return FReply::Handled();
}

void FControlRigParameterTrackEditor::SmartReduce(const TSharedPtr<ISequencer>& InSequencer, const FSmartReduceParams& InParams, UMovieSceneSection* InSection)
{
	using namespace UE::Sequencer;

	if (InSection)
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		constexpr bool bNeedToTestExisting = false;
		TOptional<FKeyHandleSet> KeyHandleSet;

		FMovieSceneChannelProxy& ChannelProxy = InSection->GetChannelProxy();
		for (const FMovieSceneChannelEntry& Entry : InSection->GetChannelProxy().GetAllEntries())
		{
			const FName ChannelTypeName = Entry.GetChannelTypeName();
			TArrayView<FMovieSceneChannel* const> Channels = Entry.GetChannels();
			for (int32 Index = 0; Index < Channels.Num(); ++Index)
			{
				FMovieSceneChannelHandle ChannelHandle = ChannelProxy.MakeHandle(ChannelTypeName, Index);
				ISequencerChannelInterface* EditorInterface = SequencerModule.FindChannelEditorInterface(ChannelHandle.GetChannelTypeName());

				FCreateCurveEditorModelParams CurveModelParams{ InSection, InSection, InSequencer.ToSharedRef() };
				if (TUniquePtr<FCurveModel> CurveModel = EditorInterface->CreateCurveEditorModel_Raw(ChannelHandle, CurveModelParams))
				{
					FKeyHandleSet OutHandleSet;
					UCurveEditorSmartReduceFilter::SmartReduce(CurveModel.Get(), InParams, KeyHandleSet, bNeedToTestExisting, OutHandleSet);
				}
			}
		}
	}
}

bool FControlRigParameterTrackEditor::LoadAnimationIntoSection(const TSharedPtr<ISequencer>& SequencerPtr, UAnimSequence* AnimSequence, USkeletalMeshComponent* SkelMeshComp,
	FFrameNumber StartFrame, bool bReduceKeys, const FSmartReduceParams& ReduceParams, bool bResetControls, const TOptional<TRange<FFrameNumber>>& InAnimFrameRange, 
	bool bOntoSelectedControls, UMovieSceneControlRigParameterSection* ParamSection)
{
	EMovieSceneKeyInterpolation DefaultInterpolation = SequencerPtr->GetKeyInterpolation();
	UMovieSceneSequence* OwnerSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
	UMovieSceneControlRigParameterSection::FLoadAnimSequenceData Data;
	Data.bKeyReduce = false; //use smart reduce
	Data.Tolerance = 0.0;
	Data.bResetControls = bResetControls;
	Data.StartFrame = StartFrame;
	Data.AnimFrameRange = InAnimFrameRange;
	Data.bOntoSelectedControls = bOntoSelectedControls;
	if (ParamSection->LoadAnimSequenceIntoThisSection(AnimSequence, FFrameNumber(0), OwnerMovieScene, SkelMeshComp,
		Data, DefaultInterpolation))
	{
		if (bReduceKeys)
		{
			SmartReduce(SequencerPtr, ReduceParams, ParamSection);
		}
		return true;
	}
	return false;
}

void FControlRigParameterTrackEditor::BakeToControlRig(const FControlRigAssetStrongReference& InAssetReference, FGuid ObjectBinding, UObject* BoundActor, USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton)
{
	FSlateApplication::Get().DismissAllMenus();
	
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();

	if (InAssetReference.IsValid() && Sequencer)
	{
		UMovieSceneSequence* OwnerSequence = Sequencer->GetFocusedMovieSceneSequence();
		UMovieSceneSequence* RootSequence = Sequencer->GetRootMovieSceneSequence();
		UMovieScene* OwnerMovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
		
		if (!ensure(OwnerMovieScene))
		{
			return;
		}
		
		{
			UAnimSequence* TempAnimSequence = NewObject<UAnimSequence>(GetTransientPackage(), NAME_None);
			TempAnimSequence->SetSkeleton(Skeleton);
			
			FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
			UAnimSeqExportOption* AnimSeqExportOption = NewObject<UAnimSeqExportOption>(GetTransientPackage(), NAME_None);
			UBakeToControlRigSettings* BakeSettings = GetMutableDefault<UBakeToControlRigSettings>();
			AnimSeqExportOption->bTransactRecording = false;
			AnimSeqExportOption->CustomDisplayRate = Sequencer->GetFocusedDisplayRate();

			TSharedPtr<SWindow> ParentWindow;
			if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}

			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(LOCTEXT("AnimSeqTitle", "Options For Baking"))
				.SizingRule(ESizingRule::UserSized)
				.AutoCenter(EAutoCenter::PrimaryWorkArea)
				.ClientSize(FVector2D(500, 445));

			TSharedPtr<SBakeToAnimAndControlRigOptionsWindow> OptionWindow;
			Window->SetContent
			(
				SAssignNew(OptionWindow, SBakeToAnimAndControlRigOptionsWindow)
				.ExportOptions(AnimSeqExportOption)
				.BakeSettings(BakeSettings)
				.WidgetWindow(Window)
			);

			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

			if (OptionWindow.Get()->ShouldExport())
			{
				FAnimExportSequenceParameters AESP;
				AESP.Player = Sequencer.Get();
				AESP.RootToLocalTransform = RootToLocalTransform;
				AESP.MovieSceneSequence = OwnerSequence;
				AESP.RootMovieSceneSequence = RootSequence;
				AESP.bForceUseOfMovieScenePlaybackRange = Sequencer->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();
				bool bResult = MovieSceneToolHelpers::ExportToAnimSequence(TempAnimSequence, AnimSeqExportOption, AESP, SkelMeshComp);
				if (bResult == false)
				{
					TempAnimSequence->MarkAsGarbage();
					AnimSeqExportOption->MarkAsGarbage();
					return;
				}

				const FScopedTransaction Transaction(LOCTEXT("BakeToControlRig_Transaction", "Bake To Control Rig"));

				OwnerMovieScene->Modify();

				// Remove existing control rig tracks on this binding
				TArray<UMovieSceneTrack*> ExistingCRTracks = OwnerMovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), ObjectBinding, NAME_None);
				for (UMovieSceneTrack* AnyOleTrack : ExistingCRTracks)
				{
					if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack))
					{
						OwnerMovieScene->RemoveTrack(*CRTrack);
					}
				}

				// Create the track and initialize the control rig via the shared helper
				UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(AddTrack(OwnerMovieScene, ObjectBinding, UMovieSceneControlRigParameterTrack::StaticClass(), NAME_None));
				if (Track)
				{
					UE::ControlRig::FBakeToControlRigResult Result = UE::ControlRig::InitControlRigFromAnimSequence(
						Sequencer, OwnerMovieScene, Track, InAssetReference,
						BoundActor, SkelMeshComp, TempAnimSequence,
						BakeSettings->bReduceKeys, BakeSettings->SmartReduce);

					if (Result.bSuccess)
					{
						// Turn off any skeletal animation tracks
						TArray<UMovieSceneSkeletalAnimationTrack*> SkelAnimationTracks;
						if (const FMovieSceneBinding* Binding = OwnerMovieScene->FindBinding(ObjectBinding))
						{
							for (UMovieSceneTrack* MovieSceneTrack : Binding->GetTracks())
							{
								if (UMovieSceneSkeletalAnimationTrack* SkelTrack = Cast<UMovieSceneSkeletalAnimationTrack>(MovieSceneTrack))
								{
									SkelAnimationTracks.Add(SkelTrack);
								}
							}
						}

						const FGuid SkelMeshGuid = Sequencer->FindObjectId(*SkelMeshComp, Sequencer->GetFocusedTemplateID());
						if (const FMovieSceneBinding* Binding = OwnerMovieScene->FindBinding(SkelMeshGuid))
						{
							for (UMovieSceneTrack* MovieSceneTrack : Binding->GetTracks())
							{
								if (UMovieSceneSkeletalAnimationTrack* SkelTrack = Cast<UMovieSceneSkeletalAnimationTrack>(MovieSceneTrack))
								{
									SkelAnimationTracks.Add(SkelTrack);
								}
							}
						}

						for (UMovieSceneSkeletalAnimationTrack* SkelTrack : SkelAnimationTracks)
						{
							SkelTrack->Modify();
							MovieSceneHelpers::DisableTrack(SkelTrack);
						}

						// Finish setup: edit mode + delegate binding
						FControlRigEditMode* ControlRigEditMode = GetEditMode();
						if (!ControlRigEditMode)
						{
							ControlRigEditMode = GetEditMode(true);
						}
						if (ControlRigEditMode)
						{
							ControlRigEditMode->AddControlRigObject(Result.ControlRig, Sequencer);
						}
						BindControlRig(Result.ControlRig, ObjectBinding);
						Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
					}
					else
					{
						OwnerMovieScene->RemoveTrack(*Track);
					}
				}

				TempAnimSequence->MarkAsGarbage();
				AnimSeqExportOption->MarkAsGarbage();
			}
		}
	}
}

static void IterateTracksInMovieScene(const UMovieScene& InMovieScene, TFunctionRef<bool(UMovieSceneControlRigParameterTrack*, FGuid)> InCallback)
{
	const TArray<FMovieSceneBinding>& Bindings = InMovieScene.GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		TArray<UMovieSceneTrack*> FoundTracks = InMovieScene.FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
		for(UMovieSceneTrack* Track : FoundTracks)
		{
			if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
			{
				InCallback(CRTrack, Binding.GetObjectGuid());
			}
		}

		// Also check for Control Rig child tracks within animation tracks (e.g., Animation Mixer)
		TArray<UMovieSceneTrack*> AllTracks = InMovieScene.FindTracks(UMovieSceneCommonAnimationTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
		for (UMovieSceneTrack* Track : AllTracks)
		{
			if (UMovieSceneCommonAnimationTrack* AnimTrack = Cast<UMovieSceneCommonAnimationTrack>(Track))
			{
				TArray<TObjectPtr<UMovieSceneTrack>> ChildTracks;
				AnimTrack->GetAllChildTracks(ChildTracks);
				for (UMovieSceneTrack* ChildTrack : ChildTracks)
				{
					if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(ChildTrack))
					{
						InCallback(CRTrack, Binding.GetObjectGuid());
					}
				}
			}
		}
	}

	for (UMovieSceneTrack* Track : InMovieScene.GetTracks())
	{
		if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
		{
			FGuid Empty;
			InCallback(CRTrack,Empty);
		}
	}
}

void FControlRigParameterTrackEditor::IterateTracks(TFunctionRef<bool(UMovieSceneControlRigParameterTrack*, FGuid)> Callback) const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	UMovieSceneSequence* OwnerSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	IterateTracksInMovieScene(*MovieScene, Callback);
}


void FControlRigParameterTrackEditor::BakeInvertedPose(UControlRig* InControlRig, UMovieSceneControlRigParameterTrack* Track)
{
	if (!InControlRig->IsAdditive())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(InControlRig->GetObjectBinding()->GetBoundObject());
	UMovieSceneSequence* RootMovieSceneSequence = Sequencer->GetRootMovieSceneSequence();
	UMovieSceneSequence* MovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	UAnimSeqExportOption* ExportOptions = NewObject<UAnimSeqExportOption>(GetTransientPackage(), NAME_None);

	if (ExportOptions == nullptr || MovieScene == nullptr || SkelMeshComp == nullptr)
	{
		UE_LOGF(LogMovieScene, Error, "FControlRigParameterTrackEditor::BakeInvertedPose All parameters must be valid.");
		return;
	}

	//@sara to do, not sure if you want to key reduce after, but BakeSettings isn't used
	//UBakeToControlRigSettings* BakeSettings = GetMutableDefault<UBakeToControlRigSettings>();
	FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();

	const FScopedTransaction Transaction(LOCTEXT("BakeInvertedPose_Transaction", "Bake Inverted Pose"));

	UnFbx::FLevelSequenceAnimTrackAdapter::FAnimTrackSettings Settings;
	Settings.MovieScenePlayer = Sequencer.Get();
	Settings.MovieSceneSequence = MovieSceneSequence;
	Settings.RootMovieSceneSequence = RootMovieSceneSequence;
	Settings.RootToLocalTransform = RootToLocalTransform;
	Settings.bForceUseOfMovieScenePlaybackRange = Sequencer->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();

	UnFbx::FLevelSequenceAnimTrackAdapter AnimTrackAdapter(Settings);
	int32 AnimationLength = AnimTrackAdapter.GetLength();
	FScopedSlowTask Progress(AnimationLength, LOCTEXT("BakingToControlRig_SlowTask", "Baking To Control Rig..."));
	Progress.MakeDialog(true);

	auto DelegateHandle = InControlRig->OnPreAdditiveValuesApplication_AnyThread().AddLambda([](UControlRig* InControlRig, const FName& InEventName)
	{
		InControlRig->InvertInputPose();
	});

	auto KeyFrame = [this, Sequencer, InControlRig, SkelMeshComp](const FFrameNumber FrameNumber)
	{
		const FFrameNumber NewTime = ConvertFrameTime(FrameNumber, Sequencer->GetFocusedDisplayRate(), Sequencer->GetFocusedTickResolution()).FrameNumber;
		float LocalTime = Sequencer->GetFocusedTickResolution().AsSeconds(FFrameTime(NewTime));

		AddControlKeys(SkelMeshComp, InControlRig, InControlRig->GetFName(), NAME_None, EControlRigContextChannelToKey::AllTransform, 
				ESequencerKeyMode::ManualKeyForced, LocalTime);
	};

	FInitAnimationCB InitCallback = FInitAnimationCB::CreateLambda([]{});

	FStartAnimationCB StartCallback = FStartAnimationCB::CreateLambda([AnimTrackAdapter, KeyFrame](FFrameNumber FrameNumber)
	{
		KeyFrame(AnimTrackAdapter.GetLocalStartFrame());
	});
	
	FTickAnimationCB TickCallback = FTickAnimationCB::CreateLambda([KeyFrame, &Progress](float DeltaTime, FFrameNumber FrameNumber)
	{
		KeyFrame(FrameNumber);
		Progress.EnterProgressFrame(1);
	});
	
	FEndAnimationCB EndCallback = FEndAnimationCB::CreateLambda([]{});

	FAnimExportSequenceParameters AESP;
	AESP.Player = Sequencer.Get();
	AESP.RootToLocalTransform = RootToLocalTransform;
	AESP.MovieSceneSequence = MovieSceneSequence;
	AESP.RootMovieSceneSequence = RootMovieSceneSequence;
	AESP.bForceUseOfMovieScenePlaybackRange = Sequencer->GetSequencerSettings()->ShouldEvaluateSubSequencesInIsolation();
	
	MovieSceneToolHelpers::BakeToSkelMeshToCallbacks(AESP,SkelMeshComp, ExportOptions,
		InitCallback, StartCallback, TickCallback, EndCallback);

	InControlRig->OnPreAdditiveValuesApplication_AnyThread().Remove(DelegateHandle);
	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

bool FControlRigParameterTrackEditor::IsLayered(UMovieSceneControlRigParameterTrack* Track) const
{
	UControlRig* ControlRig = Track->GetControlRig();
	if (!ControlRig)
	{
		return false;
	}
	return ControlRig->IsAdditive();
}

void FControlRigParameterTrackEditor::ConvertIsLayered(UMovieSceneControlRigParameterTrack* Track)
{
	UControlRig* ControlRig = Track->GetControlRig();
	if (!ControlRig)
	{
		return;
	}

	const bool bSetAdditive = !ControlRig->IsAdditive();
	UControlRigSequencerEditorLibrary::SetControlRigLayeredMode(Track, bSetAdditive);
}

void FControlRigParameterTrackEditor::RecreateControlRigWithNewSettings(UMovieSceneControlRigParameterTrack* InTrack)
{
	CreateAndShowRigSettingsWindow(InTrack);
}

bool FControlRigParameterTrackEditor::CanRecreateControlRigWithNewSettings(UMovieSceneControlRigParameterTrack* InTrack)
{
	UControlRig* ControlRig = InTrack->GetControlRig();
	if (!ensure(ControlRig))
	{
		return false;
	}

	if (ControlRig->GetPublicVariables().Num() == 0)
	{
		// no public variable means the rig is not configurable
		return false;
	}

	return true;
}


void FControlRigParameterTrackEditor::CreateAndShowRigSettingsWindow(UMovieSceneControlRigParameterTrack* Track)
{
	RecreateRigOperator = MakeShared<FControlRigParameterTrackEditor::FRecreateRigOperator>();

	RecreateRigOperator->Start(SharedThis(this), Track);
}

void FControlRigParameterTrackEditor::ResetRecreateRigOperatorIfNeeded(TSharedRef<FControlRigParameterTrackEditor::FRecreateRigOperator> InRequestingOperator)
{
	if (InRequestingOperator == RecreateRigOperator)
	{
		RecreateRigOperator.Reset();
	}
}


FControlRigParameterTrackEditor::FRecreateRigOperator::~FRecreateRigOperator()
{
	if (WeakDetailsView.IsValid())
	{
		WeakDetailsView.Pin()->GetOnFinishedChangingPropertiesDelegate().RemoveAll(this);
	}
	
	if (WeakControlRig.IsValid())
	{
		WeakControlRig->OnPostForwardsSolve_AnyThread().RemoveAll(this);
	}

}

void FControlRigParameterTrackEditor::FRecreateRigOperator::Start(const TSharedRef<FControlRigParameterTrackEditor>& InTrackEditor,
	UMovieSceneControlRigParameterTrack* InTrack)
{
	UControlRig* ControlRig = InTrack->GetControlRig();

	if (!ControlRig)
	{
		return;
	}
	
	SettingsForNewControlRig = InTrack->GetCurrentRigSettings();
	if (!SettingsForNewControlRig.IsValid())
	{
		return;
	}

	WeakTrackEditor = InTrackEditor;
	WeakTrack = InTrack;
	WeakControlRig = ControlRig;

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	FStructureDetailsViewArgs StructureDetailsViewArgs;
	
	TSharedRef<IStructureDetailsView> DetailsView=
		FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor")
		.CreateStructureProviderDetailView(
			DetailsViewArgs,
			StructureDetailsViewArgs,
			MakeShared<FInstancePropertyBagStructureDataProvider>(SettingsForNewControlRig));

	DetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP( this,
		&FControlRigParameterTrackEditor::FRecreateRigOperator::OnRigSettingsChanged);

	WeakDetailsView = DetailsView;
	
	ControlRig->OnPostForwardsSolve_AnyThread().AddSP(
		this,
		&FControlRigParameterTrackEditor::FRecreateRigOperator::OnPostControlRigForwardSolve_AnyThread
		);

	
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title( LOCTEXT("RecreateControlRigWithNewSettingsSettingsWindowTitle", "Recreate Control Rig With New Settings") )
		.bDragAnywhere(true)
		.Type(EWindowType::Normal)
		.IsTopmostWindow(true)
		.SizingRule( ESizingRule::Autosized )
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		[
			DetailsView->GetWidget().ToSharedRef()	
		];
	
		
	FSlateApplication::Get().AddWindow( Window, true);
	Window->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &FControlRigParameterTrackEditor::FRecreateRigOperator::OnRigSettingsWindowClosed));
	
	WeakWindow = Window;
}

bool FControlRigParameterTrackEditor::FRecreateRigOperator::IsValid()
{
	if (!WeakTrackEditor.IsValid())
	{
		return false;
	}
	if (!WeakTrack.IsValid())
	{
		return false;
	}
	
	UControlRig* ControlRig = WeakTrack.Get()->GetControlRig();

	if (!ControlRig)
	{
		return false;
	}

	if (ControlRig != WeakControlRig)
	{
		return false;
	}

	return true;
}

void FControlRigParameterTrackEditor::FRecreateRigOperator::Abort()
{
	if (WeakWindow.IsValid())
	{
		WeakWindow.Pin()->RequestDestroyWindow();
		WeakWindow.Reset();
	}
}

void FControlRigParameterTrackEditor::FRecreateRigOperator::OnRigSettingsWindowClosed(const TSharedRef<SWindow>& Window)
{
	if (WeakTrackEditor.IsValid())
	{
		WeakTrackEditor.Pin()->ResetRecreateRigOperatorIfNeeded(SharedThis(this));
	}
}

void FControlRigParameterTrackEditor::FRecreateRigOperator::RefreshSettingsFromControlRig()
{
	if (!IsValid())
	{
		Abort();
		return;
	}

	
	WeakTrack.Get()->SyncRigSettingsToCurrentRig(SettingsForNewControlRig);
	
	UControlRig* ControlRig = WeakTrack.Get()->GetControlRig();


	for (const FPropertyBagPropertyDesc& PropertyDesc : SettingsForNewControlRig.GetPropertyBagStruct()->GetPropertyDescs())
	{
		FRigVMExternalVariable Variable = ControlRig->GetPublicVariableByName(PropertyDesc.Name);
		if (Variable.IsValid())
		{
			const uint8* SourceAddress  = Variable.GetProperty()->ContainerPtrToValuePtr<uint8>(ControlRig->GetVariablesMemory());
			uint8* TargetAddress = SettingsForNewControlRig.GetMutableValue().GetMemory() + PropertyDesc.CachedProperty->GetOffset_ForInternal();
			FRigVMMemoryStorageStruct::CopyProperty(
				PropertyDesc.CachedProperty,
				TargetAddress,
				Variable.GetProperty(),
				SourceAddress
				);
		}
	}
}

void FControlRigParameterTrackEditor::FRecreateRigOperator::OnPostControlRigForwardSolve_AnyThread(UControlRig*, const FName&)
{
	ExecuteOnGameThread(UE_SOURCE_LOCATION, 
		[WeakThis = TWeakPtr<FControlRigParameterTrackEditor::FRecreateRigOperator>(SharedThis(this))]()
	{
		if (WeakThis.IsValid())
		{
			WeakThis.Pin()->RefreshSettingsFromControlRig();
		}
	});
}

void FControlRigParameterTrackEditor::FRecreateRigOperator::OnRigSettingsChanged(const FPropertyChangedEvent& InEvent)
{
	if (!IsValid())
	{
		Abort();
		return;
	}

	TSharedPtr<FControlRigParameterTrackEditor> TrackEditor = WeakTrackEditor.Pin();
	UMovieSceneControlRigParameterTrack* Track = WeakTrack.Get();
	UControlRig* ControlRig = Track->GetControlRig();
			
	FScopedTransaction Transaction(LOCTEXT("RecreateControlRigWithNewSettings_Transaction", "Recreated Control Rig with New Settings"));
	
	Track->UpdateControlRigSettingsOverrides(SettingsForNewControlRig);
	Track->RecreateControlRig();
	
	TrackEditor->GetSequencer()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

bool FControlRigParameterTrackEditor::FRecreateRigOperator::IsEditingObject(UObject* InRigToCheck)
{
	return WeakControlRig == InRigToCheck;
}

void FControlRigParameterTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	// Control Rig menu entries are now added via PopulateObjectBindingAnimationMenu,
	// which is called by FCommonAnimationTrackEditor::BuildObjectBindingTrackMenu
}

void FControlRigParameterTrackEditor::PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu)
{
	if(!ObjectClass)
	{
		return;
	}

	if (ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) ||
		ObjectClass->IsChildOf(AActor::StaticClass()) ||
		ObjectClass->IsChildOf(UChildActorComponent::StaticClass()))
	{
		const TSharedPtr<ISequencer> ParentSequencer = GetSequencer();
		UObject* BoundObject = nullptr;
		USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], &BoundObject, ParentSequencer);

		static const FName ControlRigStyleName(TEXT("ControlRigEditorStyle"));
		const FSlateIcon ControlRigIcon(ControlRigStyleName, "ClassIcon.ControlRigBlueprint");

		if (AActor* BoundActor = Cast<AActor>(BoundObject))
		{
			if (UControlRigComponent* ControlRigComponent = BoundActor->FindComponentByClass<UControlRigComponent>())
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddControlRigTrack", "Add Control Rig Track"),
					LOCTEXT("AddControlRigTrackTooltip", "Adds an animation Control Rig track"),
					ControlRigIcon,
					FUIAction(
						FExecuteAction::CreateSP(this, &FControlRigParameterTrackEditor::AddControlRigFromComponent, ObjectBindings[0]),
						FCanExecuteAction()
					)
				);
				return;
			}
		}

		if (Skeleton)
		{
			UMovieSceneTrack* Track = nullptr;
			TFunction<UMovieSceneControlRigParameterTrack* (FGuid)> InTrackCreationCallback = nullptr;
			MenuBuilder.AddSubMenu(
				LOCTEXT("ControlRigText", "Control Rig"),
				FText(),
				FNewMenuDelegate::CreateSP(this, &FControlRigParameterTrackEditor::HandleAddTrackSubMenu, ObjectBindings, Track, InTrackCreationCallback),
				/*bInOpenSubMenuOnClick=*/ false,
				ControlRigIcon);
		}
	}
}

void FControlRigParameterTrackEditor::ToggleIsAdditiveControlRig()
{
	bIsLayeredControlRig = bIsLayeredControlRig ? false : true;
	RefreshControlRigPickerDelegate.ExecuteIfBound(true);
}

bool FControlRigParameterTrackEditor::IsToggleIsAdditiveControlRig()
{
	return bIsLayeredControlRig;
}

void FControlRigParameterTrackEditor::ToggleFilterAssetBySkeleton()
{
	bFilterAssetBySkeleton = bFilterAssetBySkeleton ? false : true;
	RefreshControlRigPickerDelegate.ExecuteIfBound(true);
}

bool FControlRigParameterTrackEditor::IsToggleFilterAssetBySkeleton()
{
	return bFilterAssetBySkeleton;
}

void FControlRigParameterTrackEditor::ToggleFilterAssetByAnimatableControls()
{
	bFilterAssetByAnimatableControls = bFilterAssetByAnimatableControls ? false : true;
	RefreshControlRigPickerDelegate.ExecuteIfBound(true);
}

bool FControlRigParameterTrackEditor::IsToggleFilterAssetByAnimatableControls()
{
	return bFilterAssetByAnimatableControls;
}

void FControlRigParameterTrackEditor::HandleAddTrackSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TFunction<UMovieSceneControlRigParameterTrack* (FGuid)> InTrackCreationCallback)
{
	if (CVarEnableAdditiveControlRigs->GetBool())
	{
		MenuBuilder.AddMenuEntry(
		LOCTEXT("IsLayeredControlRig", "Layered"),
		LOCTEXT("IsLayeredControlRigTooltip", "When checked, a layered control rig will be added"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigParameterTrackEditor::ToggleIsAdditiveControlRig),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigParameterTrackEditor::IsToggleIsAdditiveControlRig)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterAssetBySkeleton", "Filter Asset By Skeleton"),
		LOCTEXT("FilterAssetBySkeletonTooltip", "Filters Control Rig assets to match current skeleton"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigParameterTrackEditor::ToggleFilterAssetBySkeleton),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigParameterTrackEditor::IsToggleFilterAssetBySkeleton)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterAssetByAnimatableControls", "Filter Asset By Animatable Controls"),
		LOCTEXT("FilterAssetByAnimatableControlsTooltip", "Filters Control Rig assets to only show those with Animatable Controls"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &FControlRigParameterTrackEditor::ToggleFilterAssetByAnimatableControls),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FControlRigParameterTrackEditor::IsToggleFilterAssetByAnimatableControls)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton);

	const TSharedPtr<ISequencer> ParentSequencer = GetSequencer();
	UObject* BoundObject = nullptr;
	//todo support multiple bindings?
	USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], &BoundObject, ParentSequencer);

	if (Skeleton)
	{
		TArray<FControlRigAssetStrongReference> ExistingRigs;
		USkeletalMeshComponent* SkeletalMeshComponent = AcquireSkeletalMeshFromObject(BoundObject, ParentSequencer);
		IterateTracks([&ExistingRigs, SkeletalMeshComponent](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid) -> bool
		{
			if (UControlRig* ControlRig = Track->GetControlRig())
			{
				if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
				{
					if (ObjectBinding.IsValid() && ObjectBinding->GetBoundObject() == SkeletalMeshComponent)
					{
						ExistingRigs.Add(ControlRig->GetAssetReference());
					}
				}
			}
			return true;
		});
		MenuBuilder.AddSeparator();
		MenuBuilder.AddMenuEntry(
			LOCTEXT("FKControlRig", "FK Control Rig"),
			LOCTEXT("FKControlRigTooltip", "Adds the FK Control Rig"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FControlRigParameterTrackEditor::AddFKControlRig, BoundObject, ObjectBindings[0], InTrackCreationCallback),
				FCanExecuteAction::CreateLambda([ExistingRigs]()
				{
					if (!ExistingRigs.Contains(UFKControlRig::StaticClass()))
					{
						return true;
					}
					return false;
				})
			),
			NAME_None,
			EUserInterfaceActionType::Button);
		
		FAssetPickerConfig AssetPickerConfig;
		{
			AssetPickerConfig.SelectionMode = ESelectionMode::Single;
			AssetPickerConfig.bAddFilterUI = true;
			AssetPickerConfig.bShowTypeInColumnView = false;
			AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
			AssetPickerConfig.bForceShowPluginContent = true;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FControlRigParameterTrackEditor::AddControlRig, BoundObject, ObjectBindings[0], InTrackCreationCallback);
			AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FControlRigParameterTrackEditor::AddControlRig, BoundObject, ObjectBindings[0], InTrackCreationCallback);
			AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshControlRigPickerDelegate);
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([this, ExistingRigs, Skeleton](const FAssetData& AssetData)
			{
				if (!IsControlRigAllowed(AssetData, ExistingRigs, Skeleton))
				{
					// Should be filtered out
					return true;
				}

				return false;
			});
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.Filter.ClassPaths.Add((UControlRigEditorAssetInterface::StaticClass())->GetClassPathName());
			AssetPickerConfig.Filter.ClassPaths.Add((UControlRigRuntimeAssetInterface::StaticClass())->GetClassPathName());
			AssetPickerConfig.SaveSettingsName = TEXT("SequencerControlRigTrackAssetPicker");
			TSharedRef<FFrontendFilterCategory>	ControlRigFilterCategory = MakeShared<FFrontendFilterCategory>(LOCTEXT("ControlRigFilterCategoryName", "Control Rig Tags"), LOCTEXT("ControlRigFilterCategoryToolTip", "Filter ControlRigs by variant tags specified in ControlRig Blueprint class settings"));
			const URigVMProjectSettings* Settings = GetDefault<URigVMProjectSettings>(URigVMProjectSettings::StaticClass());
			TArray<FRigVMTag> AvailableTags = Settings->VariantTags;

			for (const FRigVMTag& Tag : AvailableTags)
			{
				AssetPickerConfig.ExtraFrontendFilters.Add(MakeShared<UE::RigVM::Editor::Tools::FFilterByAssetTag>(ControlRigFilterCategory, Tag));
			}

			// This is so that we can remove the "Other Filters" section easily
			AssetPickerConfig.bUseSectionsForCustomFilterCategories = true;
			// Make sure we only show ControlRig filters to avoid confusion 
			AssetPickerConfig.OnExtendAddFilterMenu = FOnExtendAddFilterMenu::CreateLambda([ControlRigFilters = AssetPickerConfig.ExtraFrontendFilters](UToolMenu* InToolMenu)
			{
				// "AssetFilterBarFilterAdvancedAsset" taken from SAssetFilterBar.h PopulateAddFilterMenu()
				InToolMenu->RemoveSection("AssetFilterBarFilterAdvancedAsset");
				InToolMenu->RemoveSection("Other Filters");
			});
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		TSharedPtr<SBox> MenuEntry = SNew(SBox)
		// Extra space to display filter capsules horizontally
		.WidthOverride(600.f)
		.HeightOverride(300.f)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

		MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
	}
}

const UClass* FControlRigParameterTrackEditor::GetHandledMixerItemClass() const
{
	return UMovieSceneControlRigParameterTrack::StaticClass();
}

void FControlRigParameterTrackEditor::PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex)
{
	UMovieSceneCommonAnimationTrack* CommonTrack = Cast<UMovieSceneCommonAnimationTrack>(Track);
	if (!CommonTrack)
	{
		return;
	}

	// Create track creation callback that adds the track as a child of the mixer track
	auto TrackCreationCallback = [CommonTrack, RowIndex](FGuid ObjectBinding) -> UMovieSceneControlRigParameterTrack*
	{
		// Use the provided row index if valid, otherwise compute next available
		int32 NewRowIndex = (RowIndex != INDEX_NONE) ? RowIndex : CommonTrack->ComputeNextAvailableRowIndex();
		return Cast<UMovieSceneControlRigParameterTrack>(CommonTrack->AddChildTrack(ObjectBinding, UMovieSceneControlRigParameterTrack::StaticClass(), NewRowIndex));
	};

	// Wrap the control rig menu in a submenu with mixer-specific track creation
	static const FName ControlRigStyleName(TEXT("ControlRigEditorStyle"));
	MenuBuilder.AddSubMenu(
		LOCTEXT("AddControlRig", "Control Rig"),
		LOCTEXT("AddControlRigTooltip", "Adds a control rig track with FK Control Rig option and asset picker"),
		FNewMenuDelegate::CreateLambda([this, ObjectBindings, Track, TrackCreationCallback](FMenuBuilder& SubMenuBuilder)
		{
			HandleAddTrackSubMenu(SubMenuBuilder, ObjectBindings, Track, TrackCreationCallback);
		}),
		/*bInOpenSubMenuOnClick=*/ false,
		FSlateIcon(ControlRigStyleName, "ClassIcon.ControlRigBlueprint")
	);
}

bool FControlRigParameterTrackEditor::IsControlRigAllowed(const FAssetData& AssetData, TArray<FControlRigAssetStrongReference> ExistingRigs, USkeleton* Skeleton)
{
	static const FName RigModuleSettingsPropertyName = GET_MEMBER_NAME_CHECKED(UControlRigBlueprint, RigModuleSettings);
	const FProperty* RigModuleSettingsProperty = CastField<FProperty>(UControlRigBlueprint::StaticClass()->FindPropertyByName(RigModuleSettingsPropertyName));
	const FString RigModuleSettingsStr = AssetData.GetTagValueRef<FString>(RigModuleSettingsPropertyName);
	if(!RigModuleSettingsStr.IsEmpty())
	{
		FRigModuleSettings RigModuleSettings;
		RigModuleSettingsProperty->ImportText_Direct(*RigModuleSettingsStr, &RigModuleSettings, nullptr, EPropertyPortFlags::PPF_None);
		
		// Currently rig module can only be used in a modular rig, not in sequencer
		// see UControlRigBlueprint::IsControlRigModule()
		if (RigModuleSettings.Identifier.IsValid())
		{
			return false;
		}
	}
	
	if (FControlRigAssetInterfacePtr LoadedControlRig = AssetData.FastGetAsset())	
	{
		if (ExistingRigs.Contains(LoadedControlRig->GetControlRigAssetReference()))
		{
			if (!AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UControlRigBlueprint, bAllowMultipleInstances)))
			{
				return false;
			}
		}
	}
	else if (TScriptInterface<IControlRigRuntimeAssetInterface> LoadedRuntimeAsset = AssetData.FastGetAsset())
	{
		if (ExistingRigs.Contains(LoadedRuntimeAsset.GetObject()))
		{
			if (!AssetData.GetTagValueRef<bool>(GET_MEMBER_NAME_CHECKED(UControlRigBlueprintGeneratedClass, bAllowMultipleInstances)))
			{
				return false;
			}
		}
	}

	const IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	
	const bool bExposesAnimatableControls = AssetData.GetTagValueRef<bool>(TEXT("bExposesAnimatableControls"));
	if (bFilterAssetByAnimatableControls == true && bExposesAnimatableControls == false)
	{
		return false;
	}
	if (bIsLayeredControlRig)
	{		
		FAssetDataTagMapSharedView::FFindTagResult Tag = AssetData.TagsAndValues.FindTag(TEXT("SupportedEventNames"));
		if (Tag.IsSet())
		{
			bool bHasInversion = false;
			FString EventString = FRigUnit_InverseExecution::EventName.ToString();
			FString OldEventString = FString(TEXT("Inverse"));
			TArray<FString> SupportedEventNames;
			Tag.GetValue().ParseIntoArray(SupportedEventNames, TEXT(","), true);

			for (const FString& Name : SupportedEventNames)
			{
				if (Name.Contains(EventString) || Name.Contains(OldEventString))
				{
					bHasInversion = true;
					break;
				}
			}
			if (bHasInversion == false)
			{
				return false;
			}
		}
	}
	if (bFilterAssetBySkeleton)
	{
		FString SkeletonName;
		if (Skeleton)
		{
			SkeletonName = FAssetData(Skeleton).GetExportTextName();
		}
		FString PreviewSkeletalMesh = AssetData.GetTagValueRef<FString>(TEXT("PreviewSkeletalMesh"));
		if (PreviewSkeletalMesh.Len() > 0)
		{
			FAssetData SkelMeshData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(PreviewSkeletalMesh));
			FString PreviewSkeleton = SkelMeshData.GetTagValueRef<FString>(TEXT("Skeleton"));
			if (PreviewSkeleton == SkeletonName)
			{
				return true;
			}
			else if(Skeleton)
			{
				if (Skeleton->IsCompatibleForEditor(PreviewSkeleton))
				{
					return true;
				}
			}
		}
		FString PreviewSkeleton = AssetData.GetTagValueRef<FString>(TEXT("PreviewSkeleton"));
		if (PreviewSkeleton == SkeletonName)
		{
			return true;
		}
		else if (Skeleton)
		{
			if (Skeleton->IsCompatibleForEditor(PreviewSkeleton))
			{
				return true;
			}
		}
		FString SourceHierarchyImport = AssetData.GetTagValueRef<FString>(TEXT("SourceHierarchyImport"));
		if (SourceHierarchyImport == SkeletonName)
		{
			return true;
		}
		else if (Skeleton)
		{
			if (Skeleton->IsCompatibleForEditor(SourceHierarchyImport))
			{
				return true;
			}
		}
		FString SourceCurveImport = AssetData.GetTagValueRef<FString>(TEXT("SourceCurveImport"));
		if (SourceCurveImport == SkeletonName)
		{
			return true;
		}
		else if (Skeleton)
		{
			if (Skeleton->IsCompatibleForEditor(SourceCurveImport))
			{
				return true;
			}
		}

		if (!FSoftObjectPath(PreviewSkeletalMesh).IsValid() &&
			!FSoftObjectPath(PreviewSkeleton).IsValid() &&
			!FSoftObjectPath(SourceHierarchyImport).IsValid() &&
			!FSoftObjectPath(SourceCurveImport).IsValid())
		{
			// this indicates that the rig can work on any skeleton (for example, utility rigs or deformer rigs)
			return true;
		}
		
		return false;
	}
	return true;	
}

static TArray<UMovieSceneControlRigParameterTrack*> GetExistingControlRigTracksForSkeletalMeshComponent(UMovieScene* MovieScene, USkeletalMeshComponent* SkeletalMeshComponent)
{
	TArray<UMovieSceneControlRigParameterTrack*> ExistingControlRigTracks;
	IterateTracksInMovieScene(*MovieScene, [&ExistingControlRigTracks, SkeletalMeshComponent](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid) -> bool
	{
		if (UControlRig* ControlRig = Track->GetControlRig())
		{
			if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
			{
				if (ObjectBinding.IsValid() && ObjectBinding->GetBoundObject() == SkeletalMeshComponent)
				{
					ExistingControlRigTracks.Add(Track);
				}
			}
		}
		return true;
	});

	return ExistingControlRigTracks;
}

static UMovieSceneControlRigParameterTrack* AddControlRig(TSharedPtr<ISequencer> SharedSequencer , UMovieSceneSequence* Sequence, const FControlRigAssetStrongReference InSource, FGuid ObjectBinding, UControlRig* InExistingControlRig, bool bIsAdditiveControlRig, TFunction<UMovieSceneControlRigParameterTrack* (FGuid)> InTrackCreationCallback)
{
	FSlateApplication::Get().DismissAllMenus();
	if (!InSource.IsValid() || 
		!Sequence || !Sequence->GetMovieScene())
	{
		return nullptr;
	}

	UMovieScene* OwnerMovieScene = Sequence->GetMovieScene();
	ISequencer* Sequencer = nullptr; // will be valid  if we have a ISequencer AND it's focused.
	if (SharedSequencer.IsValid() && SharedSequencer->GetFocusedMovieSceneSequence() == Sequence)
	{
		Sequencer = SharedSequencer.Get();
	}
	Sequence->Modify();
	OwnerMovieScene->Modify();

	if (bIsAdditiveControlRig && InSource.GetRigClass() != UFKControlRig::StaticClass() && !InSource.GetSupportedEvents().Contains(FRigUnit_InverseExecution::EventName))
	{
		UE_LOGF(LogControlRigEditor, Error, "Cannot add an additive control rig which does not contain a backwards solve event.");
		return nullptr;
	}

	FScopedTransaction AddControlRigTrackTransaction(LOCTEXT("AddControlRigTrack", "Add Control Rig Track"));

	TArray<UMovieSceneControlRigParameterTrack*> ExistingRigTracks;

	USkeletalMeshComponent* SkeletalMeshComponent = AcquireSkeletalMeshFromObjectGuid(ObjectBinding, SharedSequencer);
	if (SkeletalMeshComponent)
	{
		ExistingRigTracks = GetExistingControlRigTracksForSkeletalMeshComponent(OwnerMovieScene, SkeletalMeshComponent);
	}
	
	UMovieSceneControlRigParameterTrack* Track = InTrackCreationCallback ? InTrackCreationCallback(ObjectBinding) : Cast<UMovieSceneControlRigParameterTrack>(OwnerMovieScene->AddTrack(UMovieSceneControlRigParameterTrack::StaticClass(), ObjectBinding));
	if (Track)
	{
		TArray<FName> ExistingObjectNames;
		for (UMovieSceneControlRigParameterTrack* RigTrack : ExistingRigTracks)
		{
			if (UControlRig* Rig = RigTrack->GetControlRig())
			{
				if (InSource.IsSourceOf(Rig))
				{
					ExistingObjectNames.Add(RigTrack->GetTrackName());
				}
			}
		}
		
		FString ObjectName = InSource.GetName(); //GetDisplayNameText().ToString();
		ObjectName.RemoveFromEnd(TEXT("_C"));

		{
			FName UniqueObjectName = *ObjectName;
			int32 UniqueSuffix = 1;
			while(ExistingObjectNames.Contains(UniqueObjectName))
			{
				UniqueObjectName = *(ObjectName + TEXT("_") + FString::FromInt(UniqueSuffix));
				UniqueSuffix++;
			}

			ObjectName = UniqueObjectName.ToString();
		}
		
		
		bool bSequencerOwnsControlRig = false;
		UControlRig* ControlRig = InExistingControlRig;
		if (ControlRig == nullptr)
		{
			ControlRig = InSource.CreateInstance(Track, FName(*ObjectName), RF_Transactional);
			bSequencerOwnsControlRig = true;
		}

		ControlRig->Modify();
		if (UFKControlRig* FKControlRig = Cast<UFKControlRig>(Cast<UControlRig>(ControlRig)))
		{
			if (bIsAdditiveControlRig)
			{
				FKControlRig->SetApplyMode(EControlRigFKRigExecuteMode::Additive);
			}
		}
		else
		{
			ControlRig->SetIsAdditive(bIsAdditiveControlRig);
		}

		// The bound object and the data source must be initialized before construction is executed so that
		// things like FRigUnit_HierarchyImportFromSkeleton can work properly 
		ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
		if (SkeletalMeshComponent)
		{
			ControlRig->GetObjectBinding()->BindToObject(SkeletalMeshComponent);
			ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, SkeletalMeshComponent);
		}
		
		// Do not re-initialize existing control rig
		if (!InExistingControlRig)
		{
			ControlRig->Initialize();
		}
		ControlRig->Evaluate_AnyThread();

		if (SharedSequencer.IsValid())
		{
			SharedSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		}

		Track->Modify();
		UMovieSceneSection* NewSection = Track->CreateControlRigSection(0, ControlRig, bSequencerOwnsControlRig);
		NewSection->Modify();

		if (bIsAdditiveControlRig)
		{
			const FString AdditiveObjectName = ObjectName + TEXT(" (Layered)");
			Track->SetTrackName(FName(*ObjectName));
			Track->SetDisplayName(FText::FromString(AdditiveObjectName));
			Track->SetColorTint(UMovieSceneControlRigParameterTrack::LayeredRigTrackColor);
		}
		else
		{
			//mz todo need to have multiple rigs with same class
			Track->SetTrackName(FName(*ObjectName));
			Track->SetDisplayName(FText::FromString(ObjectName));
			Track->SetColorTint(UMovieSceneControlRigParameterTrack::AbsoluteRigTrackColor);
		}

		if (SharedSequencer.IsValid())
		{
			SharedSequencer->EmptySelection();
			SharedSequencer->SelectSection(NewSection);
			SharedSequencer->ThrobSectionSelection();
			SharedSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
			SharedSequencer->ObjectImplicitlyAdded(ControlRig);
		}

		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!ControlRigEditMode)
		{
			GLevelEditorModeTools().ActivateMode(FControlRigEditMode::ModeName);
			ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));

		}
		if (ControlRigEditMode)
		{
			ControlRigEditMode->AddControlRigObject(ControlRig, SharedSequencer);
		}
		return Track;
	}
	return nullptr;
}
static UMovieSceneTrack* FindOrCreateControlRigTrack(TSharedPtr<ISequencer>& Sequencer, UWorld* World, const FControlRigAssetStrongReference ControlRigAssetReference, const FMovieSceneBindingProxy& InBinding, bool bIsLayeredControlRig, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	UMovieSceneTrack* BaseTrack = nullptr;
	if (MovieScene && InBinding.BindingID.IsValid())
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
		{
			if (!DoesControlRigAllowMultipleInstances(FTopLevelAssetPath(ControlRigAssetReference.GetPathName())))
			{
				TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding->GetObjectGuid(), NAME_None);
				for (UMovieSceneTrack* AnyOleTrack : Tracks)
				{
					UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack);
					if (Track && Track->GetControlRig() && ControlRigAssetReference.IsSourceOf(Track->GetControlRig()))
					{
						return Track;
					}
				}
			}

			UMovieSceneControlRigParameterTrack* Track = AddControlRig(Sequencer, InBinding.Sequence, ControlRigAssetReference, InBinding.BindingID, nullptr, bIsLayeredControlRig, InTrackCreationCallback);

			if (Track)
			{
				BaseTrack = Track;
			}
		}
	}
	return BaseTrack;
}

void FControlRigParameterTrackEditor::AddControlRig(const FControlRigAssetStrongReference InSource, UObject* BoundActor, FGuid ObjectBinding, UControlRig* InExistingControlRig, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	FMovieSceneBindingProxy BindingProxy(ObjectBinding, Sequence);

	//UControlRigSequencerEditorLibrary::FindOrCreateControlRigTrack.. in 5.5 we will redo this but for
	//5.4.4 we can't change headers so for now we just make the change here locally
	if (UMovieSceneTrack* Track = FindOrCreateControlRigTrack(Sequencer, World, InSource, BindingProxy, bIsLayeredControlRig, InTrackCreationCallback))
	{
		BindControlRig(CastChecked<UMovieSceneControlRigParameterTrack>(Track)->GetControlRig(), ObjectBinding);
	}
}

void FControlRigParameterTrackEditor::AddControlRig(const FAssetData& InAsset, UObject* BoundActor, FGuid ObjectBinding, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback)
{
	FControlRigAssetStrongReference RigSource = InAsset.GetAsset();
	if (!RigSource.IsValid())
	{
		if (FControlRigAssetInterfacePtr EditorAsset = InAsset.GetAsset())
		{
			RigSource = EditorAsset->GetRuntimeAssetInterface().GetObject();
		}
	}
	if (RigSource.IsValid())
	{
		AddControlRig(RigSource, BoundActor, ObjectBinding, nullptr, InTrackCreationCallback);
	}
}

void FControlRigParameterTrackEditor::AddControlRig(const TArray<FAssetData>& InAssets, UObject* BoundActor, FGuid ObjectBinding, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback)
{
	if (InAssets.Num() > 0)
	{
		AddControlRig(InAssets[0], BoundActor, ObjectBinding, InTrackCreationCallback);
	}
}

void FControlRigParameterTrackEditor::AddFKControlRig(UObject* BoundActor, FGuid ObjectBinding, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback)
{
	AddControlRig(UFKControlRig::StaticClass(), BoundActor, ObjectBinding, nullptr, InTrackCreationCallback);
}

void FControlRigParameterTrackEditor::AddControlRig(FControlRigAssetStrongReference InSource, UObject* BoundActor, FGuid ObjectBinding)
{
	if (InSource.Get() == UFKControlRig::StaticClass())
	{
		AcquireSkeletonFromObjectGuid(ObjectBinding, &BoundActor, GetSequencer());
	}
	AddControlRig(InSource, BoundActor, ObjectBinding, nullptr);
}

//This now adds all of the control rig components, not just the first one
void FControlRigParameterTrackEditor::AddControlRigFromComponent(FGuid InGuid)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	UObject* BoundObject = Sequencer ? Sequencer->FindSpawnedObjectOrTemplate(InGuid) : nullptr;

	if (AActor* BoundActor = Cast<AActor>(BoundObject))
	{
		TArray<UControlRigComponent*> ControlRigComponents;
		BoundActor->GetComponents(ControlRigComponents);
		for (UControlRigComponent* ControlRigComponent : ControlRigComponents)
		{
			if (UControlRig* CR = ControlRigComponent->GetControlRig())
			{
				AddControlRig(CR->GetAssetReference(), BoundActor, InGuid, CR);
			}
		}
	}
}

bool FControlRigParameterTrackEditor::HasTransformKeyOverridePriority() const
{
	return false;
}

bool FControlRigParameterTrackEditor::CanAddTransformKeysForSelectedObjects() const
{
	// WASD hotkeys to fly the viewport can conflict with hotkeys for setting keyframes (ie. s). 
	// If the viewport is moving, disregard setting keyframes.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		if (LevelVC && LevelVC->IsMovingCamera())
		{
			return false;
		}
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || !Sequencer->IsAllowedToChange())
	{
		return false;
	}

	FControlRigEditMode* ControlRigEditMode = GetEditMode();
	if (ControlRigEditMode)
	{
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		ControlRigEditMode->GetAllSelectedControls(SelectedControls);
		return (SelectedControls.Num() > 0);
	}
	
	return false;
}

void FControlRigParameterTrackEditor::SetKeysOnControls(const TMap<UControlRig*, TArray<FRigElementKey>>& SelectedControls, EMovieSceneTransformChannel Channel)
{
	const EControlRigContextChannelToKey ChannelsToKey = static_cast<EControlRigContextChannelToKey>(Channel);

	static constexpr bool bInConstraintSpace = true;
	FRigControlModifiedContext NotifyDrivenContext;
	NotifyDrivenContext.SetKey = EControlRigSetKey::Always;

	for (const TPair<UControlRig*, TArray<FRigElementKey>>& Selection : SelectedControls)
	{
		UControlRig* ControlRig = Selection.Key;
		const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding();
		if (!ObjectBinding)
		{
			continue;
		}
		UObject* Object = ObjectBinding->GetBoundObject();
		if (!Object)
		{
			continue;
		}

		const FName ControlRigName(*ControlRig->GetName());

		// Cache FRigControlElement* for each selected control to avoid repeated FindControl lookups.
		TArray<FControlRigKeyAddParams> Params;
		Params.Reserve(Selection.Value.Num());
		for (const FRigElementKey& ControlKey : Selection.Value)
		{
			if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name))
			{
				FControlRigKeyAddParams& Param = Params.AddDefaulted_GetRef();
				Param.RigControlName = ControlKey.Name;
				Param.ControlElement = ControlElement;
			}
		}

		if (Params.IsEmpty())
		{
			continue;
		}

		// Single call: resolves handle/track once, Modify()s each section once, keys all controls.
		AddControlKeys(Object, ControlRig, ControlRigName, Params,
			ChannelsToKey, ESequencerKeyMode::ManualKeyForced, FLT_MAX, bInConstraintSpace);

		for (const FControlRigKeyAddParams& Param : Params)
		{
			FControlRigEditMode::NotifyDrivenControls(ControlRig, Param.ControlElement->GetKey(), NotifyDrivenContext);
		}
	}
}

void FControlRigParameterTrackEditor::OnAddTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || !Sequencer->IsAllowedToChange())
	{
		return;
	}
	
	const FControlRigEditMode* ControlRigEditMode = GetEditMode();
	if (!ControlRigEditMode)
	{
		return;
	}

	TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
	ControlRigEditMode->GetAllSelectedControls(SelectedControls);
	if (SelectedControls.Num() <= 0)
	{
		return;
	}
	FScopedTransaction KeyTransaction(LOCTEXT("SetKeysOnControls", "Set Keys On Controls"), !GIsTransacting);
	SetKeysOnControls(SelectedControls, Channel);

}

void FControlRigParameterTrackEditor::OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || !Sequencer->IsAllowedToChange())
	{
		return;
	}

	const FControlRigEditMode* ControlRigEditMode = GetEditMode();
	if (!ControlRigEditMode)
	{
		return;
	}

	TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
	ControlRigEditMode->GetAllSelectedControls(SelectedControls);
	if (SelectedControls.Num() <= 0)
	{
		return;
	}

	FScopedTransaction DeleteTransaction(LOCTEXT("DeleteKeysOnControls", "Delete Keys On Controls"), !GIsTransacting);

	const EControlRigContextChannelToKey ChannelsToDelete = static_cast<EControlRigContextChannelToKey>(Channel);
	const FFrameNumber CurrentFrame = Sequencer->GetLocalTime().Time.FrameNumber;
	const TRange<FFrameNumber> FrameRange(
		TRangeBound<FFrameNumber>::Inclusive(CurrentFrame),
		TRangeBound<FFrameNumber>::Inclusive(CurrentFrame));

	auto DeleteKeyFromChannel = [&FrameRange](auto* TypedChannel, UMovieSceneSection* Section, bool& bModified)
	{
		TArray<FFrameNumber> KeyTimes;
		TArray<FKeyHandle> KeyHandles;
		TypedChannel->GetData().GetKeys(FrameRange, &KeyTimes, &KeyHandles);
		if (KeyHandles.Num() > 0)
		{
			if (!bModified)
			{
				Section->Modify();
				bModified = true;
			}
			TypedChannel->DeleteKeys(MakeArrayView(KeyHandles));
		}
	};

	for (const TPair<UControlRig*, TArray<FRigElementKey>>& Selection : SelectedControls)
	{
		UControlRig* ControlRig = Selection.Key;
		if (!ControlRig)
		{
			continue;
		}

		UMovieSceneControlRigParameterTrack* Track = FindTrack(ControlRig);
		if (!Track)
		{
			continue;
		}

		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

		for (UMovieSceneSection* BaseSection : Track->GetAllSections())
		{
			UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(BaseSection);
			if (!Section || Section->GetControlRig() != ControlRig)
			{
				continue;
			}

			TArrayView<FMovieSceneBoolChannel*>    BoolChannels    = Section->GetChannelProxy().GetChannels<FMovieSceneBoolChannel>();
			TArrayView<FMovieSceneByteChannel*>    ByteChannels    = Section->GetChannelProxy().GetChannels<FMovieSceneByteChannel>();
			TArrayView<FMovieSceneIntegerChannel*> IntegerChannels = Section->GetChannelProxy().GetChannels<FMovieSceneIntegerChannel>();
			TArrayView<FMovieSceneFloatChannel*>   FloatChannels   = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();

			bool bModified = false;

			for (const FRigElementKey& ControlKey : Selection.Value)
			{
				FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name);
				if (!ControlElement || !Hierarchy->IsAnimatable(ControlElement))
				{
					continue;
				}

				if (Section->GetControlNameMask(ControlElement->GetFName()) == false)
				{
					continue;
				}

				FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlElement->GetFName());
				if (!pChannelIndex)
				{
					continue;
				}

				const int32 ChannelIndex = pChannelIndex->ChannelIndex;


				//IF All then delete space and transform keys, do this first to handle previous keys etc..
				// Delete space switch key at this frame for this control
				if (Channel == EMovieSceneTransformChannel::AllTransform || Channel == EMovieSceneTransformChannel::All)
				{
					if (FSpaceControlNameAndChannel* SpaceControlChannel = Section->GetSpaceChannel(ControlElement->GetFName()))
					{
						DeleteKeyFromChannel(&SpaceControlChannel->SpaceCurve, Section, bModified);
					}

					// Delete constraint keys at this frame for this control
					for (FConstraintAndActiveChannel& ConstraintAndChannel : Section->GetConstraintsChannels())
					{
						if (Section->FindControlNameFromConstraintChannel(&ConstraintAndChannel.ActiveChannel) == ControlElement->GetFName())
						{
							DeleteKeyFromChannel(&ConstraintAndChannel.ActiveChannel, Section, bModified);
						}
					}
				}

				switch (ControlElement->Settings.ControlType)
				{
				case ERigControlType::Bool:
				{
					if (ChannelIndex < BoolChannels.Num())
					{
						DeleteKeyFromChannel(BoolChannels[ChannelIndex], Section, bModified);
					}
					break;
				}
				case ERigControlType::Integer:
				{
					if (ControlElement->Settings.ControlEnum)
					{
						if (ChannelIndex < ByteChannels.Num())
						{
							DeleteKeyFromChannel(ByteChannels[ChannelIndex], Section, bModified);
						}
					}
					else
					{
						if (ChannelIndex < IntegerChannels.Num())
						{
							DeleteKeyFromChannel(IntegerChannels[ChannelIndex], Section, bModified);
						}
					}
					break;
				}
				case ERigControlType::Float:
				case ERigControlType::ScaleFloat:
				{
					if (ChannelIndex < FloatChannels.Num())
					{
						DeleteKeyFromChannel(FloatChannels[ChannelIndex], Section, bModified);
					}
					break;
				}
				case ERigControlType::Vector2D:
				{
					// Ignore channel mask for Vector2D — delete both axes
					if (ChannelIndex < FloatChannels.Num())     DeleteKeyFromChannel(FloatChannels[ChannelIndex],     Section, bModified);
					if (ChannelIndex + 1 < FloatChannels.Num()) DeleteKeyFromChannel(FloatChannels[ChannelIndex + 1], Section, bModified);
					break;
				}
				case ERigControlType::Position:
				{
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::TranslationX) && ChannelIndex < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::TranslationY) && ChannelIndex + 1 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 1], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::TranslationZ) && ChannelIndex + 2 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 2], Section, bModified);
					break;
				}
				case ERigControlType::Rotator:
				{
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::RotationX) && ChannelIndex < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::RotationY) && ChannelIndex + 1 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 1], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::RotationZ) && ChannelIndex + 2 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 2], Section, bModified);
					break;
				}
				case ERigControlType::Scale:
				{
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::ScaleX) && ChannelIndex < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::ScaleY) && ChannelIndex + 1 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 1], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::ScaleZ) && ChannelIndex + 2 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 2], Section, bModified);
					break;
				}
				case ERigControlType::Transform:
				case ERigControlType::EulerTransform:
				case ERigControlType::TransformNoScale:
				{
					// Translation X, Y, Z
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::TranslationX) && ChannelIndex < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::TranslationY) && ChannelIndex + 1 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 1], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::TranslationZ) && ChannelIndex + 2 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 2], Section, bModified);
					// Rotation X, Y, Z
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::RotationX) && ChannelIndex + 3 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 3], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::RotationY) && ChannelIndex + 4 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 4], Section, bModified);
					if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::RotationZ) && ChannelIndex + 5 < FloatChannels.Num())
						DeleteKeyFromChannel(FloatChannels[ChannelIndex + 5], Section, bModified);
					// Scale X, Y, Z
					if ((ControlElement->Settings.ControlType) != ERigControlType::TransformNoScale)
					{
						if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::ScaleX) && ChannelIndex + 6 < FloatChannels.Num())
							DeleteKeyFromChannel(FloatChannels[ChannelIndex + 6], Section, bModified);
						if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::ScaleY) && ChannelIndex + 7 < FloatChannels.Num())
							DeleteKeyFromChannel(FloatChannels[ChannelIndex + 7], Section, bModified);
						if (EnumHasAnyFlags(ChannelsToDelete, EControlRigContextChannelToKey::ScaleZ) && ChannelIndex + 8 < FloatChannels.Num())
							DeleteKeyFromChannel(FloatChannels[ChannelIndex + 8], Section, bModified);
					}
					break;
				}
				default:
					break;
				}
			}
		}
	}
}

//function to evaluate a Control and Set it on the ControlRig
static void EvaluateThisControl(const FFrameRate& TickResolution, UMovieSceneControlRigParameterSection* Section, const FName& ControlName, const FFrameTime& FrameTime)
{
	if (!Section)
	{
		return;
	}
	UControlRig* ControlRig = Section->GetControlRig();
	if (!ControlRig)
	{
		return;
	}
	if(FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
	{ 
		FControlRigInteractionScope InteractionScope(ControlRig, ControlElement->GetKey());
		URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();

		FRigControlModifiedContext Context(EControlRigSetKey::Never);
		double Time = TickResolution.AsSeconds(FrameTime);
		Context.LocalTime = (float)(Time);
		//eval any space for this channel, if not additive section
		if (Section->GetBlendType().Get() != EMovieSceneBlendType::Additive)
		{
			TOptional<FMovieSceneControlRigSpaceBaseKey> SpaceKey = Section->EvaluateSpaceChannel(FrameTime, ControlName);
			if (SpaceKey.IsSet())
			{
				const FRigElementKey ControlKey = ControlElement->GetKey();
				switch (SpaceKey.GetValue().SpaceType)
				{
				case EMovieSceneControlRigSpaceType::Parent:
					ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
					break;
				case EMovieSceneControlRigSpaceType::World:
					ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
					break;
				case EMovieSceneControlRigSpaceType::ControlRig:
					ControlRig->SwitchToParent(ControlKey, SpaceKey.GetValue().ControlRigElement, false, true);
					break;	

				}
			}
		}
		const bool bSetupUndo = false;
		switch (ControlElement->Settings.ControlType)
		{
			case ERigControlType::Bool:
			{
				if (Section->GetBlendType().Get() != EMovieSceneBlendType::Additive)
				{
					TOptional <bool> Value = Section->EvaluateBoolParameter(FrameTime, ControlName);
					if (Value.IsSet())
					{
						ControlRig->SetControlValue<bool>(ControlName, Value.GetValue(), true, Context, bSetupUndo);
					}
				}
				break;
			}
			case ERigControlType::Integer:
			{
				if (Section->GetBlendType().Get() != EMovieSceneBlendType::Additive)
				{
					if (ControlElement->Settings.ControlEnum)
					{
						TOptional<uint8> Value = Section->EvaluateEnumParameter(FrameTime, ControlName);
						if (Value.IsSet())
						{
							int32 IVal = (int32)Value.GetValue();
							ControlRig->SetControlValue<int32>(ControlName, IVal, true, Context, bSetupUndo);
						}
					}
					else
					{
						TOptional <int32> Value = Section->EvaluateIntegerParameter(FrameTime, ControlName);
						if (Value.IsSet())
						{
							ControlRig->SetControlValue<int32>(ControlName, Value.GetValue(), true, Context, bSetupUndo);
						}
					}
				}
				break;

			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				TOptional <float> Value = Section->EvaluateScalarParameter(FrameTime, ControlName);
				if (Value.IsSet())
				{
					ControlRig->SetControlValue<float>(ControlName, Value.GetValue(), true, Context, bSetupUndo);
				}
				break;
			}
			case ERigControlType::Vector2D:
			{
				TOptional <FVector2D> Value = Section->EvaluateVector2DParameter(FrameTime, ControlName);
				if (Value.IsSet())
				{
					ControlRig->SetControlValue<FVector2D>(ControlName, Value.GetValue(), true, Context, bSetupUndo);
				}
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Scale:
			case ERigControlType::Rotator:
			{
				TOptional <FVector> Value = Section->EvaluateVectorParameter(FrameTime, ControlName);
				if (Value.IsSet())
				{
					FVector3f FloatVal = (FVector3f)Value.GetValue();
					ControlRig->SetControlValue<FVector3f>(ControlName, FloatVal, true, Context, bSetupUndo);
				}
				break;
			}

			case ERigControlType::Transform:
			case ERigControlType::TransformNoScale:
			case ERigControlType::EulerTransform:
			{
				// @MikeZ here I suppose we want to retrieve the rotation order and then also extract the Euler angles
				// instead of an assumed FRotator coming from the section?
				// EEulerRotationOrder RotationOrder = SomehowGetRotationOrder();
					
				TOptional <FEulerTransform> Value = Section->EvaluateTransformParameter(FrameTime, ControlName);
				if (Value.IsSet())
				{
					if (ControlElement->Settings.ControlType == ERigControlType::Transform)
					{
						FVector EulerAngle(Value.GetValue().Rotation.Roll, Value.GetValue().Rotation.Pitch, Value.GetValue().Rotation.Yaw);
						RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
						ControlRig->SetControlValue<FRigControlValue::FTransform_Float>(ControlName, Value.GetValue().ToFTransform(), true, Context, bSetupUndo);
					}
					else if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
					{
						FTransformNoScale NoScale = Value.GetValue().ToFTransform();
						FVector EulerAngle(Value.GetValue().Rotation.Roll, Value.GetValue().Rotation.Pitch, Value.GetValue().Rotation.Yaw);
						RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
						ControlRig->SetControlValue<FRigControlValue::FTransformNoScale_Float>(ControlName, NoScale, true, Context, bSetupUndo);
					}
					else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						const FEulerTransform& Euler = Value.GetValue();
						FVector EulerAngle(Euler.Rotation.Roll, Euler.Rotation.Pitch, Euler.Rotation.Yaw);
						FQuat Quat = RigHierarchy->GetControlQuaternion(ControlElement, EulerAngle);
						RigHierarchy->SetControlSpecifiedEulerAngle(ControlElement, EulerAngle);
						FRotator UERotator(Quat);
						FEulerTransform Transform = Euler;
						Transform.Rotation = UERotator;
						ControlRig->SetControlValue<FRigControlValue::FEulerTransform_Float>(ControlName, Transform, true, Context, bSetupUndo);

					}
				}
				break;

			}
		}
		//make sure control rig time is correct
		ControlRig->SetAbsoluteTime(Time); 
		//note we do need to evaluate the rig, within the interaction scope now
		ControlRig->Evaluate_AnyThread();
	}
}

//When a channel is changed via Sequencer we need to call SetControlValue on it so that Control Rig can handle seeing that this is a change, but just on this value
//and then it send back a key if needed, which happens with IK/FK switches. Hopefully new IK/FK system will remove need for this at some point.
//We also compensate since the changed control could happen at a space switch boundary.
//Finally, since they can happen thousands of times interactively when moving a bunch of keys on a control rig we move to doing this into the next tick
struct FChannelChangedStruct
{
	FTimerHandle TimerHandle;
	bool bWasSetAlready = false;
	TMap<UMovieSceneControlRigParameterSection*, TSet<FName>> SectionControlNames;
};

void FControlRigParameterTrackEditor::OnMovieSceneBindingsChanged()
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}
	if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(Sequencer.Get()))
	{
		for (UAnimLayer* AnimLayer : AnimLayers->AnimLayers)
		{
			if (AnimLayer)
			{
				AnimLayer->UpdateSceneObjectorGuidsForItems(Sequencer.Get());
			}
		}
	}
}

//this function used to set up a set of control names per control and evaluate them on next tick, but this doesn't work with 
//certain rigs that send events immediately on an evaluation which would happen before the tick. In that case we would get the set key events
//for the fk/ik but on a non-game thread so we couldn't set any keys.
//the fix is to just immediately set the control value on the channel(bool) and then that will send the key events on this game thread.
void FControlRigParameterTrackEditor::OnChannelChanged(const TArray<FFrameNumber>& Frames, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection* InSection)
{
	// During Undo / Redo, this interferes with the Undo / Redo key data that is applied. See CL description.
	if (GIsTransacting)
	{
		return;
	}
	
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (Section && Section->GetControlRig() && MetaData)
	{
		TArray<FString> StringArray;
		FString String = MetaData->Name.ToString();
		String.ParseIntoArray(StringArray, TEXT("."));
		if (StringArray.Num() > 0)
		{
			FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

			FName ControlName(*StringArray[0]);
			if (FRigControlElement* ControlElement = Section->GetControlRig()->FindControl(ControlName))
			{
				if (ControlElement->Settings.ControlType == ERigControlType::Bool)
				{
					if (Frames.Num() <= 0)
					{
						FFrameTime Time = Sequencer->GetLocalTime().Time;
						EvaluateThisControl(TickResolution, Section, ControlName, Time);
					}
					else
					{
						for (const FFrameNumber& Frame : Frames)
						{
							EvaluateThisControl(TickResolution,Section, ControlName, Frame);
						}
					}
				}
			}
		}
	}
}

void FControlRigParameterTrackEditor::AddConstraintToSequencer(const TSharedPtr<ISequencer>& InSequencer, UTickableTransformConstraint* InConstraint,
	const UConstraintCreationOptions* InSequencerOptions)
{
	if (InConstraint)
	{
		TGuardValue<bool> DisableTrackCreation(bAutoGenerateControlRigTrack, false);
		if (InSequencerOptions)
		{
			InConstraint->Evaluate();
		}
	
		static const FSequencerCreationOptions DefaultOptions;
		const FSequencerCreationOptions& Options = InSequencerOptions ? InSequencerOptions->SequencerOptions : DefaultOptions;
	
		FMovieSceneConstraintChannelHelper::AddConstraintToSequencer(InSequencer, InConstraint, Options);
	}
}

void FControlRigParameterTrackEditor::AddTrackForComponent(USceneComponent* InComponent, FGuid InBinding) 
{
	if (USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(InComponent))
	{
		if(bAutoGenerateControlRigTrack && !SkelMeshComp->GetDefaultAnimatingRig().IsNull())
		{
			UObject* Object = SkelMeshComp->GetDefaultAnimatingRig().LoadSynchronous();
			TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = Object;
			TScriptInterface<IControlRigEditorAssetInterface> EditorAsset = Object;
			if (RuntimeAsset || EditorAsset)
			{
				if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
				{
					FGuid Binding = InBinding.IsValid() ? InBinding : Sequencer->GetHandleToObject(InComponent, true /*bCreateHandle*/);
					if (Binding.IsValid())
					{
						UMovieSceneSequence* OwnerSequence = Sequencer->GetFocusedMovieSceneSequence();
						UMovieScene* OwnerMovieScene = OwnerSequence->GetMovieScene();
						UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(OwnerMovieScene->FindTrack(UMovieSceneControlRigParameterTrack::StaticClass(), Binding, NAME_None));
						if (Track == nullptr)
						{
							FControlRigAssetStrongReference RigSource;
							if (EditorAsset)
							{
								RigSource = EditorAsset->GetRuntimeAssetInterface().GetObject();
							}
							else
							{
								RigSource = RuntimeAsset.GetObject();
							}

							if (RigSource.IsValid())
							{
								AddControlRig(RigSource, InComponent, Binding);
							}
						}
					}
				}
			}
		}
	}
	
	TArray<USceneComponent*> ChildComponents;
	InComponent->GetChildrenComponents(false, ChildComponents);
	for (USceneComponent* ChildComponent : ChildComponents)
	{
		AddTrackForComponent(ChildComponent, FGuid());
	}
}

//test to see if actor has a constraint, in which case we need to add a constraint channel/key
//or a control rig in which case we create a track if cvar is off
void FControlRigParameterTrackEditor::HandleActorAdded(AActor* Actor, FGuid TargetObjectGuid)
{
	if (Actor == nullptr)
	{
		return;
	}

	//test for constraint
	if (bAutoGenerateControlRigTrack)
	{
		if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
		{
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(Actor->GetWorld());
			const TArray< TWeakObjectPtr<UTickableConstraint>> Constraints = Controller.GetAllConstraints();
			for (const TWeakObjectPtr<UTickableConstraint>& WeakConstraint : Constraints)
			{
				if (UTickableTransformConstraint* Constraint = WeakConstraint.IsValid() ? Cast<UTickableTransformConstraint>(WeakConstraint.Get()) : nullptr)
				{
					if (UObject* Child = Constraint->ChildTRSHandle ? Constraint->ChildTRSHandle->GetTarget().Get() : nullptr)
					{
						const AActor* TargetActor = Child->IsA<AActor>() ? Cast<AActor>(Child) : Child->GetTypedOuter<AActor>();
						if (TargetActor == Actor)
						{
							AddConstraintToSequencer(Sequencer, Constraint);
						}		
					}
				}
			}
		}
	}
	
	//test for control rig

	if (!CVarAutoGenerateControlRigTrack.GetValueOnGameThread())
	{
		return;
	}

	if (UControlRigComponent* ControlRigComponent = Actor->FindComponentByClass<UControlRigComponent>())
	{
		AddControlRigFromComponent(TargetObjectGuid);
		return;
	}

	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
	{
		AddTrackForComponent(SkeletalMeshComponent, TargetObjectGuid);
		return;
	}

	for (UActorComponent* Component : Actor->GetComponents())
	{
		if (USceneComponent* SceneComp = Cast<USceneComponent>(Component))
		{
			AddTrackForComponent(SceneComp, FGuid());
		}
	}
}

void FControlRigParameterTrackEditor::OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID)
{
	IterateTracks([this](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
	{
		if (UControlRig* ControlRig = Track->GetControlRig())
		{
			BindControlRig(ControlRig, InGuid);
		}
		return false;
	});

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (bControlRigEditModeWasOpen && Sequencer && Sequencer->IsLevelEditorSequencer())
	{
		using namespace UE::ControlRigEditor::Private;
		
		DeferToNextTick(DeactivateModeHandle, [WeakThis = SharedThis(this).ToWeakPtr()]()
		{
			TSharedPtr<FControlRigParameterTrackEditor> SharedThis = WeakThis.IsValid() ? WeakThis.Pin() : nullptr;
			if (!SharedThis.IsValid())
			{
				return;
			}

			//we need to make sure pending deactivated edit modes, including a possible control rig edit mode
			//get totally removed which only happens on a tick 
			if (GLevelEditorModeTools().HasToolkitHost())
			{
				auto GetActiveLevelViewportClient = []() -> FLevelEditorViewportClient* 
				{
					if (const FViewport* ActiveViewport = GEditor ? GEditor->GetActiveViewport() : nullptr)
					{
						for (FLevelEditorViewportClient* LevelViewportClient : GEditor->GetLevelViewportClients())
						{
							if (LevelViewportClient->Viewport == ActiveViewport)
							{
								return LevelViewportClient;
							}
						}
					}
					return nullptr;
				};
				
				if (FLevelEditorViewportClient* ViewportClient = GetActiveLevelViewportClient())
				{
					if (FEditorModeTools* EditorModeTools = SharedThis->GetEditorModeTools())
					{
						EditorModeTools->Tick(ViewportClient, 0.033f);
					}
				}
				
				// now we can recreate it
				DeferToNextTick(SharedThis->ActivateModeHandle, [WeakThis]()
				{
					TSharedPtr<FControlRigParameterTrackEditor> SharedThis = WeakThis.IsValid() ? WeakThis.Pin() : nullptr;
					if (!SharedThis.IsValid())
					{
						return;
					}
					
					constexpr bool bForceActivate = true;
					if (FControlRigEditMode* ControlRigEditMode = SharedThis->GetEditMode(bForceActivate))
					{
						bool bSequencerSet = false;

						TSharedPtr<ISequencer> Sequencer = SharedThis->GetSequencer();
						for (TPair<TWeakObjectPtr<UControlRig>,FGuid>& ControlRig : SharedThis->BoundControlRigs)
						{
							if (UControlRig* Rig = ControlRig.Key.Get())
							{
								ControlRigEditMode->AddControlRigObject(Rig, Sequencer);
								bSequencerSet = true;

								for (int32 Index = 0; Index < PreviousSelectedControlRigs.Num(); ++Index)
								{
									if (Rig->GetAssetReference() == PreviousSelectedControlRigs[Index].Key)
									{
										for (const FName& ControlName : PreviousSelectedControlRigs[Index].Value)
										{
											Rig->SelectControl(ControlName, true);
										}
										PreviousSelectedControlRigs.RemoveAt(Index);
										break;
									}
								}
							}
						}

						if (!bSequencerSet)
						{
							ControlRigEditMode->SetSequencer(Sequencer);
						}
					}
					PreviousSelectedControlRigs.Reset();
						
					SharedThis->ActivateModeHandle.Invalidate();
				});
			}
				
			SharedThis->DeactivateModeHandle.Invalidate();
		});
	}

	//update bindings here
	if (Sequencer)
	{
		if (UAnimLayers* AnimLayers = UAnimLayers::GetAnimLayers(Sequencer.Get()))
		{
			for (UAnimLayer* AnimLayer : AnimLayers->AnimLayers)
			{
				if (AnimLayer)
				{
					AnimLayer->UpdateSceneObjectorGuidsForItems(Sequencer.Get());
				}
			}
		}
	}
}


void FControlRigParameterTrackEditor::OnSequencerDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	UMovieSceneSequence* OwnerSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;

	if (!MovieScene)
	{
		return;
	}

	FControlRigEditMode* ControlRigEditMode = GetEditMode();
	static const TArray<UControlRig*> NoRigs;
	const TArray<UControlRig*>& ControlRigs = ControlRigEditMode ? ControlRigEditMode->GetControlRigsArray(false /*bIsVisible*/) : NoRigs;

	if (ControlRigs.IsEmpty())
	{
		return;
	}
	
	//if we have a valid control rig edit mode need to check and see the control rig in that mode is still in a track
	//if not we get rid of it.
	if (DataChangeType == EMovieSceneDataChangeType::MovieSceneStructureItemRemoved || DataChangeType == EMovieSceneDataChangeType::Unknown)
	{
		const float FPS = static_cast<float>(Sequencer->GetFocusedDisplayRate().AsDecimal());
		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				ControlRig->SetFramesPerSecond(FPS);

				bool bControlRigInTrack = false;
				IterateTracks([&bControlRigInTrack, ControlRig](const UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
				{
					if(Track->GetControlRig() == ControlRig)
			        {
						bControlRigInTrack = true;
						return false;
			        }

					return true;
				});

				if (!bControlRigInTrack)
				{
					ControlRigEditMode->RemoveControlRig(ControlRig);
				}
			}
		}
	}
}


void FControlRigParameterTrackEditor::PostEvaluation(UMovieScene* MovieScene, FFrameNumber Frame)
{
	if (MovieScene)
	{
		IterateTracksInMovieScene(*MovieScene, [this](const UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
		{
			if (const UControlRig* ControlRig = Track->GetControlRig())
				{
					if (ControlRig->GetObjectBinding())
					{
						if (UControlRigComponent* ControlRigComponent = Cast<UControlRigComponent>(ControlRig->GetObjectBinding()->GetBoundObject()))
						{
							ControlRigComponent->Update(.1f); //delta time doesn't matter.
					}
				}
			}
			return false;
		});
	}
}

static void GetAllKeyAreas(TSharedPtr<UE::Sequencer::FViewModel> DataModel, TArray<TSharedPtr<IKeyArea>>& Channels)
{
	using namespace UE::Sequencer;

	if (DataModel)
	{
		constexpr bool bIncludeThis = true;
		for (const FViewModelPtr& Child : DataModel->GetDescendants(bIncludeThis))
		{
			if (TSharedPtr<ITrackAreaExtension> TrackArea = Child.ImplicitCast())
			{
				for (const FViewModelPtr& TrackAreaModel : TrackArea->GetTrackAreaModelList())
				{
					if (TViewModelPtr<FChannelModel> Channel = TrackAreaModel.ImplicitCast())
					{
						Channels.Add(Channel->GetKeyArea());
					}
				}
			}
			else if (TSharedPtr<FChannelModel> Channel = Child.ImplicitCast())
			{
				Channels.Add(Channel->GetKeyArea());
			}
		}
	}
}
//mz new
static void GetJustSelectedKeyArea(TSharedPtr<ISequencer>& Sequencer, TArray<const IKeyArea*>& OutSelectedKeyAreas)
{
	using namespace UE::Sequencer;
	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
	if (TSharedPtr<FSequencerSelection> Selection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
	{
		TArray<TSharedRef<FViewModel>> NodesToKey;
		for (FViewModelPtr Node : Selection->Outliner)
		{
			if (Node->IsA<FCategoryGroupModel>() || Node->IsA<FChannelGroupModel>() || Node->IsA<FChannelModel>())
			{
				TArray<TSharedPtr<IKeyArea>> Channels;
				GetAllKeyAreas(Node, Channels);
				//just care about the first key area
				if (Channels.Num() > 0)
				{
					const IKeyArea* KeyAreaPtr = Channels[0].Get();
					OutSelectedKeyAreas.Add(KeyAreaPtr);
				}
			}
		}

	}
}

void FControlRigParameterTrackEditor::OnSelectionChanged(TArray<UMovieSceneTrack*> InTracks)
{
	if (bIsDoingSelection)
	{
		return;
	}
	
	FControlRigEditMode* ControlRigEditMode = GetEditMode();
	
	// Some control rig systems set the selection in Curve Editor. Depending on user settings, Curve Editor updates Sequencer selection.
	// If a control rig system started the selection change, don't do any further processing.
	const bool bControlRigDoingSelection = ControlRigEditMode && UE::ControlRig::IsAnimDetailsChangingSelection(*ControlRigEditMode);
	if (bControlRigDoingSelection)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	const TGuardValue<bool> Guard(bIsDoingSelection, true);

	if(bSkipNextSelectionFromTimer)
	{
		bSkipNextSelectionFromTimer = false;
		return;
	}

	const bool bEditModeExisted = ControlRigEditMode != nullptr;
	UControlRig* ControlRig = nullptr;

	TArray<const IKeyArea*> KeyAreas;
	const bool bSelectChildChannels = CVarSelectChildChannels.GetValueOnGameThread();
	if (bSelectChildChannels)
	{
		const bool UseSelectedKeys = CVarSelectedKeysSelectControls.GetValueOnGameThread();
		Sequencer->GetSelectedKeyAreas(KeyAreas, UseSelectedKeys);
	}
	else
	{
		GetJustSelectedKeyArea(Sequencer, KeyAreas);
	}
	
	if (KeyAreas.Num() <= 0)
	{ 
		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown() == false && 
			FSlateApplication::Get().GetModifierKeys().IsControlDown() == false && ControlRigEditMode)
		{
			TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
			ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
			for (TPair<UControlRig*, TArray<FRigElementKey>>& SelectedControl : AllSelectedControls)
			{
				ControlRig = SelectedControl.Key;
				if (ControlRig && ControlRig->CurrentControlSelection().Num() > 0)
				{
					FScopedTransaction ScopedTransaction(LOCTEXT("SelectControlTransaction", "Select Control"), !GIsTransacting);
					constexpr bool bSetupUndo = true;
					ControlRig->ClearControlSelection(bSetupUndo);
				}
			}
		}

		for (UMovieSceneTrack* Track : InTracks)
		{
			UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
			if (CRTrack)
			{
				UControlRig* TrackControlRig = CRTrack->GetControlRig();
				if (TrackControlRig)
				{
					if (ControlRigEditMode)
					{
						const bool bAdded = ControlRigEditMode->AddControlRigObject(TrackControlRig, Sequencer);
						if (bAdded)
						{
							ControlRigEditMode->RequestToRecreateControlShapeActors(TrackControlRig);
						}
						break;
					}
					else
					{
						ControlRigEditMode = GetEditMode(true);
						if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = TrackControlRig->GetObjectBinding())
						{
							if (ControlRigEditMode)
							{
								const bool bAdded = ControlRigEditMode->AddControlRigObject(TrackControlRig, Sequencer);
								if (bAdded)
								{
									ControlRigEditMode->RequestToRecreateControlShapeActors(TrackControlRig);
								}
							}
						}
					}
				}
			}
		}
		
		const bool bSelectedSectionSetsSectionToKey = CVarSelectedSectionSetsSectionToKey.GetValueOnGameThread();
		if (bSelectedSectionSetsSectionToKey)
		{
			TMap<UMovieSceneTrack*, TSet<UMovieSceneControlRigParameterSection*>> TracksAndSections;
			using namespace UE::Sequencer;

			for (FViewModelPtr ViewModel : Sequencer->GetViewModel()->GetSelection()->Outliner)
			{
				if (TViewModelPtr<ISectionOwnerExtension> SectionOwnerExtension = ViewModel.ImplicitCast())
				{
					for (UMovieSceneSection* Section : SectionOwnerExtension->GetSections())
					{
						if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
						{
							if (UMovieSceneTrack* Track = Section->GetTypedOuter<UMovieSceneTrack>())
							{
								TracksAndSections.FindOrAdd(Track).Add(CRSection);
							}
						}
					}
				}
			}

			//if we have only one  selected section per track and the track has more than one section we set that to the section to key
			for (TPair<UMovieSceneTrack*, TSet<UMovieSceneControlRigParameterSection*>>& TrackPair : TracksAndSections)
			{
				if (TrackPair.Key->GetAllSections().Num() > 0 && TrackPair.Value.Num() == 1)
				{
					TrackPair.Key->SetSectionToKey(TrackPair.Value.Array()[0]);
				}
			}

		}
		return;
	}
	
	SelectRigsAndControls(ControlRig, KeyAreas);

	// If the edit mode has been activated, we need to synchronize the external selection (possibly again to account for control rig control actors selection)
	if (!bEditModeExisted && GetEditMode() != nullptr)
	{
		FSequencerUtilities::SynchronizeExternalSelectionWithSequencerSelection(Sequencer.ToSharedRef());
	}
	
}

void FControlRigParameterTrackEditor::SelectRigsAndControls(UControlRig* ControlRig, const TArray<const IKeyArea*>& KeyAreas)
{
	FControlRigEditMode* ControlRigEditMode = GetEditMode();
	
	//if selection set's section to key we need to keep track of selected sections for each track.
	const bool bSelectedSectionSetsSectionToKey = CVarSelectedSectionSetsSectionToKey.GetValueOnGameThread();
	TMap<UMovieSceneTrack*, TSet<UMovieSceneControlRigParameterSection*>> TracksAndSections;

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	
	TArray<FString> StringArray;
	//we have two sets here one to see if selection has really changed that contains the attirbutes, the other to select just the parent
	TMap<UControlRig*, TSet<FName>> RigsAndControls;
	for (const IKeyArea* KeyArea : KeyAreas)
	{
		UMovieSceneControlRigParameterSection* MovieSection = Cast<UMovieSceneControlRigParameterSection>(KeyArea->GetOwningSection());
		if (MovieSection)
		{
			ControlRig = MovieSection->GetControlRig();
			if (ControlRig)
			{
				//Only create the edit mode if we have a KeyAra selected and it's not set and we have some boundobjects.
				if (!ControlRigEditMode)
				{
					ControlRigEditMode = GetEditMode(true);
					if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
					{
						if (ControlRigEditMode)
						{
							ControlRigEditMode->AddControlRigObject(ControlRig, Sequencer);
						}
					}
				}
				else
				{
					if (ControlRigEditMode->AddControlRigObject(ControlRig, Sequencer))
					{
						//force an evaluation, this will get the control rig setup so edit mode looks good.
						if (Sequencer)
						{
							Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
						}
					}
				}

				const FMovieSceneChannelMetaData* MetaData = KeyArea->GetChannel().GetMetaData();
				if (MetaData)
				{
					StringArray.SetNum(0);
					FString String = MetaData->Name.ToString();
					String.ParseIntoArray(StringArray, TEXT("."));
					if (StringArray.Num() > 0)
					{
						const FName ControlName(*StringArray[0]);

						// skip nested controls which have the shape enabled flag turned on
						if(const URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
						{

							if(const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(FRigElementKey(ControlName, ERigElementType::Control)))
							{
						
								if (ControlElement->Settings.ControlType == ERigControlType::Bool ||
									ControlElement->Settings.ControlType == ERigControlType::Float ||
									ControlElement->Settings.ControlType == ERigControlType::ScaleFloat ||
									ControlElement->Settings.ControlType == ERigControlType::Integer)
								{
									if (ControlElement->Settings.SupportsShape() || !Hierarchy->IsAnimatable(ControlElement))
									{

										if (const FRigControlElement* ParentControlElement = Cast<FRigControlElement>(Hierarchy->GetFirstParent(ControlElement)))
										{
											if (const TSet<FName>* Controls = RigsAndControls.Find(ControlRig))
											{
												if (Controls->Contains(ParentControlElement->GetFName()))
												{
													continue;
												}
											}
										}
									}
								}
								RigsAndControls.FindOrAdd(ControlRig).Add(ControlName);
							}
						}
					}
				}
			
				if (bSelectedSectionSetsSectionToKey)
				{
					if (UMovieSceneTrack* Track = MovieSection->GetTypedOuter<UMovieSceneTrack>())
					{
						TracksAndSections.FindOrAdd(Track).Add(MovieSection);
					}
				}
			}
		}
	}

	//only create transaction if selection is really different.
	bool bEndTransaction = false;
	
	TMap<UControlRig*, TArray<FName>> ControlRigsToClearSelection;
	//get current selection which we will clear if different
	if (ControlRigEditMode)
	{
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		ControlRigEditMode->GetAllSelectedControls(SelectedControls);
		for (TPair<UControlRig*, TArray<FRigElementKey>>& Selection : SelectedControls)
		{
			ControlRig = Selection.Key;
			if (ControlRig)
			{
				TArray<FName> SelectedControlNames = ControlRig->CurrentControlSelection();
				ControlRigsToClearSelection.Add(ControlRig, SelectedControlNames);
			}
		}
	}

	for (TPair<UControlRig*, TSet<FName>>& Pair : RigsAndControls)
	{
		//check to see if new selection is same als old selection
		bool bIsSame = true;
		if (TArray<FName>* SelectedNames = ControlRigsToClearSelection.Find(Pair.Key))
		{
			TSet<FName>* FullNames = RigsAndControls.Find(Pair.Key);
			if (!FullNames)
			{
				continue; // should never happen
			}
			if (SelectedNames->Num() != FullNames->Num())
			{ 
				bIsSame = false;
				if (!GIsTransacting && bEndTransaction == false)
				{
					bEndTransaction = true;
					GEditor->BeginTransaction(LOCTEXT("SelectControl", "Select Control"));
				}
				Pair.Key->ClearControlSelection();
				ControlRigsToClearSelection.Remove(Pair.Key); //remove it
			}
			else//okay if same check and see if equal...
			{
				for (const FName& Name : (*SelectedNames))
				{
					if (FullNames->Contains(Name) == false)
					{
						bIsSame = false;
						if (!GIsTransacting && bEndTransaction == false)
						{
							bEndTransaction = true;
							GEditor->BeginTransaction(LOCTEXT("SelectControl", "Select Control"));
						}
						Pair.Key->ClearControlSelection();
						ControlRigsToClearSelection.Remove(Pair.Key); //remove it
						break; //break out
					}
				}
			}
			if (bIsSame == true)
			{
				ControlRigsToClearSelection.Remove(Pair.Key); //remove it
			}
		}
		else
		{
			bIsSame = false;
		}
		if (bIsSame == false)
		{
			for (const FName& Name : Pair.Value)
			{
				if (!GIsTransacting && bEndTransaction == false)
				{
					bEndTransaction = true;
					GEditor->BeginTransaction(LOCTEXT("SelectControl", "Select Control"));
				}
				Pair.Key->SelectControl(Name, true);
			}
		}
	}
	//go through and clear those still not cleared
	for (TPair<UControlRig*, TArray<FName>>& SelectedPairs : ControlRigsToClearSelection)
	{
		if (!GIsTransacting && bEndTransaction == false)
		{
			bEndTransaction = true;
			GEditor->BeginTransaction(LOCTEXT("SelectControl", "Select Control"));
		}
		SelectedPairs.Key->ClearControlSelection();
	}
	//if we have only one  selected section per track and the track has more than one section we set that to the section to key
	for (TPair<UMovieSceneTrack*, TSet<UMovieSceneControlRigParameterSection*>>& TrackPair : TracksAndSections)
	{
		if (TrackPair.Key->GetAllSections().Num() > 0 && TrackPair.Value.Num() == 1)
		{
			TrackPair.Key->SetSectionToKey(TrackPair.Value.Array()[0]);
		}
	}
	if (bEndTransaction)
	{
		GEditor->EndTransaction();
	}
}


FMovieSceneTrackEditor::FFindOrCreateHandleResult FControlRigParameterTrackEditor::FindOrCreateHandleToObject(UObject* InObj, UControlRig* InControlRig)
{
	FFindOrCreateHandleResult Result;
	Result.bWasCreated = false;
	
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return Result;
	}
	
	constexpr bool bCreateHandleIfMissing = false;
	FName CreatedFolderName = NAME_None;

	bool bHandleWasValid = Sequencer->GetHandleToObject(InObj, bCreateHandleIfMissing).IsValid();

	Result.Handle = Sequencer->GetHandleToObject(InObj, bCreateHandleIfMissing, CreatedFolderName);
	Result.bWasCreated = bHandleWasValid == false && Result.Handle.IsValid();

	const UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

	// Prioritize a control rig parameter track on this component if it matches the handle
	if (Result.Handle.IsValid())
	{
		if (UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(MovieScene->FindTrack(UMovieSceneControlRigParameterTrack::StaticClass(), Result.Handle, NAME_None)))
		{
			if (InControlRig == nullptr || (Track->GetControlRig() == InControlRig))
			{
				return Result;
			}
		}
	}

	// If the owner has a control rig parameter track, let's use it
	if (const USceneComponent* SceneComponent = Cast<USceneComponent>(InObj))
	{
		// If the owner has a control rig parameter track, let's use it
		UObject* OwnerObject = SceneComponent->GetOwner();
		const FGuid OwnerHandle = Sequencer->GetHandleToObject(OwnerObject, bCreateHandleIfMissing);
	    bHandleWasValid = OwnerHandle.IsValid();
	    if (OwnerHandle.IsValid())
	    {
		    if (UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(MovieScene->FindTrack(UMovieSceneControlRigParameterTrack::StaticClass(), OwnerHandle, NAME_None)))
		    {
			    if (InControlRig == nullptr || (Track->GetControlRig() == InControlRig))
			    {
				    Result.Handle = OwnerHandle;
				    Result.bWasCreated = bHandleWasValid == false && Result.Handle.IsValid();
				    return Result;
			    }
		    }
	    }
    
	    // If the component handle doesn't exist, let's use the owner handle
	    if (Result.Handle.IsValid() == false)
	    {
		    Result.Handle = OwnerHandle;
		    Result.bWasCreated = bHandleWasValid == false && Result.Handle.IsValid();
		}
	}
	return Result;
}

void FControlRigParameterTrackEditor::SelectSequencerNodeInSection(UMovieSceneControlRigParameterSection* ParamSection, const FName& ControlName, bool bSelected)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}
	
	if (ParamSection)
	{
		FChannelMapInfo* pChannelIndex = ParamSection->ControlChannelMap.Find(ControlName);
		if (pChannelIndex != nullptr)
		{
			if (pChannelIndex->ParentControlIndex == INDEX_NONE)
			{
				int32 CategoryIndex = ParamSection->GetActiveCategoryIndex(ControlName);
				if (CategoryIndex != INDEX_NONE)
				{
					Sequencer->SelectByNthCategoryNode(ParamSection, CategoryIndex, bSelected);
				}
			}
			else
			{
				const FName FloatChannelTypeName = FMovieSceneFloatChannel::StaticStruct()->GetFName();

				FMovieSceneChannelProxy& ChannelProxy = ParamSection->GetChannelProxy();
				for (const FMovieSceneChannelEntry& Entry : ChannelProxy.GetAllEntries())
				{
					const FName ChannelTypeName = Entry.GetChannelTypeName();
					if (pChannelIndex->ChannelTypeName == ChannelTypeName || (ChannelTypeName == FloatChannelTypeName && pChannelIndex->ChannelTypeName == NAME_None))
					{
						constexpr bool bDoNotSelectParentInstead = false;
						const FMovieSceneChannelHandle Channel = ChannelProxy.MakeHandle(ChannelTypeName, pChannelIndex->ChannelIndex);
						Sequencer->SelectByChannels(ParamSection, {Channel}, bDoNotSelectParentInstead, bSelected);
						break;
					}
				}
			}
		}
	}
}

FMovieSceneTrackEditor::FFindOrCreateTrackResult FControlRigParameterTrackEditor::FindOrCreateControlRigTrackForObject(FGuid ObjectHandle, UControlRig* ControlRig, FName PropertyName, bool bCreateTrackIfMissing)
{
	FFindOrCreateTrackResult Result;
	bool bTrackExisted = false;

	IterateTracks([&Result, &bTrackExisted, ControlRig](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
	{
		if (Track->GetControlRig() == ControlRig)
		{
			Result.Track = Track;
			bTrackExisted = true;
		}
		return false;
	});

	// Only create track if the object handle is valid
	if (!Result.Track && bCreateTrackIfMissing && ObjectHandle.IsValid())
	{
		if(TSharedPtr<ISequencer> Sequencer = GetSequencer())
		{
			UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
			Result.Track = AddTrack(MovieScene, ObjectHandle, UMovieSceneControlRigParameterTrack::StaticClass(), PropertyName);
		}
	}

	Result.bWasCreated = bTrackExisted == false && Result.Track != nullptr;

	return Result;
}

UMovieSceneControlRigParameterTrack* FControlRigParameterTrackEditor::FindTrack(const UControlRig* InControlRig) const
{
	if (!GetSequencer().IsValid())
	{
		return nullptr;
	}
	
	UMovieSceneControlRigParameterTrack* FoundTrack = nullptr;
	IterateTracks([InControlRig, &FoundTrack](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
	{
		if (Track->GetControlRig() == InControlRig)
		{
			FoundTrack = Track;
			return false;
		}
		return true;
	});

	return FoundTrack;
}

void FControlRigParameterTrackEditor::HandleOnSpaceAdded(UMovieSceneControlRigParameterSection* Section, const FName& ControlName, FMovieSceneControlRigSpaceChannel* SpaceChannel)
{
	if (SpaceChannel)
	{
		if (!SpaceChannel->OnKeyMovedEvent().IsBound())
		{
			SpaceChannel->OnKeyMovedEvent().AddLambda([this, Section](FMovieSceneChannel* Channel, const  TArray<FKeyMoveEventItem>& MovedItems)
				{
					FMovieSceneControlRigSpaceChannel* SpaceChannel = static_cast<FMovieSceneControlRigSpaceChannel*>(Channel);
					HandleSpaceKeyMoved(Section, SpaceChannel, MovedItems);
				});
		}
		if (!SpaceChannel->OnKeyDeletedEvent().IsBound())
		{
			SpaceChannel->OnKeyDeletedEvent().AddLambda([this, Section](FMovieSceneChannel* Channel, const  TArray<FKeyAddOrDeleteEventItem>& Items)
				{
					FMovieSceneControlRigSpaceChannel* SpaceChannel = static_cast<FMovieSceneControlRigSpaceChannel*>(Channel);
					HandleSpaceKeyDeleted(Section, SpaceChannel, Items);
				});
		}
	}
	//todoo do we need to remove this or not mz
}

bool FControlRigParameterTrackEditor::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const
{
	SectionsGettingUndone.SetNum(0);
	// Check if we care about the undo/redo
	bool bGettingUndone = false;
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjects)
	{

		UObject* Object = TransactionObjectPair.Key;
		while (Object != nullptr)
		{
			const UClass* ObjectClass = Object->GetClass();
			if (ObjectClass && ObjectClass->IsChildOf(UMovieSceneControlRigParameterSection::StaticClass()))
			{
				UMovieSceneControlRigParameterSection* Section = Cast< UMovieSceneControlRigParameterSection>(Object);
				if (Section)
				{
					SectionsGettingUndone.Add(Section);
				}
				bGettingUndone =  true;
				break;
			}
			Object = Object->GetOuter();
		}
	}
	
	return bGettingUndone;
}

void FControlRigParameterTrackEditor::PostUndo(bool bSuccess)
{
	for (UMovieSceneControlRigParameterSection* Section : SectionsGettingUndone)
	{
		if (Section->GetControlRig())
		{
			TArray<FSpaceControlNameAndChannel>& SpaceChannels = Section->GetSpaceChannels();
			for(FSpaceControlNameAndChannel& Channel: SpaceChannels)
			{ 
				HandleOnSpaceAdded(Section, Channel.ControlName, &(Channel.SpaceCurve));
			}

			TArray<FConstraintAndActiveChannel>& ConstraintChannels = Section->GetConstraintsChannels();
			for (FConstraintAndActiveChannel& Channel: ConstraintChannels)
			{ 
				HandleOnConstraintAdded(Section, &(Channel.ActiveChannel));
			}
		}
	}
}


void FControlRigParameterTrackEditor::HandleSpaceKeyDeleted(
	UMovieSceneControlRigParameterSection* Section,
	FMovieSceneControlRigSpaceChannel* Channel,
	const TArray<FKeyAddOrDeleteEventItem>& DeletedItems) const
{
	const TSharedPtr<ISequencer> ParentSequencer = GetSequencer();

	if (Section && Section->GetControlRig() && Channel && ParentSequencer.IsValid())
	{
		const FName ControlName = Section->FindControlNameFromSpaceChannel(Channel);
		for (const FKeyAddOrDeleteEventItem& EventItem : DeletedItems)
		{
			FControlRigSpaceChannelHelpers::SequencerSpaceChannelKeyDeleted(
				Section->GetControlRig(), ParentSequencer.Get(), ControlName, Channel, Section,EventItem.Frame);
		}
	}
}

void FControlRigParameterTrackEditor::HandleSpaceKeyMoved(
	UMovieSceneControlRigParameterSection* Section,
	FMovieSceneControlRigSpaceChannel* SpaceChannel,
	const  TArray<FKeyMoveEventItem>& MovedItems)
{
	if (Section && Section->GetControlRig() && SpaceChannel)
	{
		const FName ControlName = Section->FindControlNameFromSpaceChannel(SpaceChannel);
		for (const FKeyMoveEventItem& MoveEventItem : MovedItems)
		{
			FControlRigSpaceChannelHelpers::HandleSpaceKeyTimeChanged(
				Section->GetControlRig(), ControlName, SpaceChannel, Section,
				MoveEventItem.Frame, MoveEventItem.NewFrame);
		}
	}
}

void FControlRigParameterTrackEditor::ClearOutAllSpaceAndConstraintDelegates(const UControlRig* InOptionalControlRig) const
{
	UTickableTransformConstraint::GetOnConstraintChanged().RemoveAll(this);

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	const UMovieScene* MovieScene = Sequencer && Sequencer->GetFocusedMovieSceneSequence() ? Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	IterateTracks([InOptionalControlRig](const UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
	{
		if (InOptionalControlRig && Track->GetControlRig() != InOptionalControlRig)
		{
			return true;
		}
				
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
			{
				// clear space channels
				TArray<FSpaceControlNameAndChannel>& Channels = CRSection->GetSpaceChannels();
				for (FSpaceControlNameAndChannel& SpaceAndChannel : Channels)
				{
					SpaceAndChannel.SpaceCurve.OnKeyMovedEvent().Clear();
					SpaceAndChannel.SpaceCurve.OnKeyDeletedEvent().Clear();
				}

				// clear constraint channels
				TArray<FConstraintAndActiveChannel>& ConstraintChannels = CRSection->GetConstraintsChannels();
				for (FConstraintAndActiveChannel& Channel: ConstraintChannels)
				{
					Channel.ActiveChannel.OnKeyMovedEvent().Clear();
					Channel.ActiveChannel.OnKeyDeletedEvent().Clear();
				}

				if (CRSection->OnConstraintRemovedHandle.IsValid())
				{
					FConstraintsManagerController::GetNotifyDelegate().Remove(CRSection->OnConstraintRemovedHandle);
					CRSection->OnConstraintRemovedHandle.Reset();
				}
			}
		}

		return false;
	});
}

namespace
{
	struct FConstraintAndControlData
	{
		static FConstraintAndControlData CreateFromSection(
			const UMovieSceneControlRigParameterSection* InSection,
			const FMovieSceneConstraintChannel* InConstraintChannel)
		{
			FConstraintAndControlData Data;
			
			// get constraint channel
			const TArray<FConstraintAndActiveChannel>& ConstraintChannels = InSection->GetConstraintsChannels();
			const int32 Index = ConstraintChannels.IndexOfByPredicate([InConstraintChannel](const FConstraintAndActiveChannel& InChannel)
			{
				return &(InChannel.ActiveChannel) == InConstraintChannel;
			});
	
			if (Index == INDEX_NONE)
			{
				return Data;
			}

			Data.Constraint = Cast<UTickableTransformConstraint>(ConstraintChannels[Index].GetConstraint().Get());

			// get constraint name
			auto GetControlName = [InSection, Index]()
			{
				for (const TPair<FName, FChannelMapInfo>& It : InSection->ControlChannelMap)
				{
					const FChannelMapInfo& Info = It.Value;
					if (Info.ConstraintsIndex.Contains(Index))
					{
						return It.Key;
					}
				}

				static const FName DummyName = NAME_None;
				return DummyName;
			};
	
			Data.ControlName = GetControlName();
			
			return Data;
		}

		bool IsValid() const
		{
			return Constraint.IsValid() && ControlName != NAME_None; 
		}
		
		TWeakObjectPtr<UTickableTransformConstraint> Constraint = nullptr;
		FName ControlName = NAME_None;
	};
}

void FControlRigParameterTrackEditor::HandleOnConstraintAdded(
	IMovieSceneConstrainedSection* InSection,
	FMovieSceneConstraintChannel* InConstraintChannel)
{
	if (!InConstraintChannel)
	{
		return;
	}

	// handle key moved
	if (!InConstraintChannel->OnKeyMovedEvent().IsBound())
	{
		InConstraintChannel->OnKeyMovedEvent().AddLambda([this, InSection](
			FMovieSceneChannel* InChannel, const TArray<FKeyMoveEventItem>& InMovedItems)
			{
				const FMovieSceneConstraintChannel* ConstraintChannel = static_cast<FMovieSceneConstraintChannel*>(InChannel);
				HandleConstraintKeyMoved(InSection, ConstraintChannel, InMovedItems);
			});
	}

	// handle key deleted
	if (!InConstraintChannel->OnKeyDeletedEvent().IsBound())
	{
		InConstraintChannel->OnKeyDeletedEvent().AddLambda([this, InSection](
			FMovieSceneChannel* InChannel, const TArray<FKeyAddOrDeleteEventItem>& InDeletedItems)
			{
				const FMovieSceneConstraintChannel* ConstraintChannel = static_cast<FMovieSceneConstraintChannel*>(InChannel);
				HandleConstraintKeyDeleted(InSection, ConstraintChannel, InDeletedItems);
			});
	}

	// handle constraint deleted
	if (InSection)
	{
		HandleConstraintRemoved(InSection);
	}

	if (!UTickableTransformConstraint::GetOnConstraintChanged().IsBoundToObject(this))
	{
		UTickableTransformConstraint::GetOnConstraintChanged().AddRaw(this, &FControlRigParameterTrackEditor::HandleConstraintPropertyChanged);
	}
}

void FControlRigParameterTrackEditor::HandleConstraintKeyDeleted(
	IMovieSceneConstrainedSection* InSection,
	const FMovieSceneConstraintChannel* InConstraintChannel,
	const TArray<FKeyAddOrDeleteEventItem>& InDeletedItems) const
{
	if (FMovieSceneConstraintChannelHelper::bDoNotCompensate)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}
	
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(InSection);

	const UControlRig* ControlRig = Section ? Section->GetControlRig() : nullptr;
	if (!ControlRig || !InConstraintChannel)
	{
		return;
	}
	
	const FConstraintAndControlData ConstraintAndControlData = FConstraintAndControlData::CreateFromSection(Section, InConstraintChannel);
	if (ConstraintAndControlData.IsValid())
	{
		UTickableTransformConstraint* Constraint = ConstraintAndControlData.Constraint.Get();
		for (const FKeyAddOrDeleteEventItem& EventItem: InDeletedItems)
		{
			FMovieSceneConstraintChannelHelper::HandleConstraintKeyDeleted(Constraint, InConstraintChannel, Sequencer, Section, EventItem.Frame);
		}
	}
}

void FControlRigParameterTrackEditor::HandleConstraintKeyMoved(
	IMovieSceneConstrainedSection* InSection,
	const FMovieSceneConstraintChannel* InConstraintChannel,
	const TArray<FKeyMoveEventItem>& InMovedItems)
{
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(InSection);

	const FConstraintAndControlData ConstraintAndControlData =
	FConstraintAndControlData::CreateFromSection(Section, InConstraintChannel);

	if (ConstraintAndControlData.IsValid())
	{
		const UTickableTransformConstraint* Constraint = ConstraintAndControlData.Constraint.Get();
		for (const FKeyMoveEventItem& MoveEventItem : InMovedItems)
		{
			FMovieSceneConstraintChannelHelper::HandleConstraintKeyMoved(
				Constraint, InConstraintChannel, Section,
				MoveEventItem.Frame, MoveEventItem.NewFrame);
		}
	}
}

void FControlRigParameterTrackEditor::HandleConstraintRemoved(IMovieSceneConstrainedSection* InSection) 
{
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(InSection);
	if (!Section)
	{
		return;
	}

	if (const UControlRig* ControlRig = Section->GetControlRig())
	{
		if (!Section->OnConstraintRemovedHandle.IsValid())
		{
			Section->OnConstraintRemovedHandle =
			FConstraintsManagerController::GetNotifyDelegate().AddLambda(
				[WeakSection = MakeWeakObjectPtr(Section), this](EConstraintsManagerNotifyType InNotifyType, UObject *InObject)
			{
				switch (InNotifyType)
				{
					case EConstraintsManagerNotifyType::ConstraintAdded:
						break;
					case EConstraintsManagerNotifyType::ConstraintRemoved:
					case EConstraintsManagerNotifyType::ConstraintRemovedWithCompensation:
						{
							const UTickableConstraint* Constraint = Cast<UTickableConstraint>(InObject);
							UMovieSceneControlRigParameterSection* Section = WeakSection.IsValid() ? WeakSection.Get() : nullptr;
							if (!IsValid(Constraint) || !Section)
							{
								return;
							}

							const FConstraintAndActiveChannel* ConstraintChannel = Section->GetConstraintChannel(Constraint->ConstraintID);
							if (!ConstraintChannel || ConstraintChannel->GetConstraint().Get() != Constraint)
							{
								return;
							}

							const TSharedPtr<ISequencer> Sequencer = GetSequencer();
							if (Sequencer)
							{
								const bool bCompensate = (InNotifyType == EConstraintsManagerNotifyType::ConstraintRemovedWithCompensation);
							   if (bCompensate && ConstraintChannel->GetConstraint().Get())
							   {
								   FMovieSceneConstraintChannelHelper::HandleConstraintRemoved(
									   ConstraintChannel->GetConstraint().Get(),
									   &ConstraintChannel->ActiveChannel,
									   Sequencer,
									   Section);
							   }
							}

							Section->RemoveConstraintChannel(Constraint);

							if (Sequencer)
							{
								Sequencer->RecreateCurveEditor();
							}
						}
						break;
					case EConstraintsManagerNotifyType::ManagerUpdated:
						if(UMovieSceneControlRigParameterSection* Section = WeakSection.IsValid() ? WeakSection.Get() : nullptr)
						{
							Section->OnConstraintsChanged();
						}
						break;
					case EConstraintsManagerNotifyType::GraphUpdated:
						break;
				}
			});

			ConstraintHandlesToClear.Add(Section->OnConstraintRemovedHandle);
		}
	}
}

void FControlRigParameterTrackEditor::HandleConstraintPropertyChanged(UTickableTransformConstraint* InConstraint, const FPropertyChangedEvent& InPropertyChangedEvent) const
{
	if (!IsValid(InConstraint))
	{
		return;
	}

	// find constraint section
	const UTransformableControlHandle* Handle = Cast<UTransformableControlHandle>(InConstraint->ChildTRSHandle);
	if (!IsValid(Handle) || !Handle->IsValid())
	{
		return;
	}

	const FConstraintChannelInterfaceRegistry& InterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();	
	ITransformConstraintChannelInterface* Interface = InterfaceRegistry.FindConstraintChannelInterface(Handle->GetClass());
	if (!Interface)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	UMovieSceneSection* Section = Sequencer ? Interface->GetHandleConstraintSection(Handle, Sequencer) : nullptr;
	IMovieSceneConstrainedSection* ConstraintSection = Cast<IMovieSceneConstrainedSection>(Section);
	if (!ConstraintSection)
	{
		return;
	}

	// find corresponding channel
	const TArray<FConstraintAndActiveChannel>& ConstraintChannels = ConstraintSection->GetConstraintsChannels();
	const FConstraintAndActiveChannel* Channel = ConstraintChannels.FindByPredicate([InConstraint](const FConstraintAndActiveChannel& Channel)
	{
		return Channel.GetConstraint() == InConstraint;
	});

	if (!Channel)
	{
		return;
	}

	FMovieSceneConstraintChannelHelper::HandleConstraintPropertyChanged(
			InConstraint, Channel->ActiveChannel, InPropertyChangedEvent, Sequencer, Section);
}

void FControlRigParameterTrackEditor::SetUpEditModeIfNeeded(UControlRig* ControlRig)
{
	if (ControlRig)
	{
		//this could clear the selection so if it does reset it
		const TArray<FName> ControlRigSelection = ControlRig->CurrentControlSelection();

		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		FControlRigEditMode* ControlRigEditMode = GetEditMode();
		
		if (!ControlRigEditMode)
		{
			ControlRigEditMode = GetEditMode(true);
			if (ControlRigEditMode)
			{
				if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
				{
					ControlRigEditMode->AddControlRigObject(ControlRig, Sequencer);
				}
			}
		}
		else
		{
			if (ControlRigEditMode->AddControlRigObject(ControlRig, Sequencer))
			{
				//force an evaluation, this will get the control rig setup so edit mode looks good.
				if (Sequencer)
				{
					Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
				}
			}
		}
		TArray<FName> NewControlRigSelection = ControlRig->CurrentControlSelection();
		if (ControlRigSelection.Num() != NewControlRigSelection.Num())
		{
			ControlRig->ClearControlSelection();
			for (const FName& Name : ControlRigSelection)
			{
				ControlRig->SelectControl(Name, true);
			}
		}
	}
}

bool FControlRigParameterTrackEditor::ShouldBeSelected(URigHierarchy* Hierarchy, FRigControlElement* ControlElement, FRigControlElement* OtherControlElement)
{
	const FRigControlSettings& OtherSettings = OtherControlElement->Settings;

	const ERigControlType OtherControlType = OtherSettings.ControlType;

	if (OtherControlType == ERigControlType::Bool ||
		OtherControlType == ERigControlType::Float ||
		OtherControlType == ERigControlType::ScaleFloat ||
		OtherControlType == ERigControlType::Integer)
	{
		if (OtherControlElement->Settings.SupportsShape() || !Hierarchy->IsAnimatable(OtherControlElement))
		{
			return false;
		}

		for (const FRigElementParentConstraint& ParentConstraint : OtherControlElement->ParentConstraints)
		{
			if (ParentConstraint.ParentElement == ControlElement)
			{
				return true;
			}
		}
	}

	if (OtherControlElement->IsAnimationChannel() && OtherSettings.Customization.AvailableSpaces.FindByKey(ControlElement->GetKey()))
	{
		return true;
	}
	return false;
}

bool FControlRigParameterTrackEditor::BakeTimeWarp(TSharedPtr<ISequencer>& Sequencer, UMovieSceneControlRigParameterSection* Section)
{
	using namespace UE::AIE;
	if (Sequencer && Section  && Section->GetControlRig() && Section->HasTimeWarp())
	{
		TRange<FFrameNumber> Range = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
		TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer->GetSubSequenceRange();
		if (OptionalRange.IsSet())
		{
			Range = TRange<FFrameNumber>(OptionalRange.GetValue().GetLowerBoundValue(), OptionalRange.GetValue().GetUpperBoundValue());
		}

		FFrameNumber FramesPerTick = FFrameRate::TransformTime(FFrameNumber(1), Sequencer->GetFocusedDisplayRate(),
			Sequencer->GetFocusedTickResolution()).RoundToFrame();

		FFrameTimeByIndex CurrentFrameTimes;
		CurrentFrameTimes = FFrameTimeByIndex(Range.GetLowerBoundValue(), Range.GetUpperBoundValue(), FramesPerTick);
		TArray<FFrameNumber>  Frames = CurrentFrameTimes.GetFullFrameArray();

		int32 NumOfTransforms = CurrentFrameTimes.NumFrames;
		UControlRig* ControlRig = Section->GetControlRig();
		TArray<FRigControlElement*> Controls;
		ControlRig->GetControlsInOrder(Controls);
		TArray<FName> ControlNames;
		ControlNames.Reserve(Controls.Num());
		for (FRigControlElement* ControlElement : Controls)
		{
			if (ControlElement == nullptr || !ControlRig->GetHierarchy()->IsAnimatable(ControlElement))
			{
				continue;
			}
			ControlNames.Add(ControlElement->GetFName());
		}

		if (ControlNames.Num() <= 0)
		{
			return false;
		}

		Section->Modify();


		//get world transforms
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::TickResolution;
		EControlRigTransformSpace Space = EControlRigTransformSpace::Global;
		ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence());
		TArray<FArrayOfRigControlTransforms> RigControlTransforms = UControlRigSequencerEditorLibrary::BatchGetControlTransforms(LevelSequence, ControlRig, ControlNames, Frames,
			Space, TimeUnit);

		//delete all keys, cept space and constraints
		Section->RemoveAllKeys(false);

		//delete timewarp
		Section->DeleteTimeWarp();

		//set world transforms back
		UControlRigSequencerEditorLibrary::BatchSetControlTransforms(LevelSequence, ControlRig, RigControlTransforms, Frames,
			Space, Section, TimeUnit);

	}
	return true;
}

void FControlRigParameterTrackEditor::HandleControlSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected)
{
	if (!IsInGameThread())
	{
		if (Subject && ControlElement)
		{
			const TWeakObjectPtr<UControlRig> WeakSubject = Subject;
			const FRigElementKey ControlKey = ControlElement->GetKey();

			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[WeakSubject, ControlKey, bSelected, WeakThis = SharedThis(this).ToWeakPtr()]()
				{
					const TSharedPtr<FControlRigParameterTrackEditor> StrongThis = WeakThis.Pin();
					UControlRig* StrongSubject = WeakSubject.Get();
					URigHierarchy* Hierarchy = StrongSubject ? StrongSubject->GetHierarchy() : nullptr;
					FRigControlElement* StrongControlElement = Hierarchy ? Hierarchy->Find<FRigControlElement>(ControlKey) : nullptr;

					if (StrongThis.IsValid() &&
						StrongSubject &&
						Hierarchy &&
						StrongControlElement)
					{
						StrongThis->HandleControlSelected(StrongSubject, StrongControlElement, bSelected);
					}
				});
		}

		return;
	}

	using namespace UE::Sequencer;

	if(ControlElement == nullptr)
	{
		return;
	}
	
	URigHierarchy* Hierarchy = Subject->GetHierarchy();
	static bool bIsSelectingIndirectControl = false;
	static TArray<FRigControlElement*> SelectedElements = {};

	// Avoid cyclic selection
	if (SelectedElements.Contains(ControlElement))
	{
		return;
	}

	TUniquePtr<FSelectionEventSuppressor> SequencerSelectionGuard;
	if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
		if (TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
		{
			SequencerSelectionGuard.Reset(new FSelectionEventSuppressor(SequencerSelection.Get()));
		}
	}
	
	if(ControlElement->CanDriveControls())
	{
		const TArray<FRigElementKey>& DrivenControls = ControlElement->Settings.DrivenControls;
		for(const FRigElementKey& DrivenKey : DrivenControls)
		{
			if(FRigControlElement* DrivenControl = Hierarchy->Find<FRigControlElement>(DrivenKey))
			{
				TGuardValue<bool> SubControlGuard(bIsSelectingIndirectControl, true);

				TArray<FRigControlElement*> NewSelection = SelectedElements;
				NewSelection.Add(ControlElement);
				TGuardValue<TArray<FRigControlElement*>> SelectedElementsGuard(SelectedElements, NewSelection);
				
				HandleControlSelected(Subject, DrivenControl, bSelected);
			}
		}
		if(ControlElement->Settings.AnimationType == ERigControlAnimationType::ProxyControl)
		{
			return;
		}
	}
	
	//if parent selected we select child here if it's a bool,integer or single float
	TArray<FRigControl> Controls;

	if (!bIsSelectingIndirectControl)
	{
		if (URigHierarchyController* Controller = Hierarchy->GetController())
		{			
			const bool bSelectChildChannels = CVarSelectChildChannels.GetValueOnGameThread();
			if (bSelectChildChannels)
			{
				Hierarchy->ForEach<FRigControlElement>([Hierarchy, ControlElement, Controller, bSelected](FRigControlElement* OtherControlElement) -> bool
				{
					if (FControlRigParameterTrackEditor::ShouldBeSelected(Hierarchy, ControlElement, OtherControlElement))
					{
						Controller->SelectElement(OtherControlElement->GetKey(), bSelected);
					}					
					return true;
				});
		}
		}
	}
	
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}
	if (bSelected) //if selected
	{
		if (FGuid* BindingID = BoundControlRigs.Find(Subject))
		{
			if (USequencerBakeSubsystem* SequencerBakeSubsystem = GEditor->GetEditorSubsystem<USequencerBakeSubsystem>())
			{
				if (TSharedPtr<UE::Sequencer::FSequencerBaker> SequencerBaker = SequencerBakeSubsystem->GetSequencerBaker(Sequencer))
				{
					constexpr bool bIsolate = false;
					SequencerBaker->IsolateRecorderWithGuid(*BindingID, bIsolate);
				}
			}
		}
	}

	if (bIsDoingSelection)
	{
		return;
	}
	
	TGuardValue<bool> Guard(bIsDoingSelection, true);
	

	const FName ControlRigName(*Subject->GetName());
	if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = Subject->GetObjectBinding())
	{
		UObject* Object = ObjectBinding->GetBoundObject();
		if (!Object)
		{
			return;
		}

		const bool bCreateTrack = false;
		const FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object, Subject);
		FFindOrCreateTrackResult TrackResult = FindOrCreateControlRigTrackForObject(HandleResult.Handle, Subject, ControlRigName, bCreateTrack);
		UMovieSceneControlRigParameterTrack* Track = CastChecked<UMovieSceneControlRigParameterTrack>(TrackResult.Track, ECastCheckedType::NullAllowed);

		if (Track)
		{
			//Just select in section to key, if deselecting makes sure deselected everywhere
			if (bSelected == true)
			{
				UMovieSceneSection* Section = Track->GetSectionToKey(ControlElement->GetFName());
				UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(Section);
				SelectSequencerNodeInSection(ParamSection, ControlElement->GetFName(), bSelected);
			}
			else
			{
				for (UMovieSceneSection* BaseSection : Track->GetAllSections())
				{
					if (UMovieSceneControlRigParameterSection* ParamSection = Cast< UMovieSceneControlRigParameterSection>(BaseSection))
					{
						SelectSequencerNodeInSection(ParamSection, ControlElement->GetFName(), bSelected);
					}
				}
			}

			SetUpEditModeIfNeeded(Subject);
			

			//Force refresh later, not now
			bSkipNextSelectionFromTimer = bSkipNextSelectionFromTimer ||
				(bIsSelectingIndirectControl && ControlElement->Settings.AnimationType == ERigControlAnimationType::AnimationControl);
			
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshTree);
		}
	}

	// force selection guard reset to ensure sequencer selection change is broadcast while bIsDoingSelection is still true
	SequencerSelectionGuard.Reset();
}

void FControlRigParameterTrackEditor::HandleOnPostConstructed(UControlRig* Subject, const FName& InEventName)
{
	if (IsInGameThread())
	{
		UControlRig* ControlRig = CastChecked<UControlRig>(Subject);

		if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
		{
			//refresh tree for ANY control rig may be FK or procedural
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshTree);
		}
	}
}

void FControlRigParameterTrackEditor::HandleControlModified(UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context)
{
	if (Context.SetKey == EControlRigSetKey::Never)
	{
		return;
	}
	
	if (!ControlElement ||
		ControlElement->Settings.AnimationType == ERigControlAnimationType::ProxyControl ||
		ControlElement->Settings.AnimationType == ERigControlAnimationType::VisualCue ) 
	{
		return;
	}
	
	if (IsInGameThread() == false)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || !Sequencer->IsAllowedToChange())
	{
		return;
	}

	FTransform  Transform = ControlRig->GetControlLocalTransform(ControlElement->GetFName());

	IterateTracks([this, ControlRig, ControlElement, Context](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
	{
		if (Track && Track->GetControlRig() == ControlRig)
		{
			FName Name(*ControlRig->GetName());
			
			if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
			{
				if (UObject* Object = ObjectBinding->GetBoundObject())
				{
					ESequencerKeyMode KeyMode = ESequencerKeyMode::AutoKey;
					if (Context.SetKey == EControlRigSetKey::Always)
					{
						KeyMode = ESequencerKeyMode::ManualKeyForced;
					}

					AddControlKeys(Object, ControlRig, Name, ControlElement->GetFName(), (EControlRigContextChannelToKey)Context.KeyMask, 
						KeyMode, Context.LocalTime);
					ControlChangedDuringUndoBracket++;
					
					return true;
				}
			}
		}
		return false;
	});
}

void FControlRigParameterTrackEditor::HandleControlUndoBracket(UControlRig* Subject, bool bOpenUndoBracket)
{
	if(IsInGameThread() && bOpenUndoBracket && ControlUndoBracket == 0)
	{
		FScopeLock ScopeLock(&ControlUndoTransactionMutex);
		ControlUndoTransaction = MakeShareable(new FScopedTransaction(LOCTEXT("KeyMultipleControls", "Auto-Key multiple controls"), !GIsTransacting));
		ControlChangedDuringUndoBracket = 0;
	}

	ControlUndoBracket = FMath::Max<int32>(0, ControlUndoBracket + (bOpenUndoBracket ? 1 : -1));
	
	if(!bOpenUndoBracket && ControlUndoBracket == 0)
	{
		FScopeLock ScopeLock(&ControlUndoTransactionMutex);

		/*
		// canceling a sub transaction cancels everything to the top. we need to find a better mechanism for this.
		if(ControlChangedDuringUndoBracket == 0 && ControlUndoTransaction.IsValid())
		{
			ControlUndoTransaction->Cancel();
		}
		*/
		ControlUndoTransaction.Reset();
	}
}

void FControlRigParameterTrackEditor::HandleOnControlRigBound(UControlRig* InControlRig)
{
	if (!InControlRig)
	{
		return;
	}
	
	UMovieSceneControlRigParameterTrack* Track = FindTrack(InControlRig);
	if (!Track)
	{
		return;
	}

	const TSharedPtr<IControlRigObjectBinding> Binding = InControlRig->GetObjectBinding();
	
	for (UMovieSceneSection* BaseSection : Track->GetAllSections())
	{
		if (UMovieSceneControlRigParameterSection* Section = Cast< UMovieSceneControlRigParameterSection>(BaseSection))
		{
			const UControlRig* ControlRig = Section->GetControlRig();
			if (ControlRig && InControlRig == ControlRig)
			{
				if (!Binding->OnControlRigBind().IsBoundToObject(this))
				{
					Binding->OnControlRigBind().AddRaw(this, &FControlRigParameterTrackEditor::HandleOnObjectBoundToControlRig);
				}
			}
		}
	}
}

void FControlRigParameterTrackEditor::HandleOnObjectBoundToControlRig(UObject* InObject)
{
	using namespace UE::ControlRigEditor::Private;
	
	//reselect these control rigs since selection may get lost
	TMap<TWeakObjectPtr<UControlRig>, TArray<FName>> ReselectIfNeeded;
	// look for sections to update
	TArray<UMovieSceneControlRigParameterSection*> SectionsToUpdate;
	for (TPair<TWeakObjectPtr<UControlRig>, FGuid>& ControlRigPair : BoundControlRigs)
	{
		TStrongObjectPtr<UControlRig> ControlRig = ControlRigPair.Key.Pin();
		if (ControlRig.IsValid() == false)
		{
			continue;
		}
		TArray<FName> Selection = ControlRig->CurrentControlSelection();
		if (Selection.Num() > 0)
		{
			ReselectIfNeeded.Add(ControlRigPair.Key, Selection);
		}
		const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
		const UObject* CurrentObject = Binding ? Binding->GetBoundObject() : nullptr;
		if (CurrentObject == InObject)
		{
			if (const UMovieSceneControlRigParameterTrack* Track = FindTrack(ControlRig.Get()))
			{
				for (UMovieSceneSection* BaseSection : Track->GetAllSections())
				{
					if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(BaseSection))
					{
						SectionsToUpdate.AddUnique(Section);
					}
				}
			}
		}
	}

	if (ReselectIfNeeded.Num() > 0)
	{
		DeferToNextTick([ReselectIfNeeded]()
		{
			DeferToNextTick([ReselectIfNeeded]()
			{
				for (const TPair <TWeakObjectPtr<UControlRig>, TArray<FName>>& Pair : ReselectIfNeeded)
				{
					if (UControlRig* Rig = Pair.Key.Get())
					{
						Rig->ClearControlSelection();
						for (const FName& ControlName : Pair.Value)
						{
							Rig->SelectControl(ControlName, true);
						}
					}
				}
			});
		});
	}
}


void FControlRigParameterTrackEditor::GetControlRigKeys(
	UControlRig* InControlRig,
	FName ParameterName,
	EControlRigContextChannelToKey ChannelsToKey,
	ESequencerKeyMode KeyMode,
	UMovieSceneControlRigParameterSection* SectionToKey,
	FGeneratedTrackKeys& OutGeneratedKeys,
	const bool bInConstraintSpace)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;	
	}

	const EKeyGroupMode KeyGroupMode = Sequencer->GetKeyGroupMode();
	const EMovieSceneTransformChannel TransformMask = SectionToKey->GetTransformMask().GetChannels();

	TArray<FRigControlElement*> Controls;
	InControlRig->GetControlsInOrder(Controls);

	// If key all is enabled, for a key on all the channels
	if (KeyMode != ESequencerKeyMode::ManualKeyForced && KeyGroupMode == EKeyGroupMode::KeyAll)
	{
		ChannelsToKey = EControlRigContextChannelToKey::AllTransform;
	}
	URigHierarchy* Hierarchy = InControlRig->GetHierarchy();

	//Need separate index for bools, ints and enums and floats since there are separate entries for each later when they are accessed by the set key stuff.
	int32 SpaceChannelIndex = 0;
	for (int32 LocalControlIndex = 0; LocalControlIndex < Controls.Num(); ++LocalControlIndex)
	{
		FRigControlElement* ControlElement = Controls[LocalControlIndex];
		check(ControlElement);

		if (!Hierarchy->IsAnimatable(ControlElement))
		{
			continue;
		}

		if (FChannelMapInfo* pChannelIndex = SectionToKey->ControlChannelMap.Find(ControlElement->GetFName()))
		{
			int32 ChannelIndex = pChannelIndex->ChannelIndex;
			const int32 MaskIndex = pChannelIndex->MaskIndex;

			bool bMaskKeyOut = (SectionToKey->GetControlNameMask(ControlElement->GetFName()) == false);
			bool bSetKey = ParameterName.IsNone() || (ControlElement->GetFName() == ParameterName && !bMaskKeyOut);
			if (bSetKey == false)
			{
				continue;
			}
			FRigControlValue ControlValue = InControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);

			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Bool:
			{
				bool Val = ControlValue.Get<bool>();
				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneBoolChannel>(ChannelIndex, Val, bSetKey));
				break;
			}
			case ERigControlType::Integer:
			{
				if (ControlElement->Settings.ControlEnum)
				{
					uint8 Val = (uint8)ControlValue.Get<uint8>();
					pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
					OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneByteChannel>(ChannelIndex, Val, bSetKey));
				}
				else
				{
					int32 Val = ControlValue.Get<int32>();
					pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
					OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneIntegerChannel>(ChannelIndex, Val, bSetKey));
				}
				break;
			}
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				float Val = ControlValue.Get<float>();
				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex, Val, bSetKey));
				break;
			}
			case ERigControlType::Vector2D:
			{
				//use translation x,y for key masks for vector2d
				bool bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX);;
				bool bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY);;
				FVector3f Val = ControlValue.Get<FVector3f>();
				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.X, bKeyX));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.Y, bKeyY));
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Scale:
			case ERigControlType::Rotator:
			{
				bool bKeyX = bSetKey;
				bool bKeyY = bSetKey;
				bool bKeyZ = bSetKey;
				if (ControlElement->Settings.ControlType == ERigControlType::Position)
				{
					bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX);
					bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY);
					bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ);
				}
				else if(ControlElement->Settings.ControlType == ERigControlType::Rotator)
				{
					bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX);
					bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY);
					bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ);
				}
				else //scale
				{
					bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX);
					bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY);
					bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ);
				}
				
				FVector3f Val = ControlValue.Get<FVector3f>();
				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.X, bKeyX));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.Y, bKeyY));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.Z, bKeyZ));
				break;
			}

			case ERigControlType::Transform:
			case ERigControlType::TransformNoScale:
			case ERigControlType::EulerTransform:
			{
				FVector Translation, Scale(1.0f, 1.0f, 1.0f);
				FVector Vector = InControlRig->GetControlSpecifiedEulerAngle(ControlElement);
				FRotator Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
				if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
				{
					FTransformNoScale NoScale = ControlValue.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
					Translation = NoScale.Location;
				}
				else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					FEulerTransform Euler = ControlValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
					Translation = Euler.Location;
					Scale = Euler.Scale;
				}
				else
				{
					FTransform Val = ControlValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
					Translation = Val.GetTranslation();
					Scale = Val.GetScale3D();
				}

				if (bInConstraintSpace)
				{
					const uint32 ControlHash = UTransformableControlHandle::ComputeHash(InControlRig, ControlElement->GetFName());
					TOptional<FTransform> Transform = UE::TransformConstraintUtil::GetRelativeTransform(InControlRig->GetWorld(), ControlHash);
					if (Transform)
					{
						Translation = Transform->GetTranslation();
						if (InControlRig->GetHierarchy()->GetUsePreferredRotationOrder(ControlElement))
						{
							Rotation = ControlElement->PreferredEulerAngles.GetRotatorFromQuat(Transform->GetRotation());
							FVector Angle = Rotation.Euler();
							//need to wind rotators still
							ControlElement->PreferredEulerAngles.SetAngles(Angle, false, ControlElement->PreferredEulerAngles.RotationOrder, true);
							Angle = InControlRig->GetControlSpecifiedEulerAngle(ControlElement);
							Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
						}
						else
						{
							Rotation = Transform->GetRotation().Rotator();
						}
						Scale = Transform->GetScale3D();
					}
				}
				
					
				FVector3f CurrentVector = (FVector3f)Translation;
				bool bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX);
				bool bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY);
				bool bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ);
				if (KeyMode != ESequencerKeyMode::ManualKeyForced && (KeyGroupMode == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ)))
				{
					bKeyX = bKeyY = bKeyZ = true;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationX))
				{
					bKeyX = false;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationY))
				{
					bKeyY = false;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationZ))
				{
					bKeyZ = false;
				}

				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();

				if (pChannelIndex->bDoesHaveSpace)
				{
					//for some saved dev files this could be -1 so we used the local incremented value which is almost always safe, if not a resave will fix the file.
					FMovieSceneControlRigSpaceBaseKey NewKey;
					int32 RealSpaceChannelIndex = pChannelIndex->SpaceChannelIndex != -1 ? pChannelIndex->SpaceChannelIndex : SpaceChannelIndex;
					++SpaceChannelIndex;
					OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneControlRigSpaceChannel>(RealSpaceChannelIndex, NewKey, false));
				}


				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.X, bKeyX));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Y, bKeyY));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Z, bKeyZ));

				FRotator3f CurrentRotator = FRotator3f(Rotation);
				bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX);
				bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY);
				bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ);
				if (KeyMode != ESequencerKeyMode::ManualKeyForced && (KeyGroupMode == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ)))
				{
					bKeyX = bKeyY = bKeyZ = true;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationX))
				{
					bKeyX = false;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationY))
				{
					bKeyY = false;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationZ))
				{
					bKeyZ = false;
				}

				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentRotator.Roll, bKeyX));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentRotator.Pitch, bKeyY));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentRotator.Yaw, bKeyZ));

				if (ControlElement->Settings.ControlType == ERigControlType::Transform || ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					CurrentVector = (FVector3f)Scale;
					bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX);
					bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY);
					bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ);
					if (KeyMode != ESequencerKeyMode::ManualKeyForced && (KeyGroupMode == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ)))
					{
						bKeyX = bKeyY = bKeyZ = true;
					}
					if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleX))
					{
						bKeyX = false;
					}
					if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleY))
					{
						bKeyY = false;
					}
					if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleZ))
					{
						bKeyZ = false;
					}
					OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.X, bKeyX));
					OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Y, bKeyY));
					OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Z, bKeyZ));
				}
				break;
			}
		}
		}
	}
}

void FControlRigParameterTrackEditor::GetControlRigKeys(
	UControlRig* InControlRig,
	FControlRigSectionParamMap& InOutSectionParamMap,
	EControlRigContextChannelToKey ChannelsToKey,
	ESequencerKeyMode KeyMode,
	const bool bInConstraintSpace)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || InOutSectionParamMap.IsEmpty())
	{
		return;
	}

	const EKeyGroupMode KeyGroupMode = Sequencer->GetKeyGroupMode();

	if (KeyMode != ESequencerKeyMode::ManualKeyForced && KeyGroupMode == EKeyGroupMode::KeyAll)
	{
		ChannelsToKey = EControlRigContextChannelToKey::AllTransform;
	}

	// Create one GeneratedKeys per section and build a flat name → section map for O(1) lookup.
	TMap<FName, UMovieSceneControlRigParameterSection*> ControlToSection;
	for (auto& [Section, SectionData] : InOutSectionParamMap)
	{
		SectionData.GeneratedKeys = MakeShared<FGeneratedTrackKeys>();
		for (const FControlRigKeyAddParams& Param : SectionData.Params)
		{
			ControlToSection.Add(Param.RigControlName, Section);
		}
	}

	TArray<FRigControlElement*> Controls;
	InControlRig->GetControlsInOrder(Controls);
	URigHierarchy* Hierarchy = InControlRig->GetHierarchy();

	// SpaceChannelIndex mirrors the original per-call behaviour: increments only for
	// batch-keyed controls that have a space channel.
	int32 SpaceChannelIndex = 0;

	for (FRigControlElement* ControlElement : Controls)
	{
		check(ControlElement);
		if (!Hierarchy->IsAnimatable(ControlElement))
		{
			continue;
		}

		UMovieSceneControlRigParameterSection** SectionPtr = ControlToSection.Find(ControlElement->GetFName());
		if (!SectionPtr)
		{
			continue;
		}

		UMovieSceneControlRigParameterSection* SectionToKey = *SectionPtr;
		FGeneratedTrackKeys& OutGeneratedKeys = *InOutSectionParamMap[SectionToKey].GeneratedKeys;

		FChannelMapInfo* pChannelIndex = SectionToKey->ControlChannelMap.Find(ControlElement->GetFName());
		if (!pChannelIndex)
		{
			continue;
		}

		int32 ChannelIndex = pChannelIndex->ChannelIndex;
		const EMovieSceneTransformChannel TransformMask = SectionToKey->GetTransformMask().GetChannels();

		const bool bMaskKeyOut = (SectionToKey->GetControlNameMask(ControlElement->GetFName()) == false);
		const bool bSetKey = !bMaskKeyOut;
		if (!bSetKey)
		{
			continue;
		}

		FRigControlValue ControlValue = InControlRig->GetControlValue(ControlElement, ERigControlValueType::Current);

		switch (ControlElement->Settings.ControlType)
		{
		case ERigControlType::Bool:
		{
			const bool Val = ControlValue.Get<bool>();
			pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneBoolChannel>(ChannelIndex, Val, bSetKey));
			break;
		}
		case ERigControlType::Integer:
		{
			if (ControlElement->Settings.ControlEnum)
			{
				const uint8 Val = (uint8)ControlValue.Get<uint8>();
				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneByteChannel>(ChannelIndex, Val, bSetKey));
			}
			else
			{
				const int32 Val = ControlValue.Get<int32>();
				pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneIntegerChannel>(ChannelIndex, Val, bSetKey));
			}
			break;
		}
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			const float Val = ControlValue.Get<float>();
			pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex, Val, bSetKey));
			break;
		}
		case ERigControlType::Vector2D:
		{
			const bool bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX);
			const bool bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY);
			const FVector3f Val = ControlValue.Get<FVector3f>();
			pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.X, bKeyX));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.Y, bKeyY));
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		case ERigControlType::Rotator:
		{
			bool bKeyX = bSetKey;
			bool bKeyY = bSetKey;
			bool bKeyZ = bSetKey;
			if (ControlElement->Settings.ControlType == ERigControlType::Position)
			{
				bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX);
				bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY);
				bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ);
			}
			else if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
			{
				bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX);
				bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY);
				bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ);
			}
			else // Scale
			{
				bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX);
				bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY);
				bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ);
			}

			const FVector3f Val = ControlValue.Get<FVector3f>();
			pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.X, bKeyX));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.Y, bKeyY));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, Val.Z, bKeyZ));
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{
			FVector Translation, Scale(1.0f, 1.0f, 1.0f);
			FVector Vector = InControlRig->GetControlSpecifiedEulerAngle(ControlElement);
			FRotator Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
			if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
			{
				const FTransformNoScale NoScale = ControlValue.Get<FRigControlValue::FTransformNoScale_Float>().ToTransform();
				Translation = NoScale.Location;
			}
			else if (ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
			{
				const FEulerTransform Euler = ControlValue.Get<FRigControlValue::FEulerTransform_Float>().ToTransform();
				Translation = Euler.Location;
				Scale = Euler.Scale;
			}
			else
			{
				const FTransform Val = ControlValue.Get<FRigControlValue::FTransform_Float>().ToTransform();
				Translation = Val.GetTranslation();
				Scale = Val.GetScale3D();
			}

			if (bInConstraintSpace)
			{
				const uint32 ControlHash = UTransformableControlHandle::ComputeHash(InControlRig, ControlElement->GetFName());
				const TOptional<FTransform> Transform = UE::TransformConstraintUtil::GetRelativeTransform(InControlRig->GetWorld(), ControlHash);
				if (Transform)
				{
					Translation = Transform->GetTranslation();
					if (InControlRig->GetHierarchy()->GetUsePreferredRotationOrder(ControlElement))
					{
						Rotation = ControlElement->PreferredEulerAngles.GetRotatorFromQuat(Transform->GetRotation());
						FVector Angle = Rotation.Euler();
						ControlElement->PreferredEulerAngles.SetAngles(Angle, false, ControlElement->PreferredEulerAngles.RotationOrder, true);
						Angle = InControlRig->GetControlSpecifiedEulerAngle(ControlElement);
						Rotation = FRotator(Vector.Y, Vector.Z, Vector.X);
					}
					else
					{
						Rotation = Transform->GetRotation().Rotator();
					}
					Scale = Transform->GetScale3D();
				}
			}

			FVector3f CurrentVector = (FVector3f)Translation;
			bool bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationX);
			bool bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationY);
			bool bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::TranslationZ);
			if (KeyMode != ESequencerKeyMode::ManualKeyForced && (KeyGroupMode == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ)))
			{
				bKeyX = bKeyY = bKeyZ = true;
			}
			if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationX)) { bKeyX = false; }
			if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationY)) { bKeyY = false; }
			if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::TranslationZ)) { bKeyZ = false; }

			pChannelIndex->GeneratedKeyIndex = OutGeneratedKeys.Num();

			if (pChannelIndex->bDoesHaveSpace)
			{
				FMovieSceneControlRigSpaceBaseKey NewKey;
				const int32 RealSpaceChannelIndex = pChannelIndex->SpaceChannelIndex != -1 ? pChannelIndex->SpaceChannelIndex : SpaceChannelIndex;
				++SpaceChannelIndex;
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneControlRigSpaceChannel>(RealSpaceChannelIndex, NewKey, false));
			}

			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.X, bKeyX));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Y, bKeyY));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Z, bKeyZ));

			FRotator3f CurrentRotator = FRotator3f(Rotation);
			bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationX);
			bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationY);
			bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::RotationZ);
			if (KeyMode != ESequencerKeyMode::ManualKeyForced && (KeyGroupMode == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ)))
			{
				bKeyX = bKeyY = bKeyZ = true;
			}
			if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationX)) { bKeyX = false; }
			if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationY)) { bKeyY = false; }
			if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::RotationZ)) { bKeyZ = false; }

			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentRotator.Roll, bKeyX));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentRotator.Pitch, bKeyY));
			OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentRotator.Yaw, bKeyZ));

			if (ControlElement->Settings.ControlType == ERigControlType::Transform || ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
			{
				CurrentVector = (FVector3f)Scale;
				bKeyX = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleX);
				bKeyY = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleY);
				bKeyZ = bSetKey && EnumHasAnyFlags(ChannelsToKey, EControlRigContextChannelToKey::ScaleZ);
				if (KeyMode != ESequencerKeyMode::ManualKeyForced && (KeyGroupMode == EKeyGroupMode::KeyGroup && (bKeyX || bKeyY || bKeyZ)))
				{
					bKeyX = bKeyY = bKeyZ = true;
				}
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleX)) { bKeyX = false; }
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleY)) { bKeyY = false; }
				if (!EnumHasAnyFlags(TransformMask, EMovieSceneTransformChannel::ScaleZ)) { bKeyZ = false; }
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.X, bKeyX));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Y, bKeyY));
				OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++, CurrentVector.Z, bKeyZ));
			}
			break;
		}
		}
	}
}

FKeyPropertyResult FControlRigParameterTrackEditor::AddKeysToControlRigBatch(
	UObject* InObject, UControlRig* InControlRig,
	FGuid ObjectHandle, UMovieSceneControlRigParameterTrack* Track,
	FFrameNumber KeyTime, FFrameNumber EvaluateTime,
	FControlRigSectionParamMap& InOutSectionParamMap,
	ESequencerKeyMode KeyMode, FName ControlRigName)
{
	FKeyPropertyResult CombinedResult;

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return CombinedResult;
	}

	// Each section is visited exactly once: one RemapTime, one bounds check, one weight calculation,
	// one interrogation, and one AddKeysToSection using the section's single GeneratedKeys.
	for (auto& [ParamSection, SectionData] : InOutSectionParamMap)
	{
		if (!SectionData.GeneratedKeys.IsValid() || SectionData.GeneratedKeys->Num() <= 0)
		{
			continue;
		}

		// Time remap handles time-warped sections.
		const FFrameNumber RemappedTime = ParamSection->RemapTime(FFrameTime(KeyTime)).GetFrame();

		// Extend section bounds once.
		if (ParamSection->HasEndFrame() && ParamSection->GetExclusiveEndFrame() < RemappedTime)
		{
			ParamSection->SetEndFrame(RemappedTime);
		}
		else if (ParamSection->HasStartFrame() && ParamSection->GetInclusiveStartFrame() > RemappedTime)
		{
			ParamSection->SetStartFrame(RemappedTime);
		}

		if (!ParamSection->GetRange().Contains(RemappedTime))
		{
			continue;
		}

		// One weight calculation per section.
		const float Weight = MovieSceneHelpers::CalculateWeightForBlending(ParamSection, RemappedTime);

		// One track interrogation for all controls in this section.
		ModifyOurGeneratedKeysByCurrentAndWeight(InObject, InControlRig, Track, ParamSection, EvaluateTime, SectionData, Weight);

		// GeneratedKeys already holds all controls' keys — pass it directly, no combining needed.
		FKeyPropertyResult KeyResult = AddKeysToSection(ParamSection, RemappedTime, *SectionData.GeneratedKeys, KeyMode, EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys);
		CombinedResult |= KeyResult;
	}

	// Compensate spaces and constraints per control.
	if (CombinedResult.bKeyCreated)
	{
		for (auto& [ParamSection, SectionData] : InOutSectionParamMap)
		{
			UControlRig* SectionControlRig = ParamSection->GetControlRig();
			if (!SectionControlRig || SectionControlRig->IsEvaluating())
			{
				continue;
			}

			const FFrameNumber RemappedTime = ParamSection->RemapTime(FFrameTime(KeyTime)).GetFrame();
			constexpr bool bComputePreviousTick = true;

			for (const FControlRigKeyAddParams& Param : SectionData.Params)
			{
				TOptional<FFrameNumber> OptionalKeyTime = RemappedTime;

				FControlRigSpaceChannelHelpers::CompensateIfNeeded(
					SectionControlRig, Sequencer.Get(), ParamSection,
					OptionalKeyTime, bComputePreviousTick);

				const uint32 ControlHash = UTransformableControlHandle::ComputeHash(SectionControlRig, Param.RigControlName);
				FMovieSceneConstraintChannelHelper::CompensateIfNeeded(Sequencer, ParamSection, OptionalKeyTime, bComputePreviousTick, ControlHash);
			}
		}
	}

	return CombinedResult;
}

FKeyPropertyResult FControlRigParameterTrackEditor::AddKeysToControlRigHandle(UObject* InObject, UControlRig* InControlRig,
	FGuid ObjectHandle, FFrameNumber KeyTime, FFrameNumber EvaluateTime, FGeneratedTrackKeys& GeneratedKeys,
	ESequencerKeyMode KeyMode, TSubclassOf<UMovieSceneTrack> TrackClass, FName ControlRigName, FName RigControlName)
{
	FKeyPropertyResult KeyPropertyResult;
	
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return KeyPropertyResult;
	}

	const EAutoChangeMode AutoChangeMode = Sequencer->GetAutoChangeMode();
	const EAllowEditsMode AllowEditsMode = Sequencer->GetAllowEditsMode();

	bool bCreateTrack =
		(KeyMode == ESequencerKeyMode::AutoKey && (AutoChangeMode == EAutoChangeMode::AutoTrack || AutoChangeMode == EAutoChangeMode::All)) ||
		KeyMode == ESequencerKeyMode::ManualKey ||
		KeyMode == ESequencerKeyMode::ManualKeyForced ||
		AllowEditsMode == EAllowEditsMode::AllowSequencerEditsOnly;

	bool bCreateSection = false;
	// we don't do this, maybe revisit if a bug occurs, but currently extends sections on autokey.
	//bCreateTrack || (KeyMode == ESequencerKeyMode::AutoKey && (AutoChangeMode != EAutoChangeMode::None));

	// Try to find an existing Track, and if one doesn't exist check the key params and create one if requested.

	FFindOrCreateTrackResult TrackResult = FindOrCreateControlRigTrackForObject(ObjectHandle, InControlRig, ControlRigName, bCreateTrack);
	UMovieSceneControlRigParameterTrack* Track = CastChecked<UMovieSceneControlRigParameterTrack>(TrackResult.Track, ECastCheckedType::NullAllowed);

	bool bTrackCreated = TrackResult.bWasCreated;

	if (Track)
	{
		UMovieSceneSection* SectionToKey = Track->GetSectionToKey(RigControlName);
		if (SectionToKey)
		{
			if (SectionToKey->HasEndFrame() && SectionToKey->GetExclusiveEndFrame() < KeyTime)
			{
				SectionToKey->SetEndFrame(KeyTime);
			}
			else if(SectionToKey->HasStartFrame() && SectionToKey->GetInclusiveStartFrame() > KeyTime)
			{
				SectionToKey->SetStartFrame(KeyTime);
			}
		}
		
		bool bSectionCreated = false;
		float Weight = 1.0f;

		// If there's no overlapping section to key, create one only if a track was newly created. Otherwise, skip keying altogether
		// so that the user is forced to create a section to key on.
		if (bTrackCreated && !SectionToKey)
		{
			Track->Modify();
			SectionToKey = Track->FindOrAddSection(KeyTime, bSectionCreated);
			if (bSectionCreated && Sequencer->GetInfiniteKeyAreas())
			{
				SectionToKey->SetRange(TRange<FFrameNumber>::All());
			}
		}

		if (SectionToKey && SectionToKey->GetRange().Contains(KeyTime))
		{
			if (!bTrackCreated)
			{
				//make sure to use weight on section to key
				Weight = MovieSceneHelpers::CalculateWeightForBlending(SectionToKey, KeyTime);
				ModifyOurGeneratedKeysByCurrentAndWeight(InObject, InControlRig, RigControlName, Track, SectionToKey, EvaluateTime, GeneratedKeys, Weight);
			}
			const UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(SectionToKey);
			if (ParamSection && !ParamSection->GetDoNotKey())
			{
				KeyPropertyResult |= AddKeysToSection(SectionToKey, KeyTime, GeneratedKeys, KeyMode, EKeyFrameTrackEditorSetDefault::SetDefaultOnAddKeys);
			}
		}


		KeyPropertyResult.bTrackCreated |= bTrackCreated || bSectionCreated;
		//if we create a key then compensate
		if (KeyPropertyResult.bKeyCreated)
		{
			UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(Track->GetSectionToKey(ControlRigName));
			if (UControlRig* SectionControlRig = ParamSection ? ParamSection->GetControlRig() : nullptr)
			{
				// avoid reentrant evaluation: space switching & constraints evaluates the rig at T-1
				// but this must not happen if it's already being evaluated.
				if (!SectionControlRig->IsEvaluating())
				{
					TOptional<FFrameNumber> OptionalKeyTime = KeyTime;

					constexpr bool bComputePreviousTick = true;
					
					// compensate spaces
					FControlRigSpaceChannelHelpers::CompensateIfNeeded(
						SectionControlRig, Sequencer.Get(), ParamSection,
						OptionalKeyTime, bComputePreviousTick);

					// compensate constraints
					const uint32 ControlHash = UTransformableControlHandle::ComputeHash(SectionControlRig, RigControlName);
					FMovieSceneConstraintChannelHelper::CompensateIfNeeded(Sequencer, ParamSection, OptionalKeyTime, bComputePreviousTick, ControlHash);
				}
			}
		}
	}
	return KeyPropertyResult;
}

FKeyPropertyResult FControlRigParameterTrackEditor::AddKeysToControlRig(
	UObject* InObject, UControlRig* InControlRig, FFrameNumber KeyTime, FFrameNumber EvaluateTime, FGeneratedTrackKeys& GeneratedKeys,
	ESequencerKeyMode KeyMode, TSubclassOf<UMovieSceneTrack> TrackClass, FName ControlRigName, FName RigControlName)
{
	FKeyPropertyResult KeyPropertyResult;
	
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || GeneratedKeys.Num() <= 0)
	{
		return KeyPropertyResult;
	}
	
	const EAutoChangeMode AutoChangeMode = Sequencer->GetAutoChangeMode();
	const EAllowEditsMode AllowEditsMode = Sequencer->GetAllowEditsMode();

	bool bCreateHandle =
		(KeyMode == ESequencerKeyMode::AutoKey && (AutoChangeMode == EAutoChangeMode::All)) ||
		KeyMode == ESequencerKeyMode::ManualKey ||
		KeyMode == ESequencerKeyMode::ManualKeyForced ||
		AllowEditsMode == EAllowEditsMode::AllowSequencerEditsOnly;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(InObject, InControlRig);
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated = HandleResult.bWasCreated;
	KeyPropertyResult |= AddKeysToControlRigHandle(InObject, InControlRig, ObjectHandle, KeyTime, EvaluateTime, GeneratedKeys, KeyMode, TrackClass, ControlRigName, RigControlName);

	return KeyPropertyResult;
}

void FControlRigParameterTrackEditor::AddControlKeys(
	UObject* InObject,
	UControlRig* InControlRig,
	FName ControlRigName,
	FName RigControlName,
	EControlRigContextChannelToKey ChannelsToKey,
	ESequencerKeyMode KeyMode,
	float InLocalTime,
	const bool bInConstraintSpace)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}
	
	if (KeyMode == ESequencerKeyMode::ManualKey || !Sequencer->IsAllowedToChange())
	{
		return;
	}
	
	bool bCreateTrack = false;
	bool bCreateHandle = false;
	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(InObject, InControlRig);
	FGuid ObjectHandle = HandleResult.Handle;
	FFindOrCreateTrackResult TrackResult = FindOrCreateControlRigTrackForObject(ObjectHandle, InControlRig, ControlRigName, bCreateTrack);
	UMovieSceneControlRigParameterTrack* Track = CastChecked<UMovieSceneControlRigParameterTrack>(TrackResult.Track, ECastCheckedType::NullAllowed);
	UMovieSceneControlRigParameterSection* ParamSection = nullptr;
	if (Track)
	{
		//track editors use a hidden time so we need to set it if we are using non sequencer times when keying.
		if (InLocalTime != FLT_MAX)
		{
			//convert from frame time since conversion may give us one frame less, e.g 1.53333330 * 24000.0/1.0 = 36799.999199999998
			FFrameTime LocalFrameTime = Sequencer->GetFocusedTickResolution().AsFrameTime((double)InLocalTime);
			BeginKeying(LocalFrameTime.RoundToFrame());
		}
		
		const FFrameNumber FrameTime = GetTimeForKey();
		UMovieSceneSection* Section = Track->GetSectionToKey(RigControlName);
		ParamSection = Cast<UMovieSceneControlRigParameterSection>(Section);

		if (ParamSection && ParamSection->GetDoNotKey())
		{
			return;
		}
	}

	if (!ParamSection)
	{
		return;
	}

	TSharedRef<FGeneratedTrackKeys> GeneratedKeys = MakeShared<FGeneratedTrackKeys>();
	GetControlRigKeys(InControlRig, RigControlName, ChannelsToKey, KeyMode, ParamSection, *GeneratedKeys, bInConstraintSpace);
	if (GeneratedKeys->Num() <= 0)
	{
		return;
	}
	TGuardValue<bool> Guard(bIsDoingSelection, true);

	auto OnKeyProperty = [=, this](FFrameNumber Time) -> FKeyPropertyResult
	{
		if (const TSharedPtr<ISequencer> Sequencer = GetSequencer())
		{
			FFrameNumber LocalTime = Time;
			//for modify weights we evaluate so need to make sure we use the evaluated time
			FFrameNumber EvaluateTime = Sequencer->GetLastEvaluatedLocalTime().RoundToFrame();
			//if InLocalTime is specified that means time value was set with SetControlValue, so we don't use sequencer times at all, but this time instead
			if (InLocalTime != FLT_MAX)
			{
				//convert from frame time since conversion may give us one frame less, e.g 1.53333330 * 24000.0/1.0 = 36799.999199999998
				FFrameTime LocalFrameTime = Sequencer->GetFocusedTickResolution().AsFrameTime((double)InLocalTime);
				LocalTime = LocalFrameTime.RoundToFrame();
				EvaluateTime = LocalTime;
			}
			LocalTime = ParamSection->RemapTime(FFrameTime(LocalTime)).GetFrame();
			return this->AddKeysToControlRig(InObject, InControlRig, LocalTime, EvaluateTime, *GeneratedKeys, KeyMode, UMovieSceneControlRigParameterTrack::StaticClass(), ControlRigName, RigControlName);
		}
		
		static const FKeyPropertyResult Dummy;
		return Dummy;
	};

	AnimatablePropertyChanged(FOnKeyProperty::CreateLambda(OnKeyProperty));
	EndKeying(); //fine even if we didn't BeginKeying
}

void FControlRigParameterTrackEditor::AddControlKeys(
	UObject* InObject,
	UControlRig* InControlRig,
	FName ControlRigName,
	TArray<FControlRigKeyAddParams>& InOutParams,
	EControlRigContextChannelToKey ChannelsToKey,
	ESequencerKeyMode KeyMode,
	float InLocalTime,
	const bool bInConstraintSpace)
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || KeyMode == ESequencerKeyMode::ManualKey || !Sequencer->IsAllowedToChange())
	{
		return;
	}

	// Resolve handle and track once for all controls in the batch.
	constexpr bool bCreateTrack = false;
	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(InObject, InControlRig);
	const FGuid ObjectHandle = HandleResult.Handle;
	FFindOrCreateTrackResult TrackResult = FindOrCreateControlRigTrackForObject(ObjectHandle, InControlRig, ControlRigName, bCreateTrack);
	UMovieSceneControlRigParameterTrack* Track = CastChecked<UMovieSceneControlRigParameterTrack>(TrackResult.Track, ECastCheckedType::NullAllowed);
	if (!Track)
	{
		return;
	}

	if (InLocalTime != FLT_MAX)
	{
		const FFrameTime LocalFrameTime = Sequencer->GetFocusedTickResolution().AsFrameTime((double)InLocalTime);
		BeginKeying(LocalFrameTime.RoundToFrame());
	}

	// Group controls by section. Modify() each section once when it first appears in the map.
	FControlRigSectionParamMap SectionParamMap;
	for (const FControlRigKeyAddParams& Param : InOutParams)
	{
		UMovieSceneSection* Section = Track->GetSectionToKey(Param.RigControlName);
		UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(Section);
		if (!ParamSection || ParamSection->GetDoNotKey())
		{
			continue;
		}
		FControlRigSectionKeyData& SectionData = SectionParamMap.FindOrAdd(ParamSection);
		if (SectionData.Params.IsEmpty())
		{
			// First control for this section — mark it dirty before any modifications.
			ParamSection->Modify();
		}
		SectionData.Params.Add(Param);
	}

	if (SectionParamMap.IsEmpty())
	{
		EndKeying();
		return;
	}

	// Single pass over the rig hierarchy; each section gets one shared GeneratedKeys.
	GetControlRigKeys(InControlRig, SectionParamMap, ChannelsToKey, KeyMode, bInConstraintSpace);

	// Drop sections whose GeneratedKeys are empty.
	for (auto It = SectionParamMap.CreateIterator(); It; ++It)
	{
		const TSharedPtr<FGeneratedTrackKeys>& Keys = It->Value.GeneratedKeys;
		if (!Keys.IsValid() || Keys->Num() <= 0)
		{
			It.RemoveCurrent();
		}
	}

	if (SectionParamMap.IsEmpty())
	{
		EndKeying();
		return;
	}

	TGuardValue<bool> Guard(bIsDoingSelection, true);

	// Move the map into the lambda — AnimatablePropertyChanged is synchronous, but owning it is cleaner.
	auto OnKeyProperty = [InObject, InControlRig, InLocalTime, KeyMode, ControlRigName, ObjectHandle, Track,
		SectionMap = MoveTemp(SectionParamMap), this](FFrameNumber Time) mutable -> FKeyPropertyResult
	{
		const TSharedPtr<ISequencer> LocalSequencer = GetSequencer();
		if (!LocalSequencer)
		{
			static const FKeyPropertyResult Dummy;
			return Dummy;
		}

		FFrameNumber LocalTime = Time;
		FFrameNumber EvaluateTime = LocalSequencer->GetLastEvaluatedLocalTime().RoundToFrame();
		if (InLocalTime != FLT_MAX)
		{
			const FFrameTime LocalFrameTime = LocalSequencer->GetFocusedTickResolution().AsFrameTime((double)InLocalTime);
			LocalTime = LocalFrameTime.RoundToFrame();
			EvaluateTime = LocalTime;
		}

		return AddKeysToControlRigBatch(InObject, InControlRig, ObjectHandle, Track,
			LocalTime, EvaluateTime, SectionMap, KeyMode, ControlRigName);
	};

	AnimatablePropertyChanged(FOnKeyProperty::CreateLambda(OnKeyProperty));
	EndKeying(); //fine even if we didn't BeginKeying
}

bool FControlRigParameterTrackEditor::ModifyOurGeneratedKeysByCurrentAndWeight(UObject* Object, UControlRig* InControlRig, FName RigControlName, UMovieSceneTrack* Track, UMovieSceneSection* SectionToKey, FFrameNumber EvaluateTime, FGeneratedTrackKeys& GeneratedTotalKeys, float Weight) const
{
	using namespace UE::MovieScene;

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return false;
	}
	
	// Start off with stable names for population since we shouldn't encounter any duplicates
	FControlRigParameterValues ParameterBufferValues(EControlRigParameterBufferIndexStability::Stable);
	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

	if (UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate())
	{
		UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = CastChecked<UMovieSceneControlRigParameterTrack>(Track);
		
		FMovieSceneInterrogationData InterrogationData;
		Sequencer->GetEvaluationTemplate().CopyActuators(InterrogationData.GetAccumulator());
		//use the EvaluateTime to do the evaluation, may be different than the actually time we key
		FMovieSceneContext Context(FMovieSceneEvaluationRange(EvaluateTime, TickResolution));

		const TUniquePtr<FMovieSceneEvaluationTrack>& MovieSceneEvaluationTrack = ControlRigParameterTrack->GetLegacyEvaluationTrack();
		if (MovieSceneEvaluationTrack.IsValid())
		{
			MovieSceneEvaluationTrack->Interrogate(Context, InterrogationData, Object);
		}
		else		
		{
			FMovieSceneEvaluationTrack EvalTrack = ControlRigParameterTrack->GenerateTrackTemplate(Track);
			EvalTrack.Interrogate(Context, InterrogationData, Object);
		}

		// Add the legacy interrogated data to the parameter buffer
		for (const FFloatInterrogationData& Val : InterrogationData.Iterate<FFloatInterrogationData>(UMovieSceneControlRigParameterSection::GetFloatInterrogationKey()))
		{
			ParameterBufferValues.Add(Val.ParameterName, Val.Val);
		}
		for (const FVector2DInterrogationData& Val : InterrogationData.Iterate<FVector2DInterrogationData>(UMovieSceneControlRigParameterSection::GetVector2DInterrogationKey()))
		{
			FVector3f Vector((float)Val.Val.X, (float)Val.Val.Y, 0.f);
			ParameterBufferValues.Add(Val.ParameterName, Vector);
		}
		for (const FVectorInterrogationData& Val : InterrogationData.Iterate<FVectorInterrogationData>(UMovieSceneControlRigParameterSection::GetVectorInterrogationKey()))
		{
			FVector3f Vector((float)Val.Val.X, (float)Val.Val.Y, (float)Val.Val.Z);
			ParameterBufferValues.Add(Val.ParameterName, Vector);
		}
		for (const FEulerTransformInterrogationData& Val : InterrogationData.Iterate<FEulerTransformInterrogationData>(UMovieSceneControlRigParameterSection::GetTransformInterrogationKey()))
		{
			ParameterBufferValues.Add(Val.ParameterName, Val.Val);
		}
	}
	else
	{
		// Create the interrogator.
		FSystemInterrogator Interrogator;
		Interrogator.TrackImportedEntities(true);

		TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Interrogator.GetLinker()->EntityManager);


		// Run an interrogation on the track at the specified time.
		FInterrogationKey InterrogationKey(FInterrogationKey::Default());
		FInterrogationChannel InterrogationChannel = Interrogator.AllocateChannel(InControlRig, FMovieScenePropertyBinding());
		InterrogationKey.Channel = InterrogationChannel;
		Interrogator.ImportTrack(Track, InterrogationChannel);

		Interrogator.AddInterrogation(EvaluateTime);
		Interrogator.Update();

		// Find the CR System
		UMovieSceneControlRigParameterEvaluatorSystem* ControlRigSystem   = Interrogator.GetLinker()->FindSystem<UMovieSceneControlRigParameterEvaluatorSystem>();
		const FControlRigParameterBuffer*              ParameterBufferPtr = ControlRigSystem ? ControlRigSystem->FindParameters(CastChecked<UMovieSceneControlRigParameterTrack>(Track)) : nullptr;

		if (!ensure(ParameterBufferPtr))
		{
			return false;
		}

		ParameterBufferValues = ParameterBufferPtr->Values;
	}

	// Make searching faster by hashing the values
	ParameterBufferValues.OptimizeForLookup();

	TArray<FRigControlElement*> Controls = InControlRig->AvailableControls();
	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey);
	FMovieSceneChannelProxy& Proxy = SectionToKey->GetChannelProxy();

	//evaluate time will be in Sequencer unwarped time since we interrogated to get the 'final' value
	//we know need to convert it to warped time so we can blend the delta with the modified value/weight
	EvaluateTime = Section->RemapTime(FFrameTime(EvaluateTime)).GetFrame();

	for (FRigControlElement* ControlElement : Controls)
	{
		if (!InControlRig->GetHierarchy()->IsAnimatable(ControlElement))
		{
			continue;
		}

		FName ControlName = ControlElement->GetFName();
		const FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlName);
		if (!pChannelIndex || pChannelIndex->GeneratedKeyIndex == INDEX_NONE || RigControlName != ControlName)
		{
			continue;
		}

		switch (ControlElement->Settings.ControlType)
		{
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			float Value = 0.f;
			if (ParameterBufferValues.Find<float>(ControlName, Value))
			{
				GeneratedTotalKeys[pChannelIndex->GeneratedKeyIndex]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void *)&Value, Weight);
			}
			break;
		}
		//no blending of bools,ints/enums
		case ERigControlType::Bool:
		case ERigControlType::Integer:
		{

			break;
		}
		case ERigControlType::Vector2D:
		{
			FVector3f Value;
			if (ParameterBufferValues.Find(ControlName, Value))
			{
				GeneratedTotalKeys[pChannelIndex->GeneratedKeyIndex  ]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void *)&Value.X, Weight);
				GeneratedTotalKeys[pChannelIndex->GeneratedKeyIndex+1]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void *)&Value.Y, Weight);
			}
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		case ERigControlType::Rotator:
		{
			FVector3f Value;
			if (ParameterBufferValues.Find(ControlName, Value))
			{
				GeneratedTotalKeys[pChannelIndex->GeneratedKeyIndex  ]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void *)&Value.X, Weight);
				GeneratedTotalKeys[pChannelIndex->GeneratedKeyIndex+1]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void *)&Value.Y, Weight);
				GeneratedTotalKeys[pChannelIndex->GeneratedKeyIndex+2]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void *)&Value.Z, Weight);
			}
			break;
		}

		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{

			FEulerTransform Value;
			if (ParameterBufferValues.Find(ControlName, Value))
			{
				FVector3f  CurrentPos(Value.GetLocation());
				FRotator3f CurrentRot(Value.Rotator());

				int32 ChannelIndex = pChannelIndex->bDoesHaveSpace ? pChannelIndex->GeneratedKeyIndex + 1 : pChannelIndex->GeneratedKeyIndex;

				GeneratedTotalKeys[ChannelIndex]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentPos.X, Weight);
				GeneratedTotalKeys[ChannelIndex + 1]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentPos.Y, Weight);
				GeneratedTotalKeys[ChannelIndex + 2]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentPos.Z, Weight);

				GeneratedTotalKeys[ChannelIndex + 3]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentRot.Roll, Weight);
				GeneratedTotalKeys[ChannelIndex + 4]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentRot.Pitch, Weight);
				GeneratedTotalKeys[ChannelIndex + 5]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentRot.Yaw, Weight);

				if (ControlElement->Settings.ControlType == ERigControlType::Transform || ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					FVector3f CurrentScale(Value.GetScale3D());
					GeneratedTotalKeys[ChannelIndex + 6]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentScale.X, Weight);
					GeneratedTotalKeys[ChannelIndex + 7]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentScale.Y, Weight);
					GeneratedTotalKeys[ChannelIndex + 8]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentScale.Z, Weight);
				}
			}
			break;
		}
		}
	}

	return true;
}

bool FControlRigParameterTrackEditor::ModifyOurGeneratedKeysByCurrentAndWeight(
	UObject* Object, UControlRig* InControlRig,
	UMovieSceneTrack* Track, UMovieSceneSection* SectionToKey, FFrameNumber EvaluateTime,
	FControlRigSectionKeyData& InOutSectionKeyData, float Weight) const
{
	using namespace UE::MovieScene;

	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer || !InOutSectionKeyData.GeneratedKeys.IsValid())
	{
		return false;
	}

	// Build a set of the control names being keyed for fast membership checks.
	TSet<FName> KeyedControlNames;
	KeyedControlNames.Reserve(InOutSectionKeyData.Params.Num());
	for (const FControlRigKeyAddParams& Param : InOutSectionKeyData.Params)
	{
		KeyedControlNames.Add(Param.RigControlName);
	}

	// Run a single track interrogation for the whole section — this is the expensive step.
	FControlRigParameterValues ParameterBufferValues(EControlRigParameterBufferIndexStability::Stable);
	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

	if (UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate())
	{
		UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = CastChecked<UMovieSceneControlRigParameterTrack>(Track);

		FMovieSceneInterrogationData InterrogationData;
		Sequencer->GetEvaluationTemplate().CopyActuators(InterrogationData.GetAccumulator());
		FMovieSceneContext Context(FMovieSceneEvaluationRange(EvaluateTime, TickResolution));

		const TUniquePtr<FMovieSceneEvaluationTrack>& MovieSceneEvaluationTrack = ControlRigParameterTrack->GetLegacyEvaluationTrack();
		if (MovieSceneEvaluationTrack.IsValid())
		{
			MovieSceneEvaluationTrack->Interrogate(Context, InterrogationData, Object);
		}
		else
		{
			FMovieSceneEvaluationTrack EvalTrack = ControlRigParameterTrack->GenerateTrackTemplate(Track);
			EvalTrack.Interrogate(Context, InterrogationData, Object);
		}

		for (const FFloatInterrogationData& Val : InterrogationData.Iterate<FFloatInterrogationData>(UMovieSceneControlRigParameterSection::GetFloatInterrogationKey()))
		{
			ParameterBufferValues.Add(Val.ParameterName, Val.Val);
		}
		for (const FVector2DInterrogationData& Val : InterrogationData.Iterate<FVector2DInterrogationData>(UMovieSceneControlRigParameterSection::GetVector2DInterrogationKey()))
		{
			FVector3f Vector((float)Val.Val.X, (float)Val.Val.Y, 0.f);
			ParameterBufferValues.Add(Val.ParameterName, Vector);
		}
		for (const FVectorInterrogationData& Val : InterrogationData.Iterate<FVectorInterrogationData>(UMovieSceneControlRigParameterSection::GetVectorInterrogationKey()))
		{
			FVector3f Vector((float)Val.Val.X, (float)Val.Val.Y, (float)Val.Val.Z);
			ParameterBufferValues.Add(Val.ParameterName, Vector);
		}
		for (const FEulerTransformInterrogationData& Val : InterrogationData.Iterate<FEulerTransformInterrogationData>(UMovieSceneControlRigParameterSection::GetTransformInterrogationKey()))
		{
			ParameterBufferValues.Add(Val.ParameterName, Val.Val);
		}
	}
	else
	{
		FSystemInterrogator Interrogator;
		Interrogator.TrackImportedEntities(true);

		TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Interrogator.GetLinker()->EntityManager);

		FInterrogationKey InterrogationKey(FInterrogationKey::Default());
		FInterrogationChannel InterrogationChannel = Interrogator.AllocateChannel(InControlRig, FMovieScenePropertyBinding());
		InterrogationKey.Channel = InterrogationChannel;
		Interrogator.ImportTrack(Track, InterrogationChannel);

		Interrogator.AddInterrogation(EvaluateTime);
		Interrogator.Update();

		UMovieSceneControlRigParameterEvaluatorSystem* ControlRigSystem   = Interrogator.GetLinker()->FindSystem<UMovieSceneControlRigParameterEvaluatorSystem>();
		const FControlRigParameterBuffer*              ParameterBufferPtr = ControlRigSystem ? ControlRigSystem->FindParameters(CastChecked<UMovieSceneControlRigParameterTrack>(Track)) : nullptr;

		if (!ensure(ParameterBufferPtr))
		{
			return false;
		}

		ParameterBufferValues = ParameterBufferPtr->Values;
	}

	ParameterBufferValues.OptimizeForLookup();

	UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey);
	FMovieSceneChannelProxy& Proxy = SectionToKey->GetChannelProxy();
	EvaluateTime = Section->RemapTime(FFrameTime(EvaluateTime)).GetFrame();

	TArray<FRigControlElement*> Controls = InControlRig->AvailableControls();
	TArray<FMovieSceneChannelValueSetter, TInlineAllocator<1>>& GeneratedKeys = *InOutSectionKeyData.GeneratedKeys;

	for (FRigControlElement* ControlElement : Controls)
	{
		if (!InControlRig->GetHierarchy()->IsAnimatable(ControlElement))
		{
			continue;
		}

		const FName ControlName = ControlElement->GetFName();
		if (!KeyedControlNames.Contains(ControlName))
		{
			continue;
		}

		const FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlName);
		if (!pChannelIndex || pChannelIndex->GeneratedKeyIndex == INDEX_NONE)
		{
			continue;
		}


		switch (ControlElement->Settings.ControlType)
		{
		case ERigControlType::Float:
		case ERigControlType::ScaleFloat:
		{
			float Value = 0.f;
			if (ParameterBufferValues.Find<float>(ControlName, Value))
			{
				GeneratedKeys[pChannelIndex->GeneratedKeyIndex]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&Value, Weight);
			}
			break;
		}
		case ERigControlType::Bool:
		case ERigControlType::Integer:
		{
			// No blending for bools or ints/enums.
			break;
		}
		case ERigControlType::Vector2D:
		{
			FVector3f Value;
			if (ParameterBufferValues.Find(ControlName, Value))
			{
				GeneratedKeys[pChannelIndex->GeneratedKeyIndex    ]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&Value.X, Weight);
				GeneratedKeys[pChannelIndex->GeneratedKeyIndex + 1]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&Value.Y, Weight);
			}
			break;
		}
		case ERigControlType::Position:
		case ERigControlType::Scale:
		case ERigControlType::Rotator:
		{
			FVector3f Value;
			if (ParameterBufferValues.Find(ControlName, Value))
			{
				GeneratedKeys[pChannelIndex->GeneratedKeyIndex    ]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&Value.X, Weight);
				GeneratedKeys[pChannelIndex->GeneratedKeyIndex + 1]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&Value.Y, Weight);
				GeneratedKeys[pChannelIndex->GeneratedKeyIndex + 2]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&Value.Z, Weight);
			}
			break;
		}
		case ERigControlType::Transform:
		case ERigControlType::TransformNoScale:
		case ERigControlType::EulerTransform:
		{
			FEulerTransform Value;
			if (ParameterBufferValues.Find(ControlName, Value))
			{
				FVector3f  CurrentPos(Value.GetLocation());
				FRotator3f CurrentRot(Value.Rotator());

				const int32 ChannelIndex = pChannelIndex->bDoesHaveSpace ? pChannelIndex->GeneratedKeyIndex + 1 : pChannelIndex->GeneratedKeyIndex;

				GeneratedKeys[ChannelIndex    ]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentPos.X, Weight);
				GeneratedKeys[ChannelIndex + 1]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentPos.Y, Weight);
				GeneratedKeys[ChannelIndex + 2]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentPos.Z, Weight);

				GeneratedKeys[ChannelIndex + 3]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentRot.Roll,  Weight);
				GeneratedKeys[ChannelIndex + 4]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentRot.Pitch, Weight);
				GeneratedKeys[ChannelIndex + 5]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentRot.Yaw,   Weight);

				if (ControlElement->Settings.ControlType == ERigControlType::Transform || ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
				{
					FVector3f CurrentScale(Value.GetScale3D());
					GeneratedKeys[ChannelIndex + 6]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentScale.X, Weight);
					GeneratedKeys[ChannelIndex + 7]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentScale.Y, Weight);
					GeneratedKeys[ChannelIndex + 8]->ModifyByCurrentAndWeight(Proxy, EvaluateTime, (void*)&CurrentScale.Z, Weight);
				}
			}
			break;
		}
		}
	}

	return true;
}

void FControlRigParameterTrackEditor::ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::Sequencer;
	if (UMovieSceneControlRigParameterTrack::ShouldUseLegacyTemplate())
	{
		TMap<UMovieSceneControlRigParameterTrack*, TUniquePtr<FScopedLegacyEvaluationTrack>> CachedEvaluationTracks;
	
		// Cache the evaluation tracks if requested
		if (UMovieSceneControlRigParameterTrack::ShouldCacheLegacyTemplate())
		{
			IterateTracks([&CachedEvaluationTracks](UMovieSceneControlRigParameterTrack* Track, FGuid InGuid)
		   {
			   if (Track && !CachedEvaluationTracks.Contains(Track))
			   {
				   CachedEvaluationTracks.Emplace(Track, new FScopedLegacyEvaluationTrack(Track));
			   }
			   return true;
		   });
		}
		
		// Legacy behavior uses all the legacy mechanisms
		return FKeyframeTrackEditor<UMovieSceneControlRigParameterTrack>::ProcessKeyOperation(InKeyTime, Operation, InSequencer, OutResults);
	}

	// ECS system performs a full recomposition on the ECS data
	auto Iterator = [this, InKeyTime, &InSequencer, OutResults](UMovieSceneTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> Operations)
	{
		UMovieSceneControlRigParameterTrack* ControlRigTrack = CastChecked<UMovieSceneControlRigParameterTrack>(Track);

		FGuid ObjectBinding = Track->FindObjectBindingGuid();
		if (ObjectBinding.IsValid())
		{
			for (TWeakObjectPtr<> WeakObject : InSequencer.FindBoundObjects(ObjectBinding, InSequencer.GetFocusedTemplateID()))
			{
				if (UObject* Object = WeakObject.Get())
				{
					this->ProcessKeyOperation(Object, ControlRigTrack, Operations, InSequencer, InKeyTime, OutResults);
					return;
				}
			}
		}

		// Default behavior
		FKeyOperation::ApplyOperations(InKeyTime, Operations, ObjectBinding, InSequencer, OutResults);
	};

	Operation.IterateOperations(Iterator);
}

void FControlRigParameterTrackEditor::ProcessKeyOperation(UObject* ObjectToKey, UMovieSceneControlRigParameterTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// @todo: This should really be unified with AddControlKeys and ModifyOurGeneratedKeysByCurrentAndWeight
	//        so that everything goes through the common FControlRigParameterValues container, but to do so 
	//        we need to port some additional logic around constraint space and a few other pieces.
	//
	//        From there constructing an FGeneratedKeys structure should be routine, and should allow us
	//        to remove FChannelMapInfo::GeneratedKeyIndex intogether
	// ~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~

	if (!ensure(Track))
	{
		return;
	}

	const EMovieSceneKeyInterpolation InterpMode = InSequencer.GetSequencerSettings()->GetKeyInterpolation();

	// Iterate each section and perform the key operation on recomposed values
	for (const FKeySectionOperation& Operation : SectionsToKey)
	{
		UMovieSceneControlRigParameterSection* ControlRigSection = Cast<UMovieSceneControlRigParameterSection>(Operation.Section->GetSectionObject());
		ControlRigSection->Modify();

		FControlRigParameterValues RecomposedValues = GetRecomposedControlValues(Track, ControlRigSection, KeyTime);

		for (TSharedPtr<IKeyArea> KeyArea : Operation.KeyAreas)
		{
			const FMovieSceneChannelHandle& ChannelHandle = KeyArea->GetChannel();
			FMovieSceneChannel*             Channel       = KeyArea->ResolveChannel();

			if (Channel == &ControlRigSection->Weight)
			{
				FKeyHandle KeyHandle = KeyArea->AddOrUpdateKey(KeyTime, FGuid(), InSequencer);
				if (OutResults)
				{
					FAddKeyResult Result;
					Result.KeyArea = KeyArea;
					Result.KeyHandle = KeyHandle;
					OutResults->Add(Result);
				}
				continue;
			}

			FControlRigChannelMetaData ChannelMetaData = ControlRigSection->GetChannelMetaData(Channel);
			if (!ChannelMetaData || !ControlRigSection->GetControlNameMask(ChannelMetaData.GetControlName()))
			{
				continue;
			}

			FControlRigValueView DesiredValue = RecomposedValues.Find(ChannelMetaData.GetControlName(), ChannelMetaData.GetType());
			if (DesiredValue.GetType() == EControlRigControlType::Space)
			{

			}
			else if (DesiredValue.GetType() == EControlRigControlType::Parameter_Bool)
			{
				FMovieSceneBoolChannel* Bool = static_cast<FMovieSceneBoolChannel*>(Channel);
				Bool->GetData().UpdateOrAddKey(KeyTime, *DesiredValue.Cast<bool>());
			}
			else if (DesiredValue.GetType() == EControlRigControlType::Parameter_Enum)
			{
				FMovieSceneByteChannel* Byte = static_cast<FMovieSceneByteChannel*>(Channel);
				Byte->GetData().UpdateOrAddKey(KeyTime, *DesiredValue.Cast<uint8>());
			}
			else if (DesiredValue.GetType() == EControlRigControlType::Parameter_Integer)
			{
				FMovieSceneIntegerChannel* Int = static_cast<FMovieSceneIntegerChannel*>(Channel);
				Int->GetData().UpdateOrAddKey(KeyTime, *DesiredValue.Cast<int32>());
			}
			else if (ChannelHandle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName())
			{
				const int32 ControlChannelIndex = ChannelMetaData.GetChannelIndex();

				float NewValue = 0.f;
				if (DesiredValue.GetType() == EControlRigControlType::Parameter_Scalar)
				{
					NewValue = *DesiredValue.Cast<float>();
				}
				else if (DesiredValue.GetType() == EControlRigControlType::Parameter_Vector)
				{
					NewValue = (*DesiredValue.Cast<FVector3f>())[ControlChannelIndex];
				}
				else if (DesiredValue.GetType() == EControlRigControlType::Parameter_Transform)
				{
					FEulerTransform Transform = (*DesiredValue.Cast<FEulerTransform>());
					if (ControlChannelIndex < 3)
					{
						NewValue = Transform.Location[ControlChannelIndex];
					}
					else if (ControlChannelIndex < 6)
					{
						NewValue = Transform.Rotation.Euler()[ControlChannelIndex - 3];
					}
					else
					{
						NewValue = Transform.Scale[ControlChannelIndex - 6];
					}
				}

				FMovieSceneFloatChannel* FloatChannel = static_cast<FMovieSceneFloatChannel*>(Channel);

				int32 KeyIndex = FloatChannel->GetData().FindKey(KeyTime);
				if (KeyIndex == INDEX_NONE)
				{
					switch (InterpMode)
					{
					case EMovieSceneKeyInterpolation::Linear:
						KeyIndex = FloatChannel->AddLinearKey(KeyTime, NewValue);
						break;
					case EMovieSceneKeyInterpolation::Constant:
						KeyIndex = FloatChannel->AddConstantKey(KeyTime, NewValue);
						break;
					case EMovieSceneKeyInterpolation::Auto:
						KeyIndex = FloatChannel->AddCubicKey(KeyTime, NewValue, ERichCurveTangentMode::RCTM_Auto);
						break;
					case EMovieSceneKeyInterpolation::SmartAuto:
					default:
						KeyIndex = FloatChannel->AddCubicKey(KeyTime, NewValue, ERichCurveTangentMode::RCTM_SmartAuto);
						break;
					}
				}
				else
				{
					FloatChannel->GetData().GetValues()[KeyIndex].Value = NewValue;
				}

				if (OutResults && KeyIndex != INDEX_NONE)
				{
					FAddKeyResult Result;
					Result.KeyArea = KeyArea;
					Result.KeyHandle = FloatChannel->GetData().GetHandle(KeyIndex);
					OutResults->Add(Result);
				}
			}
		}
	}
}

UE::MovieScene::FControlRigParameterValues FControlRigParameterTrackEditor::GetRecomposedControlValues(UMovieSceneControlRigParameterTrack* Track, UMovieSceneControlRigParameterSection* Section, FFrameNumber KeyTime)
{
	using namespace UE::MovieScene;
	using namespace UE::Sequencer;

	check(Track && Section);

	FControlRigParameterValues CurrentValues(EControlRigParameterBufferIndexStability::Unstable);
	CurrentValues.PopulateFrom(Track->GetControlRig());

	// Create the interrogator.
	FSystemInterrogator Interrogator;
	Interrogator.TrackImportedEntities(true);

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Interrogator.GetLinker()->EntityManager);

	// Run an interrogation on the track at the specified time.
	FInterrogationKey InterrogationKey(FInterrogationKey::Default());
	FInterrogationChannel InterrogationChannel = Interrogator.AllocateChannel(Track->GetControlRig(), FMovieScenePropertyBinding());
	InterrogationKey.Channel = InterrogationChannel;
	Interrogator.ImportTrack(Track, InterrogationChannel);

	Interrogator.AddInterrogation(KeyTime);
	Interrogator.Update();

	UMovieScenePiecewiseDoubleBlenderSystem* BlenderSystem = Interrogator.GetLinker()->FindSystem<UMovieScenePiecewiseDoubleBlenderSystem>();
	if (!BlenderSystem)
	{
		return CurrentValues;
	}

	Interrogator.GetLinker()->EntityManager.LockDown();
	ON_SCOPE_EXIT
	{
		Interrogator.GetLinker()->EntityManager.ReleaseLockDown();
	};

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	const float SectionWeight = Section->GetTotalWeightValue(KeyTime);

	auto RecomposeValue = [&Interrogator, InterrogationKey, BlenderSystem, BuiltInComponents](FMovieSceneEntityID EntityID, double Value, const double* InitialValue, FComponentTypeID ParameterTag, TComponentTypeID<double> ResultComponent, const FControlRigChannelMetaData& ChannelMetaData)
	{
		check(BlenderSystem);
		TOptionalComponentReader<FMovieSceneBlendChannelID> BlendChannelInput = Interrogator.GetLinker()->EntityManager.ReadComponent(EntityID, BuiltInComponents->BlendChannelInput);
		if (BlendChannelInput)
		{
			FAlignedDecomposedValue AlignedOutput;

			FValueDecompositionParams Params;
			Params.Query.Entities = MakeArrayView(&EntityID, 1);
			Params.Query.bConvertFromSourceEntityIDs = false;
			Params.DecomposeBlendChannel = BlendChannelInput->ChannelID;
			Params.ResultComponentType = ResultComponent;
			Params.PropertyTag = ParameterTag;

			FGraphEventRef Task = BlenderSystem->DispatchDecomposeTask(Params, &AlignedOutput);
			if (Task)
			{
				FTaskGraphInterface::Get().WaitUntilTaskCompletes(Task, ENamedThreads::GameThread);
			}

			return AlignedOutput.Value.Recompose(EntityID, Value, InitialValue);
		}
		return Value;
	};


	TPropertyValueStorage<FFloatParameterTraits>*     InitialScalarStorage    = FInitialValueCache::GetGlobalInitialValues()->FindStorage(TracksComponents->Parameters.Scalar);
	TPropertyValueStorage<FVector3ParameterTraits>*   InitialVectorStorage    = FInitialValueCache::GetGlobalInitialValues()->FindStorage(TracksComponents->Parameters.Vector3);
	TPropertyValueStorage<FTransformParameterTraits>* InitialTransformStorage = FInitialValueCache::GetGlobalInitialValues()->FindStorage(TracksComponents->Parameters.Transform);

	// ------------------------------------------------------------------------------------------------------
	// Recompose Scalars
	for (const FScalarParameterNameAndCurve& Scalar : Section->GetScalarParameterNamesAndCurves())
	{
		FControlRigValueView Value = CurrentValues.Find(Scalar.ParameterName, EControlRigControlType::Parameter_Scalar);
		if (Value)
		{
			FControlRigChannelMetaData ChannelMetaData = Section->GetChannelMetaData(&Scalar.ParameterCurve);
			check(ChannelMetaData);

			FMovieSceneEntityID Entity = Interrogator.FindEntityFromOwner(InterrogationKey, Section, ChannelMetaData.GetEntitySystemID());
			if (Entity)
			{
				const double* InitialValue = InitialScalarStorage ? InitialScalarStorage->FindCachedValue(Track->GetControlRig(), ChannelMetaData.GetControlName()) : nullptr;
				*Value.Cast<float>() = (float)RecomposeValue(Entity, *Value.Cast<float>(), InitialValue, TracksComponents->Parameters.Scalar.PropertyTag, BuiltInComponents->DoubleResult[0], ChannelMetaData);
			}
		}
	}

	// ------------------------------------------------------------------------------------------------------
	// Recompose Vectors
	for (const FVectorParameterNameAndCurves& Vector : Section->GetVectorParameterNamesAndCurves())
	{
		FControlRigValueView Value = CurrentValues.Find(Vector.ParameterName, EControlRigControlType::Parameter_Vector);
		if (Value)
		{
			FControlRigChannelMetaData ChannelMetaData = Section->GetChannelMetaData(&Vector.XCurve);
			check(ChannelMetaData);

			FMovieSceneEntityID Entity = Interrogator.FindEntityFromOwner(InterrogationKey, Section, ChannelMetaData.GetEntitySystemID());
			if (Entity)
			{
				const FFloatIntermediateVector* InitialValue = InitialVectorStorage ? InitialVectorStorage->FindCachedValue(Track->GetControlRig(), ChannelMetaData.GetControlName()) : nullptr;
				FVector3f* ValueAsVector = Value.Cast<FVector3f>();

				FComponentTypeID PropertyTag = TracksComponents->Parameters.Vector3.PropertyTag;
				ValueAsVector->X = (float)RecomposeValue(Entity, ValueAsVector->X, InitialValue ? &InitialValue->X : nullptr, PropertyTag, BuiltInComponents->DoubleResult[0], ChannelMetaData);
				ValueAsVector->Y = (float)RecomposeValue(Entity, ValueAsVector->Y, InitialValue ? &InitialValue->Y : nullptr, PropertyTag, BuiltInComponents->DoubleResult[1], ChannelMetaData);
				ValueAsVector->Z = (float)RecomposeValue(Entity, ValueAsVector->Z, InitialValue ? &InitialValue->Z : nullptr, PropertyTag, BuiltInComponents->DoubleResult[2], ChannelMetaData);
			}
		}
	}

	// ------------------------------------------------------------------------------------------------------
	// Recompose Vectors
	for (const FTransformParameterNameAndCurves& Transform : Section->GetTransformParameterNamesAndCurves())
	{
		FControlRigValueView Value = CurrentValues.Find(Transform.ParameterName, EControlRigControlType::Parameter_Transform);
		if (Value)
		{
			FControlRigChannelMetaData ChannelMetaData = Section->GetChannelMetaData(&Transform.Translation[0]);
			check(ChannelMetaData);

			FMovieSceneEntityID Entity = Interrogator.FindEntityFromOwner(InterrogationKey, Section, ChannelMetaData.GetEntitySystemID());
			if (Entity)
			{
				const FIntermediate3DTransform* InitialValue = InitialTransformStorage ? InitialTransformStorage->FindCachedValue(Track->GetControlRig(), ChannelMetaData.GetControlName()) : nullptr;

				FEulerTransform* ValueAsTransform = Value.Cast<FEulerTransform>();

				FComponentTypeID PropertyTag = TracksComponents->Parameters.Transform.PropertyTag;

				ValueAsTransform->Location.X     = RecomposeValue(Entity, ValueAsTransform->Location.X,     InitialValue ? &InitialValue->T_X : nullptr, PropertyTag, BuiltInComponents->DoubleResult[0], ChannelMetaData);
				ValueAsTransform->Location.Y     = RecomposeValue(Entity, ValueAsTransform->Location.Y,     InitialValue ? &InitialValue->T_Y : nullptr, PropertyTag, BuiltInComponents->DoubleResult[1], ChannelMetaData);
				ValueAsTransform->Location.Z     = RecomposeValue(Entity, ValueAsTransform->Location.Z,     InitialValue ? &InitialValue->T_Z : nullptr, PropertyTag, BuiltInComponents->DoubleResult[2], ChannelMetaData);

				ValueAsTransform->Rotation.Roll  = RecomposeValue(Entity, ValueAsTransform->Rotation.Roll,  InitialValue ? &InitialValue->R_X : nullptr, PropertyTag, BuiltInComponents->DoubleResult[3], ChannelMetaData);
				ValueAsTransform->Rotation.Pitch = RecomposeValue(Entity, ValueAsTransform->Rotation.Pitch, InitialValue ? &InitialValue->R_Y : nullptr, PropertyTag, BuiltInComponents->DoubleResult[4], ChannelMetaData);
				ValueAsTransform->Rotation.Yaw   = RecomposeValue(Entity, ValueAsTransform->Rotation.Yaw,   InitialValue ? &InitialValue->R_Z : nullptr, PropertyTag, BuiltInComponents->DoubleResult[5], ChannelMetaData);

				ValueAsTransform->Scale.X        = RecomposeValue(Entity, ValueAsTransform->Scale.X,        InitialValue ? &InitialValue->S_X : nullptr, PropertyTag, BuiltInComponents->DoubleResult[6], ChannelMetaData);
				ValueAsTransform->Scale.Y        = RecomposeValue(Entity, ValueAsTransform->Scale.Y,        InitialValue ? &InitialValue->S_Y : nullptr, PropertyTag, BuiltInComponents->DoubleResult[7], ChannelMetaData);
				ValueAsTransform->Scale.Z        = RecomposeValue(Entity, ValueAsTransform->Scale.Z,        InitialValue ? &InitialValue->S_Z : nullptr, PropertyTag, BuiltInComponents->DoubleResult[8], ChannelMetaData);
			}
		}
	}

	return CurrentValues;
}

void FControlRigParameterTrackEditor::BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* InTrack)
{
	bool bSectionAdded;
	UMovieSceneControlRigParameterTrack* Track = Cast< UMovieSceneControlRigParameterTrack>(InTrack);
	if (!Track || Track->GetControlRig() == nullptr)
	{
		return;
	}

	UMovieSceneControlRigParameterSection* SectionToKey = Cast<UMovieSceneControlRigParameterSection>(Track->GetSectionToKey());
	if (SectionToKey == nullptr)
	{
		SectionToKey = Cast<UMovieSceneControlRigParameterSection>(Track->FindOrAddSection(0, bSectionAdded));
	}
	if (!SectionToKey)
	{
		return;
	}

	
	// Check if the selected element is a section of the track
	bool bIsSection = Track->GetAllSections().Num() > 1;
	if (bIsSection)
	{
		TArray<TWeakObjectPtr<UObject>> TrackSections;
		for (UE::Sequencer::TViewModelPtr<UE::Sequencer::ISectionOwnerExtension> SectionOwnerExtension : GetSequencer()->GetViewModel()->GetSelection()->Outliner.Filter<UE::Sequencer::ISectionOwnerExtension>())
		{
			for (UMovieSceneSection* Section : SectionOwnerExtension->GetSections())
			{
				TrackSections.Add(Section);
			}
		}
		bIsSection = TrackSections.Num() > 0;
	}
	
	TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels = Track->GetNodeAndChannelMappings(SectionToKey);

	MenuBuilder.BeginSection("Control Rig IO", LOCTEXT("ControlRigIO", "Control Rig I/O"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ImportControlRigFBX", "Import Control Rig FBX"),
			LOCTEXT("ImportControlRigFBXTooltip", "Import Control Rig animation from FBX"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::ImportFBX, Track, SectionToKey, NodeAndChannels)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ExportControlRigFBX", "Export Control Rig FBX"),
			LOCTEXT("ExportControlRigFBXTooltip", "Export Control Rig animation to FBX"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::ExportFBX, Track, SectionToKey)));
	}
	MenuBuilder.EndSection();

	if (!bIsSection)
	{
		MenuBuilder.BeginSection("Control Rig", LOCTEXT("ControlRig", "Control Rig"));
		{
			// Don't show priority order for child tracks, it's handled by the layers in the anim mixer
			if (Track->ParentTrack == nullptr)
			{
				MenuBuilder.AddWidget(
					SNew(SSpinBox<int32>)
					.MinValue(0)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.ToolTipText(LOCTEXT("OrderTooltip", "Order for this Control Rig to evaluate compared to others on the same binding, higher number means earlier evaluation"))
					.Value_Lambda([Track]() { return Track->GetPriorityOrder(); })
					.OnValueChanged_Lambda([Track](int32 InValue) { Track->SetPriorityOrder(InValue); })
					,
					LOCTEXT("Order", "Order")
				);
			}

			if (CVarEnableAdditiveControlRigs->GetBool())
			{
				MenuBuilder.AddMenuEntry(
					   LOCTEXT("ConvertIsLayeredControlRig", "Convert To Layered"),
					   LOCTEXT("ConvertIsLayeredControlRigToolTip", "Converts the Control Rig from an Absolute rig to a Layered rig"),
					   FSlateIcon(),
					   FUIAction(
						   FExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::ConvertIsLayered, Track),
						   FCanExecuteAction(),
						   FIsActionChecked::CreateRaw(this, &FControlRigParameterTrackEditor::IsLayered, Track)
					   ),
					   NAME_None,
					   EUserInterfaceActionType::ToggleButton);
			}


			MenuBuilder.AddMenuEntry(
				   LOCTEXT("RecreateControlRigWithNewSettingsSettings", "Recreate Control Rig With New Settings"),
				   LOCTEXT("RecreateControlRigWithNewSettingsSettingsToolTip", "Recreate Control Rig With New Settings"),
				   FSlateIcon(),
				   FUIAction(
					   FExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::RecreateControlRigWithNewSettings, Track),
					   FCanExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::CanRecreateControlRigWithNewSettings, Track)
				   ),
				   NAME_None,
				   EUserInterfaceActionType::Button);
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.AddMenuSeparator();

	if (UFKControlRig* AutoRig = Cast<UFKControlRig>(Track->GetControlRig()))
	{
		MenuBuilder.BeginSection("FK Control Rig", LOCTEXT("FKControlRig", "FK Control Rig"));
		{

			MenuBuilder.AddMenuEntry(
				LOCTEXT("SelectBonesToAnimate", "Select Bones Or Curves To Animate"),
				LOCTEXT("SelectBonesToAnimateToolTip", "Select which bones or curves you want to directly animate"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::SelectFKBonesToAnimate, AutoRig, Track)));
		}
		MenuBuilder.EndSection();

		MenuBuilder.AddMenuSeparator();
	}
	else if (UControlRig* LayeredRig = Cast<UControlRig>(Track->GetControlRig()))
	{
		if (LayeredRig->IsAdditive())
		{
			MenuBuilder.BeginSection("Layered Control Rig", LOCTEXT("LayeredControlRig", "Layered Control Rig"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Bake Inverted Pose", "Bake Inverted Pose"),
					LOCTEXT("BakeInvertedPoseToolTip", "Bake inversion of the input pose into the rig"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw(this, &FControlRigParameterTrackEditor::BakeInvertedPose, LayeredRig, Track)));
			}
			MenuBuilder.EndSection();
			MenuBuilder.AddMenuSeparator();
		}
	}

}

bool FControlRigParameterTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	if (!IControlRigEditorAssetInterface::IsAControlRigAsset(Asset))
	{
		return false;
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return false;
	}
	
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	FControlRigAssetStrongReference RigSource;
	USkeletalMesh* SkeletalMesh = nullptr;
	if (FControlRigAssetInterfacePtr EditorAsset = Asset)
	{
		RigSource = EditorAsset->GetControlRigAssetReference();
		SkeletalMesh = EditorAsset->GetPreviewMesh();
	}
	else if (TScriptInterface<IControlRigRuntimeAssetInterface> RuntimeAsset = Asset)
	{
		RigSource = RuntimeAsset.GetObject();
		SkeletalMesh = RuntimeAsset->GetPreviewSkeletalMesh().Get();
	}
	else
	{
		return false;
	}

	if (!SkeletalMesh)
	{
		FNotificationInfo Info(LOCTEXT("NoPreviewMesh", "Control rig has no preview mesh to create a spawnable skeletal mesh actor from"));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info)->SetCompletionState(SNotificationItem::CS_Fail);
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddControlRigAsset", "Add Control Rig"));

	UE::Sequencer::FCreateBindingParams CreateBindingParams;
	CreateBindingParams.bSpawnable = true;
	CreateBindingParams.bAllowCustomBinding = true;
	FGuid NewGuid = Sequencer->CreateBinding(*ASkeletalMeshActor::StaticClass(), CreateBindingParams);

	// CreateBinding can fail if spawnables are not allowed
	if (!NewGuid.IsValid())
	{
		return false;
	}
	
	ASkeletalMeshActor* SpawnedSkeletalMeshActor = Cast<ASkeletalMeshActor>(Sequencer->FindSpawnedObjectOrTemplate(NewGuid));
	if (!ensure(SpawnedSkeletalMeshActor))
	{
		return false;
	}

	SpawnedSkeletalMeshActor->GetSkeletalMeshComponent()->SetSkeletalMesh(SkeletalMesh);

	FString NewName = MovieSceneHelpers::MakeUniqueSpawnableName(MovieScene, FName::NameToDisplayString(SkeletalMesh->GetName(), false));
	SpawnedSkeletalMeshActor->SetActorLabel(NewName, false);
	
	// Save Spawnable state as the default (with new name and skeletal mesh asset)
	{
		Sequencer->GetSpawnRegister().SaveDefaultSpawnableState(NewGuid, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState());
	}

	UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(MovieScene->FindTrack(UMovieSceneControlRigParameterTrack::StaticClass(), NewGuid, NAME_None));
	if (Track == nullptr)
	{
		AddControlRig(RigSource, SpawnedSkeletalMeshActor->GetSkeletalMeshComponent(), NewGuid);
	}

	return true;
}

void FControlRigParameterTrackEditor::ImportFBX(UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneControlRigParameterSection* InSection,
	TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels)
{
	if (NodeAndChannels)
	{
		// NodeAndChannels will be deleted later
		MovieSceneToolHelpers::ImportFBXIntoControlRigChannelsWithDialog(GetSequencer().ToSharedRef(), NodeAndChannels);
	}
}

void FControlRigParameterTrackEditor::ExportFBX(UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneControlRigParameterSection* InSection)
{
	if (InTrack && InTrack->GetControlRig())
	{
		// ControlComponentTransformsMapping will be deleted later
		MovieSceneToolHelpers::ExportFBXFromControlRigChannelsWithDialog(GetSequencer().ToSharedRef(), InTrack);
	}
}



class SFKControlRigBoneSelect : public SCompoundWidget, public FGCObject
{
public:

	SLATE_BEGIN_ARGS(SFKControlRigBoneSelect) {}
	SLATE_ATTRIBUTE(UFKControlRig*, AutoRig)
		SLATE_ATTRIBUTE(UMovieSceneControlRigParameterTrack*, Track)
		SLATE_ATTRIBUTE(ISequencer*, Sequencer)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
	{
		AutoRig = InArgs._AutoRig.Get();
		Track = InArgs._Track.Get();
		Sequencer = InArgs._Sequencer.Get();

		this->ChildSlot[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SFKControlRigBoneSelectDescription", "Select Bones You Want To Be Active On The FK Control Rig"))
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SSeparator)
				]
			+ SVerticalBox::Slot()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SBorder)
					[
						SNew(SScrollBox)
						+ SScrollBox::Slot()
				[
					//Save this widget so we can populate it later with check boxes
					SAssignNew(CheckBoxContainer, SVerticalBox)
				]
					]
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked(this, &SFKControlRigBoneSelect::ChangeAllOptions, true)
				.Text(LOCTEXT("FKRigSelectAll", "Select All"))
				]
			+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked(this, &SFKControlRigBoneSelect::ChangeAllOptions, false)
				.Text(LOCTEXT("FKRigDeselectAll", "Deselect All"))
				]

				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SSeparator)
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(8.0f, 4.0f, 8.0f, 4.0f)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked(this, &SFKControlRigBoneSelect::OnButtonClick, true)
				.Text(LOCTEXT("FKRigeOk", "OK"))
				]
			+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.OnClicked(this, &SFKControlRigBoneSelect::OnButtonClick, false)
				.Text(LOCTEXT("FKRigCancel", "Cancel"))
				]
				]
		];
	}


	/**
	* Creates a Slate check box
	*
	* @param	Label		Text label for the check box
	* @param	ButtonId	The ID for the check box
	* @return				The created check box widget
	*/
	TSharedRef<SWidget> CreateCheckBox(const FString& Label, int32 ButtonId)
	{
		return
			SNew(SCheckBox)
			.IsChecked(this, &SFKControlRigBoneSelect::IsCheckboxChecked, ButtonId)
			.OnCheckStateChanged(this, &SFKControlRigBoneSelect::OnCheckboxChanged, ButtonId)
			[
				SNew(STextBlock).Text(FText::FromString(Label))
			];
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(AutoRig);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("SFKControlRigBoneSelect");
	}

	/**
	* Returns the state of the check box
	*
	* @param	ButtonId	The ID for the check box
	* @return				The status of the check box
	*/
	ECheckBoxState IsCheckboxChecked(int32 ButtonId) const
	{
		return CheckBoxInfoMap.FindChecked(ButtonId).bActive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	/**
	* Handler for all check box clicks
	*
	* @param	NewCheckboxState	The new state of the check box
	* @param	CheckboxThatChanged	The ID of the radio button that has changed.
	*/
	void OnCheckboxChanged(ECheckBoxState NewCheckboxState, int32 CheckboxThatChanged)
	{
		FFKBoneCheckInfo& Info = CheckBoxInfoMap.FindChecked(CheckboxThatChanged);
		Info.bActive = !Info.bActive;
	}

	/**
	* Handler for the Select All and Deselect All buttons
	*
	* @param	bNewCheckedState	The new state of the check boxes
	*/
	FReply ChangeAllOptions(bool bNewCheckedState)
	{
		for (TPair<int32, FFKBoneCheckInfo>& Pair : CheckBoxInfoMap)
		{
			FFKBoneCheckInfo& Info = Pair.Value;
			Info.bActive = bNewCheckedState;
		}
		return FReply::Handled();
	}

	/**
	* Populated the dialog with multiple check boxes, each corresponding to a bone
	*
	* @param	BoneInfos	The list of Bones to populate the dialog with
	*/
	void PopulateOptions(TArray<FFKBoneCheckInfo>& BoneInfos)
	{
		for (FFKBoneCheckInfo& Info : BoneInfos)
		{
			CheckBoxInfoMap.Add(Info.BoneID, Info);

			CheckBoxContainer->AddSlot()
				.AutoHeight()
				[
					CreateCheckBox(Info.BoneName.GetPlainNameString(), Info.BoneID)
				];
		}
	}


private:

	/**
	* Handles when a button is pressed, should be bound with appropriate EResult Key
	*
	* @param ButtonID - The return type of the button which has been pressed.
	*/
	FReply OnButtonClick(bool bValid)
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		//if ok selected bValid == true
		if (bValid && AutoRig)
		{
			TArray<FFKBoneCheckInfo> BoneCheckArray;
			BoneCheckArray.SetNumUninitialized(CheckBoxInfoMap.Num());
			int32 Index = 0;
			for (TPair<int32, FFKBoneCheckInfo>& Pair : CheckBoxInfoMap)
			{
				FFKBoneCheckInfo& Info = Pair.Value;
				BoneCheckArray[Index++] = Info;

			}
			if (Track && Sequencer)
			{
				TArray<UMovieSceneSection*> Sections = Track->GetAllSections();
				for (UMovieSceneSection* IterSection : Sections)
				{
					UMovieSceneControlRigParameterSection* Section = Cast< UMovieSceneControlRigParameterSection>(IterSection);
					if (Section)
					{
						constexpr bool bReconstructChannelProxy = false;
						for (const FFKBoneCheckInfo& Info : BoneCheckArray)
						{
							Section->SetControlNameMask(Info.BoneName, Info.bActive, bReconstructChannelProxy);
						}
						Section->ReconstructChannelProxy();
					}
				}

				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
			}
			AutoRig->SetControlActive(BoneCheckArray);
		}
		return bValid ? FReply::Handled() : FReply::Unhandled();
	}

	/** The slate container that the bone check boxes get added to */
	TSharedPtr<SVerticalBox>	 CheckBoxContainer;
	/** Store the check box state for each bone */
	TMap<int32, FFKBoneCheckInfo> CheckBoxInfoMap;

	TObjectPtr<UFKControlRig> AutoRig;
	UMovieSceneControlRigParameterTrack* Track;
	ISequencer* Sequencer;
};

void FControlRigParameterTrackEditor::SelectFKBonesToAnimate(UFKControlRig* AutoRig, UMovieSceneControlRigParameterTrack* Track)
{
	if (AutoRig)
	{
		const FText TitleText = LOCTEXT("SelectBonesOrCurvesToAnimate", "Select Bones Or Curves To Animate");

		// Create the window to choose our options
		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(TitleText)
			.HasCloseButton(true)
			.SizingRule(ESizingRule::UserSized)
			.ClientSize(FVector2D(400.0f, 200.0f))
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.SupportsMinimize(false);

		TSharedRef<SFKControlRigBoneSelect> DialogWidget = SNew(SFKControlRigBoneSelect)
			.AutoRig(AutoRig)
			.Track(Track)
			.Sequencer(GetSequencer().Get());

		TArray<FName> ControlRigNames = AutoRig->GetControlNames();
		TArray<FFKBoneCheckInfo> BoneInfos;
		for (int32 Index = 0; Index < ControlRigNames.Num(); ++Index)
		{
			FFKBoneCheckInfo Info;
			Info.BoneID = Index;
			Info.BoneName = ControlRigNames[Index];
			Info.bActive = AutoRig->GetControlActive(Index);
			BoneInfos.Add(Info);
		}

		DialogWidget->PopulateOptions(BoneInfos);

		Window->SetContent(DialogWidget);
		FSlateApplication::Get().AddWindow(Window);
	}

	//reconstruct all channel proxies TODO or not to do that is the question
}


TOptional<FBakingAnimationKeySettings> SCollapseControlsWidget::CollapseControlsSettings;

void SCollapseControlsWidget::Construct(const FArguments& InArgs)
{
	Sequencer = InArgs._Sequencer;

	if (CollapseControlsSettings.IsSet() == false)
	{
		TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
		CollapseControlsSettings = FBakingAnimationKeySettings();
		const FFrameRate TickResolution = SequencerPtr->GetFocusedTickResolution();
		const FFrameTime FrameTime = SequencerPtr->GetLocalTime().ConvertTo(TickResolution);
		FFrameNumber CurrentTime = FrameTime.GetFrame();

		TRange<FFrameNumber> Range = SequencerPtr->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
		TArray<FFrameNumber> Keys;
		TArray < FKeyHandle> KeyHandles;

		CollapseControlsSettings.GetValue().StartFrame = Range.GetLowerBoundValue();
		CollapseControlsSettings.GetValue().EndFrame = Range.GetUpperBoundValue();
	}


	Settings = MakeShared<TStructOnScope<FBakingAnimationKeySettings>>();
	Settings->InitializeAs<FBakingAnimationKeySettings>(CollapseControlsSettings.GetValue());

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	DetailsView = PropertyEditor.CreateStructureDetailView(ViewArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());
	DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber",
		FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.Pin().ToSharedRef(), &ISequencer::MakeFrameNumberDetailsCustomization));
	DetailsView->SetStructureData(Settings);

	ChildSlot
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f)
				[
					DetailsView->GetWidget().ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.f)
				[

					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.HAlign(HAlign_Fill)
					[
						SNew(SSpacer)
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding(0.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text(LOCTEXT("OK", "OK"))
						.OnClicked_Lambda([this, InArgs]()
							{
								Collapse();
								CloseDialog();
								return FReply::Handled();

							})
						.IsEnabled_Lambda([this]()
							{
								return (Settings.IsValid());
							})
					]
				]
			]
		];
}

void  SCollapseControlsWidget::Collapse()
{
	FBakingAnimationKeySettings* BakeSettings = Settings->Get();
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	CollapseCB.ExecuteIfBound(SequencerPtr, *BakeSettings);
	CollapseControlsSettings = *BakeSettings;
}

class SCollapseControlsWidgetWindow : public SWindow
{
};

FReply SCollapseControlsWidget::OpenDialog(bool bModal)
{
	check(!DialogWindow.IsValid());

	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

	TSharedRef<SCollapseControlsWidgetWindow> Window = SNew(SCollapseControlsWidgetWindow)
		.Title(LOCTEXT("CollapseControls", "Collapse Controls"))
		.CreateTitleBar(true)
		.Type(EWindowType::Normal)
		.SizingRule(ESizingRule::Autosized)
		.ScreenPosition(CursorPos)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		[
			AsShared()
		];

	Window->SetWidgetToFocusOnActivate(AsShared());

	DialogWindow = Window;

	Window->MoveWindowTo(CursorPos);

	if (bModal)
	{
		GEditor->EditorAddModalWindow(Window);
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}

	return FReply::Handled();
}

void SCollapseControlsWidget::CloseDialog()
{
	if (DialogWindow.IsValid())
	{
		DialogWindow.Pin()->RequestDestroyWindow();
		DialogWindow.Reset();
	}
}
///////////////
//end of SCollapseControlsWidget
/////////////////////
struct FKeyAndValuesAtFrame
{
	FFrameNumber Frame;
	TArray<FMovieSceneFloatValue> KeyValues;
	float FinalValue;
};

bool CollapseAllLayersPerKey(TSharedPtr<ISequencer>& SequencerPtr, UMovieSceneTrack* OwnerTrack, const FBakingAnimationKeySettings& InSettings)
{
	if (SequencerPtr.IsValid() && OwnerTrack)
	{
		TArray<UMovieSceneSection*> Sections = OwnerTrack->GetAllSections();
		return MovieSceneToolHelpers::CollapseSection(SequencerPtr, OwnerTrack, Sections, InSettings);
	}
	return false;
}

bool FControlRigParameterTrackEditor::CollapseAllLayers(TSharedPtr<ISequencer>&SequencerPtr, UMovieSceneTrack * OwnerTrack, const FBakingAnimationKeySettings &InSettings)
{
	if (InSettings.BakingKeySettings == EBakingKeySettings::KeysOnly)
	{
		return CollapseAllLayersPerKey(SequencerPtr, OwnerTrack, InSettings);
	}
	else
	{
		if (SequencerPtr.IsValid() && OwnerTrack)
		{
			TArray<UMovieSceneSection*> Sections = OwnerTrack->GetAllSections();
			//make sure right type
			if (Sections.Num() < 1)
			{
				UE_LOGF(LogControlRigEditor, Log, "CollapseAllSections::No sections on track");
				return false;
			}
			if (UMovieSceneControlRigParameterSection* ParameterSection = Cast<UMovieSceneControlRigParameterSection>(Sections[0]))
			{
				if (ParameterSection->GetBlendType().Get() == EMovieSceneBlendType::Absolute)
				{
					FScopedTransaction Transaction(LOCTEXT("CollapseAllSections", "Collapse All Sections"));
					ParameterSection->Modify();
					UControlRig* ControlRig = ParameterSection->GetControlRig();
					FMovieSceneSequenceTransform RootToLocalTransform = SequencerPtr->GetFocusedMovieSceneSequenceTransformUnwarped();

					FFrameNumber StartFrame = InSettings.StartFrame;
					FFrameNumber EndFrame = InSettings.EndFrame;
					TRange<FFrameNumber> Range(StartFrame, EndFrame);
					const FFrameRate& FrameRate = SequencerPtr->GetFocusedDisplayRate();
					const FFrameRate& TickResolution = SequencerPtr->GetFocusedTickResolution();

					//frames and (optional) tangents
					TArray<TPair<FFrameNumber, TArray<FMovieSceneTangentData>>> StoredTangents; //store tangents so we can reset them
					TArray<FFrameNumber> Frames;
					FFrameNumber FrameRateInFrameNumber = TickResolution.AsFrameNumber(FrameRate.AsInterval());
					FrameRateInFrameNumber.Value *= InSettings.FrameIncrement;
					for (FFrameNumber& Frame = StartFrame; Frame <= EndFrame; Frame += FrameRateInFrameNumber)
					{
						Frames.Add(Frame);
					}

					//Store transforms
					TArray<TPair<FName, TArray<FEulerTransform>>> ControlLocalTransforms;
					TArray<FRigControlElement*> Controls;
					ControlRig->GetControlsInOrder(Controls);

					for (FRigControlElement* ControlElement : Controls)
					{
						if (!ControlRig->GetHierarchy()->IsAnimatable(ControlElement))
						{
							continue;
						}
						TPair<FName, TArray<FEulerTransform>> NameTransforms;
						NameTransforms.Key = ControlElement->GetFName();
						NameTransforms.Value.SetNum(Frames.Num());
						ControlLocalTransforms.Add(NameTransforms);
					}

					FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

					//get all of the local 
					int32 Index = 0;
					for (Index = 0; Index < Frames.Num(); ++Index)
					{
						const FFrameNumber& FrameNumber = Frames[Index];
						FFrameTime GlobalTime = LocalToRootTransform.TryTransformTime(FrameNumber).Get(FrameNumber);

						FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), SequencerPtr->GetPlaybackStatus()).SetHasJumped(true);

						SequencerPtr->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
						ControlRig->Evaluate_AnyThread();
						for (TPair<FName, TArray<FEulerTransform>>& TrailControlTransform : ControlLocalTransforms)
						{
							FEulerTransform EulerTransform(ControlRig->GetControlLocalTransform(TrailControlTransform.Key));
							FRigElementKey ControlKey(TrailControlTransform.Key, ERigElementType::Control);
							EulerTransform.Rotation = ControlRig->GetHierarchy()->GetControlPreferredRotator(ControlKey);
							TrailControlTransform.Value[Index] = EulerTransform;
						}
					}
					//delete other sections
					OwnerTrack->Modify();
					for (Index = Sections.Num() - 1; Index >= 0; --Index)
					{
						if (Sections[Index] != ParameterSection)
						{
							OwnerTrack->RemoveSectionAt(Index);
						}
					}

					//remove all keys, except Space Channels, from the Section.
					ParameterSection->RemoveAllKeys(false /*bIncludedSpaceKeys*/);

					FRigControlModifiedContext Context;
					Context.SetKey = EControlRigSetKey::Always;

					FScopedSlowTask Feedback(Frames.Num(), LOCTEXT("CollapsingSections", "Collapsing Sections"));
					Feedback.MakeDialog(true);

					const EMovieSceneKeyInterpolation InterpMode = SequencerPtr->GetSequencerSettings()->GetKeyInterpolation();
					Index = 0;
					for (Index = 0; Index < Frames.Num(); ++Index)
					{
						Feedback.EnterProgressFrame(1, LOCTEXT("CollapsingSections", "Collapsing Sections"));
						const FFrameNumber& FrameNumber = Frames[Index];
						Context.LocalTime = TickResolution.AsSeconds(FFrameTime(FrameNumber));
						//need to do the twice hack since controls aren't really in order
						for (int32 TwiceHack = 0; TwiceHack < 2; ++TwiceHack)
						{
							for (TPair<FName, TArray<FEulerTransform>>& TrailControlTransform : ControlLocalTransforms)
							{
								FRigElementKey ControlKey(TrailControlTransform.Key, ERigElementType::Control);
								ControlRig->GetHierarchy()->SetControlPreferredRotator(ControlKey, TrailControlTransform.Value[Index].Rotation);
								FTransform Transform(TrailControlTransform.Value[Index].ToFTransform());
								ControlRig->SetControlLocalTransform(TrailControlTransform.Key, Transform, false, Context, false /*undo*/, true/* bFixEulerFlips*/);
								ControlRig->GetHierarchy()->SetControlPreferredRotator(ControlKey, TrailControlTransform.Value[Index].Rotation);

							}
						}
						ControlRig->Evaluate_AnyThread();
						ParameterSection->RecordControlRigKey(FrameNumber, true, InterpMode);

						if (Feedback.ShouldCancel())
						{
							Transaction.Cancel();
							SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
							return false;
						}
					}
					if (InSettings.bReduceKeys)
					{
						FKeyDataOptimizationParams Params;
						Params.bAutoSetInterpolation = true;
						Params.Tolerance = InSettings.Tolerance;
						FMovieSceneChannelProxy& ChannelProxy = ParameterSection->GetChannelProxy();
						TArrayView<FMovieSceneFloatChannel*> FloatChannels = ChannelProxy.GetChannels<FMovieSceneFloatChannel>();

						for (FMovieSceneFloatChannel* Channel : FloatChannels)
						{
							Channel->Optimize(Params); //should also auto tangent
						}
					}
					//reset everything back
					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
					return true;
				}
				else
				{
					UE_LOGF(LogControlRigEditor, Log, "CollapseAllSections:: First section is not additive");
					return false;
				}
			}
			else
			{
				UE_LOGF(LogControlRigEditor, Log, "CollapseAllSections:: No Control Rig section");
				return false;
			}
		}
		UE_LOGF(LogControlRigEditor, Log, "CollapseAllSections:: Sequencer or track is invalid");
	}
	return false;
}

void FControlRigParameterSection::CollapseAllLayers()
{
	if (UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get()))
	{
		UMovieSceneTrack* OwnerTrack = ParameterSection->GetTypedOuter<UMovieSceneTrack>();
		FCollapseControlsCB CollapseCB = FCollapseControlsCB::CreateLambda([this,OwnerTrack](TSharedPtr<ISequencer>& InSequencer, const FBakingAnimationKeySettings& InSettings)
		{
			FControlRigParameterTrackEditor::CollapseAllLayers(InSequencer, OwnerTrack, InSettings);
		});

		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
		TSharedRef<SCollapseControlsWidget> BakeWidget =
			SNew(SCollapseControlsWidget)
			.Sequencer(Sequencer);

		BakeWidget->SetCollapseCB(CollapseCB);
		BakeWidget->OpenDialog(false);
	}
}

void FControlRigParameterSection::KeyZeroValue()
{
	UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	FScopedTransaction Transaction(LOCTEXT("KeyZeroValue", "Key Zero Value"));
	ParameterSection->Modify();
	FFrameTime Time = SequencerPtr->GetLocalTime().Time;
	EMovieSceneKeyInterpolation DefaultInterpolation = SequencerPtr->GetKeyInterpolation();
	ParameterSection->KeyZeroValue(Time.GetFrame(), DefaultInterpolation, true);
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

void FControlRigParameterSection::KeyWeightValue(float Val)
{
	UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	FScopedTransaction Transaction(LOCTEXT("KeyWeightZero", "Key Weight Zero"));
	ParameterSection->Modify();
	EMovieSceneTransformChannel Channels = ParameterSection->GetTransformMask().GetChannels();
	if ((Channels & EMovieSceneTransformChannel::Weight) == EMovieSceneTransformChannel::None)
	{
		ParameterSection->SetTransformMask(ParameterSection->GetTransformMask().GetChannels() | EMovieSceneTransformChannel::Weight);
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
	FFrameTime Time = SequencerPtr->GetLocalTime().Time;
	EMovieSceneKeyInterpolation DefaultInterpolation = SequencerPtr->GetKeyInterpolation();
	ParameterSection->KeyWeightValue(Time.GetFrame(), DefaultInterpolation, Val);
	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);
}

void FControlRigParameterSection::BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding)
{
	UMovieSceneControlRigParameterSection* const ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	if (!IsValid(ParameterSection))
	{
		return;
	}

	UControlRig* const ControlRig = ParameterSection->GetControlRig();
	if (!IsValid(ControlRig))
	{
		return;
	}

	UFKControlRig* AutoRig = Cast<UFKControlRig>(ControlRig);
	if (AutoRig || ControlRig->SupportsEvent(FRigUnit_InverseExecution::EventName))
	{
		UObject* BoundObject = nullptr;
		USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(InObjectBinding, &BoundObject, WeakSequencer.Pin());

		if (Skeleton)
		{
			// Load the asset registry module
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			// Collect a full list of assets with the specified class
			TArray<FAssetData> AssetDataList;
			AssetRegistryModule.Get().GetAssetsByClass(UAnimSequenceBase::StaticClass()->GetClassPathName(), AssetDataList, true);

			if (AssetDataList.Num())
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("ImportAnimSequenceIntoThisSection", "Import Anim Sequence Into This Section"), NSLOCTEXT("Sequencer", "ImportAnimSequenceIntoThisSectionTP", "Import Anim Sequence Into This Section"),
					FNewMenuDelegate::CreateRaw(this, &FControlRigParameterSection::LoadAnimationIntoSection, InObjectBinding, Skeleton, ParameterSection)
				);
			}
		}
	}
	TArray<FRigControlElement*> Controls;
	ControlRig->GetControlsInOrder(Controls);

	auto MakeUIAction = [this, InObjectBinding](EMovieSceneTransformChannel ChannelsToToggle)
	{
		return FUIAction(
			FExecuteAction::CreateLambda([this, InObjectBinding, ChannelsToToggle]
				{
					const TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
					if (!SequencerPtr)
					{
						return;
					}

					UMovieSceneControlRigParameterSection* const ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
					if (!IsValid(ParameterSection))
					{
						return;
					}

					FScopedTransaction Transaction(LOCTEXT("SetActiveChannelsTransaction", "Set Active Channels"));
					ParameterSection->Modify();
					EMovieSceneTransformChannel Channels = ParameterSection->GetTransformMask().GetChannels();

					if (EnumHasAllFlags(Channels, ChannelsToToggle) || (Channels & ChannelsToToggle) == EMovieSceneTransformChannel::None)
					{
						ParameterSection->SetTransformMask(ParameterSection->GetTransformMask().GetChannels() ^ ChannelsToToggle);
					}
					else
					{
						ParameterSection->SetTransformMask(ParameterSection->GetTransformMask().GetChannels() | ChannelsToToggle);
					}

					// Restore pre-animated state for the bound objects so that inactive channels will return to their default values.
					for (TWeakObjectPtr<> WeakObject : SequencerPtr->FindBoundObjects(InObjectBinding, SequencerPtr->GetFocusedTemplateID()))
					{
						if (UObject* Object = WeakObject.Get())
						{
							SequencerPtr->RestorePreAnimatedState();
						}
					}

					SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				}
			),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda([this, ChannelsToToggle]
				{
					const UMovieSceneControlRigParameterSection* const ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
					if (!IsValid(ParameterSection))
					{
						return ECheckBoxState::Unchecked;
					}

					const EMovieSceneTransformChannel Channels = ParameterSection->GetTransformMask().GetChannels();
					if (EnumHasAllFlags(Channels, ChannelsToToggle))
					{
						return ECheckBoxState::Checked;
					}
					else if (EnumHasAnyFlags(Channels, ChannelsToToggle))
					{
						return ECheckBoxState::Undetermined;
					}
					return ECheckBoxState::Unchecked;
				})
			);
	};
	
	UMovieSceneControlRigParameterTrack* Track = ParameterSection->GetTypedOuter<UMovieSceneControlRigParameterTrack>();
	if (Track)
	{
		TArray<UMovieSceneSection*> Sections = Track->GetAllSections();
		//If Base Absolute section 
		if (ParameterSection->GetBlendType().Get() == EMovieSceneBlendType::Absolute && Sections[0] == ParameterSection)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("AnimationLayers", "Animation Layers"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("CollapseAllSections", "Collapse All Sections"),
					LOCTEXT("CollapseAllSections_ToolTip", "Collapse all sections onto this section"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this] { CollapseAllLayers(); }))
				);
			}
		}
		if (ParameterSection->GetBlendType().Get() == EMovieSceneBlendType::Additive)
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("AnimationLayers", "Animation Layers"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("KeyZeroValue", "Key Zero Value"),
					LOCTEXT("KeyZeroValue_Tooltip", "Set zero key on all controls in this section"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this] { KeyZeroValue(); }))
				);
			}

			MenuBuilder.AddMenuEntry(
				LOCTEXT("KeyWeightZero", "Key Weight Zero"),
				LOCTEXT("KeyWeightZero_Tooltip", "Key a zero value on the Weight channel"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this] { KeyWeightValue(0.0f); }))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("KeyWeightOne", "Key Weight One"),
				LOCTEXT("KeyWeightOne_Tooltip", "Key a one value on the Weight channel"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this] { KeyWeightValue(1.0f); }))
			);
		}
	}
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("RigSectionActiveChannels", "Active Channels"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SetFromSelectedControls", "Set From Selected Controls"),
			LOCTEXT("SetFromSelectedControls_ToolTip", "Set active channels from the current control selection"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this] { ShowSelectedControlsChannels(); }),
				FCanExecuteAction::CreateLambda([ControlRig] { return ControlRig->CurrentControlSelection().Num() > 0; } )
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowAllControls", "Show All Controls"),
			LOCTEXT("ShowAllControls_ToolTip", "Set active channels from all controls"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this] { return ShowAllControlsChannels(); }))
		);

		const EAxisList::Type XAxis = EAxisList::Forward;
		const EAxisList::Type YAxis = EAxisList::Left;
		const EAxisList::Type ZAxis = EAxisList::Up;

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllTranslation", "Translation"), LOCTEXT("AllTranslation_ToolTip", "Causes this section to affect the translation of rig control transforms"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
				const int32 NumMenuItems = 3;
				TStaticArray<TFunction<void()>, NumMenuItems> MenuConstructors = {
					[&SubMenuBuilder, MakeUIAction, XAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(XAxis),
							FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(XAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationX), NAME_None, EUserInterfaceActionType::ToggleButton);

					},
					[&SubMenuBuilder, MakeUIAction, YAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(YAxis),
							FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(YAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationY), NAME_None, EUserInterfaceActionType::ToggleButton);
					},
					[&SubMenuBuilder, MakeUIAction, ZAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(ZAxis),
						FText::Format(LOCTEXT("ActivateTranslationChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's translation"), AxisDisplayInfo::GetAxisDisplayName(ZAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::TranslationZ), NAME_None, EUserInterfaceActionType::ToggleButton);
					}
				};

				const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
				for (int32 MenuItemIndex = 0; MenuItemIndex < NumMenuItems; MenuItemIndex++)
				{
					const int32 SwizzledComponentIndex = Swizzle[MenuItemIndex];
					MenuConstructors[SwizzledComponentIndex]();
				}
			}),
			MakeUIAction(EMovieSceneTransformChannel::Translation),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllRotation", "Rotation"), LOCTEXT("AllRotation_ToolTip", "Causes this section to affect the rotation of the rig control transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationX", "Roll"), LOCTEXT("RotationX_ToolTip", "Causes this section to affect the roll channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationX), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationY", "Pitch"), LOCTEXT("RotationY_ToolTip", "Causes this section to affect the pitch channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationY), NAME_None, EUserInterfaceActionType::ToggleButton);
				SubMenuBuilder.AddMenuEntry(
					LOCTEXT("RotationZ", "Yaw"), LOCTEXT("RotationZ_ToolTip", "Causes this section to affect the yaw channel the transform's rotation"),
					FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::RotationZ), NAME_None, EUserInterfaceActionType::ToggleButton);
				}),
			MakeUIAction(EMovieSceneTransformChannel::Rotation),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AllScale", "Scale"), LOCTEXT("AllScale_ToolTip", "Causes this section to affect the scale of the rig control transform"),
			FNewMenuDelegate::CreateLambda([=](FMenuBuilder& SubMenuBuilder) {
				const int32 NumMenuItems = 3;
				TStaticArray<TFunction<void()>, NumMenuItems> MenuConstructors = {
					[&SubMenuBuilder, MakeUIAction, XAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(XAxis),
							FText::Format(LOCTEXT("ActivateScaleChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's scale"), AxisDisplayInfo::GetAxisDisplayName(XAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::ScaleX), NAME_None, EUserInterfaceActionType::ToggleButton);

					},
					[&SubMenuBuilder, MakeUIAction, YAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
							AxisDisplayInfo::GetAxisDisplayName(YAxis),
							FText::Format(LOCTEXT("ActivateScaleChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's scale"), AxisDisplayInfo::GetAxisDisplayName(YAxis)),
							FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::ScaleY), NAME_None, EUserInterfaceActionType::ToggleButton);
					},
					[&SubMenuBuilder, MakeUIAction, ZAxis]()
					{
						SubMenuBuilder.AddMenuEntry(
						AxisDisplayInfo::GetAxisDisplayName(ZAxis),
						FText::Format(LOCTEXT("ActivateScaleChannel_Tooltip", "Causes this section to affect the {0} channel of the transform's scale"), AxisDisplayInfo::GetAxisDisplayName(ZAxis)),
						FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::ScaleZ), NAME_None, EUserInterfaceActionType::ToggleButton);
					}
				};

				const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
				for (int32 MenuItemIndex = 0; MenuItemIndex < NumMenuItems; MenuItemIndex++)
				{
					const int32 SwizzledComponentIndex = Swizzle[MenuItemIndex];
					MenuConstructors[SwizzledComponentIndex]();
				}
			}),
			MakeUIAction(EMovieSceneTransformChannel::Scale),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);

		//mz todo h
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Weight", "Weight"), LOCTEXT("Weight_ToolTip", "Causes this section to be applied with a user-specified weight curve"),
			FSlateIcon(), MakeUIAction(EMovieSceneTransformChannel::Weight), NAME_None, EUserInterfaceActionType::ToggleButton);
	}
	MenuBuilder.EndSection();
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("TimeWarp", "TimeWarp"));
	{
		using namespace UE::Sequencer;

		TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
		FSectionModelStorageExtension* SectionModelStorage = SequencerPtr->GetViewModel()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
		check(SectionModelStorage);

		TSharedPtr<FSectionModel> SectionHandle = SectionModelStorage->FindModelForSection(ParameterSection);
		TWeakViewModelPtr<FViewModel> WeakViewModel = SectionHandle->AsWeak();
		FSequencerUtilities::MakeTimeWarpMenuEntry(MenuBuilder, SectionHandle);

		if (ParameterSection->HasTimeWarp())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DeleteTimeWarp", "Delete Time Warp"),
				LOCTEXT("DeleteTimeWarp_ToolTip", "Delete Time Warp"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]
					{
						if (UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get()))
						{
							FScopedTransaction Transaction(LOCTEXT("DeleteTimeWarp", "Delete Time Warp"));
							ParameterSection->DeleteTimeWarp();
							if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
							{
								SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
							}

						}
					}))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("Bake Time Warp", "Bake Time Warp"),
				LOCTEXT("BakeTimeWarp_ToolTip", "Bake Time Warp and delete it"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this] 
					{
						if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
						{
							if (UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get()))
							{
								FScopedTransaction Transaction(LOCTEXT("BakeTimeWarp", "Bake Time Warp"));
								FControlRigParameterTrackEditor::BakeTimeWarp(SequencerPtr, ParameterSection);
							}
						}
					}))
			);
		}

	}
	MenuBuilder.EndSection();

}

void FControlRigParameterSection::ShowSelectedControlsChannels()
{
	UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	UControlRig* ControlRig = ParameterSection ? ParameterSection->GetControlRig() : nullptr;

	if (ParameterSection && ControlRig && SequencerPtr.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ShowSelecedControlChannels", "Show Selected Control Channels"));
		ParameterSection->Modify();
		ParameterSection->FillControlNameMask(false);

		TArray<FRigControlElement*> Controls;
		ControlRig->GetControlsInOrder(Controls);
		for (const FRigControlElement* RigControl : Controls)
		{
			const FName RigName = RigControl->GetFName();
			if (FChannelMapInfo* pChannelIndex = ParameterSection->ControlChannelMap.Find(RigName))
			{ 
				constexpr bool bReconstructChannelProxy = false;
				const bool bIsSelected = ControlRig->IsControlSelected(RigName);
				ParameterSection->SetControlNameMask(RigName, bIsSelected, bReconstructChannelProxy);
			}
		}
		ParameterSection->ReconstructChannelProxy();
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
}

void FControlRigParameterSection::ShowAllControlsChannels()
{
	UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	if (ParameterSection && SequencerPtr.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("ShowAllControlChannels", "Show All Control Channels"));
		ParameterSection->Modify();
		ParameterSection->FillControlNameMask(true);
		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
}

bool FControlRigParameterSection::RequestDeleteCategory(const TArray<FName>& CategoryNamePaths)
{
	UMovieSceneControlRigParameterSection* ParameterSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	if (ParameterSection && SequencerPtr)
	{
		const FName& ChannelName = CategoryNamePaths.Last();
		const int32 Index = ParameterSection->GetConstraintsChannels().IndexOfByPredicate([ChannelName](const FConstraintAndActiveChannel& InChannel)
			{
				return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->GetFName() == ChannelName : false;
			});
		// remove constraint channel if there are no keys
		const FConstraintAndActiveChannel* ConstraintChannel = Index != INDEX_NONE ? &(ParameterSection->GetConstraintsChannels()[Index]): nullptr;
		if (ConstraintChannel && ConstraintChannel->ActiveChannel.GetNumKeys() == 0)
		{
			if (ParameterSection->TryModify())
			{
				ParameterSection->RemoveConstraintChannel(ConstraintChannel->GetConstraint().Get());
				SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				return true;
			}
		}
	}
	
	return false;
}

bool FControlRigParameterSection::RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePaths)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.IsValid() ? WeakSequencer.Pin() : nullptr;
	UMovieSceneControlRigParameterSection* ControlRigSection = CastChecked<UMovieSceneControlRigParameterSection>(WeakSection.Get());
	
	if (!Sequencer || !ControlRigSection || KeyAreaNamePaths.Num() <= 0)
	{
		return false;
	}

	const FName KeyAreaLeafName = KeyAreaNamePaths.Last();
	if (KeyAreaLeafName == TEXT("TimeWarp") || KeyAreaLeafName == TEXT("PlayRate"))
	{
		ControlRigSection->DeleteTimeWarp();
		return true;
	}

	if (KeyAreaNamePaths.Num() != 2)
	{
		return false;
	}

	// KeyAreaNamePaths[0] is the control's name
	const FName ConstraintName = KeyAreaNamePaths[1];
	UTickableConstraint* Constraint = ControlRigSection->FindConstraint(ConstraintName);
	if (!Constraint)
	{
		return false;
	}

	if (UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteConstraintChannel", "Delete Constraint Channel"));
		if (ControlRigSection->TryModify())
		{
			FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
			if (Controller.RemoveConstraint(Constraint))
			{
				Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
				return true;
			}
		}
	}
	
	return false;
}


void FControlRigParameterSection::LoadAnimationIntoSection(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, USkeleton* Skeleton, UMovieSceneControlRigParameterSection* Section)
{
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
	UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FControlRigParameterSection::OnAnimationAssetSelected, ObjectBinding, Section);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FControlRigParameterSection::OnAnimationAssetEnterPressed, ObjectBinding, Section);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateRaw(this, &FControlRigParameterSection::ShouldFilterAsset);
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequenceBase::StaticClass()->GetClassPathName());
		AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateUObject(Skeleton, &USkeleton::ShouldFilterAsset, TEXT("Skeleton"));
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
}


void FControlRigParameterSection::OnAnimationAssetSelected(const FAssetData& AssetData, FGuid ObjectBinding, UMovieSceneControlRigParameterSection* Section)
{
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();
	TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();

	if (SelectedObject && SelectedObject->IsA(UAnimSequence::StaticClass()) && SequencerPtr.IsValid())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(AssetData.GetAsset());
		UObject* BoundObject = nullptr;
		AcquireSkeletonFromObjectGuid(ObjectBinding, &BoundObject, SequencerPtr);
		USkeletalMeshComponent* SkelMeshComp = AcquireSkeletalMeshFromObject(BoundObject, SequencerPtr);

		if (AnimSequence && SkelMeshComp && AnimSequence->GetDataModel()->GetNumBoneTracks() > 0)
		{
			//if we get a new anim sequence we change the start and end range times so it can be the same as the anim sequence
			static uint32 LastAnimSequenceID = (uint32)(-1);
			ULoadAnimToControlRigSettings * LoadSettings = GetMutableDefault<ULoadAnimToControlRigSettings>();
			if (LoadSettings->bUseCustomTimeRange == false || AnimSequence->GetUniqueID() != LastAnimSequenceID)
			{
				LoadSettings->StartFrame = 0;
				LoadSettings->EndFrame = AnimSequence->GetDataModel()->GetNumberOfFrames();
				LastAnimSequenceID = AnimSequence->GetUniqueID();
			}
			FLoadAnimToControlRigDelegate LoadCallback = FLoadAnimToControlRigDelegate::CreateLambda([this, Section,
				AnimSequence, SkelMeshComp]
				(ULoadAnimToControlRigSettings* LoadSettings)
				{
					if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
					{
						FScopedTransaction Transaction(LOCTEXT("LoadAnimation", "Load Animation"));
						Section->Modify();
						FFrameNumber StartFrame = SequencerPtr->GetLocalTime().Time.GetFrame();
						TOptional<TRange<FFrameNumber>> AnimLoadFrameRamge; //do whole range
						if (LoadSettings->bUseCustomTimeRange)
						{
							TRange<FFrameNumber> Range{ 0 };
							Range.SetLowerBoundValue(LoadSettings->StartFrame);
							Range.SetUpperBoundValue(LoadSettings->EndFrame);
							AnimLoadFrameRamge = Range;
						}
						if (!FControlRigParameterTrackEditor::LoadAnimationIntoSection(SequencerPtr, AnimSequence, SkelMeshComp, StartFrame,
							LoadSettings->bReduceKeys, LoadSettings->SmartReduce, LoadSettings->bResetControls, AnimLoadFrameRamge,
							LoadSettings->bOntoSelectedControls, Section))
						{
							Transaction.Cancel();
						}
						SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
					}
					
				});

			FOnWindowClosed LoadClosedCallback = FOnWindowClosed::CreateLambda([](const TSharedRef<SWindow>&) {});
			FLoadAnimToControlRigDialog::GetLoadAnimParams(LoadCallback, LoadClosedCallback);
		}
	}
}

bool FControlRigParameterSection::ShouldFilterAsset(const FAssetData& AssetData)
{
	// we don't want 

	if (AssetData.AssetClassPath == UAnimMontage::StaticClass()->GetClassPathName())
	{
		return true;
	}

	const FString EnumString = AssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
	if (EnumString.IsEmpty())
	{
		return false;
	}

	UEnum* AdditiveTypeEnum = StaticEnum<EAdditiveAnimationType>();
	return ((EAdditiveAnimationType)AdditiveTypeEnum->GetValueByName(*EnumString) != AAT_None);

}

void FControlRigParameterSection::OnAnimationAssetEnterPressed(const TArray<FAssetData>& AssetData, FGuid  ObjectBinding, UMovieSceneControlRigParameterSection* Section)
{
	if (AssetData.Num() > 0)
	{
		OnAnimationAssetSelected(AssetData[0].GetAsset(), ObjectBinding, Section);
	}
}

FEditorModeTools* FControlRigParameterTrackEditor::GetEditorModeTools() const
{
	if (const TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		if (const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer->GetToolkitHost())
		{
			return &ToolkitHost->GetEditorModeManager();
		}
	}
	
	return nullptr;
}

FControlRigEditMode* FControlRigParameterTrackEditor::GetEditMode(bool bForceActivate /*= false*/) const
{
	if (FEditorModeTools* EditorModeTools = GetEditorModeTools())
	{
		if (bForceActivate && !EditorModeTools->IsModeActive(FControlRigEditMode::ModeName))
		{
			EditorModeTools->ActivateMode(FControlRigEditMode::ModeName);

			FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(EditorModeTools->GetActiveMode(FControlRigEditMode::ModeName));
			if (EditMode && EditMode->GetToolkit().IsValid() == false)
			{
				EditMode->Enter();
			}
		}

		return static_cast<FControlRigEditMode*>(EditorModeTools->GetActiveMode(FControlRigEditMode::ModeName));
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE


