// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "KeyframeTrackEditor.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "AcquiredResources.h"
#include "ContentBrowserDelegates.h"
#include "ControlRigAssetReference.h"
#include "MovieSceneToolHelpers.h"
#include "MovieSceneToolsModule.h"
#include "Engine/EngineTypes.h"
#include "EditorUndoClient.h"
#include "Filters/CurveEditorSmartReduceFilter.h"
#include "ScopedTransaction.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"

class UTickableTransformConstraint;
struct FAssetData;
class FMenuBuilder;
class USkeleton;
class UMovieSceneControlRigParameterSection;
class UFKControlRig;
class FEditorModeTools;
class FControlRigEditMode;
struct FRigControlElement;
struct FRigControlModifiedContext;
struct FMovieSceneControlRigSpaceChannel;
struct FMovieSceneChannel;
struct FKeyAddOrDeleteEventItem;
struct FKeyMoveEventItem;
struct FBakingAnimationKeySettings;
class ISequencer;
class UMovieSceneTrack;
class IStructureDetailsView;
class UConstraintCreationOptions;

namespace UE::MovieScene
{
	struct FControlRigParameterValues;
}

#define UE_API CONTROLRIGEDITOR_API
extern UE_API TAutoConsoleVariable<bool> CVarSelectChildChannels;

//////////////////////////////////////////////////////////////
/// SCollapseControlsWidget
///////////////////////////////////////////////////////////
DECLARE_DELEGATE_TwoParams(FCollapseControlsCB, TSharedPtr<ISequencer>& InSequencer, const FBakingAnimationKeySettings& InSettings);

/** Widget allowing collapsing of controls */
class SCollapseControlsWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCollapseControlsWidget)
		: _Sequencer(nullptr)
		{}
		SLATE_ARGUMENT(TWeakPtr<ISequencer>, Sequencer)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SCollapseControlsWidget() override = default;

	FReply OpenDialog(bool bModal = true);
	void CloseDialog();

	void SetCollapseCB(FCollapseControlsCB& InCB) { CollapseCB = InCB; }
private:
	void Collapse();

	TWeakPtr<ISequencer> Sequencer;
	//static to be reused
	static TOptional<FBakingAnimationKeySettings> CollapseControlsSettings;
	//structonscope for details panel
	TSharedPtr < TStructOnScope<FBakingAnimationKeySettings>> Settings;
	TWeakPtr<SWindow> DialogWindow;
	TSharedPtr<IStructureDetailsView> DetailsView;
	FCollapseControlsCB CollapseCB;
};

/** Per-control data for batched key operations. Caller fills both fields. */
struct FControlRigKeyAddParams
{
	FName               RigControlName;
	FRigControlElement* ControlElement = nullptr;
};

/** All data needed to key a single section: the controls to key and one shared key array
 *  whose indices are set by GetControlRigKeys and consumed by AddKeysToSection. */
struct FControlRigSectionKeyData
{
	TArray<FControlRigKeyAddParams> Params;
	TSharedPtr<FGeneratedTrackKeys> GeneratedKeys; // one instance for all Params in this section
};

/** Maps each section to its key data (params + one shared GeneratedKeys). */
using FControlRigSectionParamMap = TMap<UMovieSceneControlRigParameterSection*, FControlRigSectionKeyData>;

/**
 * Tools for animation tracks
 */
class FControlRigParameterTrackEditor : public FKeyframeTrackEditor<UMovieSceneControlRigParameterTrack>, public IMovieSceneToolsAnimationBakeHelper,
	public FEditorUndoClient, public IMovieSceneAnimMixerItemMenuProvider
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FControlRigParameterTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FControlRigParameterTrackEditor();

	/**
	 * Creates an instance of this class.  Called by a sequencer
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);


	/**
	*
	* Cached version of the current track editor, that lets other animaiton tools call specific functions. in particularly setting keys on
	* current values
	* 
	**/
	static FControlRigParameterTrackEditor* CurrentEditor;
public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void OnInitialize() override;
	virtual void OnRelease() override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual void BuildObjectBindingContextMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool HasTransformKeyBindings() const override { return true; }
	virtual bool CanAddTransformKeysForSelectedObjects() const override;
	virtual void OnAddTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel);
	virtual void OnDeleteTransformKeysForSelectedObjects(EMovieSceneTransformChannel Channel) override;
	virtual bool HasTransformKeyOverridePriority() const override;
	virtual void ObjectImplicitlyAdded(UObject* InObject)  override;
	virtual void ObjectImplicitlyRemoved(UObject* InObject)  override;
	virtual void BuildTrackContextMenu(FMenuBuilder& MenuBuilder, UMovieSceneTrack* InTrack) override;
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	virtual void ProcessKeyOperation(FFrameNumber InKeyTime, const UE::Sequencer::FKeyOperation& Operation, ISequencer& InSequencer, TArray<UE::Sequencer::FAddKeyResult>* OutResults) override;

	//IMovieSceneToolsAnimationBakeHelper
	virtual void PostEvaluation(UMovieScene* MovieScene, FFrameNumber Frame);

	//FEditorUndoClient Interface
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjects) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

	//IMovieSceneAnimMixerItemMenuProvider Interface
	virtual const UClass* GetHandledMixerItemClass() const override;
	virtual void PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex) override;
	virtual int32 GetMixerItemMenuPriority() const override { return 50; }
	virtual void PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu) override;

	//function to set keys on controls
	void SetKeysOnControls(const TMap<UControlRig*, TArray<FRigElementKey>>& Controls, EMovieSceneTransformChannel Channel);
	
	/** Whether the selection is currently being changed by the track editor. */
	bool IsDoingSelection() const { return bIsDoingSelection; }
	
private:

	void ProcessKeyOperation(UObject* ObjectToKey, UMovieSceneControlRigParameterTrack* Track, TArrayView<const UE::Sequencer::FKeySectionOperation> SectionsToKey, ISequencer& InSequencer, FFrameNumber KeyTime, TArray<UE::Sequencer::FAddKeyResult>* OutResults);
	UE::MovieScene::FControlRigParameterValues GetRecomposedControlValues(UMovieSceneControlRigParameterTrack* Track, UMovieSceneControlRigParameterSection* Section, FFrameNumber KeyTime);

	void HandleAddTrackSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback = nullptr);

	void ToggleIsAdditiveControlRig();
	bool IsToggleIsAdditiveControlRig();
	
	void ToggleFilterAssetBySkeleton();
	bool IsToggleFilterAssetBySkeleton();

	void ToggleFilterAssetByAnimatableControls();
	bool IsToggleFilterAssetByAnimatableControls();
	void SelectSequencerNodeInSection(UMovieSceneControlRigParameterSection* ParamSection, const FName& ControlName, bool bSelected);

	/** Control Rig Picked */
	void AddControlRig(FControlRigAssetStrongReference InSource, UObject* BoundActor, FGuid ObjectBinding);
	void AddControlRig(const FControlRigAssetStrongReference InSource, UObject* BoundActor, FGuid ObjectBinding, UControlRig* InExistingControlRig, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback = nullptr);
	void AddControlRig(const FAssetData& InAsset, UObject* BoundActor, FGuid ObjectBinding, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback = nullptr);
	void AddControlRig(const TArray<FAssetData>& InAssets, UObject* BoundActor, FGuid ObjectBinding, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback = nullptr);
	void AddFKControlRig(UObject* BoundActor, FGuid ObjectBinding, TFunction<UMovieSceneControlRigParameterTrack*(FGuid)> InTrackCreationCallback = nullptr);
	void AddControlRigFromComponent(FGuid InGuid);

	bool IsControlRigAllowed(const FAssetData& AssetData, TArray<FControlRigAssetStrongReference> ExistingRigs, USkeleton* Skeleton);
	
	/** Delegate for Selection Changed Event */
	void OnSelectionChanged(TArray<UMovieSceneTrack*> InTracks);

	/** Returns the Edit Mode tools manager hosting the edit mode */
	FEditorModeTools* GetEditorModeTools() const;
	FControlRigEditMode* GetEditMode(bool bForceActivate = false) const;

	/** Delegate for MovieScene Changing so we can see if our track got deleted*/
	void OnSequencerDataChanged(EMovieSceneDataChangeType DataChangeType);

	/** Delegate for when track immediately changes, so we need to do an interaction edit for things like fk/ik*/
	void OnChannelChanged(const TArray<FFrameNumber>& Frames, const FMovieSceneChannelMetaData* MetaData, UMovieSceneSection* InSection);

	/** Delegate for when bindings changed, anim layers need to get updated*/
	void OnMovieSceneBindingsChanged();

	/** Delegate for difference focused movie scene sequence*/
	void OnActivateSequenceChanged(FMovieSceneSequenceIDRef ID);
	FTimerHandle DeactivateModeHandle;
	FTimerHandle ActivateModeHandle;

	/** Actor Added Delegate*/
	void HandleActorAdded(AActor* Actor, FGuid TargetObjectGuid);
	/** Add Control Rig Tracks For Skelmesh Components*/
	void AddTrackForComponent(USceneComponent* Component, FGuid Binding);

	/** Control Rig Delegates*/
	void HandleControlModified(UControlRig* Subject, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context);
	void HandleControlSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected);
	void HandleControlUndoBracket(UControlRig* Subject, bool bOpenUndoBracket);
	void HandleOnPostConstructed(UControlRig* Subject, const FName& InEventName);
	void HandleOnControlRigBound(UControlRig* InControlRig);
	void HandleOnObjectBoundToControlRig(UObject* InObject);

	//if rig not set then we clear delegates for everyone
	void ClearOutAllSpaceAndConstraintDelegates(const UControlRig* InOptionalControlRig = nullptr) const;

	/** SpaceChannel Delegates*/
	void HandleOnSpaceAdded(
		UMovieSceneControlRigParameterSection* Section,
		const FName& ControlName,
		FMovieSceneControlRigSpaceChannel* Channel);
	void HandleSpaceKeyDeleted(
		UMovieSceneControlRigParameterSection* Section,
		FMovieSceneControlRigSpaceChannel* Channel,
		const TArray<FKeyAddOrDeleteEventItem>& DeletedItems) const;
	static void HandleSpaceKeyMoved(
		UMovieSceneControlRigParameterSection* Section,
		FMovieSceneControlRigSpaceChannel* Channel,
		const TArray<FKeyMoveEventItem>& MovedItems);

	/** ConstraintChannel Delegates*/
	void HandleOnConstraintAdded(
		IMovieSceneConstrainedSection* InSection,
		FMovieSceneConstraintChannel* InConstraintChannel);
	void HandleConstraintKeyDeleted(
		IMovieSceneConstrainedSection* InSection,
		const FMovieSceneConstraintChannel* InConstraintChannel,
		const TArray<FKeyAddOrDeleteEventItem>& InDeletedItems) const;
	static void HandleConstraintKeyMoved(
		IMovieSceneConstrainedSection* InSection,
		const FMovieSceneConstraintChannel* InConstraintChannel,
		const TArray<FKeyMoveEventItem>& InMovedItems);
	void HandleConstraintRemoved(IMovieSceneConstrainedSection* InSection);
	void HandleConstraintPropertyChanged(UTickableTransformConstraint* InConstraint, const FPropertyChangedEvent& InPropertyChangedEvent) const;

	/** Select control rig if not selected, select controls from key areas */
	void SelectRigsAndControls(UControlRig* Subject, const TArray<const IKeyArea*>& KeyAreas);

	/** Handle Creation for Skeleton, SkelMeshComp or Actor Owner, they may or may not have a binding, use optional control rig to pick one */
	FMovieSceneTrackEditor::FFindOrCreateHandleResult FindOrCreateHandleToObject(UObject* InObj, UControlRig* InControlRig = nullptr);

	/** Handle Creation for control rig track given the object binding and the control rig */	
	FMovieSceneTrackEditor::FFindOrCreateTrackResult FindOrCreateControlRigTrackForObject(FGuid ObjectBinding, UControlRig* ControlRig, FName PropertyName = NAME_None, bool bCreateTrackIfMissing = true);

	/** Import FBX*/
	void ImportFBX(UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneControlRigParameterSection* InSection, 
		TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels);

	/** Export FBX*/
	void ExportFBX(UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneControlRigParameterSection* InSection);
	
	/** Find Track for given ControlRig*/
	UMovieSceneControlRigParameterTrack* FindTrack(const UControlRig* InControlRig) const;

	/** Select Bones to Animate on FK Rig*/
	void SelectFKBonesToAnimate(UFKControlRig* FKControlRig, UMovieSceneControlRigParameterTrack* Track);

	/** Bake To Control Rig Sub Menu*/
	void BakeToControlRigSubMenu(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, UObject* BoundObject, USkeletalMeshComponent* SkelMeshComp,USkeleton* Skeleton);
	
	/** Bake To Control Rig Sub Menu*/
	void BakeToControlRig(const FControlRigAssetStrongReference& InAssetReference, FGuid ObjectBinding,UObject* BoundObject, USkeletalMeshComponent* SkelMeshComp, USkeleton* Skeleton);

	/** Bake Inersion of Additive Control Rig to Rest Pose*/
	void BakeInvertedPose(UControlRig* InControlRig, UMovieSceneControlRigParameterTrack* Track);

	/** Return true if the control rig is in layered mode */
	bool IsLayered(UMovieSceneControlRigParameterTrack* Track) const;

	/** Convert an absolute control rit to a layered control rig */
	void ConvertIsLayered(UMovieSceneControlRigParameterTrack* Track);

	/** Change rig settings and recreates the rig using the new settings */
	void RecreateControlRigWithNewSettings(UMovieSceneControlRigParameterTrack* Track);
	
	bool CanRecreateControlRigWithNewSettings(UMovieSceneControlRigParameterTrack* Track);

	void CreateAndShowRigSettingsWindow(UMovieSceneControlRigParameterTrack* Track);
	
private:

	struct FRecreateRigOperator : TSharedFromThis<FRecreateRigOperator>
	{
		FInstancedPropertyBag SettingsForNewControlRig;
		
		TWeakPtr<FControlRigParameterTrackEditor> WeakTrackEditor;
		TWeakObjectPtr<UMovieSceneControlRigParameterTrack> WeakTrack;
		TWeakObjectPtr<UControlRig> WeakControlRig;
		
		TWeakPtr<SWindow> WeakWindow;
		TWeakPtr<IStructureDetailsView> WeakDetailsView;

		void Start(
			const TSharedRef<FControlRigParameterTrackEditor>& InTrackEditor,
			UMovieSceneControlRigParameterTrack* InTrack
			);
		
		~FRecreateRigOperator();
		
		bool IsValid();
		void Abort();
		void OnRigSettingsWindowClosed(const TSharedRef<SWindow>& Window);
		void RefreshSettingsFromControlRig();
		void OnPostControlRigForwardSolve_AnyThread(UControlRig*, const FName&);
		void OnRigSettingsChanged( const FPropertyChangedEvent& InEvent);
		bool IsEditingObject(UObject* InRigToCheck);
	};

	TSharedPtr<FRecreateRigOperator> RecreateRigOperator;
	
	void ResetRecreateRigOperatorIfNeeded(TSharedRef<FControlRigParameterTrackEditor::FRecreateRigOperator> InRequestingOperator);
public:
	/** Set Up EditMode for Specified Control Rig*/
	void SetUpEditModeIfNeeded(UControlRig* ControlRig);

	/** Helper functions to iterate over UMovieSceneControlRigParameterTracks in the currently focussed MovieScene*/
	void IterateTracks(TFunctionRef<bool(UMovieSceneControlRigParameterTrack*, FGuid)> Callback) const;
private:
	/** Command Bindings added by the Transform Track Editor to Sequencer and curve editor. */
	TSharedPtr<FUICommandList> CommandBindings;
	FAcquiredResources AcquiredResources;

public:

	void AddControlKeys(UObject* InObject, UControlRig* InControlRig, FName PropertyName,
		FName ParameterName, EControlRigContextChannelToKey ChannelsToKey, ESequencerKeyMode KeyMode,
		float InLocalTime, const bool bInConstraintSpace = false);
	void GetControlRigKeys(UControlRig* InControlRig, FName ParameterName, EControlRigContextChannelToKey ChannelsToKey,
		ESequencerKeyMode KeyMode, UMovieSceneControlRigParameterSection* SectionToKey,	FGeneratedTrackKeys& OutGeneratedKeys,
		const bool bInConstraintSpace = false);
	FKeyPropertyResult AddKeysToControlRig(
		UObject* InObject, UControlRig* InControlRig, FFrameNumber KeyTime, FFrameNumber EvaluateTime, FGeneratedTrackKeys& GeneratedKeys,
		ESequencerKeyMode KeyMode, TSubclassOf<UMovieSceneTrack> TrackClass, FName ControlRigName, FName RigControlName);
	FKeyPropertyResult AddKeysToControlRigHandle(UObject* InObject, UControlRig* InControlRig,
		FGuid ObjectHandle, FFrameNumber KeyTime, FFrameNumber EvaluateTime, FGeneratedTrackKeys& GeneratedKeys,
		ESequencerKeyMode KeyMode, TSubclassOf<UMovieSceneTrack> TrackClass, FName ControlRigName, FName RigControlName);

	/** Batched overload: groups controls by section, resolves handle/track once, keys all in one transaction. */
	void AddControlKeys(UObject* InObject, UControlRig* InControlRig, FName ControlRigName,
		TArray<FControlRigKeyAddParams>& InOutParams,
		EControlRigContextChannelToKey ChannelsToKey, ESequencerKeyMode KeyMode,
		float InLocalTime, const bool bInConstraintSpace = false);

	/** Batched overload: populates GeneratedKeys for every param in a single pass over the control hierarchy. */
	void GetControlRigKeys(UControlRig* InControlRig,
		FControlRigSectionParamMap& InOutSectionParamMap,
		EControlRigContextChannelToKey ChannelsToKey, ESequencerKeyMode KeyMode,
		const bool bInConstraintSpace = false);

	/**
	 * Modify the passed in Generated Keys by the current tracks values and weight at the passed in time.

	 * @param Object The handle to the object modify
	 * @param Track The track we are modifying
	 * @param SectionToKey The Sections Channels we will be modifiying
	 * @param Time The Time at which to evaluate
	 * @param InOutGeneratedTrackKeys The Keys we need to modify. We change these values.
	 * @param Weight The weight we need to modify the values by.
	 */
	bool ModifyOurGeneratedKeysByCurrentAndWeight(UObject* Object, UControlRig* InControlRig, FName RigControlName, UMovieSceneTrack *Track, UMovieSceneSection* SectionToKey, FFrameNumber EvaluateTime, FGeneratedTrackKeys& InOutGeneratedTotalKeys, float Weight) const;

	//**Function to collapse all layers from this section onto the first absoluate layer.*/
	static bool CollapseAllLayers(TSharedPtr<ISequencer>& SequencerPtr, UMovieSceneTrack* OwnerTrack, const FBakingAnimationKeySettings& InSettings);

	//** Function to load animation into a section, returns true if successful
	static bool LoadAnimationIntoSection(const TSharedPtr<ISequencer>& InSequencer,  UAnimSequence* AnimSequence, USkeletalMeshComponent* SkelMeshComp,
		FFrameNumber StartFrame, bool bReduceKeys, const FSmartReduceParams& ReduceParams, bool bResetControls,const TOptional<TRange<FFrameNumber>> &InFrameRamge,
		bool bOntoSelectedControls, UMovieSceneControlRigParameterSection* ParamSection);

	//** Function to smart reduce all keys on all constraints based on the parameters
	static void SmartReduce(const TSharedPtr<ISequencer>& InSequencer, const FSmartReduceParams& InParams, UMovieSceneSection* InSection);

	/** Adds the constraint to sequencer and create the bindings if needed. Note that ControlRig Track auto-generation is disabled in this function. */
	static void AddConstraintToSequencer(const TSharedPtr<ISequencer>& InSequencer, UTickableTransformConstraint* InConstraint,
		const UConstraintCreationOptions* InSequencerOptions = nullptr);

	/** Whether or not the Control Element should also be selected if the parent is selected, e.g. channel, hosted channel etc*/
	static bool ShouldBeSelected(URigHierarchy* Hierarchy, FRigControlElement* ParentElement, FRigControlElement* ChildElement); 

	/** Bake the timewarp, per frame on the specified section*/
	static bool BakeTimeWarp(TSharedPtr<ISequencer>& InSequencer, UMovieSceneControlRigParameterSection* InSection);

private:
	/** Iterates sections once each: extends bounds, one interrogation, one AddKeysToSection per section. */
	FKeyPropertyResult AddKeysToControlRigBatch(UObject* InObject, UControlRig* InControlRig,
		FGuid ObjectHandle, UMovieSceneControlRigParameterTrack* Track,
		FFrameNumber KeyTime, FFrameNumber EvaluateTime,
		FControlRigSectionParamMap& InOutSectionParamMap,
		ESequencerKeyMode KeyMode, FName ControlRigName);

	/** Batched weight-blend: one track interrogation per section; writes into SectionKeyData.GeneratedKeys. */
	bool ModifyOurGeneratedKeysByCurrentAndWeight(UObject* Object, UControlRig* InControlRig,
		UMovieSceneTrack* Track, UMovieSceneSection* SectionToKey, FFrameNumber EvaluateTime,
		FControlRigSectionKeyData& InOutSectionKeyData, float Weight) const;

	FDelegateHandle SelectionChangedHandle;
	FDelegateHandle SequencerChangedHandle;
	FDelegateHandle OnActivateSequenceChangedHandle;
	FDelegateHandle CurveChangedHandle;
	FDelegateHandle OnChannelChangedHandle;
	FDelegateHandle OnMovieSceneBindingsChangeHandle;
	FDelegateHandle OnMovieSceneChannelChangedHandle;
	FDelegateHandle OnActorAddedToSequencerHandle;

	//map used for space channel delegates since we loose them

	//used to sync curve editor selections/displays on next tick for performance reasons
	TSet<FName> DisplayedControls;
	TSet<FName> UnDisplayedControls;
	bool bCurveDisplayTickIsPending;
	void BindControlRig(UControlRig* ControlRig, FGuid);
	void UnbindControlRig(UControlRig* ControlRig);
	void UnbindAllControlRigs();
	TMap<TWeakObjectPtr<UControlRig>, FGuid> BoundControlRigs;

private:

	/** Guard to stop infinite loops when handling control selections*/
	bool bIsDoingSelection;

	/** A flag to determine if the next update coming from the timer should be skipped */
	bool bSkipNextSelectionFromTimer;

	/** Whether or not this rig will be used as an layered control rig */
	bool bIsLayeredControlRig;

	/** Whether or not we should check Skeleton when filtering*/
	bool bFilterAssetBySkeleton;

	/** Whether or not we should check for Animatable Controls when filtering*/
	bool bFilterAssetByAnimatableControls;

	FRefreshAssetViewDelegate RefreshControlRigPickerDelegate;

	/** Handle to help updating selection on tick tick to avoid too many flooded selections*/
	FTimerHandle UpdateSelectionTimerHandle;

	/** Array of sections that are getting undone, we need to recreate any space channel add,move key delegates to them*/
	mutable TArray<UMovieSceneControlRigParameterSection*> SectionsGettingUndone;

	/** An index counter for the opened undo brackets */
	int32 ControlUndoBracket;

	/** A counter for occurred control changes during a control undo bracket */
	int32 ControlChangedDuringUndoBracket;

	/** Lock to avoid registering multiple transactions from different tracks at the same time */
	static FCriticalSection ControlUndoTransactionMutex;

	/** A transaction used to group multiple key events */
	TSharedPtr<FScopedTransaction> ControlUndoTransaction;

	/** Controls if the control rig track for the default animating rig should be created */
	static bool bAutoGenerateControlRigTrack;

	/** Set of delegate handles we have added delegate's too, need to clear them*/
	TSet<FDelegateHandle> ConstraintHandlesToClear;

	friend class FControlRigBlueprintActions;
	friend class FControlRigAssetActions;

	/** Whether or not animation/control rig edit mode was open when we closed*/
	static bool bControlRigEditModeWasOpen;
	/** Previous selection if open*/
	static TArray <TPair<FControlRigAssetStrongReference, TArray<FName>>> PreviousSelectedControlRigs;
};



/** Class for control rig sections */
class FControlRigParameterSection : public FSequencerSection
{
public:

	/**
	* Creates a new control rig property section.
	*
	* @param InSection The section object which is being displayed and edited.
	* @param InSequencer The sequencer which is controlling this parameter section.
	*/
	FControlRigParameterSection(UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
		: FSequencerSection(InSection), WeakSequencer(InSequencer)
	{
	}

public:

	virtual void BuildSectionContextMenu(FMenuBuilder& MenuBuilder, const FGuid& InObjectBinding) override;

	//~ ISequencerSection interface
	virtual bool RequestDeleteCategory(const TArray<FName>& CategoryNamePath) override;
	virtual bool RequestDeleteKeyArea(const TArray<FName>& KeyAreaNamePath) override;
protected:
	/** Add Sub Menu */
	void LoadAnimationIntoSection(FMenuBuilder& MenuBuilder, FGuid ObjectBinding, USkeleton* Skeleton, UMovieSceneControlRigParameterSection* Section);

	/** Animation sub menu filter function */
	bool ShouldFilterAsset(const FAssetData& AssetData);

	/** Animation asset selected */
	void OnAnimationAssetSelected(const FAssetData& AssetData,FGuid ObjectBinding, UMovieSceneControlRigParameterSection* Section);

	/** Animation asset enter pressed */
	void OnAnimationAssetEnterPressed(const TArray<FAssetData>& AssetData, FGuid ObjectBinding, UMovieSceneControlRigParameterSection* Section);

	/** Select controls channels from selected pressed*/
	void ShowSelectedControlsChannels();
	/** Show all control channels*/
	void ShowAllControlsChannels();

	void KeyZeroValue();
	void KeyWeightValue(float Val);
	void CollapseAllLayers();

	/** The sequencer which is controlling this section. */
	TWeakPtr<ISequencer> WeakSequencer;

};


#undef UE_API
