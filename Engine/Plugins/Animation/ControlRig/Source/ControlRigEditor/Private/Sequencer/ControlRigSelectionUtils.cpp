// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSelectionUtils.h"

#include "AnimDetails/AnimDetailsProxyManager.h"
#include "AnimDetails/AnimDetailsSelection.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "Internationalization/Text.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/CategoryModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"

namespace UE::ControlRig
{
bool IsAnimDetailsChangingSelection(FControlRigEditMode& InControlMode)
{
	UAnimDetailsProxyManager* DetailsProxyManager = InControlMode.GetAnimDetailsProxyManager();
	
	UAnimDetailsSelection* DetailsSelection = DetailsProxyManager ? DetailsProxyManager->GetAnimDetailsSelection() : nullptr;
	const bool bIsDetailsChangingSelection = DetailsSelection && DetailsSelection->IsAnimLayersChangingSelection();
	
	return bIsDetailsChangingSelection;
}

namespace ParentControlDetail
{
static void AppendGroups(const Sequencer::FViewModelPtr& InViewModel, TArray<FString>& OutGroups)
{
	using namespace Sequencer;
	
	const TViewModelPtr<FChannelGroupOutlinerModel> ChannelModel = CastViewModel<FChannelGroupOutlinerModel>(InViewModel);
	if (ChannelModel)
	{
		for (TSharedPtr<IKeyArea> KeyArea : ChannelModel->GetAllKeyAreas())
		{
			if (const FMovieSceneChannelMetaData* MetaData = KeyArea->GetChannel().GetMetaData())
			{
				OutGroups.AddUnique(MetaData->Group.ToString());
			}
		}
	}
}

/** @return Mapping of root view models (that corresponds to a control rig), to groups found walking up the hierarchy chain (usually just leaf node has an assigned group). */
static TMap<Sequencer::FViewModelPtr, TArray<FString>> FindSelectedGroups(const TSharedPtr<Sequencer::FSequencerSelection>& InSequencerSelection)
{
	using namespace Sequencer;
	TArray<FViewModelPtr> AlreadyVisited;
	TMap<FViewModelPtr, TArray<FString>> ViewModelsToGroups;
	for (const FViewModelPtr& Selected : InSequencerSelection->Outliner)
	{
		TArray<FString> GroupsForSelected;
		for (const FViewModelPtr& Parent : FParentModelIterator(Selected, true))
		{
			if (AlreadyVisited.Contains(Parent))
			{
				break;
			}
			AlreadyVisited.Add(Parent);

			ParentControlDetail::AppendGroups(Parent, GroupsForSelected);
			
			const TViewModelPtr<FTrackModel> TrackModel = CastViewModel<FTrackModel>(Parent);
			if (!TrackModel.IsValid())
			{
				continue;
			}

			// We've collected groups thus far. If it turns out the hierarchy is not part of any control rig stuff, the work was for nothing.
			UMovieSceneControlRigParameterTrack* const ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(TrackModel->GetTrack());
			if (ControlRigTrack)
			{
				TArray<FString>& GroupsInControl = ViewModelsToGroups.FindOrAdd(Parent);
				for (const FString& Group : GroupsForSelected)
				{
					GroupsInControl.AddUnique(Group);
				}
				break;
			}
		}
	}
	
	return ViewModelsToGroups;
}

/** Walks down the root items and invokes InCallback for every node that has is in the same group. */
static void EnumerateViewModelsWithMatchingGroups(
	const TMap<Sequencer::FViewModelPtr, TArray<FString>>& ControlRigToSelectedGroups, 
	TFunctionRef<bool(const Sequencer::FViewModelPtr&)> InCallback
	)
{
	using namespace UE::Sequencer;
	
	for (const TPair<FViewModelPtr, TArray<FString>>& Pair : ControlRigToSelectedGroups)
	{
		for (const FViewModelPtr& Child : FParentFirstChildIterator(Pair.Key, true))
		{
			const TArray<FString>& GroupsToSelect = Pair.Value;
			
			TArray<FString> CurrentGroups;
			ParentControlDetail::AppendGroups(Child, CurrentGroups);
		
			const bool bContainsGroup = Algo::AnyOf(GroupsToSelect, [&CurrentGroups](const FString& SelectedGroup)
			{
				return CurrentGroups.Contains(SelectedGroup);
			});
			
			if (bContainsGroup && !InCallback(Child))
			{
				break;
			}
		}
	}
}
}

void EnumerateParentControlChildren(const ISequencer& InSequencer, TFunctionRef<bool(const Sequencer::FViewModelPtr&)> InCallback)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
	const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel ? ViewModel->GetSelection() : nullptr;
	if (!SequencerSelection.IsValid())
	{
		return;
	}
	
	// For each selected FVieModel, walk up the parent chain and determine the groups that are selected. This will produce ControlRig -> Groups map.
	const TMap<FViewModelPtr, TArray<FString>> ControlRigToSelectedGroups = ParentControlDetail::FindSelectedGroups(SequencerSelection);
	
	// For every ControlRig FViewModel, iterate all children an include those that are in the same group as the selected groups.
	// Example: User selects "root_ctrl.Location.X". "Location.X" has group root_ctrl.
	// "Location.Y" would get included even though it's not explicitly selected because it also has group "root_ctrl".
	ParentControlDetail::EnumerateViewModelsWithMatchingGroups(ControlRigToSelectedGroups, InCallback);
}

namespace SelectionDetail
{
class FControlSelectionCache
{
	/** Value: Caches result of UControlRig::CurrentControlSelection. */
	TMap<const UControlRig*, TArray<FName>> RigToSelectedControlNames;
	
public:
	
	/** @return Names of the controls selected in the given control rig. */
	const TArray<FName>& FindOrAdd(const UControlRig& InControlRig)
	{
		const TArray<FName>* SelectedControls = RigToSelectedControlNames.Find(&InControlRig);
		if (SelectedControls)
		{
			return *SelectedControls;
		}
		
		TArray<FName>& ControlNames = RigToSelectedControlNames.Add(&InControlRig);
		for (const FName ControlName : InControlRig.CurrentControlSelection())
		{
			ControlNames.AddUnique(ControlName);
		}
		return ControlNames;
	}
};
}

void EnumerateOutlinerExtensionsForCurrentControlSelection(
	const ISequencer& InSequencer, TFunctionRef<bool(const Sequencer::TViewModelPtr<Sequencer::IOutlinerExtension>&)> InCallback
	)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
	const TViewModelPtr<FViewModel> RootModel = ViewModel ? ViewModel->GetRootModel() : nullptr;
	if (!RootModel)
	{
		return;
	}

	SelectionDetail::FControlSelectionCache RigToSelectedControlNames;
	for (const TViewModelPtr<FChannelGroupOutlinerModel>& ChannelModel : RootModel->GetDescendantsOfType<FChannelGroupOutlinerModel>())
	{
		for (const TSharedRef<IKeyArea>& KeyArea : ChannelModel->GetAllKeyAreas())
		{
			const UMovieSceneControlRigParameterSection* const ControlRigSection = Cast<UMovieSceneControlRigParameterSection>(KeyArea->GetOwningSection());
			if (!ControlRigSection || !ControlRigSection->GetControlRig())
			{
				continue;
			}
			
			const UControlRig* const ControlRig = ControlRigSection->GetControlRig();
			const MovieScene::FControlRigChannelMetaData ChannelMetaData = ControlRigSection->GetChannelMetaData(KeyArea->GetChannel().Get());
			if (!ChannelMetaData)
			{
				continue;
			}

			const TArray<FName>& SelectedControls = RigToSelectedControlNames.FindOrAdd(*ControlRig);
			if (SelectedControls.Contains(ChannelMetaData.GetControlName()) && !InCallback(ChannelModel))
			{
				return;
			}
		}
	}
}
} // namespace UE::ControlRig