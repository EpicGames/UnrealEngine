// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCoreFwd.h"
#include "Templates/FunctionFwd.h"

class FControlRigEditMode;
class ISequencer;
namespace UE::Sequencer { class IOutlinerExtension; }

namespace UE::ControlRig
{
/** @return Whether anim details is changing selection. */
bool IsAnimDetailsChangingSelection(FControlRigEditMode& InControlMode);

/**
 * Goes through Sequencer selecting and finds all channels that are related to selected control rig controls.
 * For example, if you have selected root_ctrl.Location.X then this lists out Location.Y, Rotation.Yaw, etc.
 */
void EnumerateParentControlChildren(const ISequencer& InSequencer, TFunctionRef<bool(const Sequencer::FViewModelPtr&)> InCallback);

/**
 * Enumerates IOutlinerExtension that represent a channel of controls selected in the viewport & rig hierarchy.
 * 
 * The bridge from channel back to control name uses UMovieSceneControlRigParameterSection::GetChannelMetaData, which is the channel<->control mapping
 * produced by UMovieSceneControlRigParameterSection>>CacheChannelProxy.
 * 
 * @param InSequencer The Sequencer to search for selected view models on
 * @param InCallback The callback to invoke. Return true to continue iteration, false to stop.
 */
void EnumerateOutlinerExtensionsForCurrentControlSelection(
	const ISequencer& InSequencer, TFunctionRef<bool(const Sequencer::TViewModelPtr<Sequencer::IOutlinerExtension>&)> InCallback
	);

} // namespace UE::ControlRig
