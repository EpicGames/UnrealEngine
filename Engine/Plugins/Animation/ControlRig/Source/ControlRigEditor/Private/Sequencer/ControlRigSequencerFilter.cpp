// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/ControlRigSequencerFilter.h"
#include "ControlRig.h"
#include "ControlRigBlueprintLegacy.h"
#include "Filters/ISequencerTrackFilters.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Settings/ControlRigSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"

using namespace UE::Sequencer;

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSequencerFilter)

#define LOCTEXT_NAMESPACE "ControlRigSequencerTrackFilters"

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_ControlRigControlsCommands
	: public TCommands<FSequencerTrackFilter_ControlRigControlsCommands>
{
public:
	FSequencerTrackFilter_ControlRigControlsCommands()
		: TCommands<FSequencerTrackFilter_ControlRigControlsCommands>(
			TEXT("FSequencerTrackFilter_ControlRigControls"),
			LOCTEXT("FSequencerTrackFilter_ControlRigControls", "Control Rig Filters"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	/** Toggle the control rig controls filter */
	TSharedPtr< FUICommandInfo > ToggleFilter_ControlRigControls;

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleFilter_ControlRigControls, "Control Rig Controls", "Toggle the filter for Control Rig Controls.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::F9));
	}
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_ControlRigControls : public FSequencerTrackFilter
{
public:
	FSequencerTrackFilter_ControlRigControls(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr)
		: FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
	{
		if (++BindingCount == 1)
		{
			FSequencerTrackFilter_ControlRigControlsCommands::Register();
		}
	}

	virtual ~FSequencerTrackFilter_ControlRigControls() override
	{
		if (ensure(BindingCount > 0) && --BindingCount == 0)
		{
			FSequencerTrackFilter_ControlRigControlsCommands::Unregister();
		}
	}

	//~ Begin IFilter

	virtual FString GetName() const override { return TEXT("ControlRigControl"); }

	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();
		const UMovieSceneTrack* const TrackObject = FilterData.ResolveMovieSceneTrackObject(InItem);
		return TrackObject && TrackObject->IsA<UMovieSceneControlRigParameterTrack>();
	}

	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override { return LOCTEXT("SequenceTrackFilter_ControlRigControl", "Control Rig Control"); }
	virtual FSlateIcon GetIcon() const override { return FSlateIconFinder::FindIconForClass(UControlRigBlueprint::StaticClass()); }
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter

	virtual FText GetDefaultToolTipText() const override
	{
		return LOCTEXT("SequencerTrackFilter_ControlRigControlsTip", "Show only Control Rig Control tracks");
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override
	{
		return FSequencerTrackFilter_ControlRigControlsCommands::Get().ToggleFilter_ControlRigControls;
	}

	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override
	{
		return IsSequenceTrackSupported<UMovieSceneControlRigParameterTrack>(InSequence);
	}

	//~ End FSequencerTrackFilter

private:
	static int32 BindingCount;
};

int32 FSequencerTrackFilter_ControlRigControls::BindingCount = 0;

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_ControlRigSelectedControlsCommands
	: public TCommands<FSequencerTrackFilter_ControlRigSelectedControlsCommands>
{
public:
	FSequencerTrackFilter_ControlRigSelectedControlsCommands()
		: TCommands<FSequencerTrackFilter_ControlRigSelectedControlsCommands>(
			TEXT("FSequencerTrackFilter_ControlRigSelectedControls"),
			LOCTEXT("FSequencerTrackFilter_ControlRigSelectedControls", "Control Rig Selected Control Filters"),
			NAME_None,
			FAppStyle::GetAppStyleSetName())
	{}

	/** Toggle the control rig selected controls filter */
	TSharedPtr<FUICommandInfo> ToggleFilter_ControlRigSelectedControls;

	/** Initialize commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND(ToggleFilter_ControlRigSelectedControls, "Control Rig Selected Controls", "Toggle the filter for Control Rig Selected Controls.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::F10));
	}
};

//////////////////////////////////////////////////////////////////////////
//

class FSequencerTrackFilter_ControlRigSelectedControls : public FSequencerTrackFilter
{
public:
	FSequencerTrackFilter_ControlRigSelectedControls(ISequencerTrackFilters& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr)
		:  FSequencerTrackFilter(InOutFilterInterface, MoveTemp(InCategory))
	{
		if (++BindingCount == 1)
		{
			FSequencerTrackFilter_ControlRigSelectedControlsCommands::Register();
		}
	}

	virtual ~FSequencerTrackFilter_ControlRigSelectedControls() override
	{
		if (ensure(BindingCount > 0) && --BindingCount == 0)
		{
			FSequencerTrackFilter_ControlRigSelectedControlsCommands::Unregister();
		}
	}

	//~ Begin IFilter

	virtual FString GetName() const override { return TEXT("SelectedControlRigControl"); }

	virtual bool PassesFilter(FSequencerTrackFilterType InItem) const override
	{
		FSequencerFilterData& FilterData = GetFilterInterface().GetFilterData();

		UMovieSceneTrack* const TrackObject = FilterData.ResolveMovieSceneTrackObject(InItem);
		URigHierarchy* const ControlRigHierarchy = GetControlRigHierarchyFromTrackObject(TrackObject);

		if (!ControlRigHierarchy)
		{
			return false;
		}

		const TViewModelPtr<IOutlinerExtension> OutlinerExtension = InItem.AsModel()->FindAncestorOfType<IOutlinerExtension>();
		if (!OutlinerExtension.IsValid())
		{
			return false;
		}

		const FString ControlTrackLabel = OutlinerExtension->GetLabel().ToString();
		const EElementNameDisplayMode ElementNameDisplayMode = UControlRigEditorSettings::Get()->ElementNameDisplayMode;

		auto ShouldShowElement = [ControlRigHierarchy, &ControlTrackLabel, ElementNameDisplayMode](const FRigBaseElement* const InElement)
		{
			const FText ElementDisplayName = ControlRigHierarchy->GetDisplayNameForUI(InElement, ElementNameDisplayMode);
			return ElementDisplayName.ToString().StartsWith(ControlTrackLabel);
		};

		const TArray<const FRigBaseElement*> SelectedControls = ControlRigHierarchy->GetSelectedElements(ERigElementType::Control);
		for (const FRigBaseElement* const SelectedControl : SelectedControls)
		{
			if (ShouldShowElement(SelectedControl))
			{
				return true;
			}

			const FRigControlElement* const ControlElement = Cast<FRigControlElement>(SelectedControl);
			if (ControlElement && ControlElement->CanDriveControls())
			{
				for (const FRigElementKey& DrivenKey : ControlElement->Settings.DrivenControls)
				{
					const FRigBaseElement* const DrivenKeyElement = ControlRigHierarchy->Find(DrivenKey);
					if (DrivenKeyElement && ShouldShowElement(DrivenKeyElement))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override { return LOCTEXT("SequenceTrackFilter_ControlRigSelectedControl", "Selected Control Rig Control"); }
	virtual FSlateIcon GetIcon() const override { return FSlateIconFinder::FindIconForClass(UControlRigBlueprint::StaticClass()); }
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter

	virtual FText GetDefaultToolTipText() const override
	{
		return LOCTEXT("SequencerTrackFilter_ControlRigSelectedControlsTip", "Show Only Selected Control Rig Controls.");
	}

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override
	{
		return FSequencerTrackFilter_ControlRigSelectedControlsCommands::Get().ToggleFilter_ControlRigSelectedControls;
	}

	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override
	{
		return IsSequenceTrackSupported<UMovieSceneControlRigParameterTrack>(InSequence);
	}

	//~ End FSequencerTrackFilter

	static URigHierarchy* GetControlRigHierarchyFromTrackObject(UMovieSceneTrack* const InTrackObject)
	{
		const UMovieSceneControlRigParameterTrack* const Track = Cast<UMovieSceneControlRigParameterTrack>(InTrackObject);
		if (!Track)
		{
			return nullptr;
		}

		UControlRig* const ControlRig = Track->GetControlRig();
		if (!ControlRig)
		{
			return nullptr;
		}

		return ControlRig->GetHierarchy();
	}

private:
	static int32 BindingCount;
};

int32 FSequencerTrackFilter_ControlRigSelectedControls::BindingCount = 0;

//////////////////////////////////////////////////////////////////////////
//

void UControlRigTrackFilter::AddTrackFilterExtensions(ISequencerTrackFilters& InOutFilterInterface
	, const TSharedRef<FFilterCategory>& InPreferredCategory
	, TArray<TSharedRef<FSequencerTrackFilter>>& InOutFilterList) const
{
	InOutFilterList.Add(MakeShared<FSequencerTrackFilter_ControlRigControls>(InOutFilterInterface, InPreferredCategory));
	/** Removing this since FSequencerTrackFilter_Selected will now include selected control rig controls.
	 * If there are no use cases for this in the future and no users are missing it, we could remove this eventually,
	 * along with its class definition above. */
	//InOutFilterList.Add(MakeShared<FSequencerTrackFilter_ControlRigSelectedControls>(InOutFilterInterface, InPreferredCategory));
}

#undef LOCTEXT_NAMESPACE
