// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSpaceChannelEditors.h"
#include "Constraints/MovieSceneConstraintChannelHelper.inl"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceEditor.h"
#include "MovieSceneEventUtils.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "ISequencerChannelInterface.h"
#include "Widgets/SNullWidget.h"
#include "IKeyArea.h"
#include "ISequencer.h"
#include "SequencerSettings.h"
#include "MovieSceneCommonHelpers.h"
#include "GameFramework/Actor.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "MVVM/Views/KeyDrawParams.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/DoubleChannelCurveModel.h"
#include "Channels/FloatChannelCurveModel.h"
#include "Channels/IntegerChannelCurveModel.h"
#include "Channels/BoolChannelCurveModel.h"
#include "PropertyCustomizationHelpers.h"
#include "MovieSceneObjectBindingIDCustomization.h"
#include "MovieSceneObjectBindingIDPicker.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "SSocketChooser.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneSpawnableAnnotation.h"
#include "ISequencerModule.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tools/ControlRigSnapper.h"
#include "Tools/EvaluateSequencerTools.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "MovieScene.h"
#include "Sequencer/MovieSceneControlRigSpaceChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneObjectBindingID.h"
#include "ControlRig.h"
#include "IControlRigObjectBinding.h"
#include "ControlRigSpaceChannelCurveModel.h"
#include "ScopedTransaction.h"
#include "IControlRigEditorModule.h"
#include "TimerManager.h"
#include "SequencerSectionPainter.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontMeasure.h"
#include "CurveEditorSettings.h"
#include "TimeToPixel.h"
#include "Templates/Tuple.h"
#include "Sequencer/ControlRigSequencerHelpers.h"
#include "SpacePicker/Views/SRigSpacePickerBakeDialog.h"

#define LOCTEXT_NAMESPACE "ControlRigEditMode"

//flag used when we add or delete a space channel. When we do so we handle the compensation in those functions since we need to 
//compensate based upon the previous spaces, not the new space. So we set that this flag to be true and then
//do not compensate via the FControlRigParameterTrackEditor setting a key.
static bool bDoNotCompensate = false;


FKeyHandle AddOrUpdateKey(FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber Time, ISequencer& Sequencer, const FGuid& InObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings)
{
	UE_LOGF(LogControlRigEditor, Log, "We don't support adding keys to the Space Channel via the + key. Please use the Space Channel Area in the Animation Panel");

	FKeyHandle Handle = FKeyHandle::Invalid();

	return Handle;
}

bool CanCreateKeyEditor(const FMovieSceneControlRigSpaceChannel* Channel)
{
	return false; //mz todoo maybe change
}

TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params)
{
	return SNullWidget::NullWidget;
}



/*******************************************************************
*
* FControlRigSpaceChannelHelpers
*
**********************************************************************/


FSpaceChannelAndSection FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(UControlRig* ControlRig, FName ControlName, ISequencer* Sequencer, bool bCreateIfNeeded)
{
	FSpaceChannelAndSection SpaceChannelAndSection;
	SpaceChannelAndSection.SpaceChannel = nullptr;
	SpaceChannelAndSection.SectionToKey = nullptr;
	if (ControlRig == nullptr || Sequencer == nullptr)
	{
		return SpaceChannelAndSection;
	}
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!MovieScene)
	{
		return SpaceChannelAndSection;
	}

	bool bFoundTrack = false;
	
	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
	bool bRecreateCurves = false;
	TArray<TPair<UControlRig*, FName>> ControlRigPairsToReselect;
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
		for (UMovieSceneTrack* Track : Tracks)
		{
			UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(Track);
			if (ControlRigParameterTrack && ControlRigParameterTrack->GetControlRig() == ControlRig)
			{
				UMovieSceneControlRigParameterSection* ActiveSection = Cast<UMovieSceneControlRigParameterSection>(ControlRigParameterTrack->GetSectionToKey(ControlName));
				if (ActiveSection)
				{
					SpaceChannelAndSection.SectionToKey = ActiveSection;
					FSpaceControlNameAndChannel* NameAndChannel = ActiveSection->GetSpaceChannel(ControlName);
					if (NameAndChannel)
					{
						SpaceChannelAndSection.SpaceChannel = &NameAndChannel->SpaceCurve;
					}
					else if (bCreateIfNeeded)
					{
						ControlRig->Modify();
						if (ControlRig->IsControlSelected(ControlName))
						{
							TPair<UControlRig*, FName> Pair;
							Pair.Key = ControlRig;
							Pair.Value = ControlName;
							ControlRigPairsToReselect.Add(Pair);
						}
						ActiveSection->AddSpaceChannel(ControlName, true /*ReconstructChannelProxy*/);
						//important we don't want the adding of the space channel to be in the undo since we want the channel to stick around if we get undone
						ActiveSection->Modify();
						NameAndChannel = ActiveSection->GetSpaceChannel(ControlName);
						if (NameAndChannel)
						{
							SpaceChannelAndSection.SpaceChannel = &NameAndChannel->SpaceCurve;
							bRecreateCurves = true;
						}
					}
				}
				
				bFoundTrack = true;
				break;
			}
		}

		if (bFoundTrack)
		{
			break;
		}
	}
	if (bRecreateCurves)
	{
		Sequencer->RecreateCurveEditor(); //this will require the curve editor to get recreated so the ordering is correct
		for (TPair<UControlRig*, FName>& Pair : ControlRigPairsToReselect)
		{

			GEditor->GetTimerManager()->SetTimerForNextTick([Pair]()
				{
					Pair.Key->SelectControl(Pair.Value);
				});

		}
	}
	return SpaceChannelAndSection;
}

static TTuple<FRigControlElement*, FChannelMapInfo*, int32, int32> GetControlAndChannelInfos(UControlRig* ControlRig, UMovieSceneControlRigParameterSection* ControlRigSection, FName ControlName)
{
	FRigControlElement* ControlElement = nullptr;
	FChannelMapInfo* pChannelIndex = nullptr;
	int32 NumChannels = 0;
	int32 ChannelIndex = 0;
	Tie(ControlElement, pChannelIndex) = FControlRigSpaceChannelHelpers::GetControlAndChannelInfo(ControlRig, ControlRigSection, ControlName);

	if (pChannelIndex && ControlElement)
	{
		// get the number of float channels to treat
		NumChannels = FControlRigSpaceChannelHelpers::GetNumFloatChannels(ControlElement->Settings.ControlType);
		if (NumChannels > 0)
		{
			ChannelIndex = pChannelIndex->ChannelIndex;
		}
	}

	return TTuple<FRigControlElement*, FChannelMapInfo*, int32, int32>(ControlElement, pChannelIndex, NumChannels,ChannelIndex);
}

/*
*  Situations to handle\
*  0) New space sames as current space do nothing
*  1) First space switch, so no existing space keys, if at start frame, just create space key, if not at start frame create current(parent) at start frame then new
*  2) Creating space that was the same as the one just before the current time. In that case we need to delete the current space key and the previous(-1) transform keys
*  3) Creating different space than previous or current, at (-1) curren ttime set space key as current, and set transforms in that space, then at current time set new space and new transforms
*  4) For 1-3 we always compensate any transform keys to the new space
*/

FKeyHandle FControlRigSpaceChannelHelpers::SequencerKeyControlRigSpaceChannel(
	UControlRig* ControlRig,
	ISequencer* Sequencer,
	FMovieSceneControlRigSpaceChannel* Channel,
	UMovieSceneSection* SectionToKey,
	FFrameNumber Time,
	URigHierarchy* RigHierarchy,
	const FRigElementKey& ControlKey,
	const FRigElementKey& SpaceKey)
{
	FKeyHandle Handle = FKeyHandle::Invalid();

	UMovieSceneSequence* MovieSceneSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (bDoNotCompensate == true || ControlRig == nullptr || !MovieScene)
	{
		return Handle;
	}

	TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);

	FMovieSceneControlRigSpaceBaseKey ExistingValue, PreviousValue;
	using namespace UE::MovieScene;
	EvaluateChannel(Channel, Time, ExistingValue);
	EvaluateChannel(Channel, Time -1, PreviousValue);

	TArray<FRigElementKey> BeforeKeys, AfterKeys;
	Channel->GetUniqueSpaceList(&BeforeKeys);

	FMovieSceneControlRigSpaceBaseKey Value = ExistingValue;

	if (SpaceKey == RigHierarchy->GetWorldSpaceReferenceKey())
	{
		Value.SpaceType = EMovieSceneControlRigSpaceType::World;
		Value.ControlRigElement = URigHierarchy::GetWorldSpaceReferenceKey();
	}
	else if (SpaceKey == RigHierarchy->GetDefaultParentKey())
	{
		Value.SpaceType = EMovieSceneControlRigSpaceType::Parent;
		Value.ControlRigElement = RigHierarchy->GetFirstParent(ControlKey);
	}
	else
	{
		FRigElementKey DefaultParent = RigHierarchy->GetFirstParent(ControlKey);
		if (DefaultParent == SpaceKey)
		{
			Value.SpaceType = EMovieSceneControlRigSpaceType::Parent;
			Value.ControlRigElement = DefaultParent;
		}
		else  //support all types
		{
			Value.SpaceType = EMovieSceneControlRigSpaceType::ControlRig;
			Value.ControlRigElement = SpaceKey;
		}
	}

	//we only key if the value is different.
	if (Value != ExistingValue)
	{
		const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		const FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransform();
		const FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();
		
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(ControlRig->GetWorld());
		auto EvaluateSequencer = [Sequencer, TickResolution, &Controller](const FFrameTime& Time)
		{
			FMovieSceneContext SceneContext(FMovieSceneEvaluationRange(Time, TickResolution), Sequencer->GetPlaybackStatus());
			SceneContext.SetHasJumped(true);
			
			Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(SceneContext);
			Controller.EvaluateAllConstraints();
		};
		
		//make sure to evaluate first frame
		FFrameTime CurrentTime(Time);
		CurrentTime = LocalToRootTransform.TryTransformTime(CurrentTime).Get(CurrentTime);

		EvaluateSequencer(CurrentTime);
		ControlRig->Evaluate_AnyThread();

		TArray<FFrameNumber> Frames;
		Frames.Add(Time);
		
		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelInterface = Channel->GetData();
		bool bSetPreviousKey = true;
		//find all of the times in the space after this time we now need to compensate for
		TSortedMap<FFrameNumber, FFrameNumber> ExtraFrames;
		GetFramesInThisSpaceAfterThisTime(ControlRig, ControlKey.Name, ExistingValue, Channel, SectionToKey, Time, ExtraFrames);
		SectionToKey->Modify();
		UMovieSceneControlRigParameterSection* ControlRigSection = Cast<UMovieSceneControlRigParameterSection>(SectionToKey);

		if (ExtraFrames.Num() > 0)
		{
			for (const TPair<FFrameNumber, FFrameNumber>& Frame : ExtraFrames)
			{
				Frames.Add(Frame.Value);
			}
			//if the next space is the same as the new one, then we delete those transform frame and space key if it exists
			FMovieSceneControlRigSpaceBaseKey NextValue;
			using namespace UE::MovieScene;
			FFrameNumber NextSpaceTime = Frames[Frames.Num() - 1] + 1;
			EvaluateChannel(Channel, NextSpaceTime, NextValue);
			if (NextValue == Value)
			{
				int32 ExistingIndex = ChannelInterface.FindKey(NextSpaceTime);
				if (ExistingIndex != INDEX_NONE)
				{
					ChannelInterface.RemoveKey(ExistingIndex);
					if (ControlRigSection)
					{
						FControlRigSpaceChannelHelpers::DeleteTransformKeysAtThisTime(ControlRig, ControlRigSection, ControlKey.Name, NextSpaceTime -1);
						Frames.SetNum(Frames.Num() - 1);
					}
				}
			}

		}
		const FFrameNumber StartFrame = MovieScene->GetPlaybackRange().GetLowerBoundValue();
		if (StartFrame == Time)
		{
			bSetPreviousKey = false;
		}
		//if we have no keys need to set key for current space at start frame, unless setting key at start time, where then don't do previous compensation
		if (Channel->GetNumKeys() == 0)
		{
			if (StartFrame != Time)
			{
				FMovieSceneControlRigSpaceBaseKey Original = ExistingValue;
				ChannelInterface.AddKey(StartFrame, Forward<FMovieSceneControlRigSpaceBaseKey>(Original));
			}
		}

		TArray<FTransform> ControlRigParentWorldTransforms;
		ControlRigParentWorldTransforms.Init(FTransform::Identity, Frames.Num());
		
		TArray<FTransform> ControlWorldTransforms;
		FControlRigSnapper Snapper;
		Snapper.GetControlRigControlTransforms(Sequencer, ControlRig, ControlKey.Name, Frames, ControlRigParentWorldTransforms, ControlWorldTransforms);

		// store tangents to keep the animation when switching 
		TArray<FMovieSceneTangentData> Tangents;
		if (ControlRigSection)
		{
			FChannelMapInfo* pChannelIndex = nullptr;
			FRigControlElement* ControlElement = nullptr;
			int32 NumChannels = 0;
			int32 ChannelIndex = 0;
			Tie(ControlElement, pChannelIndex,NumChannels, ChannelIndex) = GetControlAndChannelInfos(ControlRig, ControlRigSection, ControlKey.Name);

			if (pChannelIndex && ControlElement && NumChannels > 0)
			{
				EvaluateTangentAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels,ControlRigSection, Time, Tangents);
			}
		}
		else
		{
			return Handle;
		}
		
		// if current Value not the same as the previous add new space key
		if (PreviousValue != Value)
		{
			SectionToKey->Modify();
			int32 ExistingIndex = ChannelInterface.FindKey(Time);
			if (ExistingIndex != INDEX_NONE)
			{
				Handle = ChannelInterface.GetHandle(ExistingIndex);
				using namespace UE::MovieScene;
				AssignValue(Channel, Handle, Forward<FMovieSceneControlRigSpaceBaseKey>(Value));
			}
			else
			{
				ExistingIndex = ChannelInterface.AddKey(Time, Forward<FMovieSceneControlRigSpaceBaseKey>(Value));
				Handle = ChannelInterface.GetHandle(ExistingIndex);
			}
		}
		else //same as previous
		{
			if (ControlRigSection)
			{
				ControlRigSection->Modify();
				TArray<FFrameNumber> OurKeyTimes;
				TArray<FKeyHandle> OurKeyHandles;
				TRange<FFrameNumber> CurrentFrameRange;
				CurrentFrameRange.SetLowerBound(TRangeBound<FFrameNumber>(Time));
				CurrentFrameRange.SetUpperBound(TRangeBound<FFrameNumber>(Time));
				ChannelInterface.GetKeys(CurrentFrameRange, &OurKeyTimes, &OurKeyHandles);
				if (OurKeyHandles.Num() > 0)
				{
					ChannelInterface.DeleteKeys(OurKeyHandles);
				}

				// now delete any extra TimeOfDelete -1
				// NOTE do we need to update the tangents here?
				DeleteTransformKeysAtThisTime(ControlRig, ControlRigSection, ControlKey.Name, Time - 1);
				bSetPreviousKey = false; //also don't set previous
			}
		}

		FRigControlModifiedContext Context(EControlRigSetKey::Always);
		constexpr bool bNoUndo = false,  bNoPython = false, bFixEulerFlips = true;

		//do any compensation or previous key adding
		// set previous key if we're going to switch space
		if (bSetPreviousKey)
		{
			FFrameTime GlobalTime(Time - 1);
			GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime);

			EvaluateSequencer(GlobalTime);

			Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Time - 1));
			if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name))
			{
				UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(ControlWorldTransforms[0], ControlRig, ControlElement, Context);
			}
			if (ControlRig->IsAdditive())
			{
				// We need to evaluate in order to trigger a notification
				ControlRig->Evaluate_AnyThread();
			}

			//need to do this after eval
			FChannelMapInfo* pChannelIndex = nullptr;
			FRigControlElement* ControlElement = nullptr;
			int32 NumChannels = 0;
			int32 ChannelIndex = 0;
			Tie(ControlElement, pChannelIndex, NumChannels, ChannelIndex) = GetControlAndChannelInfos(ControlRig, ControlRigSection, ControlKey.Name);

			if (pChannelIndex && ControlElement && NumChannels > 0)
			{
				// need to update the tangents here to keep the arriving animation
				SetTangentsAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, ControlRigSection, GlobalTime.GetFrame(), Tangents);
			}
		}

		// effectively switch to new space
		ControlRig->SwitchToParent(ControlKey, SpaceKey, false, true);
		// add new keys in the new space context
		{
			const FScopedLegacyEvaluationTrack ScopedLegacyEvaluationTrack(ControlRigSection);
			
			int32 FramesIndex = 0;
			for (const FFrameNumber& Frame : Frames)
			{
				FFrameTime GlobalTime(Frame);
				GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime);

				EvaluateSequencer(GlobalTime);

				ControlRig->Evaluate_AnyThread();

				Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
				if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name))
				{
					UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(ControlWorldTransforms[FramesIndex], ControlRig, ControlElement, Context);
				}

				if (ControlRig->IsAdditive())
				{
					// We need to evaluate in order to trigger a notification
					ControlRig->Evaluate_AnyThread();
				}
			
				// need to update the tangents here to keep the leaving animation
				if (ControlRigSection && bSetPreviousKey && FramesIndex == 0)
				{
					//need to do this after eval
					FChannelMapInfo* pChannelIndex = nullptr;
					FRigControlElement* ControlElement = nullptr;
					int32 NumChannels = 0;
					int32 ChannelIndex = 0;
					Tie(ControlElement, pChannelIndex, NumChannels, ChannelIndex) = GetControlAndChannelInfos(ControlRig, ControlRigSection, ControlKey.Name);
					
					if (NumChannels > 0) //-V547
					{
						SetTangentsAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, ControlRigSection, GlobalTime.GetFrame(), Tangents);
					}
				}

				FramesIndex++;
			}
		}
		Channel->FindSpaceIntervals();
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

		Channel->GetUniqueSpaceList(&AfterKeys);
		Channel->BroadcastSpaceNoLongerUsed(BeforeKeys, AfterKeys);
	}

	return Handle;

}

//get the mask of the transform keys at the specified time
static EControlRigContextChannelToKey GetCurrentTransformKeysAtThisTime(UControlRig* ControlRig, FName ControlName, UMovieSceneSection* SectionToKey, FFrameNumber Time)
{
	EControlRigContextChannelToKey ChannelToKey = EControlRigContextChannelToKey::None;
	if (ControlRig && SectionToKey)
	{
		if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey))
		{
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlName);
			if (pChannelIndex != nullptr)
			{
				int32 ChannelIndex = pChannelIndex->ChannelIndex;

				if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
				{
					switch (ControlElement->Settings.ControlType)
					{
					case ERigControlType::Position:
					case ERigControlType::Scale:
					case ERigControlType::Rotator:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						if (FloatChannels[ChannelIndex]->GetData().FindKey(Time) != INDEX_NONE)
						{
							if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
							{
								ChannelToKey |= EControlRigContextChannelToKey::RotationX;
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::Scale)
							{
								ChannelToKey |= EControlRigContextChannelToKey::ScaleX;
							}
							else
							{
								ChannelToKey |= EControlRigContextChannelToKey::TranslationX;
							}
						}
						if (FloatChannels[ChannelIndex + 1]->GetData().FindKey(Time) != INDEX_NONE)
						{
							if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
							{
								ChannelToKey |= EControlRigContextChannelToKey::RotationY;
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::Scale)
							{
								ChannelToKey |= EControlRigContextChannelToKey::ScaleY;
							}
							else
							{
								ChannelToKey |= EControlRigContextChannelToKey::TranslationY;
							}
						}
						if (FloatChannels[ChannelIndex + 2]->GetData().FindKey(Time) != INDEX_NONE)
						{
							if (ControlElement->Settings.ControlType == ERigControlType::Rotator)
							{
								ChannelToKey |= EControlRigContextChannelToKey::RotationZ;
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::Scale)
							{
								ChannelToKey |= EControlRigContextChannelToKey::ScaleZ;
							}
							else
							{
								ChannelToKey |= EControlRigContextChannelToKey::TranslationZ;
							}
						}
						if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
							ControlElement->Settings.ControlType == ERigControlType::EulerTransform ||
							ControlElement->Settings.ControlType == ERigControlType::TransformNoScale
							)
						{
							if (FloatChannels[ChannelIndex + 3]->GetData().FindKey(Time) != INDEX_NONE)
							{
								ChannelToKey |= EControlRigContextChannelToKey::RotationX;
							}
							if (FloatChannels[ChannelIndex + 4]->GetData().FindKey(Time) != INDEX_NONE)
							{
								ChannelToKey |= EControlRigContextChannelToKey::RotationY;
							}
							if (FloatChannels[ChannelIndex + 5]->GetData().FindKey(Time) != INDEX_NONE)
							{
								ChannelToKey |= EControlRigContextChannelToKey::RotationZ;
							}

						}
						if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
							ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
						{
							if (FloatChannels[ChannelIndex + 6]->GetData().FindKey(Time) != INDEX_NONE)
							{
								ChannelToKey |= EControlRigContextChannelToKey::ScaleX;
							}
							if (FloatChannels[ChannelIndex + 7]->GetData().FindKey(Time) != INDEX_NONE)
							{
								ChannelToKey |= EControlRigContextChannelToKey::ScaleY;
							}
							if (FloatChannels[ChannelIndex + 8]->GetData().FindKey(Time) != INDEX_NONE)
							{
								ChannelToKey |= EControlRigContextChannelToKey::ScaleZ;
							}
						}
						break;

					};

					}
				}
			}
		
		}

	}
	return ChannelToKey;
}

void  FControlRigSpaceChannelHelpers::SequencerSpaceChannelKeyDeleted(UControlRig* ControlRig, ISequencer* Sequencer, FName ControlName, FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneControlRigParameterSection* SectionToKey,
	FFrameNumber TimeOfDeletion)
{
	if (bDoNotCompensate == true || IsInGameThread() == false) //this may happen in an non game thread via a parallel for when we are deleting many channels
	{															//if that's the case we assume the user may not want to compensate
		return;
	}
	FMovieSceneControlRigSpaceBaseKey ExistingValue, PreviousValue;
	using namespace UE::MovieScene;
	EvaluateChannel(Channel, TimeOfDeletion - 1, PreviousValue);
	EvaluateChannel(Channel, TimeOfDeletion, ExistingValue);
	if (ExistingValue != PreviousValue) //if they are the same no need to do anything
	{
		TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);

		//find all key frames we need to compensate
		TArray<FFrameNumber> Frames;
		Frames.Add(TimeOfDeletion);
		TSortedMap<FFrameNumber, FFrameNumber> ExtraFrames;
		FControlRigSpaceChannelHelpers::GetFramesInThisSpaceAfterThisTime(ControlRig, ControlName, ExistingValue,
			Channel, SectionToKey, TimeOfDeletion, ExtraFrames);
		if (ExtraFrames.Num() > 0)
		{
			for (const TPair<FFrameNumber, FFrameNumber>& Frame : ExtraFrames)
			{
				Frames.Add(Frame.Value);
			}
		}
		TArray<FTransform> ControlRigParentWorldTransforms;
		ControlRigParentWorldTransforms.SetNum(Frames.Num());
		for (FTransform& WorldTransform : ControlRigParentWorldTransforms)
		{
			WorldTransform = FTransform::Identity;
		}
		TArray<FTransform> ControlWorldTransforms;
		FControlRigSnapper Snapper;
		Snapper.GetControlRigControlTransforms(Sequencer, ControlRig, ControlName, Frames, ControlRigParentWorldTransforms, ControlWorldTransforms);
		FRigElementKey ControlKey;
		ControlKey.Name = ControlName;
		ControlKey.Type = ERigElementType::Control;
		URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();

		int32 FramesIndex = 0;
		FRigControlModifiedContext Context;
		Context.SetKey = EControlRigSetKey::Always;
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransform();
		FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

		for (const FFrameNumber& Frame : Frames)
		{
			//evaluate sequencer
			FFrameTime GlobalTime(Frame);
			GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime);

			FMovieSceneContext SceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
			Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(SceneContext);
			//make sure to set rig hierarchy correct since key is not deleted yet
			switch (PreviousValue.SpaceType)
			{
			case EMovieSceneControlRigSpaceType::Parent:
				ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
				break;
			case EMovieSceneControlRigSpaceType::World:
				ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
				break;
			case EMovieSceneControlRigSpaceType::ControlRig:
				ControlRig->SwitchToParent(ControlKey, PreviousValue.ControlRigElement, false, true);
				break;
			}
			ControlRig->Evaluate_AnyThread();
			Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
			//make sure we only add keys to those that exist since they may be getting deleted also.
			Context.KeyMask = (uint32)GetCurrentTransformKeysAtThisTime(ControlRig, ControlKey.Name, SectionToKey, Frame);
			if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name))
			{
				UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(ControlWorldTransforms[FramesIndex], ControlRig, ControlElement, Context);
			}
			++FramesIndex;
		}
		//now delete any extra TimeOfDelete -1
		FControlRigSpaceChannelHelpers::DeleteTransformKeysAtThisTime(ControlRig, SectionToKey, ControlName, TimeOfDeletion - 1);
		
	}
}

void FControlRigSpaceChannelHelpers::DeleteTransformKeysAtThisTime(UControlRig* ControlRig, UMovieSceneControlRigParameterSection* Section, FName ControlName, FFrameNumber Time)
{
	if (Section && ControlRig)
	{
		TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
		FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlName);
		if (pChannelIndex != nullptr)
		{
			int32 ChannelIndex = pChannelIndex->ChannelIndex;

			if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
			{
				switch (ControlElement->Settings.ControlType)
				{
					case ERigControlType::Position:
					case ERigControlType::Scale:
					case ERigControlType::Rotator:
					case ERigControlType::Transform:
					case ERigControlType::TransformNoScale:
					case ERigControlType::EulerTransform:
					{
						int NumChannels = 0;
						if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
							ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
						{
							NumChannels = 9;
						}
						else if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
						{
							NumChannels = 6;
						}
						else //vectors
						{
							NumChannels = 3;
						}
						for (int32 Index = 0; Index < NumChannels; ++Index)
						{
							int32 KeyIndex = 0;
							for (FFrameNumber Frame : FloatChannels[ChannelIndex]->GetData().GetTimes())
							{
								if (Frame == Time)
								{
									FloatChannels[ChannelIndex]->GetData().RemoveKey(KeyIndex);
									break;
								}
								else if (Frame > Time)
								{
									break;
								}
								++KeyIndex;
							}
							++ChannelIndex;
						}
						break;
					}
					default:
						break;

				}
			}
		}
	}
}

void FControlRigSpaceChannelHelpers::GetFramesInThisSpaceAfterThisTime(
	UControlRig* ControlRig,
	FName ControlName,
	FMovieSceneControlRigSpaceBaseKey CurrentValue,
	FMovieSceneControlRigSpaceChannel* Channel,
	UMovieSceneSection* SectionToKey,
	FFrameNumber Time,
	TSortedMap<FFrameNumber,FFrameNumber>& OutMoreFrames)
{
	OutMoreFrames.Reset();
	if (ControlRig && Channel && SectionToKey)
	{
		if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey))
		{
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlName);
			if (pChannelIndex != nullptr)
			{
				int32 ChannelIndex = pChannelIndex->ChannelIndex;

				if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
				{
					FMovieSceneControlRigSpaceBaseKey Value;
					switch (ControlElement->Settings.ControlType)
					{
						case ERigControlType::Position:
						case ERigControlType::Scale:
						case ERigControlType::Rotator:
						case ERigControlType::Transform:
						case ERigControlType::TransformNoScale:
						case ERigControlType::EulerTransform:
						{
							int NumChannels = 0;
							if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
								ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
							{
								NumChannels = 9;
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
							{
								NumChannels = 6;
							}
							else //vectors
							{
								NumChannels = 3;
							}
							for (int32 Index = 0; Index < NumChannels; ++Index)
							{
								for (FFrameNumber Frame : FloatChannels[ChannelIndex++]->GetData().GetTimes())
								{
									if (Frame > Time)
									{
										using namespace UE::MovieScene;
										EvaluateChannel(Channel, Frame, Value);
										if (CurrentValue == Value)
										{
											if (OutMoreFrames.Find(Frame) == nullptr)
											{
												OutMoreFrames.Add(Frame, Frame);
											}
										}
										else
										{
											break;
										}
									}
								}
							}
							break;
						}

					}
				}
			}
		}
	}
}

void FControlRigSpaceChannelHelpers::SequencerBakeControlInSpace(UControlRig* ControlRig, ISequencer* Sequencer, FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey,
	URigHierarchy* RigHierarchy, const FRigElementKey& ControlKey, FRigSpacePickerBakeSettings Settings)
{
	if (bDoNotCompensate == true)
	{
		return;
	}
	if (ControlRig && Sequencer && Channel && SectionToKey && (Settings.Settings.StartFrame != Settings.Settings.EndFrame) && RigHierarchy
		&& Sequencer->GetFocusedMovieSceneSequence() && Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
	
		if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey))
		{
			//get the setting as a rig element key to use for comparisons for smart baking, 
			FMovieSceneControlRigSpaceBaseKey Value;
			if (Settings.TargetSpace == RigHierarchy->GetWorldSpaceReferenceKey())
			{
				Value.SpaceType = EMovieSceneControlRigSpaceType::World;
			}
			else if (Settings.TargetSpace == RigHierarchy->GetDefaultParentKey())
			{
				Value.SpaceType = EMovieSceneControlRigSpaceType::Parent;
			}
			else
			{
				FRigElementKey DefaultParent = RigHierarchy->GetFirstParent(ControlKey);
				if (DefaultParent == Settings.TargetSpace)
				{
					Value.SpaceType = EMovieSceneControlRigSpaceType::Parent;
				}
				else
				{
					Value.SpaceType = EMovieSceneControlRigSpaceType::ControlRig;
					Value.ControlRigElement = Settings.TargetSpace;
				}
			}
			TArrayView<FMovieSceneFloatChannel*> Channels = FControlRigSequencerHelpers::GetFloatChannels(ControlRig,
				ControlKey.Name, Section);
			//if smart baking we 1) Don't do any baking over times where the Settings.TargetSpace is active, 
			//and 2) we just bake over key frames
			//need to store these times first since we will still delete all of these space keys

			TArray<FFrameNumber> Frames;
			const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
			const FFrameRate  FrameRate = Sequencer->GetFocusedDisplayRate();
			//get space keys so we can delete them and use them for smart baking at them
			TArray<FFrameNumber> Keys;
			TArray < FKeyHandle> KeyHandles;
			FFrameNumber StartFrame = Settings.Settings.StartFrame;
			FFrameNumber EndFrame = Settings.Settings.EndFrame;
			TRange<FFrameNumber> Range(StartFrame, EndFrame);
			Channel->GetKeys(Range, &Keys, &KeyHandles);
			TArray<FFrameNumber> MinusOneFrames; //frames we need to delete
			TArray<TPair<FFrameNumber, TArray<FMovieSceneTangentData>>> StoredTangents; //need to set tangents at the space switch locations
			TSet<FFrameNumber> ParentFrames; // these are parent frames we keep track of so we can 
			if (Settings.Settings.BakingKeySettings == EBakingKeySettings::KeysOnly)
			{
				TSet<FFrameNumber> MinusOneKeys; 
				TSortedMap<FFrameNumber,FFrameNumber> FrameMap;
				//add space keys to bake
				{
					for (FFrameNumber& Frame : Keys)
					{
						FMovieSceneControlRigSpaceBaseKey SpaceValue;
						using namespace UE::MovieScene;
						EvaluateChannel(Channel, Frame, SpaceValue);
						if (SpaceValue != Value)
						{
							FChannelMapInfo* pChannelIndex = nullptr;
							FRigControlElement* ControlElement = nullptr;
							int32 NumChannels = 0;
							int32 ChannelIndex = 0;
							Tie(ControlElement, pChannelIndex, NumChannels, ChannelIndex) = GetControlAndChannelInfos(ControlRig, Section, ControlKey.Name);

							if (pChannelIndex && ControlElement && NumChannels > 0)
							{
								TPair<FFrameNumber, TArray<FMovieSceneTangentData>> FrameAndTangent;
								FrameAndTangent.Key = Frame;
								EvaluateTangentAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, Section, Frame, FrameAndTangent.Value);
								StoredTangents.Add(FrameAndTangent);
							}
							FrameMap.Add(Frame,Frame);
						}
						MinusOneKeys.Add(Frame - 1);
					}
				}

				TArray<FFrameNumber> TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
					Channels, Settings.Settings.StartFrame, Settings.Settings.EndFrame);
				//add transforms keys to bake
				{
					for (FFrameNumber& Frame : TransformFrameTimes)
					{
						if (MinusOneKeys.Contains(Frame) == false)
						{
							FrameMap.Add(Frame);
						}
					}
				}
				//also need to add transform keys for the parent control rig intervals over the time we bake
				TArray<FSpaceRange> SpaceRanges = Channel->FindSpaceIntervals();
				for (const FSpaceRange& SpaceRange : SpaceRanges)
				{
					if (SpaceRange.Key.SpaceType == EMovieSceneControlRigSpaceType::ControlRig && SpaceRange.Key.ControlRigElement.IsValid())
					{
						TRange<FFrameNumber> Overlap = TRange<FFrameNumber>::Intersection(Range, SpaceRange.Range);
						if (Overlap.IsEmpty() == false)
						{
							TArrayView<FMovieSceneFloatChannel*> ParentChannels = FControlRigSequencerHelpers::GetFloatChannels(ControlRig,
								SpaceRange.Key.ControlRigElement.Name, Section);

							TArray<FFrameNumber> ParentFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
								ParentChannels, StartFrame, EndFrame);
							for (FFrameNumber& Frame : ParentFrameTimes)
							{
								//if we don't have a key at the parent time, add the frame AND store it's tangent.
								if (FrameMap.Contains(Frame) == false)
								{
									FrameMap.Add(Frame);
									ParentFrames.Add(Frame);

									FChannelMapInfo* pChannelIndex = nullptr;
									FRigControlElement* ControlElement = nullptr;
									int32 NumChannels = 0;
									int32 ChannelIndex = 0;
									Tie(ControlElement, pChannelIndex, NumChannels, ChannelIndex) = GetControlAndChannelInfos(ControlRig, Section, SpaceRange.Key.ControlRigElement.Name);
									TPair<FFrameNumber, TArray<FMovieSceneTangentData>> FrameAndTangent;
									FrameAndTangent.Key = Frame;
									EvaluateTangentAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, Section, Frame, FrameAndTangent.Value);
									StoredTangents.Add(FrameAndTangent);
								}
							}
						}
					}
				
				}
				FrameMap.GenerateKeyArray(Frames);

			}
			else
			{
				FFrameNumber FrameRateInFrameNumber = TickResolution.AsFrameNumber(FrameRate.AsInterval());
				if (Settings.Settings.FrameIncrement > 1) //increment per frame by increment
				{
					FrameRateInFrameNumber.Value *= Settings.Settings.FrameIncrement;
				}
				for (FFrameNumber& Frame = Settings.Settings.StartFrame; Frame <= Settings.Settings.EndFrame; Frame += FrameRateInFrameNumber)
				{
					Frames.Add(Frame);
				}

			}
			if (Frames.Num() == 0)
			{
				return;
			}
			TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);
			TArray<FTransform> ControlRigParentWorldTransforms;
			ControlRigParentWorldTransforms.SetNum(Frames.Num());
			for (FTransform& Transform : ControlRigParentWorldTransforms)
			{
				Transform = FTransform::Identity;
			}
			//Store transforms
			FControlRigSnapper Snapper;
			TArray<FTransform> ControlWorldTransforms;
			Snapper.GetControlRigControlTransforms(Sequencer, ControlRig, ControlKey.Name, Frames, ControlRigParentWorldTransforms, ControlWorldTransforms);

			//Need to delete keys, if smart baking we only delete the -1 keys,
			//if not, we need to delete all transform
			//and always delete space keys
			Section->Modify();
			Channel->DeleteKeys(KeyHandles);
			if (Settings.Settings.BakingKeySettings == EBakingKeySettings::KeysOnly)
			{
				for (const FFrameNumber& DeleteFrame : Keys)
				{
					FControlRigSpaceChannelHelpers::DeleteTransformKeysAtThisTime(ControlRig, Section, ControlKey.Name, DeleteFrame - 1);
				}
			}
			else
			{
				FMovieSceneConstraintChannelHelper::DeleteTransformTimes(Channels, StartFrame, EndFrame);
			}
			

			URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

			//now find space at start and end see if different than the new space if so we need to compensate
			FMovieSceneControlRigSpaceBaseKey StartFrameValue, EndFrameValue;
			using namespace UE::MovieScene;
			EvaluateChannel(Channel, StartFrame, StartFrameValue);
			EvaluateChannel(Channel, EndFrame, EndFrameValue);
			
			const bool bCompensateStart = StartFrameValue != Value;
			TRange<FFrameNumber> PlaybackRange = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
			const bool bCompensateEnd = (EndFrameValue != Value && PlaybackRange.GetUpperBoundValue() != EndFrame);

			//if compensate at the start we need to set the channel key as the new value
			if (bCompensateStart)
			{
				TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelInterface = Channel->GetData();
				ChannelInterface.AddKey(StartFrame, Forward<FMovieSceneControlRigSpaceBaseKey>(Value));

			}
			//if we compensate at the end we change the last frame to frame -1(tick), and then later set the space to the other one and 
			if (bCompensateEnd)
			{
				Frames[Frames.Num() - 1] = Frames[Frames.Num() - 1] - 1;
			}
			//now set all of the key values
			FRigControlModifiedContext Context;
			Context.SetKey = EControlRigSetKey::Always;
			Context.KeyMask = (uint32)EControlRigContextChannelToKey::AllTransform;
			ControlRig->SwitchToParent(ControlKey, Settings.TargetSpace, false, true);
			ControlRig->Evaluate_AnyThread();

			FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransform();
			FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

			for (int32 Index = 0; Index < Frames.Num(); ++Index)
			{
				const FTransform GlobalTransform = ControlWorldTransforms[Index];
				const FFrameNumber Frame = Frames[Index];

				//evaluate sequencer
				FFrameTime GlobalTime(Frame);
				GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime);

				FMovieSceneContext SceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
				Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(SceneContext);

				//evaluate control rig
				ControlRig->Evaluate_AnyThread();
				Context.LocalTime = TickResolution.AsSeconds(FFrameTime(Frame));
				
				//if doing smart baking only set keys on those that exist IF not a key from the parent, in that case we need to key the  whole transform.
				if (Settings.Settings.BakingKeySettings == EBakingKeySettings::KeysOnly && ParentFrames.Contains(Frame) == false)
				{
					Context.KeyMask = (uint32)GetCurrentTransformKeysAtThisTime(ControlRig, ControlKey.Name, SectionToKey, Frame);
				}
				else
				{
					Context.KeyMask = (uint32)EControlRigContextChannelToKey::AllTransform;
				}
				if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name))
				{
					UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(GlobalTransform, ControlRig, ControlElement, Context);
				}
			}

			//if end compensated set the space that was active previously and set the compensated global value
			if (bCompensateEnd)
			{
				//EndFrameValue to SpaceKey todoo move to function
				switch (EndFrameValue.SpaceType)
				{
				case EMovieSceneControlRigSpaceType::Parent:
					ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
					break;
				case EMovieSceneControlRigSpaceType::World:
					ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
					break;
				case EMovieSceneControlRigSpaceType::ControlRig:
					ControlRig->SwitchToParent(ControlKey, EndFrameValue.ControlRigElement, false, true);
					break;
				}

				//evaluate sequencer
				FFrameTime GlobalTime(EndFrame);
				GlobalTime = LocalToRootTransform.TryTransformTime(GlobalTime).Get(GlobalTime);

				FMovieSceneContext SceneContext = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
				Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(SceneContext);
		
				//evaluate control rig
				ControlRig->Evaluate_AnyThread();

				TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelInterface = Channel->GetData();
				ChannelInterface.AddKey(EndFrame, Forward<FMovieSceneControlRigSpaceBaseKey>(EndFrameValue));
				FTransform GlobalTransform = ControlWorldTransforms[Frames.Num() - 1];
				Context.LocalTime = TickResolution.AsSeconds(FFrameTime(EndFrame));
				Context.KeyMask = (uint32)EControlRigContextChannelToKey::AllTransform;
				if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlKey.Name))
				{
					UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(GlobalTransform, ControlRig, ControlElement, Context);
				}
			}
			//Fix tangents at removed space switches if needed
			for (TPair<FFrameNumber, TArray<FMovieSceneTangentData>>& StoredTangent : StoredTangents)
			{
				FChannelMapInfo* pChannelIndex = nullptr;
				FRigControlElement* ControlElement = nullptr;
				int32 NumChannels = 0;
				int32 ChannelIndex = 0;
				Tie(ControlElement, pChannelIndex, NumChannels, ChannelIndex) = GetControlAndChannelInfos(ControlRig, Section, ControlKey.Name);

				if (pChannelIndex && ControlElement && NumChannels > 0)
				{
					SetTangentsAtThisTime<FMovieSceneFloatChannel>(ChannelIndex, NumChannels, Section, StoredTangent.Key, StoredTangent.Value);
				}
			}
			// Fix any Rotation Channels
			Section->FixRotationWinding(ControlKey.Name, Frames[0], Frames[Frames.Num() - 1]);
			// Then optimize
			if (Settings.Settings.BakingKeySettings == EBakingKeySettings::AllFrames && Settings.Settings.bReduceKeys)
			{
				FKeyDataOptimizationParams Params;
				Params.bAutoSetInterpolation = true;
				Params.Tolerance = Settings.Settings.Tolerance;
				Params.Range = TRange <FFrameNumber>(Frames[0], Frames[Frames.Num() - 1]);
				Section->OptimizeSection(ControlKey.Name, Params);
			}
			else  //need to auto set tangents, the above section will do that also
			{
				Section->AutoSetTangents(ControlKey.Name);
			}	
		}
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded); //may have added channel
	}
}

void FControlRigSpaceChannelHelpers::HandleSpaceKeyTimeChanged(UControlRig* ControlRig, FName ControlName, FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey,
	FFrameNumber CurrentFrame, FFrameNumber NextFrame)
{
	if (ControlRig && Channel && SectionToKey && (CurrentFrame != NextFrame))
	{
		if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey))
		{
			TArrayView<FMovieSceneFloatChannel*> FloatChannels = Section->GetChannelProxy().GetChannels<FMovieSceneFloatChannel>();
			FChannelMapInfo* pChannelIndex = Section->ControlChannelMap.Find(ControlName);
			if (pChannelIndex != nullptr)
			{
				int32 ChannelIndex = pChannelIndex->ChannelIndex;
				FFrameNumber Delta = NextFrame - CurrentFrame;
				if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
				{
					switch (ControlElement->Settings.ControlType)
					{
						case ERigControlType::Position:
						case ERigControlType::Scale:
						case ERigControlType::Rotator:
						case ERigControlType::Transform:
						case ERigControlType::TransformNoScale:
						case ERigControlType::EulerTransform:
						{
							int NumChannels = 0;
							if (ControlElement->Settings.ControlType == ERigControlType::Transform ||
								ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
							{
								NumChannels = 9;
							}
							else if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
							{
								NumChannels = 6;
							}
							else //vectors
							{
								NumChannels = 3;
							}
							for (int32 Index = 0; Index < NumChannels; ++Index)
							{
								FMovieSceneFloatChannel* FloatChannel = FloatChannels[ChannelIndex++];
								if (Delta > 0) //if we are moving keys positively in time we start from end frames and move them so we can use indices
								{
									for (int32 KeyIndex = FloatChannel->GetData().GetTimes().Num() - 1; KeyIndex >= 0; --KeyIndex)
									{
										const FFrameNumber Frame = FloatChannel->GetData().GetTimes()[KeyIndex];
										FFrameNumber Diff = Frame - CurrentFrame;
										FFrameNumber AbsDiff = Diff < 0 ? -Diff : Diff;
										if (AbsDiff <= 1)
										{
											FFrameNumber NewKeyTime = Frame + Delta;
											FloatChannel->GetData().MoveKey(KeyIndex, NewKeyTime);
										}
									}
								}
								else
								{
									for (int32 KeyIndex = 0; KeyIndex < FloatChannel->GetData().GetTimes().Num(); ++KeyIndex)
									{
										const FFrameNumber Frame = FloatChannel->GetData().GetTimes()[KeyIndex];
										FFrameNumber Diff = Frame - CurrentFrame;
										FFrameNumber AbsDiff = Diff < 0 ? -Diff : Diff;
										if (AbsDiff <= 1)
										{
											FFrameNumber NewKeyTime = Frame + Delta;
											FloatChannel->GetData().MoveKey(KeyIndex, NewKeyTime);
										}
									}
								}
							}
							break;
						}
						default:
							break;
					}
				}
			}
		}
	}
}

UMovieSceneControlRigParameterSection* FControlRigSpaceChannelHelpers::GetControlRigSection(ISequencer* Sequencer, const UControlRig* ControlRig)
{

	if (ControlRig == nullptr || Sequencer == nullptr)
	{
		return nullptr;
	}
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}
	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(MovieScene->FindTrack(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None));
		if (ControlRigParameterTrack && ControlRigParameterTrack->GetControlRig() == ControlRig)
		{
			UMovieSceneControlRigParameterSection* ActiveSection = Cast<UMovieSceneControlRigParameterSection>(ControlRigParameterTrack->GetSectionToKey());
			if (ActiveSection)
			{
				return ActiveSection;
			}
		}
	}
	return nullptr;
}

void FControlRigSpaceChannelHelpers::CompensateIfNeeded(
	UControlRig* ControlRig,
	ISequencer* Sequencer,
	UMovieSceneControlRigParameterSection* Section,
	const TOptional<FFrameNumber>& OptionalTime, 
	bool bCompPreviousTick)
{
	if (bDoNotCompensate == true)
	{
		return;
	}

	if (!ControlRig || ControlRig->IsEvaluating())
	{
		// avoid reentrant evaluation: space switching evaluates the rig at T-1, but this must not happen if it's already being evaluated.
		return;
	}

	// Frames to compensate
	TArray<FFrameNumber> OptionalTimeArray;
	if(OptionalTime.IsSet())
	{
		OptionalTimeArray.Add(OptionalTime.GetValue());
	}
	
	auto GetSpaceTimesToCompensate = [&OptionalTimeArray](const FSpaceControlNameAndChannel* Channel)->TArrayView<const FFrameNumber>
	{
		if (OptionalTimeArray.IsEmpty())
		{
			return Channel->SpaceCurve.GetData().GetTimes();
		}
		return OptionalTimeArray;
	};

	// keyframe context
	FRigControlModifiedContext KeyframeContext;
	KeyframeContext.SetKey = EControlRigSetKey::Always;
	const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
	
	//we need to check all controls for 1) space and 2) previous frame and if so we automatically compensate.
	bool bDidIt = false;

	// compensate spaces
	URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();
	const TArray<FRigControlElement*> Controls = RigHierarchy->GetControls();
	for (FRigControlElement* Control: Controls)
	{ 
		if(Control)
		{ 
			//only if we have a channel
			if (FSpaceControlNameAndChannel* Channel = Section->GetSpaceChannel(Control->GetFName()))
			{
				const TArrayView<const FFrameNumber> FramesToCompensate = GetSpaceTimesToCompensate(Channel);
				if (FramesToCompensate.Num() > 0)
				{
					for (const FFrameNumber& Time : FramesToCompensate)
					{
						const FFrameNumber TimeToCompensate = bCompPreviousTick ? (Time - 1) : Time;
						const FFrameNumber TimeToCompare = bCompPreviousTick ? Time : (Time - 1);
						FMovieSceneControlRigSpaceBaseKey ExistingValue, PreviousValue;
						using namespace UE::MovieScene;
						EvaluateChannel(&(Channel->SpaceCurve), TimeToCompensate, PreviousValue);
						EvaluateChannel(&(Channel->SpaceCurve), TimeToCompare, ExistingValue);

						if (ExistingValue != PreviousValue) //if they are the same no need to do anything
						{
							TGuardValue<bool> CompensateGuard(bDoNotCompensate, true);
							
							//find global value at current time
							TArray<FTransform> ControlRigParentWorldTransforms({FTransform::Identity});
							TArray<FTransform> ControlWorldTransforms;
							FControlRigSnapper Snapper;
							Snapper.GetControlRigControlTransforms(
								Sequencer, ControlRig, Control->GetFName(),
								{ TimeToCompare },
								ControlRigParentWorldTransforms, ControlWorldTransforms);

							//set space to previous space value that's different.
							const FRigElementKey ControlKey = Control->GetKey();
							switch (PreviousValue.SpaceType)
							{
								case EMovieSceneControlRigSpaceType::Parent:
									ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetDefaultParent(ControlKey), false, true);
									break;
								case EMovieSceneControlRigSpaceType::World:
									ControlRig->SwitchToParent(ControlKey, RigHierarchy->GetWorldSpaceReferenceKey(), false, true);
									break;
								case EMovieSceneControlRigSpaceType::ControlRig:
									ControlRig->SwitchToParent(ControlKey, PreviousValue.ControlRigElement, false, true);
									break;
							}
							
							//now set time -1 frame value
							ControlRig->Evaluate_AnyThread();
							KeyframeContext.LocalTime = TickResolution.AsSeconds(FFrameTime(TimeToCompensate));
							{
								UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(ControlWorldTransforms[0], ControlRig, Control, KeyframeContext);
							}				
							bDidIt = true;
						}
					}
				}
			}
		}
	}
	
	if (bDidIt)
	{
		Sequencer->ForceEvaluate();
	}
}

FLinearColor FControlRigSpaceChannelHelpers::GetColor(const FMovieSceneControlRigSpaceBaseKey& Key)
{
	const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
	static TMap<FName, FLinearColor> ColorsForSpaces;
	switch (Key.SpaceType)
	{
		case EMovieSceneControlRigSpaceType::Parent:
		{
			TOptional<FLinearColor> OptColor = Settings->GetSpaceSwitchColor(FString(TEXT("Parent")));
			if (OptColor.IsSet())
			{
				return OptColor.GetValue();
			}

			return  FLinearColor(.93f, .31f, .19f); //pastel orange

		}
		case EMovieSceneControlRigSpaceType::World:
		{
			TOptional<FLinearColor> OptColor = Settings->GetSpaceSwitchColor(FString(TEXT("World")));
			if (OptColor.IsSet())
			{
				return OptColor.GetValue();
			}

			return  FLinearColor(.198f, .610f, .558f); //pastel teal
		}
		case EMovieSceneControlRigSpaceType::ControlRig:
		{
			TOptional<FLinearColor> OptColor = Settings->GetSpaceSwitchColor(Key.ControlRigElement.Name.ToString());
			if (OptColor.IsSet())
			{
				return OptColor.GetValue();
			}
			if (FLinearColor* Color = ColorsForSpaces.Find(Key.ControlRigElement.Name))
			{
				return *Color;
			}
			else
			{
				FLinearColor RandomColor = UCurveEditorSettings::GetNextRandomColor();
				ColorsForSpaces.Add(Key.ControlRigElement.Name, RandomColor);
				return RandomColor;
			}
		}
	};
	return  FLinearColor(FColor::White);

}

FReply FControlRigSpaceChannelHelpers::OpenBakeDialog(ISequencer* Sequencer, FMovieSceneControlRigSpaceChannel* Channel,int32 KeyIndex, UMovieSceneSection* SectionToKey )
{

	if (!Sequencer || !Sequencer->GetFocusedMovieSceneSequence()|| !Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene())
	{
		return FReply::Unhandled();
	}
	if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(SectionToKey))
	{
		if (UControlRig* ControlRig = Section->GetControlRig())
		{
			FName ControlName = Section->FindControlNameFromSpaceChannel(Channel);

			if (ControlName == NAME_None)
			{
				return FReply::Unhandled();
			}
			FRigSpacePickerBakeSettings Settings;

			const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

			URigHierarchy* RigHierarchy = ControlRig->GetHierarchy();
			Settings.TargetSpace = URigHierarchy::GetDefaultParentKey();

			TRange<FFrameNumber> Range = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
			Settings.Settings.StartFrame = Range.GetLowerBoundValue();
			Settings.Settings.EndFrame = Range.GetUpperBoundValue();
			TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelData = Channel->GetData();
			TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
			if (KeyIndex >= 0 && KeyIndex < Times.Num())
			{
				Settings.Settings.StartFrame = Times[KeyIndex];
				if (KeyIndex + 1 < Times.Num())
				{
					Settings.Settings.EndFrame = Times[KeyIndex + 1];
				}
			}
			TArray<FRigElementKey> Controls;
			FRigElementKey Key;
			Key.Name = ControlName;
			Key.Type = ERigElementType::Control;
			Controls.Add(Key);

			using SRigSpacePickerBakeDialog = UE::ControlRigEditor::SRigSpacePickerBakeDialog;
			const TSharedRef<SRigSpacePickerBakeDialog> BakeWidget =
				SNew(SRigSpacePickerBakeDialog)
				.Settings(Settings)
				.WeakHierarchy(RigHierarchy)
				.Controls(Controls);

			return BakeWidget->OpenDialog(true);
			
		}
	}
	return FReply::Unhandled();
}

TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>& ChannelHandle, const UE::Sequencer::FCreateCurveEditorModelParams& Params)
{
	if (FMovieSceneControlRigSpaceChannel* Channel = ChannelHandle.Get())
	{
		if (Channel->GetNumKeys() > 0)
		{
			const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
			if (Settings == nullptr || Settings->GetShowBars())
			{
				return MakeUnique<FControlRigSpaceChannelCurveModel>(ChannelHandle, Params.OwningSection, Params.Sequencer);
			}
		}
	}
	return nullptr;
}

TArray<FKeyBarCurveModel::FBarRange> FControlRigSpaceChannelHelpers::FindRanges(FMovieSceneControlRigSpaceChannel* Channel, const UMovieSceneSection* Section)
{
	TArray<FKeyBarCurveModel::FBarRange> Range;
	if (Channel && Section)
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		TArray<FSpaceRange> SpaceRanges = Channel->FindSpaceIntervals();
		for (const FSpaceRange& SpaceRange : SpaceRanges)
		{
			FKeyBarCurveModel::FBarRange BarRange;
			double LowerValue = SpaceRange.Range.GetLowerBoundValue() / TickResolution;
			double UpperValue = SpaceRange.Range.GetUpperBoundValue() / TickResolution;

			BarRange.Range.SetLowerBound(TRangeBound<double>(LowerValue));
			BarRange.Range.SetUpperBound(TRangeBound<double>(UpperValue));

			BarRange.Name = SpaceRange.Key.GetName();
			BarRange.Color = FControlRigSpaceChannelHelpers::GetColor(SpaceRange.Key);

			Range.Add(BarRange);
		}
	}
	return  Range;
}

// NOTE use this function in HandleSpaceKeyTimeChanged, DeleteTransformKeysAtThisTime, ...
TPair<FRigControlElement*, FChannelMapInfo*> FControlRigSpaceChannelHelpers::GetControlAndChannelInfo(UControlRig* ControlRig, UMovieSceneControlRigParameterSection* Section, FName ControlName)
{
	FRigControlElement* ControlElement = ControlRig ? ControlRig->FindControl(ControlName) : nullptr;
	FChannelMapInfo* pChannelIndex = Section ? Section->ControlChannelMap.Find(ControlName) : nullptr;

	return TPair<FRigControlElement*, FChannelMapInfo*>(ControlElement, pChannelIndex);
}

// NOTE use this function in HandleSpaceKeyTimeChanged, DeleteTransformKeysAtThisTime, ...
int32 FControlRigSpaceChannelHelpers::GetNumFloatChannels(const ERigControlType InControlType)
{
	switch (InControlType)
	{
	case ERigControlType::Position:
	case ERigControlType::Scale:
	case ERigControlType::Rotator:
		return 3;
	case ERigControlType::TransformNoScale:
		return 6;
	case ERigControlType::Transform:
	case ERigControlType::EulerTransform:
		return 9;
	default:
		break;
	}
	return 0;
}

int32 DrawExtra(FMovieSceneControlRigSpaceChannel* Channel, const UMovieSceneSection* Owner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId)
{
	using namespace UE::Sequencer;

	if (const UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(Owner))
	{
		TArray<FKeyBarCurveModel::FBarRange> Ranges = FControlRigSpaceChannelHelpers::FindRanges(Channel, Owner);
		const FSlateBrush* WhiteBrush = FAppStyle::GetBrush("WhiteBrush");
		const ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;

		const FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont");
		const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

		const FVector2D LocalSize = PaintArgs.Geometry.GetLocalSize();
		const float LaneTop = 0;

		const double InputMin = PaintArgs.TimeToPixel.PixelToSeconds(0.f);
		const double InputMax = PaintArgs.TimeToPixel.PixelToSeconds(LocalSize.X);

		int32 LastLabelPos = -1;
		for (int32 Index = 0; Index < Ranges.Num(); ++Index)
		{
			const FKeyBarCurveModel::FBarRange& Range = Ranges[Index];
			const double LowerSeconds = Range.Range.GetLowerBoundValue();
			const double UpperSeconds = Range.Range.GetUpperBoundValue();
			const bool bOutsideUpper = (Index != Ranges.Num() - 1) && UpperSeconds < InputMin;
			if (LowerSeconds > InputMax || bOutsideUpper) //out of range
			{
				continue;
			}

			FLinearColor CurveColor = Range.Color;
	
			static FLinearColor ZebraTint = FLinearColor::White.CopyWithNewOpacity(0.01f);
			if (CurveColor == FLinearColor::White)
			{
				CurveColor = ZebraTint;
			}
			else
			{
				CurveColor = CurveColor * (1.f - ZebraTint.A) + ZebraTint * ZebraTint.A;
			}
			

			if (CurveColor != FLinearColor::White)
			{
				const double LowerSecondsForBox = (Index == 0 && LowerSeconds > InputMin) ? InputMin : LowerSeconds;
				const double BoxPos = PaintArgs.TimeToPixel.SecondsToPixel(LowerSecondsForBox);

				const FPaintGeometry BoxGeometry = PaintArgs.Geometry.ToPaintGeometry(
					FVector2D(PaintArgs.Geometry.GetLocalSize().X, PaintArgs.Geometry.GetLocalSize().Y),
					FSlateLayoutTransform(FVector2D(BoxPos, LaneTop))
				);

				FSlateDrawElement::MakeBox(PaintArgs.DrawElements, LayerId, BoxGeometry, WhiteBrush, DrawEffects, CurveColor);
			}
			const double LowerSecondsForLabel = (InputMin > LowerSeconds) ? InputMin : LowerSeconds;
			double LabelPos = PaintArgs.TimeToPixel.SecondsToPixel(LowerSecondsForLabel) + 10;

			const FText Label = FText::FromName(Range.Name);
			const FVector2D TextSize = FontMeasure->Measure(Label, FontInfo);
			if (Index > 0)
			{
				LabelPos = (LabelPos < LastLabelPos) ? LastLabelPos + 5 : LabelPos;
			}
			LastLabelPos = LabelPos + TextSize.X + 15;
			const FVector2D Position(LabelPos, LaneTop + (PaintArgs.Geometry.GetLocalSize().Y - TextSize.Y) * .5f);

			const FPaintGeometry LabelGeometry = PaintArgs.Geometry.ToPaintGeometry(
				FSlateLayoutTransform(Position)
			);

			FSlateDrawElement::MakeText(PaintArgs.DrawElements, LayerId, LabelGeometry, Label, FontInfo, DrawEffects, FLinearColor::White);
		}
	}

	return LayerId + 1;
}

void DrawKeys(FMovieSceneControlRigSpaceChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams)
{
	if (const UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(InOwner))
	{
		TMovieSceneChannelData<FMovieSceneControlRigSpaceBaseKey> ChannelInterface = Channel->GetData();

		for (int32 Index = 0; Index < InKeyHandles.Num(); ++Index)
		{
			FKeyHandle Handle = InKeyHandles[Index];

			FKeyDrawParams Params;
			static const FName SquareKeyBrushName("Sequencer.KeySquare");
			const FSlateBrush* SquareKeyBrush = FAppStyle::GetBrush(SquareKeyBrushName);
			Params.FillBrush = FAppStyle::Get().GetBrush("FilledBorder");
			Params.BorderBrush = SquareKeyBrush;
			const int32 KeyIndex = ChannelInterface.GetIndex(Handle);

			OutKeyDrawParams[Index] = Params;
		}
	}
}

#undef LOCTEXT_NAMESPACE
