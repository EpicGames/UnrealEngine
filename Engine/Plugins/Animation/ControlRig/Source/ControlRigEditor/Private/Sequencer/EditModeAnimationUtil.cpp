// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/EditModeAnimationUtil.h"

#include "ControlRig.h"
#include "ConstraintsManager.h"
#include "ContextObjectStore.h"
#include "EditorInteractiveGizmoManager.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "EngineUtils.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "EditMode/ControlRigEditModeUtil.h"
#include "ILevelSequenceEditorToolkit.h"
#include "InputState.h"
#include "LevelEditorSequencerIntegration.h"
#include "LevelSequence.h"
#include "SceneView.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "ToolContextInterfaces.h"
#include "EditMode/ControlRigEditModeCommands.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "EditorGizmos/EditorTransformGizmoUtil.h"
#include "EditorGizmos/TransformGizmo.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Sequencer/AnimationAuthoringSettings.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Transform/TransformConstraintUtil.h"

namespace UE::AnimationEditMode
{

TWeakPtr<ISequencer> GetSequencer()
{
	// if getting sequencer from level sequence need to use the current(leader), not the focused
	if (ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence())
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			static constexpr bool bFocusIfOpen = false;
			IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(LevelSequence, bFocusIfOpen);
			const ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
			return LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
		}
	}

	// look for custom UMovieSceneSequence as a fallback.
	const FCustomMovieSceneRegistry& Registry = FCustomMovieSceneRegistry::Get();
	const TArray<TWeakPtr<ISequencer>> Sequencers = FLevelEditorSequencerIntegration::Get().GetSequencers();
	const int32 Found = Sequencers.IndexOfByPredicate([&Registry](const TWeakPtr<ISequencer>& WeakSequence)
	{
		if (const TSharedPtr<ISequencer> Sequencer = WeakSequence.IsValid() ? WeakSequence.Pin() : nullptr)
		{
			if (UMovieSceneSequence* MovieSceneSequence = Sequencer->GetRootMovieSceneSequence())
			{
				return Registry.IsSequenceSupported(MovieSceneSequence->GetClass());
			}
		}
		return false;
	});
		
	return Found == INDEX_NONE ? nullptr : Sequencers[Found];
}

FCustomMovieSceneRegistry& FCustomMovieSceneRegistry::Get()
{
	static FCustomMovieSceneRegistry Singleton;
	return Singleton;
}

bool FCustomMovieSceneRegistry::IsSequenceSupported(const UClass* InSequenceClass) const
{
	return InSequenceClass ? SupportedSequenceTypes.Contains(InSequenceClass) : false;
}
	
namespace Private
{

void EvaluateRigIfAdditive(UControlRig* InControlRig)
{
	if (InControlRig->IsAdditive())
	{
		TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
		InControlRig->Evaluate_AnyThread();
	}
}

}

using namespace ControlRigEditMode;
using namespace Private;

FControlRigKeyframer::~FControlRigKeyframer()
{
	if (OnAnimSettingsChanged.IsValid())
	{
		if (const UAnimationAuthoringSettings* Settings = GetDefault<UAnimationAuthoringSettings>())
		{
			Settings->OnSettingsChange.Remove(OnAnimSettingsChanged);
		}
		OnAnimSettingsChanged.Reset();
	}
}
	
void FControlRigKeyframer::Initialize()
{
	EnableState = EEnableState::Disabled;
	
	if (const UAnimationAuthoringSettings* Settings = GetDefault<UAnimationAuthoringSettings>())
	{
		OnSettingsChanged(Settings);

		if (!OnAnimSettingsChanged.IsValid())
		{
			OnAnimSettingsChanged = Settings->OnSettingsChange.AddRaw(this, &FControlRigKeyframer::OnSettingsChanged);
		}
	}
}
	
void FControlRigKeyframer::Enable(const bool InEnabled)
{
	Reset();

	if (InEnabled)
	{
		EnumAddFlags(EnableState, EEnableState::EnabledDirectly);
	}
	else
	{
		EnumRemoveFlags(EnableState, EEnableState::EnabledDirectly);
	}
}
	
void FControlRigKeyframer::Reset()
{
	KeyframeData.Reset();
}

void FControlRigKeyframer::Store(const uint32 InControlHash, FControlKeyframeData&& InData)
{
	if (IsEnabled() && InControlHash != 0)
	{
		FControlKeyframeData& ControlKeyframeData = KeyframeData.FindOrAdd(InControlHash);
		ControlKeyframeData = MoveTemp(InData);
	}
}
	
void FControlRigKeyframer::Apply(const FControlRigInteractionScope& InInteractionScope, const FControlRigInteractionTransformContext& InTransformContext)
{
	if (!IsEnabled())
	{
		return;
	}
	
	const TArray<FRigElementKey>& InteractingControls = InInteractionScope.GetElementsBeingInteracted();
	UControlRig* InteractingRig = InInteractionScope.GetControlRig();
	if (!InteractingRig || InteractingControls.IsEmpty())
	{
		return;
	}

	static const FRigControlModifiedContext NoKeyContext(EControlRigSetKey::Never);
	static constexpr bool bNotify = false, bSetupUndo = false;
	
	const bool bFixEulerFlips = !InteractingRig->IsAdditive() && InTransformContext.bRotation;
	UControlRig::FControlModifiedEvent& AutoKeyEvent = InteractingRig->ControlModified();

	for (const FRigElementKey& ControlKey: InteractingControls)
	{
		if (FRigControlElement* Control = InteractingRig->FindControl(ControlKey.Name))
		{
			const uint32 ControlHash = UTransformableControlHandle::ComputeHash(InteractingRig, ControlKey.Name);
			if (const FControlKeyframeData* Data = KeyframeData.Find(ControlHash))
			{
				if (Data->bConstraintSpace)
				{
					// set the control's local transform withing its constraint space transform as it's the value that sequencer has to store
					InteractingRig->SetControlLocalTransform(ControlKey.Name, Data->LocalTransform, bNotify, NoKeyContext, bSetupUndo, bFixEulerFlips);
					EvaluateRigIfAdditive(InteractingRig);
				}
				
				AutoKeyEvent.Broadcast(InteractingRig, Control, EControlRigSetKey::DoNotCare);				
			}

			// driven controls
			if (Control->CanDriveControls())
			{
				for (const FRigElementKey& DrivenKey : Control->Settings.DrivenControls)
				{
					const bool bHandleDrivenKey = DrivenKey.Type == ERigElementType::Control && !InteractingControls.Contains(DrivenKey);						
					if (FRigControlElement* DrivenControl = bHandleDrivenKey ? InteractingRig->FindControl(DrivenKey.Name) : nullptr)
					{
						AutoKeyEvent.Broadcast(InteractingRig, DrivenControl, EControlRigSetKey::DoNotCare);
					}
				}
			}
		}
	}
}

void FControlRigKeyframer::Finalize(UWorld* InWorld)
{
	auto NeedsConstraintUpdate = [this]()
	{
		if (IsEnabled())
		{
			for (const auto& [ControlHash, Data]: KeyframeData)
			{
				if (Data.bConstraintSpace)
				{
					return true;
				}
			}
		}
		
		return false;
	};

	if (InWorld && NeedsConstraintUpdate())
	{
		TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
		Controller.EvaluateAllConstraints();
	}
}

bool FControlRigKeyframer::IsEnabled() const
{
	return OnAnimSettingsChanged.IsValid() ?
		EnumHasAllFlags(EnableState, EEnableState::FullyEnabled) : EnumHasAllFlags(EnableState, EEnableState::EnabledDirectly);
}
	
void FControlRigKeyframer::OnSettingsChanged(const UAnimationAuthoringSettings* InSettings)
{
	if (InSettings)
	{
		if (InSettings->bAutoKeyOnRelease)
		{
			EnumAddFlags(EnableState, EEnableState::EnabledBySettings);
		}
		else
		{
			EnumRemoveFlags(EnableState, EEnableState::EnabledBySettings);
		}
	}
}

FComponentDependency::FComponentDependency( USceneComponent* InComponent,UWorld* InWorld,
	TransformConstraintUtil::FConstraintsInteractionCache& InCacheRef)
	: Component(InComponent)
	, World(InWorld)
	, ConstraintsCache(InCacheRef)
{}

bool FComponentDependency::DependsOn(UObject* InObject)
{
	if (this->IsValid(InObject))
	{
		if (Component == InObject)
		{
			return true;
		}

		if (ConstraintsCache.HasAnyDependency(Component, InObject, World))
		{
			return true;
		}

		if (USceneComponent* InComponent = Cast<USceneComponent>(InObject))
		{
			for (const TObjectPtr<USceneComponent>& Child: InComponent->GetAttachChildren())
			{
				if (DependsOn(Child.Get()))
				{
					return true;
				}
			}
		}
	}
    				
	return false;
}

bool FComponentDependency::IsValid(const UObject* InObject) const
{
	return Component && World && InObject;
}

/**
 * FDirectControl   
 */

namespace DirectControl
{
	bool IsEnabled()
	{
		const UControlRigEditModeSettings* Settings = UControlRigEditModeSettings::Get();
		return Settings ? Settings->ClickAndDragBehavior != EClickAndDragBehavior::Disabled : false;
	}
		
	bool UnselectOnRelease()
	{
		const UControlRigEditModeSettings* Settings = UControlRigEditModeSettings::Get();
		return Settings ? Settings->ClickAndDragBehavior == EClickAndDragBehavior::TransientSelection : false;
	}
    
	bool ReplaceSelection()
	{
		const UControlRigEditModeSettings* Settings = UControlRigEditModeSettings::Get();
		return Settings ? Settings->ClickAndDragBehavior == EClickAndDragBehavior::PersistentSelection : false;
	}
}

FDirectControl::~FDirectControl()
{
	if (GizmoChangeHandle.IsValid())
	{
		UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().Remove(GizmoChangeHandle);
		GizmoChangeHandle.Reset();
	}
	
	UnBind();
}

void FDirectControl::Bind(const FEditorModeTools* InModeTools)
{
	UnBind();

	const UModeManagerInteractiveToolsContext* ToolsContext = InModeTools ? InModeTools->GetInteractiveToolsContext() : nullptr;
	UEditorTransformGizmoContextObject* GizmoContext = ToolsContext && ToolsContext->ContextObjectStore ?
		ToolsContext->ContextObjectStore->FindContext<UEditorTransformGizmoContextObject>() : nullptr;
	
	if (GizmoContext)
	{
		WeakContext = GizmoContext;
		BindGizmo(FindGizmo());

		if (!GizmoChangeHandle.IsValid())
		{
			GizmoChangeHandle = UEditorInteractiveGizmoManager::OnUsesNewTRSGizmosChangedDelegate().AddLambda([this](bool bInUsesNewTRSGizmos)
			{
				UTransformGizmo* Gizmo = FindGizmo();
				if (Gizmo)
				{
					Gizmo->OnPreCanInteract().Remove(PreInteractionHandle);
					Gizmo->OnPostInteraction().Remove(PostInteractionHandle);
					PostInteractionHandle.Reset();
					PreInteractionHandle.Reset();
				}
				Reset();

				if (bInUsesNewTRSGizmos)
				{
					BindGizmo(Gizmo);
				}
			});
		}
		
		if (!GizmoCreatedHandle.IsValid())
		{
			GizmoCreatedHandle = GizmoContext->OnGizmoCreatedDelegate().AddLambda([this](UTransformGizmo* InTransformGizmo)
			{
				if (UTransformGizmo* Gizmo = FindGizmo())
				{
					Gizmo->OnPreCanInteract().Remove(PreInteractionHandle);
					Gizmo->OnPostInteraction().Remove(PostInteractionHandle);
					PostInteractionHandle.Reset();
					PreInteractionHandle.Reset();
				}
				Reset();
					
				BindGizmo(InTransformGizmo);
			});
		}
	}
}

void FDirectControl::UnBind()
{
	if (GizmoCreatedHandle.IsValid())
	{
		if (UEditorTransformGizmoContextObject* GizmoContext = WeakContext.Get())
		{
			GizmoContext->OnGizmoCreatedDelegate().Remove(GizmoCreatedHandle);
		}
		GizmoCreatedHandle.Reset();
	}
	
	if (UTransformGizmo* Gizmo = FindGizmo())
	{
		Gizmo->OnPreCanInteract().Remove(PreInteractionHandle);
		Gizmo->OnPostInteraction().Remove(PostInteractionHandle);
		PostInteractionHandle.Reset();
		PreInteractionHandle.Reset();
	}
	
	HitProxyFilterFunction.Reset();	
	SequencerFunction.Reset();
	WeakContext.Reset();
	Reset();
}
	
UTransformGizmo* FDirectControl::FindGizmo() const
{
	if (const FEditorModeTools* ModeTools = GetModeTools())
	{
		if (const UModeManagerInteractiveToolsContext* ToolsContext = ModeTools->GetInteractiveToolsContext())
		{
			return EditorTransformGizmoUtil::FindDefaultTransformGizmo(ToolsContext->ToolManager);
		}		
	}
	return nullptr;
}
	
void FDirectControl::BindGizmo(UTransformGizmo* InGizmo)
{
	if (InGizmo)
	{
		PreInteractionHandle = InGizmo->OnPreCanInteract().AddRaw(this, &FDirectControl::OnPreCanGizmoInteract);
		PostInteractionHandle = InGizmo->OnPostInteraction().AddRaw(this, &FDirectControl::OnGizmoPostInteract);
	}
}

void FDirectControl::OnPreCanGizmoInteract(const FGizmoInteractionDescription& InDesc)
{
	const FEditorModeTools* ModeManager = GetModeTools();
	FEditorViewportClient* ViewportClient = ModeManager ? ModeManager->GetFocusedViewportClient() : nullptr;
	if (!DirectControl::IsEnabled() || !ViewportClient)
	{
		Reset();
		return;
	}
		
	const Widget::EWidgetMode WidgetMode = ViewportClient->GetWidgetMode();
	if (!(WidgetMode > Widget::EWidgetMode::WM_None && WidgetMode < Widget::EWidgetMode::WM_Max))
	{
		Reset();
		return;
	}
	
	if (ModeManager->HasOngoingTransform())
	{
		return;
	}
	
	const TSharedRef<const FInputChord>& Chord = FControlRigEditModeCommands::Get().ClickAndDrag->GetFirstValidChord();
	if (!Chord->IsValidChord())
	{
		Reset();
		return;
	}
	
	const FModifierKeysState Modifiers = FSlateApplication::Get().GetModifierKeys();
	const FInputChord CheckChord(ViewportClient->Viewport && ViewportClient->Viewport->KeyState(Chord->Key) ? Chord->Key : FKey(),
		EModifierKey::FromBools(Modifiers.IsControlDown(), Modifiers.IsAltDown(), Modifiers.IsShiftDown(), Modifiers.IsCommandDown()) );
	if ((*Chord) != CheckChord)
	{
		constexpr bool bDoNotResetHit = false;
		Reset(bDoNotResetHit);
		return;
	}
	
	if (bProcessing || bHasTransformChanged)
	{
		return;
	}
		
	constexpr bool bDoNotResetHit = false;
	Reset(bDoNotResetHit);
		
	const bool bIsAnythingSelected = IsAnythingSelected();
	if (!bIsAnythingSelected)
	{
		CachedId = 0;
	}
			
	if (DirectControl::ReplaceSelection())
	{
		HHitProxy* NewHitProxy = RequestNewHitProxy(InDesc.Ray);
		const FTypedHandleCombinedId& NewId = NewHitProxy ? NewHitProxy->GetElementHandle().GetId().GetCombinedId() : 0;
			
		// no need to replace the selection if we picked
		if (NewId != CachedId)
		{
			if (NewId != 0 && !IsHitProxyValidForSelection(NewHitProxy))
			{
				bProcessing = CachedId != 0;
				return;
			}
				
			using namespace UE::Sequencer;
			TUniquePtr<FSelectionEventSuppressor> SequencerSelectionGuard;
			if (TSharedPtr<ISequencer> Sequencer = SequencerFunction ? SequencerFunction() : nullptr)
			{
				TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
				if (TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
				{
					SequencerSelectionGuard.Reset(new FSelectionEventSuppressor(SequencerSelection.Get()));
				}
			}
				
			const FScopedTransaction ScopedTransaction(NSLOCTEXT("DirectControl", "DirectControl_ReplaceSelection", "Replace Selection"));
				
			UnselectAll();
			ProcessClick(EKeys::LeftMouseButton, IE_Pressed, InDesc.Ray, NewHitProxy);
			bHasSelected = IsAnythingSelected();
			CachedId = NewId;
			bProcessing = CachedId != 0;
		}
			
		return;
	}	
		
	if (!bIsAnythingSelected)
	{
		const FScopedTransaction ScopedTransaction(NSLOCTEXT("DirectControl", "DirectControl_SetSelection", "Set Selection"));
		HHitProxy* NewHitProxy = RequestNewHitProxy(InDesc.Ray);
		ProcessClick(EKeys::LeftMouseButton, IE_Pressed, InDesc.Ray, NewHitProxy);
		bHasSelected = IsAnythingSelected();
		CachedId = NewHitProxy ? NewHitProxy->GetElementHandle().GetId().GetCombinedId() : 0;
		bProcessing = CachedId != 0;
	}
}
	
void FDirectControl::OnGizmoPostInteract(const FGizmoInteractionDescription& InDesc)
{
	if (!DirectControl::IsEnabled())
	{
		Reset();
		return;
	}
		
	if (DirectControl::UnselectOnRelease())
	{
		if (bHasSelected && (bHasTransformChanged || bProcessing))
		{
			UnselectAll();
		}
		CachedId = 0;
	}
		
	constexpr bool bDoNotResetHit = false;
	Reset(bDoNotResetHit);
}
	
void FDirectControl::Reset(const bool bAlsoResetHit)
{
	bHasSelected = false;
	bHasTransformChanged = false;
	bProcessing = false;
	
	if (bAlsoResetHit)
	{
		CachedId = 0;
	}
}
	
void FDirectControl::UnselectAll()
{
	GEditor->SelectNone(true, true);	
	if (FEditorModeTools* ModeManager = GetModeTools())
	{
		ModeManager->SelectNone();
	}
	CachedId = 0;
}

bool FDirectControl::IsAnythingSelected() const
{
	FToolBuilderState State;
	if (const FEditorModeTools* ModeManager = GetModeTools())
	{
		const UWorld* World = ModeManager->GetWorld();
		if (const ULevel* CurrentLevel = IsValid(World) ? World->GetCurrentLevel() : nullptr)
		{
			const bool bAnyActorSelected = ObjectPtrDecay(CurrentLevel->Actors).ContainsByPredicate( [](const AActor* Actor)
			{
				return IsValid(Actor) && Actor->IsSelected();
			});
			return bAnyActorSelected;
		}
		
		ModeManager->GetSelectedActors()->GetSelectedObjects(State.SelectedActors);
		// ModeManager->GetSelectedComponents()->GetSelectedObjects(State.SelectedComponents);
	}
	return !(State.SelectedActors.IsEmpty() && State.SelectedComponents.IsEmpty());
}

HHitProxy* FDirectControl::RequestNewHitProxy(const FInputDeviceRay& InRay) const 
{
	FEditorViewportClient* ViewportClient = GetViewportClient();
	if (ViewportClient && ViewportClient->Viewport)
	{
		const FVector2D& ScreenPosition = InRay.ScreenPosition;
		return ViewportClient->Viewport->GetHitProxy(ScreenPosition[0],ScreenPosition[1]);
	}
	return nullptr;
}
	
void FDirectControl::ProcessClick(const FKey& InKey, const EInputEvent& InEvent, const FInputDeviceRay& InRay, HHitProxy* InHitProxy) const
{
	FEditorViewportClient* ViewportClient = GetViewportClient();
	if (ViewportClient && ViewportClient->Viewport && InHitProxy)
	{
		FSceneViewFamilyContext ViewFamily(
		   FSceneViewFamily::ConstructionValues(
			   ViewportClient->Viewport,
			   ViewportClient->GetScene(),
			   ViewportClient->EngineShowFlags)
		   .SetRealtimeUpdate(ViewportClient->IsRealtime()));

		FSceneView* View = ViewportClient->CalcSceneView( &ViewFamily );
		ViewportClient->ProcessClick(*View, InHitProxy, InKey, InEvent, InRay.ScreenPosition[0], InRay.ScreenPosition[1]);
	}
}
	
bool FDirectControl::IsHitProxyValidForSelection(HHitProxy* InHitProxy) const
{
	if (!InHitProxy)
	{
		return false;
	}

	const HActor* ActorHitProxy = HitProxyCast<HActor>(InHitProxy);
	if (!ActorHitProxy || !IsValid(ActorHitProxy->Actor))
	{
		return false;
	}
				
	// Check for translucent actors if we don't want to allow them to be selected
	const UEditorPerProjectUserSettings* const Settings = GetDefault<UEditorPerProjectUserSettings>();
	if (!Settings->bAllowSelectTranslucent && InHitProxy->IsA(HTranslucentActor::StaticGetType()))
	{
		return false;
	}
	
	// extra check function
	if (HitProxyFilterFunction)
	{
		return HitProxyFilterFunction(InHitProxy);
	}

	return true;	
}
	
FEditorModeTools* FDirectControl::GetModeTools() const
{
	if (UEditorTransformGizmoContextObject* Context = WeakContext.IsValid() ? WeakContext.Get() : nullptr)
	{
		return Context->GetModeTools();	
	}
	return nullptr;
}
	
FEditorViewportClient* FDirectControl::GetViewportClient() const
{
	if (const FEditorModeTools* ModeTools = GetModeTools())
	{
		return ModeTools->GetFocusedViewportClient();
	}
	return nullptr;
}
	
}
