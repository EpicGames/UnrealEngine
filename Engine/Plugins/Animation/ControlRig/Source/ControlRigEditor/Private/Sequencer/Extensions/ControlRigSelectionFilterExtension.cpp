// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigSelectionFilterExtension.h"

#include "Filters/Filters/SequencerTrackFilter_Selected.h"
#include "Sequencer/ControlRigSelectionUtils.h"

void UControlRigSelectionFilterExtension::EnumerateViewModelsConsideredAsSelected(
	const ISequencer& InSequencer,
	TFunctionRef<EBreakBehavior(const UE::Sequencer::FViewModelPtr&)> InCallback
)
{
	// Drive the "Selected" filter from the viewport / rig hierarchy selection rather than the sequencer outliner
	// selection. Channels whose owning control is currently selected on the rig (and their ancestors, via
	// FSequencerFilterData::FilterInNodeWithAncestors) are considered selected for filtering purposes.
	UE::ControlRig::EnumerateOutlinerExtensionsForCurrentControlSelection(
		InSequencer, 
		[&InCallback](const UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension>& ViewModel)
		{
			return InCallback(ViewModel) == EBreakBehavior::Continue;
		});
}
