// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterSelectedExtension.h"
#include "ControlRigSelectionFilterExtension.generated.h"

/**
 * Extends FSequencerTrackFilter_Selected such that selecting e.g. Dragon.ChestCtrl.Location.X shows Location.Y, Rotation, Scale, etc., as well.
 */
UCLASS()
class UControlRigSelectionFilterExtension : public USequencerTrackFilterSelectedExtension
{
	GENERATED_BODY()
public:

	virtual void EnumerateViewModelsConsideredAsSelected(
		const ISequencer& InSequencer,
		TFunctionRef<EBreakBehavior(const UE::Sequencer::FViewModelPtr&)> InCallback
		) override;
};
