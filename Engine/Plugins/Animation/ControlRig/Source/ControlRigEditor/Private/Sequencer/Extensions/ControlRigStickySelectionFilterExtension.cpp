// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigStickySelectionFilterExtension.h"

#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Sequencer/ControlRigSelectionUtils.h"
#include "Toolkits/IToolkitHost.h"

namespace UE::ControlRig::StickSelectionDetail
{
FEditorModeTools* GetEditorModeTools(const ISequencer& InSequencer)
{
	if (const TSharedPtr<IToolkitHost> ToolkitHost = InSequencer.GetToolkitHost())
	{
		return &ToolkitHost->GetEditorModeManager();
	}
	
	return nullptr;
}

static FControlRigEditMode* GetEditMode(const ISequencer& InSequencer)
{
	FEditorModeTools* EditorModeTools = GetEditorModeTools(InSequencer);
	const bool bActiveEditMode = EditorModeTools && EditorModeTools->IsModeActive(FControlRigEditMode::ModeName);
	return bActiveEditMode ? static_cast<FControlRigEditMode*>(EditorModeTools->GetActiveMode(FControlRigEditMode::ModeName)) : nullptr;
}
}

bool UControlRigStickySelectionFilterExtension::IsPerformingSelectionThatShouldClearStickySelection(const ISequencer& InSequencer)
{
	const FControlRigParameterTrackEditor* Editor = FControlRigParameterTrackEditor::CurrentEditor;
	const bool bEditorIsDoingSelection = Editor && Editor->IsDoingSelection();
	
	FControlRigEditMode* EditMode = UE::ControlRig::StickSelectionDetail::GetEditMode(InSequencer);
	const bool bIsAnimDetailsDoingSelection = EditMode && UE::ControlRig::IsAnimDetailsChangingSelection(*EditMode);
	
	const bool bIsControlRigChangingSelection = bEditorIsDoingSelection || bIsAnimDetailsDoingSelection;
	return bIsControlRigChangingSelection;
}
