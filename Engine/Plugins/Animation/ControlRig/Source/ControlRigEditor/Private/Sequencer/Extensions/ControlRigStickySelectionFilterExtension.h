// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SequencerTrackFilterStickySelectionExtension.h"
#include "ControlRigStickySelectionFilterExtension.generated.h"

/** Extends FSequencerTrackFilter_StickySelection such that clicking control rig controls in the viewport clears sticky selection. */
UCLASS()
class UControlRigStickySelectionFilterExtension : public USequencerTrackFilterStickySelectionExtension
{
	GENERATED_BODY()
public:
	
	virtual bool IsPerformingSelectionThatShouldClearStickySelection(const ISequencer& InSequencer) override;
};
