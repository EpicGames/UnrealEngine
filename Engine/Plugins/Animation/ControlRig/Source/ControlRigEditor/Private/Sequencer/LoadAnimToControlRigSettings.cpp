// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadAnimToControlRigSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LoadAnimToControlRigSettings)

ULoadAnimToControlRigSettings::ULoadAnimToControlRigSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	Reset();
}

void ULoadAnimToControlRigSettings::Reset()
{
	bReduceKeys = false;
	SmartReduce.Reset();
	bUseCustomTimeRange = false;
	bOntoSelectedControls = false;
	bResetControls = false;

}

