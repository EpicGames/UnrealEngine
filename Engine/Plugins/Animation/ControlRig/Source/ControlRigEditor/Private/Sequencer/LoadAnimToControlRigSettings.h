// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Filters/CurveEditorSmartReduceFilter.h"
#include "LoadAnimToControlRigSettings.generated.h"

//settings used when loading animation sequences into a control rig section
UCLASS(BlueprintType, config = EditorSettings)
class  ULoadAnimToControlRigSettings : public UObject
{
public:
	ULoadAnimToControlRigSettings(const FObjectInitializer& Initializer);

	GENERATED_BODY()
	/** Load animation onto only selected controls */
	UPROPERTY(EditAnywhere, Category = "Controls")
	bool bOntoSelectedControls = false;

	/** Reduce Keys */
	UPROPERTY(EditAnywhere, Category = "Reduce Keys")
	bool bReduceKeys = false;

	UPROPERTY(EditAnywhere, Category = "Reduce Keys", meta = (EditCondition = "bReduceKeys"))
	FSmartReduceParams SmartReduce;

	UPROPERTY(EditAnywhere, Category = "Animation Time Range")
	bool bUseCustomTimeRange = false;

	UPROPERTY(EditAnywhere, Category = "Animation Time Range", meta = (EditCondition = "bUseCustomTimeRange"))
	FFrameNumber StartFrame;

	UPROPERTY(EditAnywhere, Category = "Animation Time Range", meta = (EditCondition = "bUseCustomTimeRange"))
	FFrameNumber EndFrame;

	/** Reset controls to initial value on every frame */
	UPROPERTY(EditAnywhere, Category = "Reset Controls")
	bool bResetControls = false;

	/** Resets the default properties. */
	void Reset();
};


