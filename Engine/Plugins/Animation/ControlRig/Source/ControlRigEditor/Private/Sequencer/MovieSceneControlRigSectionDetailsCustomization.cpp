// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieSceneControlRigSectionDetailsCustomization.h"
#include "DetailLayoutBuilder.h"

TSharedRef<IDetailCustomization> FMovieSceneControlRigSectionDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FMovieSceneControlRigSectionDetailsCustomization);
}

void FMovieSceneControlRigSectionDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	// Hide recording UI
	DetailLayout.HideCategory("Sequence Recording");

	// @TODO: restrict subsequences property to only accept control rig sequences
}