// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLoadAnimToControlRig.h"
#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "LoadAnimToControlRigSettings.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyEditorModule.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Misc/FrameRate.h"

class SLoadAnimToControlRigDialog : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLoadAnimToControlRigDialog) {}
	SLATE_END_ARGS()
	~SLoadAnimToControlRigDialog()
	{
	}
	
	void Construct(const FArguments& InArgs)
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "Load Animation To Control Rig";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		ChildSlot
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.ContentPadding(FMargin(10, 5))
			.Text(NSLOCTEXT("ControlRig", "LoadAnimationToControlRig", "Load Animation"))
			.OnClicked(this, &SLoadAnimToControlRigDialog::OnLoadAnim)
			]

		];

		ULoadAnimToControlRigSettings* LoadSettings = GetMutableDefault<ULoadAnimToControlRigSettings>();
		DetailView->SetObject(LoadSettings);
	}

	void SetDelegate(FLoadAnimToControlRigDelegate& InDelegate)
	{
		Delegate = InDelegate;
	}

private:

	FReply OnLoadAnim()
	{
		ULoadAnimToControlRigSettings* LoadSettings = GetMutableDefault<ULoadAnimToControlRigSettings>();
		TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (LoadSettings && Delegate.IsBound())
		{
			Delegate.Execute(LoadSettings);
		}
		if (Window.IsValid())
		{
			Window->RequestDestroyWindow();
		}
		return FReply::Handled();
	}
	TSharedPtr<IDetailsView> DetailView;
	FLoadAnimToControlRigDelegate  Delegate;
};


void FLoadAnimToControlRigDialog::GetLoadAnimParams(FLoadAnimToControlRigDelegate& InDelegate, const FOnWindowClosed& OnClosedDelegate)
{
	const FText TitleText = NSLOCTEXT("ControlRig", "LoadAnimation", "Load Animation");

	// Create the window to choose our options
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(TitleText)
		.HasCloseButton(true)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(400.0f, 400.0f))
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false);

	TSharedRef<SLoadAnimToControlRigDialog> DialogWidget = SNew(SLoadAnimToControlRigDialog);
	DialogWidget->SetDelegate(InDelegate);
	Window->SetContent(DialogWidget);
	Window->SetOnWindowClosed(OnClosedDelegate);

	FSlateApplication::Get().AddWindow(Window);

}

