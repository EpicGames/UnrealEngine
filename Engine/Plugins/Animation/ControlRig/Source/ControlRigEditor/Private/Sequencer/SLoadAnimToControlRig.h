// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Widgets/SWindow.h"

class ULoadAnimToControlRigSettings;

DECLARE_DELEGATE_OneParam(FLoadAnimToControlRigDelegate, ULoadAnimToControlRigSettings*);

//dialog to show ULoadAnimToControlRigSettings properties for loading animation into a control rig section
struct  FLoadAnimToControlRigDialog
{
	static void GetLoadAnimParams(FLoadAnimToControlRigDelegate& Delegate, const FOnWindowClosed& OnClosedDelegate);
};