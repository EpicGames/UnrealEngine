// Copyright Epic Games, Inc. All Rights Reserved.

#include "SaveScopedControlRigSelection.h"
#include "Components/SceneComponent.h"
#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"

namespace UE::ControlRig::SaveScopedSelection
{

template<typename InObjectType>
static void SaveSelectedEditorObjects(USelection* const InSelection, TArray<TWeakObjectPtr<InObjectType>>& OutSavedObjects)
{
	if (!InSelection)
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	InSelection->GetSelectedObjects(SelectedObjects);

	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (InObjectType* const Object = Cast<InObjectType>(SelectedObject.Get()))
		{
			OutSavedObjects.Add(Object);
		}
	}
}

static void SaveEditorSelection(TArray<TWeakObjectPtr<AActor>>& OutSavedActors, TArray<TWeakObjectPtr<USceneComponent>>& OutSavedComponents)
{
	if (!GEditor)
	{
		return;
	}

	SaveSelectedEditorObjects(GEditor->GetSelectedActors(), OutSavedActors);
	SaveSelectedEditorObjects(GEditor->GetSelectedComponents(), OutSavedComponents);
}

static void SaveControlRigSelection(TArray<TPair<TWeakObjectPtr<UControlRig>, TArray<FName>>>& OutSavedControlSelections)
{
	FControlRigEditMode* const EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (!EditMode)
	{
		return;
	}

	for (UControlRig* const ControlRig : EditMode->GetControlRigsArray(/*bIsVisible=*/false))
	{
		if (!ControlRig)
		{
			continue;
		}

		TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
		// Empty selections are state too: restore must clear controls selected while scoped.
		OutSavedControlSelections.Emplace(ControlRig, MoveTemp(SelectedControls));
	}
}

template<typename InObjectType>
static bool HasSameObjectSelection(const TArray<TWeakObjectPtr<InObjectType>>& InSavedObjects, USelection* const InCurrentSelection)
{
	TArray<TWeakObjectPtr<UObject>> CurrentObjects;
	if (InCurrentSelection)
	{
		InCurrentSelection->GetSelectedObjects(CurrentObjects);
	}

	if (CurrentObjects.Num() != InSavedObjects.Num())
	{
		return false;
	}

	for (const TWeakObjectPtr<InObjectType>& SavedObject : InSavedObjects)
	{
		UObject* const SavedObjectPtr = SavedObject.Get();
		if (!SavedObjectPtr)
		{
			continue;
		}

		const bool bFound = CurrentObjects.ContainsByPredicate(
			[SavedObjectPtr](const TWeakObjectPtr<UObject>& CurrentObject)
			{
				return CurrentObject.Get() == SavedObjectPtr;
			});
		if (!bFound)
		{
			return false;
		}
	}

	return true;
}

template<typename InObjectType, typename InSelectFunctionType>
static void RestoreSelectedEditorObjects(const TArray<TWeakObjectPtr<UObject>>& InCurrentObjects
	, const TArray<TWeakObjectPtr<InObjectType>>& InSavedObjects, InSelectFunctionType&& InSelectFunction)
{
	for (const TWeakObjectPtr<UObject>& CurrentObject : InCurrentObjects)
	{
		if (InObjectType* const Object = Cast<InObjectType>(CurrentObject.Get()))
		{
			InSelectFunction(Object, false);
		}
	}

	for (const TWeakObjectPtr<InObjectType>& SavedObject : InSavedObjects)
	{
		if (InObjectType* const Object = SavedObject.Get())
		{
			InSelectFunction(Object, true);
		}
	}
}

static void RestoreEditorSelection(const TArray<TWeakObjectPtr<AActor>>& InSavedActorSelections
	, const TArray<TWeakObjectPtr<USceneComponent>>& InSavedComponentSelections)
{
	if (!GEditor)
	{
		return;
	}

	const bool bRestoreActors = !HasSameObjectSelection(InSavedActorSelections, GEditor->GetSelectedActors());
	const bool bRestoreComponents = !HasSameObjectSelection(InSavedComponentSelections, GEditor->GetSelectedComponents());
	if (!bRestoreActors && !bRestoreComponents)
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> CurrentActors;
	TArray<TWeakObjectPtr<UObject>> CurrentComponents;

	if (USelection* const ActorSelection = GEditor->GetSelectedActors())
	{
		ActorSelection->GetSelectedObjects(CurrentActors);
	}

	if (USelection* const ComponentSelection = GEditor->GetSelectedComponents())
	{
		ComponentSelection->GetSelectedObjects(CurrentComponents);
	}

	if (bRestoreActors)
	{
		RestoreSelectedEditorObjects<AActor>(CurrentActors, InSavedActorSelections
			, [](AActor* const InActor, const bool bInSelected)
			{
				GEditor->SelectActor(InActor, bInSelected, false);
			});
	}

	if (bRestoreComponents)
	{
		RestoreSelectedEditorObjects<USceneComponent>(CurrentComponents, InSavedComponentSelections
			, [](USceneComponent* const InSceneComponent, const bool bInSelected)
			{
				GEditor->SelectComponent(InSceneComponent, bInSelected, false);
			});
	}

	GEditor->NoteSelectionChange();
}

static void RestoreControlRigSelection(const TArray<TPair<TWeakObjectPtr<UControlRig>, TArray<FName>>>& SavedControlSelections)
{
	for (const TPair<TWeakObjectPtr<UControlRig>, TArray<FName>>& SavedSelection : SavedControlSelections)
	{
		UControlRig* const ControlRig = SavedSelection.Key.Get();
		if (!ControlRig)
		{
			continue;
		}

		const TArray<FName> CurrentSelection = ControlRig->CurrentControlSelection();
		if (CurrentSelection.Num() == SavedSelection.Value.Num())
		{
			bool bMatches = true;

			for (const FName& ControlName : SavedSelection.Value)
			{
				if (!CurrentSelection.Contains(ControlName))
				{
					bMatches = false;
					break;
				}
			}

			if (bMatches)
			{
				continue;
			}
		}

		ControlRig->ClearControlSelection();
		for (const FName& ControlName : SavedSelection.Value)
		{
			if (ControlRig->FindControl(ControlName))
			{
				ControlRig->SelectControl(ControlName, true);
			}
		}
	}
}

static void RequestSelectionCacheRefresh(const ISequencer& InSequencer)
{
	InSequencer.GetFilterInterface()->OnRequestUpdate().Broadcast();
}

} // namespace UE::ControlRig::SaveScopedSelection

namespace UE::ControlRig
{

FSaveScopedControlRigSelection::FSaveScopedControlRigSelection(const TWeakPtr<ISequencer>& InWeakSequencer)
	: WeakSequencer(InWeakSequencer)
{
	SaveScopedSelection::SaveEditorSelection(SavedActorSelections, SavedComponentSelections);
	SaveScopedSelection::SaveControlRigSelection(SavedControlSelections);
}

FSaveScopedControlRigSelection::~FSaveScopedControlRigSelection()
{
	SaveScopedSelection::RestoreEditorSelection(SavedActorSelections, SavedComponentSelections);
	SaveScopedSelection::RestoreControlRigSelection(SavedControlSelections);

	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		SaveScopedSelection::RequestSelectionCacheRefresh(*Sequencer);
	}
}

} // namespace UE::ControlRig
