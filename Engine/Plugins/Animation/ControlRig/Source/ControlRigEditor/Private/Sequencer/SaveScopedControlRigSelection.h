// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class ISequencer;
class UControlRig;
class USceneComponent;

namespace UE::ControlRig
{

/**
 * RAII helper that saves the current editor object selection and Control Rig control selection,
 * then restores both when it goes out of scope. This is used around Sequencer operations such
 * as anim layer add/merge that may rebuild sections or tracks and temporarily clear selection.
 */
struct FSaveScopedControlRigSelection : FNoncopyable
{
	explicit FSaveScopedControlRigSelection(const TWeakPtr<ISequencer>& InWeakSequencer);
	~FSaveScopedControlRigSelection();

private:
	TWeakPtr<ISequencer> WeakSequencer;

	TArray<TWeakObjectPtr<AActor>> SavedActorSelections;
	TArray<TWeakObjectPtr<USceneComponent>> SavedComponentSelections;
	TArray<TPair<TWeakObjectPtr<UControlRig>, TArray<FName>>> SavedControlSelections;
};

} // namespace UE::ControlRig
