// Copyright Epic Games, Inc. All Rights Reserved.
#include "SSelectionSets.h"
#include "SelectionSets.h"
#include "SelectionSetDragDropOp.h"
#include "SelectionSetsSettings.h"

#include "ScopedTransaction.h"
#include "SPositiveActionButton.h"
#include "Widgets/SWidget.h"
#include "EditMode/ControlRigEditMode.h"
#include "Toolkits/IToolkitHost.h"
#include "DesktopPlatformModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "EditorDirectories.h"
#include "EditorModeManager.h"
#include "ControlRig.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MovieScene.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "SceneOutlinerPublicTypes.h"
#include "ControlRigEditorStyle.h"
#include "LevelEditor.h"
#include "ToolMenus.h"

#include "Modules/ModuleManager.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Selection/Selection.h"
#include "Styling/SlateIconFinder.h"

#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/ITransaction.h"
#include "Misc/TransactionObjectEvent.h"

#define LOCTEXT_NAMESPACE "SelectionSets"

///////////////////////////////////////////////////////////
/// SSelectionSetButton
////////////////////////////////////////////////SS///////////

void SSelectionSetButton::Construct(const FArguments& InArgs)
{
	SelectionSetGuid = InArgs._Guid;
	SelectionSetsWidget = InArgs._SetsWidget;
	SButton::FArguments ButtonArgs;
	ButtonArgs.ButtonStyle(FAppStyle::Get(), "NoBorder");
	ButtonArgs._OnClicked = InArgs._OnClicked;
	ButtonArgs._Content = InArgs._Content;
	ButtonArgs.ContentPadding(1.f);

	SButton::Construct(ButtonArgs);
}

void SSelectionSetButton::AddSelectionToSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->AddSelectionToSetItem(SelectionSetGuid, SequencerPtr);
		}
	}
}

void SSelectionSetButton::RemoveSelectionFromSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->RemoveSelectionFromSetItem(SelectionSetGuid, SequencerPtr);
		}
	}
}

void SSelectionSetButton::DeleteSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->DeleteSetItem(SelectionSetGuid,SequencerPtr);
		}
	}
}

void SSelectionSetButton::RenameSet() const
{
	if (SelectionSetsWidget)
	{
		SelectionSetsWidget->SetFocusOnTextBlock(SelectionSetGuid);
	}
}

TSharedRef<SWidget> SSelectionSetButton::CreateSelectionSetColorWidget(const FGuid& InGuid, TSharedPtr<ISequencer>& SequencerPtr) const
{
	// clang-format off
	return

		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
						.Padding(FMargin(1.0f))
						[
							SNew(SColorBlock).IsEnabled(true)
								//.Size(FVector2D(6.0, 38.0))
								.Color_Lambda([InGuid, WeakSequencer = SequencerPtr.ToWeakPtr()]()
									{
										FLinearColor Color(FLinearColor::White);
										if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
										{
											if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
											{
												FLinearColor CurrentColor;
												if (SelectionSets->GetItemColor(InGuid, CurrentColor))
												{
													Color = CurrentColor;
												}
											}
										}
										return Color;
									}
								)

								.OnMouseButtonDown_Lambda([InGuid, WeakSequencer = SequencerPtr.ToWeakPtr()](const FGeometry&, const FPointerEvent&)
									{
										if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
										{
											if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
											{
												FLinearColor Color;
												if (SelectionSets->GetItemColor(InGuid, Color))
												{

													FColorPickerArgs PickerArgs;
													PickerArgs.bUseAlpha = false;
													PickerArgs.InitialColor = Color;
													PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([InGuid, WeakSequencer = SequencerPtr.ToWeakPtr()](FLinearColor Color)
													{
														if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
														{
															if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
															{
																SelectionSets->SetItemColor(InGuid, Color);
															}
														}
													});
													OpenColorPicker(PickerArgs);
												}
											}
										}
										return FReply::Handled();
									})
						]
				]
		];

	// clang-format on
}

void SSelectionSetButton::CreateMirror() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			FGuid NewGuid = SelectionSets->CreateMirror(SequencerPtr, SelectionSetGuid);
			SelectionSetsWidget->SetFocusOnTextBlock(NewGuid);

		}
	}
}

void SSelectionSetButton::SelectMirror() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			const bool bAdd = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
			const bool bToggle = FSlateApplication::Get().GetModifierKeys().IsControlDown();

			SelectionSets->SelectItem(SelectionSetGuid, true, bAdd, bToggle );
		}
	}
}

void SSelectionSetButton::HideSelectionSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->ShowOrHideControls(SelectionSetGuid, false /*bShow*/, false /*bDoMirror*/);

		}
	}
}

void SSelectionSetButton::ShowSelectionSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->ShowOrHideControls(SelectionSetGuid, true /*bShow*/, false /*bDoMirror*/);
		}
	}
}

void SSelectionSetButton::IsolateSelectionSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->IsolateControls(SelectionSetGuid);
		}
	}
}

void SSelectionSetButton::ShowAllSelectionSet() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->ShowAllControls(SelectionSetGuid);
		}
	}
}

void SSelectionSetButton::KeyAll() const
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			SelectionSets->KeyAll(SequencerPtr, SelectionSetGuid);
		}
	}
}

FReply SSelectionSetButton::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{

	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
		{
			FVector2f SummonLocation = InMouseEvent.GetScreenSpacePosition();

			const bool bShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

			MenuBuilder.BeginSection("SSelectionSetButton", LOCTEXT("SelectionSets", "Selection Sets"));
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::AddSelectionToSet));
				const FText Label = LOCTEXT("AddSelectionToSet", "Add Selection");
				const FText ToolTipText = LOCTEXT("AddSelectionToSettooltip", "Add current selection to this selection set");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::RemoveSelectionFromSet));
				const FText Label = LOCTEXT("RemoveSelectionFromSet", "Remove Selection");
				const FText ToolTipText = LOCTEXT("RemoveSelectionFromSettooltip", "Remove current selection from this selection set");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.AddMenuSeparator();
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::DeleteSet));
				const FText Label = LOCTEXT("DeleteSet", "Delete");
				const FText ToolTipText = LOCTEXT("DeleteSettooltip", "Delete selection set");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::RenameSet));
				const FText Label = LOCTEXT("RenameSet", "Rename");
				const FText ToolTipText = LOCTEXT("RenameSettooltip", "Rename selection set");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				const FText Label = LOCTEXT("SetColor", "Set Color");
				const FText ToolTipText = LOCTEXT("SetColortooltip", "Set Selection Set Color");
				const TSharedRef<SWidget> ColorWidget = CreateSelectionSetColorWidget(SelectionSetGuid, SequencerPtr);
				MenuBuilder.AddWidget(ColorWidget, Label, true, true, ToolTipText);
			}
			MenuBuilder.AddMenuSeparator();
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::CreateMirror));
				const FText Label = LOCTEXT("CreateMirror", "Create Mirror");
				const FText ToolTipText = LOCTEXT("CreateMirrortooltip", "Create selection set from mirrored");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::SelectMirror));
				const FText Label = LOCTEXT("SelectMirror", "Select Mirror");
				const FText ToolTipText = LOCTEXT("SelectMirrortooltip", "Select mirror of selection set");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.AddMenuSeparator();
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::HideSelectionSet));
				const FText Label = LOCTEXT("HideSelectionSet", "Hide Selection Set");
				const FText ToolTipText = LOCTEXT("HideSelectionSettooltip", "Hide the Controls and Actors in the Selection Set on selected objects");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::ShowSelectionSet));
				const FText Label = LOCTEXT("ShowSelectionSet", "Show Selection Set");
				const FText ToolTipText = LOCTEXT("ShowSelectionSettooltip", "Show the Controls and the Actors in the Selection Set on selected objects");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::IsolateSelectionSet));
				const FText Label = LOCTEXT("IsolateSelectionSet", "Isolate Selection Set");
				const FText ToolTipText = LOCTEXT("IsolateSelectionSettooltip", "Just show the Controls and the Actors in the Selection Set on selected objects");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::ShowAllSelectionSet));
				const FText Label = LOCTEXT("ShowAllSelection", "Show All Selection Set");
				const FText ToolTipText = LOCTEXT("ShowAllselectionSettootip", "Controls and the Actors in the Selection Set on selected objects");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.AddMenuSeparator();
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateRaw((this), &SSelectionSetButton::KeyAll));
				const FText Label = LOCTEXT("KeyAllSet", "Key Selection Set");
				const FText ToolTipText = LOCTEXT("KeyAllSettooltip", "Key everything in the selection set at this time that's active");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
			MenuBuilder.AddMenuSeparator();
			TSharedPtr<SWidget> MenuContent = MenuBuilder.MakeWidget();

			if (MenuContent.IsValid())
			{
				FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
				FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuContent.ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			}
			return FReply::Handled();
		}
	}

	// Call the base class implementation for other mouse buttons
	FReply Reply = SButton::OnMouseButtonDown(InMyGeometry, InMouseEvent);
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Reply.DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}
	return Reply;
}

FReply SSelectionSetButton::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return SButton::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

FReply SSelectionSetButton::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled().BeginDragDrop(FAIESelectionSetDragDropOp::New(SelectionSetGuid));
}

void SSelectionSetButton::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (DragDropEvent.GetOperationAs<FAIESelectionSetDragDropOp>().IsValid())
	{
		bIsDragHovered = true;
	}
	SButton::OnDragEnter(MyGeometry, DragDropEvent);
}

void SSelectionSetButton::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	bIsDragHovered = false;
	SButton::OnDragLeave(DragDropEvent);
}

FReply SSelectionSetButton::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bIsDragHovered = false;
	TSharedPtr<FAIESelectionSetDragDropOp> DragOp = DragDropEvent.GetOperationAs<FAIESelectionSetDragDropOp>();
	if (DragOp.IsValid() && DragOp->DraggedGuid != SelectionSetGuid)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
		{
			if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
			{
				int32 TargetRow = 0;
				if (SelectionSets->GetItemRow(SelectionSetGuid, TargetRow))
				{
					SelectionSets->SetItemRow(DragOp->DraggedGuid, TargetRow);
				}
			}
		}
		return FReply::Handled();
	}
	return SButton::OnDrop(MyGeometry, DragDropEvent);
}

//////////////////////////////////////////////////////////////
/// SSelectionSets
///////////////////////////////////////////////////////////

void SSelectionSets::Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode)
{
	SetTextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex));

	LastMovieSceneSig = FGuid();
	FocusOnTextBlock = FGuid();
	bSetUpTimerIsSet = false;
	TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset();
	if (SequencerPtr.IsValid())
	{
		SequencerPtr->OnActivateSequence().AddSP(this, &SSelectionSets::OnActivateSequence);
		SequencerPtr->OnPreNavigateToSequence().AddSP(this, &SSelectionSets::SaveLastJson);
		SequencerPtr->OnMovieSceneDataChanged().AddSP(this, &SSelectionSets::OnMovieSceneDataChanged);
		SequencerPtr->OnCloseEvent().AddSP(this, &SSelectionSets::OnCloseEvent);
		SequencerPtr->OnMovieSceneBindingsChanged().AddSP(this, &SSelectionSets::OnMovieSceneBindingsChanged);
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			NewSelectionSet(SelectionSets);
		}
	}
	else
	{
		return;
	}
	const FButtonStyle& ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Menu.Button");
	const float MenuIconSize = FAppStyle::Get().GetFloat("Menu.MenuIconSize");

	SAssignNew(MainAddFilterSetSlot,SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						//.FillContentWidth(1.0)
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.Padding(5.0f)
						[
							SNew(SActionButton)
								.OnClicked(this, &SSelectionSets::OnAddClicked)
								.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
								.Text(LOCTEXT("AddSet", "Add Set"))
								.ToolTipText(LOCTEXT("AddSelectionSetTooltip", "Add a new Selection Set"))
						]
					+ SHorizontalBox::Slot()
						.FillWidth(1.f)
						[
							SNew(SSpacer)
						]
						+ SHorizontalBox::Slot().AutoWidth()
						[
							SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "SimpleButton")
								.ToolTipText(LOCTEXT("DeleteAllSets", "Delete All Selection Sets"))
								.OnClicked_Lambda([]() -> FReply
								{
									if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
									{
										if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
										{
											SelectionSets->DeleteAllSetItems(SequencerPtr);
										}
									}
									return FReply::Handled();
								})
								[
									SNew(SImage)
										.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
								]
						]
						+ SHorizontalBox::Slot().AutoWidth()
						[
							SNew(SComboButton)
								.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
								.OnGetMenuContent(this, &SSelectionSets::MakeImportExportMenu)
								.ButtonContent()
								[
									SNew(SImage)
										.Image(FAppStyle::Get().GetBrush("Icons.Reimport"))
								]
						]
					+ SHorizontalBox::Slot()
						//.FillContentWidth(1.0)
						.AutoWidth()
						.HAlign(EHorizontalAlignment::HAlign_Right)
						.Padding(4.0f)
						.VAlign(VAlign_Center)
						[

							SNew(SCheckBox)
								.Padding(4)
								.Style(FAppStyle::Get(), "FilterBar.FilterButton")
								.ToolTipText(LOCTEXT("OnlyShowSelected", "Only Show Sets on the Selected or last Selected object"))
								.IsChecked_Lambda([WeakSequencer = SequencerPtr->AsWeak()]() -> ECheckBoxState
									{
										bool bEnabled = false;

										if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
										{
											if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
											{
												bEnabled = SelectionSets->GetShowAndSetSelectedOnly();
											}
										}
										return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
									})
								.OnCheckStateChanged_Lambda([this, WeakSequencer = SequencerPtr->AsWeak()](ECheckBoxState InCheckBoxState)
									{
										const bool bIsChecked = InCheckBoxState == ECheckBoxState::Checked;
										if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
										{
											if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
											{
												SelectionSets->SetShowAndSetSelectedOnly(bIsChecked);
											}
										}
										bPopulateUI = true;
									})
								.CheckBoxContentUsesAutoWidth(true)
								[
									SNew(SImage)
										.Image_Lambda([]() -> const FSlateBrush*
											{
												return FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.FilterSelected").GetIcon();
											})
										.ColorAndOpacity_Lambda([WeakSequencer = SequencerPtr->AsWeak()]()
											{
												FSlateColor SlateColor = FSlateColor::UseForeground();

												if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
												{
													if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
													{
														bool bValue = SelectionSets->GetShowAndSetSelectedOnly();
														if (bValue == true)
														{
															SlateColor = FAppStyle::GetSlateColor("SelectionColor");

														}
													}
												}
												return SlateColor;
											})
								]
						]

				];
	SAssignNew(ActiveActorsSlot, SBox)
	.Padding(0.0f)
	.WidthOverride(200.f)
	.Visibility_Lambda([this]()
		{
			return (ActiveActorsWidget.IsValid() && ActiveActorsWidget->GetChildren() && ActiveActorsWidget->GetChildren()->Num() > 0)
				? EVisibility::Visible : EVisibility::Collapsed;
		})
	  [
		SNew(SBorder)
			.Padding(2.0)
			.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
			[
				// Panel background, seen between items
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FStyleColors::Panel)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(0.0f)
					[
						SNew(SBox)
							.MaxDesiredHeight(100)
							[
								SNew(SScrollBox)
									+ SScrollBox::Slot()
									[
										SAssignNew(ActiveActorsWidget, SWrapBox)
											.UseAllottedSize(true) // This is the key property
											.InnerSlotPadding(FVector2D(10.0f, 10.0f))
											.Orientation(Orient_Horizontal) // Default, but good to be explicit
									]
							]
					]
			]
	];
	SAssignNew(SearchSetsSlot, SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		.Padding(0.0f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(4.0f, 0.f, 0.f, 0.0f)
				.MaxWidth(20)
				.FillContentWidth(1.0)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
						.WidthOverride(MenuIconSize)
						.HeightOverride(MenuIconSize)
						[
							SNew(SImage)
								.Image_Lambda([]() -> const FSlateBrush*
									{
										return FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), "ControlRig.SelectionSet").GetIcon();
									})
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]

			+ SHorizontalBox::Slot()
				.FillContentWidth(1.0)
				.MaxWidth(50.)
				.Padding(4.0f, 0.f, 0.f, 0.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Text_Lambda([InArgs]()-> FText
							{
								const FText Set = LOCTEXT("Set", "Set");
								return Set;
							})
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.f, 0.f, 0.0f)
				[
					SNew(SBox)
						.MinDesiredWidth(150)
						[
							SAssignNew(SetSearchBox, SSearchBox)
								.OnTextChanged_Lambda([this](const FText& InText)
									{
										SetTextFilter->SetFilterText(InText);
										PopulateSelectionSets();
									})
						]
				]
			+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]
		];

	SAssignNew(ActiveSetsSlot, SBox)
		.Padding(0.0f)
		[
			SNew(SBorder)
				.Padding(2.0)
				.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
				[
					// Panel background, seen between items
					SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
						.BorderBackgroundColor(FStyleColors::Panel)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.Padding(0.0f)
						[

							SAssignNew(SelectionSetsWidget, SWrapBox)
								.UseAllottedSize(true) // This is the key property
								.InnerSlotPadding(FVector2D(10.0f, 10.0f))
								.Orientation(Orient_Horizontal) // Default, but good to be explicit
						]
				]
		];

	constexpr bool bIsVertical = true;
	if (bIsVertical)
	{
		ChildSlot
		[
			SNew(SBox)
				[
					SNew(SBorder)
						.Content()
						[
							SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
								.AutoHeight()
								[
									MainAddFilterSetSlot.ToSharedRef()
								]
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
								.AutoHeight()
								[
									ActiveActorsSlot.ToSharedRef()
								]
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
								.AutoHeight()
								[
									SearchSetsSlot.ToSharedRef()
								]
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
								.AutoHeight()
								[
									ActiveSetsSlot.ToSharedRef()

								]
						]
				]
		];
	}
	else
	{
		ChildSlot
		[
			SNew(SBox)
				[
					SNew(SBorder)
						.Content()
						[
							SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
								.AutoHeight()
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
									.HAlign(EHorizontalAlignment::HAlign_Left)
									.AutoWidth()
									[
										MainAddFilterSetSlot.ToSharedRef()
									]
									+ SHorizontalBox::Slot()
									.FillWidth(1.0)
									.HAlign(EHorizontalAlignment::HAlign_Center)
									[
										SNew(SSpacer)
									]
									+ SHorizontalBox::Slot()
									.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
									.HAlign(EHorizontalAlignment::HAlign_Right)
									.AutoWidth()
									[
										SearchSetsSlot.ToSharedRef()
									]
								]
								+ SVerticalBox::Slot()
								.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
								.AutoHeight()
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
									.AutoWidth()
									[
										ActiveActorsSlot.ToSharedRef()
									]
									+ SHorizontalBox::Slot()
									.Padding(FMargin(0.0f, 0.f, 0.f, 0.f))
									.AutoWidth()
									[
										ActiveSetsSlot.ToSharedRef()
									]
								]
						]
				]
		];
	}
	
	SetEditMode(InEditMode);
	RegisterSelectionChanged();
	SetCanTick(true);
	bUpdateBindings = true;
	SetCursor(EMouseCursor::Default);

}

TSharedRef<SWidget> SSelectionSets::MakeImportExportMenu()
{
	constexpr bool bCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);


	MenuBuilder.BeginSection("ImportExport", LOCTEXT("ImportExport", "Import/Export"));

	{
		FUIAction Action = FUIAction(
			FExecuteAction::CreateLambda([this]()
				{
					ImportLastSelectionSet();
				}));

		const FText Label = LOCTEXT("FastImport", "Import Last");
		const FText ToolTipText = LOCTEXT("FastImporttooltip", "Import Last Selection Set that was opened");
		FSlateIcon Icon(FAppStyle::GetAppStyleSetName(),"Icons.Reimport");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, Icon, Action);
	}

	{
		FUIAction Action = FUIAction(
			FExecuteAction::CreateLambda([this]()
				{
					ImportFromJson();
				}));

		const FText Label = LOCTEXT("ImportSets", "Import Sets");
		const FText ToolTipText = LOCTEXT("ImportTooltip", "Import Selection Sets from JSON File");
		FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Icons.Import");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, Icon, Action);
	}
	MenuBuilder.AddSeparator();
	{
		FUIAction Action = FUIAction(
			FExecuteAction::CreateLambda([this]()
				{
					ExportToJson();
				}));

		const FText Label = LOCTEXT("ExportSets", "Export Sets");
		const FText ToolTipText = LOCTEXT("Exporttooltip", "Export Selection Sets as JSON File ");
		FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Icons.Export");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, Icon, Action);
	}

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SSelectionSets::CreateActorMenu(const FActorWithSelectionSet& ActorWithSelectionSet)
{
	constexpr bool bCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

	MenuBuilder.BeginSection("Visibility", LOCTEXT("Visibility", "Visibility"));

	FUIAction Action = FUIAction(
		FExecuteAction::CreateLambda([&ActorWithSelectionSet]()
			{
				ActorWithSelectionSet.ShowAllControlsOnThisActor();
			}));

	const FText Label = LOCTEXT("ShowAll", "Show All");
	const FText ToolTipText = LOCTEXT("ShowAlltooltip", "Show all controls or the actor");
	MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SSelectionSets::PopulateUI()
{
	PopulateActiveActors();
	PopulateSelectionSets();
}

void SSelectionSets::PopulateActiveActors()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		const float MenuIconSize = FAppStyle::Get().GetFloat("Menu.MenuIconSize");
		
		
		auto AddActor = [this, WeakSequencer = SequencerPtr->AsWeak(), MenuIconSize](SWrapBox& Grid, int32 Row, int32 Column, const TWeakObjectPtr<AActor>& ActorPtr, FActorWithSelectionSet& ActorWithSeletionSet)
			{
				const auto GetSelectionSetInfo = [WeakSequencer, ActorPtr]()
				{
					const TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin();
					UAIESelectionSets* SelectionSets = SequencerPtr ? UAIESelectionSets::GetSelectionSets(SequencerPtr) : nullptr;
					FActorWithSelectionSet* ActorWithSeletionSet = SelectionSets ? SelectionSets->ActorsWithSelectionSets.ActorsWithSet.Find(ActorPtr) : nullptr;
					return ActorWithSeletionSet;
				};
				
				Grid.AddSlot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
							.OnGetMenuContent_Lambda([this, GetSelectionSetInfo]()
							{
								const FActorWithSelectionSet* ActorWithSeletionSet = GetSelectionSetInfo();
								return ActorWithSeletionSet ? CreateActorMenu(*ActorWithSeletionSet) : SNullWidget::NullWidget;
							})
							.Style(FAppStyle::Get(), "FilterBar.FilterButton")
							.IsChecked_Lambda( [GetSelectionSetInfo]() -> ECheckBoxState
							{
								const FActorWithSelectionSet* ActorWithSeletionSet = GetSelectionSetInfo();
								const bool bEnabled = ActorWithSeletionSet && ActorWithSeletionSet->GetSelected();
								return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
							.OnCheckStateChanged_Lambda([this, ActorPtr, WeakSequencer](ECheckBoxState InCheckBoxState)
								{
									if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
									{
										if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
										{
											const bool bAdd = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
											const bool bIsChecked = InCheckBoxState == ECheckBoxState::Checked;
											if (bAdd == false && bIsChecked)
											{
												for (TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : SelectionSets->ActorsWithSelectionSets.ActorsWithSet)
												{
													Pair.Value.SetSelected(false);
												}
											}
											if (FActorWithSelectionSet* ActorWithSeletionSet = SelectionSets->ActorsWithSelectionSets.ActorsWithSet.Find(ActorPtr))
											{
												ActorWithSeletionSet->SetSelected(bIsChecked);
											}
										}
									}
											

									PopulateSelectionSets();
								})
							[
								SNew(SBorder)
									.Padding(1.0f)
									.BorderImage(FAppStyle::Get().GetBrush("FilterBar.FilterBackground"))
									[
										SNew(SHorizontalBox)
											+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.AutoWidth()
											[
												SNew(SImage)
													.Image(FAppStyle::Get().GetBrush("FilterBar.FilterImage"))
													.ColorAndOpacity_Lambda([GetSelectionSetInfo]()
													{
														if (const FActorWithSelectionSet* ActorWithSeletionSet = GetSelectionSetInfo())
														{
															return ActorWithSeletionSet->GetSelected()
																? FSlateColor(ActorWithSeletionSet->Color)
																: FAppStyle::Get().GetSlateColor("Colors.Recessed");
														}
														return FSlateColor(FLinearColor::White);
													})
											]
										+ SHorizontalBox::Slot()
											.Padding(4.0f, 0.f, 0.f, 0.0f)
											.AutoWidth()
											.HAlign(HAlign_Center)
											.VAlign(VAlign_Center)
											[
												SNew(SBox)
													.WidthOverride(MenuIconSize)
													.HeightOverride(MenuIconSize)
													[
														SNew(SImage)
															.Image_Lambda([GetSelectionSetInfo]() -> const FSlateBrush*
															{
																if (const FActorWithSelectionSet* ActorWithSeletionSet = GetSelectionSetInfo())
																{

																	for (const TWeakObjectPtr<UObject>& Object : ActorWithSeletionSet->Actors)
																	{
																		if (UControlRig* ControlRig = Cast<UControlRig>(Object.Get()))
																		{
																			return FSlateIconFinder::FindIconForClass(USkeleton::StaticClass()).GetIcon();
																		}
																	}
																	return FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetIcon();
																}
																return nullptr;
															})
															.ColorAndOpacity(FSlateColor::UseForeground())
													]
											]
										+ SHorizontalBox::Slot()
											.Padding(FMargin(4, 1, 4, 1))
											.VAlign(VAlign_Center)
											[
												SNew(STextBlock)
													.Text_Lambda([GetSelectionSetInfo]()
													{
														const FActorWithSelectionSet* SelectionSet = GetSelectionSetInfo();
														return SelectionSet ? SelectionSet->Name : FText::GetEmpty();
													})
													.IsEnabled_Lambda([GetSelectionSetInfo]
													{
														const FActorWithSelectionSet* SelectionSet = GetSelectionSetInfo();
														return SelectionSet && SelectionSet->GetSelected();
													})
											]
									]
							]
					];
			};

		if (ActiveActorsWidget.IsValid())
		{
			ActiveActorsWidget->ClearChildren();

			if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
			{
				int32 Row = 0;
				int32 Column = 0;
				constexpr int32 MaxColumn = 1;
				for (TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : SelectionSets->ActorsWithSelectionSets.ActorsWithSet)
				{
					if (SelectionSets->IsVisible(Pair.Value) == false)
					{
						continue;
					}
					AddActor(*ActiveActorsWidget, Row, Column, Pair.Key, Pair.Value);
					if (++Column > MaxColumn)
					{
						Column = 0;
						++Row;
					}
				}
			}
		}
	}
}

void SSelectionSets::PopulateSelectionSets()
{
	auto FilterString = [this](const FString& StringToTest) ->bool
		{
			return SetTextFilter->TestTextFilter(FBasicStringFilterExpressionContext(StringToTest));
		};
	
	SelectionSetsButton.Reset();
	auto AddRow = [this](SWrapBox& Grid, TSharedPtr<ISequencer>& SequencerPtr, int32 Row, FAIESelectionSetItem* Item)
	{
		if (Item == nullptr)
		{
			return;
		}
		
		
		FGuid ItemGuid = Item->Guid;
		//Item.ViewData.
		Grid.AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
					.BorderBackgroundColor_Lambda([this,ItemGuid, WeakSequencer = SequencerPtr->AsWeak()]()
						{
							if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
							{
								if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
								{
									if (FAIESelectionSetItem* SetItem = SelectionSets->SelectionSets.Find(ItemGuid))
									{
										const TSharedPtr<SSelectionSetButton>& Button = SelectionSetsButton.FindRef(ItemGuid);
										float Alpha = 0.5f;
										if (Button.IsValid() && (Button->IsHovered() || Button->IsDragHovered()))
										{
											Alpha = 1.0f;
										}
										FLinearColor ItemColor = SetItem->ViewData.Color;
										return  FSlateColor(FLinearColor(ItemColor.R, ItemColor.G, ItemColor.B, Alpha));
									}
								}
							}
							return FSlateColor(FLinearColor::White);
							})
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
				[
					SAssignNew(SelectionSetsButton.FindOrAdd(ItemGuid), SSelectionSetButton)
						.Guid(ItemGuid)
						.SetsWidget(this)
						.OnClicked_Lambda([this, WeakSequencer = SequencerPtr->AsWeak(), ItemGuid]()
							{
								if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
								{
									if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
									{
										const bool bToggle = FSlateApplication::Get().GetModifierKeys().IsControlDown();
										const bool bAdd = FSlateApplication::Get().GetModifierKeys().IsShiftDown();

										SelectionSets->SelectItem(ItemGuid,false,bAdd, bToggle);
									}
								}	
								return FReply::Handled();
							})
						[
							
							SAssignNew(SelectionSetsTextBlock.FindOrAdd(ItemGuid), SInlineEditableTextBlock)
								.Justification(ETextJustify::Center)
								.Text_Lambda([ItemGuid, WeakSequencer = SequencerPtr->AsWeak()]()
								{
										if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
										{
											if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
											{
												if (FAIESelectionSetItem* SetItem = SelectionSets->SelectionSets.Find(ItemGuid))
												{
													return SetItem->ItemName;
												}
											}
										}
										return FText();
								})
								.OnTextCommitted_Lambda([WeakSequencer = SequencerPtr->AsWeak(),ItemGuid](const FText & InNewText, ETextCommit::Type InTextCommit)
								{
										if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
										{
											if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
											{
												SelectionSets->RenameSetItem(ItemGuid, InNewText);
											}
										}
								})
								
						]
				]
			];

	};
	if (SelectionSetsWidget.IsValid())
	{
		if (SelectionSetsWidget->GetChildren() && SelectionSetsWidget->GetChildren()->Num())
		{
			SelectionSetsWidget->ClearChildren();
		}
		if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
		{
			if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, false))
			{
				int32 Row = 0;
				//int32 Column = 0;
				//constexpr int32 MaxColumn = 5;
				TArray<FGuid> Guids = SelectionSets->GetActiveSelectionSets();
				for (const FGuid& Guid : Guids)
				{
					if (FAIESelectionSetItem* SetItem = SelectionSets->SelectionSets.Find(Guid))
					{
						if (SetTextFilter->GetFilterText().IsEmpty() || FilterString(SetItem->ItemName.ToString()))
						{
							AddRow(*SelectionSetsWidget, SequencerPtr, Row, SetItem);

							/*
							AddRow(*SelectionSetsWidget, Row, Column, SetItem);
							if (++Column > MaxColumn)
							{
								Column = 0;
								++Row;
							}
							*/
						}
					}
				}
			}
		}
	}
}

void SSelectionSets::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (ISequencer* Sequencer = GetSequencer())
	{
		FGuid CurrentMovieSceneSig = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetSignature();
		if (LastMovieSceneSig != CurrentMovieSceneSig)
		{
			LastMovieSceneSig = CurrentMovieSceneSig;
		}
	}
	if (bUpdateBindings == true)
	{
		bUpdateBindings = false;
		bPopulateUI = false;
		UpdateBindings();
	}
	if (bPopulateUI)
	{
		bPopulateUI = false;
		PopulateUI();
	}
	if (FocusOnTextBlock.IsValid())
	{
		if (TSharedPtr<SInlineEditableTextBlock>* TextBlock = SelectionSetsTextBlock.Find(FocusOnTextBlock))
		{
			if (TextBlock->IsValid())
			{
				(*TextBlock)->EnterEditingMode();
			}
			FocusOnTextBlock = FGuid();
		}
	}
}

FReply SSelectionSets::OnSelectionFilterClicked()
{
	return FReply::Handled();
}

bool SSelectionSets::IsSelectionFilterActive() const
{
	return false;
}

SSelectionSets::SSelectionSets()
{
}

SSelectionSets::~SSelectionSets()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		SaveLastJson();
		SequencerPtr->OnActivateSequence().RemoveAll(this);
		SequencerPtr->OnPreNavigateToSequence().RemoveAll(this);
		SequencerPtr->OnMovieSceneDataChanged().RemoveAll(this);
		SequencerPtr->OnCloseEvent().RemoveAll(this);
		SequencerPtr->OnMovieSceneBindingsChanged().RemoveAll(this);
	}

	// unregister previous one
	if (OnSelectionChangedHandle.IsValid())
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		FLevelEditorModule::FActorSelectionChangedEvent& ActorSelectionChangedEvent = LevelEditor.OnActorSelectionChanged();
		ActorSelectionChangedEvent.Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}

	for (TWeakObjectPtr<UControlRig>& ControlRig : BoundControlRigs)
	{
		if (ControlRig.IsValid())
		{
			ControlRig.Get()->ControlRigBound().RemoveAll(this);
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig.Get()->GetObjectBinding();
			if (Binding)
			{
				Binding->OnControlRigBind().RemoveAll(this);
			}
		}
	}
	BoundControlRigs.SetNum(0);
}


void SSelectionSets::PostUndo(bool bSuccess)
{
	 bPopulateUI = true;
}

void SSelectionSets::PostRedo(bool bSuccess)
{
	 bPopulateUI = true;
}

bool SSelectionSets::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	for (const TWeakObjectPtr<UControlRig>& ControlRig : BoundControlRigs)
	{
		if (ControlRig.IsValid())
		{
			if (InContext.PrimaryObject == ControlRig.Get())
			{
				return true;
			}
			for (const TPair<UObject*, FTransactionObjectEvent>& ObjectTransactionPair : TransactionObjectContexts)
			{
				if (ObjectTransactionPair.Get<0>() == ControlRig.Get())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void SSelectionSets::HandleControlSelected(UControlRig* Subject, FRigControlElement* ControlElement, bool bSelected)
{
	FControlRigBaseDockableView::HandleControlSelected(Subject, ControlElement, bSelected);
	bPopulateUI = true;
}

void SSelectionSets::RegisterSelectionChanged()
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	FLevelEditorModule::FActorSelectionChangedEvent& ActorSelectionChangedEvent = LevelEditor.OnActorSelectionChanged();

	// unregister previous one
	if (OnSelectionChangedHandle.IsValid())
	{
		ActorSelectionChangedEvent.Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}

	// register
	OnSelectionChangedHandle = ActorSelectionChangedEvent.AddSP(this, &SSelectionSets::OnActorSelectionChanged);
}

void SSelectionSets::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	bPopulateUI = true;
}

FString  SSelectionSets::LastSelectionSetJSON;

void  SSelectionSets::OnCloseEvent(TSharedRef<ISequencer> InSequencer)
{
	SaveLastJson();
}

void SSelectionSets::SaveLastJson()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence()))
		{
			if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(LevelSequence, false /*Add if doesn't exist*/))
			{
				if (SelectionSets->ExportAsJsonString(LastSelectionSetJSON) == false)
				{
					UE_LOGF(LogControlRig, Error, "Selection Sets: Error fast JSON export");
				}
			}
		}
	}
}

void SSelectionSets::UpdateBindings()
{
	GEditor->GetTimerManager()->SetTimerForNextTick([this, WeakThis = this->AsWeak()]()
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
		{
			if (WeakThis.IsValid())
			{
				if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, false /*Add if doesn't exist*/))
				{
					SelectionSets->SelectionListChanged().RemoveAll(this);
					SelectionSets->SelectionListChanged().AddSP(this, &SSelectionSets::HandleSelectionListChanged);

					SelectionSets->SequencerBindingsAdded(SequencerPtr);
				}
				PopulateUI();
			}
		}
	});
	
}

void SSelectionSets::OnMovieSceneBindingsChanged()
{
	bUpdateBindings = true; //do it next tick
}

void SSelectionSets::OnActivateSequence(FMovieSceneSequenceIDRef ID)
{
	bUpdateBindings = true; //do it next tick;
}

void SSelectionSets::OnMovieSceneDataChanged(EMovieSceneDataChangeType Type)
{
	/** Handle to help updating selection on tick tick to avoid too many flooded selections*/
	static bool bTimerIsSet = false;
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, false /*Add if doesn't exist*/))
		{
			switch (Type)
			{

				case EMovieSceneDataChangeType::MovieSceneStructureItemAdded:
				case EMovieSceneDataChangeType::MovieSceneStructureItemRemoved:
				case EMovieSceneDataChangeType::MovieSceneStructureItemsChanged:
					if (bSetUpTimerIsSet == false)
					{
						bSetUpTimerIsSet = true;
						GEditor->GetTimerManager()->SetTimerForNextTick([this, WeakSequencer = SequencerPtr->AsWeak(), WeakThis = this->AsWeak()]()
							{
								bSetUpTimerIsSet = false;
								if (!WeakThis.IsValid())
								{
									return;
								}
								if (TSharedPtr<ISequencer> SequencerPtr = WeakSequencer.Pin())
								{
									if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, false /*Add if doesn't exist*/))
									{
										SelectionSets->SequencerBindingsAdded(SequencerPtr);
										PopulateActiveActors();
									}
								}
							});
					}
					break;
				default:
					break;

			}
		}
	}
}

void SSelectionSets::SetEditMode(FControlRigEditMode& InEditMode)
{
	FControlRigBaseDockableView::SetEditMode(InEditMode);
	ModeTools = InEditMode.GetModeManager();
	if (FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(ModeTools->GetActiveMode(FControlRigEditMode::ModeName)))
	{
		TArrayView<TWeakObjectPtr<UControlRig>> ControlRigs = EditMode->GetControlRigs();
		for (TWeakObjectPtr<UControlRig>& ControlRig : ControlRigs)
		{
			if (ControlRig.IsValid())
			{
				if (!ControlRig.Get()->ControlRigBound().IsBoundToObject(this))
				{
					ControlRig.Get()->ControlRigBound().AddSP(this, &SSelectionSets::HandleOnControlRigBound);
					BoundControlRigs.Add(ControlRig);
				}
				const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig.Get()->GetObjectBinding();
				if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
				{
					Binding->OnControlRigBind().AddSP(this, &SSelectionSets::HandleOnObjectBoundToControlRig);
				}
			}
		}
	}
}

void SSelectionSets::HandleControlAdded(UControlRig* ControlRig, bool bIsAdded)
{
	FControlRigBaseDockableView::HandleControlAdded(ControlRig, bIsAdded);
	if (ControlRig)
	{
		if (bIsAdded == true)
		{
			if (!ControlRig->ControlRigBound().IsBoundToObject(this))
			{
				ControlRig->ControlRigBound().AddSP(this, &SSelectionSets::HandleOnControlRigBound);
				BoundControlRigs.Add(ControlRig);
			}
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
			{
				Binding->OnControlRigBind().AddSP(this, &SSelectionSets::HandleOnObjectBoundToControlRig);
			}
		}
		else
		{
			BoundControlRigs.Remove(ControlRig);
			ControlRig->ControlRigBound().RemoveAll(this);
			const TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			if (Binding)
			{
				Binding->OnControlRigBind().RemoveAll(this);
			}
		}
	}
}

void SSelectionSets::HandleOnControlRigBound(UControlRig* InControlRig)
{
	if (!InControlRig)
	{
		return;
	}

	const TSharedPtr<IControlRigObjectBinding> Binding = InControlRig->GetObjectBinding();

	if (Binding && !Binding->OnControlRigBind().IsBoundToObject(this))
	{
		Binding->OnControlRigBind().AddSP(this, &SSelectionSets::HandleOnObjectBoundToControlRig);
	}
}

//mz todo need to test recompiling
void SSelectionSets::HandleOnObjectBoundToControlRig(UObject* InObject)
{

}

void SSelectionSets::NewSelectionSet(UAIESelectionSets* NewSelectionSet)
{
	if (NewSelectionSet)
	{
		NewSelectionSet->SelectionListChanged().RemoveAll(this);
		NewSelectionSet->SelectionListChanged().AddSP(this, &SSelectionSets::HandleSelectionListChanged);
		if (NewSelectionSet->ExportAsJsonString(LastSelectionSetJSON) == false)
		{
			UE_LOGF(LogControlRig, Error, "Selection Sets: Error fast JSON export");
		}
	}
}


FReply SSelectionSets::OnAddClicked()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, true /*Add if doesn't exist*/))
		{
			NewSelectionSet(SelectionSets);

			FocusOnTextBlock = SelectionSets->CreateSetItemFromSelection(SequencerPtr);
		}
	}
	return FReply::Handled();
}

bool SSelectionSets::ImportOrExportPath(FString& OutPath, bool bImport)
{
	TArray<FString> Filenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform == nullptr)
	{
		return false;
	}
	static FString LastFileName;

	bool bOpen;

	if (bImport)
	{
		FString LastRigMapperImportDirectory;
		if (!GConfig->GetString(TEXT("AIESelectionSets"), TEXT("LastImportDirectory"), LastRigMapperImportDirectory, GEditorPerProjectIni))
		{
			LastRigMapperImportDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_IMPORT);
		}

		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ImportSelectionSetFromJson", "Import Selection Set from Json...").ToString(),
			LastRigMapperImportDirectory,
			TEXT(""),
			TEXT("json (*.json)|*.json|"),
			EFileDialogFlags::None,
			Filenames
		);
	}
	else
	{
		FString LastRigMapperExportDirectory;
		if (!GConfig->GetString(TEXT("AIESelectionSets"), TEXT("LastExportDirectory"), LastRigMapperExportDirectory, GEditorPerProjectIni))
		{
			LastRigMapperExportDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT);
		}

		bOpen = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("ExportToJson", "Export Selection Set to Json...").ToString(),
			LastRigMapperExportDirectory,
			LastFileName,
			TEXT("json (*.json)|*.json|"),
			EFileDialogFlags::None,
			Filenames
		);
	}

	if (bOpen && !Filenames.IsEmpty())
	{
		LastFileName = FPaths::GetBaseFilename(Filenames[0]);
		OutPath = Filenames[0];
	}

	return bOpen;
}


bool SSelectionSets::ImportLastSelectionSet()
{
	if (LastSelectionSetJSON.IsEmpty() == false)
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
		{
			if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, true /*Add if doesn't exist*/))
			{
				FScopedTransaction Transaction(LOCTEXT("LoadFromLastJson", "Load Selection Set from last open Selection Set"));

				if (SelectionSets->LoadFromJsonString(SequencerPtr, LastSelectionSetJSON))
				{
					bPopulateUI = true;
					if (SelectionSets->ExportAsJsonString(LastSelectionSetJSON) == false) //just resave the string, it may add have more things added to it
					{
						UE_LOGF(LogControlRig, Error, "Selection Sets: Error fast JSON export");
						return false;
					}
				}
				else
				{
					UE_LOGF(LogControlRig, Error, "Selection Sets: Error fast JSON import");
					return false;
				}
			}
		}
	}
	return true;
}

bool SSelectionSets::ImportFromJson()
{
	FFilePath JsonFile;
	bool bSucceeded = false;
	FString LastPath;

	if (ImportOrExportPath(JsonFile.FilePath, true))
	{
		if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
		{
			if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr, true /*Add if doesn't exist*/))
			{
				NewSelectionSet(SelectionSets);
				FScopedTransaction Transaction(LOCTEXT("LoadFromJson", "Load Selection Sets from Json file"));

				if (SelectionSets->LoadFromJsonFile(SequencerPtr, JsonFile))
				{
					LastPath = JsonFile.FilePath;
					// set the new import directory from the last successful save
					FString NewDirectory = FPaths::GetPath(LastPath);
					GConfig->SetString(TEXT("AIESelectionSets"), TEXT("LastImportDirectory"), *NewDirectory, GEditorPerProjectIni);
					GConfig->Flush(false, GEditorPerProjectIni);
					bPopulateUI = true;
					return true;
				}
			}
		}
	}

	return false;
}
bool SSelectionSets::ExportToJson()
{
	if (TSharedPtr<ISequencer> SequencerPtr = UAIESelectionSets::GetSequencerFromAsset())
	{
		if (UAIESelectionSets* SelectionSets = UAIESelectionSets::GetSelectionSets(SequencerPtr))
		{
			FFilePath Path;
			bool bSucceeded = false;

			if (ImportOrExportPath(Path.FilePath, false))
			{
				if (SelectionSets->ExportAsJsonFile(Path))
				{
					// set the new export directory
					FString NewDirectory = FPaths::GetPath(Path.FilePath);
					GConfig->SetString(TEXT("AIESelectionSets"), TEXT("LastExportDirectory"), *NewDirectory, GEditorPerProjectIni);
					GConfig->Flush(false, GEditorPerProjectIni);
					return true;
				}
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE




