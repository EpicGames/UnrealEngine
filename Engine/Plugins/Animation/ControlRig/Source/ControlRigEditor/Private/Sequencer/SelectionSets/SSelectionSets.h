// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EditMode/ControlRigBaseDockableView.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "ISequencer.h"
#include "Misc/Guid.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UAIESelectionSets;
class SSelectionSets;
class SSearchBox;
class FTextFilterExpressionEvaluator;
struct FActorWithSelectionSet;
class SBox;
class SBorder;
class SScrollBox;
class SWrapBox;
class SInlineEditableTextBlock;

class SSelectionSetButton: public SButton
{
public:
	SLATE_BEGIN_ARGS(SSelectionSetButton)
		{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_ARGUMENT(FGuid, Guid)
		SLATE_ARGUMENT(SSelectionSets*, SetsWidget )
	SLATE_END_ARGS()
	
	void Construct(const FArguments& Args);
	
	bool IsDragHovered() const { return bIsDragHovered; }

protected:
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:
	bool bIsDragHovered = false;
	FGuid SelectionSetGuid;
	SSelectionSets* SelectionSetsWidget = nullptr;
	void AddSelectionToSet() const;
	void RemoveSelectionFromSet() const;
	void DeleteSet() const;
	void RenameSet() const;
	TSharedRef<SWidget> CreateSelectionSetColorWidget(const FGuid& InGuid, TSharedPtr<ISequencer>& SequencerPtr) const;
	void CreateMirror() const;
	void SelectMirror() const;
	void HideSelectionSet() const;
	void ShowSelectionSet() const;
	void IsolateSelectionSet() const;
	void ShowAllSelectionSet() const;
	void KeyAll() const;
};

class SSelectionSets : public FControlRigBaseDockableView, public SCompoundWidget, FSelfRegisteringEditorUndoClient
{
	SLATE_BEGIN_ARGS(SSelectionSets) {}
	SLATE_END_ARGS()
	SSelectionSets();
	~SSelectionSets();

	/** FSelfRegisteringEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess );
	virtual void PostRedo( bool bSuccess );
	virtual bool MatchesContext( const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts ) const;	
	/** END FSelfRegisteringEditorUndoClient interface */

	void Construct(const FArguments& InArgs, FControlRigEditMode& InEditMode);
	//SCompuntWidget
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	//FControlRigBaseDockableView overrides
	virtual TSharedRef<FControlRigBaseDockableView> AsSharedWidget() override { return SharedThis(this); }
	virtual void SetEditMode(FControlRigEditMode& InEditMode) override;
	virtual void HandleControlAdded(UControlRig* ControlRig, bool bIsAdded) override;

	void SetFocusOnTextBlock(const FGuid& InGuid) { FocusOnTextBlock = InGuid; }
private:
	void HandleOnControlRigBound(UControlRig* InControlRig);
	void HandleOnObjectBoundToControlRig(UObject* InObject);
	virtual void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected) override;

	//set of control rigs we are bound too and need to clear delegates from
	TArray<TWeakObjectPtr<UControlRig>> BoundControlRigs;
private:
	//actor selection changing
	void RegisterSelectionChanged();
	void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);
	FDelegateHandle OnSelectionChangedHandle;

	//sequencer changing so need to refresh list
	void OnActivateSequence(FMovieSceneSequenceIDRef ID);
	void OnMovieSceneDataChanged(EMovieSceneDataChangeType);
	void OnCloseEvent(TSharedRef<ISequencer> InSequencer);
	void OnMovieSceneBindingsChanged();
	void UpdateBindings();

	FGuid LastMovieSceneSig = FGuid();


	void HandleSelectionListChanged(UAIESelectionSets*) { bPopulateUI = true; }
private:
	FReply OnSelectionFilterClicked();
	bool IsSelectionFilterActive() const;
	FReply OnAddClicked();
	bool ImportOrExportPath(FString& OutPath, bool bImport);
	bool ImportLastSelectionSet();
	bool ImportFromJson();
	bool ExportToJson();
	void SaveLastJson();
	void NewSelectionSet(UAIESelectionSets* NewSelectionSet);

private:
	
	TSharedRef<SWidget> CreateActorMenu(const FActorWithSelectionSet& ActorWithSelectionSet);
	TSharedRef<SWidget> MakeImportExportMenu();

//widgets
private:
	//main UI elements
	TSharedPtr<SBorder> MainAddFilterSetSlot;
	TSharedPtr<SBox> ActiveActorsSlot;
	TSharedPtr<SBorder> SearchSetsSlot;
	TSharedPtr<SBox> ActiveSetsSlot;

	TSharedPtr<SWrapBox> ActiveActorsWidget;
	TSharedPtr<SWrapBox> SelectionSetsWidget;

	TMap<FGuid, TSharedPtr<SSelectionSetButton>> SelectionSetsButton;
	TMap<FGuid, TSharedPtr<SInlineEditableTextBlock>> SelectionSetsTextBlock;
	TSharedPtr<SSearchBox> SetSearchBox;
	TSharedPtr<FTextFilterExpressionEvaluator>  SetTextFilter;

	FGuid  FocusOnTextBlock;

private:
	void PopulateUI();
	void PopulateActiveActors();
	void PopulateSelectionSets();

	//things we handle on next tick
	bool bSetUpTimerIsSet = false;
	bool bUpdateBindings = false;
	bool bPopulateUI = false;
private:
	//cache of last select set closed, for quick loading of sets onto new level sequences
	static FString  LastSelectionSetJSON;
};
