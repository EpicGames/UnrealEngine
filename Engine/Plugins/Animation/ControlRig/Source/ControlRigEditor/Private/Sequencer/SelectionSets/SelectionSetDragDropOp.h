// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Misc/Guid.h"

class FAIESelectionSetDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAIESelectionSetDragDropOp, FDecoratedDragDropOp)

	FGuid DraggedGuid;

	static TSharedRef<FAIESelectionSetDragDropOp> New(const FGuid& InGuid)
	{
		TSharedRef<FAIESelectionSetDragDropOp> Operation = MakeShared<FAIESelectionSetDragDropOp>();
		Operation->DraggedGuid = InGuid;
		Operation->DefaultHoverText = FText::GetEmpty();
		Operation->CurrentHoverText = FText::GetEmpty();
		Operation->Construct();
		return Operation;
	}
};
