// Copyright Epic Games, Inc. All Rights Reserved.
#include "SelectionSets.h"
#include "SelectionSetsSettings.h"

#include "ScopedTransaction.h"
#include "ControlRig.h"
#include "IControlRigObjectBinding.h"
#include "Misc/Optional.h"
#include "ISequencer.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/Selection/SequencerCoreSelection.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "LevelSequence.h"
#include "ControlRigSequencerEditorLibrary.h"
#include "Sequencer/ControlRigParameterTrackEditor.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "ILevelSequenceEditorToolkit.h"
#include "LevelSequencePlayer.h"
#include "EditMode/ControlRigEditMode.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "MovieScene.h"
#include "Tools/EvaluateSequencerTools.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "Tools/ControlRigPose.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonTypes.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#include "Tools/ControlRigPoseMirrorTable.h"

#include "Animation/AnimationSettings.h"
#include "Animation/MirrorDataTable.h"

#define LOCTEXT_NAMESPACE "SelectionSets"


///////////////////////////////////////////////////////////
/// FAIESelectionSetItem
///////////////////////////////////////////////////////////

bool FAIESelectionSetItem::IsMultiAsset() const
{
	for (const FAIESelectionSetItemName& LocalName : Names)
	{
		if (LocalName.OwnerActorName.IsEmpty() == false)
		{
			return true;
		}
	}
	return false;
}


bool FAIESelectionSetItem::ContainsOwnerActor(const FString& ActorLabel) const
{
	for (const FAIESelectionSetItemName& LocalName : Names)
	{
		if (LocalName.OwnerActorName == ActorLabel)
		{
			return true;
		}
	}
	return false;
}

bool FAIESelectionSetItem::ContainsItem(const FString& InName) const
{
	for (const FAIESelectionSetItemName& LocalName : Names)
	{
		if (LocalName.Name == InName)
		{
			return true;
		}
	}
	return false;
}

FAIESelectionSetItemName* FAIESelectionSetItem::GetItem(const FString& InName)
{
	for (FAIESelectionSetItemName& LocalName : Names)
	{
		if (LocalName.Name == InName)
		{
			return &LocalName;
		}
	}
	return nullptr;
}

bool FAIESelectionSetItem::RemoveItem(const FString& InName, const FString& ActorLabel)
{
	TArray<int32> IndicesToRemove; //may have dups
	for (int32 Index = 0; Index < Names.Num(); ++Index)
	{
		FAIESelectionSetItemName& LocalName = Names[Index];
		if (LocalName.Name == InName)
		{
			if (LocalName.OwnerActorName.IsEmpty() || ActorLabel.IsEmpty()
				|| LocalName.OwnerActorName == ActorLabel)
			{
				IndicesToRemove.Insert(Index, 0);
			}
		}
	}
	if (IndicesToRemove.Num() > 0)
	{
		for (int32 RemoveIndex : IndicesToRemove)
		{
			Names.RemoveAt(RemoveIndex);
		}
		return true;
	}
	return false;
}

FString FAIESelectionSetItem::GetMirror(const FString& InString)
{
	FString NewString = InString;
	if (UAnimationSettings* AnimSettings = UAnimationSettings::Get())
	{
		FString PossibleNewString = FControlRigPoseMirrorTable::GetMirrorString(InString, AnimSettings->MirrorFindReplaceExpressions);
		if (PossibleNewString.IsEmpty() == false)
		{
			NewString = PossibleNewString;
		}
	}
	return NewString;
}

///////////////////////////////////////////////////////////
/// FActorWithSelectionSet
///////////////////////////////////////////////////////////

AActor* FActorWithSelectionSet::GetControlRigActor(const UControlRig* InControlRig)
{
	TSharedPtr<IControlRigObjectBinding> ObjectBinding = InControlRig->GetObjectBinding();
	if (ObjectBinding.IsValid())
	{
		if (USceneComponent* BoundSceneComponent = Cast<USceneComponent>(ObjectBinding->GetBoundObject()))
		{
			return BoundSceneComponent->GetOwner();
		}
	}
	return (AActor*)(nullptr);
}

void FActorWithSelectionSet::SetUpOwnedActors(const TWeakObjectPtr<AActor>& ActorOwner, TSharedPtr<ISequencer>& InSequencer, UAIESelectionSets* SelectionSets)
{
	if (AActor* Actor = Cast<AActor>(ActorOwner))
	{
		OwnedActors.Reset();
		for (const FGuid& Guid : SetsThatMatch)
		{
			if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets->SelectionSets.Find(Guid))
			{
				for (const FAIESelectionSetItemName& SetItem : CurrentSetItem->Names)
				{
					if (SetItem.OwnerActorName.Len() > 0 && SetItem.Name == ActorOwner->GetActorLabel(false))
					{
						OwnedActors.Add(Guid, SetItem.OwnerActorName);
					}
				}
			}
		}
	}
}

///////////////////////////////////////////////////////////
/// FActorsWithSelectionSets
///////////////////////////////////////////////////////////

bool FActorsWithSelectionSets::ContainsActor(const FString& InString, bool bShowSelectedOnly) const
{
	for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSet)
	{
		if (Pair.Value.IsActive(bShowSelectedOnly))
		{
			if (const AActor* Actor = Pair.Key.Get())
			{
				const FString ActorLabel = Actor->GetActorLabel();
				if (ActorLabel == InString)
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FActorsWithSelectionSets::SetUpFromSequencer(TSharedPtr<ISequencer>& Sequencer, UAIESelectionSets* SelectionSets, const TSet<TWeakObjectPtr<UObject>>& MakeSelectedIfPresent)
{
	//if nothing there than automatically selected even if not in MakeSelectedIfPresent

	// const TMap<FGuid, FAIESelectionSetItem>& SelectionSets
	if (Sequencer.IsValid() == false || SelectionSets == nullptr || SelectionSets->SelectionSets.Num() == 0)
	{
		ActorsWithSet.Reset();
		return;
	}
	
	//used to seee if we have any left overs
	TArray<TWeakObjectPtr<AActor>> CurrentSet;
	ActorsWithSet.GenerateKeyArray(CurrentSet);

	
	if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequencer->GetFocusedMovieSceneSequence()))
	{
		TArray<UControlRig*> ControlRigs;
		TArray<FControlRigSequencerBindingProxy> Proxies = UControlRigSequencerEditorLibrary::GetControlRigs(LevelSequence);
		for (FControlRigSequencerBindingProxy& Proxy : Proxies)
		{
			if (UControlRig* ControlRig = Proxy.ControlRig.Get())
			{
				ControlRigs.Add(ControlRig);
			}
		}
		const UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (MovieScene)
		{
			const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
			for (const FMovieSceneBinding& Binding : Bindings)
			{
				for (TWeakObjectPtr<> BoundObject : Sequencer->FindBoundObjects(Binding.GetObjectGuid(), Sequencer->GetFocusedTemplateID()))
				{
					if (AActor* Actor = Cast<AActor>(BoundObject))
					{
						FString ActorName = Actor->GetActorLabel(false);
						for (const TPair<FGuid, FAIESelectionSetItem>& Pair : SelectionSets->SelectionSets)
						{
							if (Pair.Value.ContainsItem(ActorName))
							{
								const bool bExists = ActorsWithSet.Contains(Actor);
								FActorWithSelectionSet& Set = ActorsWithSet.FindOrAdd(Actor);
								Set.SetsThatMatch.Add(Pair.Key);
								Set.Name = FText::FromString(Actor->GetActorLabel());
								if (bExists == false)
								{
									Set.Actors.Add(Actor);
									Set.Color = SelectionSets->GetNextActorColor();
									Set.SetSelected(true);
								}
								if (MakeSelectedIfPresent.Contains(Actor))
								{
									Set.SetSelected(true);
								}
								CurrentSet.Remove(Actor);
							}
						}
					}
				}
			}
		}

		for (UControlRig* ControlRig : ControlRigs)
		{
			if (ControlRig)
			{
				const TArray<FRigControlElement*> Controls = ControlRig->AvailableControls();
				for (const FRigControlElement* Control : Controls)
				{
					if (Control)
					{
						for (const TPair<FGuid, FAIESelectionSetItem>& Pair : SelectionSets->SelectionSets)
						{
							if (Pair.Value.ContainsItem(Control->GetName()))
							{
								if (AActor* Actor = FActorWithSelectionSet::GetControlRigActor(ControlRig))
								{
									FString ActorName = Actor->GetActorLabel(false);

									const bool bExists = ActorsWithSet.Contains(Actor);
									FActorWithSelectionSet& Set = ActorsWithSet.FindOrAdd(Actor);

									Set.Name = FText::FromString(ActorName);
									if (bExists == false)
									{
										Set.Color = SelectionSets->GetNextActorColor();
										Set.SetSelected(true);
									}
									Set.Actors.Add(ControlRig);
									Set.SetsThatMatch.Add(Pair.Key);
									if (MakeSelectedIfPresent.Contains(ControlRig))
									{
										Set.SetSelected(true);
									}
									CurrentSet.Remove(Actor);
								}
							}
						}
					}
				}
			}
		}
	}

	//now remove those that no longer exist with any set's active, BUT if the name matches make them matchstate
	if (CurrentSet.Num() > 0)
	{
		for (TWeakObjectPtr<AActor>& Object : CurrentSet)
		{
			if (FActorWithSelectionSet* Set = ActorsWithSet.Find(Object))
			{
				for (TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSet)
				{
					if ((Object != Pair.Key) && (Set->Name.EqualTo(Pair.Value.Name)))
					{
						Pair.Value.SetSelected(Set->GetSelected());
						Pair.Value.Color = Set->Color;
					}
				}
				ActorsWithSet.Remove(Object);
			}
		}
	}
	for (TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSet)
	{
		Pair.Value.SetUpOwnedActors(Pair.Key, Sequencer, SelectionSets);
	}
}

void FAIESelectionSetItem::GetAllChildren(const UAIESelectionSets* SelectionSets, TArray<FGuid>& AllChildren) const
{
	AllChildren.Add(Guid);
	if (SelectionSets)
	{
		for (const FGuid& Child : Children)
		{
			if (const FAIESelectionSetItem* Item = SelectionSets->SelectionSets.Find(Child))
			{
				Item->GetAllChildren(SelectionSets, AllChildren);
			}
		}
	}
}

///////////////////////////////////////////////////////////
/// FActorWithSelectionSet
///////////////////////////////////////////////////////////

bool FActorWithSelectionSet::IsSelectedInViewport(bool bOnlyInViewport) const
{
	USelection* ActorSelection = GEditor->GetSelectedActors();
	TArray<TWeakObjectPtr<UObject>> SelectedActors;
	if (ActorSelection)
	{
		ActorSelection->GetSelectedObjects(SelectedActors);
	}
	for (const TWeakObjectPtr<UObject>& Object : Actors)
	{
		if (const AActor* Actor = Cast<const AActor>(Object.Get()))
		{
			if (SelectedActors.Contains(Object))
			{
				bWasSelectedInViewport = true;
				return true;
			}
		}
		else if (const UControlRig* ControlRig = Cast<const UControlRig>(Object.Get()))
		{
			TArray<FName> SelectedControls = ControlRig->CurrentControlSelection();
			if (SelectedControls.Num() > 0)
			{
				bWasSelectedInViewport = true;
				return true;
			}
			if (AActor* CRActor = GetControlRigActor(ControlRig))
			{
				if (SelectedActors.Contains(CRActor))
				{
					bWasSelectedInViewport = true;
					return true;
				}
			}
		}
	}
	if (bOnlyInViewport == false && bWasSelectedInViewport == true)
	{
		return true;
	}

	return false;

}
bool FActorWithSelectionSet::IsControlRig() const
{
	for (const TWeakObjectPtr<UObject>& Object : Actors)
	{
		if (const UControlRig* ControlRig = Cast<const UControlRig>(Object.Get()))
		{
			return true;
		}
	}
	return false;
}

bool FActorWithSelectionSet::IsActive(bool bShowSelectedOnly) const
{
	if (IsControlRig() == false)
	{
		return true;
	}
	if (bShowSelectedOnly)
	{
		return IsSelectedInViewport();
	}
	return bIsSelected;
}

void FActorWithSelectionSet::ShowAllControlsOnThisActor() const
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		for (const TWeakObjectPtr<UObject>& Object : Actors)
		{
			if (UControlRig* ControlRig = Cast<UControlRig>(Object.Get()))
			{
				TSet<FString> ControlNames;
				const TArray<FRigControlElement*> AvailbleControls = ControlRig->AvailableControls();
				ControlNames.Reserve(AvailbleControls.Num());
				for (const FRigControlElement* Element : AvailbleControls)
				{
					ControlNames.Add(Element->GetName());
				}
				//not undoable yet
				EditMode->ShowControlRigControls(ControlRig, ControlNames, true);
			}
			else if (AActor* Actor = Cast<AActor>(Object.Get()))
			{
				Actor->SetActorHiddenInGame(false);
				Actor->SetIsTemporarilyHiddenInEditor(false);
			}
		}
	}
}

///////////////////////////////////////////////////////////
/// UAIESelectionSets
///////////////////////////////////////////////////////////

UAIESelectionSets::UAIESelectionSets(class FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{

}
TSharedPtr<ISequencer> UAIESelectionSets::GetSequencerFromAsset()
{
	ULevelSequence* LevelSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, false);
	ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
	TSharedPtr<ISequencer> SequencerPtr = LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	return SequencerPtr;
}

UAIESelectionSets* UAIESelectionSets::GetSelectionSets(const TSharedPtr<ISequencer>& SequencerPtr, bool bAddIfDoesNotExist)
{
	if (SequencerPtr.IsValid() == false)
	{
		return nullptr;
	}
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
	return UAIESelectionSets::GetSelectionSets(LevelSequence, bAddIfDoesNotExist);
}

UAIESelectionSets* UAIESelectionSets::GetSelectionSets(ULevelSequence* LevelSequence, bool bAddIfDoesNotExist)
{
	if (!LevelSequence)
	{
		return nullptr;
	}
	if (LevelSequence && LevelSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			UAIESelectionSets* SelectionSets = AssetUserDataInterface->GetAssetUserData< UAIESelectionSets >();
			if (!SelectionSets && bAddIfDoesNotExist)
			{
				SelectionSets = NewObject<UAIESelectionSets>(LevelSequence, NAME_None, RF_Public | RF_Transactional);
				AssetUserDataInterface->AddAssetUserData(SelectionSets);
			}
			return SelectionSets;
		}
	}
	return nullptr;
}

bool UAIESelectionSets::IsVisible(const FActorWithSelectionSet& ActorWithSelectionSet) const
{

	if (bShowSelectedOnly)
	{
		return false;
	}
	if (ActorWithSelectionSet.IsControlRig() == true)
	{
		for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
		{
			if (Pair.Value.IsControlRig() == true)
			{
				if (&ActorWithSelectionSet != &Pair.Value)
				{
					return true;
				}
			}
		}
	}
	return false;
}

TArray<FGuid> UAIESelectionSets::GetActiveSelectionSets() 
{
	//when bShowSelected is on we cache the last set and use that when none are selected. Also need to keep track
	//of when it was previously selected
	//if not show selected we just use the internal select flag (or it's an actor)

	TSet<FGuid> ActiveSelectionSetsSet;
	TSet<FGuid> SelectedInViewport;
	TSet<TWeakObjectPtr<AActor>> SelectedActors;
	for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
	{
		if (bShowSelectedOnly)
		{
			constexpr bool bOnlyInViewport = true;
			if (Pair.Value.IsSelectedInViewport(true))
			{
				SelectedInViewport.Append(Pair.Value.SetsThatMatch);
				ActiveSelectionSetsSet.Append(Pair.Value.SetsThatMatch);
				SelectedActors.Add(Pair.Key);
			}
		}
		else 
		{
			if (Pair.Value.IsActive(bShowSelectedOnly))
			{
				ActiveSelectionSetsSet.Append(Pair.Value.SetsThatMatch);
			}
		}
		
	}
	
	if (bShowSelectedOnly)
	{
		if (SelectedInViewport.Num() == 0)
		{
			for(const FGuid& Guid: LastActiveSelectionSetsSet)
			{
				if (SelectionSets.Contains(Guid))
				{
					ActiveSelectionSetsSet.Add(Guid);
				}
			}
		}
		else//something is selected we need to clear the unselected actors previous state
		{
			for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
			{
				if (SelectedActors.Contains(Pair.Key) == false)
				{
					Pair.Value.bWasSelectedInViewport = false;
				}
			}
		}
		LastActiveSelectionSetsSet = ActiveSelectionSetsSet;
	}

	//remove ones that aren't selected if multiasset
	TArray<FGuid> SetsToRemove;
	for (const FGuid& Guid : ActiveSelectionSetsSet)
	{
		if (FAIESelectionSetItem* SetItem = SelectionSets.Find(Guid))
		{
			if (SetItem->IsMultiAsset())
			{
				bool bHasOneActive = false;
				for (const FAIESelectionSetItemName& Name : SetItem->Names)
				{
					if (Name.OwnerActorName.IsEmpty() == false)
					{
						if (ActorsWithSelectionSets.ContainsActor(Name.OwnerActorName,bShowSelectedOnly))
						{
							bHasOneActive = true;
							break;
						}
					}
				}
				if (bHasOneActive == false)
				{
					SetsToRemove.Add(Guid);
				}
			}
		}
	}
	if (SetsToRemove.Num() > 0)
	{
		for (const FGuid& Guid : SetsToRemove)
		{
			ActiveSelectionSetsSet.Remove(Guid);
		}
	}

	TArray<FGuid> ActiveSelectionSetsArray;
	if (ActiveSelectionSetsSet.Num() > 0)
	{
		ActiveSelectionSetsArray = ActiveSelectionSetsSet.Array();
		ActiveSelectionSetsArray = SortSelectionSetsByRow(ActiveSelectionSetsArray);
	}



	return ActiveSelectionSetsArray;
}

TArray<FGuid> UAIESelectionSets::SortSelectionSetsByRow(TArray<FGuid>& InSelectionSets) 
{
	TSortedMap<int32, FGuid> SortedSelectionSets;

	for (const FGuid& Guid : InSelectionSets)
	{
		if (FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
		{
			int32 Row = CurrentSetItem->ViewData.Row;
			while (SortedSelectionSets.Contains(Row))
			{
				++Row;
			}
			CurrentSetItem->ViewData.Row = Row;
			SortedSelectionSets.Add(Row, Guid);
		}
	}
	TArray<FGuid> OutSelectionSets;
	SortedSelectionSets.GenerateValueArray(OutSelectionSets);
	return OutSelectionSets;
}

FGuid UAIESelectionSets::CreateSetItemFromSelection()
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return CreateSetItemFromSelection(SequencerPtr);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return FGuid();
}

FGuid UAIESelectionSets::CreateSetItemFromSelection(TSharedPtr<ISequencer>& InSequencer)
{
	using namespace UE::AIE;
	TArray<FControlRigAndControlsAndTrack> SelectedCRs;
	TArray<FObjectAndTrack> SelectedBoundObjects;
	{
		const FScopedTransaction Transaction(LOCTEXT("AddSelctionSet_Transcation", "Add Selection Set"), !GIsTransacting);
		Modify();

		TSet<TWeakObjectPtr<UObject>> ObjectsAdded;

		FAIESelectionSetItem Item;
		Item.Guid = FGuid::NewGuid();
		Item.ViewData.Color = GetNextSelectionSetColor();
		FSequencerSelected::GetSelectedControlRigsAndBoundObjects(InSequencer.Get(), SelectedCRs, SelectedBoundObjects);

		if (SelectedCRs.Num() <= 0 && SelectedBoundObjects.Num() <= 0)
		{
			return FGuid();
		}

		FString Name = FString::Printf(TEXT("Set_%d"), SelectionSets.Num());
		FText Text;
		Text = Text.FromString(Name);
		Item.ItemName = Text;

		TMultiMap<FString, FString> ActiveNames;
		TArray<FString> Duplicates;
		FControlRigPoseMirrorTable MirrorTable;
		FControlRigControlPose TempPose;
		UControlRig* LastOwnerControlRig = nullptr;
		if (SelectedCRs.Num() > 0)
		{
			const bool bOwnerActorForCRs = SelectedCRs.Num() > 1 || SelectedBoundObjects.Num() > 0;
			for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
			{
				UControlRig* ControlRig = CRControls.ControlRig;
				if (ControlRig)
				{
					LastOwnerControlRig = ControlRig;
					TempPose.SavePose(ControlRig, true);
					for (FName& ControlName : CRControls.Controls)
					{
						FAIESelectionSetItemName ItemName;
						ItemName.Name = ControlName.ToString();
						if (FRigControlCopy* CopyRigControl = MirrorTable.GetControl(ControlRig, TempPose, ControlName, true))
						{
							ItemName.MirrorName = CopyRigControl->Name.ToString();
						}
						else
						{
							ItemName.MirrorName = ItemName.Name;
						}

						ItemName.Type = (int32)EAIESelectionSetItemType::ControlRig;
						Duplicates.SetNum(0);
						ActiveNames.MultiFind(ItemName.Name, Duplicates);
						ItemName.Duplicates = Duplicates.Num();
						if (bOwnerActorForCRs)
						{
							if (AActor* Actor = FActorWithSelectionSet::GetControlRigActor(ControlRig))
							{
								ItemName.OwnerActorName = Actor->GetActorLabel(false);
							}
						}
						Item.Names.Add(ItemName);
						ActiveNames.Add(ItemName.Name, ItemName.Name);
					}
					ObjectsAdded.Add(CRControls.ControlRig);
				}
			}
		}
		TOptional<FString> OwnerActorName;
		if (LastOwnerControlRig)
		{
			if (AActor* Actor = FActorWithSelectionSet::GetControlRigActor(LastOwnerControlRig))
			{
				OwnerActorName = Actor->GetActorLabel(false);
			}
		}
		for (FObjectAndTrack& ObjectAndTrack : SelectedBoundObjects)
		{
			if (AActor* Actor = Cast<AActor>(ObjectAndTrack.BoundObject))
			{
				FAIESelectionSetItemName ItemName;
				ItemName.Name = Actor->GetActorLabel(true);
				if (OwnerActorName.IsSet())
				{
					ItemName.OwnerActorName = OwnerActorName.GetValue();
				}
				ItemName.MirrorName = ItemName.Name;
				ItemName.Type = (int32)EAIESelectionSetItemType::Actor;
				Duplicates.SetNum(0);
				ActiveNames.MultiFind(ItemName.Name, Duplicates);
				ItemName.Duplicates = Duplicates.Num();
				Item.Names.Add(ItemName);
				ActiveNames.Add(ItemName.Name, ItemName.Name);
				ObjectsAdded.Add(Actor);
			}
		}
		Item.ViewData.Row = SelectionSets.Num();
		if (Item.Names.Num() == 1)
		{
			Text = Text.FromString(Item.Names[0].Name);
			Item.ItemName = Text;
		}

		SelectionSets.Add(Item.Guid, Item);

		ActorsWithSelectionSets.SetUpFromSequencer(InSequencer, this, ObjectsAdded);

		SelectionSetsChangedBroadcast();
		return Item.Guid;
	}
}

FGuid UAIESelectionSets::CreateMirror(const FGuid& InGuid)
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return CreateMirror(SequencerPtr, InGuid);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return FGuid();
}

FGuid UAIESelectionSets::CreateMirror(TSharedPtr<ISequencer>& InSequencer, const FGuid& InGuid)
{
	using namespace UE::Sequencer;

	TArray<FGuid> AllGuids;
	FGuid NewGuid;
	if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(InGuid))
	{
		const FScopedTransaction Transaction(LOCTEXT("CreateMirrorSelectionSet", "Create Mirror Selection Set"), !GIsTransacting);
		bool bNoteSelectionChange = false;
		FAIESelectionSetItem Item;
		NewGuid = FGuid::NewGuid();
		Item.Guid = NewGuid;
		Item.ViewData.Color = GetNextSelectionSetColor();
		FString NewName = FAIESelectionSetItem::GetMirror(CurrentSetItem->ItemName.ToString());
		if (NewName.IsEmpty())
		{
			NewName = FString::Printf(TEXT("Set %d"), SelectionSets.Num());
		}
		FText Text;
		Text = Text.FromString(NewName);
		Item.ItemName = Text;
		for (const FAIESelectionSetItemName& Name : CurrentSetItem->Names)
		{
			FAIESelectionSetItemName NewItem;
			NewItem.Duplicates = Name.Duplicates;
			NewItem.Name = Name.MirrorName;
			NewItem.OwnerActorName = Name.OwnerActorName;
			NewItem.MirrorName = Name.Name;
			NewItem.Type = Name.Type;
			
			Item.Names.Add(NewItem);
		}
		for (const FGuid& ChildGuid : CurrentSetItem->Children)
		{
			Item.ViewData.Row = SelectionSets.Num();
			SelectionSets.Add(Item.Guid, Item);
			FGuid NewChildGuid = CreateMirror(InSequencer, ChildGuid);
			Item.Children.Add(NewChildGuid);
		}
		if (NewGuid.IsValid())
		{
			Item.ViewData.Row = SelectionSets.Num();
			SelectionSets.Add(Item.Guid, Item);
			TSet<TWeakObjectPtr<UObject>> NoObjectsAdded;
			if (LastActiveSelectionSetsSet.Contains(InGuid))
			{
				LastActiveSelectionSetsSet.Add(NewGuid);
			}
			ActorsWithSelectionSets.SetUpFromSequencer(InSequencer, this, NoObjectsAdded);

			SelectionSetsChangedBroadcast();
		}
	}
	return NewGuid;
}

bool SetActorAsActive(AActor* InActor, bool bSetActive);

/* Get the actors which are selectable*/
UFUNCTION(BlueprintCallable, Category = "Actor")
TArray<AActor> GetActiveActors(AActor* InActore);

void UAIESelectionSets::SequencerBindingsAdded(TSharedPtr<ISequencer>& InSequencer)
{
	TSet<TWeakObjectPtr<UObject>> NoObjectsAdded;
	ActorsWithSelectionSets.SetUpFromSequencer(InSequencer, this, NoObjectsAdded);
	SelectionSetsChangedBroadcast();

}

bool UAIESelectionSets::AddSelectionToSetItem(const FGuid& InGuid)
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return AddSelectionToSetItem(InGuid,SequencerPtr);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return false;
}

bool UAIESelectionSets::AddSelectionToSetItem(const FGuid& InGuid, TSharedPtr<ISequencer>& InSequencer)
{
	using namespace UE::AIE;
	if (FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		TArray<FControlRigAndControlsAndTrack> SelectedCRs;
		TArray<FObjectAndTrack> SelectedBoundObjects;
		
		FSequencerSelected::GetSelectedControlRigsAndBoundObjects(InSequencer.Get(), SelectedCRs, SelectedBoundObjects);

		if (SelectedCRs.Num() <= 0 && SelectedBoundObjects.Num() <= 0)
		{
			return false;
		}
		const FScopedTransaction Transaction(LOCTEXT("AddSelectionToSetItem", "Add Selection To Set Item"), !GIsTransacting);
		Modify();
		TMultiMap<FString, FString> ActiveNames;
		TArray<FString> Duplicates;
		//set up ActiveNames so we can get Duplicate counts
		for (const FAIESelectionSetItemName& Name : SetItem->Names)
		{
			ActiveNames.Add(Name.Name, Name.Name);

		}
		FControlRigPoseMirrorTable MirrorTable;
		FControlRigControlPose TempPose;
		for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
		{
			UControlRig* ControlRig = CRControls.ControlRig;
			if (ControlRig)
			{
				TempPose.SavePose(ControlRig, true);
				for (FName& ControlName : CRControls.Controls)
				{
					FAIESelectionSetItemName ItemName;
					ItemName.Name = ControlName.ToString();
					ItemName.Type = (int32)EAIESelectionSetItemType::ControlRig;
					if (FRigControlCopy* CopyRigControl = MirrorTable.GetControl(ControlRig, TempPose, ControlName, true))
					{
						ItemName.MirrorName = CopyRigControl->Name.ToString();
					}
					Duplicates.SetNum(0);
					ActiveNames.MultiFind(ItemName.Name, Duplicates);
					ItemName.Duplicates = Duplicates.Num();
					SetItem->Names.Add(ItemName);
				}
			}
		}
		for (FObjectAndTrack& ObjectAndTrack : SelectedBoundObjects)
		{
			if (AActor* Actor = Cast<AActor>(ObjectAndTrack.BoundObject))
			{
				FAIESelectionSetItemName ItemName;
				ItemName.Name = Actor->GetActorLabel(true);
				ItemName.Type = (int32)EAIESelectionSetItemType::Actor;
				Duplicates.SetNum(0);
				ActiveNames.MultiFind(ItemName.Name, Duplicates);
				ItemName.Duplicates = Duplicates.Num();
				SetItem->Names.Add(ItemName);
			}
		}
		return true;
		
	}
	return false;
}

bool UAIESelectionSets::RemoveSelectionFromSetItem(const FGuid& InGuid)
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return RemoveSelectionFromSetItem(InGuid, SequencerPtr);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return false;
}

bool UAIESelectionSets::RemoveSelectionFromSetItem(const FGuid& InGuid, TSharedPtr<ISequencer>& InSequencer)
{
	using namespace UE::AIE;
	if (FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		//wrap scoped transaction since it can deselect control rigs
		TArray<FControlRigAndControlsAndTrack> SelectedCRs;
		TArray<FObjectAndTrack> SelectedBoundObjects;

		FSequencerSelected::GetSelectedControlRigsAndBoundObjects(InSequencer.Get(), SelectedCRs, SelectedBoundObjects);

		if (SelectedCRs.Num() <= 0 && SelectedBoundObjects.Num() <= 0)
		{
			return false;
		}
		const FScopedTransaction Transaction(LOCTEXT("RemoveSelectionFromSetItem", "Remove Selection From Set Item"), !GIsTransacting);
		Modify();
	
		FString OwnerActorName;
		for (FControlRigAndControlsAndTrack& CRControls : SelectedCRs)
		{
			if (AActor* Actor = FActorWithSelectionSet::GetControlRigActor(CRControls.ControlRig))
			{
				OwnerActorName = Actor->GetActorLabel(false);
			}
			else
			{
				OwnerActorName.Empty();
			}
			for (FName& ControlName : CRControls.Controls)
			{
				FString Name = ControlName.ToString();
				SetItem->RemoveItem(Name, OwnerActorName); //mz todo handle dups
			}
		}
		for (FObjectAndTrack& ObjectAndTrack : SelectedBoundObjects)
		{
			if (AActor* Actor = Cast<AActor>(ObjectAndTrack.BoundObject))
			{
				OwnerActorName = Actor->GetActorLabel(false);
				FString Name = Actor->GetActorLabel(true);
				SetItem->RemoveItem(Name, OwnerActorName);
			}
		}
		return true;

	}
	return false;
}

void UAIESelectionSets::GetItemGuids(const FText& ItemName, TArray<FGuid>& OutGuids) const
{
	for (const TPair<FGuid,FAIESelectionSetItem>& Pair : SelectionSets)
	{
		if (Pair.Value.ItemName.IdenticalTo(ItemName))
		{
			OutGuids.Add(Pair.Key);
		}
	}
}

bool UAIESelectionSets::IsMultiAsset(const FGuid& InGuid) const
{
	if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		return SetItem->IsMultiAsset();
	}
	return false;
}

bool UAIESelectionSets::GetItemName(const FGuid& InGuid, FText& OutName) const
{
	if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		OutName = SetItem->ItemName;
		return true;
	}
	return false;
}

bool UAIESelectionSets::RenameSetItem(const FGuid& InGuid, const  FText& NewName)
{
	FScopedTransaction Transaction(LOCTEXT("RenameSetItem_transaction", "Rename Selection Set"), !GIsTransacting);
	if (FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		Modify();
		FString String = NewName.ToString();
		String = String.Replace(TEXT(" "), TEXT("_"), ESearchCase::CaseSensitive);
		SetItem->ItemName = FText::FromString(String);
		return true;
	}
	Transaction.Cancel();
	return false;
}

bool UAIESelectionSets::DeleteSetItem(const FGuid& InGuid) 
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return DeleteSetItem(InGuid, SequencerPtr);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return false;
}

void UAIESelectionSets::UpdateRowValues(int32 StartRow, int32 EndRow, bool bIncrease)
{
	if (StartRow > EndRow)
	{
		int32 Swap = EndRow;
		EndRow = StartRow;
		StartRow = Swap;
	}
	for (TPair<FGuid, FAIESelectionSetItem>& Pair : SelectionSets)
	{
		if (Pair.Value.ViewData.Row >= StartRow && Pair.Value.ViewData.Row <= EndRow)
		{
			if (bIncrease)
			{
				Pair.Value.ViewData.Row++;
			}
			else
			{
				Pair.Value.ViewData.Row--;
			}
		}
	}
}

bool UAIESelectionSets::DeleteSetItem(const FGuid& InGuid, TSharedPtr<ISequencer>& InSequencer)
{
	FScopedTransaction Transaction(LOCTEXT("DeleteSetItem_transaction", "Delete Selection Set"), !GIsTransacting);
	if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		Modify();
		ShowOrHideControls(InGuid, true, false);
		int32 Row = SetItem->ViewData.Row;
		SelectionSets.Remove(InGuid);
		constexpr bool bIncrease = false;
		const int32 StartRow = Row;
		const int32 EndRow = SelectionSets.Num() - 1;
		UpdateRowValues(StartRow, EndRow, bIncrease);
		TSet<TWeakObjectPtr<UObject>> NoObjectsAdded;
		ActorsWithSelectionSets.SetUpFromSequencer(InSequencer, this, NoObjectsAdded);
		SelectionSetsChangedBroadcast();

		return true;
	}
	Transaction.Cancel();
	return false;
}

bool UAIESelectionSets::DeleteAllSetItems()
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return DeleteAllSetItems(SequencerPtr);
	}
	return false;
}

bool UAIESelectionSets::DeleteAllSetItems(TSharedPtr<ISequencer>& InSequencer)
{
	if (SelectionSets.IsEmpty())
	{
		return false;
	}
	FScopedTransaction Transaction(LOCTEXT("DeleteAllSetItems_transaction", "Delete All Selection Sets"), !GIsTransacting);
	Modify();
	for (const TPair<FGuid, FAIESelectionSetItem>& SetItem : SelectionSets)
	{
		ShowOrHideControls(SetItem.Key, true, false);
	}
	SelectionSets.Empty();
	TSet<TWeakObjectPtr<UObject>> NoObjectsAdded;
	ActorsWithSelectionSets.SetUpFromSequencer(InSequencer, this, NoObjectsAdded);
	SelectionSetsChangedBroadcast();
	return true;
}


bool UAIESelectionSets::SetActorAsActive(AActor* InActor, bool bSetActive)
{
	if (FActorWithSelectionSet* ActorWithSeletionSet = ActorsWithSelectionSets.ActorsWithSet.Find(InActor))
	{
		ActorWithSeletionSet->SetSelected(bSetActive);
		return true;
	}
	return false;
}

TArray<AActor*> UAIESelectionSets::GetAllActors() const
{
	TArray<AActor*>  Actors;
	for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
	{
		if (Pair.Key.IsValid())
		{
			Actors.Add(Pair.Key.Get());
		}
	}
	return Actors;
}


TArray<AActor*> UAIESelectionSets::GetActiveActors() const
{
	TArray<AActor*>  Actors;
	for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
	{
		if (Pair.Key.IsValid() && Pair.Value.IsActive(bShowSelectedOnly))
		{
			Actors.Add(Pair.Key.Get());
		}
	}
	return Actors;
}

bool UAIESelectionSets::GetItemColor(const FGuid& InGuid, FLinearColor& OutColor) const
{
	if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		OutColor = SetItem->ViewData.Color;
		return true;
	}
	return false;
}

bool UAIESelectionSets::SetItemColor(const FGuid& InGuid, const FLinearColor& InColor)
{
	FScopedTransaction Transaction(LOCTEXT("ColorSetItem_transaction", "Color Selection Set"), !GIsTransacting);
	if (FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		Modify();
		SetItem->ViewData.Color = InColor;
		SelectionSetsChangedBroadcast();
		return true;
	}
	Transaction.Cancel();
	return false;
}

bool UAIESelectionSets::GetItemRow(const FGuid& InGuid, int32& OutRow) const
{
	if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		OutRow = SetItem->ViewData.Row;
		return true;
	}
	return false;
}

bool UAIESelectionSets::SetItemRow(const FGuid& InGuid, int32 InRow)
{
	FScopedTransaction Transaction(LOCTEXT("SetItemRow_transaction", "Set Item Row"), !GIsTransacting);
	if (FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		const int32 OldRow = SetItem->ViewData.Row;
		if (OldRow != InRow)
		{
			Modify();
			if (InRow < OldRow)
			{
				const bool bIncrease = true;
				UpdateRowValues(InRow , OldRow, bIncrease);
			}
			else
			{
				const bool bIncrease = false;
				UpdateRowValues(OldRow, InRow, bIncrease);
			}
			SetItem->ViewData.Row = InRow;

			SelectionSetsChangedBroadcast();
			return true;
		}
	}
	Transaction.Cancel();
	return false;
}

static void ClearSelection()
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		if ((GEditor->GetSelectedActorCount() || GEditor->GetSelectedComponentCount()))
		{
			GEditor->SelectNone(false, true);
			GEditor->NoteSelectionChange();
		}
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		EditMode->GetAllSelectedControls(SelectedControls);
		for (TPair<UControlRig*, TArray<FRigElementKey>>& CurrentSelection : SelectedControls)
		{
			if (CurrentSelection.Key)
			{
				CurrentSelection.Key->ClearControlSelection(true);
			}
		}
	}
}

bool UAIESelectionSets::SelectItem(const FGuid& InGuid, bool bDoMirror, bool bAdd, bool bToggle, bool bSelect) const
{
	using namespace UE::Sequencer;

	TSharedPtr<ISequencer> Sequencer = UAIESelectionSets::GetSequencerFromAsset();
	if (Sequencer.IsValid() == false)
	{
		return false;
	}

	using namespace UE::Sequencer;
	TUniquePtr<FSelectionEventSuppressor> SequencerSelectionGuard;
	TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
	if (TSharedPtr<FSequencerSelection> SequencerSelection = SequencerViewModel ? SequencerViewModel->GetSelection() : nullptr)
	{
		SequencerSelectionGuard.Reset(new FSelectionEventSuppressor(SequencerSelection.Get()));
	}
	
	TArray<FGuid> AllGuids;
	if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
	{
		//first the do is get cached control rig selections, we do this for two reasons 1) we need to check selection state before clearing selection if bShowSelectedOnly is true
		//and 2. we need to select the controls AFTER the actors or the actor selection will deselect the controls
		//so we need do a two pass here
		const FScopedTransaction Transaction(LOCTEXT("SelectSelectionSet", "Select Selection Set"), !GIsTransacting);
		SetItem->GetAllChildren(this, AllGuids);
		bool bNoteSelectionChange = false;
		//we select actors first then control rigs second
		TMap<UControlRig*,const TArray<FAIESelectionSetItemName>> CachedControlRigs;
		TSet<FString> CachedControlRigActorNames;
		for (const FGuid& Guid : AllGuids)
		{
			if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
			{
				for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
			{
					if (Pair.Value.SetsThatMatch.Contains(Guid))
					{
						if (AActor* Key = Cast<AActor>(Pair.Key.Get()))
						{
							bool bIsActive = Pair.Value.IsActive(bShowSelectedOnly);
							if (!bIsActive && CurrentSetItem->IsMultiAsset()) //not active but it is a multiasset
							{
								if (CurrentSetItem->ContainsOwnerActor(Key->GetActorLabel()))
								{
									bIsActive = true;
								}
							}
							if (bIsActive)
							{
								for (const TWeakObjectPtr<UObject>& Object : Pair.Value.Actors)
								{
									if (UControlRig* ControlRig = Cast<UControlRig>(Object.Get()))
									{
										ControlRig->Modify();
										CachedControlRigs.Add(ControlRig, CurrentSetItem->Names);
										if (AActor* CRActor = FActorWithSelectionSet::GetControlRigActor(ControlRig))
										{
											FString ActorLabel = CRActor->GetActorLabel(false);
											CachedControlRigActorNames.Add(ActorLabel);
										}
									}
								}
							}
						}
					}
				}
			}
		}
		//now clear selection
		if (bSelect == true && (bToggle == false && bAdd == false))
		{
			ClearSelection();
		}

		TArray<TWeakObjectPtr<UObject>> SelectedActors;
		if (bToggle)
		{
			USelection* ActorSelection = GEditor->GetSelectedActors();
			if (ActorSelection)
			{
				ActorSelection->GetSelectedObjects(SelectedActors);
			}
		}
		//now do actors
		for (const FGuid& Guid : AllGuids)
		{
			if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
			{
				for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
				{
					bool bIsActive = Pair.Value.IsActive(bShowSelectedOnly);
					if (bIsActive)					
					{
						if (Pair.Value.SetsThatMatch.Contains(Guid))
						{
							if (AActor* Key = Cast<AActor>(Pair.Key.Get()))
							{
								//cache the
								for (const TWeakObjectPtr<UObject>& Object : Pair.Value.Actors)
								{
									if (AActor* Actor = Cast<AActor>(Object.Get()))
									{
										if (const FString* OwnedName = Pair.Value.OwnedActors.Find(Guid))
										{
											if (CachedControlRigActorNames.Contains(*OwnedName) == false)
											{
												continue;
											}
										}
										if (bToggle == false)
										{
											GEditor->SelectActor(Actor, bSelect, true);
										}
										else
										{
											const bool bToggleSelect = !SelectedActors.Contains(Actor);
											GEditor->SelectActor(Actor, bToggleSelect, true);
										}
									}
								}
							}
						}
					}
				}
			}
		}
		//do control rigs last
		TArray<FName> SelectedControls;
		for (TPair<UControlRig*, const TArray<FAIESelectionSetItemName>>& CachedControlRig : CachedControlRigs)
		{
			if (UControlRig* ControlRig = CachedControlRig.Key)
			{
				if (bToggle)
				{
					SelectedControls = ControlRig->CurrentControlSelection();
				}
				for (const FAIESelectionSetItemName& String : CachedControlRig.Value)
				{
					if (String.OwnerActorName.IsEmpty() == false)
					{
						if (AActor* Actor = FActorWithSelectionSet::GetControlRigActor(ControlRig))
						{
							FString ActorName = Actor->GetActorLabel(false);
							if (ActorName != String.OwnerActorName)
							{
								continue;
							}
						}
						else
						{
							continue;
						}
					}
					if (bDoMirror)
					{
						const FName Name(*String.MirrorName);
						bool bControlRigSelect = bSelect;
						if (bToggle)
						{
							bControlRigSelect = !SelectedControls.Contains(Name);
						}
						ControlRig->SelectControl(Name, bControlRigSelect,true);
					}
					else
					{
						const FName Name(*String.Name);
						bool bControlRigSelect = bSelect;
						if (bToggle)
						{
							bControlRigSelect = !SelectedControls.Contains(Name);
						}
						ControlRig->SelectControl(Name, bControlRigSelect,true);
					}
				}}
		}
	}
	return true;
}

bool UAIESelectionSets::ShowOrHideControls(const FGuid& InGuid, bool bShow, bool bDoMirror) const
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		//fscopedtransaction here
		if (bShow == false)//deselect before hidding
		{
			constexpr bool bDoAdd = false;
			constexpr bool bDoToggle = false;
			constexpr bool bSelect = false;
			SelectItem(InGuid, bDoMirror,bDoAdd, bDoToggle, bSelect);
		}
		TArray<FGuid> AllGuids;
		if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
		{
			SetItem->GetAllChildren(this, AllGuids);
			for (const FGuid& Guid : AllGuids)
			{
				if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
				{
					for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
					{
						if (Pair.Value.IsSelectedInViewport())
						{
							if (Pair.Value.SetsThatMatch.Contains(Guid))
							{
								if (AActor* Key = Cast<AActor>(Pair.Key.Get()))
								{
									for (const TWeakObjectPtr<UObject>& Object : Pair.Value.Actors)
									{
										if (UControlRig* ControlRig = Cast<UControlRig>(Object.Get()))
										{
											TSet<FString> ControlNames;
											ControlNames.Reserve(CurrentSetItem->Names.Num());
											for (const FAIESelectionSetItemName& String : CurrentSetItem->Names)
											{
												ControlNames.Add(String.Name);
											}
											//not undoable yet
											EditMode->ShowControlRigControls(ControlRig, ControlNames, bShow);
										}
										else if (AActor* Actor = Cast<AActor>(Object.Get()))
										{
											Actor->SetActorHiddenInGame(!bShow);
											Actor->SetIsTemporarilyHiddenInEditor(!bShow);
										}
									}
								}
							}
						}
					}
				}
			}
			return true;
		}
	}
	return false;
}

bool UAIESelectionSets::IsolateControls(const FGuid& InGuid) const
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		TArray<FGuid> AllGuids;
		if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
		{
			SetItem->GetAllChildren(this, AllGuids);
			for (const FGuid& Guid : AllGuids)
			{
				if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
				{
					for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
					{
						if (Pair.Value.IsSelectedInViewport())
						{
							if (Pair.Value.SetsThatMatch.Contains(Guid))
							{
								if (AActor* Key = Cast<AActor>(Pair.Key.Get()))
								{
									for (const TWeakObjectPtr<UObject>& Object : Pair.Value.Actors)
									{
										if (UControlRig* ControlRig = Cast<UControlRig>(Object.Get()))
										{
											TSet<FString> ControlNames;
											ControlNames.Reserve(CurrentSetItem->Names.Num());
											for (const FAIESelectionSetItemName& String : CurrentSetItem->Names)
											{
												ControlNames.Add(String.Name);
											}
											TSet<FString> ControlsToHide;
											TArray<FRigControlElement*> Controls = ControlRig->AvailableControls();
											bool bHasControlSoWeIsolate = false;
											for (const FRigControlElement* Control : Controls)
											{
												if (ControlNames.Contains(Control->GetName()) == false)
												{
													ControlsToHide.Add(Control->GetName());
												}
												else
												{
													bHasControlSoWeIsolate = true;
												}
											}
											//not undoable yet
											if (bHasControlSoWeIsolate)
											{
												EditMode->ShowControlRigControls(ControlRig, ControlNames, true);
												EditMode->ShowControlRigControls(ControlRig, ControlsToHide, false);
											}

										}
										else if (AActor* Actor = Cast<AActor>(Object.Get()))
										{
											//to do how to handle actors
											const bool bShow = true;
											Actor->SetActorHiddenInGame(!bShow);
											Actor->SetIsTemporarilyHiddenInEditor(!bShow);
										}
									}
								}
							}
						}
					}
				}
			}
			return true;
		}
	}
	return false;
}

bool UAIESelectionSets::ShowAllControls(const FGuid& InGuid) const
{
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		TArray<FGuid> AllGuids;
		if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
		{
			SetItem->GetAllChildren(this, AllGuids);
			for (const FGuid& Guid : AllGuids)
			{
				if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
				{
					for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
					{
						if (Pair.Value.IsSelectedInViewport())
						{
							if (Pair.Value.SetsThatMatch.Contains(Guid))
							{
								Pair.Value.ShowAllControlsOnThisActor();
							}
						}
					}
				}
			}
			return true;
		}
	}
	return false;
}

static UMovieSceneControlRigParameterTrack* GetControlRigTrack(TSharedPtr<ISequencer>& Sequencer, UControlRig* ControlRig)
{
	UMovieSceneSequence* OwnerSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	const UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return  nullptr;
	}

	const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		TArray<UMovieSceneTrack*> FoundTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
		for (UMovieSceneTrack* Track : FoundTracks)
		{
			if (UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
			{
				if (CRTrack->GetControlRig() == ControlRig)
				{
					return CRTrack;
				}
			}
		}
	}
	return  nullptr;
}

static UMovieScene3DTransformTrack* Get3DTransformTrack(TSharedPtr<ISequencer>& Sequencer, const AActor* InActor)
{
	UMovieSceneSequence* OwnerSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	const UMovieScene* MovieScene = OwnerSequence ? OwnerSequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return  nullptr;
	}

	const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		if (UMovieScene3DTransformTrack* TransformTrack = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->FindTrack<UMovieScene3DTransformTrack>(Binding.GetObjectGuid()))
		{
			for (TWeakObjectPtr<> BoundObject : Sequencer->FindBoundObjects(Binding.GetObjectGuid(), Sequencer->GetFocusedTemplateID()))
			{
				if (!BoundObject.IsValid())
				{
					continue;
				}
				if (AActor* BoundActor = Cast<AActor>(BoundObject.Get()))
				{
					if (BoundActor == InActor)
					{
						return TransformTrack;
					}
				}
				else if (USceneComponent* BoundComponent = Cast<USceneComponent>(BoundObject.Get()))
				{
					if (BoundComponent->GetOwner() == InActor)
					{
						return TransformTrack;
					}
				}
			}
		}
	}
	return nullptr;
}

void UAIESelectionSets::SetShowAndSetSelectedOnly(bool bInShowSelectedOnly)
{
	bShowSelectedOnly = bInShowSelectedOnly;
	SelectionSetsChangedBroadcast();
}

bool UAIESelectionSets::KeyAll(const FGuid& InGuid) const
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return KeyAll(SequencerPtr,InGuid);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return false;
}

bool UAIESelectionSets::KeyAll(TSharedPtr<ISequencer>& InSequencer, const FGuid& InGuid) const
{
	if (InSequencer.IsValid() == false)
	{
		return false;
	}
	using namespace UE::AIE;
	FControlRigEditMode* EditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	if (EditMode)
	{
		FFrameNumber FrameNumber = InSequencer->GetLocalTime().Time.GetFrame();
		EMovieSceneKeyInterpolation DefaultInterpolation = InSequencer->GetKeyInterpolation();

		const FScopedTransaction Transaction(LOCTEXT("KeySelectionSet_transaction", "Key Selection Set"), !GIsTransacting);

		TArray<FGuid> AllGuids;
		TMap<UControlRig*, TArray<FRigElementKey>> SelectedControls;
		if (const FAIESelectionSetItem* SetItem = SelectionSets.Find(InGuid))
		{
			SetItem->GetAllChildren(this, AllGuids);
			for (const FGuid& Guid : AllGuids)
			{
				if (const FAIESelectionSetItem* CurrentSetItem = SelectionSets.Find(Guid))
				{
					for (const TPair<TWeakObjectPtr<AActor>, FActorWithSelectionSet>& Pair : ActorsWithSelectionSets.ActorsWithSet)
					{
						if (Pair.Value.IsActive(bShowSelectedOnly))
						{
							if (Pair.Value.SetsThatMatch.Contains(Guid))
							{
								if (AActor* Key = Cast<AActor>(Pair.Key.Get()))
								{
									for (const TWeakObjectPtr<UObject>& Object : Pair.Value.Actors)
									{
										if (UControlRig* ControlRig = Cast<UControlRig>(Object.Get()))
										{
											if (UMovieSceneControlRigParameterTrack* CRTrack = GetControlRigTrack(InSequencer, ControlRig))
											{
												TArray<FRigElementKey> ControlKeys;
												for (const FAIESelectionSetItemName& Name : CurrentSetItem->Names)
												{
													FName ControlName(*Name.Name);
													if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(CRTrack->GetSectionToKey(ControlName)))
													{
														CRSection->Modify();
														if (FRigControlElement* Control = ControlRig->FindControl(ControlName))
														{
															ControlKeys.Add(Control->GetKey());
														}
													}
												}
												if (ControlKeys.Num() > 0)
												{
													SelectedControls.Add(ControlRig, ControlKeys);
												}
											}
										}
										else if (AActor* Actor = Cast<AActor>(Object.Get()))
										{
											if (UMovieScene3DTransformTrack* Track = Get3DTransformTrack(InSequencer, Actor))
											{
												if (Track->GetAllSections().Num() > 0)
												{
													UMovieSceneSection* Section = Track->GetSectionToKey() ?
														Track->GetSectionToKey() :
														Track->GetAllSections()[0];
													if (Section)
													{
														Section->Modify();
														MovieSceneToolHelpers::AddTransformKeysForObject(Object.Get(), EMovieSceneTransformChannel::AllTransform);
													}
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}
			if (FControlRigParameterTrackEditor::CurrentEditor && SelectedControls.Num() > 0)
			{
				EMovieSceneTransformChannel Channel = EMovieSceneTransformChannel::AllTransform;
				FControlRigParameterTrackEditor::CurrentEditor->SetKeysOnControls(SelectedControls, Channel);
			}
			InSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::TrackValueChanged);

			return true;
		}
	}
	return false;
}

bool UAIESelectionSets::AddChildToItem(const FGuid& InParentGuid, const FGuid& InChildGuid)
{
	return false;
}
bool UAIESelectionSets::RemoveChildFromItem(const FGuid& InParentGuid, const FGuid& InChildGuid)
{
	return false;
}

void UAIESelectionSets::SelectionSetsChangedBroadcast() 
{
	GetActiveSelectionSets(); //this will sett up the sets
	OnSelectionSetsChanged.Broadcast(this);
}

namespace UE::AIE
{
	FLinearColor GetNextRandomColor(int32& NextIndex, bool bIncrement)
	{
		FLinearColor Color = FLinearColor::Gray;

		if (const USelectionSetsSettings* Settings = GetDefault<USelectionSetsSettings>())
		{
			const TArray<FLinearColor> CustomColors = Settings->GetCustomColors();
			if (CustomColors.Num() > 0)
			{
				if (NextIndex < 0)
				{
					NextIndex = 0;
				}
				else if (NextIndex >= CustomColors.Num())
				{
					NextIndex = CustomColors.Num() - 1;
				}
				Color = CustomColors[NextIndex];
				if (bIncrement)
				{
					++NextIndex;
					NextIndex = (NextIndex % CustomColors.Num());

				}
				else
				{
					--NextIndex;
					if (NextIndex < 0)
					{
						NextIndex = CustomColors.Num() - 1;
					}
				}
			}
		}
		return Color;
	}
}
FLinearColor UAIESelectionSets::GetNextSelectionSetColor()
{
	static int32 NextIndex = 0;
	return UE::AIE::GetNextRandomColor(NextIndex, true);
}

FLinearColor UAIESelectionSets::GetNextActorColor()
{
	static int32 NextIndex = MAX_int32;
	return UE::AIE::GetNextRandomColor(NextIndex, false);
}

bool UAIESelectionSets::LoadFromJsonFile(const FFilePath& JsonFilePath)
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return LoadFromJsonFile(SequencerPtr, JsonFilePath);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return false;
}

bool UAIESelectionSets::LoadFromJsonFile(TSharedPtr<ISequencer>& InSequencer, const FFilePath& JsonFilePath)
{
	FString JsonAsString;

	if (!FFileHelper::LoadFileToString(JsonAsString, *JsonFilePath.FilePath))
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Could not load from file");
		return false;
	}

	return LoadFromJsonString(InSequencer, JsonAsString);
}

bool UAIESelectionSets::ExportAsJsonFile(const FFilePath& JsonFilePath) const
{
	FString JsonString;

	if (ExportAsJsonString(JsonString))
	{
		return FFileHelper::SaveStringToFile(JsonString, *JsonFilePath.FilePath);
	}
	return false;
}

bool UAIESelectionSets::LoadFromJsonString(const FString& JsonString)
{
	if (TSharedPtr<ISequencer> SequencerPtr = GetSequencerFromAsset())
	{
		return LoadFromJsonString(SequencerPtr, JsonString);
	}
	else
	{
		UE_LOGF(LogControlRig, Error, "Selection Sets: Need open Sequencer");
	}
	return false;
}

bool UAIESelectionSets::LoadFromJsonString(TSharedPtr<ISequencer>& InSequencer, const FString& JsonString)
{
	TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
	const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);

	if (!FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid() == false)
	{
		UE_LOGF(LogControlRig, Warning, "Could not deserialize json data for selection sets")
		return false;
	}

	if (!JsonObject->HasTypedField(TEXT("sets"), EJson::Array))
	{
		UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: Missing sets field");
		return false;
	}
	FString ItemName, Guid, Parent, Name, MirrorName, OwnerActorName;
	int32 Type, Duplicates, Row, Column;
	float Red, Green, Blue, Alpha;
	const TArray<TSharedPtr<FJsonValue>>* NamesArray;
	//const TArray<TSharedPtr<FJsonValue>>* ChildArray; //not supported yet
	const FScopedTransaction Transaction(LOCTEXT("ImportSelectionSetJSON_transaction", "Import Selection Set"), !GIsTransacting);
	Modify();
	TArray<TSharedPtr<FJsonValue>> SetsArray = JsonObject->GetArrayField(TEXT("sets"));
	for (TSharedPtr<FJsonValue> SetValue : SetsArray)
	{
		TSharedPtr<FJsonObject> SetObject = SetValue->AsObject();
		if (SetObject.IsValid())
		{
			FAIESelectionSetItem SetItem;
			if (SetObject->TryGetStringField(TEXT("item_name"), ItemName))
			{
				SetItem.ItemName = FText::FromString(ItemName);
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No item_name");
				return false;
			}
			if (SetObject->TryGetStringField(TEXT("guid"), Guid))
			{
				SetItem.Guid = FGuid(Guid);
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No guid");
				return false;
			}
			if (SetObject->TryGetStringField(TEXT("parent"), Parent))
			{
				SetItem.Parent = FGuid(Parent);
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No parent");
				return false;
			}
			if (SetObject->TryGetNumberField(TEXT("row"), Row))
			{
				SetItem.ViewData.Row = Row;
			}
			else //may not exist on older
			{
				SetItem.ViewData.Row = 0;
			}
			if (SetObject->TryGetNumberField(TEXT("column"), Column))
			{
				SetItem.ViewData.Column = Column;
			}
			else //may not exist on older
			{
				SetItem.ViewData.Column = 0;
			}
			if (SetObject->TryGetNumberField(TEXT("red"), Red))
			{
				SetItem.ViewData.Color.R = Red;
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No red");
				return false;
			}
			if (SetObject->TryGetNumberField(TEXT("green"), Green))
			{
				SetItem.ViewData.Color.G = Green;
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No green");
				return false;
			}
			if (SetObject->TryGetNumberField(TEXT("blue"), Blue))
			{
				SetItem.ViewData.Color.B = Blue;
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No blue");
				return false;
			}
			if (SetObject->TryGetNumberField(TEXT("alpha"), Alpha))
			{
				SetItem.ViewData.Color.A = Alpha;
			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No alpha");
				return false;
			}
			if (SetObject->TryGetArrayField(TEXT("names"), NamesArray))
			{
				SetItem.Names.Reserve(NamesArray->Num());
				for (TSharedPtr<FJsonValue> NamesSource : *NamesArray)
				{
					TSharedPtr<FJsonObject> NamesObject = NamesSource->AsObject();
					if (NamesObject.IsValid())
					{
						FAIESelectionSetItemName SetItemName;
						if (NamesObject->TryGetStringField(TEXT("name"), Name))
						{
							SetItemName.Name = Name;
						}
						else
						{
							UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No name");
							return false;
						}
						if (NamesObject->TryGetStringField(TEXT("owner_actor_name"), OwnerActorName))
						{
							SetItemName.OwnerActorName = OwnerActorName;
						}
						else //may not exist
						{
							SetItemName.OwnerActorName = FString("");
						}

						if (NamesObject->TryGetStringField(TEXT("mirror_name"), MirrorName))
						{
							SetItemName.MirrorName = MirrorName;
						}
						else
						{
							UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No mirror name");
							return false;
						}

						if (NamesObject->TryGetNumberField(TEXT("type"), Type))
						{
							SetItemName.Type = Type;
						}
						else
						{
							UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No type");
							return false;
						}
						if (NamesObject->TryGetNumberField(TEXT("duplicates"), Duplicates))
						{
							SetItemName.Duplicates = Duplicates;
						}
						else
						{
							UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No type");
							return false;
						}

						SetItem.Names.Add(SetItemName);

					}
					else
					{
						UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No names Object");
						return false;
					}
				}

			}
			else
			{
				UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No names Array");
				return false;
			}
			SelectionSets.Add(SetItem.Guid, SetItem);

		}
		else
		{
			UE_LOGF(LogControlRig, Warning, "Selection Sets JSON: No set Object");
			return false;
		}

	}
	
	TSet<TWeakObjectPtr<UObject>> NoObjectsAdded;
	ActorsWithSelectionSets.SetUpFromSequencer(InSequencer, this, NoObjectsAdded);

	SelectionSetsChangedBroadcast();	
	return true;
}


bool UAIESelectionSets::ExportAsJsonString(FString& OutJsonString) const
{
	
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutJsonString);

	JsonWriter->WriteObjectStart();
	JsonWriter->WriteArrayStart(TEXT("sets"));
	for (const TPair<FGuid, FAIESelectionSetItem>& Pair : SelectionSets)
	{
;
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("item_name"), Pair.Value.ItemName.ToString());
		JsonWriter->WriteValue(TEXT("guid"), Pair.Value.Guid.ToString());
		JsonWriter->WriteValue(TEXT("parent"), Pair.Value.Parent.ToString());
		JsonWriter->WriteValue(TEXT("row"), Pair.Value.ViewData.Row);
		JsonWriter->WriteValue(TEXT("column"), Pair.Value.ViewData.Column);
		JsonWriter->WriteValue(TEXT("red"), Pair.Value.ViewData.Color.R);
		JsonWriter->WriteValue(TEXT("green"), Pair.Value.ViewData.Color.G);
		JsonWriter->WriteValue(TEXT("blue"), Pair.Value.ViewData.Color.B);
		JsonWriter->WriteValue(TEXT("alpha"), Pair.Value.ViewData.Color.A);

		JsonWriter->WriteArrayStart(TEXT("children"));
		for (const FGuid& ChildGuid : Pair.Value.Children)
		{
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(TEXT("child"), ChildGuid.ToString());
			JsonWriter->WriteObjectEnd();
		}
		JsonWriter->WriteArrayEnd();
		//JsonWriter->WriteValue(TEXT("children"), Pair.Value.Children);
		JsonWriter->WriteArrayStart(TEXT("names"));
		for (const FAIESelectionSetItemName& Name : Pair.Value.Names)
		{
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(TEXT("name"), Name.Name);
			if (Name.OwnerActorName.Len() > 0)
			{
				JsonWriter->WriteValue(TEXT("owner_actor_name"), Name.OwnerActorName);
			}
			JsonWriter->WriteValue(TEXT("mirror_name"), Name.MirrorName);
			JsonWriter->WriteValue(TEXT("type"), (int32)Name.Type);
			JsonWriter->WriteValue(TEXT("duplicates"), Name.Duplicates);
			JsonWriter->WriteObjectEnd();
		}
		JsonWriter->WriteArrayEnd();
		JsonWriter->WriteObjectEnd();

	}
	JsonWriter->WriteArrayEnd();
	JsonWriter->WriteObjectEnd();

	
	JsonWriter->Close();
	return true;

}

#undef LOCTEXT_NAMESPACE


