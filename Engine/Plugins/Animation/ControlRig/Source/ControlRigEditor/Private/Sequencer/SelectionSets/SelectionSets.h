// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/AssetUserData.h"

#include "SelectionSets.generated.h"


class ISequencer;
class ULevelSequence;
class UControlRig;
class UAIESelectionSets;


//Type  of the selection set item
UENUM()
enum EAIESelectionSetItemType : int32
{
	/*Control Rig Type*/
	ControlRig = 0x0,
	/*Actor Type*/
	Actor = 0x1,
//causing mac issues not used yet so removing	Component = 0x2,
};
ENUM_CLASS_FLAGS(EAIESelectionSetItemType)

/* Individual 'Name" within a Seletion Set, maybe Actor Labe or Control Rig Control Name*/
USTRUCT(BlueprintType)
struct FAIESelectionSetItemName
{
	GENERATED_BODY();

	/* The Name of the selection set item*, maybe an Actor Label or a ControlRig Control Name */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	FString Name;

	/* The Name of the selection set mirrored item for Control Rig Controls that have them */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	FString MirrorName;

	/* The EAIESelectionSetItemType, 0 is a Control Rig Control, 1 is an Actor */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	int32 Type = 1;

	//currently not used but maybe in future
	UPROPERTY()
	int32 Duplicates = 0;

	/* The Name of the Owner Actor for Multi-Sets containing multiple rigs, may not be set*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Data")
	FString OwnerActorName;
};

/* View Data for the set, contains location and color in the UI*/
USTRUCT(BlueprintType)
struct FAIESelectionSetItemViewData
{
	GENERATED_BODY();

	/* The Color of the Set*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	FLinearColor Color = FLinearColor::White;

	/* The logical row the set should be in, basically it's order. This is an internal order for just one set*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	int32 Row = 0;
	
	//currently not used but maybe in future
	UPROPERTY()
	int32 Column = 0;
};

/* Main selection set item, has it's name, it's view data, and array of names making up the set*/
USTRUCT(BlueprintType)
struct FAIESelectionSetItem
{
	GENERATED_BODY();

	/* Unique ID*/
	UPROPERTY()
	FGuid Guid;
	
	/* The Name of the Set*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	FText ItemName;

	/* The collection of set items that make up the set, basically an array of names with some metadata*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	TArray<FAIESelectionSetItemName> Names;

	/* Viewing Data for the set, like location and color*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Data")
	FAIESelectionSetItemViewData ViewData;

	//currently not used but maybe in future
	UPROPERTY()
	FGuid Parent;
	//currently not used but maybe in future
	TArray<FGuid> Children;

	bool ContainsItem(const FString& Name) const;
	FAIESelectionSetItemName* GetItem(const FString& Name);
	bool RemoveItem(const FString& Name, const FString& ActorLabel);

	bool IsMultiAsset() const;
	bool ContainsOwnerActor(const FString& ActorLabel) const;
	void GetAllChildren(const UAIESelectionSets* SelectionSets, TArray<FGuid>& AllChildren) const;
	static FString GetMirror(const FString& InString);
};

/*  Internal Struct to specify a set of actors/control rigs and the sets they operate upon*/
struct FActorWithSelectionSet
{
	/*Name, usually the Actor Label*/
	FText Name;

	/*The Color in the UI*/
	FLinearColor Color;

	/*Actors in the level sequence that may be operated on by a Set*/
	TSet<TWeakObjectPtr<UObject>> Actors; 

	/*Set of selection sets that match up with these Actors*/
	TSet<FGuid> SetsThatMatch;

	/** If this Actor group is selected in the viewport*/
	bool IsSelectedInViewport(bool bOnlyInViewport = false) const;

	/** If this Actor group is a Control Rig Actor*/
	bool IsControlRig() const;

	/** Make all Controls visisable on this Control Rig Actor*/
	void ShowAllControlsOnThisActor() const;

	/** Make this Set Actor Selected, may not be Active though based upon UAIESelectionSets::bShowSelectedOnly filter etc..*/
	void SetSelected(bool bInSelected) { bIsSelected = bInSelected; }
	
	/** Get if this Set Actor is Selected. Note may be Selected by not Active for a set based upon UAIESelectionSets::bShowSelectedOnly filter*/
	bool GetSelected() const {return bIsSelected;}

	/** Get if this Set Actor is Active and may be operated upon by a Set Selection, takes into account filters/settings etc*/
	bool IsActive(bool bShowSelectedOnly) const;

	//if the set contains owned actors but they came from a different control rig we mark that. This is done in SetUpFromSequencer
	void SetUpOwnedActors(const TWeakObjectPtr<AActor>& ActorOwner, TSharedPtr<ISequencer>& InSequencer, UAIESelectionSets* SelectionSets);
	TMap <FGuid,FString> OwnedActors;

	// it was previously selected but no more
	mutable bool bWasSelectedInViewport = false;

private:
	bool bIsSelected = false;
public:
	static AActor* GetControlRigActor(const UControlRig* InControlRig);

};

/* Internal struct containing all ofthe actor sets*/
struct FActorsWithSelectionSets
{
	TMap<TWeakObjectPtr<AActor>, FActorWithSelectionSet> ActorsWithSet;
	bool ContainsActor(const FString& ActorLabel, bool bShowSelectedOnly) const;
	void SetUpFromSequencer(TSharedPtr<ISequencer>& InSequencer,  UAIESelectionSets* SelectionSets, const TSet<TWeakObjectPtr<UObject>>& MakeSelectedIfPresent);
};


/** Link To Set of Anim Sequences that we may be linked to.*/
UCLASS(BlueprintType, MinimalAPI)
class UAIESelectionSets : public UAssetUserData
{
	GENERATED_UCLASS_BODY()

public:

	virtual bool IsEditorOnly() const override { return true; }

	UPROPERTY(BlueprintReadOnly, Category = SelectionSet)
	TMap<FGuid, FAIESelectionSetItem> SelectionSets;

	/* Whether we are only showing and setting selections on current selections*/
	UFUNCTION(BlueprintPure, Category = "State")
	bool GetShowAndSetSelectedOnly() const
	{
		return bShowSelectedOnly;
	}

	/* Set whether we are only showing and setting selections on current selections*/
	UFUNCTION(BlueprintCallable, Category = "State")
	void SetShowAndSetSelectedOnly(bool bInShowSelectedOnly);
		
	/* Get the current set of active selection sets based upon active actors. This function will set it up also if needed*/
	UFUNCTION(BlueprintCallable, Category = "State")
	TArray<FGuid> GetActiveSelectionSets();

	/* Set the AActor as Active or not, so it will or won't be selected from a selection set. Will return false if it doesn't match any sets*.
	Note if GetShowAndSetSelectedOnly() is true the selected state of the Actor will take precedence over this*/
	UFUNCTION(BlueprintCallable, Category = "Actor")
	bool SetActorAsActive(AActor* InActor, bool bSetActive);

	/* Get all the actors which are present to be selected by a selection set*/
	UFUNCTION(BlueprintPure, Category = "Actor")
	TArray<AActor*> GetAllActors() const;

	/* Get the actors which are selectable*/
	UFUNCTION(BlueprintPure, Category = "Actor")
	TArray<AActor*> GetActiveActors() const;

	/* Create a selection set from current selection. Will return empty FGuid if no set created*/
	UFUNCTION(BlueprintCallable, Category = "Sets")
	FGuid CreateSetItemFromSelection();
	FGuid CreateSetItemFromSelection(TSharedPtr<ISequencer>& InSequencer);

	/* Create a mirrored selection set from an existing set. Will return empty FGuid if no set created*/
	UFUNCTION(BlueprintCallable, Category = "Sets")
	FGuid CreateMirror(const FGuid& InGuid);
	FGuid CreateMirror(TSharedPtr<ISequencer>& InSequencer, const FGuid& InGuid);

	/* Add selection to specified selection set*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool AddSelectionToSetItem(const FGuid& InGuid);
	bool AddSelectionToSetItem(const FGuid& InGuid, TSharedPtr<ISequencer>& InSequencer);

	/* Remove selection from specified selection set*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool RemoveSelectionFromSetItem(const FGuid& InGuid);
	bool RemoveSelectionFromSetItem(const FGuid& InGuid, TSharedPtr<ISequencer>& InSequencer);

	/* Rename selection set*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool RenameSetItem(const FGuid& InGuid,const  FText& NewName);

	/* Get selection set name*/
	UFUNCTION(BlueprintPure, Category = "Set")
	bool GetItemName(const FGuid& InGuid, FText& OutName) const;

	/* Rename selection set*/
	UFUNCTION(BlueprintPure, Category = "Set")
	bool IsMultiAsset(const FGuid& InGuid) const;

	/* Get the guids of set items associated with this name*/
	UFUNCTION(BlueprintPure, Category = "State")
	void GetItemGuids(const FText& ItemName, TArray<FGuid>& OutGuids) const;

	/* Delete selection set*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool DeleteSetItem(const FGuid& InGuid);
	bool DeleteSetItem(const FGuid& InGuid, TSharedPtr<ISequencer>& InSequencer);

	/* Delete all selection sets*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool DeleteAllSetItems();
	bool DeleteAllSetItems(TSharedPtr<ISequencer>& InSequencer);

	/* Get selection set color*/
	UFUNCTION(BlueprintPure, Category = "Set")
	bool GetItemColor(const FGuid& InGuid, FLinearColor& OutColor) const;

	/* Set selection set color*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool SetItemColor(const FGuid& InGuid, const FLinearColor& InColor);

	/* Get selection set row*/
	UFUNCTION(BlueprintPure, Category = "Set")
	bool GetItemRow(const FGuid& InGuid, int32& OutRow) const;

	/* Set selection set row*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool SetItemRow(const FGuid& InGuid, int32 InRow);

	/* Select from selection set, will select on the active actors*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool SelectItem(const FGuid& InGuid, bool bDoMirror, bool bAdd, bool bToggle,bool bSelect = true) const;

	/* Show or hide controls on the active actors from the specified set*/
	UFUNCTION(BlueprintCallable, Category = "Visibility")
	bool ShowOrHideControls(const FGuid& InGuid, bool bShow, bool bDoMirror) const;

	/* Isolate controls from just this selection set on active actors*/
	UFUNCTION(BlueprintCallable, Category = "Visibility")
	bool IsolateControls(const FGuid& InGuid) const;

	/* Show all controls from this selection set's active actors*/
	UFUNCTION(BlueprintCallable, Category = "Visibility")
	bool ShowAllControls(const FGuid& InGuid) const;

	/* Key the selection sets on active actors*/
	UFUNCTION(BlueprintCallable, Category = "Set")
	bool KeyAll(const FGuid& InGuid) const;
	bool KeyAll(TSharedPtr<ISequencer>& InSequencer, const FGuid& InGuid) const;

	/* Load sets from a JSON file*/
	UFUNCTION(BlueprintCallable, Category = "ImportExport")
	bool LoadFromJsonFile(const FFilePath& JsonFilePath);
	bool LoadFromJsonFile(TSharedPtr<ISequencer>& InSequencer, const FFilePath& JsonFilePath);

	/* Export sets to a JSON file*/
	UFUNCTION(BlueprintCallable, Category = "ImportExport")
	bool ExportAsJsonFile(const FFilePath& JsonFilePath)  const;

	/* Load sets from a string*/
	UFUNCTION(BlueprintCallable, Category = "ImportExport")
	bool LoadFromJsonString(const FString& JsonString);
	bool LoadFromJsonString(TSharedPtr<ISequencer>& InSequencer, const FString& JsonString);

	/* Export sets from a string*/
	UFUNCTION(BlueprintCallable, Category = "ImportExport")
	bool ExportAsJsonString(FString& OutJsonString) const;

	//Get updated visibility for the actor selections, using settings and properties
	bool IsVisible(const FActorWithSelectionSet& ActorWithSelectionSet) const;

	//utility functions
	void SequencerBindingsAdded(TSharedPtr<ISequencer>& InSequencer);
	//tbd
	bool AddChildToItem(const FGuid& InParentGuid, const FGuid& InChildGuid);
	bool RemoveChildFromItem(const FGuid& InParentGuid, const FGuid& InChildGuid);

	static UAIESelectionSets* GetSelectionSets(const TSharedPtr<ISequencer>& InSequencer, bool bAddIfDoesNotExist = false);
	static UAIESelectionSets* GetSelectionSets(ULevelSequence* InLevelSequence, bool bAddIfDoesNotExist = false);
	static 	TSharedPtr<ISequencer> GetSequencerFromAsset();

public:
	//current set of actors we can work on
	FActorsWithSelectionSets  ActorsWithSelectionSets;

	
public:
	DECLARE_EVENT_OneParam(UAIESelectionSets, FSelectionSetsChanged, UAIESelectionSets*);

	FSelectionSetsChanged& SelectionListChanged() { return OnSelectionSetsChanged; };


	void SelectionSetsChangedBroadcast();
	FSelectionSetsChanged OnSelectionSetsChanged;

	FLinearColor GetNextSelectionSetColor();
	FLinearColor GetNextActorColor();


	friend struct FActorsWithSelectionSets;

private:
	TArray<FGuid> SortSelectionSetsByRow(TArray<FGuid>& InSelectionSets);

	//starting from StartRow going to EndRow(inclusive) increase or decrease row values, used for when changing row value or deleting the item
	void UpdateRowValues(int32 StartRow, int32 EndRow, bool bIncrease);
private:
	//to handle selecting last selected object when none are selected
	mutable TSet<FGuid> LastActiveSelectionSetsSet;

	//store it but don't expose it, use ufunctions

	UPROPERTY()
	bool bShowSelectedOnly = true;
};
