// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSetsOverlayManager.h"
#include "SSelectionSets.h"
#include "EditMode/ControlRigEditModeCommands.h"
#include "EditMode/ControlRigEditModeSettings.h"
#include "Misc/Flyout/Host/ToolkitBasedWidgetHost.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/IToolkitHost.h"

namespace UE::AIE
{
namespace Private
{
static void SaveWidgetState(FToolWidget_FlyoutSavedState State)
{
	UControlRigEditModeSettings* ControlRigEditModeSettings = GetMutableDefault<UControlRigEditModeSettings>();
	ControlRigEditModeSettings->LastUIStates.SelectionSetOverlayState = State;
	ControlRigEditModeSettings->SaveConfig();
}
}
	
FSelectionSetsOverlayManager::FSelectionSetsOverlayManager(
	const TSharedRef<IToolkitHost>& InToolkitHost,
	const TSharedRef<FUICommandList>& InToolkitCommandList,
	const TSharedRef<FControlRigEditMode>& InOwningEditMode
	)
	: SelectionSetWidget(SNew(SSelectionSets, *InOwningEditMode)),
	  FlyoutWidgetManager(
		UE::ControlRigEditor::FFlyoutWidgetArgs(
			SelectionSetWidget.ToSharedRef(),
			MakeShared<UE::ControlRigEditor::FToolkitBasedWidgetHost>(InToolkitHost)
			)
			.SetStateToRestoreFrom(GetMutableDefault<UControlRigEditModeSettings>()->LastUIStates.SelectionSetOverlayState)
			.SetSaveStateDelegate(UE::ControlRigEditor::FSaveFlyoutState::CreateStatic(&UE::AIE::Private::SaveWidgetState))
		),
	CommandBinder(FlyoutWidgetManager, 
		ControlRigEditor::FFlyoutCommandArgs(InToolkitCommandList)
		.SetToggleVisibility(FControlRigEditModeCommands::Get().ToggleSelectionSetsWidget)
		.SetSummonToCursor(FControlRigEditModeCommands::Get().SummonSelectionSetsWidget)
		)
{
}


}
