// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditMode/ControlRigBaseDockableView.h"
#include "Misc/Flyout/Command/FlyoutCommandBinder.h"
#include "Misc/Flyout/FlyoutOverlayManager.h"

namespace UE::AIE
{
/**
 * Manages all logic around the selectio set overlay widget.
 
 */
class FSelectionSetsOverlayManager : public FNoncopyable
{
public:
	
	explicit FSelectionSetsOverlayManager(
		const TSharedRef<IToolkitHost>& InToolkitHost,
		const TSharedRef<FUICommandList>& InToolkitCommandList,
		const TSharedRef<FControlRigEditMode>& InOwningEditMode
		);

	void ToggleVisibility() { FlyoutWidgetManager.ToggleVisibility(); }
	void ShowWidget() { FlyoutWidgetManager.ShowWidget(); }
	void HideWidget() { FlyoutWidgetManager.HideWidget(); }
	void DestroyWidget() { FlyoutWidgetManager.DestroyWidget(); }
	bool IsShowingWidget() const { return FlyoutWidgetManager.IsShowingWidget(); }
	

private:
	
	TSharedPtr<SWidget> SelectionSetWidget;

	ControlRigEditor::FFlyoutOverlayManager FlyoutWidgetManager;
	
	/** Process the command bindings that affect FlyoutWidgetManager. */
	ControlRigEditor::FFlyoutCommandBinder CommandBinder;
	
};
}
