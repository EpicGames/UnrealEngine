// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionSetsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SelectionSetsSettings)

USelectionSetsSettings::USelectionSetsSettings()
{
	CategoryName = TEXT("General");
	SectionName = TEXT("Animation Selection Sets");

	if (CustomColors.Num() == 0)
	{
		CustomColors.Add(FLinearColor(.904f, .323f, .539f)); //pastel red
		CustomColors.Add(FLinearColor(.552f, .737f, .328f)); //pastel green
		CustomColors.Add(FLinearColor(.947f, .418f, .219f)); //pastel orange
		CustomColors.Add(FLinearColor(.156f, .624f, .921f)); //pastel blue
		CustomColors.Add(FLinearColor(.921f, .314f, .337f)); //pastel red 2
		CustomColors.Add(FLinearColor(.361f, .651f, .332f)); //pastel green 2
		CustomColors.Add(FLinearColor(.982f, .565f, .254f)); //pastel orange 2
		CustomColors.Add(FLinearColor(.246f, .223f, .514f)); //pastel purple
		CustomColors.Add(FLinearColor(.208f, .386f, .687f)); //pastel blue2
		CustomColors.Add(FLinearColor(.223f, .590f, .337f)); //pastel green 3
		CustomColors.Add(FLinearColor(.230f, .291f, .591f)); //pastel blue 3
	}
}

