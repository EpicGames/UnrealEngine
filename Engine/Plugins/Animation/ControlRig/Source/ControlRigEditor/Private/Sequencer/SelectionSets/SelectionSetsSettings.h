// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Engine/DeveloperSettings.h"
#include "Containers/EnumAsByte.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/Object.h"
#include "Math/Color.h"
#include "Containers/Array.h"

#include "SelectionSetsSettings.generated.h"


#define UE_API CONTROLRIGEDITOR_API


/** Serializable options for animation mode selection sets. */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class USelectionSetsSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()

	UE_API USelectionSetsSettings();

	const TArray<FLinearColor>& GetCustomColors() const { return CustomColors; }

protected:

	/** Set of colors to use for selection sets*/
	UPROPERTY(config, EditAnywhere, Category="Colors")
	TArray<FLinearColor> CustomColors;

private:


};

#undef UE_API
