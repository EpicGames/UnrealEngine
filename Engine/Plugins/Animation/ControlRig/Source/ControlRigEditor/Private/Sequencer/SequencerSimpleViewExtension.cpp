// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerSimpleViewExtension.h"
#include "AnimLayers/SAnimLayerComboBox.h"
#include "ControlRigEditorStyle.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "MovieSceneSequence.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "Sequencer/ControlRigSelectionUtils.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "SequencerToolMenuContext.h"
#include "SimpleView/SequencerSimpleViewExtender.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SequencerSimpleViewExtension"

namespace UE::ControlRig::SimpleView
{

static const FName SimpleViewControlRigToolbarOwner(TEXT("SimpleViewControlRigToolbar"));

static FDelegateHandle ChannelFilterDelegateHandle;
static FDelegateHandle AdditionalSelectedModelsDelegateHandle;

void FSequencerSimpleViewExtension::Register()
{
	AddSequencerToolbarMenuExtension();
	AddSequencerSimpleViewChannelFilters();
	AddAdditionalSelectedViewModels();
}

void FSequencerSimpleViewExtension::Unregister()
{
	RemoveSequencerToolbarMenuExtension();
	RemoveSequencerSimpleViewChannelFilters();
	RemoveAdditionalSelectedViewModels();
}

void FSequencerSimpleViewExtension::AddSequencerToolbarMenuExtension()
{
	if (!FModuleManager::Get().IsModuleLoaded(TEXT("ToolMenus")))
	{
		return;
	}

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(SimpleViewControlRigToolbarOwner);

	UToolMenu* const ExtendedMenu = ToolMenus->ExtendMenu(TEXT("Sequencer.MainToolBar"));
	if (!ExtendedMenu)
	{
		return;
	}

	FToolMenuSection& Section = ExtendedMenu->FindOrAddSection(TEXT("SimpleView")
		, LOCTEXT("SimpleViewSection", "Simple View")
		, FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));

	FToolMenuEntry& AnimLayersEntry = Section.AddDynamicEntry(TEXT("AnimLayers"), FNewToolMenuSectionDelegate::CreateStatic(&AddAnimLayersComboBox));
	AnimLayersEntry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
	AnimLayersEntry.WidgetData.bNoPadding = true;
}

void FSequencerSimpleViewExtension::RemoveSequencerToolbarMenuExtension()
{
	if (!FModuleManager::Get().IsModuleLoaded(TEXT("ToolMenus")))
	{
		return;
	}

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	ToolMenus->UnregisterOwnerByName(SimpleViewControlRigToolbarOwner);
}

void FSequencerSimpleViewExtension::AddAnimLayersComboBox(FToolMenuSection& InSection)
{
	const USequencerToolMenuContext* const Context = InSection.FindContext<USequencerToolMenuContext>();
	if (!Context)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Context->WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TWeakPtr<ISequencer> WeakSequencer = Context->WeakSequencer;

	if (!SupportsAnimLayerComboBox(WeakSequencer))
	{
		return;
	}

	const TSharedRef<SAnimLayerComboBox> AnimLayersComboBox = SNew(SAnimLayerComboBox, WeakSequencer)
		.Visibility(TAttribute<EVisibility>::CreateStatic(&GetAnimLayerComboBoxVisibility, WeakSequencer));

	const FText AnimLayersLabel = LOCTEXT("AnimLayersLabel", "Anim Layers");
	FToolMenuEntry AnimLayersEntry = FToolMenuEntry::InitWidget(TEXT("AnimLayerComboBox")
		, AnimLayersComboBox, AnimLayersLabel);
	AnimLayersEntry.StyleNameOverride = TEXT("SequencerToolbar");
	AnimLayersEntry.ToolTip = AnimLayersLabel;
	AnimLayersEntry.Icon = FSlateIcon(FControlRigEditorStyle::Get().GetStyleSetName(), TEXT("ControlRig.AnimLayers"));
	AnimLayersEntry.Visibility = TAttribute<EVisibility>::CreateStatic(&GetAnimLayerComboBoxVisibility, WeakSequencer);
	AnimLayersEntry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
	AnimLayersEntry.WidgetData.bNoPadding = true;
	InSection.AddEntry(AnimLayersEntry);
}

bool FSequencerSimpleViewExtension::SupportsAnimLayerComboBox(const TWeakPtr<ISequencer> InWeakSequencer)
{
	const TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	const UMovieSceneSequence* const RootSequence = Sequencer->GetRootMovieSceneSequence();
	return RootSequence && RootSequence->IsA<ULevelSequence>();
}

EVisibility FSequencerSimpleViewExtension::GetAnimLayerComboBoxVisibility(const TWeakPtr<ISequencer> InWeakSequencer)
{
	if (!SupportsAnimLayerComboBox(InWeakSequencer))
	{
		return EVisibility::Collapsed;
	}

	const auto Sequencer = InWeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return EVisibility::Collapsed;
	}

	return Sequencer->IsSimpleView() ? EVisibility::Visible : EVisibility::Collapsed;
}

void FSequencerSimpleViewExtension::AddSequencerSimpleViewChannelFilters()
{
	ChannelFilterDelegateHandle = Sequencer::SimpleView::FSequencerSimpleViewExtender::RegisterChannelFilter(
		[](ISequencer& InSequencer, const Sequencer::FChannelModel& InChannelModel) -> bool
		{
			const UMovieSceneSection* const ChannelSection = InChannelModel.GetSection();
			if (!ChannelSection)
			{
				return false;
			}

			const UAnimLayers* const AnimLayers = UAnimLayers::GetAnimLayers(&InSequencer);
			if (!AnimLayers)
			{
				return true;
			}

			TSet<const UMovieSceneSection*> SelectedLayerSections;
			bool bHasAnySelectedLayer = false;

			for (UAnimLayer* const AnimLayer : AnimLayers->AnimLayers)
			{
				if (!AnimLayer || !AnimLayer->GetSelectedInList())
				{
					continue;
				}

				bHasAnySelectedLayer = true;

				TArray<UMovieSceneSection*> LayerSections;
				AnimLayer->GetSections(LayerSections);

				for (UMovieSceneSection* const Section : LayerSections)
				{
					if (Section)
					{
						SelectedLayerSections.Add(Section);
					}
				}
			}

			// No layer selected in the Anim Layers list => no anim-layer filtering
			if (!bHasAnySelectedLayer)
			{
				return true;
			}

			return SelectedLayerSections.Contains(ChannelSection);
		});
}

void FSequencerSimpleViewExtension::RemoveSequencerSimpleViewChannelFilters()
{
	Sequencer::SimpleView::FSequencerSimpleViewExtender::UnregisterChannelFilter(ChannelFilterDelegateHandle);
}

static void EnumerateChannelModelsForCurrentControlSelection(const ISequencer& InSequencer
	, TFunctionRef<bool(const Sequencer::FViewModelPtr&)> InCallback)
{
	using namespace UE::Sequencer;

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
	if (!ViewModel.IsValid())
	{
		return;
	}

	const TViewModelPtr<FViewModel> RootModel = ViewModel->GetRootModel();
	if (!RootModel.IsValid())
	{
		return;
	}

	TMap<const UControlRig*, TSet<FName>> SelectedControlsByRig;

	for (const TViewModelPtr<FChannelModel>& ChannelModel : RootModel->GetDescendantsOfType<FChannelModel>())
	{
		if (!ChannelModel.IsValid() || !ChannelModel->GetKeyArea().IsValid())
		{
			continue;
		}

		const UMovieSceneControlRigParameterSection* const ControlRigSection = Cast<UMovieSceneControlRigParameterSection>(ChannelModel->GetSection());
		if (!ControlRigSection)
		{
			continue;
		}

		const UControlRig* const ControlRig = ControlRigSection->GetControlRig();
		if (!ControlRig)
		{
			continue;
		}

		const TSet<FName>* SelectedControls = SelectedControlsByRig.Find(ControlRig);
		if (!SelectedControls)
		{
			TSet<FName> ControlNames;

			for (const FName ControlName : ControlRig->CurrentControlSelection())
			{
				ControlNames.Add(ControlName);
			}

			SelectedControls = &SelectedControlsByRig.Add(ControlRig, MoveTemp(ControlNames));
		}

		if (SelectedControls->IsEmpty())
		{
			continue;
		}

		const MovieScene::FControlRigChannelMetaData ChannelMetaData = ControlRigSection->GetChannelMetaData(ChannelModel->GetChannel());
		if (!ChannelMetaData || !SelectedControls->Contains(ChannelMetaData.GetControlName()))
		{
			continue;
		}

		if (!InCallback(ChannelModel))
		{
			return;
		}
	}
}

void FSequencerSimpleViewExtension::AddAdditionalSelectedViewModels()
{
	AdditionalSelectedModelsDelegateHandle = Sequencer::SimpleView::FSequencerSimpleViewExtender::RegisterAdditionalSelectedModels(
		FOnEnumerateAdditionalSimpleViewSelectedModels::FDelegate::CreateLambda(
		[](const ISequencer& InSequencer, TFunctionRef<bool(const Sequencer::FViewModelPtr&)> InCallback)
		{
			// E.g. if "Dragon.ChestCtrl.Location.X" is selected, then also show the other child properties: Location, Rotation, etc.
			EnumerateParentControlChildren(InSequencer, InCallback);

			// Also keep channels for viewport-selected rig controls visible even if actor selection sync replaces the outliner selection.
			EnumerateChannelModelsForCurrentControlSelection(InSequencer, InCallback);
		}));
}

void FSequencerSimpleViewExtension::RemoveAdditionalSelectedViewModels()
{
	Sequencer::SimpleView::FSequencerSimpleViewExtender::UnregisterAdditionalSelectedModels(AdditionalSelectedModelsDelegateHandle);
}

} // UE::ControlRig::SimpleView

#undef LOCTEXT_NAMESPACE
