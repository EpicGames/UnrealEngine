// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"

class ISequencer;
struct FToolMenuSection;

namespace UE::ControlRig::SimpleView
{

/**  */
class FSequencerSimpleViewExtension
{
public:
	static void Register();
	static void Unregister();

private:
	static void AddSequencerToolbarMenuExtension();
	static void RemoveSequencerToolbarMenuExtension();

	static void AddAnimLayersComboBox(FToolMenuSection& InSection);

	static bool SupportsAnimLayerComboBox(const TWeakPtr<ISequencer> InWeakSequencer);
	static EVisibility GetAnimLayerComboBoxVisibility(const TWeakPtr<ISequencer> InWeakSequencer);

	static void AddSequencerSimpleViewChannelFilters();
	static void RemoveSequencerSimpleViewChannelFilters();

	static void AddAdditionalSelectedViewModels();
	static void RemoveAdditionalSelectedViewModels();
};

} // namespace UE::ControlRig::SimpleView
