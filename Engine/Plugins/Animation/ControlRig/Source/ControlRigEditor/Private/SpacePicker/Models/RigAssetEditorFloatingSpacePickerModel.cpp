// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigAssetEditorFloatingSpacePickerModel.h"

#include "ControlRig.h"
#include "ControlRigEditorAsset.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "SControlRigDismissDependencyDialog.h"
#include "ScopedTransaction.h"
#include "SpacePicker/Models/RigSpacePickerItem.h"

#define LOCTEXT_NAMESPACE "RigAssetEditorSpacePickerModel"

namespace UE::ControlRigEditor
{
	namespace RigAssetEditorSpacePickerModelDetails
	{
		struct FGuardAbsoluteTime
			: FNoncopyable
		{
			explicit FGuardAbsoluteTime(UControlRig& InControlRig UE_LIFETIMEBOUND) 
				: ControlRig(InControlRig)
				, AbsoluteTime(ControlRig.GetAbsoluteTime()) 
			{
			}

			~FGuardAbsoluteTime() 
			{ 
				ControlRig.SetAbsoluteTime(AbsoluteTime);
			}

		private:
			UControlRig& ControlRig;
			const float AbsoluteTime;
		};
	}
	
	void FRigAssetEditorFloatingSpacePickerModel::SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		UControlRig* ControlRig = GetControlRig();
		if (!Hierarchy ||
			!ControlRig)
		{
			return;
		}		

		const FScopedTransaction SetActiveSpaceTransaction(LOCTEXT("SetActiveSpaceTransaction", "Set Active Space"));
		ModifyAsset(Hierarchy);

		// Technically we should never be able to set invalid spaces here, 
		// to be safe still test with dependency provider and warn in case anything goes wrong
		FRigDependenciesProviderForControlRig DependenciesProvider(ControlRig);
		DependenciesProvider.SetInteractiveDialogEnabled(true);
		const FControlRigDismissDependencyDialogGuard DependencyDialogGuard(Hierarchy);

		FString OutFailureReason;
		if (ControlRig->IsAdditive())
		{
			for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
			{
				const FTransform Transform = ControlRig->GetControlGlobalTransform(Binding.ControlKey.Name);

				constexpr bool bInitial = false;
				constexpr bool bAffectChildren = true;
				if (Hierarchy->SwitchToParent(Binding.ControlKey, Binding.SpaceKey, bInitial, bAffectChildren))
				{
					{
						const RigAssetEditorSpacePickerModelDetails::FGuardAbsoluteTime Guard(*ControlRig);
						ControlRig->Evaluate_AnyThread();
					}

					const FRigControlValue ControlValue = ControlRig->GetControlValueFromGlobalTransform(Binding.ControlKey.Name, Transform, ERigTransformType::CurrentGlobal);
					ControlRig->SetControlValue(Binding.ControlKey.Name, ControlValue);
					{
						const RigAssetEditorSpacePickerModelDetails::FGuardAbsoluteTime Guard(*ControlRig);
						ControlRig->Evaluate_AnyThread();
					}
				}
				else
				{
					if (URigHierarchyController* Controller = Hierarchy->GetController())
					{
						static constexpr TCHAR MessageFormat[] = TEXT("Could not switch %s to parent %s: %s");
						Controller->ReportAndNotifyErrorf(MessageFormat,
							*Binding.ControlKey.Name.ToString(),
							*Binding.SpaceKey.Name.ToString(),
							*OutFailureReason);
					}
				}
			}
		}
		else
		{
			for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
			{
				constexpr bool bInitial = false;
				constexpr bool bAffectChildren = true;

				const FTransform Transform = Hierarchy->GetGlobalTransform(Binding.ControlKey);

				if (Hierarchy->SwitchToParent(Binding.ControlKey, Binding.SpaceKey, bInitial, bAffectChildren, DependenciesProvider, &OutFailureReason))
				{
					Hierarchy->SetGlobalTransform(Binding.ControlKey, Transform);
				}
				else
				{
					if (URigHierarchyController* Controller = Hierarchy->GetController())
					{
						static constexpr TCHAR MessageFormat[] = TEXT("Could not switch %s to parent %s: %s");
						Controller->ReportAndNotifyErrorf(MessageFormat,
							*Binding.ControlKey.Name.ToString(),
							*Binding.SpaceKey.Name.ToString(),
							*OutFailureReason);
					}
				}
			}
		}
	}

	bool FRigAssetEditorFloatingSpacePickerModel::CanAddSpace() const
	{
		return true;
	}

	void FRigAssetEditorFloatingSpacePickerModel::AddSpace(const FRigElementKeyWithLabel& SpaceKeyWithLabel)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		if (!Hierarchy)
		{
			return;
		}

		const FScopedTransaction AddSpaceTransaction(LOCTEXT("AddSpaceTransaction", "Add Spaces to Controls"));
		ModifyAsset(Hierarchy);

		for (const FRigElementKey& ControlKey : GetControlKeys())
		{
			// Don't add the default parent anew if we already have it
			if (Hierarchy->GetDefaultParent(ControlKey) == SpaceKeyWithLabel)
			{
				const TSharedPtr<FRigSpacePickerItem>* DefaultItemPtr = GetItems().FindByPredicate(
					[](const TSharedPtr<FRigSpacePickerItem>& Item)
					{
						return
							Item.IsValid() &&
							!Item->GetBindings().IsEmpty() &&
							Item->GetBindings()[0].SpaceKey == URigHierarchy::GetDefaultParentKey();
					});
				if (DefaultItemPtr)
				{
					// Bring to notice the default item is already added by flashing it
					constexpr float FlashDuration = 2.5f;
					(*DefaultItemPtr)->Flash(FlashDuration);
				}
			}
			else
			{
				EditAvailableSpaces(Hierarchy, ControlKey,
					[&SpaceKeyWithLabel](TArray<FRigElementKeyWithLabel>& AvailableSpaces)
					{
						AvailableSpaces.AddUnique(SpaceKeyWithLabel);
					});
			}
		}
	}

	bool FRigAssetEditorFloatingSpacePickerModel::CanDeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) const
	{
		return !Item->IsDefaultSpace();
	}

	void FRigAssetEditorFloatingSpacePickerModel::DeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		if (!Hierarchy)
		{
			return;
		}

		const FScopedTransaction DeleteSpacesTransaction(LOCTEXT("DeleteSpacesTransaction", "Delete Spaces"));
		ModifyAsset(Hierarchy);

		for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
		{
			EditAvailableSpaces(Hierarchy, Binding.ControlKey,
				[&Binding](TArray<FRigElementKeyWithLabel>& AvailableSpaces)
				{
					AvailableSpaces.RemoveAll([&Binding](const FRigElementKeyWithLabel& ElementKeyWithLabel)
						{
							return ElementKeyWithLabel.Key == Binding.SpaceKey;
						});
				});
		}
	}

	bool FRigAssetEditorFloatingSpacePickerModel::CanMoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const
	{
		return CanMoveSpacesImpl(Item, Direction);
	}

	void FRigAssetEditorFloatingSpacePickerModel::MoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		if (!Hierarchy)
		{
			return;		
		}

		const FScopedTransaction MoveSpacesTransaction(LOCTEXT("MoveSpacesTransaction", "Move Spaces"));
		ModifyAsset(Hierarchy);

		const int32 IndexDelta = Direction == ERigSpacePickerMoveSpaceDirection::Down ? 1 : -1;
		for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
		{
			EditAvailableSpaces(Hierarchy, Binding.ControlKey,
				[&Binding, IndexDelta](TArray<FRigElementKeyWithLabel>& AvailableSpaces)
				{
					const int32 CurrentIndex = AvailableSpaces.IndexOfByPredicate([&Binding](const FRigElementKeyWithLabel& ElementKeyWithLabel)
						{
							return ElementKeyWithLabel.Key == Binding.SpaceKey;
						});

					if (AvailableSpaces.IsValidIndex(CurrentIndex) &&
						AvailableSpaces.IsValidIndex(CurrentIndex + IndexDelta))
					{
						AvailableSpaces.Swap(CurrentIndex, CurrentIndex + IndexDelta);
					}
				});
		}
	}

	void FRigAssetEditorFloatingSpacePickerModel::ModifyAsset(URigHierarchy* Hierarchy)
	{
		UControlRig* ControlRig = GetControlRig();
		if (!Hierarchy ||
			!ControlRig)
		{
			return;
		}

		FControlRigAssetInterfacePtr EditorAsset = ControlRig->GetAssetReference().GetEditorAsset();
		URigHierarchy* AssetHierarchy = EditorAsset ? EditorAsset->GetHierarchy() : nullptr;
		if (!AssetHierarchy)
		{
			return;
		}

		AssetHierarchy->Modify();

		// Also update the debugged instance
		if (AssetHierarchy != Hierarchy)
		{
			Hierarchy->Modify();
		}
	}

	void FRigAssetEditorFloatingSpacePickerModel::EditAvailableSpaces(
		URigHierarchy* Hierarchy, 
		const FRigElementKey& ControlKey, 
		TFunctionRef<void(TArray<FRigElementKeyWithLabel>&)> Function)
	{
		UControlRig* ControlRig = GetControlRig();
		if (!Hierarchy ||
			!ControlRig)
		{
			return;
		}

		FControlRigAssetInterfacePtr EditorAsset = ControlRig->GetAssetReference().GetEditorAsset();
		URigHierarchy* AssetHierarchy = EditorAsset ? EditorAsset->GetHierarchy() : nullptr;
		if (!AssetHierarchy)
		{
			return;
		}

		// Update the settings in the control element
		if (FRigControlElement* ControlElement = AssetHierarchy->Find<FRigControlElement>(ControlKey))
		{
			Function(ControlElement->Settings.Customization.AvailableSpaces);
			AssetHierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);
		}

		// Also update the debugged instance
		if (AssetHierarchy != Hierarchy)
		{
			if (FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(ControlKey))
			{
				Function(ControlElement->Settings.Customization.AvailableSpaces);
				Hierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);
			}
		}
	}

	UControlRig* FRigAssetEditorFloatingSpacePickerModel::GetControlRig() const
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;	

		return ControlRig;
	}
}

#undef LOCTEXT_NAMESPACE
