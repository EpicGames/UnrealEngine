// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigSpacePickerModelBase.h"

namespace UE::ControlRigEditor
{
	/** Minimal model for the floating space picker working in the asset editor */
	class FRigAssetEditorFloatingSpacePickerModel
		: public FRigSpacePickerModelBase
		, public IRigSpacePickerSetActiveSpacesInterface
		, public IRigSpacePickerAddSpacesInterface
		, public IRigSpacePickerDeleteSpacesInterface
		, public IRigSpacePickerMoveSpacesInterface
	{
	protected:
		//~ Begin FRigSpacePickerModelBase interface
		virtual IRigSpacePickerAddSpacesInterface* GetAddSpacesInterface() override { return static_cast<IRigSpacePickerAddSpacesInterface*>(this); }
		virtual IRigSpacePickerSetActiveSpacesInterface* GetSetActiveSpacesInterface() override { return static_cast<IRigSpacePickerSetActiveSpacesInterface*>(this); }
		virtual IRigSpacePickerDeleteSpacesInterface* GetDeleteSpacesInterface() override { return static_cast<IRigSpacePickerDeleteSpacesInterface*>(this); }
		virtual IRigSpacePickerMoveSpacesInterface* GetMoveSpacesInterface() override { return static_cast<IRigSpacePickerMoveSpacesInterface*>(this); }
		//~ End FRigSpacePickerModelBase interface

		//~ Begin IRigSpacePickerSetActiveSpacesInterface interface
		virtual void SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item) override;
		//~ End IRigSpacePickerSetActiveSpacesInterface interface

		//~ Begin IRigSpacePickerAddSpacesInterface interface
		virtual bool CanAddSpace() const override;
		virtual void AddSpace(const FRigElementKeyWithLabel& SpaceKeyWithLabel) override;
		//~ End IRigSpacePickerAddSpacesInterface interface

		//~ Begin IRigSpacePickerDeleteSpacesInterface interface
		virtual bool CanDeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) const override;
		virtual void DeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) override;
		//~ End IRigSpacePickerDeleteSpacesInterface interface
		
		//~ Begin IRigSpacePickerMoveSpacesInterface interface
		virtual bool CanMoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const override;
		virtual void MoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) override;
		//~ End IRigSpacePickerMoveSpacesInterface interface

	private:
		/** Modifies the asset */
		void ModifyAsset(URigHierarchy* Hierarchy);

		/** Edits the available spaces of a control by function */
		void EditAvailableSpaces(URigHierarchy* Hierarchy, const FRigElementKey& ControlKey, TFunctionRef<void(TArray<FRigElementKeyWithLabel>&)> Function);

		/** Returns the control rig */
		UControlRig* GetControlRig() const;
	};
}
