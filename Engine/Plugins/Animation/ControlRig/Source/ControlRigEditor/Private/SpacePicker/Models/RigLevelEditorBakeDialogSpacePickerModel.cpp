// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLevelEditorBakeDialogSpacePickerModel.h"

#include "ControlRig.h"
#include "ControlRigSpaceChannelEditors.h"
#include "Rigs/RigHierarchy.h"
#include "RigSpacePickerBakeSettings.h"
#include "ScopedTransaction.h"
#include "SpacePicker/Models/RigSpacePickerItem.h"

namespace UE::ControlRigEditor
{
	FRigLevelEditorBakeDialogSpacePickerModel::FRigLevelEditorBakeDialogSpacePickerModel(const FRigSpacePickerBakeSettings& InSettings) 
	{
		SettingsStructOnScope = MakeShared<TStructOnScope<FRigSpacePickerBakeSettings>>();
		SettingsStructOnScope->InitializeAs<FRigSpacePickerBakeSettings>();

		// Copy initial settings
		*SettingsStructOnScope = InSettings;
	}

	const FRigSpacePickerBakeSettings* FRigLevelEditorBakeDialogSpacePickerModel::GetSettings() const
	{
		return SettingsStructOnScope.IsValid() ?
			SettingsStructOnScope->Get() :
			nullptr;
	}

	bool FRigLevelEditorBakeDialogSpacePickerModel::CanPerformBake() const
	{
		return GetItems().ContainsByPredicate(
			[](const TSharedPtr<FRigSpacePickerItem>& Item)
			{
				return
					Item.IsValid() &&
					Item->GetActiveState() == ERigSpacePickerItemActiveState::FullyActive;
			});
	}

	void FRigLevelEditorBakeDialogSpacePickerModel::PerformBake()
	{
		const TSharedPtr<FRigSpacePickerItem>* ActiveItemPtr = GetItems().FindByPredicate(
			[](const TSharedPtr<FRigSpacePickerItem>& Item)
			{
				return
					Item.IsValid() &&
					Item->GetActiveState() == ERigSpacePickerItemActiveState::FullyActive;
			});

		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		FRigSpacePickerBakeSettings* SettingsPtr = SettingsStructOnScope.IsValid() ? SettingsStructOnScope->Get() : nullptr;
		if (!Sequencer.IsValid() ||
			!ActiveItemPtr ||
			!SettingsPtr)
		{
			return;
		}

		const FScopedTransaction Transaction(NSLOCTEXT("FRigLevelEditorBakeDialogSpacePickerModel", "BakeControlsToSpace", "Bake Controls To Space"));
			
		for (const FRigSpacePickerControlToSpaceBinding& Binding : (*ActiveItemPtr)->GetBindings())
		{
			URigHierarchy* Hierarchy = Binding.GetHierarchy();
			UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
			if (!Hierarchy ||
				!ControlRig)
			{
				continue;
			}

			// Since setting only supports one space, update settings per binding 
			SettingsPtr->TargetSpace = Binding.SpaceKey;

			// When baking we will now create a channel if one doesn't exist, was causing confusion
			constexpr bool bCreateIfNeeded = true;
			const FSpaceChannelAndSection ChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, Binding.ControlKey.Name, Sequencer.Get(), bCreateIfNeeded);
			if (ChannelAndSection.SpaceChannel)
			{
				FControlRigSpaceChannelHelpers::SequencerBakeControlInSpace(
					ControlRig,
					Sequencer.Get(),
					ChannelAndSection.SpaceChannel,
					ChannelAndSection.SectionToKey,
					Hierarchy,
					Binding.ControlKey,
					*SettingsPtr);
			}
		}
	}

	void FRigLevelEditorBakeDialogSpacePickerModel::SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item)
	{		
		// Don't write to the rig, instead hold a temporary state in items
		Item->OverrideActiveState(ERigSpacePickerItemActiveState::FullyActive);

		for (const TSharedPtr<FRigSpacePickerItem>& OtherItem : GetItems())
		{
			if (OtherItem != Item)
			{
				OtherItem->OverrideActiveState(ERigSpacePickerItemActiveState::Inactive);
			}
		}
	}
}
