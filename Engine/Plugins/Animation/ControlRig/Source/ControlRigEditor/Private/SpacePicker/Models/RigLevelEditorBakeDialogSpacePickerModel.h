// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigLevelEditorSpacePickerModel.h"

struct FRigSpacePickerBakeSettings;
template<typename T> class TStructOnScope;

namespace UE::ControlRigEditor
{
	/**
	 * The bake dialog shows a space picker similar to other level editor focused space pickers. 
	 * Hence a model for the space picker in the bake spaces dialog. 
	 */
	class FRigLevelEditorBakeDialogSpacePickerModel
		: public FRigLevelEditorSpacePickerModel
	{
	public:
		FRigLevelEditorBakeDialogSpacePickerModel(const FRigSpacePickerBakeSettings& InSettings);

		/** Returns the settings struct so it can be displayed in a details view */
		const FRigSpacePickerBakeSettings* GetSettings() const;

		/** Returns the settings struct so it can be displayed in a details view */
		const TSharedPtr<TStructOnScope<FRigSpacePickerBakeSettings>>& GetSettingsStructOnScope() const { return SettingsStructOnScope; }

		/** Returns true if baking can be performed */
		bool CanPerformBake() const;

		/** Performs baking of the currently active item */
		void PerformBake();

	protected:
		//~ Begin FRigSpacePickerModelBase interface
		virtual TArray<FRigElementKeyWithLabel> GatherAdditionalSpaces(URigHierarchy* Hierarchy, const FRigElementKey& ControlKey) const { return {}; }

		virtual IRigSpacePickerDeleteSpacesInterface* GetDeleteSpacesInterface() override { return nullptr; }					// Disabled
		virtual IRigSpacePickerMoveSpacesInterface* GetMoveSpacesInterface() override { return nullptr; }						// Disabled
		virtual IRigSpacePickerCompensateKeysInterface* GetCompensateKeysInterface() override { return nullptr; }				// Disabled
		virtual IRigSpacePickerSupportsBakeDialogInterface* GetSupportsBakeDialogInterface() override { return nullptr; }		// Disabled, we're the dialog
		//~ End FRigSpacePickerModelBase interface

		//~ Begin IRigSpacePickerSetActiveSpacesInterface interface
		virtual void SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item) override;
		//~ End IRigSpacePickerSetActiveSpacesInterface interface

	private:
		/** The settings when baking */		
		TSharedPtr<TStructOnScope<FRigSpacePickerBakeSettings>> SettingsStructOnScope;
	};
}
