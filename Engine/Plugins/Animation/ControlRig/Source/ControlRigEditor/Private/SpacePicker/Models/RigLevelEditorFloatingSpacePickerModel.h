// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigLevelEditorSpacePickerModel.h"

class FControlRigEditMode;
class ISequencer;
struct FRigControlElement;

namespace UE::ControlRigEditor
{
	/** Model for the floating space picker in the level editor. */
	class FRigLevelEditorFloatingSpacePickerModel
		: public FRigLevelEditorSpacePickerModel
	{
	protected:
		//~ Begin FRigSpacePickerModelBase interface
		virtual IRigSpacePickerAddSpacesInterface* GetAddSpacesInterface() override { return nullptr; }						// Disabled
		virtual IRigSpacePickerDeleteSpacesInterface* GetDeleteSpacesInterface() override { return nullptr; }				// Disabled
		virtual IRigSpacePickerMoveSpacesInterface* GetMoveSpacesInterface() override { return nullptr; }					// Disabled
		virtual IRigSpacePickerCompensateKeysInterface* GetCompensateKeysInterface() override { return nullptr; }			// Disabled
		virtual IRigSpacePickerSupportsBakeDialogInterface* GetSupportsBakeDialogInterface() override { return nullptr; }	// Disabled
		//~ End FRigSpacePickerModelBase interface
	};
}
