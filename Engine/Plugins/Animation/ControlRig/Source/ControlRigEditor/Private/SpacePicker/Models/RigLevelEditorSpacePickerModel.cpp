// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLevelEditorSpacePickerModel.h"

#include "ControlRig.h"
#include "ControlRigSpaceChannelEditors.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditorModeManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/FrameTime.h"
#include "Rigs/RigDependency.h"
#include "Rigs/RigHierarchy.h"
#include "SControlRigDismissDependencyDialog.h"
#include "ScopedTransaction.h"
#include "SpacePicker/Models/RigSpacePickerItem.h"
#include "SpacePicker/Views/SRigSpacePickerBakeDialog.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "RigLevelEditorSpacePickerModel"

namespace UE::ControlRigEditor
{
	TSharedPtr<ISequencer> FRigLevelEditorSpacePickerModel::GetSequencer() const
	{
		FControlRigEditMode* EditMode = GetEditMode();
		return EditMode ? EditMode->GetWeakSequencer().Pin() : nullptr;
	}

	void FRigLevelEditorSpacePickerModel::SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		if (!Hierarchy ||
			!ControlRig ||
			!Sequencer.IsValid())
		{
			return;
		}

		// Technically we should never be able to set invalid spaces here, 
		// to be safe still test with dependency provider and warn in case anything goes wrong
		FString FailureReason;
		FRigDependenciesProviderForControlRig DependencyProvider(ControlRig);
		DependencyProvider.SetInteractiveDialogEnabled(true);
		FControlRigDismissDependencyDialogGuard DependencyDialogGuard(Hierarchy);

		for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
		{
			if (!Hierarchy->CanSwitchToParent(Binding.ControlKey, Binding.SpaceKey, DependencyProvider, &FailureReason))
			{
				FNotificationInfo Info(FText::FromString(FailureReason));
				Info.bFireAndForget = true;
				Info.FadeOutDuration = 2.0f;
				Info.ExpireDuration = 8.0f;

				const TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
				if (NotificationItem.IsValid())
				{
					NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
				}
				return;
			}
		}

		const FScopedTransaction SetActiveSpaceTransaction(LOCTEXT("SetActiveSpaceTransaction", "Key Control Rig Space"));
		Hierarchy->Modify();

		// FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl creates new tracks and selects it in the rig.
		// Changing the rig selection here is not desired whatsoever, hence restore what was previously selected.
		struct FScopedRestoreControlSelection : FNoncopyable
		{
			FScopedRestoreControlSelection(UControlRig& InControlRig, const URigHierarchy& InHierarchy)
				: ControlRig(InControlRig)
			{
				SelectedControls = InHierarchy.GetSelectedKeys(ERigElementType::Control);
			}

			~FScopedRestoreControlSelection()
			{			
				ControlRig.ClearControlSelection();
				for (const FRigElementKey& SelectedControl : SelectedControls)
				{
					ControlRig.SelectControl(SelectedControl.Name, true);
				}
			}

		private:
			TArray<FRigElementKey> SelectedControls;
			UControlRig& ControlRig;
		};
		const FScopedRestoreControlSelection ScopedRestoreControlSelection(*ControlRig, *Hierarchy);

		// Perform actual space switching
		for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
		{
			if (const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Binding.ControlKey))
			{
				constexpr bool bCreateIfNeeded = true;
				FSpaceChannelAndSection SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, Binding.ControlKey.Name, Sequencer.Get(), !bCreateIfNeeded);
				if (!SpaceChannelAndSection.SpaceChannel)
				{
					if (Hierarchy->GetActiveParent(Binding.ControlKey) == Binding.SpaceKey)
					{
						continue;
					}
				}

				SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, Binding.ControlKey.Name, Sequencer.Get(), bCreateIfNeeded);
				if (SpaceChannelAndSection.SectionToKey &&
					SpaceChannelAndSection.SpaceChannel)
				{					
					SpaceChannelAndSection.SectionToKey->Modify();

					const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
					const FFrameTime FrameTime = Sequencer->GetLocalTime().ConvertTo(TickResolution);
					const FFrameNumber CurrentTime = FrameTime.GetFrame();
					FControlRigSpaceChannelHelpers::SequencerKeyControlRigSpaceChannel(
						ControlRig,
						Sequencer.Get(),
						SpaceChannelAndSection.SpaceChannel,
						SpaceChannelAndSection.SectionToKey,
						CurrentTime,
						Hierarchy,
						Binding.ControlKey,
						Binding.SpaceKey);
				}
			}
		}
	}

	void FRigLevelEditorSpacePickerModel::NotifyControlSettingChanged(URigHierarchy* Hierarchy, const FRigControlElement* ControlElement) const
	{
		if (Hierarchy &&
			ControlElement)
		{
			if (FControlRigEditMode* EditMode = GetEditMode())
			{
				EditMode->SuspendHierarchyNotifs(true);
				Hierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);
				EditMode->SuspendHierarchyNotifs(false);
			}
			else
			{
				Hierarchy->Notify(ERigHierarchyNotification::ControlSettingChanged, ControlElement);
			}
		}
	}

	bool FRigLevelEditorSpacePickerModel::CanAddSpace() const
	{
		return TryGetSingleHierarchy() != nullptr;
	}

	void FRigLevelEditorSpacePickerModel::AddSpace(const FRigElementKeyWithLabel& SpaceKeyWithLabel)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
		if (!Hierarchy ||
			!ControlRig)
		{
			return;
		}

		const FScopedTransaction AddSpaceToControlsTransaction(NSLOCTEXT("SpacePickerLevelEditorAddSpace", "AddSpaceToControlsTransaction", "Add Space To Controls"));
		ControlRig->Modify();

		for (const FRigElementKey& ControlKey : GetControlKeys())
		{
			const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(ControlKey);
			if (ControlElement &&
				Hierarchy->Contains(SpaceKeyWithLabel.Key))
			{
				// Don't add the default parent anew if we already have it
				if (Hierarchy->GetDefaultParent(ControlKey) == SpaceKeyWithLabel)
				{
					const TSharedPtr<FRigSpacePickerItem>* DefaultItemPtr = GetItems().FindByPredicate(
						[](const TSharedPtr<FRigSpacePickerItem>& Item)
						{
							return
								Item.IsValid() &&
								!Item->GetBindings().IsEmpty() &&
								Item->GetBindings()[0].SpaceKey == URigHierarchy::GetDefaultParentKey();
						});
					if (DefaultItemPtr)
					{
						// Close the add menu if it's still open
						FSlateApplication::Get().DismissAllMenus();

						// Bring to notice the default item is already added by flashing it
						constexpr float FlashDuration = 2.5f;
						(*DefaultItemPtr)->Flash(FlashDuration);
					}
				}
				else
				{
					const FRigControlElementCustomization* const ControlCustomization = ControlRig->GetControlCustomization(ControlKey);
										
					FRigControlElementCustomization New =
						ControlCustomization ? *ControlCustomization : FRigControlElementCustomization{};
					if (!New.AvailableSpaces.Contains(SpaceKeyWithLabel.Key))
					{
						New.AvailableSpaces.Add(SpaceKeyWithLabel);
					}
					New.RemovedSpaces.Remove(SpaceKeyWithLabel.Key);

					ControlRig->SetControlCustomization(ControlKey, New);
					NotifyControlSettingChanged(Hierarchy, ControlElement);
				}
			}
		}
	}

	bool FRigLevelEditorSpacePickerModel::CanDeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) const
	{
		return 
			!Item->IsDefaultSpace() &&
			TryGetSingleHierarchy() != nullptr;
	}

	void FRigLevelEditorSpacePickerModel::DeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
		if (!Hierarchy ||
			!ControlRig ||
			Item->IsDefaultSpace())
		{
			return;
		}

		const FScopedTransaction DeleteSpacesTransaction(LOCTEXT("DeleteSpacesTransaction", "Delete Space"));
		ControlRig->Modify();

		for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
		{
			if (const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Binding.ControlKey))
			{
				if (Binding.SpaceKey == URigHierarchy::GetDefaultParentKey() ||
					Binding.SpaceKey == URigHierarchy::GetWorldSpaceReferenceKey())
				{
					continue;
				}

				const FRigControlElementCustomization* const ControlCustomization = ControlRig->GetControlCustomization(Binding.ControlKey);

				FRigControlElementCustomization New = ControlCustomization ? *ControlCustomization : FRigControlElementCustomization{};
				New.AvailableSpaces.RemoveAll([&Binding](const FRigElementKeyWithLabel& SpaceKeyWithLabel) { return SpaceKeyWithLabel.Key == Binding.SpaceKey; });

				New.RemovedSpaces.AddUnique(Binding.SpaceKey);

				ControlRig->SetControlCustomization(Binding.ControlKey, New);
				NotifyControlSettingChanged(Hierarchy, ControlElement);
			}
		}
	}

	bool FRigLevelEditorSpacePickerModel::CanMoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const
	{
		return CanMoveSpacesImpl(Item, Direction);
	}

	void FRigLevelEditorSpacePickerModel::MoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction)
	{
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
		if (!Hierarchy ||
			!ControlRig ||
			Item->IsDefaultSpace())
		{
			return;
		}

		const int32 IndexDelta = Direction == ERigSpacePickerMoveSpaceDirection::Down ? 1 : -1;
		const int32 ItemIndex = GetItems().IndexOfByPredicate(
			[&Item](const TSharedPtr<FRigSpacePickerItem>& Existing) 
			{ 
				return Existing == Item; 
			});

		if (!GetItems().IsValidIndex(ItemIndex) ||
			!GetItems().IsValidIndex(ItemIndex + IndexDelta))
		{
			return;
		}

		const FScopedTransaction MoveSpacesTransaction(LOCTEXT("MoveSpacesTransaction", "Move Space"));
		ControlRig->Modify();

		// Reorder items
		TArray<TSharedPtr<FRigSpacePickerItem>> ReorderedItems = GetItems();
		ReorderedItems.Swap(ItemIndex, ItemIndex + IndexDelta);

		// Apply reordered spaces to the actual controls
		for (const FRigSpacePickerControlToSpaceBinding& Binding : Item->GetBindings())
		{
			if (const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(Binding.ControlKey))
			{
				const FRigElementKey& ControlKey = ControlElement->GetKey();
				const FRigControlElementCustomization* ExistingCustomization = ControlRig->GetControlCustomization(ControlKey);

				TArray<FRigElementKeyWithLabel> OrderedSpaces;
				for (const TSharedPtr<FRigSpacePickerItem>& ReorderedItem : ReorderedItems)
				{
					if (!ReorderedItem.IsValid() || ReorderedItem->IsDefaultSpace()) { continue; }
					for (const FRigSpacePickerControlToSpaceBinding& ReorderedBinding : ReorderedItem->GetBindings())
					{
						if (ReorderedBinding.GetHierarchy() == Hierarchy &&
							ReorderedBinding.ControlKey == ControlKey)
						{
							OrderedSpaces.Emplace(ReorderedBinding.SpaceKey, ReorderedBinding.GetDisplayName());
							break;
						}
					}
				}

				FRigControlElementCustomization NewElementCustomization;
				if (ExistingCustomization)
				{ 
					NewElementCustomization.RemovedSpaces = ExistingCustomization->RemovedSpaces;
				}
				NewElementCustomization.AvailableSpaces = OrderedSpaces;

				// Append spaces not present in items from the existing customization
				if (ExistingCustomization)
				{
					for (const FRigElementKeyWithLabel& ExistingSpace : ExistingCustomization->AvailableSpaces)
					{
						const bool bAlreadyIncluded = NewElementCustomization.AvailableSpaces.ContainsByPredicate(
							[&ExistingSpace](const FRigElementKeyWithLabel& Space)
							{
								return Space.Key == ExistingSpace.Key;
							});

						if (!bAlreadyIncluded)
						{
							NewElementCustomization.AvailableSpaces.Add(ExistingSpace);
						}
					}
				}

				ControlRig->SetControlCustomization(ControlKey, NewElementCustomization);
				NotifyControlSettingChanged(Hierarchy, ControlElement);
			}
		}
	}

	bool FRigLevelEditorSpacePickerModel::CanCompensateKeys() const
	{
		return CanBakeOrCompensate();
	}

	void FRigLevelEditorSpacePickerModel::CompensateKeys()
	{
		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		if (!Sequencer.IsValid())
		{
			return;
		}

		const FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		const FFrameTime FrameTime = Sequencer->GetLocalTime().ConvertTo(TickResolution);
		const TOptional<FFrameNumber> OptionalKeyTime = FrameTime.GetFrame();

		constexpr bool bCompensatePreviousTick = true;
		CompensateImpl(OptionalKeyTime, bCompensatePreviousTick);
	}

	bool FRigLevelEditorSpacePickerModel::CanCompensateAllKeys() const
	{
		return CanBakeOrCompensate();
	}

	void FRigLevelEditorSpacePickerModel::CompensateAllKeys()
	{
		constexpr bool bCompensatePreviousTick = true;
		const TOptional<FFrameNumber> OptionalKeyTime{};

		CompensateImpl(OptionalKeyTime, bCompensatePreviousTick);
	}

	bool FRigLevelEditorSpacePickerModel::CanShowBakeDialog() const
	{
		return CanBakeOrCompensate();
	}
	
	void FRigLevelEditorSpacePickerModel::ShowBakeDialog()
	{
		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		if (!Sequencer.IsValid())
		{
			return;
		}

		// Bake is only supported for a single hierarchy
		URigHierarchy* Hierarchy = TryGetSingleHierarchy();
		if (!Hierarchy)
		{
			return;
		}

		UControlRig* ControlRig = Hierarchy->GetTypedOuter<UControlRig>();
		if (!ControlRig)
		{
			return;
		}

		const TArray<FRigElementKey>* ControlKeysPtr = GetWeakHierarchyToControlKeysMap().Find(Hierarchy);
		if (!ControlKeysPtr || ControlKeysPtr->IsEmpty())
		{
			return;
		}
		const TArray<FRigElementKey>& ControlKeys = *ControlKeysPtr;

		FSpaceChannelAndSection SpaceChannelAndSection = FControlRigSpaceChannelHelpers::FindSpaceChannelAndSectionForControl(ControlRig, ControlKeys[0].Name, Sequencer.Get(), true /*bCreateIfNeeded*/);
		if (!SpaceChannelAndSection.SpaceChannel)
		{
			return;
		}

		using namespace UE::MovieScene;

		FRigSpacePickerBakeSettings Settings;
		Settings.TargetSpace = URigHierarchy::GetDefaultParentKey();

		const TRange<FFrameNumber> Range = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();
		Settings.Settings.StartFrame = Range.GetLowerBoundValue();
		Settings.Settings.EndFrame = Range.GetUpperBoundValue();

		const TSharedRef<SRigSpacePickerBakeDialog> BakeWidget =
			SNew(SRigSpacePickerBakeDialog)
			.Settings(Settings)
			.WeakHierarchy(Hierarchy)
			.Controls(ControlKeys);

		constexpr bool bModalWindow = true;
		BakeWidget->OpenDialog(bModalWindow);
	}

	bool FRigLevelEditorSpacePickerModel::CanBakeOrCompensate() const
	{
		if (GetControlKeys().IsEmpty())
		{
			return false;
		}

		if (TryGetSingleHierarchy() == nullptr)
		{
			return false;
		}

		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		return
			Sequencer.IsValid() &&
			Sequencer->GetFocusedMovieSceneSequence() != nullptr &&
			Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene() != nullptr;
	}

	void FRigLevelEditorSpacePickerModel::CompensateImpl(const TOptional<FFrameNumber>& OptionalKeyTime, const bool bCompensatePreviousTick) const
	{
		const TSharedPtr<ISequencer> Sequencer = GetSequencer();
		if (!Sequencer.IsValid())
		{
			return;
		}

		for (const URigHierarchy* Hierarchy : GetHierarchies())
		{
			UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;
			if (!Hierarchy ||
				!ControlRig)
			{
				continue;
			}

			if (UMovieSceneControlRigParameterSection* ControlRigSection = FControlRigSpaceChannelHelpers::GetControlRigSection(Sequencer.Get(), ControlRig))
			{
				FControlRigSpaceChannelHelpers::CompensateIfNeeded(ControlRig, Sequencer.Get(), ControlRigSection, OptionalKeyTime, bCompensatePreviousTick);
			}
		}
	}

	FControlRigEditMode* FRigLevelEditorSpacePickerModel::GetEditMode() const
	{
		return static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
	}
}

#undef LOCTEXT_NAMESPACE
