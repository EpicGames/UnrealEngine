// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigSpacePickerModelBase.h"

class FControlRigEditMode;
class ISequencer;
struct FRigControlElement;
struct FRigElementKey;
struct FRigElementKeyWithLabel;

namespace UE::ControlRigEditor
{
	/** Full model for a space pickers working within the level editor */
	class FRigLevelEditorSpacePickerModel
		: public FRigSpacePickerModelBase
		, public IRigSpacePickerSetActiveSpacesInterface
		, public IRigSpacePickerAddSpacesInterface
		, public IRigSpacePickerDeleteSpacesInterface
		, public IRigSpacePickerMoveSpacesInterface
		, public IRigSpacePickerCompensateKeysInterface
		, public IRigSpacePickerSupportsBakeDialogInterface
	{
	public:
		/** Returns the current sequencer or nullptr if sequencer is not active */
		TSharedPtr<ISequencer> GetSequencer() const;

	protected:
		//~ Begin FRigSpacePickerModelBase interface
		virtual IRigSpacePickerSetActiveSpacesInterface* GetSetActiveSpacesInterface() override { return static_cast<IRigSpacePickerSetActiveSpacesInterface*>(this); }
		virtual IRigSpacePickerAddSpacesInterface* GetAddSpacesInterface() override { return static_cast<IRigSpacePickerAddSpacesInterface*>(this); }
		virtual IRigSpacePickerDeleteSpacesInterface* GetDeleteSpacesInterface() override { return static_cast<IRigSpacePickerDeleteSpacesInterface*>(this); }
		virtual IRigSpacePickerMoveSpacesInterface* GetMoveSpacesInterface() override { return static_cast<IRigSpacePickerMoveSpacesInterface*>(this); }
		virtual IRigSpacePickerCompensateKeysInterface* GetCompensateKeysInterface() override { return static_cast<IRigSpacePickerCompensateKeysInterface*>(this); }
		virtual IRigSpacePickerSupportsBakeDialogInterface* GetSupportsBakeDialogInterface() override { return static_cast<IRigSpacePickerSupportsBakeDialogInterface*>(this); }
		//~ End FRigSpacePickerModelBase interface

		//~ Begin IRigSpacePickerSetActiveSpacesInterface interface
		virtual void SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item) override;
		//~ End IRigSpacePickerSetActiveSpacesInterface interface

		//~ Begin IRigSpacePickerAddSpacesInterface interface
		virtual bool CanAddSpace() const override;
		virtual void AddSpace(const FRigElementKeyWithLabel& SpaceKeyWithLabel) override;
		//~ End IRigSpacePickerAddSpacesInterface interface

		//~ Begin IRigSpacePickerDeleteSpacesInterface interface
		virtual bool CanDeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) const override;
		virtual void DeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) override;
		//~ End IRigSpacePickerDeleteSpacesInterface interface

		//~ Begin IRigSpacePickerMoveSpacesInterface interface
		virtual bool CanMoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const override;
		virtual void MoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) override;
		//~ End IRigSpacePickerMoveSpacesInterface interface

		//~ Begin IRigSpacePickerCompensateKeysInterface interface
		virtual bool CanCompensateKeys() const override;
		virtual void CompensateKeys() override;
		virtual bool CanCompensateAllKeys() const override;
		virtual void CompensateAllKeys() override;
		//~ End IRigSpacePickerCompensateKeysInterface interface

		//~ Begin IRigSpacePickerSupportsBakeDialogInterface interface
		virtual bool CanShowBakeDialog() const override;
		virtual void ShowBakeDialog() override;
		//~ End IRigSpacePickerSupportsBakeDialogInterface interface

		/** Notifies space settings of a control changed */
		void NotifyControlSettingChanged(URigHierarchy* Hierarchy, const FRigControlElement* ControlElement) const;

	private:
		/** Returns true if baking or compensating is possible */
		bool CanBakeOrCompensate() const;

		/** Implements the compensate functionality */
		void CompensateImpl(const TOptional<FFrameNumber>& OptionalKeyTime, const bool bCompensatePreviousTick) const;

		/** Returns the control rig edit mode of the level editor, or nullptr if the mode is not active */
		FControlRigEditMode* GetEditMode() const;
	};
}
