// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigSpacePickerItem.h"

#include "Algo/AllOf.h"
#include "ControlRigEditorStyle.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyItem.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyElements.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FRigSpacePickerControlToSpaceBinding"

namespace UE::ControlRigEditor
{
	namespace SpacePickerItemDetails
	{
		/** Validates the bindings of an item */
		static void ValidateBindings(const TArray<FRigSpacePickerControlToSpaceBinding>& Bindings)
		{
			ensureMsgf(!Bindings.IsEmpty(), TEXT("Unexpected, FRigSpacePickerItem without bindings, displaying a void item"));

			const bool bCommonSpaceName = Algo::AllOf(Bindings,
				[&Bindings](const FRigSpacePickerControlToSpaceBinding& Binding)
				{
					return Binding.GetCommonSpaceName() == Bindings[0].GetCommonSpaceName();
				});

			ensureMsgf(bCommonSpaceName, TEXT("Unexpected, FRigSpacePickerItem without a common space name, leaving multi-editing spaces in a non-functional state."));

			// Each (hierarchy, control) pair must be unique within an item - a control of a rig may not contribute two spaces to the same row
			const bool bUniqueControls = Algo::AllOf(Bindings,
				[&Bindings](const FRigSpacePickerControlToSpaceBinding& Binding)
				{
					int32 Count = 0;
					for (const FRigSpacePickerControlToSpaceBinding& Other : Bindings)
					{
						if (Other.GetHierarchy() == Binding.GetHierarchy() &&
							Other.ControlKey == Binding.ControlKey)
						{
							++Count;
						}
					}
					return Count == 1;
				});

			ensureMsgf(bUniqueControls, TEXT("Unexpected, FRigSpacePickerItem with duplicate (hierarchy, control) pairs. A control of a rig may only appear once per item."));
		}

		/** Computes the display name from the bindings */
		static FText ComputeDisplayName(const TArray<FRigSpacePickerControlToSpaceBinding>& Bindings)
		{
			if (Bindings.IsEmpty())
			{
				return FText::GetEmpty();
			}

			if (Bindings[0].SpaceKey == URigHierarchy::GetDefaultParentKey())
			{
				return LOCTEXT("Parent", "Parent");
			}
			if (Bindings[0].SpaceKey == URigHierarchy::GetWorldSpaceReferenceKey())
			{
				return LOCTEXT("World", "World");
			}
			if (Bindings.Num() == 1)
			{
				return FText::FromName(Bindings[0].GetDisplayName());
			}

			// Use the full display name if all bindings use the same long name, otherwise the short name
			const FName FirstDisplayName = Bindings[0].GetDisplayName();
			const bool bCommonDisplayName = Algo::AllOf(Bindings,
				[&FirstDisplayName](const FRigSpacePickerControlToSpaceBinding& B)
				{
					return B.GetDisplayName() == FirstDisplayName;
				});
			return FText::Format(
				FText::FromString(TEXT("{0} {1}")),
				LOCTEXT("SpacesForMultipleControls", "[Multiple]"),
				FText::FromName(bCommonDisplayName ? FirstDisplayName : Bindings[0].GetCommonSpaceName()));
		}

		/** Computes the icon brush from the bindings */
		static const FSlateBrush* ComputeIconBrush(const TArray<FRigSpacePickerControlToSpaceBinding>& Bindings)
		{
			if (Bindings.IsEmpty() || Bindings[0].SpaceKey == URigHierarchy::GetDefaultParentKey())
			{
				return FAppStyle::Get().GetBrush("Icons.Transform");
			}
			if (Bindings[0].SpaceKey == URigHierarchy::GetWorldSpaceReferenceKey())
			{
				return FAppStyle::GetBrush("EditorViewport.RelativeCoordinateSystem_World");
			}

			TOptional<const FSlateBrush*> CommonBrush;
			for (const FRigSpacePickerControlToSpaceBinding& Binding : Bindings)
			{
				if (const URigHierarchy* Hierarchy = Binding.GetHierarchy())
				{
					const FSlateBrush* BindingBrush =
						SRigHierarchyItem::GetBrushForElementType(Hierarchy, Binding.SpaceKey).Key;
					if (!CommonBrush.IsSet())
					{
						CommonBrush = BindingBrush;
					}
					else if (CommonBrush.GetValue() != BindingBrush)
					{
						return FAppStyle::Get().GetBrush("Icons.Transform");
					}
				}
			}
			return CommonBrush.IsSet() ? CommonBrush.GetValue() : FAppStyle::Get().GetBrush("Icons.Transform");
		}

		/** Computes the active state from the bindings */
		static ERigSpacePickerItemActiveState ComputeActiveState(const TArray<FRigSpacePickerControlToSpaceBinding>& Bindings)
		{
			int32 NumActive = 0;
			for (const FRigSpacePickerControlToSpaceBinding& Binding : Bindings)
			{
				if (URigHierarchy* Hierarchy = Binding.GetHierarchy())
				{
					if (Hierarchy->GetActiveParent(Binding.ControlKey) == Binding.SpaceKey)
					{
						NumActive++;
					}
				}
			}

			if (NumActive == Bindings.Num())
			{
				return ERigSpacePickerItemActiveState::FullyActive;
			}
			if (NumActive > 0)
			{
				return ERigSpacePickerItemActiveState::PartiallyActive;
			}
			return ERigSpacePickerItemActiveState::Inactive;
		}

		/** Computes the final display color, incorporating active state overrides */
		static FSlateColor ComputeColor(
			const ERigSpacePickerItemActiveState InActiveState,
			const TArray<FRigSpacePickerControlToSpaceBinding>& Bindings)
		{
			if (InActiveState == ERigSpacePickerItemActiveState::FullyActive)
			{
				return FControlRigEditorStyle::Get().GetSlateColor("ControlRig.SpacePicker.ActiveSpaceColor");
			}
			if (InActiveState == ERigSpacePickerItemActiveState::PartiallyActive)
			{
				return FControlRigEditorStyle::Get().GetSlateColor("ControlRig.SpacePicker.PartiallyActiveSpaceColor");
			}

			if (Bindings.IsEmpty() ||
				Bindings[0].SpaceKey == URigHierarchy::GetDefaultParentKey() ||
				Bindings[0].SpaceKey == URigHierarchy::GetWorldSpaceReferenceKey())
			{
				return FSlateColor::UseForeground();
			}

			TOptional<FSlateColor> CommonColor;
			for (const FRigSpacePickerControlToSpaceBinding& Binding : Bindings)
			{
				if (const URigHierarchy* Hierarchy = Binding.GetHierarchy())
				{
					const FSlateColor BindingColor = SRigHierarchyItem::GetBrushForElementType(Hierarchy, Binding.SpaceKey).Value;
					if (!CommonColor.IsSet())
					{
						CommonColor = BindingColor;
					}
					else if (CommonColor.GetValue() != BindingColor)
					{
						return FSlateColor::UseForeground();
					}
				}
			}
			return CommonColor.IsSet() ? CommonColor.GetValue() : FSlateColor::UseForeground();
		}

		/** Computes the tooltip text */
		static FText ComputeTooltip(const TArray<FRigSpacePickerControlToSpaceBinding>& Bindings)
		{
			const auto CreateBindingTooltips = [&Bindings]() -> FString
				{
					if (Bindings.IsEmpty())
					{
						return TEXT("");
					}

					const bool bShowControlName = Bindings.Num() > 1;

					FString BindingsTooltip;
					for (const FRigSpacePickerControlToSpaceBinding& Binding : Bindings)
					{						
						if (!BindingsTooltip.IsEmpty())
						{
							BindingsTooltip += TEXT("\n");
						}

						if (URigHierarchy* Hierarchy = Binding.GetHierarchy())
						{
							if (Hierarchy->GetActiveParent(Binding.ControlKey) == Binding.SpaceKey)
							{
								BindingsTooltip += LOCTEXT("BindingIsActiveTooltipHint", "[Active] ").ToString();
							}

							if (bShowControlName)
							{
								BindingsTooltip += Hierarchy->GetDisplayNameForUI(Binding.ControlKey).ToString() + TEXT(" / ");
							}

							if (Binding.SpaceKey == URigHierarchy::GetDefaultParentKey())
							{
								// If the space is the default parent, show the name of the actual space instead of "Parent"
								BindingsTooltip += Hierarchy->GetDisplayNameForUI(Hierarchy->GetDefaultParent(Binding.ControlKey), EElementNameDisplayMode::ForceShort).ToString();
							}
							else
							{
								BindingsTooltip += Binding.GetDisplayName().ToString();
							}
						}
					}
					return BindingsTooltip;
				};

			return FText::FromString(CreateBindingTooltips());
		}
	}

	FRigSpacePickerControlToSpaceBinding::FRigSpacePickerControlToSpaceBinding(
		URigHierarchy& InHierarchy,
		const FRigElementKey& InControlKey,
		const FRigElementKeyWithLabel& InSpaceKey)
		: ControlKey(InControlKey)
		, SpaceKey(InSpaceKey.Key)
		, WeakHierarchy(&InHierarchy)
	{
		DisplayName = InSpaceKey.GetLabel();

		CommonSpaceName = FRigSpacePickerItem::MakeCommonSpaceName(InSpaceKey);
	}

	FRigSpacePickerItem::FRigSpacePickerItem(
		const TArray<FRigSpacePickerControlToSpaceBinding>& InBindings)
		: Bindings(InBindings)
	{
		SpacePickerItemDetails::ValidateBindings(Bindings);

		DisplayName = SpacePickerItemDetails::ComputeDisplayName(Bindings);
		IconBrush = SpacePickerItemDetails::ComputeIconBrush(Bindings);
		ActiveState = SpacePickerItemDetails::ComputeActiveState(Bindings);
		Color = SpacePickerItemDetails::ComputeColor(ActiveState, Bindings);
		CachedTooltip = SpacePickerItemDetails::ComputeTooltip(Bindings);
	}

	FName FRigSpacePickerItem::MakeCommonSpaceName(const FRigElementKeyWithLabel& SpaceKey)
	{
		const FName FullSpaceLabel = SpaceKey.GetLabel();

		FString CommonSpaceNameString;
		if (!FullSpaceLabel.ToString().Split(TEXT("/"), nullptr, &CommonSpaceNameString, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
		{
			CommonSpaceNameString = FullSpaceLabel.ToString();
		}

		return *CommonSpaceNameString;
	}

	bool FRigSpacePickerItem::IsDefaultSpace() const
	{
		if (!Bindings.IsEmpty())
		{
			return
				Bindings[0].SpaceKey == URigHierarchy::GetDefaultParentKey() ||
				Bindings[0].SpaceKey == URigHierarchy::GetWorldSpaceReferenceKey();
		}

		return false;
	}

	void FRigSpacePickerItem::OverrideActiveState(const ERigSpacePickerItemActiveState NewActiveState)
	{
		ActiveState = NewActiveState;
		Color = SpacePickerItemDetails::ComputeColor(ActiveState, Bindings);
	}

	void FRigSpacePickerItem::Flash(const float Duration)
	{
		if (FlashTimer > 0.f)
		{
			// Already flashing, flash longer
			FlashTimer = FMath::Max(FlashTimer, Duration);
			return;
		}

		FlashTimer = Duration;
		const FLinearColor BaseColor = SpacePickerItemDetails::ComputeColor(ERigSpacePickerItemActiveState::FullyActive, Bindings).GetColor(FWidgetStyle());

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[Duration, BaseColor, WeakThis = AsWeak(), this](float DeltaSeconds) mutable
			{
				if (!WeakThis.IsValid())
				{
					return false;
				}

				FlashTimer -= DeltaSeconds;
				if (FlashTimer <= 0.f)
				{
					FlashTimer = 0.f;
					Color = SpacePickerItemDetails::ComputeColor(ActiveState, Bindings);
					
					return false;
				}

				// Nearest integer cycle to 2Hz so the wave completes cleanly at duration
				const int32 Cycles = FMath::Max(1, FMath::RoundToInt(2.f * Duration));
				const float Frequency = static_cast<float>(Cycles) / Duration;
				
				const float Wave = FMath::Cos(2.f * PI * Frequency * FlashTimer);
								
				const float Alpha = ActiveState == ERigSpacePickerItemActiveState::Inactive ? 
					(-Wave + 1.f) * 0.375f :			// 0.0 to 0.75
					0.25f + (Wave + 1.f) * 0.375f;		// 1.0 to 0.25

				Color = BaseColor.CopyWithNewOpacity(Alpha);

				return true;
			}));
	}

	bool FRigSpacePickerItem::IsFlashing() const
	{
		return FlashTimer > 0.f;
	}
}

#undef LOCTEXT_NAMESPACE
