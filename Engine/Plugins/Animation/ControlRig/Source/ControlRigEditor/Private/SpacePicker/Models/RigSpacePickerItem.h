// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyDefines.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

class URigHierarchy;
struct FRigElementKeyWithLabel;
struct FSlateBrush;

namespace UE::ControlRigEditor
{
	/** A control bound to a space. Useful to group bindings by their common space name */
	struct FRigSpacePickerControlToSpaceBinding
	{
		FRigSpacePickerControlToSpaceBinding(
			URigHierarchy& Hierarchy,
			const FRigElementKey& InControlKey,
			const FRigElementKeyWithLabel& InSpaceKey);

		const FRigElementKey ControlKey;
		const FRigElementKey SpaceKey;

		/** Returns the name common to spaces that can be multi edited */
		const FName& GetCommonSpaceName() const { return CommonSpaceName; }

		/** Returns the display name of the space, useful when not multi-editing */
		const FName& GetDisplayName() const { return DisplayName; }

		/** Returns the hierarchy or nullptr if the hierarchy of this binding is no longer valid */
		URigHierarchy* GetHierarchy() const { return WeakHierarchy.Get(); }

		inline bool operator==(const FRigSpacePickerControlToSpaceBinding& Other) const
		{
			return
				WeakHierarchy == Other.WeakHierarchy &&
				ControlKey == Other.ControlKey &&
				SpaceKey == Other.SpaceKey;
		}

		inline bool operator!=(const FRigSpacePickerControlToSpaceBinding& Other) const
		{
			return !(*this == Other);
		}

	private:
		/** The name common to spaces that can be multi edited */
		FName CommonSpaceName;

		/** The display name of the space, useful when not multi-editing */
		FName DisplayName;

		/** The hierarchy that contains the control and space */
		TWeakObjectPtr<URigHierarchy> WeakHierarchy;
	};

	/** Enumerates active states an item can have */
	enum class ERigSpacePickerItemActiveState : uint8
	{
		/** The item is not active */
		Inactive,

		/** Some bindings in this item are active */
		PartiallyActive,

		/** All bindings in this item are active. Implies no other item is active. */
		FullyActive
	};

	/** 
	 * An item that is displayed as a single row in the space editor widget. 
	 * 
	 * The item contains an array of control to space bindings, one space for each control
	 * This is to enable multi-editing when multiple controls with common space names are selected.
	 * 
	 * Items are guaranteed to have common space names accross all their bindings (ensured).
	 */
	class FRigSpacePickerItem
		: public TSharedFromThis<FRigSpacePickerItem>
	{
	public:
		/** 
		 * Constructs this item 
		 * 
		 * @param InBindings		The bindings for this item
		 */
		explicit FRigSpacePickerItem(const TArray<FRigSpacePickerControlToSpaceBinding>& InBindings);

		/** Makes a common space name from the space key. Only bindings with common space names should be stored in an item */
		static FName MakeCommonSpaceName(const FRigElementKeyWithLabel& SpaceKey);

		/** Returns true if this item is a default space */
		bool IsDefaultSpace() const;

		/** Returns the bindings for this item */
		const TArray<FRigSpacePickerControlToSpaceBinding>& GetBindings() const { return Bindings; }

		/** Returns the color for this item */
		const FSlateColor& GetColor() const { return Color; }

		/** The icon brush to display for this item */
		const FSlateBrush* GetIconBrush() const { return IconBrush; }

		/** Returns the display name for this item */
		const FText& GetDisplayName() const { return DisplayName; }

		/** Returns the tooltip for this item */
		FText GetTooltip() const { return CachedTooltip; }

		/** Returns the active state of this item */
		ERigSpacePickerItemActiveState GetActiveState() const { return ActiveState; }

		/** Overrides the active state. Useful if the active state shouldn't follow the hierarchy */
		void OverrideActiveState(const ERigSpacePickerItemActiveState NewActiveState);

		/** Flashes this item's color during duration, useful to provide hints to the user */
		void Flash(const float Duration);

		/** True while the item is flashing */
		bool IsFlashing() const;

	private:
		/** Bindings for this item */
		TArray<FRigSpacePickerControlToSpaceBinding> Bindings;

		/** The name displayed for this item */
		FText DisplayName;

		/** The icon brush to display for this item */
		const FSlateBrush* IconBrush = nullptr;

		/** The final display color, incorporating active state */
		FSlateColor Color = FSlateColor::UseForeground();

		/** The active state of this item */
		ERigSpacePickerItemActiveState ActiveState = ERigSpacePickerItemActiveState::Inactive;

		/** Cached tooltip text */
		FText CachedTooltip;

		/** Timer for the icon to flash */
		float FlashTimer = 0.f;
	};
}
