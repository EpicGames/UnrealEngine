// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigSpacePickerModelBase.h"

#include "ControlRig.h"
#include "Rigs/RigHierarchy.h"
#include "RigSpacePickerItem.h"

#define LOCTEXT_NAMESPACE "FRigSpacePickerModelBase"

namespace UE::ControlRigEditor
{
	namespace SpacePickerModelDetails
	{
		struct FRigSpacePickerItemDataGroups
		{
			/** 
			 * Groups bindings so they are suitable to be used in an item.
			 * The logic considers cases where a control can have multiple possible spaces with same short name, 
			 * e.g. arm_l_ctrl / PV null and arm_r_ctrl / PV null, and creates two groups to handle such cases.
			 */
			void Add(const FRigSpacePickerControlToSpaceBinding& Binding)
			{
				int32 SpaceMatchIndex = INDEX_NONE;
				int32 FallbackIndex   = INDEX_NONE;
				for (int32 ItemDataGroupIndex = 0; ItemDataGroupIndex < ItemDataGroups.Num(); ItemDataGroupIndex++)
				{
					const bool bControlPresent = ItemDataGroups[ItemDataGroupIndex].ContainsByPredicate(
						[&Binding](const FRigSpacePickerControlToSpaceBinding& Existing)
						{
							return Existing.GetHierarchy() == Binding.GetHierarchy() &&
								Existing.ControlKey == Binding.ControlKey;
						});
					if (bControlPresent)
					{
						continue;
					}
					if (SpaceMatchIndex == INDEX_NONE &&
						ItemDataGroups[ItemDataGroupIndex].ContainsByPredicate(
							[&Binding](const FRigSpacePickerControlToSpaceBinding& Existing)
							{
								return Existing.SpaceKey == Binding.SpaceKey;
							}))
					{
						SpaceMatchIndex = ItemDataGroupIndex;
					}
					else if (FallbackIndex == INDEX_NONE)
					{
						FallbackIndex = ItemDataGroupIndex;
					}
				}
				const int32 TargetIndex = SpaceMatchIndex != INDEX_NONE ? SpaceMatchIndex : FallbackIndex;
				if (TargetIndex != INDEX_NONE)
				{
					ItemDataGroups[TargetIndex].Add(Binding);
				}
				else
				{
					ItemDataGroups.Add({ Binding });
				}
			}

			const TArray<TArray<FRigSpacePickerControlToSpaceBinding>>& GetItemDataGroups() const { return ItemDataGroups; }

		private:
			TArray<TArray<FRigSpacePickerControlToSpaceBinding>> ItemDataGroups;
		};

		/** Builds items for the model. Keeps track of items to never create the same thing twice */
		class FRigSpacePickerItemsFactory
		{
		public:
			/** Creates default items */
			TArray<TSharedPtr<FRigSpacePickerItem>> CreateDefaultItems() const
			{
				TArray<TSharedPtr<FRigSpacePickerItem>> Items;
				TryCreateDefaultItems(Items);
				return Items;
			}

			/** Creates additional items */
			TArray<TSharedPtr<FRigSpacePickerItem>> CreateAdditionalItems() const
			{
				TArray<TSharedPtr<FRigSpacePickerItem>> Items;
				TryCreateAdditionalItems(Items);
				return Items;
			}

			/** Constructs this factory */
			FRigSpacePickerItemsFactory(const FRigSpacePickerModelBase& InModel)
				: Model(InModel)
			{
			}

		private:
			/** Tries to create items for default spaces, stores them in the item array */
			void TryCreateDefaultItems(TArray<TSharedPtr<FRigSpacePickerItem>>& InOutItems) const
			{
				InOutItems.Reset();
				if (!Model.ShouldGenerateDefaultItems())
				{
					return;
				}

				const FRigElementKeyWithLabel ParentKey(URigHierarchy::GetDefaultParentKey(), *LOCTEXT("Parent", "Parent").ToString());
				const FRigElementKeyWithLabel WorldKey(URigHierarchy::GetWorldSpaceReferenceKey(), *LOCTEXT("World", "World").ToString());

				TArray<FRigSpacePickerControlToSpaceBinding> AllParentBindings;
				TArray<FRigSpacePickerControlToSpaceBinding> AllWorldBindings;

				for (const auto& Pair : Model.GetWeakHierarchyToControlKeysMap())
				{
					URigHierarchy* Hierarchy = Pair.Key.Get();
					if (!Hierarchy)
					{
						continue;
					}

					for (const FRigElementKey& ControlKey : Pair.Value)
					{
						AllParentBindings.Emplace(*Hierarchy, ControlKey, ParentKey);
						AllWorldBindings.Emplace(*Hierarchy, ControlKey, WorldKey);
					}
				}

				if (!AllParentBindings.IsEmpty())
				{
					InOutItems.Add(MakeShared<FRigSpacePickerItem>(AllParentBindings));
				}

				if (!AllWorldBindings.IsEmpty())
				{
					InOutItems.Add(MakeShared<FRigSpacePickerItem>(AllWorldBindings));
				}
			}

			/** Tries to create items for additional spaces, stores them in the item array */
			void TryCreateAdditionalItems(TArray<TSharedPtr<FRigSpacePickerItem>>& InOutItems) const
			{
				InOutItems.Reset();

				int32 NumControls = 0;
				TArray<FRigSpacePickerControlToSpaceBinding> AllBindings;

				for (const TPair<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& WeakHierarchyToControlKeyPair : Model.GetWeakHierarchyToControlKeysMap())
				{
					URigHierarchy* Hierarchy = WeakHierarchyToControlKeyPair.Key.Get();
					if (!Hierarchy)
					{
						continue;
					}

					for (const FRigElementKey& ControlKey : WeakHierarchyToControlKeyPair.Value)
					{
						AllBindings.Append(GatherSpacesForControl(*Hierarchy, ControlKey));
						++NumControls;
					}
				}

				if (AllBindings.IsEmpty())
				{
					return;
				}

				TMap<FName, FRigSpacePickerItemDataGroups> SpaceNameToItemDataGroupsMap;
				for (const FRigSpacePickerControlToSpaceBinding& Binding : AllBindings)
				{
					SpaceNameToItemDataGroupsMap.FindOrAdd(Binding.GetCommonSpaceName()).Add(Binding);
				}

				for (const TTuple<FName, FRigSpacePickerItemDataGroups>& SpaceNameToItemDataGroupsPair : SpaceNameToItemDataGroupsMap)
				{
					for (const TArray<FRigSpacePickerControlToSpaceBinding>& ItemDataGroup : SpaceNameToItemDataGroupsPair.Value.GetItemDataGroups())
					{
						if (ItemDataGroup.Num() == NumControls)
						{
							InOutItems.Add(MakeShared<FRigSpacePickerItem>(ItemDataGroup));
						}
					}
				}
			}

			/** Gathers all spaces that are available to controls, depending on the model and the control's customization */
			TArray<FRigSpacePickerControlToSpaceBinding> GatherSpacesForControl(URigHierarchy& Hierarchy, const FRigElementKey& ControlKey) const
			{
				TArray<FRigSpacePickerControlToSpaceBinding> Result;

				UControlRig* ControlRig = Hierarchy.GetTypedOuter<UControlRig>();
				if (!ControlRig)
				{
					return Result;
				}

				if (!Hierarchy.Contains(ControlKey))
				{
					return Result;
				}

				TSet<FRigElementKey> SeenKeys;
				auto EmplaceIfNew = [&Result, &SeenKeys, &ControlKey, &Hierarchy](const FRigElementKeyWithLabel& Space)
					{
						bool bAlreadyPresent = false;
						if (Space.Key.IsValid() &&
							Space.Key != URigHierarchy::GetDefaultParentKey() &&
							Space.Key != URigHierarchy::GetWorldSpaceReferenceKey() &&
							Hierarchy.Contains(Space.Key))
						{
							SeenKeys.Add(Space.Key, &bAlreadyPresent);
							if (!bAlreadyPresent)
							{
								Result.Emplace(Hierarchy, ControlKey, Space);
							}
						}
					};

				const FRigControlElementCustomization* ControlCustomizationPtr = ControlRig->GetControlCustomization(ControlKey);

				if (Model.ShouldGenerateFavoriteItems())
				{
					if (ControlCustomizationPtr)
					{
						for (const FRigElementKeyWithLabel& Space : ControlCustomizationPtr->AvailableSpaces)
						{
							EmplaceIfNew(Space);
						}
					}

					if (const FRigControlElement* ControlElement = Hierarchy.Find<FRigControlElement>(ControlKey))
					{
						for (const FRigElementKeyWithLabel& Space : ControlElement->Settings.Customization.AvailableSpaces)
						{
							if (ControlCustomizationPtr &&
								ControlCustomizationPtr->RemovedSpaces.Contains(Space.Key))
							{
								continue;
							}
														
							EmplaceIfNew(Space);
						}
					}
				}

				if (Model.ShouldGenerateAdditionalItems())
				{
					for (const FRigElementKeyWithLabel& Space : GetParents(Hierarchy, ControlKey))
					{
						if (ControlCustomizationPtr &&
							ControlCustomizationPtr->RemovedSpaces.Contains(Space.Key))
						{
							continue;
						}

						EmplaceIfNew(Space);
					}
				}

				const FRigElementKey DefaultParent = Hierarchy.GetDefaultParent(ControlKey);
				for (const FRigElementKeyWithLabel& Space : Model.GatherAdditionalSpaces(&Hierarchy, ControlKey))
				{
					if (ControlCustomizationPtr &&
						ControlCustomizationPtr->RemovedSpaces.Contains(Space.Key))
					{
						continue;
					}

					// Don't add the default parent if it's already added to default items
					if (Model.ShouldGenerateDefaultItems() &&
						Space == DefaultParent)
					{
						continue;
					}

					EmplaceIfNew(Space);
				}

				return Result;
			}

			/** Returns the parents of a control key */
			TArray<FRigElementKeyWithLabel> GetParents(
				URigHierarchy& Hierarchy,
				const FRigElementKey& InControlKey) const
			{
				TArray<FRigElementKey> Parents = Hierarchy.GetParents(InControlKey);
				if (!Parents.IsEmpty())
				{
					// The first key is the default parent, hence use the default parent key
					if (Parents[0].IsValid() &&
						Parents[0] != URigHierarchy::GetDefaultParentKey() &&
						Parents[0] != URigHierarchy::GetWorldSpaceReferenceKey())
					{
						Parents[0] = URigHierarchy::GetDefaultParentKey();
					}
				}

				TArray<FRigElementKeyWithLabel> ParentSpaces;
				ParentSpaces.Reserve(Parents.Num());
				for (const FRigElementKey& ParentKey : Parents)
				{
					ParentSpaces.Emplace(ParentKey, Hierarchy.GetDisplayLabelForParent(InControlKey, ParentKey));
				}

				return ParentSpaces;
			}

			/** The model for which items are generated */
			const FRigSpacePickerModelBase& Model;
		};
	}

	FRigSpacePickerModelBase::~FRigSpacePickerModelBase()
	{
		for (const TTuple<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& Pair : WeakHierarchyToControlKeysMap)
		{
			if (URigHierarchy* Hierarchy = Pair.Key.Get())
			{
				Hierarchy->OnModified().RemoveAll(this);
			}
		}
	}

	void FRigSpacePickerModelBase::Update()
	{
		Update(WeakHierarchyToControlKeysMap);
	}

	void FRigSpacePickerModelBase::Update(const TWeakObjectPtr<URigHierarchy>& InWeakHierarchy, const TArray<FRigElementKey>& InSelectedControls)
	{
		TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>> NewWeakHierarchyToControlKeysMap;
		NewWeakHierarchyToControlKeysMap.Add(InWeakHierarchy, InSelectedControls);

		Update(NewWeakHierarchyToControlKeysMap);
	}

	void FRigSpacePickerModelBase::Update(const TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& InWeakHierarchyToControlKeysMap)
	{
		// Stop listening to old hierarchy modifications
		for (const TTuple<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& OldWeakHierarchyToControlKeysPair : WeakHierarchyToControlKeysMap)
		{
			const TWeakObjectPtr<URigHierarchy>& OldWeakHierarchy = OldWeakHierarchyToControlKeysPair.Key;
			if (OldWeakHierarchy.IsValid())
			{
				OldWeakHierarchy->OnModified().RemoveAll(this);
			}
		}

		// Filter out invalid hierarchies and controls
		WeakHierarchyToControlKeysMap = InWeakHierarchyToControlKeysMap;
		for (auto WeakHierarchyToControlKeysIt = WeakHierarchyToControlKeysMap.CreateIterator(); WeakHierarchyToControlKeysIt; ++WeakHierarchyToControlKeysIt)
		{
			// Remove invalid hierarchies
			URigHierarchy* Hierarchy = WeakHierarchyToControlKeysIt->Key.Get();
			if (!Hierarchy)
			{
				WeakHierarchyToControlKeysIt.RemoveCurrent();
				continue;
			}

			for (auto ControlIt = (*WeakHierarchyToControlKeysIt).Value.CreateIterator(); ControlIt; ++ControlIt)
			{
				// Remove invalid controls and controls that don't exist in the hierarchy
				if (!ControlIt->IsValid() ||
					!ControlIt->IsTypeOf(ERigElementType::Control) ||
					!Hierarchy->Contains(*ControlIt))
				{
					ControlIt.RemoveCurrent();
					continue;
				}

				if (const FRigControlElement* ControlElement = Hierarchy->Find<FRigControlElement>(*ControlIt))
				{
					// Remove controls that have no shape or are not animatable 
					if (!ControlElement->Settings.SupportsShape() ||
						!Hierarchy->IsAnimatable(ControlElement))
					{
						ControlIt.RemoveCurrent();
						continue;
					}

					// Remove scalar/channel controls that are children of another control
					if (ControlElement->Settings.ControlType == ERigControlType::Bool ||
						ControlElement->Settings.ControlType == ERigControlType::Float ||
						ControlElement->Settings.ControlType == ERigControlType::ScaleFloat ||
						ControlElement->Settings.ControlType == ERigControlType::Integer)
					{
						// Only top-level single-channel controls are eligible
						if (const FRigControlElement* ParentControlElement = Cast<FRigControlElement>(Hierarchy->GetFirstParent(ControlElement)))
						{
							ControlIt.RemoveCurrent();
							continue;
						}
					}
				}
			}

			if (WeakHierarchyToControlKeysIt->Value.IsEmpty())
			{
				WeakHierarchyToControlKeysIt.RemoveCurrent();
			}
		}

		// Listen to new hierarchy modifications
		for (const TTuple<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& NewWeakHierarchyToControlKeysPair : WeakHierarchyToControlKeysMap)
		{
			const TWeakObjectPtr<URigHierarchy>& NewWeakHierarchy = NewWeakHierarchyToControlKeysPair.Key;
			if (NewWeakHierarchy.IsValid())
			{
				NewWeakHierarchy->OnModified().AddSP(this, &FRigSpacePickerModelBase::OnHierarchyModified);
			}
		}

		if (OnRequestRefreshMVVM.IsBound())
		{
			// If the view is constructed, let it handle item creation
			OnRequestRefreshMVVM.Broadcast();
		}
		else
		{
			GenerateItems();
		}
	}

	void FRigSpacePickerModelBase::GenerateItems()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRigSpacePickerModelBase::GenerateItems);

		Items.Reset();

		using FRigSpacePickerItemsFactory = SpacePickerModelDetails::FRigSpacePickerItemsFactory;
		const FRigSpacePickerItemsFactory Factory(*this);

		Items.Append(Factory.CreateDefaultItems());
		Items.Append(Factory.CreateAdditionalItems());

		// Update caches
		CacheIsSpaceSwitchingRestricted();
	}

	TArray<FRigElementKey> FRigSpacePickerModelBase::GetControlKeys() const
	{
		TArray<FRigElementKey> ControlKeys;

		ForEachControlKey(
			[&ControlKeys](URigHierarchy* Hierarchy, const FRigElementKey& ControlKey)
			{
				ControlKeys.Add(ControlKey);
			});

		return ControlKeys;
	}

	bool FRigSpacePickerModelBase::DoesControlHaveSpace(const URigHierarchy* Hierarchy, const FRigElementKey& ControlKey, const FRigElementKey& SpaceKey) const
	{
		const auto IsContainedInBinding =
			[Hierarchy, &ControlKey, &SpaceKey](const FRigSpacePickerControlToSpaceBinding& Binding)
			{
				return
					Binding.GetHierarchy() == Hierarchy &&
					Binding.ControlKey == ControlKey &&
					Binding.SpaceKey == SpaceKey;
			};

		const auto IsContainedInItem = [&IsContainedInBinding](const TSharedPtr<FRigSpacePickerItem>& Item)
			{
				return
					Item->GetBindings().ContainsByPredicate(IsContainedInBinding);
			};

		return Items.ContainsByPredicate(IsContainedInItem);
	}

	TArray<URigHierarchy*> FRigSpacePickerModelBase::GetHierarchies() const
	{
		TArray<URigHierarchy*> Hierarchies;
		Algo::TransformIf(WeakHierarchyToControlKeysMap, Hierarchies,
			[](const TTuple< TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& WeakHierarchyToControlsPair)
			{
				return WeakHierarchyToControlsPair.Key.IsValid();
			},
			[](const TTuple< TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& WeakHierarchyToControlsPair)
			{
				return WeakHierarchyToControlsPair.Key.Get();
			});

		return Hierarchies;
	}

	URigHierarchy* FRigSpacePickerModelBase::TryGetSingleHierarchy() const
	{
		if (WeakHierarchyToControlKeysMap.Num() == 1)
		{
			return WeakHierarchyToControlKeysMap.CreateConstIterator()->Key.Get();
		}

		return nullptr;
	}

	void FRigSpacePickerModelBase::PostUndo(bool bSuccess)
	{
		OnRequestRefreshMVVM.Broadcast();
	}

	void FRigSpacePickerModelBase::PostRedo(bool bSuccess)
	{
		OnRequestRefreshMVVM.Broadcast();
	}

	bool FRigSpacePickerModelBase::CanMoveSpacesImpl(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const
	{
		const int32 MovableItemOffset = ShouldGenerateDefaultItems() ? 2 : 0;

		const bool bEnoughItemsToReorder = GetItems().Num() > MovableItemOffset + 1;
		const bool bFirst = bEnoughItemsToReorder ? GetItems()[MovableItemOffset] == Item : false;
		const bool bLast = bEnoughItemsToReorder ? GetItems().Last() == Item : false;

		const bool bCanMoveUp = bEnoughItemsToReorder && !bFirst;
		const bool bCanMoveDown = bEnoughItemsToReorder && !bLast;

		return
			Direction == ERigSpacePickerMoveSpaceDirection::Up ?
			bCanMoveUp :
			bCanMoveDown;
	}

	void FRigSpacePickerModelBase::CacheIsSpaceSwitchingRestricted()
	{
		bCachedIsSpaceSwitchingRestricted = false;
		ForEachControlKeyWithBreak([this](URigHierarchy* Hierarchy, const FRigElementKey& ControlKey)
			{
				const FRigControlElement* ControlElement = Hierarchy ? Hierarchy->Find<FRigControlElement>(ControlKey) : nullptr;
				if (ControlElement &&
					ControlElement->Settings.bRestrictSpaceSwitching)
				{
					bCachedIsSpaceSwitchingRestricted = true;
					return false;
				}

				return true;
			});
	}

	void FRigSpacePickerModelBase::OnHierarchyModified(
		ERigHierarchyNotification InNotif,
		URigHierarchy* InHierarchy,
		const FRigNotificationSubject& InSubject)
	{
		switch (InNotif)
		{
			case ERigHierarchyNotification::ParentChanged:
			case ERigHierarchyNotification::ParentWeightsChanged:
			case ERigHierarchyNotification::ControlSettingChanged:
			{
				if (IsInGameThread())
				{
					OnRequestRefreshMVVM.Broadcast();
				}
				else
				{
					ExecuteOnGameThread(UE_SOURCE_LOCATION, 
						[WeakThis = AsWeak()]()
						{
							if (const TSharedPtr<FRigSpacePickerModelBase> PinnedThis = WeakThis.Pin())
							{
								PinnedThis->OnRequestRefreshMVVM.Broadcast();
							}
						});
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}

	void FRigSpacePickerModelBase::ForEachControlKey(TFunctionRef<FHierarchyToControlFunctionSignature> Function) const
	{
		for (const TTuple<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& WeakHierarchyToControlKeysPair : WeakHierarchyToControlKeysMap)
		{
			if (URigHierarchy* Hierarchy = WeakHierarchyToControlKeysPair.Key.Get())
			{
				for (const FRigElementKey& ControlKey : WeakHierarchyToControlKeysPair.Value)
				{
					Function(Hierarchy, ControlKey);
				}
			}
		}
	}

	void FRigSpacePickerModelBase::ForEachControlKeyWithBreak(TFunctionRef<FHierarchyToControlFunctionWithBreakSignature> Function) const
	{
		for (const TTuple<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& WeakHierarchyToControlKeysPair : WeakHierarchyToControlKeysMap)
		{
			if (URigHierarchy* Hierarchy = WeakHierarchyToControlKeysPair.Key.Get())
			{
				for (const FRigElementKey& ControlKey : WeakHierarchyToControlKeysPair.Value)
				{
					if (!Function(Hierarchy, ControlKey))
					{
						return;
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
