// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Rigs/RigHierarchyElements.h"
#include "Templates/SharedPointer.h"

class FRigHierarchyTreeElement;
class UControlRig;
class URigHierarchy;
enum class ERigHierarchyNotification : uint8;
struct FRigControlElementCustomization;
struct FRigElementKeyWithLabel;
struct FRigNotificationSubject;

namespace UE::ControlRigEditor
{
	class FRigSpacePickerItem;
	struct FRigSpacePickerControlToSpaceBinding;

	/** Interface for models that can set active spaces */
	class IRigSpacePickerSetActiveSpacesInterface
	{
	public:
		virtual ~IRigSpacePickerSetActiveSpacesInterface() = default;

		/** Sets the active spaces */
		virtual void SetActiveSpaces(const TSharedRef<FRigSpacePickerItem>& Item) = 0;
	};

	/** Interface for models that can add spaces */
	class IRigSpacePickerAddSpacesInterface
	{
	public:
		virtual ~IRigSpacePickerAddSpacesInterface() = default;

		/** Returns true if spaces can currently be added */
		virtual bool CanAddSpace() const = 0;

		/** Adds spaces */
		virtual void AddSpace(const FRigElementKeyWithLabel& SpaceKeyWithLabel) = 0;
	};

	/** Interface for models that can delete spaces */
	class IRigSpacePickerDeleteSpacesInterface
	{
	public:
		virtual ~IRigSpacePickerDeleteSpacesInterface() = default;

		/** Returns true if spaces can currently be deleted */
		virtual bool CanDeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) const = 0;

		/** Deletes spaces of the item */
		virtual void DeleteSpaces(const TSharedRef<FRigSpacePickerItem>& Item) = 0;
	};

	/** Direction in which items are moved */
	enum class ERigSpacePickerMoveSpaceDirection : uint8
	{
		Up,
		Down
	};

	/** Interface for models that can move spaces */
	class IRigSpacePickerMoveSpacesInterface
	{
	public:
		virtual ~IRigSpacePickerMoveSpacesInterface() = default;

		/** Returns true if spaces can currently be moved */
		virtual bool CanMoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const = 0;

		/** Moves spaces of the item */
		virtual void MoveSpaces(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) = 0;
	};

	/** Interface for models that can compensate keys */
	class IRigSpacePickerCompensateKeysInterface
	{
	public:
		virtual ~IRigSpacePickerCompensateKeysInterface() = default;

		/** Returns true if keys can be compensated in sequencer */
		virtual bool CanCompensateKeys() const = 0;

		/** Compensates keys in sequencer */
		virtual void CompensateKeys() = 0;

		/** Returns true if all keys can be compensated in sequencer */
		virtual bool CanCompensateAllKeys() const = 0;

		/** Compensates all keys in sequencer */
		virtual void CompensateAllKeys() = 0;
	};

	/** Interface for models that support showing the baking dialog */
	class IRigSpacePickerSupportsBakeDialogInterface
	{
	public:
		virtual ~IRigSpacePickerSupportsBakeDialogInterface() = default;

		/** Returns true if a bake dialog can be displayed */
		virtual bool CanShowBakeDialog() const = 0;

		/** Shows the bake dialog */
		virtual void ShowBakeDialog() = 0;
	};

	/** Base view model for the space picker */
	class FRigSpacePickerModelBase
		: public FSelfRegisteringEditorUndoClient
		, public TSharedFromThis<FRigSpacePickerModelBase>
	{		
	public:
		virtual ~FRigSpacePickerModelBase();

		/** Updates the model by reevaluating the current hierarchies and controls */
		void Update();

		/** Updates the model from a single hierarchy and its currently selected controls */
		void Update(const TWeakObjectPtr<URigHierarchy>& InWeakHierarchy, const TArray<FRigElementKey>& InSelectedControls);

		/** Updates the model from hierarchies and their currently selected controls */
		void Update(const TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& InWeakHierarchyToControlKeysMap);

		/** Generates item from this model, does not update the model */
		void GenerateItems();

		/** Returns the current items */
		const TArray<TSharedPtr<FRigSpacePickerItem>>& GetItems() const { return Items; }

		/** Returns true if any of the selected controls restricts space switching */
		bool IsSpaceSwitchingRestricted() const { return bCachedIsSpaceSwitchingRestricted; }

		/** Returns the interface to set active spaces, or nullptr if setting active spaces is not supported */
		virtual IRigSpacePickerSetActiveSpacesInterface* GetSetActiveSpacesInterface() { return nullptr; }

		/** Returns the interface to add spaces, or nullptr if adding spaces is not supported */
		virtual IRigSpacePickerAddSpacesInterface* GetAddSpacesInterface() { return nullptr; }

		/** Returns the interface to delete spaces, or nullptr if deleting spaces is not supported */
		virtual IRigSpacePickerDeleteSpacesInterface* GetDeleteSpacesInterface() { return nullptr; }

		/** Returns the interface to move spaces, or nullptr if moving spaces is not supported */
		virtual IRigSpacePickerMoveSpacesInterface* GetMoveSpacesInterface() { return nullptr; }

		/** Returns the interface to compensate keys for spaces, or nullptr compensating keys is not supported */
		virtual IRigSpacePickerCompensateKeysInterface* GetCompensateKeysInterface() { return nullptr; }

		/** Returns the interface to show a bake dialog, or nullptr if showing a bake dialog is not supported */
		virtual IRigSpacePickerSupportsBakeDialogInterface* GetSupportsBakeDialogInterface() { return nullptr; }

		/** Returns the current control keys */
		TArray<FRigElementKey> GetControlKeys() const;

		/** Returns true if given control already has given space assigned */
		bool DoesControlHaveSpace(const URigHierarchy* Hierarchy, const FRigElementKey& ControlKey, const FRigElementKey& SpaceKey) const;

		/** Returns the current hierarchies */
		TArray<URigHierarchy*> GetHierarchies() const;

		/** Tries to get the single hierarchy currently present in this model. Returns nullptr if there are none or many. */
		URigHierarchy* TryGetSingleHierarchy() const;

		/** Returns true if the model supports editing spaces of multiple rigs */
		virtual bool SupportsEditingMultipleRigs() const { return false; }

		/** Returns true if default items 'World' and 'Parent' should be generated */
		virtual bool ShouldGenerateDefaultItems() const { return true; }

		/** Returns true if items from control customizations should be generated */
		virtual bool ShouldGenerateFavoriteItems() const { return true; }

		/** Returns true if additional items should be generated */
		virtual bool ShouldGenerateAdditionalItems() const { return true; }

		/** Can be implemented to provide additional spaces from external sources (e.g. sequencer channel) */
		virtual TArray<FRigElementKeyWithLabel> GatherAdditionalSpaces(URigHierarchy* Hierarchy, const FRigElementKey& ControlKey) const { return {}; }

		/** Returns current hierarchies with their control keys for which items are generated */
		const TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>& GetWeakHierarchyToControlKeysMap() const { return WeakHierarchyToControlKeysMap; }
				
		DECLARE_EVENT(FRigSpacePickerModelBase, FRigSpacePickerModelRequestRefreshMVVM)
		/** Event raised when model and view needs refresh */
		FRigSpacePickerModelRequestRefreshMVVM OnRequestRefreshMVVM;

	protected:
		//~ Begin FSelfRegisteringEditorUndoClient interface
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;
		//~ End FSelfRegisteringEditorUndoClient interface

		/** Default implementation to determine if spaces can be moved */
		bool CanMoveSpacesImpl(const TSharedRef<FRigSpacePickerItem>& Item, const ERigSpacePickerMoveSpaceDirection Direction) const;

	private:
		/** Caches if space switching restricted into the bIsSpaceSwitchingRestricted member */
		void CacheIsSpaceSwitchingRestricted();

		/** Called when the current hierarchy was modified */
		void OnHierarchyModified(
			ERigHierarchyNotification InNotif,
			URigHierarchy* InHierarchy,
			const FRigNotificationSubject& InSubject);

		using FHierarchyToControlFunctionSignature = void(URigHierarchy*, const FRigElementKey&);
		/** Executes function for each control key */
		void ForEachControlKey(TFunctionRef<FHierarchyToControlFunctionSignature> Function) const;

		using FHierarchyToControlFunctionWithBreakSignature = bool(URigHierarchy*, const FRigElementKey&);
		/** Executes function for each control key, breaks when function returns false */
		void ForEachControlKeyWithBreak(TFunctionRef<FHierarchyToControlFunctionWithBreakSignature> Function) const;

		/** True if one or more controls restrict space switching */
		bool bCachedIsSpaceSwitchingRestricted = false;

		/** The current items */
		TArray<TSharedPtr<FRigSpacePickerItem>> Items;

		/** The current hierarchies with their control keys for which items are generated */
		TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>> WeakHierarchyToControlKeysMap;
	};
}
