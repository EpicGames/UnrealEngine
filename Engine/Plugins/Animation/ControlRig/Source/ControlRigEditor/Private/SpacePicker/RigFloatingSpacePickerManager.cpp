// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigFloatingSpacePickerManager.h"

#include "ControlRig.h"
#include "Misc/RigEditModeUtils.h"
#include "SpacePicker/Models/RigAssetEditorFloatingSpacePickerModel.h"
#include "SpacePicker/Models/RigLevelEditorFloatingSpacePickerModel.h"
#include "SpacePicker/Views/SRigSpacePickerDialog.h"
#include "SpacePicker/Views/SRigSpacePickerMenuEntry.h"

namespace UE::ControlRigEditor
{
	namespace FloatingSpacePickerManagerDetails
	{
		/** Creates a view model depending on this being a level or asset editor. Returns nullptr if no valid model can be created */
		TSharedPtr<FRigSpacePickerModelBase> CreateViewModel(
			const FControlRigEditMode& OwningMode,
			const FInitialSpacePickerSelection& InitialSelection)
		{
			if (!InitialSelection.IsValid())
			{
				return nullptr;
			}
			
			const TSharedRef<FRigSpacePickerModelBase> ViewModel = [&InitialSelection, &OwningMode]() -> TSharedRef<FRigSpacePickerModelBase>
				{
					if (OwningMode.AreEditingControlRigDirectly())
					{
						return MakeShared<FRigAssetEditorFloatingSpacePickerModel>();
					}
					else
					{
						return MakeShared<FRigLevelEditorFloatingSpacePickerModel>();
					}
				}();

			ViewModel->Update(InitialSelection.WeakHierarchyToControlsMap);

			return ViewModel;
		}
	}

	FRigFloatingSpacePickerManager::FRigFloatingSpacePickerManager(FControlRigEditMode& InOwningMode)
		: OwningMode(InOwningMode)
	{
	}

	FRigFloatingSpacePickerManager::~FRigFloatingSpacePickerManager()
	{
		TryCloseDialog();
	}

	void FRigFloatingSpacePickerManager::SummonSpacePickerAtCursor()
	{
		// We could reuse the window but it's easier to just recreate the widget from scratch. The overhead is fine.
		TryCloseDialog();

		if (const TSharedPtr<SRigSpacePickerDialog> SpacePickerDialog = CreateSpacePickerDialog())
		{
			constexpr bool bModal = false;
			WeakWindow = SpacePickerDialog->OpenDialog(bModal);
		}
	}

	TSharedPtr<SRigSpacePickerMenuEntry> FRigFloatingSpacePickerManager::CreateSpacePickerMenuEntry()
	{
		using namespace UE::ControlRigEditor;

		const FInitialSpacePickerSelection InitialSelection = DetermineInitialSpacePickerSelection(OwningMode);
		ViewModel = FloatingSpacePickerManagerDetails::CreateViewModel(OwningMode, InitialSelection);
		if (ViewModel.IsValid())
		{
			return SNew(SRigSpacePickerMenuEntry, ViewModel.ToSharedRef());
		}

		return nullptr;
	}

	TSharedPtr<SRigSpacePickerDialog> FRigFloatingSpacePickerManager::CreateSpacePickerDialog()
	{
		using namespace UE::ControlRigEditor;

		const FInitialSpacePickerSelection InitialSelection = DetermineInitialSpacePickerSelection(OwningMode);
		ViewModel = FloatingSpacePickerManagerDetails::CreateViewModel(OwningMode, InitialSelection);
		if (ViewModel.IsValid())
		{
			const FText Title = NSLOCTEXT("SRigSpacePickerDialog", "SpacePickerDialogTitle", "Switch Space");
			return SNew(SRigSpacePickerDialog, ViewModel.ToSharedRef())
				.Title(Title);
		}

		return nullptr;
	}

	void FRigFloatingSpacePickerManager::TryCloseDialog()
	{
		if (const TSharedPtr<SWindow> PinnedWindow = WeakWindow.Pin())
		{
			PinnedWindow->RequestDestroyWindow();
			WeakWindow.Reset();
		}

		ViewModel.Reset();
	}
}
