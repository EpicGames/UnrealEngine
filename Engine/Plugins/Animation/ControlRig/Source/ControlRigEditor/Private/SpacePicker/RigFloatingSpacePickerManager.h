// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditMode/ControlRigEditMode.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ControlRigEditor
{
	class FRigSpacePickerModelBase;
	class SRigSpacePickerDialog;
	class SRigSpacePickerMenuEntry;

	/** Manages the floating space picker, which by default is spawned by pressing tab. */
	class FRigFloatingSpacePickerManager 
		: public FNoncopyable
	{
	public:
		explicit FRigFloatingSpacePickerManager(FControlRigEditMode& InOwningMode UE_LIFETIMEBOUND);
		
		~FRigFloatingSpacePickerManager();

		/** Spawns a new space picker at the cursor. Replaces the old one if one exists. */
		void SummonSpacePickerAtCursor();

		/** Creates a space picker menu entry */
		TSharedPtr<SRigSpacePickerMenuEntry> CreateSpacePickerMenuEntry();

	private:
		/** Creates a space picker dialog */
		TSharedPtr<SRigSpacePickerDialog> CreateSpacePickerDialog();

		/** Tries to close a currently open dialog */
		void TryCloseDialog();

		/** The edit mode that owns us. Needed to display some information about the space picker. */
		FControlRigEditMode& OwningMode;

		/** The window that contained the last created space picker. If still valid, we replace the window. */
		TWeakPtr<SWindow> WeakWindow;

		/** Model for the currently active floating manager */
		TSharedPtr<FRigSpacePickerModelBase> ViewModel;
	};
}
