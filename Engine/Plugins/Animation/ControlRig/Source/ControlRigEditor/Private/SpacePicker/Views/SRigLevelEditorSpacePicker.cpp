// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigLevelEditorSpacePicker.h"

#include "ControlRig.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditMode/Models/RigSelectionViewModel.h"
#include "Editor.h"
#include "SpacePicker/Models/RigLevelEditorSpacePickerModel.h"
#include "SpacePicker/Views/SRigSpacePicker.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "ControlRigSpacePicker"

namespace UE::ControlRigEditor
{
	void SRigLevelEditorSpacePicker::Construct(
		const FArguments& InArgs, 
		FControlRigEditMode& InEditMode, 
		const TSharedRef<UE::ControlRigEditor::FRigSelectionViewModel>& InSelectionViewModel)
	{
		SelectionViewModel = InSelectionViewModel;
		InSelectionViewModel->OnControlSelected().AddSP(this, &SRigLevelEditorSpacePicker::HandleControlSelected);
		InSelectionViewModel->OnControlsChanged().AddSP(this, &SRigLevelEditorSpacePicker::RequestRefresh);

		SpacePickerViewModel = MakeShared<FRigLevelEditorSpacePickerModel>();
		SpacePickerViewModel->Update(InArgs._InitialSelection);

		ChildSlot
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.AutoSize()
			[
				SNew(SRigSpacePicker, SpacePickerViewModel.ToSharedRef())
			]
		];

		SetEditMode(InEditMode);
	}

	SRigLevelEditorSpacePicker::~SRigLevelEditorSpacePicker()
	{
		if (SelectionViewModel.IsValid())
		{
			SelectionViewModel->OnControlSelected().RemoveAll(this);
			SelectionViewModel->OnControlsChanged().RemoveAll(this);
		}
	}

	void SRigLevelEditorSpacePicker::HandleControlSelected(
		UControlRig* const Subject,
		FRigControlElement* const ControlElement,
		const bool bSelected)
	{
		if (!Subject ||
			!ControlElement)
		{
			return;
		}

		if (IsInGameThread())
		{
			FControlRigBaseDockableView::HandleControlSelected(Subject, ControlElement, bSelected);
			RequestRefresh();
		}
		else
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION,
				[This = SharedThis(this),
				 WeakSubject = TWeakObjectPtr<UControlRig>(Subject),
				 Key = ControlElement->GetKey(),
				 bSelected]()
				{
					UControlRig* const PinnedSubject = WeakSubject.Get();
					FRigBaseElement* const BaseElement = PinnedSubject ? PinnedSubject->GetHierarchy()->Find(Key) : nullptr;

					if (PinnedSubject && BaseElement)
					{
						if (FRigControlElement* const ResolvedElement = Cast<FRigControlElement>(BaseElement))
						{
							This->FControlRigBaseDockableView::HandleControlSelected(PinnedSubject, ResolvedElement, bSelected);
							This->RequestRefresh();
						}
					}
				});
		}
	}

	void SRigLevelEditorSpacePicker::RequestRefresh()
	{
		const auto ForceRefreshLambda = 
			[WeakThis = AsWeak(), this]()
			{
				if (WeakThis.IsValid())
				{
					TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>> HierarchyToControlsMap;
					for (UControlRig* ControlRig : GetControlRigs())
					{
						if (ControlRig)
						{
							URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
							TArray<FRigElementKey> SelectedControls = Hierarchy->GetSelectedKeys(ERigElementType::Control);
							if (!SelectedControls.IsEmpty())
							{
								HierarchyToControlsMap.Add(Hierarchy, MoveTemp(SelectedControls));
							}
						}
					}

					SpacePickerViewModel->Update(HierarchyToControlsMap);

					PendingRefreshHandle.Invalidate();
				}
			};

		if (!PendingRefreshHandle.IsValid() && GEditor && GEditor->IsTimerManagerValid())
		{
			PendingRefreshHandle = GEditor->GetTimerManager()->SetTimerForNextTick(
				FTimerDelegate::CreateLambda(ForceRefreshLambda)
			);
		}
	}
}

#undef LOCTEXT_NAMESPACE
