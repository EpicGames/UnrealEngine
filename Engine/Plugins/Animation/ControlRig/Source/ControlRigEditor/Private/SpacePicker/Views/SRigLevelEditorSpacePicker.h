// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditMode/ControlRigBaseDockableView.h"
#include "Engine/TimerHandle.h"
#include "Rigs/RigHierarchyDefines.h"
#include "Widgets/SCompoundWidget.h"

class URigHierarchy;

namespace UE::ControlRigEditor 
{ 
	class FRigLevelEditorSpacePickerModel;
	class FRigSelectionViewModel;

	class SRigLevelEditorSpacePicker
		: public SCompoundWidget
		, public FControlRigBaseDockableView
	{
	public:
		using FWeakHierarchytoControlsMap = TMap<TWeakObjectPtr<URigHierarchy>, TArray<FRigElementKey>>;

		SLATE_BEGIN_ARGS(SRigLevelEditorSpacePicker)
			{}

			/** The initially selected hierarchy. */
			SLATE_ARGUMENT(FWeakHierarchytoControlsMap, InitialSelection)

		SLATE_END_ARGS()

		virtual ~SRigLevelEditorSpacePicker() override;

		void Construct(
			const FArguments& InArgs,
			FControlRigEditMode& InEditMode, 
			const TSharedRef<FRigSelectionViewModel>& InSelectionViewModel);

	protected:
		//~ Begin FControlRigBaseDockableView interface
		virtual void HandleControlSelected(UControlRig* Subject, FRigControlElement* InControl, bool bSelected) override;
		virtual TSharedRef<FControlRigBaseDockableView> AsSharedWidget() override { return SharedThis(this); }
		//~ End FControlRigBaseDockableView interface

	private:
		/** Updates this widget on the next tick */
		void RequestRefresh();

		/** View model for when control rig selection changes. */
		TSharedPtr<FRigSelectionViewModel> SelectionViewModel;

		/** Model driving the space picker widget */
		TSharedPtr<FRigLevelEditorSpacePickerModel> SpacePickerViewModel;

		/** Timer handle to refresh this widget */
		FTimerHandle PendingRefreshHandle;
	};
}
