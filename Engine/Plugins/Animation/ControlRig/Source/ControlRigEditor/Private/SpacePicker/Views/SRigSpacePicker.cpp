// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigSpacePicker.h"

#include "ControlRigEditorStyle.h"
#include "DetailLayoutBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SActionButton.h"
#include "SpacePicker/Models/RigSpacePickerItem.h"
#include "SpacePicker/Models/RigSpacePickerModelBase.h"
#include "SpacePicker/Widgets/SRigSpacePickerAddSpaceButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SRigSpacePicker"

namespace UE::ControlRigEditor
{
	void SRigSpacePicker::Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InModel)
	{
		Model = InModel;
		Model->OnRequestRefreshMVVM.AddSP(this, &SRigSpacePicker::OnRequestRefreshMVVM);

		// Get items now in case the model was updated before constructing this view
		Items = Model->GetItems();

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SAssignNew(TopLevelListBox, SVerticalBox)
			]
		];

		TryAddTitle(InArgs._Title);

		// Add a list view for spaces
		TopLevelListBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			.Padding(0.0)
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([this]()
					{
						return
							Model->SupportsEditingMultipleRigs() ||
							Model->GetWeakHierarchyToControlKeysMap().Num() == 1 ?
							EVisibility::Visible :
							EVisibility::Collapsed;
					})
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.Padding(0)
				[
					SAssignNew(SpacesList, SListView<TSharedPtr<FRigSpacePickerItem>>)
					.SelectionMode(ESelectionMode::None) // The space label drives selection
					.ListItemsSource(&Items)
					.OnGenerateRow(this, &SRigSpacePicker::OnGenerateItemRow)
					.OnSelectionChanged(this, &SRigSpacePicker::OnItemSelected)						
				]
			];

		if (!Model->SupportsEditingMultipleRigs())
		{
			// Add a note that space switching is only supported for single rigs currently
			TopLevelListBox->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.Padding(0.0)
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]()
						{
							return
								Model->GetWeakHierarchyToControlKeysMap().Num() > 1 ?
								EVisibility::Visible :
								EVisibility::Collapsed;
						})
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Fill)
					.Padding(8.f)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Justification(ETextJustify::Center)
						.Text(LOCTEXT("HintCannotSpaceSwitchControlsFromMultipleRigs", "Controls from multiple rigs selected.\n\nEditing spaces of multiple rigs is currently not supported."))
					]
				];
		}

		TryCreateBottomButtons(InArgs);
	}

	void SRigSpacePicker::TryAddTitle(const FText& Title)
	{
		if (Title.IsEmpty())
		{
			return;
		}

		if (TopLevelListBox.IsValid())
		{
			TopLevelListBox->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(8.f, 4.f)
				[
					SNew(STextBlock)
					.Text(Title)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				];
		}
	}

	void SRigSpacePicker::OnItemSelected(TSharedPtr<FRigSpacePickerItem> Item, ESelectInfo::Type SelectInfo)
	{
		IRigSpacePickerSetActiveSpacesInterface* SetActiveSpaceInterface = Model->GetSetActiveSpacesInterface();
		if (SetActiveSpaceInterface &&
			Item.IsValid())
		{
			SetActiveSpaceInterface->SetActiveSpaces(Item.ToSharedRef());
		}
	}

	void SRigSpacePicker::TryCreateBottomButtons(const FArguments& InArgs)
	{
		const IRigSpacePickerAddSpacesInterface* const AddSpacesInterface = Model->GetAddSpacesInterface();
		const IRigSpacePickerCompensateKeysInterface* const CompensateKeysInterface = Model->GetCompensateKeysInterface();
		const IRigSpacePickerSupportsBakeDialogInterface* const SupportsBakingInterface = Model->GetSupportsBakeDialogInterface();
		if (!AddSpacesInterface &&
			!CompensateKeysInterface &&
			!SupportsBakingInterface)
		{
			return;
		}

		const TSharedRef<SHorizontalBox> BottomButtonsListBox = 
			SNew(SHorizontalBox)
			.Visibility_Lambda([this]()
				{
					return
						Model->GetWeakHierarchyToControlKeysMap().Num() == 1 ?
						EVisibility::Visible :
						EVisibility::Collapsed;
				});

		if (AddSpacesInterface)
		{
			BottomButtonsListBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(0.f)
				[
					SNew(SRigSpacePickerAddSpaceButton, Model.ToSharedRef())
					.OnIsMenuOpenChanged(this, &SRigSpacePicker::OnIsAddMenuOpenChanged)
				];
		}

		if (AddSpacesInterface &&
			(CompensateKeysInterface || SupportsBakingInterface))
		{
			BottomButtonsListBox->AddSlot()
				.FillWidth(1.f)
				.HAlign(HAlign_Fill)
				[
					SNew(SSpacer)
				];
		}

		if (CompensateKeysInterface)
		{
			BottomButtonsListBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 0.f)
				[
					SNew(SActionButton)
					.Text(LOCTEXT("CompensateKeyButton", "Comp Key"))
					.OnClicked_Lambda([this]()
						{
							if (IRigSpacePickerCompensateKeysInterface* const CompensateKeysInterface = Model->GetCompensateKeysInterface())
							{
								CompensateKeysInterface->CompensateKeys();
							}

							return FReply::Handled();
						})
					.IsEnabled_Lambda([this]() 
						{ 
							if (const IRigSpacePickerCompensateKeysInterface* const CompensateKeysInterface = Model->GetCompensateKeysInterface())
							{
								return CompensateKeysInterface->CanCompensateKeys();
							}
							return false;
						})
					.ToolTipText(LOCTEXT("CompensateKeyTooltip", "Compensate key at the current time."))
				];

			BottomButtonsListBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 0.f)
				[
					SNew(SActionButton)
					.Text(LOCTEXT("CompensateAllButton", "Comp All"))
					.OnClicked_Lambda([this]()
						{
							if (IRigSpacePickerCompensateKeysInterface* const CompensateKeysInterface = Model->GetCompensateKeysInterface())
							{
								CompensateKeysInterface->CompensateAllKeys();
							}

							return FReply::Handled();
						})
					.IsEnabled_Lambda([this]() 
						{
							if (const IRigSpacePickerCompensateKeysInterface* const CompensateKeysInterface = Model->GetCompensateKeysInterface())
							{
								return CompensateKeysInterface->CanCompensateAllKeys();
							}
							return false;
						})
					.ToolTipText(LOCTEXT("CompensateAllTooltip", "Compensate all space switch keys."))
				];
		}

		if (SupportsBakingInterface)
		{
			BottomButtonsListBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 0.f)
				[
					SNew(SActionButton)
					.Text(LOCTEXT("BakeButton", "Bake..."))
					.OnClicked_Lambda([this]()
						{
							if (IRigSpacePickerSupportsBakeDialogInterface* const SupportsBakingInterface = Model->GetSupportsBakeDialogInterface())
							{
								SupportsBakingInterface->ShowBakeDialog();
							}

							return FReply::Handled();
						})
					.IsEnabled_Lambda([this]() 
						{
							if (const IRigSpacePickerSupportsBakeDialogInterface* const SupportsBakingInterface = Model->GetSupportsBakeDialogInterface())
							{
								return SupportsBakingInterface->CanShowBakeDialog();
							}
							return false;
						})
					.ToolTipText(LOCTEXT("BakeButtonToolTip", "Allows to bake the animation of one or more controls to a single space."))
				];
		}

		TopLevelListBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			.Padding(11.f, 8.f, 4.f, 4.f)
			[
				BottomButtonsListBox
			];
	}

	TSharedRef<ITableRow> SRigSpacePicker::OnGenerateItemRow(TSharedPtr<FRigSpacePickerItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		const TSharedRef<SBorder> LabelWidget =
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.OnMouseButtonUp_Lambda([Item, this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					// Drive selection from clicking on the label, so e.g. move buttons cannot possibly affect selection
					OnItemSelected(Item, ESelectInfo::OnMouseClick);

					return FReply::Handled();
				})
			[			
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(5.0, 2.0, 5.0, 2.0))
					.BorderImage_Lambda([Item]
						{
							static const FSlateBrush* RoundedBoxBrush = FControlRigEditorStyle::Get().GetBrush(TEXT("ControlRig.SpacePicker.RoundedRect"));

							const bool bShowBoder =
								Item->GetActiveState() != ERigSpacePickerItemActiveState::Inactive ||
								Item->IsFlashing();

							return bShowBoder ?	
								RoundedBoxBrush : 
								FStyleDefaults::GetNoBrush();
						})
					.BorderBackgroundColor_Lambda([Item]
						{
							return Item->GetColor();
						})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(FMargin(0.f, 0.f, 3.f, 0.f))
						[
							SNew(SImage)
							.Image(Item->GetIconBrush())
							.ColorAndOpacity(FSlateColor::UseForeground())
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(0.f)
						[
							SNew(STextBlock)
							.Text(Item->GetDisplayName())
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ToolTipText(Item->GetTooltip())
						]
					]
				]
			];

		const TSharedRef<SHorizontalBox> RowBox = SNew(SHorizontalBox);
			
		RowBox->AddSlot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(0.f)
			[
				LabelWidget
			];

		if (!Item->IsDefaultSpace())
		{
			const TAttribute<EVisibility> RestrictedVisibility = TAttribute<EVisibility>::CreateLambda([this]
				{
					return Model->IsSpaceSwitchingRestricted() ? EVisibility::Collapsed : EVisibility::Visible;
				});

			const IRigSpacePickerMoveSpacesInterface* MoveSpacesInterface = Model->GetMoveSpacesInterface();
			const IRigSpacePickerDeleteSpacesInterface* DeleteSpacesInterface = Model->GetDeleteSpacesInterface();

			if (MoveSpacesInterface)
			{
				RowBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(0.f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.ContentPadding(0.f)
						.OnClicked_Lambda(
							[this, Item, Direction = ERigSpacePickerMoveSpaceDirection::Up]()
							{
								IRigSpacePickerMoveSpacesInterface* MoveSpacesInterface = Model->GetMoveSpacesInterface();
								if (MoveSpacesInterface &&
									Item.IsValid())
								{
									MoveSpacesInterface->MoveSpaces(Item.ToSharedRef(), Direction);
								}
								return FReply::Handled();
							})
						.IsEnabled_Lambda(
							[this, Item, Direction = ERigSpacePickerMoveSpaceDirection::Up]()
							{
								IRigSpacePickerMoveSpacesInterface* MoveSpacesInterface = Model->GetMoveSpacesInterface();
								if (MoveSpacesInterface &&
									Item.IsValid())
								{
									return MoveSpacesInterface->CanMoveSpaces(Item.ToSharedRef(), Direction);
								}
								return false;
							})
						.ToolTipText(LOCTEXT("MoveSpaceUp", "Move this space up in the list."))
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.ChevronUp"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						.Visibility(RestrictedVisibility)
					];

				RowBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(0.f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.ContentPadding(0.f)
						.OnClicked_Lambda(
							[this, Item, Direction = ERigSpacePickerMoveSpaceDirection::Down]()
							{
								IRigSpacePickerMoveSpacesInterface* MoveSpacesInterface = Model->GetMoveSpacesInterface();
								if (MoveSpacesInterface &&
									Item.IsValid())
								{
									MoveSpacesInterface->MoveSpaces(Item.ToSharedRef(), Direction);
								}
								return FReply::Handled();
							})
						.IsEnabled_Lambda(
							[this, Item, Direction = ERigSpacePickerMoveSpaceDirection::Down]()
							{
								IRigSpacePickerMoveSpacesInterface* MoveSpacesInterface = Model->GetMoveSpacesInterface();
								if (MoveSpacesInterface &&
									Item.IsValid())
								{
									return MoveSpacesInterface->CanMoveSpaces(Item.ToSharedRef(), Direction);
								}
								return false;
							})
						.ToolTipText(LOCTEXT("MoveSpaceDown", "Move this space down in the list."))
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.ChevronDown"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						.Visibility(RestrictedVisibility)
					];
			}

			if (DeleteSpacesInterface)
			{
				const auto OnDeleteSpaces = [Item, this]()
					{
						IRigSpacePickerDeleteSpacesInterface* DeleteSpacesInterface = Model->GetDeleteSpacesInterface();
						if (DeleteSpacesInterface &&
							Item.IsValid())
						{
							return DeleteSpacesInterface->DeleteSpaces(Item.ToSharedRef());
						}
					};

				const auto IsDeleteSpacesEnabled = [Item, this]()
					{
						const IRigSpacePickerDeleteSpacesInterface* DeleteSpacesInterface = Model->GetDeleteSpacesInterface();
						if (DeleteSpacesInterface &&
							Item.IsValid())
						{
							return DeleteSpacesInterface->CanDeleteSpaces(Item.ToSharedRef());
						}
						return false;
					};

				const TSharedRef<SWidget> DeleteButton = PropertyCustomizationHelpers::MakeClearButton(
					FSimpleDelegate::CreateLambda(OnDeleteSpaces),
					LOCTEXT("DeleteSpace", "Remove this space."), 
					TAttribute<bool>::CreateLambda(IsDeleteSpacesEnabled)
				);

				DeleteButton->SetVisibility(RestrictedVisibility);

				RowBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(0)
					[
						DeleteButton
					];
			}
		}

		return
			SNew(STableRow<TSharedPtr<FRigSpacePickerItem>>, OwnerTable)
			.Style(FControlRigEditorStyle::Get(), "ControlRig.SpacePicker.TableRowStyle")
			[
				RowBox
			];
	}

	void SRigSpacePicker::OnRequestRefreshMVVM()
	{
		if (SpacesList.IsValid())
		{
			Model->GenerateItems();
			Items = Model->GetItems();

			SpacesList->RequestListRefresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
