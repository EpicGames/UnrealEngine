// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FRigSpacePickerItem;
class ITableRow;
class STableViewBase;
class SVerticalBox;
class URigHierarchy;
template <typename ItemType> class SListView;

namespace UE::ControlRigEditor
{
	class FRigSpacePickerModelBase;
	class FRigSpacePickerItem;
	enum class ERigSpacePickerMoveSpaceDirection : uint8;

	/** A generic widget to edit spaces of controls in control rigs, feature set depends on the model */
	class SRigSpacePicker
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRigSpacePicker) 
			{}

			/** The title for this widget */
			SLATE_ARGUMENT(FText, Title)
						
		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InModel);

	protected:
		/** Called when an item was selected */
		virtual void OnItemSelected(TSharedPtr<FRigSpacePickerItem> Item, ESelectInfo::Type SelectInfo);

		/** Called when the add menu is opened or closed */
		virtual void OnIsAddMenuOpenChanged(bool bIsOpen) {}

	private:
		/** Tries to add a title widget, fails quietly if no title is set */
		void TryAddTitle(const FText& Title);

		/** Tries to create buttons to operate on spaces at the bottom of the widget */
		void TryCreateBottomButtons(const FArguments& InArgs);

		/** Called when an item row is generated */
		TSharedRef<ITableRow> OnGenerateItemRow(TSharedPtr<FRigSpacePickerItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

		/** Called when spaces changed */
		void OnRequestRefreshMVVM();

		/** Top level box containing all content */
		TSharedPtr<SVerticalBox> TopLevelListBox;

		/** Vertical box containing spaces */
		TSharedPtr<SListView<TSharedPtr<FRigSpacePickerItem>>> SpacesList;

		/** A model for this view */
		TSharedPtr<FRigSpacePickerModelBase> Model;

		/** Items displayed in this view */
		TArray<TSharedPtr<FRigSpacePickerItem>> Items;
	};
}
