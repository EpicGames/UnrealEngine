// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigSpacePickerBakeDialog.h"

#include "Editor.h"
#include "ISequencer.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "SpacePicker/Models/RigLevelEditorBakeDialogSpacePickerModel.h"
#include "SpacePicker/Views/SRigSpacePickerDialog.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SRigSpacePickerBakeDialog"

namespace UE::ControlRigEditor
{	
	void SRigSpacePickerBakeDialog::Construct(const FArguments& InArgs)
	{
		if (!InArgs._WeakHierarchy.IsValid() ||
			InArgs._Controls.IsEmpty())
		{
			return;
		}

		ViewModel = MakeShared<FRigLevelEditorBakeDialogSpacePickerModel>(InArgs._Settings);
		ViewModel->Update(InArgs._WeakHierarchy, InArgs._Controls);

		const TSharedPtr<ISequencer> Sequencer = ViewModel->GetSequencer();
		if (!Sequencer.IsValid())
		{
			ViewModel.Reset();
			return;
		}

		FStructureDetailsViewArgs StructureDetailsViewArgs;
		StructureDetailsViewArgs.bShowObjects = true;
		StructureDetailsViewArgs.bShowAssets = true;
		StructureDetailsViewArgs.bShowClasses = true;
		StructureDetailsViewArgs.bShowInterfaces = true;

		FDetailsViewArgs ViewArgs;
		ViewArgs.bAllowSearch = false;
		ViewArgs.bHideSelectionTip = false;
		ViewArgs.bShowObjectLabel = false;

		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		DetailsView = PropertyEditor.CreateStructureDetailView(ViewArgs, StructureDetailsViewArgs, TSharedPtr<FStructOnScope>());

		DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber",
			FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer.Get(), &ISequencer::MakeFrameNumberDetailsCustomization));

		DetailsView->SetStructureData(ViewModel->GetSettingsStructOnScope());

		ChildSlot
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SRigSpacePickerDialog, ViewModel.ToSharedRef())
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 0.f, 0.f, 0.f)
				[
					DetailsView->GetWidget().ToSharedRef()
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 16.f, 0.f, 16.f)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SSpacer)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(8.f, 0.f, 0.f, 0.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text(LOCTEXT("OK", "OK"))
						.OnClicked_Lambda([this, InArgs]()
						{
							ViewModel->PerformBake();
							CloseDialog();

							return FReply::Handled();
						})
						.IsEnabled_Lambda([this]()
						{
							return ViewModel->CanPerformBake();
						})
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(8.f, 0.f, 16.f, 0.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.Text(LOCTEXT("Cancel", "Cancel"))
						.OnClicked_Lambda([this]()
						{
							CloseDialog();
							return FReply::Handled();
						})
					]
				]
			]
		];
	}

	FReply SRigSpacePickerBakeDialog::OpenDialog(bool bModal)
	{
		if (!ViewModel.IsValid())
		{
			return FReply::Handled();
		}

		ensure(!DialogWindow.IsValid());
		
		const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

		const TSharedRef<SWindow> Window = SNew(SWindow)
			.Title( LOCTEXT("SRigSpacePickerBakeDialogTitle", "Bake Controls To Specified Space") )
			.CreateTitleBar(true)
			.Type(EWindowType::Normal)
			.SizingRule(ESizingRule::Autosized)
			.ScreenPosition(CursorPos)
			.FocusWhenFirstShown(true)
			.ActivationPolicy(EWindowActivationPolicy::FirstShown)
			[
				AsShared()
			];
	
		Window->SetWidgetToFocusOnActivate(AsShared());
	
		DialogWindow = Window;

		Window->MoveWindowTo(CursorPos);

		if(bModal)
		{
			GEditor->EditorAddModalWindow(Window);
		}
		else
		{
			FSlateApplication::Get().AddWindow( Window );
		}

		return FReply::Handled();
	}

	void SRigSpacePickerBakeDialog::CloseDialog()
	{
		if (DialogWindow.IsValid())
		{
			DialogWindow.Pin()->RequestDestroyWindow();
			DialogWindow.Reset();
		}
	}
}

#undef LOCTEXT_NAMESPACE
