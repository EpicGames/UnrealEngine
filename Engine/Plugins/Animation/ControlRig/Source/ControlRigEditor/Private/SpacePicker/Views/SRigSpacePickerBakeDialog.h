// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigSpacePickerBakeSettings.h"
#include "Widgets/SCompoundWidget.h"

class ISequencer;
class IStructureDetailsView;
class SRigSpacePicker;
class URigHierarchy;
struct FRigElementKey;

namespace UE::ControlRigEditor
{
	class FRigLevelEditorBakeDialogSpacePickerModel;

	/** Dialog for baking controls to a space */
	class SRigSpacePickerBakeDialog 
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRigSpacePickerBakeDialog)
			{}

			SLATE_ARGUMENT(TWeakObjectPtr<URigHierarchy>, WeakHierarchy)
			SLATE_ARGUMENT(TArray<FRigElementKey>, Controls)
			SLATE_ARGUMENT(FRigSpacePickerBakeSettings, Settings)

		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs);

		/** Opens the bake dialog */
		FReply OpenDialog(bool bModal = true);

		/** Closes the bake dialog */
		void CloseDialog();

	private:
		/** The dialog window, valid while the dialog is open */
		TWeakPtr<SWindow> DialogWindow;

		/** Details view used to display bake settings */
		TSharedPtr<IStructureDetailsView> DetailsView;

		/** A model for the space picker view displayed in this bake dialog */
		TSharedPtr<FRigLevelEditorBakeDialogSpacePickerModel> ViewModel;
	};
}
