// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigSpacePickerDialog.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SRigSpacePickerDialog"

namespace UE::ControlRigEditor
{
	void SRigSpacePickerDialog::Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InModel)
	{
		SRigSpacePicker::Construct(InArgs, InModel);
	}

	TSharedPtr<SWindow> SRigSpacePickerDialog::OpenDialog(bool bModal)
	{
		ensure(!WeakDialogWindow.IsValid());

		WeakWidgetToFocusOnDialogClosed = FSlateApplication::Get().GetKeyboardFocusedWidget();

		const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

		const TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("SpacePickerWindowTitle", "Pick a new space"))
			.CreateTitleBar(false)
			.Type(EWindowType::Menu)
			.IsPopupWindow(true) // the window automatically closes when user clicks outside of it
			.SizingRule(ESizingRule::Autosized)
			.ScreenPosition(CursorPos)
			.FocusWhenFirstShown(true)
			.ActivationPolicy(EWindowActivationPolicy::FirstShown)
			[
				AsShared()
			];

		Window->SetWidgetToFocusOnActivate(AsShared());
		Window->GetOnWindowDeactivatedEvent().AddSP(this, &SRigSpacePickerDialog::OnWindowDeactivated);

		WeakDialogWindow = Window;

		Window->MoveWindowTo(CursorPos);

		if (bModal)
		{
			GEditor->EditorAddModalWindow(Window);
		}
		else
		{
			FSlateApplication::Get().AddWindow(Window);
		}

		return Window;
	}

	void SRigSpacePickerDialog::CloseDialog()
	{
		if (TSharedPtr<SWindow> DialogWindowShared = WeakDialogWindow.Pin())
		{
			WeakDialogWindow.Reset(); // we have to reset before calling request destroy window, or an infinite recursion will happen on Mac
			DialogWindowShared->GetOnWindowDeactivatedEvent().RemoveAll(this);
			DialogWindowShared->RequestDestroyWindow();

			// Restore keyboard focus to what was focused before the dialog was opened
			FSlateApplication::Get().SetKeyboardFocus(WeakWidgetToFocusOnDialogClosed.Pin());
		}
	}

	bool SRigSpacePickerDialog::SupportsKeyboardFocus() const
	{
		return true;
	}

	FReply SRigSpacePickerDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			if (WeakDialogWindow.IsValid())
			{
				CloseDialog();
			}
		}

		return SRigSpacePicker::OnKeyDown(MyGeometry, InKeyEvent);
	}

	void SRigSpacePickerDialog::OnItemSelected(TSharedPtr<FRigSpacePickerItem> Item, ESelectInfo::Type SelectInfo)
	{
		SRigSpacePicker::OnItemSelected(Item, SelectInfo);

		if (WeakDialogWindow.IsValid())
		{
			if (!Item.IsValid())
			{
				CloseDialog();
			}
			else
			{
				// Close the window slightly deferred, allowing the user see the selection
				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
					[Time = 0.f, WeakThis = SharedThis(this).ToWeakPtr()](float DeltaTime) mutable
					{
						if (const TSharedPtr<SRigSpacePickerDialog> This = WeakThis.Pin())
						{
							Time += DeltaTime;
							if (Time > .25f)
							{
								if (This->WeakDialogWindow.IsValid())
								{
									This->CloseDialog();
								}

								return false;
							}

							return true;
						}

						return false;
					})
				);
			}
		}
	}

	void SRigSpacePickerDialog::OnIsAddMenuOpenChanged(const bool bIsOpen)
	{
		bIsAddMenuOpen = bIsOpen;
	}

	void SRigSpacePickerDialog::OnWindowDeactivated()
	{
		// Do not reset if we lost focus because of opening the context menu
		if (!bIsAddMenuOpen)
		{
			CloseDialog();
		}
	}
}

#undef LOCTEXT_NAMESPACE
