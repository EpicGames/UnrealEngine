// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SRigSpacePicker.h"

namespace UE::ControlRigEditor
{
	/** Wraps SRigSpacePicker to be suitable as a dialog */	
	class SRigSpacePickerDialog
		: public SRigSpacePicker
	{
	public:
		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InModel);

		/** Shows this widget as a dialog */
		TSharedPtr<SWindow> OpenDialog(bool bModal = true);

		/** Closes the dialog */
		void CloseDialog();

	protected:
		//~ Begin SWidget interface
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		virtual bool SupportsKeyboardFocus() const override;
		//~ End SWidget interface

		//~ Begin SRigSpacePicker interface
		virtual void OnItemSelected(TSharedPtr<FRigSpacePickerItem> Item, ESelectInfo::Type SelectInfo) override;
		virtual void OnIsAddMenuOpenChanged(const bool bIsOpen) override;
		//~ End SRigSpacePicker interface

	private:
		/** Called when the window was deactivated */
		void OnWindowDeactivated();

		/** The dialog window if currently displayed as dialog */
		TWeakPtr<SWindow> WeakDialogWindow;

		/** The widget we want to pass keyboard focus to when this widget is displayed as a dialog */
		TWeakPtr<SWidget> WeakWidgetToFocusOnDialogClosed;

		/** True when the add menu is open */
		bool bIsAddMenuOpen = false;
	};
}
