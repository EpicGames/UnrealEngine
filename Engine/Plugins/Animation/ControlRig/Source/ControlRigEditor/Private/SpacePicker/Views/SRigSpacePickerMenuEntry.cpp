// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigSpacePickerMenuEntry.h"

#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SRigSpacePickerMenuEntry"

namespace UE::ControlRigEditor
{
	void SRigSpacePickerMenuEntry::Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InModel)
	{
		SRigSpacePicker::Construct(InArgs, InModel);
	}

	bool SRigSpacePickerMenuEntry::SupportsKeyboardFocus() const
	{
		return true;
	}

	FReply SRigSpacePickerMenuEntry::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			FSlateApplication::Get().DismissAllMenus();
		}

		return SRigSpacePicker::OnKeyDown(MyGeometry, InKeyEvent);
	}

	void SRigSpacePickerMenuEntry::OnItemSelected(TSharedPtr<FRigSpacePickerItem> Item, ESelectInfo::Type SelectInfo)
	{
		SRigSpacePicker::OnItemSelected(Item, SelectInfo);
	
		FSlateApplication::Get().DismissAllMenus();
	}
}

#undef LOCTEXT_NAMESPACE
