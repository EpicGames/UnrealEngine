// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SRigSpacePicker.h"

namespace UE::ControlRigEditor
{
	/** Wraps SRigSpacePicker to be suitable as a menu entry */
	class SRigSpacePickerMenuEntry
		: public SRigSpacePicker
	{
	public:
		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InModel);

	protected:
		//~ Begin SWidget interface
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		virtual bool SupportsKeyboardFocus() const override;
		//~ End SWidget interface

		//~ Begin SRigSpacePicker interface
		virtual void OnItemSelected(TSharedPtr<FRigSpacePickerItem> Item, ESelectInfo::Type SelectInfo) override;
		//~ End SRigSpacePicker interface
	};
}
 