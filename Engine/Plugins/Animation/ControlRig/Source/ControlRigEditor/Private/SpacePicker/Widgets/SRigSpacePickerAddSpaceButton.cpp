// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigSpacePickerAddSpaceButton.h"

#include "Algo/AllOf.h"
#include "ControlRig.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchySearchableTreeView.h"
#include "Editor/Hierarchy/Widgets/SRigHierarchyTreeView.h"
#include "Rigs/RigHierarchy.h"
#include "SpacePicker/Models/RigSpacePickerModelBase.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SRigSpacePickerAddSpaceButton"

namespace UE::ControlRigEditor
{
	namespace RigSpacePickerAddSpaceButtonDetail
	{
		/**
		 * Sort predicate for the space picker hierarchy tree.
		 * Returns true if A should be ordered before B.
		 * Priority: elements before components, then controls before other element types.
		 */
		static bool ElementsFirstThenControls(const FRigHierarchyKey& A, const FRigHierarchyKey& B)
		{
			if (A.IsElement() && B.IsComponent())
			{
				return true;
			}
			if (B.IsElement() && A.IsComponent())
			{
				return false;
			}

			if (A.IsElement() && B.IsElement())
			{
				if (A.GetElement().Type == ERigElementType::Control && B.GetElement().Type != ERigElementType::Control)
				{
					return true;
				}
				if (B.GetElement().Type == ERigElementType::Control && A.GetElement().Type != ERigElementType::Control)
				{
					return false;
				}
			}
			return A < B;
		}
	}

	void SRigSpacePickerAddSpaceButton::Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InViewModel)
	{		
		ViewModel = InViewModel;

		ChildSlot
		[
			SAssignNew(ComboButton, SPositiveActionButton)
			.OnGetMenuContent(this, &SRigSpacePickerAddSpaceButton::OnGetMenuContent)
			.OnMenuOpenChanged(InArgs._OnIsMenuOpenChanged)
			.Cursor(EMouseCursor::Default)
			.ToolTipText(this, &SRigSpacePickerAddSpaceButton::GetTooltipText)
			.Text(LOCTEXT("AddSpace.Label", "Add"))
			.IsEnabled_Lambda([this]()
				{
					if (const IRigSpacePickerAddSpacesInterface* AddSpaceInterface = ViewModel->GetAddSpacesInterface())
					{
						return AddSpaceInterface->CanAddSpace();
					}
					return false;
				})
		];
	}

	TSharedRef<SWidget> SRigSpacePickerAddSpaceButton::OnGetMenuContent()
	{
		HierarchyDisplaySettings.bShowConnectors = false;
		HierarchyDisplaySettings.bShowSockets = false;
		HierarchyDisplaySettings.bShowComponents = false;

		URigHierarchy* Hierarchy = ViewModel->TryGetSingleHierarchy();
		if (!ensureMsgf(Hierarchy, TEXT("Unexpected, trying to show menu to add spaces for multiple hierarchies. This is currently not supported")))
		{
			return SNullWidget::NullWidget;
		}

		UControlRig* ControlRig = Hierarchy ? Hierarchy->GetTypedOuter<UControlRig>() : nullptr;

		const TSharedRef<FRigDependenciesProviderForControlRig> DependencyProvider = MakeShared<FRigDependenciesProviderForControlRig>(ControlRig);

		FRigHierarchyTreeDelegates TreeDelegates;
		TreeDelegates.OnGetDisplaySettings = FOnGetRigTreeDisplaySettings::CreateSP(this, &SRigSpacePickerAddSpaceButton::GetDisplaySettingsRef);
		TreeDelegates.OnGetHierarchy = FOnGetRigTreeHierarchy::CreateSP(this, &SRigSpacePickerAddSpaceButton::GetHierarchy);
		TreeDelegates.OnMouseButtonClick = FOnRigTreeMouseButtonClick::CreateSP(this, &SRigSpacePickerAddSpaceButton::HandleClickTreeItem);
		TreeDelegates.OnRigTreeIsItemVisible = FOnRigTreeIsItemVisible::CreateSP(this, &SRigSpacePickerAddSpaceButton::IsRigTreeItemVisible, DependencyProvider);
		TreeDelegates.OnCompareKeys = FOnRigTreeCompareKeys::CreateStatic(&RigSpacePickerAddSpaceButtonDetail::ElementsFirstThenControls);
		TreeDelegates.OnGetTagDisplayMode = FOnRigTreeGetTagDisplayMode::CreateStatic([]() { return ERigHierarchyConnectorTagDisplayMode::Single; });

		constexpr const TCHAR* SpacePickerRigHierarchyViewName = TEXT("SpacePickerAddButtonRigHierarchyView");
		const TSharedRef<SRigHierarchySearchableTreeView> SearchableTreeView =
			SNew(SRigHierarchySearchableTreeView, SpacePickerRigHierarchyViewName)
			.RigTreeDelegates(TreeDelegates)
			.ForModalWindow(true);

		SearchableTreeView->GetTreeView()->RefreshTreeView(true);
		ComboButton->SetMenuContentWidgetToFocus(SearchableTreeView->GetSearchBox());

		return SearchableTreeView;
	}

	void SRigSpacePickerAddSpaceButton::HandleClickTreeItem(TSharedPtr<FRigHierarchyTreeElement> TreeElement)
	{
		URigHierarchy* Hierarchy = ViewModel->TryGetSingleHierarchy();
		if (!Hierarchy)
		{
			return;
		}

		const FRigElementKeyWithLabel AdditionalSpace = TreeElement.IsValid() ?
			FRigElementKeyWithLabel(TreeElement->Key.GetElement(), *Hierarchy->GetDisplayNameForUI(TreeElement->Key.GetElement()).ToString()) :
			FRigElementKeyWithLabel{};

		if (IRigSpacePickerAddSpacesInterface* AddSpaceInterface = ViewModel->GetAddSpacesInterface())
		{
			AddSpaceInterface->AddSpace(AdditionalSpace);
		}

		// Close menu after selection
		if (ComboButton.IsValid())
		{
			constexpr bool bIsOpen = false;
			constexpr bool bIsFocused = true;
			ComboButton->SetIsMenuOpen(bIsOpen, bIsFocused);
		}
	}

	bool SRigSpacePickerAddSpaceButton::IsRigTreeItemVisible(
		const FRigHierarchyKey& HierarchyKey,
		TSharedRef<FRigDependenciesProviderForControlRig> DependencyProvider) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SRigSpacePickerAddSpaceButton::IsRigTreeItemVisible);

		URigHierarchy* Hierarchy = ViewModel->TryGetSingleHierarchy();
		if (!ensureMsgf(Hierarchy, TEXT("Unexpected, trying to show menu to add spaces for multiple hierarchies. This is not supported")))
		{
			return false;
		}

		const FRigBaseElement* Element = Hierarchy ? Hierarchy->Find(HierarchyKey.GetElement()) : nullptr;

		if (Hierarchy && 
			Element)
		{
			const bool bValidType = [Element]()
				{
					if (Element->IsA<FRigBoneElement>() ||
						Element->IsA<FRigNullElement>())
					{
						return true;
					}
					else if (const FRigControlElement* ControlElement = Cast<FRigControlElement>(Element))
					{
						return
							ControlElement->Settings.ControlType == ERigControlType::Transform ||
							ControlElement->Settings.ControlType == ERigControlType::TransformNoScale ||
							ControlElement->Settings.ControlType == ERigControlType::EulerTransform;
					}
					return false;
				}();

			if (!bValidType)
			{
				return false;
			}

			const FRigElementKey SpaceKey = HierarchyKey.GetElement();
			const TArray<FRigElementKey>& SelectedControls = ViewModel->GetControlKeys();

			for (const FRigElementKey& SelectedControlKey : SelectedControls)
			{
				if (!Hierarchy->CanSwitchToParent(SelectedControlKey, SpaceKey, *DependencyProvider))
				{
					return false;
				}
			}

			// Hide if every selected control already has this space
			if (!SelectedControls.IsEmpty())
			{
				const bool bAllHaveSpace = Algo::AllOf(SelectedControls,
					[Hierarchy , &SpaceKey, this](const FRigElementKey& ControlKey)
					{
						return ViewModel->DoesControlHaveSpace(Hierarchy, ControlKey, SpaceKey);
					});
				if (bAllHaveSpace)
				{
					return false;
				}
			}

			return true;
		}

		return false;
	}

	bool SRigSpacePickerAddSpaceButton::IsButtonEnabled() const
	{
		if (const IRigSpacePickerAddSpacesInterface* AddSpaceInterface = ViewModel->GetAddSpacesInterface())
		{
			return AddSpaceInterface->CanAddSpace();
		}
		return false;
	}

	FText SRigSpacePickerAddSpaceButton::GetTooltipText() const
	{
		return IsButtonEnabled()
			? LOCTEXT("AddSpace.ToolTip", "Add Space")
			: LOCTEXT("AddSpace.ToolTip.NoSelection", "Select a control to add space.");
	}

	const URigHierarchy* SRigSpacePickerAddSpaceButton::GetHierarchy() const
	{
		return ViewModel->TryGetSingleHierarchy();
	}
}

#undef LOCTEXT_NAMESPACE
