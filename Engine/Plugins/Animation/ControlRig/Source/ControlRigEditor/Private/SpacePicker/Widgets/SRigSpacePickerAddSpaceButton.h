// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/Hierarchy/RigHierarchyTreeDisplaySettings.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/SCompoundWidget.h"

class FRigHierarchyTreeElement;
class SPositiveActionButton;
struct FRigDependenciesProviderForControlRig;
struct FRigHierarchyKey;

namespace UE::ControlRigEditor
{
	class FRigSpacePickerModelBase;

	/** Add Space button for the space picker */
	class SRigSpacePickerAddSpaceButton : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRigSpacePickerAddSpaceButton)
			{}

			/** Executed when the add menu is opened or closed */
			SLATE_EVENT(FOnIsOpenChanged, OnIsMenuOpenChanged)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<FRigSpacePickerModelBase>& InViewModel);

	private:
		/** @return Menu content for the combo button */
		TSharedRef<SWidget> OnGetMenuContent();

		/** Assigns the clicked space to the current control selection */
		void HandleClickTreeItem(TSharedPtr<FRigHierarchyTreeElement> TreeElement);

		/** Returns true if the rig tree item should be visible */
		bool IsRigTreeItemVisible(const FRigHierarchyKey& HierarchyKey, TSharedRef<FRigDependenciesProviderForControlRig> DependencyProvider) const;

		/** Returns true when the button should be interactive */
		bool IsButtonEnabled() const;

		/** Returns the tooltip text */
		FText GetTooltipText() const;

		/** Returns the current hierarchy */
		const URigHierarchy* GetHierarchy() const;

		/** Returns the hierarchy display settings */
		const FRigHierarchyTreeDisplaySettings& GetDisplaySettingsRef() const { return HierarchyDisplaySettings; }

		/** Combo button that opens the add space menu */
		TSharedPtr<SPositiveActionButton> ComboButton;

		/** Display settings for the rig hierarchy */
		FRigHierarchyTreeDisplaySettings HierarchyDisplaySettings;

		/** The model of the view that displays this button */
		TSharedPtr<FRigSpacePickerModelBase> ViewModel;
	};
}
