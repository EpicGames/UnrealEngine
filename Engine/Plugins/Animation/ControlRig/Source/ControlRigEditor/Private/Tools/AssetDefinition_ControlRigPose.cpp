// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_ControlRigPose.h"

#include "ContentBrowserModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ControlRig.h"
#include "EditorModeManager.h"
#include "EditMode/ControlRigEditMode.h"
#include "EditMode/SControlRigRenamePoseControls.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Tools/ControlRigPose.h"
#include "UnrealEdGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_ControlRigPose)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_AnimationAsset
{
	void ExecutePastePose(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UControlRigPoseAsset*> ControlRigPoseAssets = Context->LoadSelectedObjects<UControlRigPoseAsset>();

		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!ControlRigEditMode)
		{
			return;
		}
		
		TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
		ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
		TArray<UControlRig*> ControlRigs;
		AllSelectedControls.GenerateKeyArray(ControlRigs);
			
		for (UControlRigPoseAsset* ControlRigPoseAsset : ControlRigPoseAssets)
		{
			for (UControlRig* ControlRig : ControlRigs)
			{
				ControlRigPoseAsset->PastePose(ControlRig, false, false);
			}
		}
	}

	void ExecuteSelectControls(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UControlRigPoseAsset*> ControlRigPoseAssets = Context->LoadSelectedObjects<UControlRigPoseAsset>();

		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!ControlRigEditMode)
		{
			return;
		}

		TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
		ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
		TArray<UControlRig*> ControlRigs;
		AllSelectedControls.GenerateKeyArray(ControlRigs);

		for (UControlRigPoseAsset* ControlRigPoseAsset : ControlRigPoseAssets)
		{
			for (UControlRig* ControlRig : ControlRigs)
			{
				ControlRigPoseAsset->SelectControls(ControlRig);
			}
		}
	}

	void ExecuteUpdatePose(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UControlRigPoseAsset*> ControlRigPoseAssets = Context->LoadSelectedObjects<UControlRigPoseAsset>();

		FControlRigEditMode* ControlRigEditMode = static_cast<FControlRigEditMode*>(GLevelEditorModeTools().GetActiveMode(FControlRigEditMode::ModeName));
		if (!ControlRigEditMode)
		{
			return;
		}

		TMap<UControlRig*, TArray<FRigElementKey>> AllSelectedControls;
		ControlRigEditMode->GetAllSelectedControls(AllSelectedControls);
		TArray<UControlRig*> ControlRigs;
		AllSelectedControls.GenerateKeyArray(ControlRigs);

		for (UControlRigPoseAsset* ControlRigPoseAsset : ControlRigPoseAssets)
		{
			for (UControlRig* ControlRig : ControlRigs)
			{
				ControlRigPoseAsset->SavePose(ControlRig, false);
			}
		}
	}

	void ExecuteRenameControls(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		TArray<UControlRigPoseAsset*> ControlRigPoseAssets = Context->LoadSelectedObjects<UControlRigPoseAsset>();

		FControlRigRenameControlsDialog::RenameControls(ControlRigPoseAssets);
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, [] {
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UControlRigPoseAsset::StaticClass());

		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			{
				const TAttribute<FText> Label = LOCTEXT("ControlRigPose_PastePose", "Paste Pose");
				const TAttribute<FText> ToolTip = LOCTEXT("ControlRigPose_PastePoseTooltip", "Paste the selected pose");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecutePastePose);
				InSection.AddMenuEntry("ControlRigPose_PastePose", Label, ToolTip, FSlateIcon(), UIAction);
			}
			{
				const TAttribute<FText> Label = LOCTEXT("ControlRigPose_SelectControls", "Select Controls");
				const TAttribute<FText> ToolTip = LOCTEXT("ControlRigPose_SelectControlsTooltip", "Select controls in this pose on active control rig");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteSelectControls);
				InSection.AddMenuEntry("ControlRigPose_SelectControls", Label, ToolTip, FSlateIcon(), UIAction);
			}
			{
				const TAttribute<FText> Label = LOCTEXT("ControlRigPose_UpdatePose", "Update Pose");
				const TAttribute<FText> ToolTip = LOCTEXT("ControlRigPose_UpdatePoseTooltip", "Update the pose based upon current control rig pose and selected controls");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteUpdatePose);
				InSection.AddMenuEntry("ControlRigPose_UpdatePose", Label, ToolTip, FSlateIcon(), UIAction);
			}
			{
				const TAttribute<FText> Label = LOCTEXT("ControlRigPose_RenameControls", "Rename Controls");
				const TAttribute<FText> ToolTip = LOCTEXT("ControlRigPose_RenameControlsTooltip", "Rename controls on selected poses");

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteRenameControls);
				InSection.AddMenuEntry("ControlRigPose_RenameControls", Label, ToolTip, FSlateIcon(), UIAction);
			}

		}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
