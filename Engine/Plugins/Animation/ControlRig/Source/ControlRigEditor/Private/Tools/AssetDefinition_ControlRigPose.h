// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/ControlRigPose.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_ControlRigPose.generated.h"

UCLASS()
class UAssetDefinition_ControlRigPose : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ControlRigPose", "Control Rig Pose"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(222, 128, 64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UControlRigPoseAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation };
		return Categories;
	}
	// UAssetDefinition End
};
