// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ConstraintBaker.h"

#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "ConstraintChannel.h"

#include "ISequencer.h"

#include "LevelSequence.h"
#include "MovieSceneToolHelpers.h"

#include "Constraints/MovieSceneConstraintChannelHelper.inl"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"

#define LOCTEXT_NAMESPACE "ConstraintBaker"


// at this stage, we suppose that any of these arguments are safe to use. Make sure to test them before using that function
void FConstraintBaker::GetMinimalFramesToBake(
	UWorld* InWorld,
	const UTickableTransformConstraint* InConstraint,
	const TSharedPtr<ISequencer>& InSequencer,
	UMovieSceneSection* InSection,
	const TOptional< FBakingAnimationKeySettings>& InSettings,
	TArray<FFrameNumber>& OutFramesToBake)
{
	const UMovieSceneSequence* MovieSceneSequence = InSequencer->GetFocusedMovieSceneSequence();
	const UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr;
	if (!MovieScene || !InConstraint || !InConstraint->ChildTRSHandle || !InSection)
	{
		return;
	}
	
	IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(InSection);

	// note that we might want to bake a constraint which is not animated
	FConstraintAndActiveChannel* ThisActiveChannel = ConstrainedSection->GetConstraintChannel(InConstraint->ConstraintID);
	if (!ThisActiveChannel)
	{
		return;
	}
	
	const TMovieSceneChannelData<const bool> ConstraintChannelData = ThisActiveChannel->ActiveChannel.GetData();
	// get constraint channel data if any
	TArrayView<const FFrameNumber> ConstraintFrames = ConstraintChannelData.GetTimes();
	TArrayView<const bool> ConstraintValues = ConstraintChannelData.GetValues();

	const TOptional<bool> bDefault = ThisActiveChannel->ActiveChannel.GetDefault();
	const bool bInfinite = bDefault && *bDefault;
	
	// there's no channel or an empty one and the constraint is inactive so no need to do anything if the constraint
	// is not active
	if (ConstraintFrames.IsEmpty() && !InConstraint->IsFullyActive())
	{
		if (!bInfinite)
		{
			return;
		}
	}

	// default init bounds to the scene bounds
	const bool bHasSettings = InSettings.IsSet();
	TOptional<TRange<FFrameNumber>> OptionalRange = InSequencer->GetSubSequenceRange();
	FFrameNumber MovieSceneStartFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetLowerBoundValue() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
	FFrameNumber MovieSceneEndFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetUpperBoundValue() : MovieScene->GetPlaybackRange().GetUpperBoundValue();

	FFrameNumber FirstBakingFrame = bHasSettings ? InSettings->StartFrame : MovieSceneStartFrame;
	FFrameNumber LastBakingFrame = bHasSettings ? InSettings->EndFrame : MovieSceneEndFrame;
	
	// set start to first active frame if any, if we have no range
	if (!bHasSettings)
	{
		const int32 FirstActiveIndex = ConstraintValues.IndexOfByKey(true);
		if (ConstraintValues.IsValidIndex(FirstActiveIndex))
		{
			FirstBakingFrame = ConstraintFrames[FirstActiveIndex];
		}

		// set end to the last key if inactive
		const bool bIsLastKeyInactive = ConstraintValues.IsEmpty() ? false : (ConstraintValues.Last() == false);
		if (bIsLastKeyInactive)
		{
			LastBakingFrame = ConstraintFrames.Last();
		}
	}
	// keep frames within active state
	auto IsConstraintActive = [InConstraint, ThisActiveChannel](const FFrameNumber& Time)
	{
		if (ThisActiveChannel)
		{
			bool bIsActive = false; ThisActiveChannel->ActiveChannel.Evaluate(Time, bIsActive);
			return bIsActive;
		}
		return InConstraint->IsFullyActive();
	};

	if (!bHasSettings || InSettings->BakingKeySettings != EBakingKeySettings::KeysOnly)
	{
		TArray<FFrameNumber> Frames;
		MovieSceneToolHelpers::CalculateFramesBetween(MovieScene, FirstBakingFrame, LastBakingFrame,  1 /* use increment of 1 since we reduce later*/,
			Frames);

		// Fill the frames we want to bake

		// add constraint keys
		for (const FFrameNumber& ConstraintFrame : ConstraintFrames)
		{
			OutFramesToBake.Add(ConstraintFrame);
		}

		const int32 FrameInc = bHasSettings ? InSettings->FrameIncrement : 1;
		for (int32 FrameIndex = 0; FrameIndex < Frames.Num(); ++FrameIndex)
		{
			if (FrameIndex % FrameInc == 0)
			{
				const FFrameNumber& InFrame = Frames[FrameIndex];
				if (IsConstraintActive(InFrame))
				{
					OutFramesToBake.Add(InFrame);
				}
			}
		}
		
		if (FrameInc > 1 && bHasSettings) //if skipping make sure we get all transforms
		{
			FMovieSceneConstraintChannelHelper::GetTransformFramesForConstraintHandles(InConstraint, InSequencer,
				InSettings->StartFrame, InSettings->EndFrame, OutFramesToBake);
		}
		
	}
	else //doing per frame key get constraint keys and transform keys
	{
		for (const FFrameNumber& Frame : ConstraintFrames)
		{
			if (Frame >= FirstBakingFrame && Frame <= LastBakingFrame)
			{
				OutFramesToBake.Add(Frame);
			}
		}

		FMovieSceneConstraintChannelHelper::GetTransformFramesForConstraintHandles(InConstraint, InSequencer,
			InSettings->StartFrame, InSettings->EndFrame, OutFramesToBake);

	}

	// we also need to store which T-1 frames need to be kept for other constraints compensation
	// and also store transform key locations for parents if doing per key or per frame with non-1 increments
	{
		// gather the sorted child's constraint
		static constexpr bool bSorted = true;
		const uint32 ChildHash = InConstraint->ChildTRSHandle->GetHash();
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(InWorld);
		using ConstraintPtr = TWeakObjectPtr<UTickableConstraint>;
		const TArray<ConstraintPtr> Constraints = Controller.GetParentConstraints(ChildHash, bSorted);

		// store the other channels that may need compensation
		TArray<FConstraintAndActiveChannel*> OtherChannels;
		for (const ConstraintPtr& Constraint: Constraints)
		{
			const UTickableTransformConstraint* TransformConstraint = Cast<UTickableTransformConstraint>(Constraint.Get());
			if (TransformConstraint && TransformConstraint->NeedsCompensation() && TransformConstraint != InConstraint) 
			{
				const FGuid ConstraintID= TransformConstraint->ConstraintID;
				if (FConstraintAndActiveChannel* ConstraintChannel = ConstrainedSection->GetConstraintChannel(ConstraintID))
				{
					OtherChannels.Add(ConstraintChannel);
				}

				//if doing keys or per frame increments we need to also 
				if (bHasSettings && (InSettings->BakingKeySettings == EBakingKeySettings::KeysOnly || InSettings->FrameIncrement > 1))
				{
					FMovieSceneConstraintChannelHelper::GetTransformFramesForConstraintHandles(TransformConstraint, InSequencer,
						InSettings->StartFrame, InSettings->EndFrame, OutFramesToBake);
				}
			}
		}

		// check if any other channel needs to compensate at T
		for (const FFrameNumber& ConstraintFrame: ConstraintFrames)
		{
			const bool bNeedsCompensation = OtherChannels.ContainsByPredicate(
		[ConstraintFrame](const FConstraintAndActiveChannel* OtherChannel)
			{
				const bool bHasAtLeastOneActiveKey = OtherChannel->ActiveChannel.GetValues().Contains(true);
				if (!bHasAtLeastOneActiveKey)
				{
					return false;
				}
				return OtherChannel->ActiveChannel.GetTimes().Contains(ConstraintFrame);
			} );

			// add T-1 if needed so that we will compensate int the remaining constraints' space 
			if (bNeedsCompensation)
			{
				const FFrameNumber& FrameMinusOne(ConstraintFrame-1);
				OutFramesToBake.Add(FrameMinusOne);
			}
		}
	}
	
	// uniqueness
	OutFramesToBake.Sort();
	OutFramesToBake.SetNum(Algo::Unique(OutFramesToBake));
}

bool FConstraintBaker::BakeMultiple(
	UWorld* InWorld,
	TArray< UTickableTransformConstraint*>& InConstraints,
	const TSharedPtr<ISequencer>& InSequencer,
	const FBakingAnimationKeySettings& InSettings)
{
	TOptional<TArray<FFrameNumber>> Frames;
	TOptional< FBakingAnimationKeySettings> Settings = InSettings;
	for (UTickableTransformConstraint* Constraint : InConstraints)
	{
		if (IsValid(Constraint->ChildTRSHandle) && Constraint->ChildTRSHandle->IsValid()
			&& IsValid(Constraint->ParentTRSHandle) && Constraint->ParentTRSHandle->IsValid())
		{
			FConstraintBaker::Bake(InWorld, Constraint, InSequencer, Settings, Frames);
		}
	}
	return true;
}

void FConstraintBaker::Bake(UWorld* InWorld, 
	UTickableTransformConstraint* InConstraint,
	const TSharedPtr<ISequencer>& InSequencer, 
	const TOptional< FBakingAnimationKeySettings>& InSettings,
	const TOptional<TArray<FFrameNumber>>& InFrames)
{
	if ((InFrames && InFrames->IsEmpty()) || (InConstraint == nullptr) || (InConstraint->ChildTRSHandle == nullptr))
	{
		return;
	}

	const TObjectPtr<UTransformableHandle>& Handle = InConstraint->ChildTRSHandle;

	FConstraintSections ConstraintSections = FMovieSceneConstraintChannelHelper::GetConstraintSectionAndChannel(
		InConstraint, InSequencer);
	IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(ConstraintSections.ConstraintSection);
	if (ConstrainedSection == nullptr || ConstraintSections.ActiveChannel == nullptr)
	{
		return;
	}
	// compute transforms
	TArray<FFrameNumber> FramesToBake;
	if (InFrames)
	{
		FramesToBake = *InFrames;
	}
	else
	{
		GetMinimalFramesToBake(InWorld, InConstraint, InSequencer, ConstraintSections.ConstraintSection, InSettings, FramesToBake);

		// if it needs recomposition 
		if (ConstraintSections.ConstraintSection != ConstraintSections.ChildTransformSection && !FramesToBake.IsEmpty())
		{
			const TMovieSceneChannelData<const bool> ConstraintChannelData = ConstraintSections.ActiveChannel->ActiveChannel.GetData();
			const TArrayView<const FFrameNumber> ConstraintFrames = ConstraintChannelData.GetTimes();
			const TArrayView<const bool> ConstraintValues = ConstraintChannelData.GetValues();
			for (int32 Index = 0; Index < ConstraintFrames.Num(); ++Index)
			{
				// if the key is active
				if (ConstraintValues[Index])
				{
					const int32 FrameIndex = FramesToBake.IndexOfByKey(ConstraintFrames[Index]);
					if (FrameIndex != INDEX_NONE)
					{
						// T-1 is not part of the frames to bake, then we need to compensate:
						// in the context of additive, we need to add a 'zeroed' key to prevent from popping
						const FFrameNumber FrameMinusOne(ConstraintFrames[Index] - 1);
						const bool bAddMinusOne = (FrameIndex == 0) ? true : !ConstraintFrames.Contains(FrameMinusOne);  
						if (bAddMinusOne)
						{
							FramesToBake.Insert(FrameMinusOne, FrameIndex);
						}
					}
				}
			}
		}
	}
	if (FramesToBake.Num() == 0 || ConstraintSections.ConstraintSection == nullptr ||
		ConstraintSections.ActiveChannel == nullptr)
	{
		return;
	}

	FCompensationEvaluator Evaluator(InConstraint);
	const FCompensationEvaluator::FEvalParameters EvalParams(InSequencer, FramesToBake);
	Evaluator.ComputeLocalTransformsForBaking(InWorld, EvalParams);
	TArray<FTransform> Transforms = Evaluator.ChildLocals;
	if (FramesToBake.Num() != Transforms.Num())
	{
		return;
	}

	// get transform channels
	const TArrayView<FMovieSceneFloatChannel*> FloatTransformChannels = Handle->GetFloatChannels(ConstraintSections.ChildTransformSection);
	const TArrayView<FMovieSceneDoubleChannel*> DoubleTransformChannels = Handle->GetDoubleChannels(ConstraintSections.ChildTransformSection);

	const FMovieSceneConstraintChannel& ConstraintChannel = ConstraintSections.ActiveChannel->ActiveChannel;
	const bool bIsInfinite = ConstraintChannel.IsInfinite();

	TArray<FFrameNumber> FramesToCompensate;
	TArray<FTransform> TransformsToCompensate;
	if (bIsInfinite)
	{
		if (FloatTransformChannels.Num() > 0)
		{
			FMovieSceneConstraintChannelHelper::GetFramesWithinActiveState(ConstraintChannel, FloatTransformChannels, FramesToCompensate);
		}
		else
		{
			FMovieSceneConstraintChannelHelper::GetFramesWithinActiveState(ConstraintChannel, DoubleTransformChannels, FramesToCompensate);
		}

		// remove frames to bake
		if (!FramesToCompensate.IsEmpty())
		{
			const TSet<FFrameNumber> SetFramesToBake(FramesToBake);
			for (int32 Index = FramesToCompensate.Num()-1; Index >= 0; --Index)
			{
				if (SetFramesToBake.Contains(FramesToCompensate[Index]))
				{
					FramesToCompensate.RemoveAt(Index);
				}
			}
		}

		// compute compensation
		if (!FramesToCompensate.IsEmpty())
		{
			FCompensationEvaluator EvaluatorForCompensation(InConstraint);
			const FCompensationEvaluator::FEvalParameters EvalParamsForCompensation(InSequencer, FramesToCompensate);
			EvaluatorForCompensation.ComputeLocalTransformsBeforeDeletion(InWorld, EvalParamsForCompensation);
			TransformsToCompensate = MoveTemp(EvaluatorForCompensation.ChildLocals);
		}
	}
	
	ConstraintSections.ConstraintSection->Modify();

	// disable constraint and delete extra transform keys
	TMovieSceneChannelData<bool> ConstraintChannelData = ConstraintSections.ActiveChannel->ActiveChannel.GetData();
	const TArrayView<const FFrameNumber> ConstraintFrames = ConstraintChannelData.GetTimes();
	const TArray<bool> PreviousConstraintValues(ConstraintChannelData.GetValues());
	
	EMovieSceneKeyInterpolation KeyType = InSequencer->GetKeyInterpolation();

	for (int32 Index = 0; Index < ConstraintFrames.Num(); ++Index)
	{
		const FFrameNumber Frame = ConstraintFrames[Index];
		//todo we need to add a key at the begin/end if there is no frame there
		if (Frame >= FramesToBake[0] && Frame <= FramesToBake.Last())
		{
			// set constraint key to inactive
			ConstraintChannelData.UpdateOrAddKey(Frame, false);

			// delete minus one transform frames
			const FFrameNumber FrameMinusOne = Frame - 1;
			if (FloatTransformChannels.Num() > 0)
			{
				FMovieSceneConstraintChannelHelper::DeleteTransformKeys(FloatTransformChannels, FrameMinusOne);
				//we also set the tangent at the break to the default type
				FMovieSceneConstraintChannelHelper::ChangeKeyInterpolation(FloatTransformChannels, Frame, KeyType);
			}
			else if (DoubleTransformChannels.Num() > 0)
			{
				FMovieSceneConstraintChannelHelper::DeleteTransformKeys(DoubleTransformChannels, FrameMinusOne);
				//we also set the tangent at the break to the default type
				FMovieSceneConstraintChannelHelper::ChangeKeyInterpolation(DoubleTransformChannels, Frame, KeyType);
			}
			if (const int32 RemoveIndex = Algo::BinarySearch(FramesToBake, FrameMinusOne);
				RemoveIndex != INDEX_NONE)
			{
				FramesToBake.RemoveAt(RemoveIndex);
				Transforms.RemoveAt(RemoveIndex);
			}
		}
	}
	
	// remove transform keys in active sections if baking all frames
	if (InSettings.IsSet() && InSettings->BakingKeySettings == EBakingKeySettings::AllFrames)
	{
		EMovieSceneTransformChannel ChannelsToKey = InConstraint->GetChannelsToKey();
		//if we aren't doing a full transform change ChannelsToKey to make sure it's the ones present
		//assuming if it's the wrong mix(like rotator with a position constraint) we wouldn't be here anyway
		if (FloatTransformChannels.Num() == 3 || DoubleTransformChannels.Num() == 3)
		{
			ChannelsToKey = EMovieSceneTransformChannel::Translation;
		}
		DeleteTransformKeysInActiveRanges(
			FloatTransformChannels, DoubleTransformChannels, ChannelsToKey,
			FramesToBake, ConstraintFrames, PreviousConstraintValues);
	}

	// now bake to channel curves
	const EMovieSceneTransformChannel Channels = InConstraint->GetChannelsToKey();
	ConstraintSections.Interface->AddHandleTransformKeys(InSequencer, Handle, FramesToBake, Transforms, Channels);

	// reduce keys
	if (InSettings.IsSet() && InSettings->BakingKeySettings != EBakingKeySettings::KeysOnly && InSettings->bReduceKeys)
	{
		FKeyDataOptimizationParams Param;
		Param.bAutoSetInterpolation = true;
		Param.Tolerance = InSettings.GetValue().Tolerance;
		TRange<FFrameNumber> Range(InSettings.GetValue().StartFrame, InSettings.GetValue().EndFrame);
		Param.Range = Range;
		MovieSceneToolHelpers::OptimizeSection(Param, ConstraintSections.ConstraintSection);
	}

	// add compensation keys outside the baking range when infinite
	if (bIsInfinite && !FramesToCompensate.IsEmpty() && FramesToCompensate.Num() == TransformsToCompensate.Num())
	{
		ConstraintSections.Interface->AddHandleTransformKeys(InSequencer, Handle, FramesToCompensate, TransformsToCompensate, Channels);
	}

	// remove useless constraint keys
	CleanupConstraintKeys(*ConstraintSections.ActiveChannel);
	
	// notify
	InSequencer->RequestEvaluate();
}

void FConstraintBaker::DeleteTransformKeysInActiveRanges(
	const TArrayView<FMovieSceneFloatChannel*>& InFloatTransformChannels,
	const TArrayView<FMovieSceneDoubleChannel*>& InDoubleTransformChannels,
	const EMovieSceneTransformChannel& InChannels,
	const TArray<FFrameNumber>& InFramesToBake,
	const TArrayView<const FFrameNumber>& InConstraintFrames,
	const TArray<bool>& InConstraintValues)
{
	// get transform channels
	auto CleanRange = [&](const FFrameNumber& StartTime, const FFrameNumber& EndTime)
	{
		if (InFloatTransformChannels.Num() > 0)
		{
			FMovieSceneConstraintChannelHelper::DeleteTransformTimes(InFloatTransformChannels, StartTime, EndTime, InChannels);
		}
		else if (InDoubleTransformChannels.Num() > 0)
		{
			FMovieSceneConstraintChannelHelper::DeleteTransformTimes(InDoubleTransformChannels, StartTime, EndTime, InChannels);
		}
	};

	auto GetRange = [&](const int32 FirstActiveIndex)
	{
		TArray<int32> Range;
		if (InConstraintValues.IsValidIndex(FirstActiveIndex))
		{
			Range.Add(FirstActiveIndex);
			for (int32 Index = FirstActiveIndex+1; Index < InConstraintValues.Num(); ++Index)
			{
				if (InConstraintValues[Index] != InConstraintValues[Range.Last()])
				{
					Range.Add(Index);
				}
				else if (InConstraintValues[Index] == false)
				{
					Range.Last() = Index;
				}
			}
		}
		return Range;
	};
		
	const int32 ActiveIndex = InConstraintValues.IndexOfByKey(true);
	if (ActiveIndex != INDEX_NONE)
	{
		const TArray<int32> Range = GetRange(ActiveIndex);
		if (Range.Num() == 1)
		{
			// one single active key, clean all range
			CleanRange(InConstraintFrames[ActiveIndex] + 1, InFramesToBake.Last());
		}
		else
		{
			// clean closed ranges
			const int32 NumClosedRange = Range.Num() / 2;
			for (int32 RangeIndex = 0; RangeIndex < NumClosedRange; ++RangeIndex)
			{
				const int32 Start = Range[2*RangeIndex], End = Range[2*RangeIndex+1];    
				CleanRange(InConstraintFrames[Start] + 1, InConstraintFrames[End]);
			}

			// clean last opened range
			if (Range.Num() % 2 == 1)
			{
				const int32 Start = Range.Last();
				CleanRange(InConstraintFrames[Start] + 1, InFramesToBake.Last());
			}
		}
	}
}

void FConstraintBaker::CleanupConstraintKeys(FConstraintAndActiveChannel& InOutActiveChannel)
{
	TMovieSceneChannelData<bool> ConstraintChannelData = InOutActiveChannel.ActiveChannel.GetData();

	const bool bHasAnyActiveKey = ConstraintChannelData.GetValues().Contains(true);
	if (!bHasAnyActiveKey)
	{
		// all the keys are inactive so clear the channel without notification at all
		InOutActiveChannel.ActiveChannel.Reset();
		return;
	}
		
	// disable extra compensation when removing keys
	TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
			
	// store a copy to keep track of values as we remove them
	const TArray<bool> ConstraintValues(ConstraintChannelData.GetValues());
	for (int32 Index = ConstraintValues.Num()-1; Index >= 0; Index--)
	{
		// if inactive, check if the previous one is also inactive. if that's the case, then this key is useless
		if (!ConstraintValues[Index])
		{
			if (Index == 0)
			{
				ConstraintChannelData.RemoveKey(Index);
			}
			else if (!ConstraintValues[Index-1])
			{
				ConstraintChannelData.RemoveKey(Index);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE


