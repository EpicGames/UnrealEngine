// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigFrustumSelectInteraction.h"
#include "Settings/ControlRigSettings.h"
#include "ViewportInteractions/ViewportInteractionsBehaviorSource.h"
#include "ViewportInteractions/ViewportMoveYawInteraction.h"

UControlRigFrustumSelectInteraction::UControlRigFrustumSelectInteraction()
{
	UControlRigEditorSettings::Get()->OnSettingChanged().AddUObject(
		this, &UControlRigFrustumSelectInteraction::OnControlRigEditorSettingChanged
	);

	InteractionName = TEXT("Control Rig Frustum Select");
}

void UControlRigFrustumSelectInteraction::Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource)
{
	Super::Initialize(InViewportInteractionsBehaviorSource);
	
	UpdateBindings();
}

void UControlRigFrustumSelectInteraction::BeginDestroy()
{
	UControlRigEditorSettings::Get()->OnSettingChanged().RemoveAll(this);
	Super::BeginDestroy();	
}

void UControlRigFrustumSelectInteraction::OnControlRigEditorSettingChanged(
	UObject* InSettingsChanged, FPropertyChangedEvent& InPropertyChangedEvent
)
{
	if (InPropertyChangedEvent.Property->GetFName()
		== GET_MEMBER_NAME_CHECKED(UControlRigEditorSettings, bLeftMouseDragDoesMarquee))
	{
		UpdateBindings();
	}
}

void UControlRigFrustumSelectInteraction::UpdateBindings()
{
	if (UViewportClickDragBehavior* ClickDragBehavior = ClickDragInputBehavior.Get())
	{
		if (UControlRigEditorSettings::Get()->bLeftMouseDragDoesMarquee)
		{
			ClickDragBehavior->SetBindings({
				UE::Editor::ViewportInteractions::FButtonBinding(EKeys::LeftMouseButton).TriggersStart()
			});
			
			if (UViewportInteractionsBehaviorSource* Source = GetViewportInteractionsBehaviorSource())
			{
				if (UViewportInteraction* Interaction = Source->FindInteraction(UViewportMoveYawInteraction::StaticClass()))
				{
					Interaction->SetEnabled(false);
				}
			}
		}
		else
		{
			ClickDragBehavior->SetBindings({
				UE::Editor::ViewportInteractions::FButtonBinding(EKeys::LeftMouseButton).TriggersStart(),
				EKeys::LeftControl,
				EKeys::LeftAlt
			});
			
			if (UViewportInteractionsBehaviorSource* Source = GetViewportInteractionsBehaviorSource())
			{
				if (UViewportInteraction* Interaction = Source->FindInteraction(UViewportMoveYawInteraction::StaticClass()))
				{
					Interaction->SetEnabled(true);
				}
			}
		}
	}
}
