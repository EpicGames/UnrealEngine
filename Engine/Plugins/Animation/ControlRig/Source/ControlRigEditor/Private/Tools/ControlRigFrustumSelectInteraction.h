// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorDragTools/FrustumSelectInteraction.h"
#include "ControlRigFrustumSelectInteraction.generated.h"

/**
 * Similar to its base class, this marquee select tool switches between LMB only and Ctrl + Alt + LMB to activate marquee selection.
 * Could potentially be used to add other custom logic.
 */
UCLASS(MinimalAPI, Transient)
class UControlRigFrustumSelectInteraction : public UFrustumSelectInteraction
{
	GENERATED_BODY()

public:
	UControlRigFrustumSelectInteraction();

	virtual void Initialize(UViewportInteractionsBehaviorSource* const InViewportInteractionsBehaviorSource) override;
	virtual void BeginDestroy() override;

	virtual TArray<FEditorModeID> GetUnsupportedModes() const override {return {};}

private:
	void OnControlRigEditorSettingChanged(UObject* InSettingsChanged, FPropertyChangedEvent& InPropertyChangedEvent);
	void UpdateBindings();
};

