// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigPoseThumbnailRenderer.h"
#include "Tools/ControlRigPose.h"
#include "ThumbnailHelpers.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ControlRigObjectBinding.h"
#include "AnimCustomInstanceHelper.h"
#include "SceneView.h"
#include "Sequencer/ControlRigLayerInstance.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigPoseThumbnailRenderer)


/*
***************************************************************
FControlRigPoseThumbnailScene
***************************************************************
*/

class FControlRigPoseThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	FControlRigPoseThumbnailScene();

	bool SetControlRigPoseAsset(UControlRigPoseAsset* PoseAsset);

protected:
	// FThumbnailPreviewScene implementation
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

	//Clean up the children of this component
	void CleanupComponentChildren(USceneComponent* Component);

private:
	/** The skeletal mesh actor*/
	ASkeletalMeshActor* PreviewActor;

	/** ControlRig Pose Asset*/
	UControlRigPoseAsset* PoseAsset;

};


FControlRigPoseThumbnailScene::FControlRigPoseThumbnailScene()
	: FThumbnailPreviewScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<ASkeletalMeshActor>(SpawnInfo);

	PreviewActor->SetActorEnableCollision(false);
}

bool FControlRigPoseThumbnailScene::SetControlRigPoseAsset(UControlRigPoseAsset* InPoseAsset)
{
	PoseAsset = InPoseAsset;
	bool bSetSucessfully = false;

	if (PoseAsset)
	{
		USkeletalMesh* SkeletalMesh = nullptr; // PoseAsset->GetSkeletalMeshAsset();
		UControlRig* ControlRig = nullptr; // PoseAsset->GetControlRig();
		PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();


		if (SkeletalMesh && ControlRig)
		{

			PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(SkeletalMesh);
			PreviewActor->GetSkeletalMeshComponent()->AnimScriptInstance = nullptr;


			bSetSucessfully = true;
			//now set up the control rig and the anim instance fo rit
			FControlRigAssetStrongReference AssetReference = ControlRig->GetAssetReference();
			UControlRig* TempControlRig = AssetReference.CreateInstance(GetTransientPackage(),NAME_None, RF_Transient);
			TempControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
			TempControlRig->GetObjectBinding()->BindToObject(SkeletalMesh);
			TempControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, TempControlRig->GetObjectBinding()->GetBoundObject());


			bool bWasCreated;
			if (UControlRigLayerInstance* AnimInstance = FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UControlRigLayerInstance>(PreviewActor->GetSkeletalMeshComponent(), bWasCreated))
			{

				if (bWasCreated)
				{
					AnimInstance->RecalcRequiredBones();
					AnimInstance->AddControlRigTrack(TempControlRig->GetUniqueID(), TempControlRig);
					TempControlRig->CreateRigControlsForCurveContainer();
				}
				TempControlRig->Initialize();

				float Weight = 1.0f;
				FControlRigIOSettings InputSettings;
				InputSettings.bUpdateCurves = false;
				InputSettings.bUpdatePose = true;
				AnimInstance->UpdateControlRigTrack(TempControlRig->GetUniqueID(), Weight, InputSettings, true);
				PoseAsset->PastePose(TempControlRig);
				TempControlRig->Evaluate_AnyThread();

			}

			PreviewActor->GetSkeletalMeshComponent()->TickAnimation(0.f, false);
			PreviewActor->GetSkeletalMeshComponent()->RefreshBoneTransforms();
			PreviewActor->GetSkeletalMeshComponent()->FinalizeBoneTransform();

			FTransform MeshTransform = FTransform::Identity;

			PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
			PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

			// Center the mesh at the world origin then offset to put it on top of the plane
			const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
			PreviewActor->SetActorLocation(-PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin + FVector(0, 0, BoundsZOffset), false);
			PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
			TempControlRig->MarkAsGarbage();

		}

		if (!bSetSucessfully)
		{
			CleanupComponentChildren(PreviewActor->GetSkeletalMeshComponent());
			PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(nullptr);
			PreviewActor->GetSkeletalMeshComponent()->SetAnimInstanceClass(nullptr);
			PreviewActor->GetSkeletalMeshComponent()->AnimScriptInstance = nullptr;
		}

	}
	return bSetSucessfully;
}

void FControlRigPoseThumbnailScene::CleanupComponentChildren(USceneComponent* Component)
{
	if (Component)
	{
		for (int32 ComponentIdx = Component->GetAttachChildren().Num() - 1; ComponentIdx >= 0; --ComponentIdx)
		{
			CleanupComponentChildren(Component->GetAttachChildren()[ComponentIdx]);
			Component->GetAttachChildren()[ComponentIdx]->DestroyComponent();
		}
		check(Component->GetAttachChildren().Num() == 0);
	}
}

void FControlRigPoseThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor->GetSkeletalMeshComponent());
	check(PreviewActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// No need to add extra size to view slightly outside of the sphere to compensate for perspective since skeletal meshes already buffer bounds.
	const float HalfMeshSize = PreviewActor->GetSkeletalMeshComponent()->Bounds.SphereRadius;
	const float BoundsZOffset = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = nullptr;// Cast<USceneThumbnailInfo>(PoseAsset->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}



/*
***************************************************************
UControlRigPoseThumbnailRenderer
***************************************************************
*/

UControlRigPoseThumbnailRenderer::UControlRigPoseThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ControlRigPoseAsset = nullptr;
	ThumbnailScene = nullptr;
}

bool UControlRigPoseThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return Cast<UControlRigPoseAsset>(Object) != nullptr;
}

void UControlRigPoseThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UControlRigPoseAsset* PoseAsset = Cast<UControlRigPoseAsset>(Object);
	if (PoseAsset)
	{
		
		if (ThumbnailScene == nullptr)
		{
			ThumbnailScene = new FControlRigPoseThumbnailScene();
		}

		ThumbnailScene->SetControlRigPoseAsset(PoseAsset);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetControlRigPoseAsset(nullptr);
	}
}

void UControlRigPoseThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}


