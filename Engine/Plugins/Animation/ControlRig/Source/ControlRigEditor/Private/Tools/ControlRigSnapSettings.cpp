// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tools/ControlRigSnapSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSnapSettings)

UControlRigSnapSettings::UControlRigSnapSettings()
{
}

#if WITH_EDITOR
void UControlRigSnapSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		// Dragging spinboxes causes this to be called every frame so we wait until they've finished dragging before saving.
		SaveConfig();
	}
}
#endif
