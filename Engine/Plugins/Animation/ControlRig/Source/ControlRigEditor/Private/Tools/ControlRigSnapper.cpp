// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/ControlRigSnapper.h"
#include "Algo/Reverse.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "ControlRig.h"
#include "ISequencer.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "Rigs/RigControlHierarchy.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "IKeyArea.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "ILevelSequenceEditorToolkit.h"
#include "Tools/ControlRigSnapper.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "ControlRigObjectBinding.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Tools/ControlRigSnapSettings.h"
#include "MovieSceneToolsModule.h"
#include "Components/SkeletalMeshComponent.h"
#include "ScopedTransaction.h"
#include "MovieSceneToolHelpers.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceActor.h"
#include "Containers/SortedMap.h"
#include "Sequencer/ControlRigSequencerHelpers.h"
#include "Sequencer/EditModeAnimationUtil.h"
#include "SequencerBakingSetupRestore.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformConstraintUtil.h"
#include "Constraints/MovieSceneConstraintChannelHelper.inl"
#include "Tools/EvaluateSequencerTools.h"
#include "Transform/AnimationEvaluation.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlRigSnapper)

#define LOCTEXT_NAMESPACE "ControlRigSnapper"


FText FControlRigSnapperSelection::GetName() const
{
	FText Name;

	int32 Num = NumSelected();
	if (Num == 0)
	{
		Name = LOCTEXT("None", "None");
	}
	else if (Num == 1)
	{
		for (const FActorForWorldTransforms& Actor : Actors)
		{
			if (Actor.Actor.IsValid())
			{
				FString ActorLabel = Actor.Actor->GetActorLabel();
				if (Actor.SocketName != NAME_None)
				{
					FString SocketString = Actor.SocketName.ToString();
					ActorLabel += (FString(":") + SocketString);
				}
				Name = FText::FromString(ActorLabel);
				break;
			}
		}
		for (const FControlRigForWorldTransforms& Selection : ControlRigs)
		{
			if (Selection.ControlRig.IsValid())
			{
				if (Selection.ControlNames.Num() > 0)
				{
					FName ControlName = Selection.ControlNames[0];
					Name = FText::FromString(ControlName.ToString());
					break;
				}
			}
		}
	}
	else
	{
		Name = LOCTEXT("Multiple", "--Multiple--");
	}
	return Name;
}

int32 FControlRigSnapperSelection::NumSelected() const
{
	int32 Selected = 0;
	for (const FActorForWorldTransforms& Actor : Actors)
	{
		if (Actor.Actor.IsValid())
		{
			++Selected;
		}
	}
	for (const FControlRigForWorldTransforms& Selection : ControlRigs)
	{
		if (Selection.ControlRig.IsValid())
		{
			Selected += Selection.ControlNames.Num();
		}
	}
	return Selected;
}

TWeakPtr<ISequencer> FControlRigSnapper::GetSequencer()
{
	return UE::AnimationEditMode::GetSequencer();
}

static bool LocalGetControlRigControlTransforms(IMovieScenePlayer* Player, const TOptional<FFrameNumber>& CurrentFrame, UMovieSceneSequence* MovieSceneSequence, FMovieSceneSequenceIDRef Template, FMovieSceneSequenceTransform& RootToLocalTransform,
	UControlRig* ControlRig, const FName& ControlName,
	const TArray<FFrameNumber>& Frames, const TArray<FTransform>& ParentTransforms, TArray<FTransform>& OutTransforms)
{
	if (ControlRig == nullptr || ControlRig->GetHierarchy() == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "No Control Rig or Hierarchy");
		return false;
	}
	if (Frames.Num() > ParentTransforms.Num())
	{
		UE_LOGF(LogControlRig, Error, "Number of Frames %d to Snap greater  than Parent Frames %d", Frames.Num(), ParentTransforms.Num());
		return false;
	}
	FRigTransformElement* ControlElement = ControlRig->FindControl(ControlName);
	if (ControlElement == nullptr)
	{
		UE_LOGF(LogControlRig, Error, "Can not find Control %ls", *(ControlName.ToString()));
		return false;
	}
	if (UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene())
	{
		UWorld* World = ControlRig->GetWorld();
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();

		const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
		const float CachingDeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
		
		OutTransforms.SetNum(Frames.Num());
		for (int32 Index = 0; Index < Frames.Num(); ++Index)
		{
			const FFrameNumber& FrameNumber = Frames[Index];
			double DeltaTime = 0.0;
			if ((Index != 0) || (CurrentFrame.IsSet() == false || CurrentFrame.GetValue() != FrameNumber))
			{
				FFrameTime GlobalTime = LocalToRootTransform.TryTransformTime(FrameNumber).Get(FrameNumber); //player evals in root time so need to go back to it.

				FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Player->GetPlaybackStatus()).SetHasJumped(true);

				DeltaTime = 1.0/Context.GetFrameRate().AsDecimal();
				Player->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
				{
					const UE::Anim::FEvaluationForCachingScope CachingScope(CachingDeltaTime);
					Controller.EvaluateAllConstraints();
				}
			}
			if (ControlRig->IsAdditive())
			{
				ControlRig->EvaluateSkeletalMeshComponent(DeltaTime);
			}
			else
			{
				ControlRig->SetDeltaTime(CachingDeltaTime);
				ControlRig->Evaluate_AnyThread();
			}
			OutTransforms[Index] = ControlRig->GetHierarchy()->GetTransform(ControlElement, ERigTransformType::CurrentGlobal) * ParentTransforms[Index];
		}
	}
	return true;
}

bool FControlRigSnapper::GetControlRigControlTransforms(ISequencer* Sequencer,  UControlRig* ControlRig, const FName& ControlName,
	const TArray<FFrameNumber> &Frames, const TArray<FTransform>& ParentTransforms,TArray<FTransform>& OutTransforms)
{
	if (Sequencer->GetFocusedMovieSceneSequence())
	{
		FMovieSceneSequenceIDRef Template = Sequencer->GetFocusedTemplateID();
		FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
		TOptional<FFrameNumber> FrameNumber;
		return LocalGetControlRigControlTransforms(Sequencer, FrameNumber, Sequencer->GetFocusedMovieSceneSequence(), Template, RootToLocalTransform,
			ControlRig, ControlName, Frames, ParentTransforms, OutTransforms);
	
	}
	return false;
}

bool FControlRigSnapper::GetControlRigControlTransforms(UWorld* World,ULevelSequence* LevelSequence, UControlRig* ControlRig, const FName& ControlName,
	const TArray<FFrameNumber>& Frames, const TArray<FTransform>& ParentTransforms, TArray<FTransform>& OutTransforms)
{
	if (LevelSequence && Frames.Num() > 0)
	{
		ALevelSequenceActor* OutActor;
		FMovieSceneSequencePlaybackSettings Settings;
		FMovieSceneSequenceIDRef Template = MovieSceneSequenceID::Root;
		FMovieSceneSequenceTransform RootToLocalTransform;
		ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
		TOptional<FFrameNumber> OptFrame;
		bool Success = LocalGetControlRigControlTransforms(Player, OptFrame, LevelSequence, Template, RootToLocalTransform,
			ControlRig, ControlName, Frames, ParentTransforms, OutTransforms);
		World->DestroyActor(OutActor);
		return Success;
	}
	return false;
}

struct FGuidAndActor
{
	FGuidAndActor(FGuid InGuid, AActor* InActor) : Guid(InGuid), Actor(InActor) {};
	FGuid Guid;
	AActor* Actor;

	bool SetTransform(
		TSharedPtr<ISequencer>& Sequencer, const TArray<FFrameNumber>& Frames,
		const TArray<FTransform>& WorldTransformsToSnapTo, const UControlRigSnapSettings* SnapSettings) const
	{
		// get section
		UMovieScene3DTransformSection* TransformSection = MovieSceneToolHelpers::GetTransformSection(Sequencer.Get(), Guid);
		if (!TransformSection)
		{
			return false;
		}
		TransformSection->Modify();
		// fill parent transforms
		TArray<FTransform> ParentWorldTransforms; 
		AActor* ParentActor = Actor->GetAttachParentActor();
		if (ParentActor)
		{
			FActorForWorldTransforms ActorSelection;
			ActorSelection.Actor = ParentActor;
			ActorSelection.SocketName = Actor->GetAttachParentSocketName();
			MovieSceneToolHelpers::GetActorWorldTransforms(Sequencer.Get(), ActorSelection, Frames, ParentWorldTransforms);
		}
		else
		{
			ParentWorldTransforms.SetNumUninitialized(Frames.Num());
			for (FTransform& Transform : ParentWorldTransforms)
			{
				Transform = FTransform::Identity;
			}
		}

		// fill channels to key
		EMovieSceneTransformChannel Channels = EMovieSceneTransformChannel::AllTransform;
		if (SnapSettings)
		{
			Channels = EMovieSceneTransformChannel::None;
			if (SnapSettings->bSnapPosition)
			{
				EnumAddFlags(Channels, EMovieSceneTransformChannel::Translation);
			}
			if (SnapSettings->bSnapRotation)
			{
				EnumAddFlags(Channels, EMovieSceneTransformChannel::Rotation);
			}
			if (SnapSettings->bSnapScale)
			{
				EnumAddFlags(Channels, EMovieSceneTransformChannel::Scale);
			}
			if (Channels == EMovieSceneTransformChannel::None)
			{
				return false;
			}
		}

		// compute local transforms
		TArray<FTransform> LocalTransforms;
		LocalTransforms.Reserve(Frames.Num());
		for (int32 Index = 0; Index < Frames.Num(); ++Index)
		{
			const FTransform& ParentTransform = ParentWorldTransforms[Index];
			const FTransform& WorldTransform = WorldTransformsToSnapTo[Index];
			LocalTransforms.Add(WorldTransform.GetRelativeTransform(ParentTransform));
		}

		if (USceneComponent* SceneComponent = Actor->GetRootComponent())
		{
			UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
			FFrameRate TickResolution = MovieScene->GetTickResolution();
			FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();

			//adjust keys for constraint
			const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(Actor->GetWorld());

			const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
			
			for (int32 Index = 0; Index < LocalTransforms.Num(); ++Index) 
			{
				const FFrameNumber Frame = Frames[Index];
				FTransform& CurrentTransform = LocalTransforms[Index];

				FFrameTime GlobalTime = LocalToRootTransform.TryTransformTime(Frame).Get(Frame);

				FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
				if (Index == 0) // similar with baking first time in we need to evaluate twice (think due to double buffering that happens with skel mesh components).
				{
					Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
				}
				Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
				{
					const UE::Anim::FEvaluationForCachingScope CachingScope(DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f);
					Controller.EvaluateAllConstraints();
				}
				UE::TransformConstraintUtil::UpdateTransformBasedOnConstraint(CurrentTransform, SceneComponent);
			}
		}

		// add keys
		MovieSceneToolHelpers::AddTransformKeys(TransformSection, Frames, LocalTransforms, Channels);
		
		if (SnapSettings && SnapSettings->BakingKeySettings != EBakingKeySettings::KeysOnly
			&& SnapSettings->bReduceKeys == true && Frames.Num() > 2)
		{
			FKeyDataOptimizationParams Param;
			Param.bAutoSetInterpolation = true;
			Param.Tolerance = SnapSettings->Tolerance;
			TRange<FFrameNumber> Range(Frames[0], Frames[Frames.Num() -1]);
			Param.Range = Range;
			MovieSceneToolHelpers::OptimizeSection(Param, TransformSection);
		}


	
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);

		return true;
	}
};

//returns true if world is calculated, false if there are no parents
static bool CalculateWorldTransformsFromParents(ISequencer* Sequencer, const FControlRigSnapperSelection& ParentToSnap,
	const TArray<FFrameNumber>& Frames, TArray<FTransform>& OutParentWorldTransforms)
{
	UMovieSceneSequence* MovieSceneSequence = Sequencer ? Sequencer->GetFocusedMovieSceneSequence() : nullptr;
	if (UMovieScene* MovieScene = MovieSceneSequence ? MovieSceneSequence->GetMovieScene() : nullptr)
	{
		//just do first for now but may do average later.
		for (const FActorForWorldTransforms& ActorSelection : ParentToSnap.Actors)
		{
			if (ActorSelection.Actor.IsValid())
			{
				MovieSceneToolHelpers::GetActorWorldTransforms(Sequencer, ActorSelection, Frames, OutParentWorldTransforms);
				return true;
			}
		}

		for (const FControlRigForWorldTransforms& ControlRigAndSelection : ParentToSnap.ControlRigs)
		{
			if (UControlRig* ControlRig = ControlRigAndSelection.ControlRig.Get())
			{
				if (const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
				{
					USceneComponent* Component = Cast<USceneComponent>(ObjectBinding->GetBoundObject());
					if (!Component)
					{
						continue;
					}
					AActor* Actor = Component->GetTypedOuter< AActor >();
					if (!Actor)
					{
						continue;
					}
				
					if (ControlRigAndSelection.ControlNames.Num() > 0)
					{
						OutParentWorldTransforms.Init(FTransform::Identity, Frames.Num());

						const FName& ControlName = ControlRigAndSelection.ControlNames[0];

						const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
						const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;
					
						const UE::AIE::FControlRigWorldTransformProvider Provider(Actor, Component, ControlRig, DeltaTime);
						MovieSceneToolHelpers::RunSequencerTask(Sequencer, Actor->GetWorld(), Frames, 
						[&OutParentWorldTransforms, &Provider, ControlName](int32 Index)
						{
							if (OutParentWorldTransforms.IsValidIndex(Index))
							{
								constexpr bool bUpdateAnimation = true;
								OutParentWorldTransforms[Index] = Provider.GetTransform(ControlName, bUpdateAnimation);
							}
						});
						return true;
					}
				}
			}
		}
	}
	
	OutParentWorldTransforms.Init(FTransform::Identity, Frames.Num());
	return false;
}

//todo handle constraints/spaces
static void GetControlRigParents(const FControlRigForWorldTransforms& ControlRigAndSelection, 
	TArray<FActorForWorldTransforms>& OutParentActors, TArray<FControlRigForWorldTransforms>& OutControlRigs)
{

	if (UControlRig* ControlRig = ControlRigAndSelection.ControlRig.Get())
	{
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
		for (const FName& ControlName : ControlRigAndSelection.ControlNames)
		{
			if (FRigControlElement* ControlElement = ControlRig->FindControl(ControlName))
			{
				if (FRigControlElement* ParentControlElement = Cast<FRigControlElement>(Hierarchy->GetFirstParent(ControlElement)))
				{
					FControlRigForWorldTransforms OutControlRig;
					OutControlRig.ControlRig = ControlRig;
					OutControlRig.ControlNames.Add(ParentControlElement->GetKey().Name);
					OutControlRigs.Add(OutControlRig);
					GetControlRigParents(OutControlRig, OutParentActors, OutControlRigs);
				}
				else
				{
					if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
					{
						USceneComponent* Component = Cast<USceneComponent>(ObjectBinding->GetBoundObject());
						if (!Component)
						{
							continue;
						}
						AActor* Actor = Component->GetTypedOuter< AActor >();
						if (!Actor)
						{
							continue;
						}
						FActorForWorldTransforms OutParentActor;
						OutParentActor.Actor = Actor;
						OutParentActors.Add(OutParentActor);
						MovieSceneToolHelpers::GetActorParents(OutParentActor, OutParentActors);
					}
				}
			}
		}
	}
}

static UMovieSceneControlRigParameterSection* GetControlRigSection(ISequencer* Sequencer, const UControlRig* ControlRig, const FName& ControlName)
{

	if (ControlRig == nullptr || Sequencer == nullptr)
	{
		return nullptr;
	}
	UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}
	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		UMovieSceneControlRigParameterTrack* ControlRigParameterTrack = Cast<UMovieSceneControlRigParameterTrack>(MovieScene->FindTrack(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None));
		if (ControlRigParameterTrack && ControlRigParameterTrack->GetControlRig() == ControlRig)
		{
			UMovieSceneControlRigParameterSection* ActiveSection = Cast<UMovieSceneControlRigParameterSection>(ControlRigParameterTrack->GetSectionToKey(ControlName));
			if (ActiveSection)
			{
				return ActiveSection;
			}
		}
	}
	return nullptr;
}

//Get Transform frames for specified snap items, will add them to the OutFrameMap
static void GetTransformFrames(TSharedPtr<ISequencer>&  Sequencer, const FControlRigSnapperSelection& SnapItem, FFrameNumber StartFrame, FFrameNumber EndFrame, TSortedMap<FFrameNumber, FFrameNumber>& OutFrameMap)
{
	TArray<FFrameNumber> FramesToUse;
	TArray<FActorForWorldTransforms> ParentActors;
	TArray<FControlRigForWorldTransforms> ParentControlRigs;

	for (const FActorForWorldTransforms& ActorsFor : SnapItem.Actors)
	{
		FramesToUse.Reset();
		ParentActors.Reset();
		ParentControlRigs.Reset();
		ParentActors.Add(ActorsFor);
		MovieSceneToolHelpers::GetActorParents(ActorsFor, ParentActors);
	}
	for (const FControlRigForWorldTransforms& ControlRigsFor : SnapItem.ControlRigs)
	{
		FramesToUse.Reset();
		ParentActors.Reset();
		ParentControlRigs.Reset();
		ParentControlRigs.Add(ControlRigsFor);
		GetControlRigParents(ControlRigsFor, ParentActors, ParentControlRigs);
	}

	for (FActorForWorldTransforms& Actor : ParentActors)
	{
		FGuid Guid;
		if (Actor.Component.Get())
		{
			Guid = Sequencer->GetHandleToObject(Actor.Component.Get(), false);
		}
		if (!Guid.IsValid())
		{
			Guid = Sequencer->GetHandleToObject(Actor.Actor.Get(), false);
		}
		if (Guid.IsValid())
		{
			if (UMovieScene3DTransformSection* TransformSection = MovieSceneToolHelpers::GetTransformSection(Sequencer.Get(), Guid))
			{
				TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
				TArray<FFrameNumber> TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
					Channels, StartFrame, EndFrame);
				for (FFrameNumber& FrameNumber : TransformFrameTimes)
				{
					OutFrameMap.Add(FrameNumber);
				}
			}
		}
	}
	for (FControlRigForWorldTransforms& ControlRig : ParentControlRigs)
	{
		for (FName& ControlName : ControlRig.ControlNames)
		{
			UMovieSceneControlRigParameterSection* Section = GetControlRigSection(Sequencer.Get(), ControlRig.ControlRig.Get(), ControlName);
			if (Section)
			{
				TArrayView<FMovieSceneFloatChannel*> Channels = FControlRigSequencerHelpers::GetFloatChannels(ControlRig.ControlRig.Get(),
					ControlName, Section);
				TArray<FFrameNumber> TransformFrameTimes = FMovieSceneConstraintChannelHelper::GetTransformTimes(
					Channels, StartFrame, EndFrame);
				for (FFrameNumber& FrameNumber : TransformFrameTimes)
				{
					OutFrameMap.Add(FrameNumber);
				}
			}
		}
	}
}

bool FControlRigSnapper::SnapIt(FFrameNumber StartFrame, FFrameNumber EndFrame,const FControlRigSnapperSelection& ActorToSnap,
	const FControlRigSnapperSelection& ParentToSnap, const UControlRigSnapSettings* SnapSettings)
{
	TWeakPtr<ISequencer> InSequencer = GetSequencer();
	if (InSequencer.IsValid() && InSequencer.Pin()->GetFocusedMovieSceneSequence() && ActorToSnap.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

	
		TArray<FFrameNumber> Frames;
		if (SnapSettings && SnapSettings->BakingKeySettings == EBakingKeySettings::AllFrames)
		{
			MovieSceneToolHelpers::CalculateFramesBetween(MovieScene, StartFrame, EndFrame, SnapSettings->FrameIncrement,Frames);
		}
		else
		{
			TSortedMap<FFrameNumber, FFrameNumber> FrameMap;
			FFrameNumber LocalStartFrame = StartFrame;
			FFrameNumber LocalEndFrame = EndFrame;
			if (EndFrame < StartFrame)
			{
				Swap<FFrameNumber>(LocalStartFrame, LocalEndFrame);
			}
			GetTransformFrames(Sequencer, ParentToSnap, LocalStartFrame, LocalEndFrame, FrameMap);
			GetTransformFrames(Sequencer, ActorToSnap, LocalStartFrame, LocalEndFrame, FrameMap);
			FrameMap.GenerateKeyArray(Frames);
			if (LocalStartFrame == EndFrame) //reverse it
			{
				Algo::Reverse(Frames);
			}

		}
		if (Frames.Num() == 0)
		{
			UE_LOGF(LogControlRig, Error, "No Frames To Snap, probably no keys within time range");
			return false;
		}

		//if in subsquence and we want to, turn on ShouldEvaluateSubSequencesInIsolation, use the following object for that
		//disabling this for nowUE::Sequencer::FSequencerBakingSetupRestore RestoreBaking(Sequencer);
		FScopedTransaction ScopedTransaction(LOCTEXT("SnapAnimation", "Snap Animation"));

		FAllSpawnableRestoreState SpawnableRestoreState(Sequencer->GetSharedPlaybackState().ToSharedPtr());

		TArray<FTransform> WorldTransformToSnap;
		bool bSnapToFirstFrameNotParents = !CalculateWorldTransformsFromParents(Sequencer.Get(), ParentToSnap, Frames, WorldTransformToSnap);
		if (Frames.Num() != WorldTransformToSnap.Num())
		{
			UE_LOGF(LogControlRig, Error, "Number of Frames %d to Snap different than Parent Frames %d", Frames.Num(),WorldTransformToSnap.Num());
			ScopedTransaction.Cancel();
			return false;
		}

		const bool bAvoidEvaluates = (Frames.Num() == 0) || (Frames.Num() == 1 && Frames[0] == Sequencer->GetLocalTime().Time.RoundToFrame());

		//need to make sure binding is setup
		if (bAvoidEvaluates == false)
		{
			Sequencer->ForceEvaluate();
		}
		
		TArray<FGuidAndActor > ActorsToSnap;
		//There may be Actors here not in Sequencer so we add them to sequencer also
		for (const FActorForWorldTransforms& ActorSelection : ActorToSnap.Actors)
		{
			if (ActorSelection.Actor.IsValid())
			{
				AActor* Actor = ActorSelection.Actor.Get();
				FGuid ObjectHandle = Sequencer->GetHandleToObject(Actor,false);
				if (ObjectHandle.IsValid() == false)
				{
					TArray<TWeakObjectPtr<AActor> > ActorsToAdd;
					ActorsToAdd.Add(Actor);
					TArray<FGuid> ActorTracks = Sequencer->AddActors(ActorsToAdd, false);
					if (ActorTracks[0].IsValid())
					{
						ActorsToSnap.Add(FGuidAndActor(ActorTracks[0], Actor));
					}
				}
				else
				{
					ActorsToSnap.Add(FGuidAndActor(ObjectHandle, Actor));
				}
			}
		}

		const bool bKeepOffset = SnapSettings && SnapSettings->bKeepOffset;
		
		//set transforms on these actors
		for (FGuidAndActor& GuidActor : ActorsToSnap)
		{
			if (bSnapToFirstFrameNotParents || bKeepOffset) //if we are snapping to the first frame or keep offset we just don't set the parent transforms
			{
				TArray<FFrameNumber> OneFrame;
				OneFrame.Add(Frames[0]);
				TArray<FTransform> OneTransform;
				FActorForWorldTransforms ActorSelection;
				ActorSelection.Actor = GuidActor.Actor;
				MovieSceneToolHelpers::GetActorWorldTransforms(Sequencer.Get(), ActorSelection, OneFrame, OneTransform);
				if (bSnapToFirstFrameNotParents)
				{
					for (FTransform& Transform : WorldTransformToSnap)
					{
						Transform = OneTransform[0];
					}
				}
				else //we keep offset
				{
					FTransform FirstWorld = WorldTransformToSnap[0];
					for (FTransform& Transform : WorldTransformToSnap)
					{
						Transform = OneTransform[0].GetRelativeTransform(FirstWorld) * Transform;
					}
				}

			}
			GuidActor.SetTransform(Sequencer, Frames, WorldTransformToSnap, SnapSettings);
		}

		//Now do Control Rigs
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FRigControlModifiedContext Context;
		Context.SetKey = EControlRigSetKey::Always;

		TSet<UMovieSceneControlRigParameterSection*> ControlRigSections;
		for (const FControlRigForWorldTransforms& ControlRigAndSelection : ActorToSnap.ControlRigs)
		{
			//get actor transform...
			UControlRig* ControlRig = ControlRigAndSelection.ControlRig.Get();

			if (ControlRig)
			{
				if (TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding())
				{
					USceneComponent* Component = Cast<USceneComponent>(ObjectBinding->GetBoundObject());
					if (!Component)
					{
						continue;
					}
					AActor* Actor = Component->GetTypedOuter< AActor >();
					if (!Actor)
					{
						continue;
					}
					ControlRig->Modify();
					TArray<FTransform> ControlRigParentWorldTransforms;
					FActorForWorldTransforms ControlRigActorSelection;
					ControlRigActorSelection.Actor = Actor;
					MovieSceneToolHelpers::GetActorWorldTransforms(Sequencer.Get(), ControlRigActorSelection, Frames, ControlRigParentWorldTransforms);
					FControlRigSnapper Snapper;
					for (const FName& Name : ControlRigAndSelection.ControlNames)
					{
						TArray<FFrameNumber> OneFrame;
						OneFrame.SetNum(1);
						TArray<FTransform> CurrentControlRigTransform, CurrentParentWorldTransform;
						CurrentControlRigTransform.SetNum(1);
						CurrentParentWorldTransform.SetNum(1);
						if (bSnapToFirstFrameNotParents || bKeepOffset)
						{
							OneFrame[0] = Frames[0];
							CurrentParentWorldTransform[0] = ControlRigParentWorldTransforms[0];
							Snapper.GetControlRigControlTransforms(Sequencer.Get(), ControlRig, Name, OneFrame, CurrentParentWorldTransform, CurrentControlRigTransform);
							if (bSnapToFirstFrameNotParents)
							{
								for (FTransform& Transform : WorldTransformToSnap)
								{
									Transform = CurrentControlRigTransform[0];
								}
							}
							else if (bKeepOffset)
							{
								FTransform FirstWorld = WorldTransformToSnap[0];
								for (FTransform& Transform : WorldTransformToSnap)
								{
									Transform =  CurrentControlRigTransform[0].GetRelativeTransform(FirstWorld) * Transform;
								}
							}
						}

						//only set keys on those specified
						if (SnapSettings)
						{
							Context.KeyMask = 0;

							if (SnapSettings->bSnapPosition == true)
							{
								Context.KeyMask |= (uint32)EControlRigContextChannelToKey::Translation;
							}
							if (SnapSettings->bSnapRotation == true)
							{
								Context.KeyMask |= (uint32)EControlRigContextChannelToKey::Rotation;
							}
							if (SnapSettings->bSnapScale == true)
							{
								Context.KeyMask |= (uint32)EControlRigContextChannelToKey::Scale;
							}
						}

						for (int32 Index = 0; Index < WorldTransformToSnap.Num(); ++Index)
						{
							OneFrame[0] = Frames[Index];
							CurrentParentWorldTransform[0] = ControlRigParentWorldTransforms[Index];
							//this will evaluate at the current frame which we want
							GetControlRigControlTransforms(Sequencer.Get(), ControlRig, Name, OneFrame, CurrentParentWorldTransform, CurrentControlRigTransform);
							if (SnapSettings)
							{
								FTransform& Transform = WorldTransformToSnap[Index];
								const FTransform& CurrentTransform = CurrentControlRigTransform[0];
								if (SnapSettings->bSnapPosition == false)
								{
									FVector CurrentPosition = CurrentTransform.GetLocation();
									Transform.SetLocation(CurrentPosition);
								}
								if (SnapSettings->bSnapRotation == false)
								{
									FQuat CurrentRotation = CurrentTransform.GetRotation();
									Transform.SetRotation(CurrentRotation);
								}
								if (SnapSettings->bSnapScale == false)
								{
									FVector Scale = CurrentTransform.GetScale3D();
									Transform.SetScale3D(Scale);
								}
							}
							const FFrameNumber& FrameNumber = Frames[Index];
							Context.LocalTime = TickResolution.AsSeconds(FFrameTime(FrameNumber));
							FTransform GlobalTransform = WorldTransformToSnap[Index].GetRelativeTransform(ControlRigParentWorldTransforms[Index]);
							if (FRigControlElement* ControlElement = ControlRig->FindControl(Name))
							{
								UE::AIE::FSetTransformHelpers::SetControlGlobalTransform(GlobalTransform, ControlRig, ControlElement, Context);
							}
							UMovieSceneControlRigParameterSection* ControlRigSection = GetControlRigSection(Sequencer.Get(), ControlRig,Name);
							ControlRigSections.Add(ControlRigSection);
						}
					}

					if (SnapSettings && SnapSettings->BakingKeySettings != EBakingKeySettings::KeysOnly
						&& SnapSettings->bReduceKeys == true && Frames.Num() > 2)
					{
						for (UMovieSceneControlRigParameterSection* ControlRigSection : ControlRigSections)
						{
							FKeyDataOptimizationParams Param;
							Param.bAutoSetInterpolation = true;
							Param.Tolerance = SnapSettings->Tolerance;
							TRange<FFrameNumber> Range(Frames[0], Frames[Frames.Num() - 1]);
							Param.Range = Range;
							MovieSceneToolHelpers::OptimizeSection(Param, ControlRigSection);
						}
					}
				}
			}
		}
	}
	return true;
}






#undef LOCTEXT_NAMESPACE



