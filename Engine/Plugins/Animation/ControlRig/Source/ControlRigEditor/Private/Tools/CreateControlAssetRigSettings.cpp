// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/CreateControlAssetRigSettings.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CreateControlAssetRigSettings)


UCreateControlPoseAssetRigSettings::UCreateControlPoseAssetRigSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	AssetName = FString("ControlRigPose");
}

