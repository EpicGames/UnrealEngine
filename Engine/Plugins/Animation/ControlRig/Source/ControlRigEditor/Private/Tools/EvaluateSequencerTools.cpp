// Copyright Epic Games, Inc. All Rights Reserved.


#include "Tools/EvaluateSequencerTools.h"
#include "ControlRig.h"
#include "ISequencer.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneToolHelpers.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "ControlRigObjectBinding.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "ILevelSequenceEditorToolkit.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "Transform/TransformConstraint.h"
#include "ConstraintsManager.h"
#include "Constraints/MovieSceneConstraintChannelHelper.h"
#include "Constraints/ControlRigTransformableHandle.h"
#include "Units/Execution/RigUnit_BeginExecution.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DPathTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "Constraints/TransformConstraintChannelInterface.h"
#include "Components/SkeletalMeshComponent.h"
#include "Transform/TransformConstraintUtil.h"
#include "EditMode/ControlRigEditMode.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/TrackRowModelStorageExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/SequencerOutlinerViewModel.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"

class UControlRig;


namespace UE
{
namespace AIE
{
	FTransform FArrayOfTransforms::Interp(const FFrameNumber& InTime, const TArray<int32>& TransformIndices, const TArray<FFrameNumber>& CurrentFrames)
	{
		FTransform Value = FTransform::Identity;
		const int32 StartIndex = Algo::LowerBound(CurrentFrames, InTime);
		if (StartIndex != INDEX_NONE)
		{
			if (StartIndex >= CurrentFrames.Num())
			{
				const int32 Index = TransformIndices[CurrentFrames.Num() -1];
				Value = Transforms[Index];
			}
			else if (InTime == CurrentFrames[StartIndex] ||
				(StartIndex + 1) == CurrentFrames.Num())
			{
				const int32 Index = TransformIndices[StartIndex];
				Value = Transforms[Index];
			}
			else
			{
				const FFrameNumber Frame1 = CurrentFrames[StartIndex];
				const FFrameNumber Frame2 = CurrentFrames[StartIndex + 1];
				if (Frame1 != Frame2) //should never happen ... but?
				{
					const double FrameDiff = (double)(Frame2.Value - Frame1.Value);
					const double T = (double)(InTime.Value - Frame1.Value) / FrameDiff;
					const int32 Index1 = TransformIndices[StartIndex];
					const int32 Index2 = TransformIndices[StartIndex + 1];

					FTransform KeyAtom1 = Transforms[Index1];
					FTransform KeyAtom2 = Transforms[Index1];

					KeyAtom1.NormalizeRotation();
					KeyAtom2.NormalizeRotation();

					Value.Blend(KeyAtom1, KeyAtom2, T);
				}
			}
		}
		return Value;
	}


	static bool GetActorWorldTransform(const FActorForWorldTransforms& ActorSelection, FTransform& OutTransform, FTransform& OutParentTransform, bool bUpdateAnimation, float InDeltaTime)
	{
		USceneComponent* SceneComponent = nullptr;
		AActor* Actor = ActorSelection.Actor.Get();
		if (Actor)
		{
			SceneComponent = Actor->GetRootComponent();
		}
		else
		{
			SceneComponent = ActorSelection.Component.IsValid() ? Cast<USceneComponent>(ActorSelection.Component.Get()) : nullptr;
			if (SceneComponent)
			{
				Actor = SceneComponent->GetTypedOuter<AActor>();
			}
		}

		if (Actor && SceneComponent)
		{
			USkeletalMeshComponent* SkelMeshComp = Cast<USkeletalMeshComponent>(SceneComponent);
			if (!SkelMeshComp)
			{
				SkelMeshComp = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(Actor);
			}

			if (bUpdateAnimation)
			{
				AActor* Parent = Actor;
				while (Parent)
				{
					TArray<USkeletalMeshComponent*> MeshComps;
					Parent->GetComponents(MeshComps, true);	
					for (USkeletalMeshComponent* MeshComp : MeshComps)
					{
						MeshComp->TickAnimation(InDeltaTime, false);
						MeshComp->RefreshBoneTransforms();
						MeshComp->RefreshFollowerComponents();
						MeshComp->UpdateComponentToWorld();
						MeshComp->FinalizeBoneTransform();
					}

					Parent = Parent->GetAttachParentActor();
				}
			}

			OutTransform = (SkelMeshComp && ActorSelection.SocketName != NAME_None)
				? SkelMeshComp->GetSocketTransform(ActorSelection.SocketName)
				: SceneComponent->GetComponentToWorld();

			AActor* ParentActor = Actor->GetAttachParentActor();
			if (ParentActor)
			{
				SceneComponent = ParentActor->GetRootComponent();
				SkelMeshComp = Cast<USkeletalMeshComponent>(SceneComponent);

				if (!SkelMeshComp)
				{
					SkelMeshComp = MovieSceneToolHelpers::AcquireSkeletalMeshFromObject(Actor);
				}
				FName SocketName = Actor->GetAttachParentSocketName();

				OutParentTransform = (SkelMeshComp && SocketName != NAME_None)
					? SkelMeshComp->GetSocketTransform(SocketName)
					: SceneComponent->GetComponentToWorld();
			}
			else
			{
				OutParentTransform = FTransform::Identity;	
			}
			return true;
		}
		return false;
	}

	bool FEvalHelpers::EvaluateSequencer(UWorld* World, ISequencer* Sequencer, const TSet<const UMovieSceneTrack*>& RelevantTracks)
	{
		if (World == nullptr || Sequencer == nullptr || Sequencer->GetFocusedMovieSceneSequence() == nullptr
			|| Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
		{
			return false;
		}
		
		FAllSpawnableRestoreState SpawnableRestoreState(Sequencer->GetSharedPlaybackState().ToSharedPtr());

		const FQualifiedFrameTime QFrameTime = Sequencer->GetGlobalTime();
		FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(QFrameTime.Time, QFrameTime.Rate), Sequencer->GetPlaybackStatus()).SetHasJumped(true);

		//have no way to just evaluate specific tracks in Sequencer so do all of them.
		Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		Controller.EvaluateAllConstraints();

		for (const UMovieSceneTrack* Track : RelevantTracks)
		{
			if (const UMovieSceneControlRigParameterTrack* CRTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
			{
				if (UControlRig* ControlRig = CRTrack->GetControlRig())
				{
					ControlRig->Evaluate_AnyThread();
					if (ControlRig->GetObjectBinding())
					{
						const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding();
						USceneComponent* Component = ObjectBinding ? Cast<USceneComponent>(ObjectBinding->GetBoundObject()) : nullptr;
						if (AActor* Actor = Component ? Component->GetTypedOuter< AActor >() : nullptr)
						{
							constexpr bool bUpdateAnimation = true;
							constexpr float DeltaTime = 0.0f; //no physics just resetting
							FControlRigWorldTransformProvider Provider(Actor, Component, ControlRig, DeltaTime);
							//this updates all skel meshes
							Provider.GetComponentTransform(bUpdateAnimation);
						}
					}
				}
			}
		}
		return true;
	}


	bool FEvalHelpers::CalculateWorldTransforms(UWorld* World, ISequencer* Sequencer, const UE::AIE::FFrameTimeByIndex& FrameTimeByIndex,
		const TArray<int32>& Indices, TArray<UE::AIE::FActorAndWorldTransforms>& Actors, TMap<UControlRig*, UE::AIE::FControlRigAndWorldTransforms>& ControlRigs)
	{
		if (World == nullptr || Sequencer == nullptr || Sequencer->GetFocusedMovieSceneSequence() == nullptr
			|| Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene() == nullptr)
		{
			return false;
		}
		FMovieSceneSequenceIDRef Template = Sequencer->GetFocusedTemplateID();
		FMovieSceneSequenceTransform RootToLocalTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped();
		FMovieSceneInverseSequenceTransform LocalToRootTransform = RootToLocalTransform.Inverse();
		UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(World);
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		TOptional<TRange<FFrameNumber>> OptionalRange = Sequencer->GetSubSequenceRange();
		const FFrameNumber StartFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetLowerBoundValue() : MovieScene->GetPlaybackRange().GetLowerBoundValue();
		const FFrameNumber EndFrame = OptionalRange.IsSet() ? OptionalRange.GetValue().GetUpperBoundValue() : MovieScene->GetPlaybackRange().GetUpperBoundValue();

		FAllSpawnableRestoreState SpawnableRestoreState(Sequencer->GetSharedPlaybackState().ToSharedPtr());

		const FControlRigEditMode::FTurnOffPosePoseUpdate  TurnOff; //stop flashing

		const float DisplayRate = static_cast<float>(MovieScene->GetDisplayRate().AsDecimal());
		const float DeltaTime = DisplayRate > 0.f ? 1.f / DisplayRate : 1.f / 30.f;

		// track actors already evaluated to avoid ticking them multiple times (for performance reasons)
		TSet<AActor*> CachedActors;
		auto ShouldUpdateAnimation = [&CachedActors](AActor* Actor)
		{
			AActor* Parent = Actor;
			while (Parent)
			{
				if (CachedActors.Contains(Parent))
				{
					return false;
				}
				Parent = Parent->GetAttachParentActor();
			}
			return true;
		};
		
		for (int32 Index : Indices)
		{
			CachedActors.Reset();
			
			FFrameNumber FrameNumber = FrameTimeByIndex.CalculateFrame(Index);
			if (FrameNumber < StartFrame || FrameNumber > EndFrame)
			{
				continue; //if out of range bail
			}
			if (FrameNumber == EndFrame && FrameNumber != StartFrame) //if on end boundary subtract one tick to handle subsequence eval issues
			{
				--FrameNumber;
			}
			FFrameTime GlobalTime = LocalToRootTransform.TryTransformTime(FrameNumber).Get(FrameNumber); //player evals in root time so need to go back to it.

			FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
			Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
			{
				const Anim::FEvaluationForCachingScope CachingScope(DeltaTime);
				Controller.EvaluateAllConstraints();
			}

			for (FActorAndWorldTransforms& ActorAndTransforms : Actors)
			{
				const FActorForWorldTransforms& ActorData = ActorAndTransforms.Actor;
				auto GetResolvedActor = [&ActorData]() -> AActor*
				{
					if (AActor* Actor = ActorData.Actor.Get())
					{
						return Actor;
					}
					if (USceneComponent* Component = ActorData.Component.Get())
					{
						return Component->GetTypedOuter<AActor>();
					}
					return nullptr;
				};
				
				if (AActor* Actor = GetResolvedActor())
				{
					FTransform& Transform = ActorAndTransforms.WorldTransforms->Transforms[Index];
					FTransform& ParentTransform = ActorAndTransforms.ParentTransforms->Transforms[Index];

					const bool bUpdateAnimation = ShouldUpdateAnimation(Actor);
					GetActorWorldTransform(ActorData, Transform, ParentTransform, bUpdateAnimation, DeltaTime);
					CachedActors.Add(Actor);
				}
			}

			for (TPair<UControlRig*, FControlRigAndWorldTransforms>& CR : ControlRigs)
			{
				if (UControlRig* ControlRig = CR.Value.ControlRig.Get())
				{
					const TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding();
					USceneComponent* Component = ObjectBinding ? Cast<USceneComponent>(ObjectBinding->GetBoundObject()) : nullptr;
					if (AActor* Actor = Component ? Component->GetTypedOuter< AActor >() : nullptr)
					{
						bool bUpdateAnimation = ShouldUpdateAnimation(Actor);
						
						FControlRigWorldTransformProvider Provider(Actor,  Component,  ControlRig, DeltaTime);
						for (TPair<FName, TSharedPtr<FArrayOfTransforms>>& Pair : CR.Value.ControlAndWorldTransforms)
						{
							const FName ControlName = Pair.Key;
							FTransform& Transform = Pair.Value->Transforms[Index];
							Transform = Provider.GetTransform(ControlName, bUpdateAnimation);
							bUpdateAnimation = false;
						}

						FTransform& ParentTransform = CR.Value.ParentTransforms->Transforms[Index];
						ParentTransform = Provider.GetComponentTransform(bUpdateAnimation);

						CachedActors.Add(Actor);
					}
				}
			}
		}
		return true;
	}

	static FTransform GetThisControlRigComponentTransform(UControlRig* ControlRig)
	{
		FTransform Transform = FTransform::Identity;
		TSharedPtr<IControlRigObjectBinding> ObjectBinding = ControlRig->GetObjectBinding();
		if (ObjectBinding.IsValid())
		{
			if (USceneComponent* BoundSceneComponent = Cast<USceneComponent>(ObjectBinding->GetBoundObject()))
			{
				return BoundSceneComponent->GetComponentTransform();
			}
		}
		return Transform;
	}

	bool FSetTransformHelpers::SetConstrainedTransform(FTransform LocalTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& InContext)
	{
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(ControlRig->GetWorld());
		const uint32 ControlHash = UTransformableControlHandle::ComputeHash(ControlRig, ControlElement->GetFName());
		const TArray< TWeakObjectPtr<UTickableConstraint> > Constraints = Controller.GetParentConstraints(ControlHash, true);
		if (Constraints.IsEmpty())
		{
			return false;
		}
		const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(Constraints);
		const bool bNeedsConstraintPostProcess = Constraints.IsValidIndex(LastActiveIndex);

		if (!bNeedsConstraintPostProcess)
		{
			return false;
		}
		static constexpr bool bNotify = true, bFixEuler = true, bUndo = true;
		FRigControlModifiedContext Context = InContext;
		Context.EventName = FRigUnit_BeginExecution::EventName;
		Context.bConstraintUpdate = true;
		Context.SetKey = EControlRigSetKey::Never;

		// set the global space, assumes it's attached to actor
		// no need to compensate for constraints here, this will be done after when setting the control in the constraint space
		{
			TGuardValue<bool> CompensateGuard(FMovieSceneConstraintChannelHelper::bDoNotCompensate, true);
			ControlRig->SetControlLocalTransform(
				ControlElement->GetKey().Name, LocalTransform, bNotify, Context, bUndo, bFixEuler);
		}
		FTransform GlobalTransform = ControlRig->GetControlGlobalTransform(ControlElement->GetKey().Name);

		// switch to constraint space
		FTransform ToWorldTransform = GetThisControlRigComponentTransform(ControlRig);
		const FTransform WorldTransform = GlobalTransform * ToWorldTransform;

		const TOptional<FTransform> RelativeTransform =
			UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, LocalTransform, WorldTransform);
		if (RelativeTransform)
		{
			LocalTransform = *RelativeTransform;
		}

		Context.bConstraintUpdate = false;
		Context.SetKey = InContext.SetKey;
		ControlRig->SetControlLocalTransform(ControlElement->GetKey().Name, LocalTransform, bNotify, Context, bUndo, bFixEuler);
		ControlRig->Evaluate_AnyThread();
		Controller.EvaluateAllConstraints();

		return true;
	}


	void FSetTransformHelpers::SetControlTransform(const FEulerTransform& InLocalEulerTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context)
	{
		if (ControlElement && ControlRig)
		{
			FVector TLocation = InLocalEulerTransform.GetLocation();
			FRotator TRotation = InLocalEulerTransform.Rotation;
			FVector TScale = InLocalEulerTransform.Scale;
			FTransform LocalTransform(TRotation, TLocation, TScale);
			if (SetConstrainedTransform(LocalTransform, ControlRig, ControlElement, Context))
			{
				return;
			}

			ControlRig->SetControlLocalTransform(ControlElement->GetKey().Name, LocalTransform, true, Context, false /*undo*/, true/* bFixEulerFlips*/);
			ControlRig->Evaluate_AnyThread();
		}
	}

	void FSetTransformHelpers::SetControlGlobalTransform(const FTransform& GlobalTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context)
	{
		if (ControlElement && ControlRig)
		{
			FTransform LocalGlobal = GlobalTransform;
			constexpr ERigTransformType::Type TransformType = ERigTransformType::CurrentGlobal;
			FRigControlValue RigValue = ControlRig->GetControlValueFromGlobalTransform(ControlElement->GetKey().Name, LocalGlobal, TransformType);
			LocalGlobal = RigValue.GetAsTransform(ControlElement->Settings.ControlType, ControlElement->Settings.PrimaryAxis);
			FEulerTransform EulerTransform(LocalGlobal);
			UE::AIE::FSetTransformHelpers::SetControlTransform(EulerTransform, ControlRig, ControlElement, Context);
		}
	}
	

	bool FSetTransformHelpers::SetActorTransform(ISequencer* Sequencer, USceneComponent* SceneComponent, UMovieScene3DTransformSection* TransformSection, const TArray<FFrameNumber>& Frames,
		const TArray<FTransform>& WorldTransformsToSnapTo, const TArray<FTransform>& ParentWorldTransforms)
	{

		if (!Sequencer || !Sequencer->GetFocusedMovieSceneSequence())
		{
			return false;
		}
		if (!TransformSection)
		{
			return false;
		}
		if (!SceneComponent)
		{
			return false;
		}

		AActor* Actor = SceneComponent->GetTypedOuter<AActor>();

		TransformSection->Modify();

		FMovieSceneInverseSequenceTransform LocalToRootTransform = Sequencer->GetFocusedMovieSceneSequenceTransformUnwarped().Inverse();
		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();

		//adjust keys for constraints
		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(Actor->GetWorld());
		TArrayView<FMovieSceneDoubleChannel*> Channels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
		const bool bhasContraint = TransformSection->GetConstraintsChannels().Num() > 0;
		FControlRigEditMode::FTurnOffPosePoseUpdate  TurnOff; //stop flashing

		for (int32 Index = 0; Index < Frames.Num(); ++Index)
		{
			const FFrameNumber& Frame = Frames[Index];
			FTransform ParentTransform = ParentWorldTransforms.Num() > 0 ? ParentWorldTransforms[Index] : FTransform::Identity;
			FTransform WorldTransform = WorldTransformsToSnapTo[Index];
			FTransform LocalTransform = WorldTransform.GetRelativeTransform(ParentTransform);
			
			//todo test FFrameTime GlobalTime = LocalToRootTransform.TryTransformTime(Frame).Get(Frame);
			FFrameTime GlobalTime(Frame);
			const FFrameNumber GlobalFrame = GlobalTime.GetFrame();

			/*
			// Account for the transform origin only if this is not parented because the transform origin is already being applied to the parent.
			if (!SceneComponent->GetAttachParent() && Section->GetBlendType() == EMovieSceneBlendType::Absolute)
			{
				CurrentTransform *= GetTransformOrigin().Inverse();
			}
			*/

			if (bhasContraint)
			{
				FMovieSceneContext Context = FMovieSceneContext(FMovieSceneEvaluationRange(GlobalTime, TickResolution), Sequencer->GetPlaybackStatus()).SetHasJumped(true);
				Sequencer->GetEvaluationTemplate().EvaluateSynchronousBlocking(Context);
				Controller.EvaluateAllConstraints();
				//UE::TransformConstraintUtil::UpdateTransformBasedOnConstraint(LocalTransform, SceneComponent);
			//	if (AActor* Actor = SceneComponent->GetTypedOuter<AActor>())
				{
					TArray< TWeakObjectPtr<UTickableConstraint> > Constraints;
					UE::TransformConstraintUtil::GetParentConstraints(SceneComponent->GetWorld(), Actor, Constraints);

					const int32 LastActiveIndex = UE::TransformConstraintUtil::GetLastActiveConstraintIndex(Constraints);
					if (Constraints.IsValidIndex(LastActiveIndex))
					{
						// switch to constraint space
						const TOptional<FTransform> RelativeTransform = UE::TransformConstraintUtil::GetConstraintsRelativeTransform(Constraints, LocalTransform, WorldTransform);
						if (RelativeTransform)
						{
							LocalTransform = *RelativeTransform;
						}
					}
				}
			}		

			FVector Location = LocalTransform.GetLocation();
			FRotator Rotation = LocalTransform.GetRotation().Rotator();

			TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = Channels[0]->GetData();
			MovieSceneToolHelpers::SetOrAddKey(ChannelData, GlobalFrame, Location.X);
			ChannelData = Channels[1]->GetData();
			MovieSceneToolHelpers::SetOrAddKey(ChannelData, GlobalFrame, Location.Y);
			ChannelData = Channels[2]->GetData();
			MovieSceneToolHelpers::SetOrAddKey(ChannelData, GlobalFrame, Location.Z);
		}

		Channels[0]->AutoSetTangents();
		Channels[1]->AutoSetTangents();
		Channels[2]->AutoSetTangents();

		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
		return true;
	}

	FMovieSceneBinding FSequencerTransformDependencies::GetBindingFromTrack(UMovieScene* InMovieScene, UMovieSceneTrack* Track)
	{
		const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)InMovieScene)->GetBindings();
		for (const FMovieSceneBinding& Binding : Bindings)
		{
			for (UMovieSceneTrack* PossibleTrack : Binding.GetTracks())
			{
				if (PossibleTrack == Track)
				{
					return Binding;
				}
			}
		}
		FMovieSceneBinding EmptyBinding;
		return EmptyBinding;
	}

	TArray<UMovieSceneTrack*> FSequencerTransformDependencies::GetTransformAffectingTracks(UMovieScene* InMovieScene, const FMovieSceneBinding& Binding)
	{
		TArray<UMovieSceneTrack*> TransformTracks;
		for (UMovieSceneTrack* PossibleTrack : Binding.GetTracks())
		{
			if (PossibleTrack)
			{
				if (PossibleTrack->GetClass() == UMovieScene3DTransformTrack::StaticClass() ||
					PossibleTrack->GetClass() == UMovieSceneControlRigParameterTrack::StaticClass() ||
					PossibleTrack->GetClass() == UMovieScene3DAttachTrack::StaticClass() ||
					PossibleTrack->GetClass() == UMovieScene3DPathTrack::StaticClass() ||
					PossibleTrack->GetClass() == UMovieSceneSkeletalAnimationTrack::StaticClass()
					)
				{
					TransformTracks.Add(PossibleTrack);
				}
			}
		}
		return TransformTracks;
	}

	void FSequencerTransformDependencies::CalculateDependencies(ISequencer* InSequencer, AActor* InActor, TArray<UMovieSceneTrack*>& InTracks)
	{
		Tracks.Reset();
		SequencerActors.Reset();
		NonSequencerActors.Reset();
		if (InSequencer == nullptr)
		{
			return;
		}
		UMovieSceneSequence* Sequence = InSequencer->GetFocusedMovieSceneSequence();
		if (Sequence == nullptr || Sequence->GetMovieScene() == nullptr)
		{
			return;
		}
		//find non-sequencer parents
		TArray<const UObject*> Parents;
		MovieSceneToolHelpers::GetParents(Parents, InActor);
		//find non sequencer parents
		for (const UObject* Parent : Parents)
		{
			//unfortunately uses non-const
			if (Parent != InActor)
			{
				UObject* NonConstObj = const_cast<UObject*>(Parent);
				FGuid Binding = Sequence->FindBindingFromObject(NonConstObj, InSequencer->GetSharedPlaybackState());
				if (Binding.IsValid() == false)
				{
					if (AActor* Actor = Cast<AActor>(NonConstObj))
					{
						NonSequencerActors.Add(Actor);
					}
					else if (AActor* OuterActor = NonConstObj->GetTypedOuter<AActor>())
					{
						NonSequencerActors.Add(OuterActor);
					}
				}
				else
				{
					if (AActor* Actor = Cast<AActor>(NonConstObj))
					{
						SequencerActors.Add(Actor, Binding);
					}
					else if (AActor* OuterActor = NonConstObj->GetTypedOuter<AActor>())
					{
						SequencerActors.Add(OuterActor, Binding);
					}
				}
			}
		}
		//now for each track see if has any constraints/spaces
		for (UMovieSceneTrack* Track : InTracks)
		{
			AddTrack(InSequencer, Track);
		}
		//now get tracks from parents, only add them though if not in the list, they may be there because of attach/constraint tracks and current evaluation
		UMovieScene* MovieScene = Sequence->GetMovieScene();

		for (TPair<AActor*,FGuid>& Pair : SequencerActors)
		{
			if (FMovieSceneBinding* Binding = MovieScene->FindBinding(Pair.Value))
			{
				TArray<UMovieSceneTrack*> TransformTracks = GetTransformAffectingTracks(MovieScene, *Binding);
				for (UMovieSceneTrack* PossibleTrack : TransformTracks)
				{
					if (Tracks.Contains(PossibleTrack) == false)
					{
						AddTrack(InSequencer, PossibleTrack);
					}
				}
			}
		}
	}

	void FSequencerTransformDependencies::CopyFrom(const FSequencerTransformDependencies& Other)
	{
		SequencerActors = Other.SequencerActors;
		NonSequencerActors = Other.NonSequencerActors;
		Tracks = Other.Tracks;
	}


	bool FSequencerTransformDependencies::Compare(FSequencerTransformDependencies& Other) const
	{
		if (SequencerActors.Num() != Other.SequencerActors.Num())
		{
			return false;
		}
		else
		{
			for (TPair<AActor*, FGuid>& Pair : Other.SequencerActors)
			{
				if (SequencerActors.Contains(Pair.Key) == false)
				{
					return false;
				}
			}
		}
		if (NonSequencerActors.Num() != Other.NonSequencerActors.Num())
		{
			return false;
		}
		else
		{
			for (AActor* Actor : Other.NonSequencerActors)
			{
				if (NonSequencerActors.Contains(Actor) == false)
				{
					return false;
				}
			}
		}
		if (Tracks.Num() != Other.Tracks.Num())
		{
			return false;
		}
		else
		{
			for (TPair<TWeakObjectPtr<UMovieSceneTrack>,FGuid>& Pair : Other.Tracks)
			{
				const FGuid* Guid = Tracks.Find(Pair.Key);
				if (Guid == nullptr)
				{
					return false;
				}
				else
				{
					if (*Guid != Pair.Value)
					{
						return false;
					}
				}
			}
		}
		//get here then the same
		return true;
	}

	//track may be an attach/consraint track or a cr/tranfsorm track with constraint channels
	void FSequencerTransformDependencies::CalculateTrackDependents(ISequencer* InSequencer, UMovieSceneTrack* InTrack)
	{
		UMovieScene* MovieScene = InSequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		if (UMovieScene3DConstraintTrack* ConstraintTrack = Cast<UMovieScene3DConstraintTrack>(InTrack))
		{
			for (UMovieSceneSection* ConstraintSection : ConstraintTrack->GetAllSections())
			{
				FMovieSceneObjectBindingID ConstraintBindingID = (Cast<UMovieScene3DConstraintSection>(ConstraintSection))->GetConstraintBindingID();
				if (FMovieSceneBinding* Binding = MovieScene->FindBinding(ConstraintBindingID.GetGuid()))
				{
					TArray<UMovieSceneTrack*> TransformTracks = GetTransformAffectingTracks(MovieScene,*Binding);
					for (UMovieSceneTrack* PossibleTrack : TransformTracks)
					{
						AddTrack(InSequencer, PossibleTrack); 

					}
				}
			}
		}
		for (UMovieSceneSection* Section : InTrack->GetAllSections())
		{
			if (IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(Section))
			{
				for (FConstraintAndActiveChannel& ConstraintChannel : ConstrainedSection->GetConstraintsChannels())
				{
					if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
					{
						const FConstraintChannelInterfaceRegistry& InterfaceRegistry = FConstraintChannelInterfaceRegistry::Get();
						ITransformConstraintChannelInterface* ParentInterface = InterfaceRegistry.FindConstraintChannelInterface(TransformConstraint->ParentTRSHandle->GetClass());
						if (ParentInterface)
						{
							if (UMovieSceneSection* ParentSection = ParentInterface->GetHandleSection(TransformConstraint->ParentTRSHandle, InSequencer->AsShared()))
							{
								if (UMovieSceneTrack* ParentTrack = ParentSection->GetTypedOuter<UMovieSceneTrack>())
								{
									AddTrack(InSequencer, ParentTrack); //always add this track it may be a control rig track
									FMovieSceneBinding Binding = GetBindingFromTrack(MovieScene, ParentTrack);
									if (Binding.GetObjectGuid().IsValid())
									{
										TArray<UMovieSceneTrack*> TransformTracks = GetTransformAffectingTracks(MovieScene, Binding);
										for (UMovieSceneTrack* PossibleTrack : TransformTracks)
										{
											AddTrack(InSequencer, PossibleTrack); 
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
	void FSequencerTransformDependencies::AddTrack(ISequencer* InSequencer, UMovieSceneTrack* InTrack)
	{
		if (Tracks.Contains(InTrack) == false)
		{
			Tracks.Add(InTrack, InTrack->GetSignature());
			CalculateTrackDependents(InSequencer, InTrack);
		}
	}

	TArray<FGuid> FSequencerSelected::GetSelectedOutlinerGuids(ISequencer* SequencerPtr)
	{
		using namespace UE::Sequencer;
		TArray<FGuid> SelectedObjects;// = SequencerPtr->GetViewModel()->GetSelection()->GetBoundObjectsGuids(); //this is not exposed :(

		for (TViewModelPtr<FObjectBindingModel> ObjectBindingNode : SequencerPtr->GetViewModel()->GetSelection()->Outliner.Filter<FObjectBindingModel>())
		{
			FGuid Guid = ObjectBindingNode->GetObjectGuid();
			SelectedObjects.Add(Guid);
		}
		return SelectedObjects;
	}

	void FSequencerSelected::GetSelectedControlRigsAndBoundObjects(ISequencer* SequencerPtr, TArray<FControlRigAndControlsAndTrack>& OutSelectedCRs, TArray<FObjectAndTrack>& OutBoundObjects)
	{
		if (SequencerPtr == nullptr || SequencerPtr->GetViewModel() == nullptr)
		{
			return;
		}
		ULevelSequence* LevelSequence = Cast<ULevelSequence>(SequencerPtr->GetFocusedMovieSceneSequence());
		if (LevelSequence == nullptr)
		{
			return;
		}

		using namespace UE::Sequencer;
		TArray<FGuid> SelectedObjects = GetSelectedOutlinerGuids(SequencerPtr);;// = SequencerPtr->GetViewModel()->GetSelection()->GetBoundObjectsGuids();

		const UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (MovieScene)
		{
			const TArray<FMovieSceneBinding>& Bindings = MovieScene->GetBindings();
			for (const FMovieSceneBinding& Binding : Bindings)
			{
				bool bHaveControlRig = false;
				TArray<UMovieSceneTrack*> CRTracks = MovieScene->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
				for (UMovieSceneTrack* AnyOleTrack : CRTracks)
				{
					if (UMovieSceneControlRigParameterTrack* Track = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack))
					{
						if (UControlRig* ControlRig = Track->GetControlRig())
						{
							FControlRigAndControlsAndTrack CRControls;
							CRControls.ControlRig = ControlRig;
							CRControls.Track = Track;
							CRControls.Controls = ControlRig->CurrentControlSelection();
							if (CRControls.Controls.Num() > 0)
							{
								bHaveControlRig = true;
								OutSelectedCRs.Add(CRControls);
							}
						}
					}
				}
				//if we have control rig controls don't add the base skel mesh for now
				if (bHaveControlRig == false && SelectedObjects.Contains(Binding.GetObjectGuid()))
				{
					TArray<UMovieSceneTrack*> Tracks = MovieScene->FindTracks(UMovieScenePropertyTrack::StaticClass(), Binding.GetObjectGuid(), NAME_None);
					if (Tracks.Num() < 1)
					{
						continue;
					}
					const FMovieSceneSequenceID& SequenceID = SequencerPtr->GetEvaluationState()->FindSequenceId(LevelSequence);

					for (UMovieSceneTrack* AnyOleTrack : Tracks)
					{
						if (UMovieScenePropertyTrack* Track = Cast<UMovieScenePropertyTrack>(AnyOleTrack))
						{
							const FMovieSceneBlendTypeField SupportedBlendTypes = Track->GetSupportedBlendTypes();
							if (SupportedBlendTypes.Num() == 0)
							{
								continue;
							}
							const TArrayView<TWeakObjectPtr<UObject>>& BoundObjects = SequencerPtr->FindBoundObjects(Binding.GetObjectGuid(), SequenceID);
							for (const TWeakObjectPtr<UObject> CurrentBoundObject : BoundObjects)
							{
								if (UObject* BoundObject = CurrentBoundObject.Get())
								{
									FObjectAndTrack ObjectAndTrack;
									ObjectAndTrack.BoundObject = BoundObject;
									ObjectAndTrack.Track = Track;
									ObjectAndTrack.SequencerGuid = Binding.GetObjectGuid();
									OutBoundObjects.Add(ObjectAndTrack);
								}
							}
						}
					}
				}
			}
		}
	}


	bool FControlRigKeys::GetStartEndIndicesForControl(UMovieSceneControlRigParameterSection* BaseSection, const FRigControlElement* ControlElement, int& OutStartIndex, int& OutEndIndex)
	{
		if (FChannelMapInfo* pChannelIndex = BaseSection->ControlChannelMap.Find(ControlElement->GetFName()))
		{
			const int32 ChannelIndex = pChannelIndex->ChannelIndex;
			switch (ControlElement->Settings.ControlType)
			{
			case ERigControlType::Bool:
			case ERigControlType::Integer:
			case ERigControlType::Float:
			case ERigControlType::ScaleFloat:
			{
				OutStartIndex = ChannelIndex;
				OutEndIndex = ChannelIndex;
				break;
			}
			case ERigControlType::Vector2D:
			{
				OutStartIndex = ChannelIndex;
				OutEndIndex = ChannelIndex + 1;
				break;
			}
			case ERigControlType::Position:
			case ERigControlType::Scale:
			case ERigControlType::Rotator:
			{
				OutStartIndex = ChannelIndex;
				OutEndIndex = ChannelIndex + 2;
				break;
			}

			case ERigControlType::Transform:
			case ERigControlType::TransformNoScale:
			case ERigControlType::EulerTransform:
			{
				if (ControlElement->Settings.ControlType == ERigControlType::TransformNoScale)
				{
					OutStartIndex = ChannelIndex;
					OutEndIndex = ChannelIndex + 5;

				}
				else
				{
					OutStartIndex = ChannelIndex;
					OutEndIndex = ChannelIndex + 8;
				}
				break;
			}
			default:
				break;
			}
			return true;
		}
		return false;
	}

	FControlRigWorldTransformProvider::FControlRigWorldTransformProvider(
		AActor* InActor, USceneComponent* InComponent, UControlRig* InControlRig, float InDeltaTime)
		: Actor(InActor)
		, Component(InComponent)
		, ControlRig(InControlRig)
		, DeltaTime(InDeltaTime)
	{
		ensure(IsValid());
	}

	bool FControlRigWorldTransformProvider::IsValid() const
	{
		return Actor.IsValid() && Component.IsValid() && ControlRig.IsValid();
	}
						
	FTransform FControlRigWorldTransformProvider::GetTransform(const FName& InControlName, const bool bUpdateAnimation) const
	{
		if (IsValid() && ControlRig->FindControl(InControlName))
		{
			const FTransform ComponentTransform = GetComponentTransform(bUpdateAnimation);		
			return ControlRig->GetControlGlobalTransform(InControlName) * ComponentTransform;
		}
		return FTransform::Identity;
	}

	FTransform FControlRigWorldTransformProvider::GetComponentTransform(const bool bUpdateAnimation) const
	{
		if (IsValid())
		{
			if (bUpdateAnimation)
			{
				AActor* Parent = Actor.Get();
				while (Parent)
				{
					TArray<USkeletalMeshComponent*> MeshComps;
					Parent->GetComponents(MeshComps, true);

					for (USkeletalMeshComponent* MeshComp : MeshComps)
					{
						MeshComp->TickAnimation(DeltaTime, false);
						MeshComp->RefreshBoneTransforms();
						MeshComp->RefreshFollowerComponents();
						MeshComp->UpdateComponentToWorld();
						MeshComp->FinalizeBoneTransform();
					}

					Parent = Parent->GetAttachParentActor();
				}
			}
			
			return Component->GetComponentToWorld();
		}
		return FTransform::Identity;
	}

} // namespace AIE
} // namespace UE

