// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigAssetReference.h"
#include "Filters/CurveEditorSmartReduceFilter.h"

#define UE_API CONTROLRIGEDITOR_API

class FMenuBuilder;
class ISequencer;
class UMovieScene;
class UAnimSequence;
class UAnimSeqExportOption;
class USkeletalMeshComponent;
class USkeleton;
class UControlRig;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;

namespace UE::ControlRig
{

/** Result returned by InitControlRigFromAnimSequence / HandleBakeToControlRig.
 *  Track, Section, and ControlRig are only guaranteed valid when bSuccess is true. */
struct FBakeToControlRigResult
{
	UMovieSceneControlRigParameterTrack* Track = nullptr;
	UMovieSceneControlRigParameterSection* Section = nullptr;
	UControlRig* ControlRig = nullptr;
	bool bSuccess = false;
};

/**
 * Delegate that callers provide to populate a temp UAnimSequence with animation data.
 * Called after the user confirms the bake options dialog.
 * Return true on success, false to abort the bake.
 *
 * CR track editor: fills via MovieSceneToolHelpers::ExportToAnimSequence (from skel mesh).
 * Mixer track editor: fills via ExportMixerToAnimSequence (from EvaluateRange).
 */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FPopulateAnimSequenceDelegate,
	UAnimSequence* /*AnimSequence*/,
	UAnimSeqExportOption* /*ExportOptions*/,
	USkeletalMeshComponent* /*SkelMeshComp*/);

/**
 * Low-level helper: given an already-created track and a pre-populated anim sequence,
 * creates the control rig instance, validates it, initializes it, creates the section,
 * loads the animation, and optionally reduces keys.
 *
 * Callers are responsible for creating/placing the track, managing edit mode,
 * binding delegates, disabling source tracks, and notifying sequencer.
 */
UE_API FBakeToControlRigResult InitControlRigFromAnimSequence(
	const TSharedPtr<ISequencer>& Sequencer,
	UMovieScene* MovieScene,
	UMovieSceneControlRigParameterTrack* Track,
	const FControlRigAssetStrongReference& AssetReference,
	UObject* BoundActor,
	USkeletalMeshComponent* SkelMeshComp,
	UAnimSequence* AnimSequence,
	bool bReduceKeys,
	const FSmartReduceParams& SmartReduceParams);

/**
 * Builds the "Filter Asset By Skeleton" toggle and "Bake To Control Rig" submenu
 * into the provided MenuBuilder. When the user picks a rig, the full bake flow
 * runs automatically: options dialog, PopulateDelegate to fill the anim sequence,
 * TrackFactory to create/place the track, InitControlRigFromAnimSequence, then OnComplete.
 *
 * @param bFilterAssetBySkeleton  Mutable reference — caller owns the bool (persists across menu opens).
 * @param PopulateDelegate        Fills the temp anim sequence (caller-specific).
 * @param TrackFactory            Creates and returns the CR track (standalone AddTrack or mixer AddChildTrack).
 * @param OnComplete              Called after successful bake for caller-specific post-processing
 *                                (edit mode, BindControlRig, muting source tracks, etc.).
 */
UE_API void BuildBakeToControlRigMenu(
	FMenuBuilder& MenuBuilder,
	USkeleton* Skeleton,
	bool& bFilterAssetBySkeleton,
	const FPopulateAnimSequenceDelegate& PopulateDelegate,
	const TSharedPtr<ISequencer>& Sequencer,
	FGuid ObjectBinding,
	UObject* BoundActor,
	USkeletalMeshComponent* SkelMeshComp,
	TFunction<UMovieSceneControlRigParameterTrack*(UMovieScene*)> TrackFactory,
	TFunction<void(const FBakeToControlRigResult&)> OnComplete = nullptr,
	const FText& SubmenuLabelOverride = FText::GetEmpty());

/**
 * Full bake flow: shows options dialog, calls PopulateDelegate, creates track via
 * TrackFactory, initializes rig via InitControlRigFromAnimSequence.
 * Called internally by BuildBakeToControlRigMenu, but also usable directly.
 */
UE_API FBakeToControlRigResult HandleBakeToControlRig(
	const FControlRigAssetStrongReference& AssetReference,
	const TSharedPtr<ISequencer>& Sequencer,
	FGuid ObjectBinding,
	UObject* BoundActor,
	USkeletalMeshComponent* SkelMeshComp,
	USkeleton* Skeleton,
	const FPopulateAnimSequenceDelegate& PopulateDelegate,
	TFunction<UMovieSceneControlRigParameterTrack*(UMovieScene*)> TrackFactory);

} // namespace UE::ControlRig

#undef UE_API
