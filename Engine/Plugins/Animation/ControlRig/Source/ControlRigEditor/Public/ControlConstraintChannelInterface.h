// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Constraints/TransformConstraintChannelInterface.h"

class ISequencer;
class UMovieSceneControlRigParameterSection;
class IMovieSceneConstrainedSection;
class UMovieSceneSection;
class UTransformableHandle;
class UTickableTransformConstraint;
class UTransformableControlHandle;
struct FFrameNumber;

/**
* Interface that defines animatable capabilities for UTransformableControlHandle
*/

class UMovieSceneControlRigParameterSection;
class UTransformableControlHandle;

struct FControlConstraintChannelInterface : ITransformConstraintChannelInterface
{
	virtual ~FControlConstraintChannelInterface() override = default;

	/** Get the UMovieSceneControlRigParameterSection from the ControlRig. */
	
	virtual UMovieSceneSection* GetHandleSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer) override;
	virtual UMovieSceneSection* GetHandleConstraintSection(const UTransformableHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer) override;

	/** Get the world from the ControlRig. */
	virtual UWorld* GetHandleWorld(UTransformableHandle* InHandle) override;

	/** Add an active/inactive key to the constraint channel if needed and does the transform compensation on the control's transform channels. */
	virtual bool SmartConstraintKey(
		UTickableTransformConstraint* InConstraint, const TOptional<bool>& InOptActive,
		const TRange<FFrameNumber>& InTimeRange, const TSharedPtr<ISequencer>& InSequencer) override;
	
	/** Add keys on the controls's transform channels. */
	virtual void AddHandleTransformKeys(
		const TSharedPtr<ISequencer>& InSequencer,
		const UTransformableHandle* InHandle,
		const TArray<FFrameNumber>& InFrames,
		const TArray<FTransform>& InLocalTransforms,
		const EMovieSceneTransformChannel& InChannels) override;

protected:
		virtual void UnregisterTrack(UMovieSceneTrack* InTrack, UWorld* InWorld) override;
	
private:
	static UMovieSceneControlRigParameterSection* GetControlSection(const UTransformableControlHandle* InHandle, const TSharedPtr<ISequencer>& InSequencer, const bool bIsConstraint);
};

