// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Factories/Factory.h"
#include "ControlRigRuntimeAsset.h"
#include "ControlRig.h"
#include "ControlRigAssetFactory.generated.h"

#define UE_API CONTROLRIGEDITOR_API

UCLASS(MinimalAPI, HideCategories=Object)
class UControlRigAssetFactory : public UFactory
{
	GENERATED_BODY()
	
	UClass* EditorOnlyClass;

public:
	UE_API UControlRigAssetFactory();

	/** The class of the created instances */
	UPROPERTY(EditAnywhere, Category="Control Rig Factory", meta=(AllowAbstract = ""))
	TSubclassOf<UControlRig> ParentClass;

	/** If true, the control rig is created as Control Rig Module */
	UPROPERTY(EditAnywhere, Category = "Control Rig Factory", meta = (AllowAbstract = ""))
	bool bCreateAsControlRigModule = false;

	// UFactory Interface
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;

	/**
	 * Create a new control rig asset within the contents space of the project.
	 * @param InDesiredPackagePath The package path to use for the control rig asset
	 * @param bModularRig If true the rig will be created as a modular rig
	 */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	static UE_API UControlRigRuntimeAsset* CreateNewControlRigAsset(const FString& InDesiredPackagePath, const bool bModularRig = false);

	/**
	 * Create a new control rig asset within the contents space of the project
	 * based on a skeletal mesh or skeleton object.
	 * @param InSelectedObject The SkeletalMesh / Skeleton object to base the control rig asset on
	 * @param bModularRig If true the rig will be created as a modular rig
	 */
	UFUNCTION(BlueprintCallable, Category = "Control Rig")
	static UE_API UControlRigRuntimeAsset* CreateControlRigFromSkeletalMeshOrSkeleton(UObject* InSelectedObject, const bool bModularRig = false);
};

#undef UE_API
