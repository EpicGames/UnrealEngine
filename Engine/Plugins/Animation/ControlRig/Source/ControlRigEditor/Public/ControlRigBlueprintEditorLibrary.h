// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given Control Rig
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ControlRigBlueprintLegacy.h"
#include "RigVMEditorBlueprintLibrary.h"
#include "ControlRigBlueprintEditorLibrary.generated.h"

#define UE_API CONTROLRIGEDITOR_API

UENUM(BlueprintType)
enum class ECastToControlRigBlueprintCases : uint8
{
	CastSucceeded,
	CastFailed
};

UCLASS(MinimalAPI, BlueprintType, meta=(ScriptName="ControlRigBlueprintLibrary"))
class UControlRigBlueprintEditorLibrary : public URigVMEditorBlueprintLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint", Meta = (ExpandEnumAsExecs = "Branches"))
	static UE_API void CastToControlRigBlueprint(
		UObject* Object, 
		ECastToControlRigBlueprintCases& Branches,
		UControlRigBlueprint*& AsControlRigBlueprint);

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	static UE_API void SetPreviewMesh(UControlRigBlueprint* InRigBlueprint, USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true);
	
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API USkeletalMesh* GetPreviewMesh(UControlRigBlueprint* InRigBlueprint);

	UFUNCTION(BlueprintCallable, Category = "Control Rig Blueprint")
	static UE_API void RequestControlRigInit(UControlRigBlueprint* InRigBlueprint);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API TArray<UControlRigBlueprint*> GetCurrentlyOpenRigBlueprints();

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API TArray<UStruct*> GetAvailableRigUnits();

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API URigHierarchy* GetHierarchy(UControlRigBlueprint* InRigBlueprint);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API URigHierarchyController* GetHierarchyController(UControlRigBlueprint* InRigBlueprint);

	UFUNCTION(BlueprintCallable, BlueprintCallable, Category = "Control Rig Blueprint")
	static UE_API void SetupAllEditorMenus();

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API TArray<FRigModuleDescription> GetAvailableRigModules();
	
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Control Rig Blueprint")
	static UE_API UControlRigRuntimeAsset* CreateControlRigAssetFromBlueprint(UControlRigBlueprint* InRigBlueprint);
};

#undef UE_API
