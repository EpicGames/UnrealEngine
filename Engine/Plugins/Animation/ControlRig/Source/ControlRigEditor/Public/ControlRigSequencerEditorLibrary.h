// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * Control Rig Sequencer Exposure
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneBindingProxy.h"
#include "Tools/ControlRigSnapper.h"
#include "TransformNoScale.h"
#include "EulerTransform.h"
#include "MovieSceneToolsUserSettings.h"
#include "Tools/ControlRigSnapSettings.h"
#include "MovieSceneTimeUnit.h"
#include "RigSpacePickerBakeSettings.h"
#include "Filters/CurveEditorSmartReduceFilter.h"


#include "ControlRigSequencerEditorLibrary.generated.h"

#define UE_API CONTROLRIGEDITOR_API

struct FRigControlElement;
class ULevelSequence;
class UTickableConstraint;
class UTickableTransformConstraint;
class UTransformableHandle;
struct FBakingAnimationKeySettings;
struct FMergeAnimLayerSettings;
class UControlRig;
class UAnimLayer;
class UAIESelectionSets;
class UMovieSceneScriptingChannel;

USTRUCT(BlueprintType)
struct FControlRigSequencerBindingProxy
{
	GENERATED_BODY()

	FControlRigSequencerBindingProxy()
		: ControlRig(nullptr)
		, Track(nullptr)
	{}

	FControlRigSequencerBindingProxy(const FMovieSceneBindingProxy& InProxy, UControlRig* InControlRig, UMovieSceneControlRigParameterTrack* InTrack)
		: Proxy(InProxy)
		, ControlRig(InControlRig)
		, Track(InTrack)
	{}

	UPROPERTY(BlueprintReadOnly, Category = ControlRig)
	FMovieSceneBindingProxy Proxy;

	UPROPERTY(BlueprintReadOnly, Category = ControlRig)
	TObjectPtr<UControlRig> ControlRig;

	UPROPERTY(BlueprintReadOnly, Category = ControlRig)
	TObjectPtr<UMovieSceneControlRigParameterTrack> Track;
};

UENUM(BlueprintType)
enum class EAnimToolBlendOperation : uint8
{
	Tween,
	BlendToNeighbor,
	PushPull,
	BlendRelative,
	BlendToEase,
	SmoothRough,
};


USTRUCT(BlueprintType)
struct FArrayOfRigControlValues
{
	GENERATED_BODY()

	//Name of the Control
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ControlRig)
	FName ControlName;

	//Array of control rig values, will correspond usually to a frame value
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ControlRig)
	TArray<FRigControlValue> Values;

	//cached value, may be empty but will be set after UControlRigSequencerEditorLibrary::GetControlValues
	FRigControlElement* CachedElement = nullptr;
};

UENUM(BlueprintType)
enum class EControlRigTransformSpace : uint8
{
	Local = 0,
	Global
};

USTRUCT(BlueprintType)
struct FArrayOfRigControlTransforms
{
	GENERATED_BODY()

	//Name of the Control
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ControlRig)
	FName ControlName;

	//Array of control rig transforms, will correspond usually to a frame value
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = ControlRig)
	TArray<FTransform> Transforms;

	//cached value, may be empty but will be set after UControlRigSequencerEditorLibrary::BatchGetControlTransforms or BatchSetControlTransforms
	FRigControlElement* CachedElement = nullptr;
};

/**
* This is a set of helper functions to access various parts of the Sequencer and Control Rig API via Python and Blueprints.
*/
UCLASS(MinimalAPI, meta=(Transient, ScriptName="ControlRigSequencerLibrary"))
class UControlRigSequencerEditorLibrary : public UBlueprintFunctionLibrary
{

	public:
	GENERATED_BODY()

public:

	/**
	* Get all of the visible control rigs in the level
	* @return returns list of visible Control Rigs
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<UControlRig*> GetVisibleControlRigs();

	/**
	* Get all of the control rigs and their bindings in the level sequence
	* @param LevelSequence The movie scene sequence to look for Control Rigs
	* @return returns list of Control Rigs in the level sequence.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FControlRigSequencerBindingProxy> GetControlRigs(ULevelSequence* LevelSequence);

	/**
	* Find or create a Control Rig track of a specific class based upon the binding
	* @param World The world used to spawn into temporarily if binding is a spawnable
	* @param LevelSequence The LevelSequence to find or create
	* @param ControlRigClass The class of the Control Rig
	* @param InBinding The binding (actor or component binding) to find or create the Control Rig track
	* @return returns Return the found or created track
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API UMovieSceneTrack* FindOrCreateControlRigTrack(UWorld* World, ULevelSequence* LevelSequence, const UClass* ControlRigClass, const FMovieSceneBindingProxy& InBinding, bool bIsLayeredControlRig = false);

	/**
	* Find or create a Control Rig Component
	* @param World The world used to spawn into temporarily if binding is a spawnable
	* @param LevelSequence The LevelSequence to find or create
	* @param InBinding The binding (actor or component binding) to find or create the Control Rig tracks
	* @return returns Find array of component Control Rigs that were found or created
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<UMovieSceneTrack*> FindOrCreateControlRigComponentTrack(UWorld* World, ULevelSequence* LevelSequence, const FMovieSceneBindingProxy& InBinding);
	
	/*
	 * Export Passed in Binding as an Anim Sequence, using open Level Sequence in Sequencer. Use this instead of
	 * USequencerToolsFunctionLibrary::ExportAnimSequence when you may be exporting an anim sequence from a Spawnable that can't be created dynamically
	 * in a closed system.
	 *
	 * @AnimSequence The AnimSequence to save into.
	 * @ExportOption The export options for the sequence.
	 * @InBinding Binding to export that has a skelmesh component on it
	 * @InAnimSequenceFilename File to create
	 * @bCreateLink If true will create a link between the animation sequence and the level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool ExportAnimSequenceFromSequencer(UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOption, const FMovieSceneBindingProxy& Binding, bool bCreateLink);

	/**
	 * Remove the bidirectional link between the AnimSequence owned by the given binding and its LevelSequence.
	 * @param Binding The binding whose linked AnimSequence should be unlinked
	 * @return True if a linked AnimSequence was found and unlinked
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool DeleteLinkedAnimSequence(const FMovieSceneBindingProxy& Binding);

	/**
	 * Remove the linked animation track and its linked animation sequence for the given binding.
	 * @param Binding The binding whose linked track and anim sequence should be removed
	 * @return True if a linked track was found and removed
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool DeleteLinkedAnimTrack(const FMovieSceneBindingProxy& Binding);

	/**
	 * Get the linked animation track for the given binding (read-only lookup).
	 * @param InBinding The binding to look up
	 * @return The linked UMovieSceneTrack, or nullptr if none exists
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API UMovieSceneTrack* GetLinkedAnimTrack(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Find or create the linked animation track for the given binding. Same as Configure AutoBake in the UI.
	 * Creates the necessary track structure, linked anim sequence, and section if none exist.
	 * @param InBinding The binding to find or create the linked track for
	 * @return The linked UMovieSceneTrack, or nullptr on failure
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API UMovieSceneTrack* GetOrCreateLinkedAnimTrack(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Set up auto-bake recorders for the linked anim sequences on the given bindings.
	 * Equivalent to the "Add AutoBake" context-menu entry in the Sequencer UI.
	 * @param InBindings The bindings to configure for auto-baking
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API void AddAutoBake(const TArray<FMovieSceneBindingProxy>& InBindings);

	/**
	 * Isolate or un-isolate the linked animation track for the given binding.
	 * When isolating: enables the linked anim track and disables other anim/control rig tracks.
	 * When un-isolating: restores the previous evaluation state.
	 * @param InBinding The binding whose linked track to isolate
	 * @param bIsolate True to isolate, false to un-isolate
	 * @return True if any track evaluation state was changed
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool IsolateLinkedAnimTrack(const FMovieSceneBindingProxy& InBinding, bool bIsolate);

	/**
	 * Isolate or un-isolate the linked animation tracks for every binding in the focused Sequencer movie scene.
	 * @param bIsolate True to isolate, false to un-isolate
	 * @return True if any track evaluation state was changed
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool IsolateLinkedAnimTracksForAllBindings(bool bIsolate);

	/**
	* Load anim sequence into this control rig section
	* @param MovieSceneSection The MovieSceneSectionto load into
	* @param AnimSequence The Sequence to load
	* @param MovieScene The MovieScene getting loaded into
	* @param SkelMeshComponent The Skeletal Mesh component getting loaded into.
	* @param InStartFrame Frame to insert the animation
	* @param bUseCustomAnimRange If true we will load in a specific animation range from the anim sequence
	* @param AnimStartRange  The start of the animation in the anim sequence frame rate that we load the animation from, used if bUseCustomAnimRange is true
	* @param AnimEndRange The end of the animation in the anim sequence frame rate we load the animation from, used if bUseCustomAnimRange is true
	* @param TimeUnit Unit for all frame and time values, either in display rate or tick resolution
	* @param bKeyReduce If true do key reduction based upon Tolerance, if false don't
	* @param Tolerance If reducing keys, tolerance about which keys will be removed, smaller tolerance, more keys usually.
	* @param Interpolation The key interpolation type to set the keys, defaults to EMovieSceneKeyInterpolation::SmartAuto
	* @param bResetControls If true will reset all controls to initial value on every frame
	* @param bOntoSelectedControls if true will only load onto selected controls on the Control Rig, otherwise will load onto all controls that aren't masked out
	* @return returns True if successful, False otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool LoadAnimSequenceIntoControlRigSectionWithRange(UMovieSceneSection* MovieSceneSection, UAnimSequence* AnimSequence, USkeletalMeshComponent* SkelMeshComp,
		FFrameNumber InStartFrame, bool bUseCustomAnimRange, FFrameNumber AnimStartRange, FFrameNumber AnimEndRange,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bKeyReduce = false, float Tolerance = 0.001f,
		EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::SmartAuto, bool bResetControls = true,
		bool bOntoSelectedControls = false);

	UE_DEPRECATED(5.6, "Use LoadAnimSequenceIntoControlRigSectionWithRange that takes UseCustomTimeRange")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig", meta = (DeprecatedFunction, DeprecationMessage = "Use LoadAnimSequenceIntoControlRigSectionWithRange that takes UseCustomTimeRange"))
	static UE_API bool LoadAnimSequenceIntoControlRigSection(UMovieSceneSection* MovieSceneSection, UAnimSequence* AnimSequence, USkeletalMeshComponent* SkelMeshComp,
		FFrameNumber InStartFrame, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bKeyReduce = false, float Tolerance = 0.001f,
		EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::SmartAuto, bool bResetControls = true);

	/** Copy the internal parameter curves from vector control rig curves to transform curves, this is needed if a vector type is changed to transform type
	* You would run this script first, resave, and then on next reload the curves should now show up on correct curves in the transform control
	* @param InTrack Control Rig track to convert
	* @param InControlName The name of the existing Vector type control
	* @param InType The type of the control to convert, ERigControlType::Position,Rotator, or Scale
	* @param returns true if at least one section was converted, false if nothing was converted
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool ControlRigCopyVectorParameterCurvesToTransform(UMovieSceneControlRigParameterTrack* InTrack,
		const FName& InControlName, ERigControlType InType);

	/**
	* Bake the current animation in the binding to a Control Rig track
	* @param World The active world
	* @param LevelSequence The LevelSequence we are baking
	* @param ControlRigClass The class of the Control Rig
	* @param ExportOptions Export options for creating an animation sequence
	* @param bKeyReduce If true do key reduction based upon Tolerance, if false don't
	* @param Tolerance If reducing keys, tolerance about which keys will be removed, smaller tolerance, more keys usually.
	* @param Binding The binding upon which to bake
	* @param bResetControls If true will reset all controls to initial value on every frame
	* @return returns True if successful, False otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BakeToControlRig(UWorld* World, ULevelSequence* LevelSequence, UClass* ControlRigClass, UAnimSeqExportOption* ExportOptions, bool bReduceKeys, float Tolerance,
			const FMovieSceneBindingProxy& Binding, bool bResetControls = true);

	/**
	* Peform new Smart Reduce filter over the specified control rig section in the current open level sequence. Note existing
	* functions like LoadAnimSequenceIntoControlRigSection and BakeToControlRig, will still use the old key reduction algorithm,
	* so if you want to bake and then key reduce with the new function, set the bKeyReduce param as false with those functions,
	* but then call this function after.
	* @param ReduceParams Key reduction parameters
	* @param MovieSceneSection The Control rig section we want to reduce
	* @return returns True if successful, False otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SmartReduce(FSmartReduceParams& ReduceParams, UMovieSceneSection* MovieSceneSection);


	/**
	* Bake the constraint to keys based on the passed in frames. This will use the open sequencer to bake. See ConstraintsScriptingLibrary to get the list of available constraints
	* @param World The active world
	* @param Constraint The Constraint to bake. After baking it will be keyed to be inactive of the range of frames that are baked
	* @param Frames The frames to bake, if the array is empty it will use the active time ranges of the constraint to determine where it should bake
	* @param TimeUnit Unit for all frame and time values, either in display rate or tick resolution
	* @return Returns True if successful, False otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BakeConstraint(UWorld* World, UTickableConstraint* Constraint, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Bake the constraint to keys based on the passed in settings. This will use the open sequencer to bake. See ConstraintsScriptingLibrary to get the list of available constraints
	* @param World The active world
	* @param InConstraints The Constraints tobake.  After baking they will be keyed to be inactive of the range of frames that are baked
	* @param InSettings Settings to use for baking
	* @return Returns True if successful, False otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BakeConstraints(UWorld* World, TArray<UTickableConstraint*>& InConstraints,const FBakingAnimationKeySettings& InSettings);
	/**
	* Add a constraint possibly adding to sequencer also if one is open.
	* @param World The active world
	* @param InType Type of constraint to create
	* @param InChild The handle to the transormable to be constrainted
	* @param InParent The handle to the parent of the constraint
	* @param bMaintainOffset Whether to maintain offset between child and parent when setting the constraint
	* @return Returns the constraint if created all nullptr if not
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API UTickableConstraint* AddConstraint(UWorld* World, ETransformConstraintType InType, UTransformableHandle* InChild, UTransformableHandle* InParent, const bool bMaintainOffset);
	
	/**
	* Get the constraint keys for the specified constraint
	* @param InConstraint The constraint to get
	* @param ConstraintSection Section containing Cosntraint Key
	* @param OutBools Array of whether or not it's active at the specified times
	* @param OutFrames The Times for the keys
	* @param TimeUnit Unit for the time params, either in display rate or tick resolution
	* @return Returns true if we got the keys from this constraint
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool GetConstraintKeys(UTickableConstraint* InConstraint, UMovieSceneSection* ConstraintSection,TArray<bool>& OutBools, TArray<FFrameNumber>& OutFrames, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);
	
	/**
	* Set the constraint active key in the current open Sequencer
	* @param InConstraint The constraint to set the key
	* @param bActive Whether or not it's active
	* @param FrameTime Time to set the value
	* @param TimeUnit Unit for the time params, either in display rate or tick resolution
	* @return Returns true if we set the constraint to be the passed in value, false if not. We may not do so if the value is the same.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SetConstraintActiveKey(UTickableConstraint* InConstraint, bool bActive, FFrameNumber InFrame, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get all constraints for this object, which is described by a transformable handle
	* @param InChild The handle to look for constraints controlling it
	* @return Returns array of Constraints this handle is constrained to.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray <UTickableConstraint*> GetConstraintsForHandle(UWorld* InWorld, const UTransformableHandle* InChild);

	/** Move the constraint active key in the current open Sequencer
	* @param InConstraint The constraint whose key to move
	* @param ConstraintSection Section containing Cosntraint Key
	* @param InTime Original time of the constraint key
	* @param InNewTime New time for the constraint key
	* @param TimeUnit Unit for the time params, either in display rate or tick resolution
	* @return Will return false if function fails, for example if there is no key at this time it will fail, or if the new time is invalid it could fail also
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool MoveConstraintKey(UTickableConstraint* Constraint, UMovieSceneSection* ConstraintSection, FFrameNumber InTime, FFrameNumber InNewTime, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Delete the Key for the Constraint at the specified time. 
	* @param InConstraint The constraint whose key to move
	* @param ConstraintSection Section containing Cosntraint Key
	* @param InTime Time to delete the constraint.
	* @param TimeUnit Unit for the InTime, either in display rate or tick resolution
	* @return Will return false if function fails,  for example if there is no key at this time it will fail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool DeleteConstraintKey(UTickableConstraint* Constraint, UMovieSceneSection* ConstraintSection, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);
	/**
	* Compensate constraint at the specfied time 
	* @param InConstraint The constraint to compensate
	* @param InTime Time to compensate
	* @param TimeUnit Unit for the InTime, either in display rate or tick resolution
	* @return Returns true if it can compensate
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool Compensate(UTickableConstraint* InConstraint,  FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Compensate constraint at all keys
	* @param InConstraint The constraint to compensate
	* @return Returns true if it can compensate
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool CompensateAll(UTickableConstraint* InConstraint);

	/**
	* Peform a Tween operation on the current active sequencer time(must be visible).
	* @param LevelSequence The LevelSequence that's loaded in the editor
	* @param ControlRig The Control Rig to tween.
	* @param TweenValue The tween value to use, range from -1(blend to previous) to 1(blend to next)
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool TweenControlRig(ULevelSequence* LevelSequence, UControlRig* ControlRig, float TweenValue);
	
	/**
	* Peform specified blend operation based upon selected keys in the curve editor or selected control rig controls
	* @param LevelSequence The LevelSequence that's loaded in the editor
	* @param EAnimToolBlendOperation The operation to perform
	* @param BlendValue The blend value to use, range from -1(blend to previous) to 1(blend to next)
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BlendValuesOnSelected(ULevelSequence* LevelSequence, EAnimToolBlendOperation BlendOperation, 
		float BlendValue);

	/**
	* Peform a Snap operation to snap the children to the parent. 
	* @param LevelSequence Active Sequence to snap
	* @param StartFrame Beginning of the snap
	* @param EndFrame End of the snap
	* @param ChildrenToSnap The children objects that snap and get keys set onto. They need to live in an active Sequencer in the level editor
	* @param ParentToSnap The parent object to snap relative to. If animated, it needs to live in an active Sequencer in the level editor
	* @param SnapSettings Settings to use
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param Returns True if successful
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SnapControlRig(ULevelSequence* LevelSequence, FFrameNumber StartFrame, FFrameNumber EndFrame, const FControlRigSnapperSelection& ChildrenToSnap,
		const FControlRigSnapperSelection& ParentToSnap, const UControlRigSnapSettings* SnapSettings, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);
	
	/**
	* Get Actors World Transform at a specific time
	* @param LevelSequence Active Sequence to get transform for
	* @param Actor The actor
	* @param Frame Time to get the transform
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns World Transform
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FTransform GetActorWorldTransform(ULevelSequence* LevelSequence,AActor* Actor, FFrameNumber Frame, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get Actors World Transforms at specific times
	* @param LevelSequence Active Sequence to get transform for
	* @param Actor The actor
	* @param Frames Times to get the transform
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns World Transforms
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FTransform> GetActorWorldTransforms(ULevelSequence* LevelSequence, AActor* Actor, const TArray<FFrameNumber>& Frames, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get SkeletalMeshComponent World Transform at a specific time
	* @param LevelSequence Active Sequence to get transform for
	* @param SkeletalMeshComponent The SkeletalMeshComponent
	* @param Frame Time to get the transform
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param ReferenceName Optional name of the referencer
	* @return Returns World Transform
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FTransform GetSkeletalMeshComponentWorldTransform(ULevelSequence* LevelSequence, USkeletalMeshComponent* SkeletalMeshComponent, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate,FName ReferenceName = NAME_None);

	/**
	* Get SkeletalMeshComponents World Transforms at specific times
	* @param LevelSequence Active Sequence to get transform for
	* @param SkeletalMeshComponent The SkeletalMeshComponent
	* @param Frames Times to get the transform
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param ReferenceName Optional name of the referencer
	* @return Returns World Transforms
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FTransform> GetSkeletalMeshComponentWorldTransforms(ULevelSequence* LevelSequence, USkeletalMeshComponent* SkeletalMeshComponent, const TArray<FFrameNumber>& Frames, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate,FName ReferenceName = NAME_None);

	/**
	* Sets the interaction state of the following SetLocalControlRig* functions (default is False).
	* @param bIsInteracting If True the SetLocalControlRig* functions will be treated as if the ControlRig was interacting to react to interaction events.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static void SetInteraction(bool bIsInteracting);

	/**
	* Get an array per frame of specified control rig values for the specified control names. Use URigHierarchy::Get*FromControlValue functions to convert these
	* FRigControlValue to Bool,Rotator,EulerTransform, etc..
	* @param LevelSequence Active Sequence to get values for, if not set will use the level sequence open in Sequencer
	* @param ControlRig The ControlRig
	* @param ControlNames Names of the Controls
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns an array per control containing control values for each frame. Note that that lhe length of this array may not be equal to the length of the Control Names array
	* if there are invalid ControlNames
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FArrayOfRigControlValues> GetControlValues(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FName>& ControlNames, const TArray<FFrameNumber>& Frames, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);
	

	/**
	* Get an array per frame of specified control rig transforms for the specified control names. 
	* @param LevelSequence Active Sequence to get values for, if not set will use the level sequence open in Sequencer
	* @param ControlRig The ControlRig
	* @param ControlNames Names of the Controls
	* @param Frames Times to get the values
	* @param Space The space (Local or Global) the transforms will be in
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns an array per control containing transforms for each frame. Note that that lhe length of this array may not be equal to the length of the ControlNames array
	* if there are invalid Control names
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FArrayOfRigControlTransforms> BatchGetControlTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FName>& ControlNames, const TArray<FFrameNumber>& Frames,
		EControlRigTransformSpace Space, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set from Control transforms for different controls in the specified space at different frames
	* @param LevelSequence Active Sequence to get values for, if not set will use the level sequence open in Sequencer
	* @param ControlRig The ControlRig
	* @param Controls The set of Controls with their transforms, the CachedElements may get set on this if not set
	* @param Frames Times to get the values
	* @param Space The space (Local or Global) the transforms will be in
	* @param OptionalSectionToSet The section that should get the keys, if not set, it will use the current section to key setup on the track.
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns true if successful, false otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BatchSetControlTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, const TArray<FArrayOfRigControlTransforms>& Controls, const TArray<FFrameNumber>& Frames,
		EControlRigTransformSpace Space, UMovieSceneControlRigParameterSection* OptionalSectionToSet = nullptr, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's World Transform at a specific time
	* @param LevelSequence Active Sequence to get transform for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control
	* @param Frame Time to get the transform
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns World Transform
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FTransform GetControlRigWorldTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's World Transforms at specific times
	* @param LevelSequence Active Sequence to get transform for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control 
	* @param Frames Times to get the transform
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns World Transforms
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FTransform> GetControlRigWorldTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's World Transform at a specific time
	* @param LevelSequence Active Sequence to set transforms for. Must be loaded in Level Editor.
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control
	* @param Frame Time to set the transform
	* @param WorldTransform World Transform to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey Whether or not to set a key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetControlRigWorldTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, const FTransform& WorldTransform,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's World Transforms at a specific times.
	* @param LevelSequence Active Sequence to set transforms for. Must be loaded in Level Editor.
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control
	* @param Frames Times to set the transform
	* @param WorldTransform World Transform to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetControlRigWorldTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, 
		 const TArray<FTransform>& WorldTransforms, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);
	
	/**
	* Get ControlRig Control's float value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a float control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API float GetLocalControlRigFloat(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's float values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a float control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<float> GetLocalControlRigFloats(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's float value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a float control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigFloat(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, float Value,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate,bool bSetKey = true);

	/**
	* Set ControlRig Control's float values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a float control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigFloats(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<float> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's bool value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a bool control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool GetLocalControlRigBool(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's bool values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a bool control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<bool> GetLocalControlRigBools(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's bool value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a bool control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigBool(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, bool Value,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's bool values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a bool control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigBools(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		const TArray<bool> Values, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's integer value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a integer control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API int32 GetLocalControlRigInt(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's integer values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a intteger control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<int32> GetLocalControlRigInts(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's int value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a int control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigInt(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, int32 Value,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate,bool bSetKey = true);


	/**
	* Set ControlRig Control's int values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a int control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigInts(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<int32> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Vector2D value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Vector2D control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FVector2D GetLocalControlRigVector2D(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Vector2D values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Vector2D control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FVector2D> GetLocalControlRigVector2Ds(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's Vector2D value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Vector2D control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigVector2D(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FVector2D Value,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate,bool bSetKey = true);


	/**
	* Set ControlRig Control's Vector2D values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Vector2D control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigVector2Ds(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FVector2D> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Position value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Position control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FVector GetLocalControlRigPosition(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Position values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Position control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FVector> GetLocalControlRigPositions(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's Position value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Position control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigPosition(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FVector Value,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's Position values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Position control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigPositions(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FVector> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Rotator value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Rotator control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FRotator GetLocalControlRigRotator(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Rotator values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Rotator control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FRotator> GetLocalControlRigRotators(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's Rotator value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Rotator control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigRotator(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FRotator Value, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's Rotator values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Rotator control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigRotators(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FRotator> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Scale value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Scale control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FVector GetLocalControlRigScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Scale values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Scale control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FVector> GetLocalControlRigScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's Scale value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Scale control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FVector Value, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's Scale values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Scale control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FVector> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);


	/**
	* Get ControlRig Control's EulerTransform value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a EulerTransfom control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FEulerTransform GetLocalControlRigEulerTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's EulerTransform values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a EulerTransform control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FEulerTransform> GetLocalControlRigEulerTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's EulerTransform value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a EulerTransform control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigEulerTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FEulerTransform Value,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's EulerTransform values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a EulerTransform control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigEulerTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FEulerTransform> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's TransformNoScale value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a TransformNoScale control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FTransformNoScale GetLocalControlRigTransformNoScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's TransformNoScale values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a TransformNoScale control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FTransformNoScale> GetLocalControlRigTransformNoScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's TransformNoScale value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a TransformNoScale control
	* @param Frame Time to set the value
	* @param Value The value to set
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigTransformNoScale(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FTransformNoScale Value, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's TransformNoScale values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a TransformNoScale control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigTransformNoScales(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FTransformNoScale> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Transform value at a specific time
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Transform control
	* @param Frame Time to get the value
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Value at that time
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FTransform GetLocalControlRigTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Get ControlRig Control's Transform values at specific times
	* @param LevelSequence Active Sequence to get value for
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Transform control
	* @param Frames Times to get the values
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @return Returns Values at those times
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API TArray<FTransform> GetLocalControlRigTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	* Set ControlRig Control's Transform value at specific time
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Transform control
	* @param Frame Time to set the value
	* @param Value The value to set 
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	* @param bSetKey If True set a key, if not just set the value
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigTransform(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, FFrameNumber Frame, FTransform Value, 
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, bool bSetKey = true);

	/**
	* Set ControlRig Control's Transform values at specific times
	* @param LevelSequence Active Sequence to set value on
	* @param ControlRig The ControlRig
	* @param ControlName Name of the Control, should be a Transform control
	* @param Frames Times to set the values
	* @param Values The values to set at those times
	* @param TimeUnit Unit for frame values, either in display rate or tick resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetLocalControlRigTransforms(ULevelSequence* LevelSequence, UControlRig* ControlRig, FName ControlName, const TArray<FFrameNumber>& Frames, const TArray<FTransform> Values,
		EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/*
	 * Import FBX onto a control rig with the specified track and section
	 *
	 * @param InWorld World to import to
	 * @param InSequence Sequence to import
	 * @param InTrack Track to import onto
	 * @param InSection Section to import onto, may be null in which case we use the track's section to key
	 * @param SelectedControlRigNames  List of selected control rig names. Will use them if  ImportFBXControlRigSettings->bImportOntoSelectedControls is true
	 * @param ImportFBXControlRigSettings Settings to control import.
	 * @param InImportFileName Path to fbx file to create
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | FBX")
	static UE_API bool ImportFBXToControlRigTrack(UWorld* World, ULevelSequence* InSequence, UMovieSceneControlRigParameterTrack* InTrack, UMovieSceneControlRigParameterSection* InSection,
			const TArray<FString>& SelectedControlRigNames,
			UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
			const FString& ImportFilename);

	/** Exports an FBX from the given control rig section. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | FBX")
	static UE_API bool ExportFBXFromControlRigSection(ULevelSequence* Sequence, const UMovieSceneControlRigParameterSection* Section,
	                                    const UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings);

	/*
	 * Collapse and bake all sections and layers on a control rig track to just one section.
	 *
	 * @param InSequence Sequence that has track to collapse
	 * @param InTrack Track for layers to collapse
	 * @param bKeyReduce If true do key reduction based upon Tolerance, if false don't
	 * @param Tolerance If reducing keys, tolerance about which keys will be removed, smaller tolerance, more keys usually.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool CollapseControlRigAnimLayers(ULevelSequence* InSequence,UMovieSceneControlRigParameterTrack* InTrack, bool bKeyReduce = false, float Tolerance = 0.001f);

	/*
	 * Collapse and bake all sections and layers on a control rig track to just one section using passed in settings.
	 *
	 * @param InSequence Sequence that has track to collapse
	 * @param InTrack Track for layers to collapse
	 * @param InSettings Settings that determine how to collapse
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool CollapseControlRigAnimLayersWithSettings(ULevelSequence* InSequence, UMovieSceneControlRigParameterTrack* InTrack, const FBakingAnimationKeySettings& InSettings);

	/*
	 * Remove any timewarp from section
	 *
	 * @param InSection Section to remove time warp from
	 * @return true if successful false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool DeleteTimeWarp(UMovieSceneControlRigParameterSection* InSection);

	/*
	 * Bake any timewarp from section
	 *
	 * @param InSection Section to bake time warp from
	 * @return true if successful false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BakeTimeWarp(UMovieSceneControlRigParameterSection* InSection);

	/*
	 * Get the space channel for the specified control rig and control name. Will create it if needed. Note spaces will always live on the first section.
	 *
	 * @param InSequence Sequence that to get space channel
	 * @param InControlRig ControlRig with the Control
	 * @param InControlName Name of control to have space channel
	 * @param bCreateIfNeeded Create the channel if it doesn't exist
	 * @return return the space channel if one exists or was created based upon bCreateIfNeeded
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UMovieSceneScriptingChannel* GetSpaceChannel(UMovieSceneSequence* InSequence, UControlRig* InControlRig, FName InControlName, bool bCreateIfNeeded = true);

	/*
	 * Get the default parent key, can be used a parent space.
	 *
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FRigElementKey GetDefaultParentKey();

	/*
	 * Get the default world space key, can be used a world space.
	 *
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API FRigElementKey GetWorldSpaceReferenceKey();

	/*
	 * Set the a key for the Control Rig Space for the Control at the specified time. If space is the same as the current no key will be set.
	 *
	 * @param InSequence Sequence to set the space
	 * @param InControlRig ControlRig with the Control
	 * @param InControlName The name of the Control
	 * @param InSpaceKey  The new space for the Control
	 * @param InTime Time to change the space.
	 * @param TimeUnit Unit for the InTime, either in display rate or tick resolution
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SetControlRigSpace(ULevelSequence* InSequence, UControlRig* InControlRig, FName InControlName, const FRigElementKey& InSpaceKey, FFrameNumber InTime,  EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Bake specified Control Rig Controls to a specified Space based upon the current settings
	* @param InSequence Sequence to bake
	* @param InControlRig ControlRig to bake
	* @param InControlNames The name of the Controls to bake
	* @param InSettings  The settings for the bake, e.g, how long to bake, to key reduce etc.
	* @param TimeUnit Unit for the start and end times in the InSettings parameter.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool BakeControlRigSpace(ULevelSequence* InSequence, UControlRig* InControlRig, const TArray<FName>& InControlNames, FRigSpacePickerBakeSettings InSettings, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);
	
	/** Perform compensation for any spaces at the specified time for the specified control rig
	* @param InControlRig Control Rig to compensate
	* @param InTime  The time to look for a space key to compensate
	* @param TimeUnit Unit for the InTime
	* @return Will return false if function fails
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SpaceCompensate(UControlRig* InControlRig, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Perform compensation for all spaces for the specified control rig
	* @param InControlRig Control Rig to compensate
	* @return Will return false if function fails
	*/
	static UE_API bool SpaceCompensateAll(UControlRig* InControlRig);

	/** Delete the Control Rig Space Key for the Control at the specified time. This will delete any attached Control Rig keys at this time and will perform any needed compensation to the new space.
	*
	* @param InSequence Sequence to set the space
	* @param InControlRig ControlRig with the Control
	* @param InControlName The name of the Control
	* @param InTime Time to delete the space.
	* @param TimeUnit Unit for the InTime, either in display rate or tick resolution
	* @return Will return false if function fails,  for example if there is no key at this time it will fail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool DeleteControlRigSpace(ULevelSequence* InSequence, UControlRig* InControlRig, FName InControlName, FFrameNumber InTime, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Move the Control Rig Space Key for the Control at the specified time to the new time. This will also move any Control Rig keys at this space switch boundary.
	*
	* @param InSequence Sequence to set the space
	* @param InControlRig ControlRig with the Control
	* @param InControlName The name of the Control
	* @param InTime Original time of the space key
	* @param InNewTime New time for the space key
	* @param TimeUnit Unit for the time params, either in display rate or tick resolution
	* @return Will return false if function fails, for example if there is no key at this time it will fail, or if the new time is invalid it could fail also
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool MoveControlRigSpace(ULevelSequence* InSequence, UControlRig* InControlRig, FName InControlName, FFrameNumber InTime, FFrameNumber InNewTime, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Replace the Control Rig on the track with a new Control Rig. Note it will now be owned by the track
	* @param InControlRigTrack ControlRigTrack to replace
	* @param NewControlRig ControlRig to put on track
	* @return Return true if the function succeeds, false if it doesn't 
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool ReplaceControlRig(UMovieSceneTrack* InControlRigTrack, UControlRig* NewControlRig);

	/** Rename the Control Rig Channels in Sequencer to the specified new control names, which should be present on the Control Rig
	* @param InSequence Sequence to rename controls
	* @param InControlRig ControlRig to rename controls
	* @param InOldControlNames The name of the old Control Rig Control Channels to change. Will be replaced by the corresponding name in the InNewControlNames array
	* @param InNewControlNames  The name of the new Control Rig Channels
	* @return Return true if the function succeeds, false if it doesn't which can happen if the name arrays don't match in size or any of the new Control Names aren't valid
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool RenameControlRigControlChannels(ULevelSequence* InSequence, UControlRig* InControlRig, const TArray<FName>& InOldControlNames, const TArray<FName>& InNewControlNames);

	/** Get the controls mask for the given ControlName */
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool GetControlsMask(UMovieSceneSection* InSection, FName ControlName);

	/** Set the controls mask for the given ControlNames */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetControlsMask(UMovieSceneSection* InSection, const TArray<FName>& ControlNames, bool bVisible);

	/** Clear Selection*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void ClearSelection();

	/** Frame to current Control Selection*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void FrameSelection();

	/** Select Mirrored Controls on Current Selection*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SelectMirroredControls();

	/** Select Mirrored Controls on Current Selection, keeping current selection*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void AddMirroredControlsToSelection();

	/** Put Selected Controls To Mirrored Pose*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void MirrorSelectedControls();

	/** Put Unselected Controls To Mirrored Pose*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void MirrorUnselectedControls();

	/** Reset Transforms to default values, usually zero values but which may not be zero if initial values are present
	@param bSelectionOnly Only onto rigs with selected controls
	@param bIncludeChannels Include channels in addition to parent controls
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void ZeroTransforms(bool bSelectionOnly, bool bIncludeChannels);

	/** Invert Input Pose, this will set layered controls rigs to default pose, for non-layered same as ZeroTransforms
	@param bSelectionOnly Only onto rigs with selected controls
	@param bIncludeChannels Include channels in addition to parent controls
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void InvertInputPose(bool bSelectionOnly, bool bIncludeChannels);

	/** Opens the Control Rig Pose Library panel. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void OpenPoseLibraryTab();

	/** Closes the Control Rig Pose Library panel. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void ClosePoseLibraryTab();

	/**
	* Key Controls On Section at the current frame
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void KeyControls(UMovieSceneSection* InSection, const TArray<FName>& ControlNames);

	/**
	* Key Controls On Section at the specified frame
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void KeyControlsAtFrames(UMovieSceneSection* InSection, const TArray<FName>& ControlNames , const TArray<FFrameNumber>& AtTimes, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Shows all of the controls for the given section */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void ShowAllControls(UMovieSceneSection* InSection);

	/** Hides all of the controls for the given section */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void HideAllControls(UMovieSceneSection* InSection);

	/** Set Control Rig priority order */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API void SetControlRigPriorityOrder(UMovieSceneTrack* InTrack,int32 PriorityOrder);

	/** Get Control Rig prirority order */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API int32 GetControlRigPriorityOrder(UMovieSceneTrack* InTrack);

	/**	Whether or not the control rig is an FK Control Rig.
	@param InControlRig Rig to test to see if FK Control Rig
	**/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool IsFKControlRig(UControlRig* InControlRig);

	/**	Whether or not the control rig is an Layered Control Rig.
	@param InControlRig Rig to test to see if Layered Control Rig
	**/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool IsLayeredControlRig(UControlRig* InControlRig);

	/**
	 * Mark the layered mode of a control rig on the color and display name of a track
	 * @param InTrack The track to modify the color and display name if necessary
	 */
	static UE_API bool MarkLayeredModeOnTrackDisplay(UMovieSceneControlRigParameterTrack* InTrack);

	/*
	 * Convert the control rig track into absolute or layered rig
	 *
	 * @param InTrack Control rig track to convert 
	 * @param bSetIsLayered Convert to layered rig if true, or absolute if false
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SetControlRigLayeredMode(UMovieSceneControlRigParameterTrack* InTrack, bool bSetIsLayered);

	/**	Get FKControlRig Apply Mode.
	@param InControlRig Rig to test
	@return The EControlRigFKRigExecuteMode mode it is in, either Replace,Additive or Direct
	**/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API EControlRigFKRigExecuteMode  GetFKControlRigApplyMode(UControlRig* InControlRig);

	/**	Set the FK Control Rig to apply mode
	@param InControlRig Rig to set 
	@param InApplyMode Set the EControlRigFKRigExecuteMode mode (Replace,Addtiive or Direct)
	@return returns True if the mode was set, may not be set if the Control Rig doesn't support these modes currently only FKControlRig's do.
	**/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool SetControlRigApplyMode(UControlRig* InControlRig, EControlRigFKRigExecuteMode InApplyMode);

	/*
	 * Recreate a control rig track with new settings
	 *
	 * @param InTrack Track to recreate Control Rig
	 * @param InSettingsToOverride Map of public variable name and value strings that should be applied to the rig.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig")
	static UE_API bool RecreateControlRigWithNewSettings(UMovieSceneControlRigParameterTrack* InTrack, const TMap<FName, FString>& InSettingsToOverride);
	
	/**
	* Delete anim layer at specified index
	* @param Index The index where the anim layer exists
	* @return Returns true if successful, false otherwise
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API bool DeleteAnimLayer(int32 Index);

	/**
	* Duplicate anim layer at specified index
	* @param Index The index where the anim layer exists
	* @return Returns index of new layer, -1 if none created
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API int32 DuplicateAnimLayer(int32 Index);

	/**
	* Add anim layer from objects selected in Sequencer
	* @return Returns Index of created anim layer
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API int32 AddAnimLayerFromSelection();

	/**
	* Move Anim Layer to new index, this will also change evaluation order
	* @param OldIndex The old index where the anim layer exists, can not be zero, can not move the base layer
	* @param NewIndex The new where the anim layer exists, can not be zero, can not move into the base layer.
	* @return Returns true if successful, false if not, for example if the indices are invalid, or if you are trying to move the base layer
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API bool ReorderAnimLayers(int32 OldIndex, int32 NewIndex);


	UE_DEPRECATED(5.6, "Use MergeAnimLayersWithSettings that takes FMergeAnimLayerSettings")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | AnimationLayers", meta = (DeprecatedFunction, DeprecationMessage = "Use MergeAnimLayersWithSettings that takes FMergeAnimLayerSettings"))
	static UE_API bool MergeAnimLayers(const TArray<int32>& Indices);

	/**
	* Merge specified anim layers into one layer. Will merge onto the anim layer with the lowest index
	* @param Indices The indices to merge
	* @param MergeAnimLayerSettings Settings with per frame or key options and key reduction
	* @return Returns true if successful, false otherwise
	*/

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API bool MergeAnimLayersWithSettings(const TArray<int32>& Indices, const FMergeAnimLayerSettings& MergeAnimLayerSettings);

	/**
	* Get the animation layer objects, will create it bAddIfDoesNotExist is set to true
	* @param bAddIfDoesNotExist If it doesn't exist create it
	* @return Returns array of anim layer objects if they exist on active Sequencer
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API TArray<UAnimLayer*> GetAnimLayers(bool bAddIfDoesNotExist = true);
	
	/**
	* Helper function to get the index in the anim layer array from the anim layer
	* @param Anim Layer to get the index for
	* @return Returns index for the anim layer or INDEX_NONE(-1) if it doesn't exist
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Animation Layers")
	static UE_API int32 GetAnimLayerIndex(UAnimLayer* AnimLayer);

	/**
	* Get the selection set objects, will create it bAddIfDoesNotExist is set to true
	* @param bAddIfDoesNotExist If it doesn't exist create it
	* @return Returns main selection set object that contains operations you can perform to create and use selection sets
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Control Rig | Selection Sets")
	static UE_API UAIESelectionSets* GetSelectionSets(bool bAddIfDoesNotExist = true);

};

#undef UE_API
