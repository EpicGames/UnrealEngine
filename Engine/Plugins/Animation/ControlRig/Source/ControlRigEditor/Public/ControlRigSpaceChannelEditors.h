// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"

#include "MovieSceneKeyStruct.h"
#include "SequencerChannelTraits.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "RigSpacePickerBakeSettings.h"

#include "Sequencer/MovieSceneControlRigSpaceChannel.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Containers/SortedMap.h"
#include "KeyBarCurveModel.h"

struct FKeyHandle;
struct FKeyDrawParams;
class UControlRig;
class ISequencer;
class UMovieSceneSection;
class URigHierarchy;
struct FRigElementKey;
class SWidget;
class ISequencer;
class FMenuBuilder;


struct FSpaceChannelAndSection
{
	FSpaceChannelAndSection() : SectionToKey(nullptr), SpaceChannel(nullptr) {};
	UMovieSceneSection* SectionToKey;
	FMovieSceneControlRigSpaceChannel* SpaceChannel;
};


/*
* Class that contains helper functions for various space switching activities
*/
struct FControlRigSpaceChannelHelpers
{
	static FKeyHandle SequencerKeyControlRigSpaceChannel(UControlRig* ControlRig, ISequencer* Sequencer, FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber Time, URigHierarchy* RigHierarchy, const FRigElementKey& ControlKey, const FRigElementKey& SpaceKey);
	static void SequencerSpaceChannelKeyDeleted(UControlRig* ControlRig, ISequencer* Sequencer, FName ControlName, FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneControlRigParameterSection* SectionToKey, FFrameNumber TimeOfDeletion);
	static UMovieSceneControlRigParameterSection* GetControlRigSection(ISequencer* Sequencer, const UControlRig* ControlRig);
	static void CompensateIfNeeded(UControlRig* ControlRig, ISequencer* Sequencer, UMovieSceneControlRigParameterSection* Section, const TOptional<FFrameNumber>& Time, bool bCompPreviousTick);
	static FSpaceChannelAndSection FindSpaceChannelAndSectionForControl(UControlRig* ControlRig, FName ControlName, ISequencer* Sequencer, bool bCreateIfNeeded);
	static void SequencerBakeControlInSpace(UControlRig* ControlRig, ISequencer* Sequencer, FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey,
		URigHierarchy* RigHierarchy, const FRigElementKey& ControlKey, FRigSpacePickerBakeSettings InSettings);
	static void GetFramesInThisSpaceAfterThisTime(UControlRig* ControlRig, FName ControlName, FMovieSceneControlRigSpaceBaseKey CurrentValue,
		FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey,
		FFrameNumber Time, TSortedMap<FFrameNumber,FFrameNumber>& OutMoreFrames);
	static void HandleSpaceKeyTimeChanged(UControlRig* ControlRig, FName ControlName,FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey,
		FFrameNumber CurrentFrame, FFrameNumber NextFrame);
	static void DeleteTransformKeysAtThisTime(UControlRig* ControlRig, UMovieSceneControlRigParameterSection* Section, FName ControlName, FFrameNumber Time);
	static FLinearColor GetColor(const FMovieSceneControlRigSpaceBaseKey& Key);
	static FReply OpenBakeDialog(ISequencer* Sequencer, FMovieSceneControlRigSpaceChannel* Channel, int32 KeyIndex, UMovieSceneSection* SectionToKey);
	static TArray<FKeyBarCurveModel::FBarRange> FindRanges(FMovieSceneControlRigSpaceChannel* Channel, const UMovieSceneSection* Section);
	// retrieve the control and the channel infos for that ControlRig/Section.
	static TPair<FRigControlElement*, FChannelMapInfo*> GetControlAndChannelInfo(UControlRig* ControlRig, UMovieSceneControlRigParameterSection* Section, FName ControlName);
	// retrieve the number of float channels based on the control type.
	static int32 GetNumFloatChannels(const ERigControlType InControlType);
};

//template specialization
FKeyHandle AddOrUpdateKey(FMovieSceneControlRigSpaceChannel* Channel, UMovieSceneSection* SectionToKey, FFrameNumber Time, ISequencer& Sequencer, const FGuid& ObjectBindingID, FTrackInstancePropertyBindings* PropertyBindings);

/** Key editor overrides */
bool CanCreateKeyEditor(const FMovieSceneControlRigSpaceChannel* Channel);
TSharedRef<SWidget> CreateKeyEditor(const TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>& Channel, const UE::Sequencer::FCreateKeyEditorParams& Params);

/** Key drawing overrides */
void DrawKeys(FMovieSceneControlRigSpaceChannel* Channel, TArrayView<const FKeyHandle> InKeyHandles, const UMovieSceneSection* InOwner, TArrayView<FKeyDrawParams> OutKeyDrawParams);
int32 DrawExtra(FMovieSceneControlRigSpaceChannel* Channel, const UMovieSceneSection* Owner, const FSequencerChannelPaintArgs& PaintArgs, int32 LayerId);

//UMovieSceneKeyStructType* IstanceGeneratedStruct(FMovieSceneControlRigSpaceChannel* Channel, FSequencerKeyStructGenerator* Generator);

//void PostConstructKeystance(const TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>& ChannelHandle, FKeyHandle InHandle, FStructOnScope* Struct)

/** Context menu overrides */
//void ExtendSectionMenu(FMenuBuilder& OuterMenuBuilder, TArray<TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>>&& Channels, TArrayView<UMovieSceneSection* const> Sections, TWeakPtr<ISequencer> InSequencer);
//void ExtendKeyMenu(FMenuBuilder& OuterMenuBuilder, TArray<TExtendKeyMenuParams<FMovieSceneControlRigSpaceChannel>>&& Channels, TWeakPtr<ISequencer> InSequencer);

/** Curve editor models */
inline bool SupportsCurveEditorModels(const TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>& Channel) { return true; }
TUniquePtr<FCurveModel> CreateCurveEditorModel(const TMovieSceneChannelHandle<FMovieSceneControlRigSpaceChannel>& Channel, const UE::Sequencer::FCreateCurveEditorModelParams& Params);


