// Copyright Epic Games, Inc. All Rights Reserved.

/**
 *
 * This thumbnail renderer displays a given Control Rig
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"
#include "ControlRigBlueprintLegacy.h"
#include "Engine/StaticMeshActor.h"
#include "ControlRigThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

UCLASS(config=Editor, MinimalAPI)
class UControlRigThumbnailRenderer : public USkeletalMeshThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	CONTROLRIGEDITOR_API virtual bool CanVisualizeAsset(UObject* Object) override;
	CONTROLRIGEDITOR_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// USkeletalMeshThumbnailRenderer implementation
	CONTROLRIGEDITOR_API virtual void AddAdditionalPreviewSceneContent(UObject* Object, UWorld* PreviewWorld) override;

protected:
	FControlRigAssetInterfacePtr RigBlueprint;
	TMap<FName, AActor*> ShapeActors;
};

