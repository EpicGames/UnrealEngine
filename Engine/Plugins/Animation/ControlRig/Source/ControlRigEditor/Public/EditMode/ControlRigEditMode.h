// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRig.h"
#include "InputCoreTypes.h"
#include "IControlRigObjectBinding.h"
#include "RigVMModel/RigVMGraph.h"
#include "Rigs/RigHierarchyContainer.h"
#include "Units/RigUnitContext.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/StrongObjectPtr.h"
#include "UnrealWidgetFwd.h"
#include "IPersonaEditMode.h"
#include "Misc/Guid.h"
#include "EditorDragTools.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "IDetailKeyframeHandler.h"
#include "WidgetFocusUtils.h"
#include "Transform/TransformConstraintUtil.h"
#include "ControlRigEditModeUtil.h"
#include "Sequencer/EditModeAnimationUtil.h"

#include "ControlRigEditMode.generated.h"

#define UE_API CONTROLRIGEDITOR_API

class UAnimDetailsProxyManager;
class UTickableConstraint;
class FDelegateHandle;
class FEditorViewportClient;
class FRigVMEditorBase;
class FViewport;
class UActorFactory;
struct FViewportClick;
class UControlRig;
class FControlRigInteractionScope;
class ISequencer;
class UControlManipulator;
class FUICommandList;
class FPrimitiveDrawInterface;
class FToolBarBuilder;
class FExtender;
class IMovieScenePlayer;
class AControlRigShapeActor;
class UDefaultControlRigManipulationLayer;
class UControlRigDetailPanelControlProxies;
class UControlRigPoseAsset;
struct FRigControl;
class IControlRigManipulatable;
class ISequencer;
enum class EControlRigSetKey : uint8;
class UToolMenu;
struct FGizmoState;
enum class EMovieSceneDataChangeType;
struct FMovieSceneChannelMetaData;
class UMovieSceneSection;
class UControlRigEditModeSettings;
class UEditorTransformGizmoContextObject;
struct FRotationContext;
namespace UE::ControlRigEditor { class FRigFloatingSpacePickerManager; }

DECLARE_DELEGATE_RetVal_ThreeParams(FTransform, FOnGetRigElementTransform, const FRigElementKey& /*RigElementKey*/, bool /*bLocal*/, bool /*bOnDebugInstance*/);
DECLARE_DELEGATE_ThreeParams(FOnSetRigElementTransform, const FRigElementKey& /*RigElementKey*/, const FTransform& /*Transform*/, bool /*bLocal*/);
DECLARE_DELEGATE_RetVal(TSharedPtr<FUICommandList>, FNewMenuCommandsDelegate);
DECLARE_MULTICAST_DELEGATE_TwoParams(FControlRigAddedOrRemoved, UControlRig*, bool /*true if added, false if removed*/);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FControlRigSelected, UControlRig*, const FRigElementKey& /*RigElementKey*/,const bool /*bIsSelected*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnControlRigVisibilityChanged, TArray<UControlRig*>);
DECLARE_DELEGATE_RetVal(UToolMenu*, FOnGetContextMenu);

class FControlRigEditMode;

enum class ERecreateControlRigShape
{
	RecreateNone,
	RecreateAll,
	RecreateSpecified
};


UCLASS()
class UControlRigEditModeDelegateHelper : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION()
	void OnPoseInitialized();

	UFUNCTION()
	void PostPoseUpdate();

	void AddDelegates(USkeletalMeshComponent* InSkeletalMeshComponent);
	void RemoveDelegates();

	TWeakObjectPtr<USkeletalMeshComponent> BoundComponent;
	FControlRigEditMode* EditMode = nullptr;

private:
	FDelegateHandle OnBoneTransformsFinalizedHandle;
};


struct FDetailKeyFrameCacheAndHandler: public IDetailKeyframeHandler
{
	FDetailKeyFrameCacheAndHandler() { UnsetDelegates(); }

	/** IDetailKeyframeHandler interface*/
	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const class IPropertyHandle& PropertyHandle) const override;
	virtual bool IsPropertyKeyingEnabled() const override;
	virtual void OnKeyPropertyClicked(const IPropertyHandle& KeyedPropertyHandle) override;
	virtual bool IsPropertyAnimated(const class IPropertyHandle& PropertyHandle, UObject* ParentObject) const override;
	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const override;

	/** Delegates Resetting Cached Data */
	void OnGlobalTimeChanged();
	void OnMovieSceneDataChanged(EMovieSceneDataChangeType);
	void OnChannelChangedWithTime(const TArray<FFrameNumber>&, const FMovieSceneChannelMetaData*, UMovieSceneSection*);

	void SetDelegates(TWeakPtr<ISequencer>& InWeakSequencer, FControlRigEditMode* InEditMode);
	void UnsetDelegates();
	void ResetCachedData();
	void UpdateIfDirty();
	/** Map to the last calculated property keyed status. Resets when Scrubbing, changing Movie Scene Data, etc */
	mutable TMap<const IPropertyHandle*, EPropertyKeyedStatus> CachedPropertyKeyedStatusMap;

	/* flag to specify that we need to update values, will poll this on edit mode tick for performance */
	bool bValuesDirty = false;
private:
	TWeakPtr<ISequencer> WeakSequencer;
	FControlRigEditMode* EditMode = nullptr;

};

USTRUCT(BlueprintType)
struct FMultiControlRigElementSelection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TWeakObjectPtr<UControlRig>> Rigs;

	UPROPERTY()
	TArray<FRigElementKeyCollection> KeysPerRig;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FControlRigEditModeInteractionStartedEvent, FMultiControlRigElementSelection, InteractionKeys, EControlRigInteractionType, InteractionType);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FControlRigEditModeInteractionEndedEvent, FMultiControlRigElementSelection, InteractionKeys);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FControlRigEditModeInteractionUpdatedEvent, FMultiControlRigElementSelection, InteractionKeys, FControlRigInteractionTransformContext, TransformContext);

class FControlRigEditMode : public IPersonaEditMode
{
public:
	static inline const FLazyName ModeName = FLazyName(TEXT("EditMode.ControlRig"));

	UE_API FControlRigEditMode();
	UE_API ~FControlRigEditMode();

	/** Set the Control Rig Object to be active in the edit mode. You set both the Control Rig and a possible binding together with an optional Sequencer
	 This will remove all other control rigs present and should be called for stand alone editors, like the Control Rig Editor*/
	UE_API void SetObjects(UControlRig* InControlRig, UObject* BindingObject, const TWeakPtr<ISequencer>& InSequencer);

	/** Add a Control Rig object if it doesn't exist, will return true if it was added, false if it wasn't since it's already there. You can also set the Sequencer.*/
	UE_API bool AddControlRigObject(UControlRig* InControlRig, const TWeakPtr<ISequencer>& InSequencer);

	/* Remove control rig */
	UE_API void RemoveControlRig(UControlRig* InControlRig);

	/*Replace old Control Rig with the New Control Rig, perhaps from a recompile in the level editor*/
	UE_API void ReplaceControlRig(UControlRig* OldControlRig, UControlRig* NewControlRig);

	/** This edit mode is re-used between the level editor and the asset editors (control rig editor etc.). Calling this indicates which context we are in */
	UE_API virtual bool IsInLevelEditor() const;
	/** This is used to differentiate between the control rig editor and any other (asset/level) editors in which this edit mode is used */
	virtual bool AreEditingControlRigDirectly() const { return false; }

	// FEdMode interface
	UE_API virtual bool UsesToolkits() const override;
	UE_API virtual void Enter() override;
	UE_API virtual void Exit() override;
	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	UE_API virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UE_API virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	UE_API virtual bool BeginTransform(const FGizmoState& InState) override;
	UE_API virtual bool EndTransform(const FGizmoState& InState) override;
	UE_API virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click) override;
	UE_API virtual bool BoxSelect(FBox& InBox, bool InSelect = true) override;
	UE_API virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true) override;
	UE_API virtual void SelectNone() override;
	UE_API virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	UE_API virtual bool UsesTransformWidget() const override;
	UE_API virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	UE_API virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	UE_API virtual FVector GetWidgetLocation() const override;
	UE_API virtual bool GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	UE_API virtual bool GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	UE_API virtual bool ShouldDrawWidget() const override;
	UE_API virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	UE_API virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override;
	UE_API virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;
	UE_API virtual bool MouseEnter( FEditorViewportClient* ViewportClient,FViewport* Viewport,int32 x, int32 y ) override;
	UE_API virtual bool MouseLeave(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	UE_API virtual void PostUndo() override;
	virtual EEditAction::Type GetActionDragDuplicate() override;

	/* IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override { return false; }
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override { check(false); return *(IPersonaPreviewScene*)this; }
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override {}

	/** FGCObject interface */
	UE_API virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	/** Refresh our internal object list (they may have changed) */
	UE_API void RefreshObjects();

	/** Find the edit mode corresponding to the specified world context */
	static UE_API FControlRigEditMode* GetEditModeFromWorldContext(UWorld* InWorldContext);

	/** Bone Manipulation Delegates */
	FOnGetRigElementTransform& OnGetRigElementTransform() { return OnGetRigElementTransformDelegate; }
	FOnSetRigElementTransform& OnSetRigElementTransform() { return OnSetRigElementTransformDelegate; }

	/** Context Menu Delegates */
	FOnGetContextMenu& OnGetContextMenu() { return OnGetContextMenuDelegate; }
	FNewMenuCommandsDelegate& OnContextMenuCommands() { return OnContextMenuCommandsDelegate; }
	FSimpleMulticastDelegate& OnAnimSystemInitialized() { return OnAnimSystemInitializedDelegate; }

	/* Control Rig Changed Delegate*/
	FControlRigAddedOrRemoved& OnControlRigAddedOrRemoved() { return OnControlRigAddedOrRemovedDelegate; }

	/* Control Rig Selected Delegate*/
	FControlRigSelected& OnControlRigSelected() { return OnControlRigSelectedDelegate; }

	/* Control Rig Visibility Delegate*/
	FOnControlRigVisibilityChanged& OnControlRigVisibilityChanged() { return OnControlRigVisibilityChangedDelegate; }

	/** Broadcasts a notification when a gizmo manipulation has started */
	FControlRigEditModeInteractionStartedEvent& OnGizmoInteractionStarted() { return OnGizmoInteractionStartedDelegate; }

	/** Returns a delegate broadcast when control rig shape actors were recreated */
	FSimpleMulticastDelegate& OnControlRigShapeActorsRecreated() { return OnControlRigShapeActorsRecreatedDelegate; }

	/** Broadcasts a notification when a gizmo manipulation has ended */
	FControlRigEditModeInteractionEndedEvent& OnGizmoInteractionEnded() { return OnGizmoInteractionEndedDelegate; }

	/** Broadcasts a notification when a gizmo manipulation has been updated (before the update actually takes place) */
	FControlRigEditModeInteractionUpdatedEvent& OnGizmoInteractionPreUpdated() { return OnGizmoInteractionPreUpdatedDelegate; }
	/** Broadcasts a notification when a gizmo manipulation has been updated (after the transforms have been updated) */
	FControlRigEditModeInteractionUpdatedEvent& OnGizmoInteractionPostUpdated() { return OnGizmoInteractionPostUpdatedDelegate; }

	// callback that gets called when rig element is selected in other view
	UE_API void OnHierarchyModified(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	UE_API void OnHierarchyModified_AnyThread(ERigHierarchyNotification InNotif, URigHierarchy* InHierarchy, const FRigNotificationSubject& InSubject);
	UE_API void OnControlModified(UControlRig* Subject, FRigControlElement* InControlElement, const FRigControlModifiedContext& Context);
	UE_API void OnPreConstruction_AnyThread(UControlRig* InRig, const FName& InEventName);
	UE_API void OnPostConstruction_AnyThread(UControlRig* InRig, const FName& InEventName);

	/** return true if it can be removed from preview scene 
	- this is to ensure preview scene doesn't remove shape actors */
	UE_API bool CanRemoveFromPreviewScene(const USceneComponent* InComponent);

	TSharedPtr<FUICommandList> GetCommandBindings() const { return CommandBindings; }

	/** Requests to recreate the shape actors in the next tick. Will recreate only the ones for the specified
	Control Rig, otherwise will recreate all of them*/
	UE_API void RequestToRecreateControlShapeActors(UControlRig* ControlRig = nullptr); 

	static uint32 ValidControlTypeMask()
	{
		return FRigElementTypeHelper::ToMask(ERigElementType::Control);
	}

	//show or hide the specified Controls on this Control Rig;
	UE_API void ShowControlRigControls(UControlRig* InControlRig,const TSet<FString>& Names, bool bVal);
	UE_API void ClearOutHidden(UControlRig* InControlRig);

protected:

	UE_API EControlRigInteractionTransformSpace GetTransformSpace() const;
	
	// shape related functions wrt enable/selection
	/** Get the node name from the property path */
	UE_API AControlRigShapeActor* GetControlShapeFromControlName(UControlRig* InControlRig,const FName& InControlName) const;

	/** Helper function: set ControlRigs array to the details panel */
	UE_API void SetObjects_Internal();

	/** Set up Details Panel based upon Selected Objects*/
	UE_API void SetUpDetailPanel() const;

	/** Updates cached pivot transforms */
	UE_API void UpdatePivotTransforms();
	UE_API bool ComputePivotFromEditedShape(UControlRig* InControlRig, FTransform& OutTransform) const;
	UE_API bool ComputePivotFromShapeActors(UControlRig* InControlRig, const bool bEachLocalSpace, const EControlRigInteractionTransformSpace InSpace, FTransform& OutTransform) const;
	UE_API bool ComputePivotFromElements(UControlRig* InControlRig, FTransform& OutTransform) const;
	UE_API FTransform GetPivotOrientation(const FRigElementKey& InControlKey, const UControlRig* InControlRig, URigHierarchy* InHierarchy, const EControlRigInteractionTransformSpace InSpace, const FTransform& InComponentTransform) const;
	
	/** Get the current coordinate system space */
	UE_API ECoordSystem GetCoordSystemSpace() const;
	
	/** Handle selection internally */
	UE_API void HandleSelectionChanged();

	/** Toggles visibility of acive control rig shapes in the viewport */
	UE_API void ToggleManipulators();

	/** Toggles visibility of all  control rig shapes in the viewport */
	UE_API void ToggleAllManipulators();

	/** Returns true if all control rig shapes are visible in the viewport */
	UE_API bool AreControlsVisible() const;

	UE_API virtual bool HandleBeginTransform(const FEditorViewportClient* InViewportClient);
	UE_API virtual bool HandleEndTransform(FEditorViewportClient* InViewportClient);

public:
	UE_API virtual bool IsDisplayControlsEnabled() const;
	UE_API virtual void ToggleDisplayControlsEnabled();

	UE_API virtual bool IsControlsAsOverlayEnabled() const;
	UE_API virtual void ToggleControlsAsOverlay();

	UE_API virtual bool IsDisplayNullsEnabled() const;
	UE_API virtual void ToggleDisplayNulls();

	UE_API virtual bool IsDisplaySocketsEnabled() const;
	UE_API virtual void ToggleDisplaySockets();

	UE_API virtual bool IsDisplayAxesOnSelectionEnabled() const;
	UE_API virtual void ToggleDisplayAxesOnSelection();

	UE_API virtual float GetAxesOnSelectionScale() const;

	/** Toggles visibility of acive control rig shapes inside the selected module in the viewport */
	UE_API void ToggleModuleManipulators();

	/** Opens the Control Rig Pose Library panel */
	UE_API void OpenPoseLibraryTab();

	/** Closes the Control Rig Pose Library panel */
	UE_API void ClosePoseLibraryTab();

	/** Clear Selection*/
	UE_API void ClearSelection();

	/** Frame to current Control Selection*/
	UE_API void FrameSelection();

	/** Frame a list of provided items*/
   	UE_API void FrameItems(const TArray<FRigElementKey>& InItems);

	/** Sets Passthrough Key on selected anim layers */
	UE_API void SetAnimLayerPassthroughKey();

	/** Select weight channel on selected anim layers */
	UE_API void SelectAnimLayerWeightChannel();

	/** Select Mirrored Controls on Current Selection*/
	UE_API void SelectMirroredControls();

	/** Select Mirrored Controls on Current Selection, keeping current selection*/
	UE_API void AddMirroredControlsToSelection();

	/** Put Selected Controls To Mirrored Pose*/
	UE_API void MirrorSelectedControls();

	/** Put Unselected Controls To Mirrored Pose*/
	UE_API void MirrorUnselectedControls();

	/** Select All Controls*/
	UE_API void SelectAllControls();

	//for the following pose functions we only support one pose(PoseNum = 0)
	//but may support more later
	/** Save a pose of selected controls*/
	UE_API void SavePose(int32 PoseNum = 0);

	/** Select controls in saved pose*/
	UE_API void SelectPose(bool bDoMirror, int32 PoseNum = 0 );

	/** Paste saved pose */
	UE_API void PastePose(bool bDoMirror, int32 PoseNum = 0);

	/** Opens up the space picker widget */
	UE_API void OpenSpacePickerWidget();

	/** Reset Transforms to Default(which may not be zero if initial values are present*/
	UE_API void ZeroTransforms(bool bSelectionOnly, bool bIncludeChannels = true, UControlRig* InControlRig = nullptr);

	/** Invert Input Pose, this will set layered controls rigs to default pose, for non-layered same as ZeroTransforms*/
	UE_API void InvertInputPose(bool bSelectionOnly, bool bIncludeChannels = true);

	/** Reset Transforms for this Control Rig Controls based upon selection and channel states*/
	static UE_API void ZeroTransforms(UControlRig* ControlRig, const FRigControlModifiedContext& Context,  bool bSelectionOnly, bool bIncludeChannels);

	/** Invert Input Pose for this Control Rig Controls based upon selection and channel states*/
	static UE_API void InvertInputPose(UControlRig* ControlRig, const FRigControlModifiedContext& Context, bool bSelectionOnly, bool bIncludeChannels);

	/** Toggle Motion Trail Mode*/
	UE_API void ToggleMotionTrails();

	/** Toggle Pivot Mode*/
	UE_API void TogglePivotMode();

	/** Isolate or un-isolate linked anim tracks for every binding in the current Sequencer */
	UE_API void IsolateLinkedAnimTracksForAllBindings(bool bIsolate);

private:

	/** Opens the space picker or a contextual menu with the picker and utility actions if any. */
	void OpenTabMenu();

	/** Whether or not we should Frame Selection or not*/
	UE_API bool CanFrameSelection();

	/** Pending focus handler */
	FPendingWidgetFocus PendingFocus;

	/** Pending focus cvar binding functions to enable/disable pending focus mode */
	UE_API void RegisterPendingFocusMode();
	UE_API void UnregisterPendingFocusMode();
	FDelegateHandle PendingFocusHandle;

	/** Listen to sequencer and optimize notifications and performances when playing. */
	UE_API void SetSequencerDelegates(const TWeakPtr<ISequencer>& InWeakSequencer);
	UE_API void UnsetSequencerDelegates() const;
	UE_API void UpdateSequencerStatus();
	bool bSequencerPlaying = false;
	
public:
	
	/** Toggle Shape Transform Edit*/
	UE_API void ToggleControlShapeTransformEdit();

private:
	
	/** The hotkey text is passed to a viewport notification to inform users how to toggle shape edit*/
	UE_API FText GetToggleControlShapeTransformEditHotKey() const;

	/** Bind our keyboard commands */
	UE_API void BindCommands();

	/** It creates if it doesn't have it */
	UE_API void RecreateControlShapeActors();

	/** Let the preview scene know how we want to select components */
	UE_API bool ShapeSelectionOverride(const UPrimitiveComponent* InComponent) const;

	/** Enable editing of control's shape transform instead of control's transform*/
	bool bIsChangingControlShapeTransform;

protected:

	TWeakPtr<ISequencer> WeakSequencer;
	FGuid LastMovieSceneSig;

	/**  Handles spawning the floating space picker. Ensures only one can be created at the time. Removes the widget when destroyed.  */
	TPimplPtr<UE::ControlRigEditor::FRigFloatingSpacePickerManager> FloatingSpacePickerManager;
	
	/** Lazy creation of the picker manager. */
	UE::ControlRigEditor::FRigFloatingSpacePickerManager& GetSpacePickerManager();

	/** The scope for the interaction, one per manipulated Control rig */
	TMap<UControlRig*, TUniquePtr<FControlRigInteractionScope>> InteractionScopes;

	/** True if there's tracking going on right now */
	bool bIsTracking;

	/** Whether a manipulator actually made a change when transacting */
	bool bManipulatorMadeChange;

	/** Guard value for selection */
	bool bSelecting;

	/** If selection was changed, we set up proxies on next tick */
	bool bSelectionChanged;

	/** Cached transform of pivot point for selected objects for each Control Rig */
	TMap<UControlRig*,FTransform> PivotTransforms;

	/** Previous cached transforms, need this to check on tick if any transform changed, gizmo may have changed*/
	TMap<UControlRig*, FTransform> LastPivotTransforms;

	/** Command bindings for keyboard shortcuts */
	TSharedPtr<FUICommandList> CommandBindings;

	/** Called from the editor when a blueprint object replacement has occurred */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Copy control visibility from the source rig (and its modules if necessary) to the target rig */ 
	UE_API void CopyControlsVisibility(const UControlRig* SourceRig, UControlRig* TargetRig);

	/** Return true if transform setter/getter delegates are available */
	UE_API bool IsTransformDelegateAvailable() const;

	FOnGetRigElementTransform OnGetRigElementTransformDelegate;
	FOnSetRigElementTransform OnSetRigElementTransformDelegate;
	FOnGetContextMenu OnGetContextMenuDelegate;
	FNewMenuCommandsDelegate OnContextMenuCommandsDelegate;
	FSimpleMulticastDelegate OnAnimSystemInitializedDelegate;
	FControlRigAddedOrRemoved OnControlRigAddedOrRemovedDelegate;
	FControlRigSelected OnControlRigSelectedDelegate;
	FOnControlRigVisibilityChanged OnControlRigVisibilityChangedDelegate;

	/** Broadcasts a notification when a gizmo manipulation has started */
	FControlRigEditModeInteractionStartedEvent OnGizmoInteractionStartedDelegate;

	/** Broadcasts a notification when a gizmo manipulation has ended */
	FControlRigEditModeInteractionEndedEvent OnGizmoInteractionEndedDelegate;

	/** Broadcasts a notification when a gizmo manipulation has been updated (before the update actually takes place) */
	FControlRigEditModeInteractionUpdatedEvent OnGizmoInteractionPreUpdatedDelegate;

	/** Broadcasts a notification when a gizmo manipulation has been updated (after the transforms have been updated) */
	FControlRigEditModeInteractionUpdatedEvent OnGizmoInteractionPostUpdatedDelegate;
	
	/** Broadcasts a notification when a control rig shape actors were recreated */
	FSimpleMulticastDelegate OnControlRigShapeActorsRecreatedDelegate;

	/** GetSelectedRigElements, if InControlRig is nullptr get the first one */
	UE_API TArray<FRigElementKey> GetSelectedRigElements() const;
	static UE_API TArray<FRigElementKey> GetSelectedRigElements(UControlRig* InControlRig);

	/** Get the rig elements, based upon the selection or if it's a channel, also will do internal additive filters(like no bools)  */
	static UE_API TArray<FRigElementKey> GetRigElementsForSettingTransforms(UControlRig* InControlRig, bool bSelectionOnly, bool bIncludeChannels);

	/* Flag to recreate shapes during tick */
	ERecreateControlRigShape RecreateControlShapesRequired;
	/* List of Control Rigs we should recreate*/
	TArray<UControlRig*> ControlRigsToRecreate;

	/* Flag to temporarily disable handling notifs from the hierarchy */
	bool bSuspendHierarchyNotifs;

	/** Shape actors */
	TMap<TWeakObjectPtr<UControlRig>,TArray<TObjectPtr<AControlRigShapeActor>>> ControlRigShapeActors;


	/** Manager for anim details proxies */
	TObjectPtr<UAnimDetailsProxyManager> AnimDetailsProxyManager;

	/** Utility functions for UI/Some other viewport manipulation*/
	UE_API bool IsControlSelected(const bool bUseShapes = false) const;
	UE_API bool AreRigElementSelectedAndMovable(UControlRig* InControlRig) const;
	
	/** Set initial transform handlers */
	UE_API void OpenContextMenu(FEditorViewportClient* InViewportClient);

	/** previous Gizmo(Widget) scale before we enter this mode, used to set it back*/
	UE_DEPRECATED(5.8, "The ControlRigEditMode-Specific gizmo scale feature has been removed.")
	float PreviousGizmoScale = 1.0f;

	/** Per ControlRig dependencies between the selected controls during interaction. */
	TMap<UControlRig*, UE::ControlRigEditMode::FInteractionDependencyCache> InteractionDependencies;
	
	/** Returns the interaction dependencies of that ControlRig. */
	UE_API UE::ControlRigEditMode::FInteractionDependencyCache& GetInteractionDependencies(UControlRig* InControlRig);
	
public: 
	/** Clear all selected RigElements */
	UE_API void ClearRigElementSelection(uint32 InTypes);

	/** Set a RigElement's selection state */
	UE_API void SetRigElementSelection(UControlRig* ControlRig, ERigElementType Type, const FName& InRigElementName, bool bSelected);

	/** Set multiple RigElement's selection states */
	UE_API void SetRigElementSelection(UControlRig* ControlRig, ERigElementType Type, const TArray<FName>& InRigElementNames, bool bSelected);

	/** Check if any RigElements are selected */
	UE_API bool AreRigElementsSelected(uint32 InTypes, UControlRig* InControlRig) const;

	/** Get all of the selected Controls*/
	UE_API void GetAllSelectedControls(TMap<UControlRig*, TArray<FRigElementKey>>& OutSelectedControls) const;

	/** Get all of the ControlRigs, maybe not valid anymore */
	UE_API TArrayView<const TWeakObjectPtr<UControlRig>> GetControlRigs() const;
	UE_API TArrayView<TWeakObjectPtr<UControlRig>> GetControlRigs();
	/* Get valid  Control Rigs possibly just visible*/
	UE_API TArray<UControlRig*> GetControlRigsArray(bool bIsVisible);
	UE_API TArray<const UControlRig*> GetControlRigsArray(bool bIsVisible) const;

	/** Get the detail proxies control rig*/
	UAnimDetailsProxyManager* GetAnimDetailsProxyManager() const { return AnimDetailsProxyManager; }

	/** Get Sequencer Driving This*/
	TWeakPtr<ISequencer> GetWeakSequencer() { return WeakSequencer; }

	/** Suspend Rig Hierarchy Notifies*/
	void SuspendHierarchyNotifs(bool bVal) { bSuspendHierarchyNotifs = bVal; }

	/** Request a certain transform widget for the next update */
	UE_API void RequestTransformWidgetMode(UE::Widget::EWidgetMode InWidgetMode);
	
private:
	/** Adapts hovered viewport hit proxies to the controls-only selectability policy used for cursor feedback. */
	UE_API bool IsViewportHitProxySelectable(HHitProxy* const InHitProxy) const;

	/** Returns whether an object is selectable under the current Control Rig viewport policy, including controls-only mode. */
	UE_API bool IsViewportObjectSelectable(UObject* const InObject) const;

	/** Set a RigElement's selection state. */
	UE_API void SetRigElementSelectionInternal(UControlRig* ControlRig, ERigElementType Type, const FName& InRigElementName, bool bSelected);

	/** Set multiple RigElements' selection states. */
	UE_API void SetRigElementsSelectionInternal(const TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>>& InRigElementsToSelect, bool bSelected);

	/** Whether or not Pivot Transforms have changed, in which case we need to redraw viewport. */
	UE_API bool HasPivotTransformsChanged() const;

	/** Updates the pivot transforms before ticking to ensure that they are up-to-date when needed. */
	UE_API void UpdatePivotTransformsIfNeeded(UControlRig* InControlRig, FTransform& InOutTransform) const;
	
	FEditorViewportClient* CurrentViewportClient;
	TArray<UE::Widget::EWidgetMode> RequestedWidgetModes;

	FDelegateHandle ViewportSelectabilityDelegateHandle;

/* store coordinate system per widget mode*/
private:
	UE_API void OnWidgetModeChanged(UE::Widget::EWidgetMode InWidgetMode);
	UE_API void OnCoordSystemChanged(ECoordSystem InCoordSystem);
	TArray<ECoordSystem> CoordSystemPerWidgetMode;
	bool bIsChangingCoordSystem;

	UE_API bool CanChangeControlShapeTransform();

	UE_API void OnSettingsChanged(const UControlRigEditModeSettings* InSettings);
	UE_API void OnSettingsPropertyChanged(UObject* InObject, FPropertyChangedEvent& PropertyChangedEvent);
	
public:
	//Toolbar functions
	UE_API void SetOnlySelectRigControls(bool val);
	UE_API bool GetOnlySelectRigControls()const;
	UE_API bool SetSequencer(TWeakPtr<ISequencer> InSequencer);

private:
	UE_API TSet<FName> GetActiveControlsFromSequencer(UControlRig* ControlRig);

	/** Create/Delete/Update shape actors for the specified ControlRig */
	UE_API void CreateShapeActors(UControlRig* InControlRig);
	UE_API void DestroyShapesActors(UControlRig* InControlRig);
	UE_API bool TryUpdatingControlsShapes(UControlRig* InControlRig);

	/*Internal function for adding ControlRig*/
	UE_API void AddControlRigInternal(UControlRig* InControlRig);

	/**
	 * Updates the bound components (skeletal meshes or control rig component) and the control shapes so that all transform data are updated.
	 * if InRig is empty, all rigs / control shapes will be updated.
	 */
	UE_API void TickManipulatableObjects(const TArray<TWeakObjectPtr<UControlRig>>& InRigs = TArray<TWeakObjectPtr<UControlRig>>()) const;

	/* Check on tick to see if movie scene has changed, returns true if it has*/
	UE_API bool CheckMovieSceneSig();
	UE_API void SetControlShapeTransform( const AControlRigShapeActor* InShapeActor, const FTransform& InGlobalTransform,
		const FTransform& InToWorldTransform, const FRigControlModifiedContext& InContext, const bool bPrintPython,
		const FControlRigInteractionTransformContext& InTransformContext, const bool bFixEulerFlips = true) const;
	static UE_API FTransform GetControlShapeTransform(const AControlRigShapeActor* ShapeActor);

	static UE_API void ChangeControlShapeTransform(AControlRigShapeActor* ShapeActor, const FControlRigInteractionTransformContext& InContext, const FTransform& ToWorldTransform);

	UE_API bool ModeSupportedByShapeActor(const AControlRigShapeActor* ShapeActor, UE::Widget::EWidgetMode InMode) const;

public:
	//notify driven controls, should this be inside CR instead?
	static UE_API void NotifyDrivenControls(UControlRig* InControlRig, const FRigElementKey& InKey, const FRigControlModifiedContext& InContext);

protected:

	UE_API bool MoveControlShapeLocally( AControlRigShapeActor* ShapeActor,
		const FControlRigInteractionTransformContext& InTransformContext, const FTransform& ToWorldTransform,
		const FTransform& InLocal);
	
	UE_API void MoveControlShape(AControlRigShapeActor* ShapeActor,
		const FControlRigInteractionTransformContext& InContext, const FTransform& ToWorldTransform,
		const bool bUseLocal, const bool bCalcLocal, FTransform* InOutLocal, TArray<TFunction<void()>>& OutTasks);

	/** Get bindings to a runtime object */
	//If the passed in ControlRig is nullptr we use the first Control Rig(this can happen from the BP Editors).
	UE_API USceneComponent* GetHostingSceneComponent(const UControlRig* ControlRig = nullptr) const;
	UE_API FTransform	GetHostingSceneComponentTransform(const UControlRig* ControlRig =  nullptr) const;

	//Get if the hosted component is visible
	UE_API bool IsControlRigSkelMeshVisible(const UControlRig* InControlRig) const;
	
public:  
		TSharedPtr<FDetailKeyFrameCacheAndHandler> DetailKeyFrameCache;

private:

	// Post pose update handler
	UE_API void OnPoseInitialized();

	/**
	 * Updates the control shapes properties (transform, visibility, ...) depending on the current viewport state.
	 * If InRig is empty, all control shapes will be updated.
	 */
	UE_API void PostPoseUpdate(const FEditorViewportClient* InViewportClient = nullptr, const TArray<TWeakObjectPtr<UControlRig>>& InRigs = TArray<TWeakObjectPtr<UControlRig>>()) const;
	
	UE_API void UpdateSelectabilityOnSkeletalMeshes(UControlRig* InControlRig, bool bEnabled);

	UE_API bool IsMovingCamera(const FViewport* InViewport) const;
	UE_API bool IsDoingDrag(const FViewport* InViewport) const;

	// world clean up handlers
	FDelegateHandle OnWorldCleanupHandle;
	UE_API void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources);
	UWorld* WorldPtr = nullptr;

	UE_API void OnEditorClosed();

	struct FMarqueeDragTool
    {
    	FMarqueeDragTool();
    	~FMarqueeDragTool() = default;
    
    	bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
    	bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport);
    	void MakeDragTool(FEditorViewportClient* InViewportClient);
    	bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale);
    
    	bool UsingDragTool() const;
    	void Render3DDragTool(const FSceneView* View, FPrimitiveDrawInterface* PDI);
    	void RenderDragTool(const FSceneView* View, FCanvas* Canvas);
    
    private:
    	
    	/**
    	 * If there is a dragging tool being used, this will point to it.
    	 * Gets newed/deleted in StartTracking/EndTracking.
    	 */
    	TSharedPtr<FDragTool> DragTool;
    
    	/** Tracks whether the drag tool is in the process of being deleted (to protect against reentrancy) */
    	bool bIsDeletingDragTool = false;

		FControlRigEditMode* EditMode;
    };

	FMarqueeDragTool DragToolHandler;

protected:
	TArray<TWeakObjectPtr<UControlRig>> RuntimeControlRigs;
	bool bShowControlsAsOverlay;

	//if selection flag is set, get control rigs with selected controls, if not get all control rigs if there are none selected
	TArray<UControlRig*> GetControlRigsBasedOnSelection(bool bSelectionOnly) const;

private:
	TMap<UControlRig*,TStrongObjectPtr<UControlRigEditModeDelegateHelper>> DelegateHelpers;

	TArray<FRigElementKey> DeferredItemsToFrame;

	/** Computes the current interaction types based on the widget mode */
	static UE_API uint8 GetInteractionType(const FEditorViewportClient* InViewportClient);
	uint8 InteractionType;

	bool bPivotsNeedUpdate = true;

	bool bIsConstructionEventRunning;
	TArray<uint32> LastHierarchyHash;
	TArray<uint32> LastShapeLibraryHash;

	/** A list of rigs we need to run during this tick */
	TArray<UControlRig*> RigsToEvaluateDuringThisTick;
	uint32 RigEvaluationBracket = 0;

	class FPendingControlRigEvaluator
	{
		public:

		FPendingControlRigEvaluator(FControlRigEditMode* InEditMode)
		: EditMode(InEditMode)
		{
			EditMode->RigEvaluationBracket++;
		}

		~FPendingControlRigEvaluator();
		
	private:

		FControlRigEditMode* EditMode;
	};

	static UE_API void EvaluateRig(UControlRig* InControlRig);
	
	//to disable post pose update, needed for offfline evlauations
	static UE_API bool bDoPostPoseUpdate;

private:

	// get default/mutable settings
	// todo: have a local setting object that listen to property changes to send updates
	// instead of getting data directly from the CDO
	UE_API const UControlRigEditModeSettings* GetSettings() const;
	UE_API UControlRigEditModeSettings* GetMutableSettings() const;
	mutable TWeakObjectPtr<UControlRigEditModeSettings> WeakSettings;

	//pose used by the hotkeys
	TObjectPtr<UControlRigPoseAsset> StoredPose;
	
	TWeakObjectPtr<UEditorTransformGizmoContextObject> WeakGizmoContext;
	UE_API FRotationContext& GetRotationContext() const;
	UE_API void UpdateRotationContext();

	UE_API TOptional<FTransform> GetConstraintParentTransform(const UControlRig* InControlRig, const FName& InControlName) const;
	mutable UE::TransformConstraintUtil::FConstraintsInteractionCache ConstraintsCache;

	// Used to store and apply keyframes (if deferred)  
	mutable UE::AnimationEditMode::FControlRigKeyframer Keyframer;

	// whether there are other objects selected in the world that are not shape actors or components bound to the rigs
	bool bOtherSelectedObjects = false;
	TSet<TWeakObjectPtr<UObject>> SelectedObjects;
	void CacheAnyOtherSelectedObjects();

	void HandleActorMoving(AActor* InMovedActor);
	bool bUpdateNonInteractingRigs = false;
	void CacheNonInteractingRigsToUpdate();
	TArray<TWeakObjectPtr<UControlRig>> NonInteractingRigs;
	
	FDelegateHandle BuildViewportInteractionsDelegate;
	
	// direct control system for viewport click-and-drag interactions
	UE::AnimationEditMode::FDirectControl DirectControl;
	
	// binds the direct control system to the gizmo
	void BindDirectControl();
	
	// This structure is used to cache the last mouse screen position and last viewport in FControlRigEditMode::MouseMove
	// to prevent synthesized mouse movements from triggering useless hover updates. 
	struct FMouseMoveCache
	{
		bool Update(const FViewport* InViewport, const int32 InX, const int32 InY)
		{
			const bool bUpdated = InViewport != Viewport || InX != X || InY != Y;
			if (bUpdated)
			{
				Viewport = InViewport;
				X = InX;
				Y = InY;
			}
			return bUpdated;
		}
    
	private:
		const FViewport* Viewport = nullptr;
		int32 X = 0;
		int32 Y = 0;
	};
	FMouseMoveCache MouseMoveCache;
	
	friend class FControlRigEditorModule;
	friend class FControlRigBaseEditor;
#if WITH_RIGVMLEGACYEDITOR
	friend class FControlRigLegacyEditor;
#endif
	friend class FControlRigEditor;
	friend class UControlRigEditModeDelegateHelper;
	friend class SControlRigEditModeTools;

public:

	class FTurnOffPosePoseUpdate
	{
	public:
		FTurnOffPosePoseUpdate()
		{
			bLastVal = FControlRigEditMode::bDoPostPoseUpdate;
			FControlRigEditMode::bDoPostPoseUpdate = false;
		}
		~FTurnOffPosePoseUpdate()
		{
			FControlRigEditMode::bDoPostPoseUpdate = bLastVal;
		}
	private:
		bool bLastVal;
	};
};

#undef UE_API
