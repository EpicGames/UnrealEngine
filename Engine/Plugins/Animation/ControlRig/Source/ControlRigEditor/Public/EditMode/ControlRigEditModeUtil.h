// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlRigGizmoActor.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "Math/Rotator.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyDefines.h"
#include "UObject/WeakObjectPtr.h"
#include "UnrealWidgetFwd.h"

#include "ControlRigEditModeUtil.generated.h"

class FEditorViewportClient;
class UControlRig;
class ULevelEditorViewportSettings;
class URigHierarchy;
class UTickableConstraint;

struct FConvexVolume;
struct FRigControlModifiedContext;
struct FRigControlElement;


UENUM()
enum class EControlRigInteractionTransformSpace
{
	World,
	Local,
	Parent,
	Explicit
};

/**
 * FControlRigInteractionTransformContext provides a way of passing the various transform parameters to functions that need to know what the transform context is.
 * Extend it if necessary, particularly to avoid overloading some functions signatures.
 */
USTRUCT(BlueprintType)
struct FControlRigInteractionTransformContext
{
	GENERATED_BODY()
	FControlRigInteractionTransformContext() = default;
	
	FControlRigInteractionTransformContext(const UE::Widget::EWidgetMode& InWidgetMode)
		: bTranslation(InWidgetMode == UE::Widget::WM_Translate || InWidgetMode == UE::Widget::WM_TranslateRotateZ)
		, bRotation(InWidgetMode == UE::Widget::WM_Rotate || InWidgetMode == UE::Widget::WM_TranslateRotateZ)
		, bScale(InWidgetMode == UE::Widget::WM_Scale)
	{}
	
	bool CanTransform() const
	{
		return bTranslation || bRotation || bScale;
	}

	bool bTranslation = false;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Interaction")
	FVector Drag = FVector::ZeroVector;

	bool bRotation = false;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Interaction")
	FRotator Rot = FRotator::ZeroRotator;

	bool bScale = false;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Interaction")
	FVector Scale = FVector::OneVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Interaction")
	EControlRigInteractionTransformSpace Space = EControlRigInteractionTransformSpace::World;
};

namespace UE::ControlRigEditMode
{

/**
 * FInteractionDependencyCache provides a minimal "dependency graph" between the selected controls during interaction.
 * it stores information on who is a child and who is a parent in whatever is being manipulated, as well as
 * the parents' pose versions (in the complete hierarchy), to know whether any previous manipulation has modified
 * the parents' transform of an element being manipulated.
 */

struct FInteractionDependencyCache
{
	bool HasDownwardDependencies(const FRigElementKey& InKey) const
	{
		return Parents.Contains(InKey);
	}

	bool HasUpwardDependencies(const FRigElementKey& InKey) const
	{
		return Children.Contains(InKey);
	}

	bool CheckAndUpdateParentsPoseVersion()
	{
		if (ParentsPoseVersion.IsEmpty())
		{
			return false;
		}
		
		if (URigHierarchy* Hierarchy = WeakHierarchy.Get())
		{
			bool bHasChanged = false;
			
			for (auto&[ElementIndex, PoseVersion]: ParentsPoseVersion)
			{
				const int32 NewPoseVersion = Hierarchy->GetPoseVersion(Hierarchy->Get<FRigTransformElement>(ElementIndex));
				if (NewPoseVersion != PoseVersion)
				{
					PoseVersion = NewPoseVersion;
					bHasChanged = true;
				} 
			}

			return bHasChanged;
		}
		return false;
	}

	TSet<FRigElementKey> Parents;
	TSet<FRigElementKey> Children;
	TMap<int32, int32> ParentsPoseVersion;
	TWeakObjectPtr<URigHierarchy> WeakHierarchy;
};

/**
* FExplicitRotationInteraction is a wrapper struct to apply euler angle deltas to controls.
*/
	
struct FExplicitRotationInteraction
{
	FExplicitRotationInteraction(const FControlRigInteractionTransformContext& InContext,
		UControlRig* InControlRig,
		URigHierarchy* InHierarchy,
		FRigControlElement* InControlElement,
		const FTransform& InComponentWorldTransform);
	
	bool IsValid() const;

	void Apply(const FTransform& InGlobalTransform, const FRigControlModifiedContext& InContext, const bool bPrintPython,
		const TArray< TWeakObjectPtr<UTickableConstraint> >& InConstraints = TArray< TWeakObjectPtr<UTickableConstraint> >()) const;

private:
	const FControlRigInteractionTransformContext& TransformContext;
	UControlRig* ControlRig = nullptr;
	URigHierarchy* Hierarchy = nullptr;
	FRigControlElement* ControlElement = nullptr;
	const FTransform& ComponentWorldTransform;
};

/**
* FSelectionHelper is a wrapper struct to handle control rig related viewport selection.
*/

struct FSelectionHelper
{
	FSelectionHelper(
		FEditorViewportClient* InViewportClient,
		const TMap<TWeakObjectPtr<UControlRig>, TArray<TObjectPtr<AControlRigShapeActor>>>& InControlRigShapeActors,
		TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>>& OutElements);

	/**
	 * Gets the elements contained in the frustum.
	 * Note that the function will actually use a screen space rectangle selection if occluded objects must be skipped.
	 */
	void GetFromFrustum(const FConvexVolume& InFrustum) const;

private:

	/** Returns true if the viewport to select into is not null. */
	bool IsValid() const;
	
	/** Returns a screen space rectangle based of the frustum. (it assumes that there's a valid near plane) */
	TOptional<FIntRect> RectangleFromFrustum(const FConvexVolume& InFrustum) const;

	/** Gets the no-occluded elements contained in the screen space rectangle. */
	void GetNonOccludedElements(const FIntRect& InRect) const;

	/** The viewport client being interacted with. */
	FEditorViewportClient* ViewportClient = nullptr;

	/** A reference to the edit mode's control shapes. */
	const TMap<TWeakObjectPtr<UControlRig>, TArray<TObjectPtr<AControlRigShapeActor>>>& ControlRigShapeActors;

	/** The elements to be selected. */
	TMap<TWeakObjectPtr<UControlRig>, TArray<FRigElementKey>>& Elements;

	/** Current viewport settings. (used for strict box & transparent selection) */
	const ULevelEditorViewportSettings* LevelEditorViewportSettings = nullptr;

	/** List of layers that are hidden in this view. (only valid for level editor vpc) */
	TArray<FName> HiddenLayers;
};
	
}

