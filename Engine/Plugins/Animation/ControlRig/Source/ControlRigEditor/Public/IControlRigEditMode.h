// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPersonaEditMode.h"

class ISequencer;

class IControlRigEditMode : public IPersonaEditMode
{
public:	
	virtual void SetObjects(const TWeakObjectPtr<>& InSelectedObject,  UObject* BindingObject, TWeakPtr<ISequencer> InSequencer) = 0;
};
