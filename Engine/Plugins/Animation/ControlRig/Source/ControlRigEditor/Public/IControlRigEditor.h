// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_RIGVMLEGACYEDITOR
#include "Editor/RigVMLegacyEditor.h"
#endif
#include "Editor/RigVMNewEditor.h"

class UControlRigBlueprint;
class FBlueprintActionDatabaseRegistrar;

class IControlRigEditor
{
public:
	virtual URigVMHost* GetRigVMHost() const = 0;
	virtual bool IsEditorInitialized() const = 0;
};

#if WITH_RIGVMLEGACYEDITOR
class IControlRigLegacyEditor : public FRigVMLegacyEditor, public IControlRigEditor
{
};
#endif

class IControlRigNewEditor : public FRigVMNewEditor, public IControlRigEditor
{
};
