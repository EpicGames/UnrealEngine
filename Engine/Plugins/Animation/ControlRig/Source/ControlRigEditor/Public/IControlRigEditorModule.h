// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IControlRigEditor.h"
#include "RigVMEditorModule.h"
#include "Kismet2/StructureEditorUtils.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ClassViewerModule.h"
#include "ClassViewerFilter.h"

#define UE_API CONTROLRIGEDITOR_API

DECLARE_LOG_CATEGORY_EXTERN(LogControlRigEditor, Log, All);

class IToolkitHost;
class UControlRigBlueprint;
class UControlRigRuntimeAsset;
class UControlRigGraphNode;
class UControlRigGraphSchema;
class FConnectionDrawingPolicy;

class FControlRigClassFilter : public IClassViewerFilter
{
public:
	UE_API FControlRigClassFilter(bool bInCheckSkeleton, bool bInCheckAnimatable, bool bInCheckInversion, USkeleton* InSkeleton);
	UE_API virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override;
	UE_API virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override;	

	UE_API virtual bool MatchesFilter(const FAssetData& AssetData);

private:
	bool bFilterAssetBySkeleton;
	bool bFilterExposesAnimatableControls;
	bool bFilterInversion;

	USkeleton* Skeleton;
	const IAssetRegistry& AssetRegistry;
};

class IControlRigEditorModule : public FRigVMEditorModule
{
public:

	static IControlRigEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IControlRigEditorModule >(TEXT("ControlRigEditor"));
	}

	/**
	 * Creates an instance of a Control Rig editor.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	Blueprint				The blueprint object to start editing.
	 *
	 * @return	Interface to the new Control Rig editor
	 */
	virtual TSharedRef<IControlRigEditor> CreateControlRigEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UControlRigBlueprint* Blueprint) = 0;
	
	/**
	 * Creates an instance of a Control Rig editor.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	Asset					The control rig asset to start editing.
	 *
	 * @return	Interface to the new Control Rig editor
	 */
	virtual TSharedRef<IControlRigEditor> CreateControlRigEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UControlRigRuntimeAsset* Asset) = 0;

};

#undef UE_API
