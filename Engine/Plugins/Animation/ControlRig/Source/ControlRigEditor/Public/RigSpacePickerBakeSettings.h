// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rigs/RigHierarchyDefines.h"
#include "BakingAnimationKeySettings.h"
#include "RigSpacePickerBakeSettings.generated.h"


USTRUCT(BlueprintType)
struct FRigSpacePickerBakeSettings
{
	GENERATED_BODY();

	FRigSpacePickerBakeSettings()
	{
		TargetSpace = FRigElementKey();
		Settings = FBakingAnimationKeySettings();
		StartFrame_DEPRECATED = 0;
		EndFrame_DEPRECATED = 100;
		bReduceKeys_DEPRECATED = false;
		Tolerance_DEPRECATED = 0.001f;
	}

	UPROPERTY(BlueprintReadWrite, Category = "Settings")
	FRigElementKey TargetSpace;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Bake", meta = (ShowOnlyInnerProperties))
	FBakingAnimationKeySettings Settings;

	/** DEPRECATED 5.3 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Settings.StartFrame instead"))
	FFrameNumber StartFrame_DEPRECATED;

	/** DEPRECATED 5.3 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Settings.EndFrame instead"))
	FFrameNumber EndFrame_DEPRECATED;

	/** DEPRECATED 5.3 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Settings.bReduceKeys instead"))
	bool bReduceKeys_DEPRECATED;

	/** DEPRECATED 5.3 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Settings.Tolerance instead"))
	float Tolerance_DEPRECATED;
};
