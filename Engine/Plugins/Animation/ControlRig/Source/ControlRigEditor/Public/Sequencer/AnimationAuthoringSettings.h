// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"

#include "AnimationAuthoringSettings.generated.h"

#define UE_API CONTROLRIGEDITOR_API

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta = (DisplayName = "Animation Authoring Settings"))
class UAnimationAuthoringSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	// Whether controls' keyframes should be added on release only.
	UPROPERTY(config, EditAnywhere, Category = Interaction)
	bool bAutoKeyOnRelease = true;
    
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUpdateSettings,  const UAnimationAuthoringSettings*);
	static UE_API FOnUpdateSettings OnSettingsChange;

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};

#undef UE_API
