// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Math/Transform.h"
#include "Misc/EnumClassFlags.h"
#include "MovieSceneSequence.h"

#define UE_API CONTROLRIGEDITOR_API

class UEditorTransformGizmoContextObject;
struct FGizmoInteractionDescription;
class UTransformGizmo;
enum EInputEvent : int;
struct FInputDeviceRay;
struct FKey;
class FEditorViewportClient;
class FEditorModeTools;
class FControlRigInteractionScope;
class ISequencer;
class UAnimationAuthoringSettings;
class UControlRig;
class HHitProxy;
struct FRigControlElement;
struct FControlRigInteractionTransformContext;

namespace UE::TransformConstraintUtil
{
	struct FConstraintsInteractionCache;
}

namespace UE::AnimationEditMode
{

// Returns the current sequencer.
TWeakPtr<ISequencer> GetSequencer();
	
/**
 * FCustomMovieSceneRegistry contains custom UMovieSceneSequence that support constraints (among other things)
 * This allows other types than ULevelSequence to manage constraints.
 * Registration can be done at module startup (for example) as follows:
 * FCustomMovieSceneRegistry& Registry = FCustomMovieSceneRegistry::Get();
 * Registry.RegisterSequence<UMyCustomSequence>();
*/

class FCustomMovieSceneRegistry
{
public:
	~FCustomMovieSceneRegistry() = default;

	static CONTROLRIGEDITOR_API FCustomMovieSceneRegistry& Get();

	// Registers a particular UMovieSceneSequence subclass to support constraints.
	template<typename SequenceType>
	void RegisterSequence()
	{
		static_assert(TIsDerivedFrom<SequenceType, UMovieSceneSequence>::Value,
			"The template class SequenceType must be a subclass of UMovieSceneSequence.");
	
		UClass* SequenceClass = SequenceType::StaticClass();
		if (ensureAlways(!SequenceClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			SupportedSequenceTypes.Add(SequenceClass);
		}
	}

	// Whether a particular UMovieSceneSequence subclass is supported.
	bool CONTROLRIGEDITOR_API IsSequenceSupported(const UClass* InSequenceClass) const;

private:
	FCustomMovieSceneRegistry() = default;

	// List of supported UMovieSceneSequence classes.
	TSet<UClass*> SupportedSequenceTypes;
};

/**
 * FControlKeyframeData provides a way of passing the various keyframe parameters a control needs to set / know about.
 * Extend it if necessary to pass in more data to the keyframer.
 */

struct FControlKeyframeData
{
	// Local transform data of the control to be keyed.
	FTransform LocalTransform = FTransform::Identity;

	// Whether this local transform represents a constraint space local transform.
	bool bConstraintSpace = false;
};

/**
 * FControlRigKeyframer enables the storage and application of controls' keyframe data.
 * It stores keyframe data per control (represented as hash values) that can be applied on demand (on mouse release for example)
 * This struct works in conjunction with FControlRigInteractionScope, and captures data from controls currently interacting (whether via the viewport
 * or any other widget that would need to deffer keyframing. 
 */
	
struct FControlRigKeyframer
{
	UE_API ~FControlRigKeyframer();
	
	// Initializes this keyframer and bounds it to the animation authoring settings.
	UE_API void Initialize();

	// Resets the data storage and enable/disable the keyframer.
	UE_API void Enable(const bool InEnabled);

	// Returns true if enabled. 
	UE_API bool IsEnabled() const;
	
	// Stores the keyframe data for a specific control.
	UE_API void Store(const uint32 InControlHash, FControlKeyframeData&& InData);

	// Does the actual work off adding keyframes to the controls currently interacting.
	UE_API void Apply(const FControlRigInteractionScope& InInteractionScope, const FControlRigInteractionTransformContext& InTransformContext);

	// Updates whatever needs to once the keyframes have been added (updating constraints is one of them).
	UE_API void Finalize(UWorld* InWorld);

	// Empties the storage.
	UE_API void Reset();

private:
	
	// Storage representing keyframe data per control.
	TMap<uint32, FControlKeyframeData> KeyframeData;

	// Current state of the keyframer
	enum class EEnableState : uint8
	{
		Disabled = 0x000,			// Keyframing disabled
		EnabledDirectly = 0x001,	// Keyframing enabled via code
		EnabledBySettings = 0x002,	// Keyframing enabled by settings
		
		FullyEnabled = EnabledDirectly | EnabledBySettings
	};
	FRIEND_ENUM_CLASS_FLAGS(EEnableState)
	
	// Whether the keyframer is enabled or not.
	EEnableState EnableState = EEnableState::Disabled;

	// Handle to UAnimationAuthoringSettings::OnSettingsChange delegate.
	FDelegateHandle OnAnimSettingsChanged;
	
	// Used to track changes to animation authoring settings.
	void OnSettingsChanged(const UAnimationAuthoringSettings* InSettings);
};

/**
 * FComponentDependency helps query dependencies between scene components from a constraint standpoint   
 */

struct FComponentDependency
{
	FComponentDependency(USceneComponent* InComponent, UWorld* InWorld, TransformConstraintUtil::FConstraintsInteractionCache& InCacheRef);

	/**
	 * Returns true if the stored component depends on InObject or one of its children if it's a scene component.
	 */
	bool DependsOn(UObject* InObject);
	
private:

	bool IsValid(const UObject* InObject) const;

	USceneComponent* Component = nullptr;
	UWorld* World = nullptr;
	TransformConstraintUtil::FConstraintsInteractionCache& ConstraintsCache;
};

	
/**
 * FDirectControl manages direct viewport click-and-drag control interactions by attaching to the default transform
 * gizmo's pre and post-interaction delegates.
 * It supports the three EClickAndDragBehavior modes:
 *	- Disabled (no action)
 *	- TransientSelection (auto-unselects on release)
 *	- and PersistentSelection (replaces the current selection on drag)
 */
	
struct FDirectControl
{
	~FDirectControl();
	
	// Binds to the given editor mode tools (stores a weak reference to the gizmo context).
	void Bind(const FEditorModeTools* InModeTools);
	
	// Removes all delegates and resets all states.
	void UnBind();
	
	// Optional extra filter (return false to reject a hit proxy from triggering selection).
	TFunction<bool (HHitProxy*)> HitProxyFilterFunction;
	
	// Set externally to indicate the transform was modified during the current interaction.
	bool bHasTransformChanged = false;
	
	// Optional function returning the active sequencer (if any).
	TFunction<TSharedPtr<ISequencer>()> SequencerFunction;

	// Clears transient interaction state flags (bAlsoResetHit is true (default), also clears CachedId).
	void Reset(const bool bAlsoResetHit = true);
	
private:
	
	// Deselects all actors.
	void UnselectAll();
	
	// Returns true if any actor is currently selected.
	bool IsAnythingSelected() const;
	
	// Queries and returns the hit proxy at the given screen-space ray position from the focused viewport.
	HHitProxy* RequestNewHitProxy(const FInputDeviceRay& InRay) const;
	
	// Forwards a click event to the viewport client at the given ray's screen position.
	void ProcessClick(const FKey& InKey, const EInputEvent& InEvent, const FInputDeviceRay& InRay, HHitProxy* InHitProxy) const;
	
	// Tracks whether a selection was made by the direct control system.
	bool bHasSelected = false;
	
	// Indicates that the direct control system is actively processing a gizmo interaction.
	bool bProcessing = false;
	
	// Caches the element handle ID of the last selected hit proxy to avoid redundant selection changes across interaction callbacks.
	FTypedHandleCombinedId CachedId = 0;
	
	// Finds and returns the default transform gizmo from the mode tools' interactive tools context, or nullptr if not found.
	UTransformGizmo* FindGizmo() const;
	
	// Registers pre and post-interaction delegates.
	void BindGizmo(UTransformGizmo* InGizmo);
	
	// Returns the editor mode tools pointer (or nullptr if no longer valid).
	FEditorModeTools* GetModeTools() const;
	
	// Returns the focused viewport client from the editor mode tools (or nullptr if unavailable).
	FEditorViewportClient* GetViewportClient() const;
	
	// Returns true if the hit proxy represents a selectable element.
	bool IsHitProxyValidForSelection(HHitProxy* InHitProxy) const;
	
	// Weak pointer to the gizmo context.
	TWeakObjectPtr<UEditorTransformGizmoContextObject> WeakContext = nullptr;
	
	// Delegate handle for the transform gizmo's OnPreCanInteract event, used to trigger selection before the gizmo begins interacting.
	FDelegateHandle PreInteractionHandle;
	
	// Function bound to the transform gizmo's OnPreCanInteract event, used to trigger selection before the gizmo begins interacting.
	void OnPreCanGizmoInteract(const FGizmoInteractionDescription& InDesc);
	
	// Delegate handle for the transform gizmo's OnPostInteraction event, used to clean up selection state after the gizmo finishes interacting.
	FDelegateHandle PostInteractionHandle;
	
	// Function bound to the transform gizmo's OnPostInteraction event, used to clean up selection state after the gizmo finishes interacting.
	void OnGizmoPostInteract(const FGizmoInteractionDescription& InDesc);
	
	// Delegate handle to listen to TRS gizmo activation / deactivation.
	FDelegateHandle GizmoChangeHandle;
	
	// Delegate handle to listen to TRS gizmo creation
	FDelegateHandle GizmoCreatedHandle;
};
	
}

#undef UE_API