// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "Templates/SharedPointer.h"
#include "BakingAnimationKeySettings.h"

struct FConstraintAndActiveChannel;
class UMovieSceneSection;
class UTransformableHandle;
class UTickableTransformConstraint;
class ISequencer;
class UWorld;
struct FFrameNumber;
struct FMovieSceneFloatChannel;
struct FMovieSceneDoubleChannel;
enum class EMovieSceneTransformChannel : uint32;

struct FConstraintBaker
{
public:
	/** Bake constraint over specified frames, frames must be in order*/
	static void Bake(
		UWorld* InWorld,
		UTickableTransformConstraint* InConstraint,
		const TSharedPtr<ISequencer>& InSequencer,
		const TOptional< FBakingAnimationKeySettings>& InSettings,
		const TOptional<TArray<FFrameNumber>>& InFrames);
	
	/** Bake Multiple Constraints using settings*/
	static bool BakeMultiple(
		UWorld* InWorld,
		TArray< UTickableTransformConstraint*>& InConstraints,
		const TSharedPtr<ISequencer>& InSequencer,
		const FBakingAnimationKeySettings& Settings);
	
private:

	/** Computes the minimal frames to bake for that constraint. */
	static void GetMinimalFramesToBake(
		UWorld* InWorld,
		const UTickableTransformConstraint* InConstraint,
		const TSharedPtr<ISequencer>& InSequencer,
		UMovieSceneSection* InSection,
		const TOptional< FBakingAnimationKeySettings>& Settings,
		TArray<FFrameNumber>& OutFramesToBake);

	/** Delete the handle's transform keys within active ranges. */
	static void DeleteTransformKeysInActiveRanges(
		const TArrayView<FMovieSceneFloatChannel*>& InFloatTransformChannels,
		const TArrayView<FMovieSceneDoubleChannel*>& InDoubleTransformChannels,
		const EMovieSceneTransformChannel& InChannels,
		const TArray<FFrameNumber>& InFramesToBake,
		const TArrayView<const FFrameNumber>& InConstraintFrames,
		const TArray<bool>& InConstraintValues);

	/** Removes unnecessary inactive keys to ensure the cleanest possible active channel once baked */
	static void CleanupConstraintKeys(FConstraintAndActiveChannel& InOutActiveChannel);
};

