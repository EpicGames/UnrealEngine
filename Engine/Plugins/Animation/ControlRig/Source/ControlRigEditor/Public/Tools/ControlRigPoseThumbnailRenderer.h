// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This thumbnail renderer displays a given Control Rig Pose Asset
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"
#include "Tools/ControlRigPose.h"
#include "ControlRigPoseThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;
class FControlRigPoseThumbnailScene;

UCLASS(config = Editor, MinimalAPI)
class UControlRigPoseThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// UObject Implementation
	virtual void BeginDestroy() override;
	// End UObject Implementation

private:
	FControlRigPoseThumbnailScene* ThumbnailScene;
protected:
	UControlRigPoseAsset* ControlRigPoseAsset;
};
