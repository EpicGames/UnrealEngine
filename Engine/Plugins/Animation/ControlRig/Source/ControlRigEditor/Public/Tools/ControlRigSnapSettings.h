// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
/**
* Per Project Settings Animation Snapping
*/
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "BakingAnimationKeySettings.h"
#include "ControlRigSnapSettings.generated.h"

#define UE_API CONTROLRIGEDITOR_API


UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UControlRigSnapSettings : public UObject
{

public:
	GENERATED_BODY()

	UE_API UControlRigSnapSettings();

	/** When snapping keep offset, If false snap completely */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Snap Settings", meta = (ToolTip = "When snapping keep offset, if false snap completely"))
	bool bKeepOffset = false;

	/** When snapping, also snap position */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Snap Settings", meta = (ToolTip = "When snapping, also snap position"))
	bool bSnapPosition = true;

	/** When snapping, also snap rotation */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Snap Settings", meta = (ToolTip = "When snapping, also snap rotation"))
	bool bSnapRotation = false;

	/** When snapping, also snap scale */
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Snap Settings", meta = (ToolTip = "When snapping, also snap scale"))
	bool bSnapScale = false;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Bake")
	EBakingKeySettings BakingKeySettings = EBakingKeySettings::KeysOnly;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Bake", meta = (ClampMin = "1", UIMin = "1", EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames"))
	int32 FrameIncrement = 1;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Bake", meta = (EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames"))
	bool bReduceKeys = false;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Bake", meta = (EditCondition = "BakingKeySettings == EBakingKeySettings::AllFrames || bReduceKeys"))
	float Tolerance = 0.001f;

protected:

#if WITH_EDITOR
	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif
};


#undef UE_API
