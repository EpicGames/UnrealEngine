// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Misc/FrameNumber.h"
#include "ActorForWorldTransforms.h"
#include "ControlRigSnapper.generated.h"

class AActor;
class UControlRig;
class ISequencer;
class UMovieScene;
class ULevelSequence;
class UControlRigSnapSettings;

//Specification containing a Control Rig and a list of selected Controls we use to get World Transforms for Snapping.
USTRUCT(BlueprintType)
struct FControlRigForWorldTransforms
{
	GENERATED_BODY()

	FControlRigForWorldTransforms() : ControlRig(nullptr) {};
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Control Rig")
	TWeakObjectPtr<UControlRig> ControlRig;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Control Rig")
	TArray<FName> ControlNames;
};

//Selection from the UI to Snap To. Contains a set of Actors and/or Control Rigs to snap onto or from.
USTRUCT(BlueprintType)
struct FControlRigSnapperSelection
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Selection")
	TArray<FActorForWorldTransforms> Actors;
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Selection")
	TArray<FControlRigForWorldTransforms>  ControlRigs;

	bool IsValid() const { return (NumSelected() > 0); }
	FText GetName() const;
	int32 NumSelected() const;
	void Clear() { Actors.SetNum(0); ControlRigs.SetNum(0); }
};



struct FControlRigSnapper 
{
	/**
	*  Get Current Active Sequencer in the Level.
	*/
	TWeakPtr<ISequencer> GetSequencer();

	/**
	*  Snap the passed in children over the start and end frames.
	* @param StartFrame Start of the snap interval.
	* @param EndFrame End of the snap interval.
	* @param ChildrenToSnap The children to snap over the interval. Will set a key per frame on them.
	* @param ParentToSnap  The parent to snap to.
	* @param SnapSettings The settings to use.
	*/
	bool SnapIt(FFrameNumber StartFrame, FFrameNumber EndFrame, const FControlRigSnapperSelection& ChildrenToSnap,
		const FControlRigSnapperSelection& ParentToSnap, const UControlRigSnapSettings* SnapSettings);


	/**
	*  Get the ControlRig Transforms for current Sequencer
	* @param Sequencer Sequencer Evaluating
	* @param ControlRig The Control Rig to evaluate
	* @param ControlName The name of the Control to get.
	* @param Frames  The times to get the transforms.
	* @param ParentTransforms List of Parent Transforms for each time that the resulting transforms will be concatenated to.
	* @param OutTransforms  Results
	*/
	bool GetControlRigControlTransforms(ISequencer* Sequencer, UControlRig* ControlRig, const FName& ControlName,
		const TArray<FFrameNumber>& Frames, const TArray<FTransform>& ParentTransforms, TArray<FTransform>& OutTransforms);


	/**
	*  Get the ControlRig Transforms from specified MovieSceneSequence. This will create a new player to evalaute so can be used on MovieSceneSequence that isn't in the scene
	* @param World Active World
	* @param LevelSequence LevelSequence Evaluating
	* @param ControlRig The Control Rig to evaluate
	* @param ControlName The name of the Control to get.
	* @param Frames  The times to get the transforms.
	* @param ParentTransforms List of Parent Transforms for each time that the resulting transforms will be concatenated to.
	* @param OutTransforms  Results
	*/
	bool GetControlRigControlTransforms(UWorld* World, ULevelSequence* LevelSequence,
		UControlRig* ControlRig, const FName& ControlName,const TArray<FFrameNumber>& Frames, const TArray<FTransform>& ParentTransforms, TArray<FTransform>& OutTransforms);

};


