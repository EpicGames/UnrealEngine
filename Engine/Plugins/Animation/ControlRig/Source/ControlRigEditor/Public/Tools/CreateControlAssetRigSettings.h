// Copyright Epic Games, Inc. All Rights Reserved.


/**
 * Default Settings for the Control Rig Asset, including the Default Asset Name
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "CreateControlAssetRigSettings.generated.h"

UCLASS(BlueprintType, config = EditorSettings)
class  UCreateControlPoseAssetRigSettings : public UObject
{
public:
	UCreateControlPoseAssetRigSettings(const FObjectInitializer& Initializer);

	GENERATED_BODY()

	/** Asset Name*/
	UPROPERTY(EditAnywhere, Category = "Control Pose")
	FString AssetName;

};
