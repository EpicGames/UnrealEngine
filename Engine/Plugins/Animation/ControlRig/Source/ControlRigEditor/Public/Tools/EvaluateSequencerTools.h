// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorForWorldTransforms.h"
#include "MovieSceneBinding.h"
#include "Misc/Guid.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneChannelTraits.h"
#define UE_API CONTROLRIGEDITOR_API

class UControlRig;
class UWorld;
class ISequencer;
struct FRigControlElement;
struct FRigControlModifiedContext;
struct FEulerTransform;
class UMovieSceneTrack;
class UMovieScene;
struct FGuid;
class UMovieScene3DTransformSection;
class UMovieScenePropertyTrack;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;

namespace UE
{
namespace AIE
{
/**
*  This file contains a set of structures that can be used to evaluate a set of Actors/Scene Components/Sockets and Control Rigs all at once over a set
* of spase times that will then fill out a set of transform arrays. It als constains a set of helper functions for setting Control Rig trranfsorms with constraints 
* and getting Sequencer dependencies.
**/
// Specify a range of time using start time and a frame step with 
// accessors to get the index from the time, and the time from the index
struct  FFrameTimeByIndex
{
	FFrameNumber StartFrame;
	FFrameNumber EndFrame;
	FFrameNumber FrameStep;
	int32 NumFrames;
	FFrameTimeByIndex() :
		StartFrame(0),
		EndFrame(0),
		FrameStep(0),
		NumFrames(0)
	{}
	FFrameTimeByIndex(const FFrameTimeByIndex& Index) = default;

	FFrameTimeByIndex(const FFrameNumber& InStartFrame, const FFrameNumber& InFrameStep, const int32 InNumFrames) :
		StartFrame(InStartFrame),
		FrameStep(InFrameStep),
		NumFrames(InNumFrames)
	{
		EndFrame = StartFrame + FrameStep * (NumFrames - 1);
	};

	FFrameTimeByIndex(const FFrameNumber& InStartFrame, const FFrameNumber& InEndFrame, const FFrameNumber& InFrameStep) :
		StartFrame(InStartFrame),
		FrameStep(InFrameStep)
	{
		//make sure end frame falls on frame step 
		NumFrames = (InEndFrame.Value - StartFrame.Value) / (FrameStep.Value) + 1;
		EndFrame.Value = StartFrame.Value + (NumFrames * FrameStep.Value);
	};

	FFrameNumber CalculateFrame(int32 Index) const
	{
		return StartFrame + FrameStep * Index;
	};

	int32 CalculateIndex(const FFrameNumber& InCurrentFrame) const
	{
		if (InCurrentFrame > StartFrame && InCurrentFrame < EndFrame)
		{
			return ((InCurrentFrame.Value - StartFrame.Value) / (FrameStep.Value));
		}
		else if (InCurrentFrame <= StartFrame)
		{
			return 0;
		}
		else if (InCurrentFrame >= EndFrame)
		{
			return NumFrames - 1;
		}
		return INDEX_NONE;
	}
	TArray<int32> GetFullIndexArray() const
	{
		TArray<int32> IndexArray;
		if (NumFrames > 0)
		{
			IndexArray.SetNum(NumFrames);
			for (int32 Index = 0; Index < NumFrames; ++Index)
			{
				IndexArray[Index] = Index;
			}
		}
		return IndexArray;
	}
	TArray<FFrameNumber> GetFullFrameArray() const
	{
		TArray<FFrameNumber> FrameArray;
		if (NumFrames > 0)
		{
			FrameArray.SetNum(NumFrames);
			for (int32 Index = 0; Index < NumFrames; ++Index)
			{
				FrameArray[Index] = CalculateFrame(Index);
			}
		}
		return FrameArray;
	}
};

//Array of transforms, that maybe be sparse with only some transforms set
struct FArrayOfTransforms
{
	TArray<FTransform> Transforms;
	void SetNum(int32 InNum)
	{
		Transforms.SetNum(InNum);
	}
	int32 Num() const
	{
		return Transforms.Num();
	}
	FTransform& operator[](int32 Index)
	{
		return Transforms[Index];
	}
	//interpolate with sparse set of indices
	UE_API FTransform Interp(const FFrameNumber& InTime, const TArray<int32>& ValidIndices, const TArray<FFrameNumber>& ValidTimes);
};

// An 'Actor'(which may be an AActor/SceneComponent/Socket) and a set of sparse world transforms with corrensonding parent transforms
struct FActorAndWorldTransforms
{
	FActorAndWorldTransforms() {};

	FActorAndWorldTransforms(TSharedPtr<FArrayOfTransforms>& InWorldTransforms, TSharedPtr<FArrayOfTransforms>& InParentTransforms)
		: WorldTransforms(InWorldTransforms), ParentTransforms(InParentTransforms)
	{};
	FActorForWorldTransforms Actor;
	TSharedPtr<FArrayOfTransforms> WorldTransforms;
	TSharedPtr<FArrayOfTransforms> ParentTransforms;

	void MakeCopy(FActorAndWorldTransforms& OutCopy) const
	{
		OutCopy.Actor = Actor;
		OutCopy.WorldTransforms = MakeShared<FArrayOfTransforms>();
		OutCopy.ParentTransforms = MakeShared<FArrayOfTransforms>();
		if (WorldTransforms.IsValid())
		{
			OutCopy.SetNumOfTransforms(WorldTransforms->Num());
		}
	}


	void SetNumOfTransforms(int32 Num)
	{
		if (WorldTransforms)
		{
			WorldTransforms->SetNum(Num);
		}
		if (ParentTransforms)
		{
			ParentTransforms->SetNum(Num);
		}
	}
};

//A Control rig with a set of Controls
struct FControlRigAndWorldTransforms
{
	void AddWorldTransform(bool but,  TSharedPtr<FArrayOfTransforms>& InWorldTransforms)
	{
		ParentTransforms = InWorldTransforms;
	}
	void AddWorldTransform(const FName& InName, TSharedPtr<FArrayOfTransforms>& InWorldTransforms)
	{
		ControlAndWorldTransforms.Add(InName, InWorldTransforms);
	}

	void SetNumOfTransforms(int32 Num)
	{
		if (ParentTransforms)
		{
			ParentTransforms->SetNum(Num);
		}
		for (TPair<FName, TSharedPtr<FArrayOfTransforms>>& Pair : ControlAndWorldTransforms)
		{
			if (Pair.Value)
			{
				Pair.Value->SetNum(Num);
			}
		}
	}

	void MakeCopy(FControlRigAndWorldTransforms& OutCopy) const
	{
		OutCopy.ControlRig = ControlRig;
		OutCopy.ParentTransforms = MakeShared<FArrayOfTransforms>();
		for (const TPair<FName, TSharedPtr<FArrayOfTransforms>>& Pair : ControlAndWorldTransforms)
		{
			TSharedPtr<FArrayOfTransforms> SharedPtr = MakeShared<FArrayOfTransforms>();
			OutCopy.AddWorldTransform(Pair.Key, SharedPtr);
		}
		if (ParentTransforms.IsValid())
		{
			OutCopy.SetNumOfTransforms(ParentTransforms->Num());
		}
	}

	TWeakObjectPtr<UControlRig> ControlRig;
	TSharedPtr<FArrayOfTransforms > ParentTransforms;
	TMap<FName, TSharedPtr<FArrayOfTransforms>> ControlAndWorldTransforms; //one for each control in the FControlRigAndWorldTransform

};

struct FSetTransformHelpers
{
	//get a copy of a transform and possibly use it to set a constrainted transform on a control, if this happens it returns true
	static UE_API bool SetConstrainedTransform(FTransform LocalTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context);
	//set a control transform from a local transform, will set a constrainted transform if one is needed, otherwise will just set a normal one.
	static UE_API void SetControlTransform(const FEulerTransform& LocalTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context);
	//set a control transform from a global transform, will call SetControlTransform but handles doing the converstion to local space
	static UE_API void SetControlGlobalTransform(const FTransform& GlobalTransform, UControlRig* ControlRig, FRigControlElement* ControlElement, const FRigControlModifiedContext& Context);

	
	//set a set of tranfsorms on an scene component, will handle constraints
	static UE_API bool SetActorTransform(ISequencer* Sequencer, USceneComponent* SceneComponent, UMovieScene3DTransformSection* TransformSection, const TArray<FFrameNumber>& Frames,
		const TArray<FTransform>& WorldTransformsToSnapTo, const TArray<FTransform>& ParentWorldTransforms);

};

//Calculates for a set of tracks under an actor in sequencer the set of dependencies that it requires to caluclate it's transform, so parent actors, attachments/constraints etc.
//This is somewhat similar to what you would calculate with a DAG to figure out dependencies but since we don't have a DAG we need to do this manually by walking parents/tracks.
//It will also calcualate the corresponding UMovieSceneSignedObject FGuid signatures, which will then be used to determine if we need to recalculate an objects tranforms, like
//with motion trails since some downstream dependency has changed.
struct FSequencerTransformDependencies
{
	//Get the list of tracks in the moviescene with this binding that may effect the final transform of something in the binding.
	//Currently it will return a set of transform, control rig, attachment, path and  skeletal anim tracks.
	static UE_API TArray<UMovieSceneTrack*> GetTransformAffectingTracks(UMovieScene* InMovieScene, const FMovieSceneBinding& Binding);

	//Main function to calculate the dependencies for an actor and a set of tracks under that actor
	UE_API void CalculateDependencies(ISequencer* InSequencer, AActor* InActor, TArray<UMovieSceneTrack*>& InTracks);
	//Compare 2 set of dependencies to see if they have changed.
	UE_API bool Compare(FSequencerTransformDependencies& CopyInIfDifferent) const;
	//Copy other set of dependencies to this one
	UE_API void CopyFrom(const FSequencerTransformDependencies& Other);
	//If setup or not
	bool IsEmpty() const { return Tracks.Num() == 0; }
public:
	//These are the values calculated by CalculateDependents function
	//Track dependencies
	TMap<TWeakObjectPtr<UMovieSceneTrack>, FGuid> Tracks;
	//Sequencer actor dependencies
	TMap<AActor*, FGuid> SequencerActors;
	//Non sequencer actors that this object depends upon
	TSet<AActor*> NonSequencerActors;
private:
	UE_API FMovieSceneBinding GetBindingFromTrack(UMovieScene* InMovieScene, UMovieSceneTrack* Track);
	UE_API void CalculateTrackDependents(ISequencer* InSequencer, UMovieSceneTrack* InTrack);
	//Add a track to the set of calculated onces and recursively get it's dependencies
	UE_API void AddTrack(ISequencer* InSequencer, UMovieSceneTrack* InTrack);

};

struct FEvalHelpers
{
	//The main evaluation function to use a sparse set of times and caclculate their world transforms for a set of sequencer actors/control rigs.
	static UE_API bool CalculateWorldTransforms(UWorld* World, ISequencer* Sequencer, const FFrameTimeByIndex& FrameTimeByIndex,
		const TArray<int32>& Indices, TArray<FActorAndWorldTransforms>& Actors, TMap<UControlRig*, FControlRigAndWorldTransforms>& ControlRigs);

	//Evaluate sequencer for the relevant tracks, making sure depencencies such as constraints and control rigs also tick
	static UE_API bool EvaluateSequencer(UWorld* World, ISequencer* Sequencer, const TSet<const UMovieSceneTrack*>& RelevantTracks);
};

struct FControlRigAndControlsAndTrack
{
	UMovieSceneControlRigParameterTrack* Track;
	UControlRig* ControlRig;
	TArray<FName> Controls;
};

struct FObjectAndTrack
{
	UMovieScenePropertyTrack* Track;
	UObject* BoundObject;
	FGuid SequencerGuid;
};

//helper to get selected items in sequencer, control rigs and properties
struct FSequencerSelected
{
	static UE_API TArray<FGuid> GetSelectedOutlinerGuids(ISequencer* SequencerPtr);
	static UE_API void GetSelectedControlRigsAndBoundObjects(ISequencer* SequencerPtr, TArray<FControlRigAndControlsAndTrack>& OutSelectedCRs, TArray<FObjectAndTrack>& OutBoundObjects);
};

//get start end times for control sections keys
struct FControlRigKeys
{
	static UE_API bool GetStartEndIndicesForControl(UMovieSceneControlRigParameterSection* BaseSection, const FRigControlElement* ControlElement, int& OutStartIndex, int& OutEndIndex);
};

template<typename ChannelType, typename ValueType>
void AssigneOrSetValue(ChannelType* Channel, ValueType Value, const FFrameNumber& FrameNumber, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	using namespace UE::MovieScene;

	TArray<FKeyHandle> KeysAtCurrentTime;
	Channel->GetKeys(TRange<FFrameNumber>(FrameNumber), nullptr, &KeysAtCurrentTime);
	if (KeysAtCurrentTime.Num() > 0)
	{
		AssignValue(Channel, KeysAtCurrentTime[0], Value);
	}
	else
	{
		EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(Channel, FrameNumber, DefaultInterpolation);
		AddKeyToChannel(Channel, FrameNumber, Value, Interpolation);
	}
}



template<typename ChannelType, typename ValueType>
void SetCurrentKeys(TArrayView<ChannelType*> Channels, int32 StartIndex, int32 EndIndex, EMovieSceneKeyInterpolation DefaultInterpolation, const FFrameNumber& FrameNumber)
{
	ValueType Value = 0.0;
	FFrameTime FrameTime(FrameNumber);
	for (int32 Index = StartIndex; Index <= EndIndex; ++Index)
	{
		ChannelType* Channel = Channels[Index];
		if (Channel->Evaluate(FrameTime, Value))
		{
			AssigneOrSetValue(Channel, Value, FrameNumber, DefaultInterpolation);
		}
	}
}

struct FControlRigWorldTransformProvider
{
	FControlRigWorldTransformProvider(AActor* InActor,  USceneComponent* InComponent,  UControlRig* InControlRig, float InDeltaTime);

	bool IsValid() const;
	FTransform GetTransform(const FName& InControlName, const bool bUpdateAnimation = false) const;
	FTransform GetComponentTransform(const bool bUpdateAnimation = false) const;
						
	TWeakObjectPtr<AActor> Actor = nullptr;	
	TWeakObjectPtr<USceneComponent> Component = nullptr;
	TWeakObjectPtr<UControlRig> ControlRig = nullptr;
	float DeltaTime = 1.f / 30.f;
};
	
} // namespace AIE
} // namespace UE

#undef UE_API
